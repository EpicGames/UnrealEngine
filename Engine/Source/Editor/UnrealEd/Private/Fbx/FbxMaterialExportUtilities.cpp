// Copyright Epic Games, Inc. All Rights Reserved.

#include "FbxMaterialExportUtilities.h"

#include "Exporters/FbxExportOption.h"

#include "Editor.h"
#include "IMeshMergeUtilities.h"
#include "MeshMergeModule.h"
#include "Modules/ModuleManager.h"
#include "IImageWrapperModule.h"
#include "IImageWrapper.h"
#include "IMaterialBakingModule.h"
#include "Misc/FileHelper.h"
#include "EditorFramework/AssetImportData.h"

#include "Rendering/SkeletalMeshRenderData.h"
#include "Components/BrushComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshComponentLODInfo.h"

#include "Engine/MapBuildDataRegistry.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/Brush.h"
#include "MaterialPropertyEx.h"
#include "MaterialBakingStructures.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialExpressionCustomOutput.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Materials/MaterialExpressionTextureSampleParameter2D.h"
#include "Materials/MaterialExpressionTextureObject.h"
#include "Materials/MaterialExpressionTextureObjectParameter.h"
#include "Materials/MaterialExpressionConstant.h"
#include "Materials/MaterialExpressionConstant2Vector.h"
#include "Materials/MaterialExpressionConstant3Vector.h"
#include "Materials/MaterialExpressionConstant4Vector.h"
#include "Materials/MaterialExpressionLinearInterpolate.h"
#include "Materials/MaterialExpressionScalarParameter.h"
#include "Model.h"
#include "TextureCompiler.h"

#include "InterchangeAssetImportData.h"

THIRD_PARTY_INCLUDES_START
#include <fbxsdk.h>
THIRD_PARTY_INCLUDES_END

namespace UnFbx
{
	namespace FbxMaterialExportHelpers
	{
		UMaterialExpressionCustomOutput* GetCustomOutputByName(const UMaterialInterface* Material, const FString& FunctionName)
		{
			for (const TObjectPtr<UMaterialExpression>& Expression : Material->GetMaterial()->GetExpressions())
			{
				UMaterialExpressionCustomOutput* CustomOutput = Cast<UMaterialExpressionCustomOutput>(Expression);
				if (CustomOutput != nullptr && CustomOutput->GetFunctionName() == FunctionName)
				{
					return CustomOutput;
				}
			}

			return nullptr;
		}

		const FExpressionInput* GetInputForProperty(const UMaterialInterface* Material, const FMaterialPropertyEx& Property)
		{
			if (Property.IsCustomOutput())
			{
				const FString FunctionName = Property.CustomOutput.ToString();
				const UMaterialExpressionCustomOutput* CustomOutput = GetCustomOutputByName(Material, FunctionName);
				if (CustomOutput == nullptr)
				{
					return nullptr;
				}

				// Assume custom outputs always have a single input (which is true for all supported custom outputs)
				return const_cast<UMaterialExpressionCustomOutput*>(CustomOutput)->GetInput(0);
			}

			UMaterial* UnderlyingMaterial = const_cast<UMaterial*>(Material->GetMaterial());
			return UnderlyingMaterial->GetExpressionInputForProperty(Property.Type);
		}

		template <typename ExpressionType>
		void GetAllInputExpressionsOfType(const UMaterialInterface* Material, const FMaterialPropertyEx& Property, TArray<ExpressionType*>& OutExpressions)
		{
			const FExpressionInput* Input = GetInputForProperty(Material, Property);
			if (Input == nullptr)
			{
				return;
			}

			UMaterialExpression* InputExpression = Input->Expression;
			if (InputExpression == nullptr)
			{
				return;
			}

			TArray<UMaterialExpression*> AllInputExpressions;
			InputExpression->GetAllInputExpressions(AllInputExpressions);

			for (UMaterialExpression* Expression : AllInputExpressions)
			{
				if (ExpressionType* ExpressionOfType = Cast<ExpressionType>(Expression))
				{
					OutExpressions.Add(ExpressionOfType);
				}

				if (UMaterialFunctionInterface* MaterialFunction = UMaterial::GetExpressionFunctionPointer(Expression))
				{
					MaterialFunction->GetAllExpressionsOfType<ExpressionType>(OutExpressions);
				}
				else if (TOptional<UMaterial::FLayersInterfaces> LayersInterfaces = UMaterial::GetExpressionLayers(Expression))
				{
					for (UMaterialFunctionInterface* Layer : LayersInterfaces->Layers)
					{
						if (Layer != nullptr)
						{
							Layer->GetAllExpressionsOfType<ExpressionType>(OutExpressions);
						}
					}

					for (UMaterialFunctionInterface* Blend : LayersInterfaces->Blends)
					{
						if (Blend != nullptr)
						{
							Blend->GetAllExpressionsOfType<ExpressionType>(OutExpressions);
						}
					}
				}
			}
		}
		UTexture* GetTextureFromSample(const UMaterialInterface* Material, const UMaterialExpressionTextureSample* SampleExpression)
		{
			if (const UMaterialExpressionTextureSampleParameter2D* SampleParameter = ExactCast<UMaterialExpressionTextureSampleParameter2D>(SampleExpression))
			{
				UTexture* ParameterValue = SampleParameter->Texture;

				if (!Material->GetTextureParameterValue(SampleParameter->GetParameterName(), ParameterValue))
				{
					return nullptr;
				}

				return ParameterValue;
			}

			if (const UMaterialExpressionTextureSample* Sample = ExactCast<UMaterialExpressionTextureSample>(SampleExpression))
			{
				UMaterialExpression* ObjectExpression = Sample->TextureObject.Expression;
				if (ObjectExpression == nullptr)
				{
					return Sample->Texture;
				}

				if (const UMaterialExpressionTextureObjectParameter* ObjectParameter = ExactCast<UMaterialExpressionTextureObjectParameter>(ObjectExpression))
				{
					UTexture* ParameterValue = ObjectParameter->Texture;

					if (!Material->GetTextureParameterValue(ObjectParameter->GetParameterName(), ParameterValue))
					{
						return nullptr;
					}

					return ParameterValue;
				}

				if (const UMaterialExpressionTextureObject* Object = ExactCast<UMaterialExpressionTextureObject>(ObjectExpression))
				{
					return Object->Texture;
				}

				return nullptr;
			}

			if (const UMaterialExpressionTextureObjectParameter* ObjectParameter = ExactCast<UMaterialExpressionTextureObjectParameter>(SampleExpression))
			{
				UTexture* ParameterValue = ObjectParameter->Texture;

				if (!Material->GetTextureParameterValue(ObjectParameter->GetParameterName(), ParameterValue))
				{
					return nullptr;
				}

				return ParameterValue;
			}

			if (const UMaterialExpressionTextureObject* Object = ExactCast<UMaterialExpressionTextureObject>(SampleExpression))
			{
				return Object->Texture;
			}


			return nullptr;
		}
		FIntPoint TryGetMaxTextureSize(const UMaterialInterface* Material, const FMaterialPropertyEx& Property, const FIntPoint& DefaultMaxSize)
		{
			TArray<UMaterialExpressionTextureSample*> TextureSamples;
			GetAllInputExpressionsOfType(Material, Property, TextureSamples);

			if (TextureSamples.Num() == 0)
			{
				return DefaultMaxSize;
			}

			FIntPoint MaxSize = { 0, 0 };

			for (const UMaterialExpressionTextureSample* TextureSample : TextureSamples)
			{
				UTexture* Texture = GetTextureFromSample(Material, TextureSample);
				if (Texture == nullptr ||
					!(Texture->IsA<UTexture2D>() || Texture->IsA<UTextureRenderTarget2D>()))
				{
					continue;
				}


				//Load Texture
				{
#if WITH_EDITOR
					FTextureCompilingManager::Get().FinishCompilation({ Texture });
#endif
					Texture->SetForceMipLevelsToBeResident(30.0f);
					Texture->WaitForStreaming();
				}

				auto GetMipBias = [](const UTexture* Texture)
					{
						if (const UTexture2D* Texture2D = Cast<UTexture2D>(Texture))
						{
							return Texture2D->GetNumMips() - Texture2D->GetNumMipsAllowed(true);
						}

						return Texture->GetCachedLODBias();
					};

				auto GetInGameSize = [&GetMipBias](const UTexture* Texture) -> FIntPoint
					{
						const int32 Width = FMath::CeilToInt(Texture->GetSurfaceWidth());
						const int32 Height = FMath::CeilToInt(Texture->GetSurfaceHeight());

						const int32 MipBias = GetMipBias(Texture);

						const int32 InGameWidth = FMath::Max(Width >> MipBias, 1);
						const int32 InGameHeight = FMath::Max(Height >> MipBias, 1);

						return { InGameWidth, InGameHeight };
					};

				FIntPoint TextureSize = GetInGameSize(Texture);

				MaxSize = MaxSize.ComponentMax(TextureSize);
			}

			return (MaxSize.X == 0 || MaxSize.Y == 0) ? DefaultMaxSize : MaxSize;
		}

		void AnalyzeMaterialProperty(const UMaterialInterface* InMaterial, const FMaterialPropertyEx& InProperty, FMaterialAnalysisResult& OutAnalysis)
		{
			if (GetInputForProperty(InMaterial, InProperty) == nullptr)
			{
				OutAnalysis = FMaterialAnalysisResult();
				return;
			}

			UMaterial* BaseMaterial = const_cast<UMaterial*>(InMaterial->GetMaterial());
			bool bRequiresPrimitiveData = false;

			if (BaseMaterial)
			{
				BaseMaterial->AnalyzeMaterialPropertyEx(InProperty.Type, OutAnalysis);
			}

			// Also make sure the analysis takes into account primitive data
			OutAnalysis.bRequiresVertexData |= bRequiresPrimitiveData;
		}

		bool NeedsMeshDataForProperty(const UMaterialInterface* Material, const FMaterialPropertyEx& Property)
		{
			if (Material != nullptr)
			{
				FMaterialAnalysisResult Analysis;

				AnalyzeMaterialProperty(Material, Property, Analysis);

				return Analysis.bRequiresVertexData;
			}

			return false;
		}

		void GetAllTextureCoordinateIndices(const UMaterialInterface* Material, const FMaterialPropertyEx& Property, TArray<int32>& OutTexCoords)
		{
			FMaterialAnalysisResult Analysis;
			AnalyzeMaterialProperty(Material, Property, Analysis);

			const TBitArray<>& TexCoords = Analysis.TextureCoordinates;
			for (int32 Index = 0; Index < TexCoords.Num(); Index++)
			{
				if (TexCoords[Index])
				{
					OutTexCoords.Add(Index);
				}
			}
		}

		void TransformColorSpace(TArray<FColor>& Pixels, bool bFromSRGB, bool bToSRGB)
		{
			if (bFromSRGB == bToSRGB)
			{
				return;
			}

			if (bToSRGB)
			{
				for (FColor& Pixel : Pixels)
				{
					Pixel = Pixel.ReinterpretAsLinear().ToFColor(true);
				}
			}
			else
			{
				for (FColor& Pixel : Pixels)
				{
					Pixel = FLinearColor(Pixel).ToFColor(false);
				}
			}
		}

		TArray<int32> GetSectionIndices(const UStaticMesh* StaticMesh, int32 LODIndex, const int32& MaterialIndex)
		{
			if (StaticMesh == nullptr)
			{
				return {};
			}

			const FStaticMeshLODResources& RenderData = StaticMesh->GetLODForExport(LODIndex);
			const FStaticMeshSectionArray& Sections = RenderData.Sections;

			TArray<int32> SectionIndices;
			SectionIndices.Reserve(Sections.Num());

			for (int32 SectionIndex = 0; SectionIndex < Sections.Num(); ++SectionIndex)
			{
				if (Sections[SectionIndex].MaterialIndex == MaterialIndex)
				{
					SectionIndices.Add(SectionIndex);
				}
			}

			return SectionIndices;
		}

		TArray<int32> GetSectionIndices(const USkeletalMesh* SkeletalMesh, int32 LODIndex, const int32& MaterialIndex)
		{
			if (SkeletalMesh == nullptr)
			{
				return {};
			}

			const FSkeletalMeshLODRenderData& RenderData = SkeletalMesh->GetResourceForRendering()->LODRenderData[LODIndex];			
			const TArray<FSkelMeshRenderSection>& Sections = RenderData.RenderSections;

			TArray<int32> SectionIndices;
			SectionIndices.Reserve(Sections.Num());

			for (int32 SectionIndex = 0; SectionIndex < Sections.Num(); ++SectionIndex)
			{
				if (Sections[SectionIndex].MaterialIndex == MaterialIndex)
				{
					SectionIndices.Add(SectionIndex);
				}
			}

			return SectionIndices;
		}

		namespace InterchangeMaterialProcessHelpers
		{
			FString LambertSurfaceMaterialPath = TEXT("/Interchange/Materials/LambertSurfaceMaterial.LambertSurfaceMaterial");
			FString PhongSurfaceMaterialPath = TEXT("/Interchange/Materials/PhongSurfaceMaterial.PhongSurfaceMaterial");
			FString MF_PhongToMetalRoughnessPathName = TEXT("/Interchange/Functions/MF_PhongToMetalRoughness.MF_PhongToMetalRoughness");

			bool HandleMaterialProperty(UMaterialInterface* MaterialInterface, fbxsdk::FbxScene* Scene, fbxsdk::FbxSurfaceMaterial* FbxMaterial,
				const FString& ParameterPropertyName, const FMaterialPropertyEx& Property, 
				const char* FbxPropertyName, const char* FbxFactorPropertyName,
				bool bPrimaryPropertyIsMap = true)
				{
					fbxsdk::FbxProperty FbxProperty = FbxMaterial->FindProperty(FbxPropertyName);
					fbxsdk::FbxProperty FbxFactorProperty = FbxMaterial->FindProperty(FbxFactorPropertyName);
					if (!FbxProperty.IsValid())
					{
						return false;
					}

					bool bPropertyProcessed = false;

					//If Map is present(non-default) then Factor should be acquired from the Weight (and Color is presumed black and does not need to be set on the fbx material (it cannot be set))
					//If no Map is present then the factor cannot be deduced, and just the Color should be set(If R = G = B then it should be a float otherwise RGB)

					//Use Map
					UTexture* Texture = nullptr;
					if (bPrimaryPropertyIsMap && MaterialInterface->GetTextureParameterValue(*(ParameterPropertyName + TEXT("Map")), Texture, true))
					{
						FString TextureSourceFullPath = Texture->AssetImportData->GetFirstFilename();
						//Create a fbx property
						FbxFileTexture* lTexture = FbxFileTexture::Create(Scene, "EnvSamplerTex");
						lTexture->SetFileName(TCHAR_TO_UTF8(*TextureSourceFullPath));
						lTexture->SetTextureUse(FbxTexture::eStandard);
						lTexture->SetMappingType(FbxTexture::eUV);
						lTexture->ConnectDstProperty(FbxProperty);
						
						float Weight;
						if (MaterialInterface->GetScalarParameterValue(*(ParameterPropertyName + TEXT("MapWeight")), Weight, true))
						{
							if (FbxFactorProperty.IsValid()) FbxFactorProperty.Set(Weight);
						}

						bPropertyProcessed = true;
					}
					else
					{
						FLinearColor Color;
						float Value;
						if (MaterialInterface->GetVectorParameterValue(*ParameterPropertyName, Color, false))
						{
							FLinearColor DefaultColor;
							MaterialInterface->GetVectorParameterDefaultValue(*ParameterPropertyName, DefaultColor);
							
							if (Color != DefaultColor)
							{
								if (Color.R == Color.G
									&& Color.G == Color.B)
								{
									if (FbxProperty.IsValid()) FbxProperty.Set(Color.R);
								}
								else
								{
									FbxDouble3 FbxColor(Color.R, Color.G, Color.B);
									if (FbxProperty.IsValid()) FbxProperty.Set(FbxColor);
								}
							}

							bPropertyProcessed = true;
						}
						else if (MaterialInterface->GetScalarParameterValue(*ParameterPropertyName, Value, false))
						{
							float DefaultValue;
							MaterialInterface->GetScalarParameterDefaultValue(*ParameterPropertyName, DefaultValue);

							if (Value != DefaultValue
								&& FbxProperty.IsValid())
							{
								FbxProperty.Set(Value);
							}

							bPropertyProcessed = true;
						}
					}
					
					return bPropertyProcessed;
				};

			UMaterialExpressionMaterialFunctionCall* GetInterchangeMFPongToMetalRoughness(UMaterialInterface* MaterialInterface)
			{
				if (!MaterialInterface || !MaterialInterface->GetMaterial())
				{
					return nullptr;
				}

				//Acquire the Material's expressions and check if any matches the MF_PhongToMetalRoughness MaterialFunctionCall:
				TConstArrayView<TObjectPtr<UMaterialExpression>> MaterialExpressions = MaterialInterface->GetMaterial()->GetExpressions();

				UMaterialExpressionMaterialFunctionCall* ImportedMaterialFunction = nullptr;

				for (const TObjectPtr<UMaterialExpression>& MaterialExpression : MaterialExpressions)
				{
					if (UMaterialExpressionMaterialFunctionCall* FunctionCallExpression = Cast<UMaterialExpressionMaterialFunctionCall>(MaterialExpression))
					{
						if (FunctionCallExpression->MaterialFunction)
						{
							FString MF_PathName = FunctionCallExpression->MaterialFunction->GetPathName();

							if (MF_PhongToMetalRoughnessPathName == MF_PathName)
							{
								ImportedMaterialFunction = FunctionCallExpression;
								break;
							}
						}
					}
				}

				return ImportedMaterialFunction;
			}

			void HandleMaterialExpression(UMaterialInterface* MaterialInterface, fbxsdk::FbxScene* Scene, fbxsdk::FbxSurfaceMaterial* FbxMaterial,
				const char* FbxPropertyName, const char* FbxFactorPropertyName,
				UMaterialExpression* MaterialExpression)
			{
				fbxsdk::FbxProperty FbxProperty = FbxMaterial->FindProperty(FbxPropertyName);
				fbxsdk::FbxProperty FbxFactorProperty = FbxMaterial->FindProperty(FbxFactorPropertyName);
				if (!FbxProperty.IsValid())
				{
					return;
				}

				//Check for Lerp
				// Interchange import creates a Lepr Setup with:
				// A => Color
				// B => Map
				// Alpha => Weight (Factor)
				//If Map is present(non-default) then Factor should be acquired from the Weight (and Color is presumed black and does not need to be set on the fbx material (it cannot be set))
				//If no Map is present then the factor cannot be deduced, and just the Color should be set(If R = G = B then it should be a float otherwise RGB)
				if (UMaterialExpressionLinearInterpolate* LerpExpr = Cast< UMaterialExpressionLinearInterpolate>(MaterialExpression))
				{
					if (UMaterialExpressionTextureBase* TextureBaseExpr = Cast<UMaterialExpressionTextureBase>(LerpExpr->B.Expression))
					{
						FString TextureSourceFullPath = TextureBaseExpr->Texture->AssetImportData->GetFirstFilename();
						//Create a fbx property
						FbxFileTexture* lTexture = FbxFileTexture::Create(Scene, "EnvSamplerTex");
						lTexture->SetFileName(TCHAR_TO_UTF8(*TextureSourceFullPath));
						lTexture->SetTextureUse(FbxTexture::eStandard);
						lTexture->SetMappingType(FbxTexture::eUV);
						lTexture->ConnectDstProperty(FbxProperty);

						if (UMaterialExpressionConstant* ConstExpr1 = Cast<UMaterialExpressionConstant>(LerpExpr->Alpha.Expression))
						{
							if (FbxFactorProperty.IsValid()) FbxFactorProperty.Set(ConstExpr1->R);
						}
						else if (UMaterialExpressionScalarParameter* ScalarParamExpr = Cast<UMaterialExpressionScalarParameter>(LerpExpr->Alpha.Expression))
						{
							const FHashedMaterialParameterInfo ParameterInfo(ScalarParamExpr->GetParameterName());
							float Value;
							if (MaterialInterface->GetScalarParameterValue(ParameterInfo, Value))
							{
								if (FbxFactorProperty.IsValid()) FbxFactorProperty.Set(Value);
							}
						}
					}
				}
				else
				{
					//If Texture/Constant is directly connected
					//Then Texture and Constant is targetting the FbxProperty.
					if (UMaterialExpressionTextureBase* TextureBaseExpr = Cast<UMaterialExpressionTextureBase>(MaterialExpression))
					{
						FString TextureSourceFullPath = TextureBaseExpr->Texture->AssetImportData->GetFirstFilename();
						//Create a fbx property
						FbxFileTexture* lTexture = FbxFileTexture::Create(Scene, "EnvSamplerTex");
						lTexture->SetFileName(TCHAR_TO_UTF8(*TextureSourceFullPath));
						lTexture->SetTextureUse(FbxTexture::eStandard);
						lTexture->SetMappingType(FbxTexture::eUV);
						lTexture->ConnectDstProperty(FbxProperty);
					}
					else if (UMaterialExpressionConstant* ConstExpr1 = Cast<UMaterialExpressionConstant>(MaterialExpression))
					{
						FbxDouble3 FbxColor(ConstExpr1->R, ConstExpr1->R, ConstExpr1->R);
						FbxProperty.Set(FbxColor);
					}
					if (UMaterialExpressionConstant3Vector* ConstExpr3 = Cast<UMaterialExpressionConstant3Vector>(MaterialExpression))
					{
						FbxDouble3 FbxColor(ConstExpr3->Constant.R, ConstExpr3->Constant.G, ConstExpr3->Constant.B);
						FbxProperty.Set(FbxColor);
					}
					if (UMaterialExpressionConstant4Vector* ConstExpr4 = Cast<UMaterialExpressionConstant4Vector>(MaterialExpression))
					{
						FbxDouble3 FbxColor(ConstExpr4->Constant.R, ConstExpr4->Constant.G, ConstExpr4->Constant.B);
						FbxProperty.Set(FbxColor);
					}
				}
			}

			void HandleExpressionInput(UMaterialInterface* MaterialInterface, fbxsdk::FbxScene* Scene, fbxsdk::FbxSurfaceMaterial* FbxMaterial,
				const char* FbxPropertyName, const char* FbxFactorPropertyName,
				FExpressionInput* ExpressionInput)
			{
				if (ExpressionInput && ExpressionInput->Expression && ExpressionInput->IsConnected())
				{
					HandleMaterialExpression(MaterialInterface, Scene, FbxMaterial,
						FbxPropertyName, FbxFactorPropertyName,
						ExpressionInput->GetTracedInput().Expression);
				}
			}
		}
	}

	FFbxMaterialBakingMeshData::FFbxMaterialBakingMeshData()
	{
	}

	FFbxMaterialBakingMeshData::FFbxMaterialBakingMeshData(UModel* Model, ABrush* Actor, int32 InLODIndex)
		: bHasMeshData(true)
		, LODIndex(InLODIndex)
	{
		FMeshDescription Mesh;
		FStaticMeshAttributes(Mesh).Register();
		TArray<FStaticMaterial>	Materials;
		GetBrushMesh(Actor, Model, Mesh, Materials);

		StaticMesh = CreateStaticMesh(Mesh, Materials, GetTransientPackage(), Actor ? Actor->GetFName() : Model->GetFName());

		const IMeshMergeUtilities& MeshMergeUtilities = FModuleManager::Get().LoadModuleChecked<IMeshMergeModule>("MeshMergeUtilities").GetUtilities();
		FStaticMeshAttributes(Description).Register();

		if (StaticMeshComponent != nullptr)
		{
			//MeshMergeUtilities.RetrieveMeshDescription (for StaticMeshComponent) uses bApplyComponentTransform==true, 
			// however MeshData->Description is only used for DegenerateVertices/Triangles and UV checks, and material baking.
			MeshMergeUtilities.RetrieveMeshDescription(StaticMeshComponent, LODIndex, Description, true);

			constexpr int32 LightMapLODIndex = 0; // TODO: why is this zero?
			if (StaticMeshComponent->LODData.IsValidIndex(LightMapLODIndex))
			{
				const FStaticMeshComponentLODInfo& LODData = StaticMeshComponent->LODData[LightMapLODIndex];
				const FMeshMapBuildData* BuildData = StaticMeshComponent->GetMeshMapBuildData(LODData);
				if (BuildData != nullptr)
				{
					LightMap = BuildData->LightMap;
					LightMapResourceCluster = BuildData->ResourceCluster;
				}
			}
		}
		else
		{
			MeshMergeUtilities.RetrieveMeshDescription(StaticMesh, LODIndex, Description);
		}

		LightMapTexCoord = StaticMesh->GetLightMapCoordinateIndex();
		const int32 NumTexCoords = StaticMesh->GetLODForExport(LODIndex).VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords();
		BakeUsingTexCoord = FMath::Min(LightMapTexCoord, NumTexCoords - 1);
	}

	FFbxMaterialBakingMeshData::FFbxMaterialBakingMeshData(const UStaticMesh* InStaticMesh, const UStaticMeshComponent* InStaticMeshComponent, int32 InLODIndex)
		: bHasMeshData(true)
		, StaticMeshComponent(InStaticMeshComponent)
		, StaticMesh(InStaticMesh)
		, LODIndex(InLODIndex)
	{
		const IMeshMergeUtilities& MeshMergeUtilities = FModuleManager::Get().LoadModuleChecked<IMeshMergeModule>("MeshMergeUtilities").GetUtilities();
		FStaticMeshAttributes(Description).Register();

		if (StaticMeshComponent != nullptr)
		{
			//MeshMergeUtilities.RetrieveMeshDescription (for StaticMeshComponent) uses bApplyComponentTransform==true, 
			// however MeshData->Description is only used for DegenerateVertices/Triangles and UV checks, and material baking.
			MeshMergeUtilities.RetrieveMeshDescription(StaticMeshComponent, LODIndex, Description, true);

			constexpr int32 LightMapLODIndex = 0; // TODO: why is this zero?
			if (StaticMeshComponent->LODData.IsValidIndex(LightMapLODIndex))
			{
				const FStaticMeshComponentLODInfo& LODData = StaticMeshComponent->LODData[LightMapLODIndex];
				const FMeshMapBuildData* BuildData = StaticMeshComponent->GetMeshMapBuildData(LODData);
				if (BuildData != nullptr)
				{
					LightMap = BuildData->LightMap;
					LightMapResourceCluster = BuildData->ResourceCluster;
				}
			}
		}
		else
		{
			MeshMergeUtilities.RetrieveMeshDescription(StaticMesh, LODIndex, Description);
		}

		LightMapTexCoord = StaticMesh->GetLightMapCoordinateIndex();
		const int32 NumTexCoords = StaticMesh->GetLODForExport(LODIndex).VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords();
		BakeUsingTexCoord = FMath::Min(LightMapTexCoord, NumTexCoords - 1);
	}

	FFbxMaterialBakingMeshData::FFbxMaterialBakingMeshData(const USkeletalMesh* InSkeletalMesh, const USkeletalMeshComponent* SkeletalMeshComponent, int32 InLODIndex)
		: bHasMeshData(true)
		, SkeletalMesh(InSkeletalMesh)
		, LODIndex(InLODIndex)
	{
		const IMeshMergeUtilities& MeshMergeUtilities = FModuleManager::Get().LoadModuleChecked<IMeshMergeModule>("MeshMergeUtilities").GetUtilities();
		FStaticMeshAttributes(Description).Register();

		if (SkeletalMeshComponent != nullptr)
		{
			MeshMergeUtilities.RetrieveMeshDescription(SkeletalMeshComponent, LODIndex, Description, true);
		}
		else
		{
			// NOTE: this is a workaround for the fact that there's no overload for FMeshMergeHelpers::RetrieveMesh
			// that accepts a USkeletalMesh, only a USkeletalMeshComponent.
			// Writing a custom utility function that would work on a "standalone" skeletal mesh is problematic
			// since we would need to implement an equivalent of USkinnedMeshComponent::GetCPUSkinnedVertices too.

			if (UWorld* World = GEditor->GetEditorWorldContext().World())
			{
				FActorSpawnParameters SpawnParams;
				SpawnParams.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
				SpawnParams.ObjectFlags |= RF_Transient;
				SpawnParams.bAllowDuringConstructionScript = true;

				if (AActor* TempActor = World->SpawnActor<AActor>(SpawnParams))
				{
					USkeletalMeshComponent* TempComponent = NewObject<USkeletalMeshComponent>(TempActor, TEXT(""), RF_Transient);
					TempComponent->RegisterComponent();
					TempComponent->SetSkeletalMesh(const_cast<USkeletalMesh*>(SkeletalMesh));

					MeshMergeUtilities.RetrieveMeshDescription(TempComponent, LODIndex, Description, true);

					World->DestroyActor(TempActor, false, false);
				}
			}
		}

		// TODO: don't assume last UV channel is non-overlapping
		const int32 NumTexCoords = SkeletalMesh->GetResourceForRendering()->LODRenderData[LODIndex].StaticVertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords();
		BakeUsingTexCoord = NumTexCoords - 1;
	}

	TArray<int32> FFbxMaterialBakingMeshData::GetSectionIndices(const int32& MaterialIndex) const
	{
		if (StaticMesh)
		{
			return FbxMaterialExportHelpers::GetSectionIndices(StaticMesh, LODIndex, MaterialIndex);
		}

		if (SkeletalMesh)
		{
			return FbxMaterialExportHelpers::GetSectionIndices(SkeletalMesh, LODIndex, MaterialIndex);
		}

		return TArray<int32>();
	}

	int32 FFbxMaterialBakingMeshData::GetUModelStaticMeshMaterialIndex(const UMaterialInterface* MaterialInterface) const
	{
		//UModel to StaticMesh generation usaes the MaterialInterface's FName to set the Slot naming:
		return StaticMesh->GetMaterialIndexFromImportedMaterialSlotName(MaterialInterface->GetFName());
	}

	namespace FFbxMaterialExportUtilities
	{
		void BakeMaterialProperty(const UFbxExportOption* FbxExportOptions,
			fbxsdk::FbxScene* Scene, fbxsdk::FbxSurfaceMaterial* FbxMaterial, const char* FbxPropertyName,
			const FMaterialPropertyEx& Property, const UMaterialInterface* Material, const int32& MaterialIndex,
			const FFbxMaterialBakingMeshData& MeshData,
			const FString& ExportFolderPath)
		{
			using namespace FbxMaterialExportHelpers;

			const FIntPoint DefaultBakeSize = FIntPoint(512, 512);
			const FBox2f TexCoordBounds = FBox2f{ { 0.0f, 0.0f }, { 1.0f, 1.0f } };
			const bool bFillAlpha = true;

			if (FbxExportOptions->BakeMaterialInputs == EFbxMaterialBakeMode::Disabled)
			{
				return;
			}

			fbxsdk::FbxProperty FbxColorProperty = FbxMaterial->FindProperty(FbxPropertyName);
			if (!FbxColorProperty.IsValid())
			{
				return;
			}

			FIntPoint BakeSize = FbxExportOptions->DefaultMaterialBakeSize.bAutoDetect ? TryGetMaxTextureSize(Material, Property, DefaultBakeSize) : FbxExportOptions->DefaultMaterialBakeSize.Size;

			if (BakeSize == FIntPoint(0, 0))
			{
				return;
			}

			bool bNeedsMeshData = FbxExportOptions->BakeMaterialInputs == EFbxMaterialBakeMode::UseMeshData && MeshData.bHasMeshData && NeedsMeshDataForProperty(Material, Property);

			TArray<int32> TexCoords;
			GetAllTextureCoordinateIndices(Material, Property, TexCoords);

			int32 TexCoordIndex = bNeedsMeshData ? MeshData.BakeUsingTexCoord : (TexCoords.Num() > 0 ? TexCoords[0] : 0);

			FMeshData MeshSet;
			MeshSet.TextureCoordinateBox = FBox2D(TexCoordBounds);
			MeshSet.TextureCoordinateIndex = TexCoordIndex;
			MeshSet.MaterialIndices = MeshData.GetSectionIndices(MaterialIndex); // NOTE: MaterialIndices is actually section indices
			if (bNeedsMeshData)
			{
				MeshSet.MeshDescription = const_cast<FMeshDescription*>(&MeshData.Description);
				MeshSet.LightMap = MeshData.LightMap;
				MeshSet.LightMapIndex = MeshData.LightMapTexCoord;
				MeshSet.LightmapResourceCluster = MeshData.LightMapResourceCluster;
				MeshSet.PrimitiveData = MeshData.StaticMeshComponent ? FPrimitiveData(MeshData.StaticMeshComponent) : (MeshData.StaticMesh ? FPrimitiveData(MeshData.StaticMesh) : FPrimitiveData(MeshData.SkeletalMesh));
			}

			FMaterialDataEx MatSet;
			MatSet.Material = const_cast<UMaterialInterface*>(Material);
			MatSet.PropertySizes.Add(Property, BakeSize);
			MatSet.bTangentSpaceNormal = true;

			TArray<FMeshData*> MeshSettings;
			TArray<FMaterialDataEx*> MatSettings;
			MeshSettings.Add(&MeshSet);
			MatSettings.Add(&MatSet);

			TArray<FBakeOutputEx> BakeOutputs;
			IMaterialBakingModule& Module = FModuleManager::Get().LoadModuleChecked<IMaterialBakingModule>("MaterialBaking");

			Module.SetLinearBake(true);
			Module.BakeMaterials(MatSettings, MeshSettings, BakeOutputs);
			const bool bIsLinearBake = Module.IsLinearBake(Property);
			Module.SetLinearBake(false);

			FBakeOutputEx& BakeOutput = BakeOutputs[0];

			TArray<FColor> BakedPixels = MoveTemp(BakeOutput.PropertyData.FindChecked(Property));
			const FIntPoint BakedSize = BakeOutput.PropertySizes.FindChecked(Property);
			const float EmissiveScale = BakeOutput.EmissiveScale;

			if (bFillAlpha)
			{
				// NOTE: alpha is 0 by default after baking a property, but we prefer 255 (1.0).
				// It makes it easier to view the exported textures.
				for (FColor& Pixel : BakedPixels)
				{
					Pixel.A = 255;
				}
			}

			if (Property == MP_EmissiveColor)
			{
				bool bFromSRGB = !bIsLinearBake;
				bool bToSRGB = true;
				TransformColorSpace(BakedPixels, bFromSRGB, true);
			}

			//Save Out:
			const void* InRawData = BakedPixels.GetData();
			const int64 ByteLength = BakedSize.X * BakedSize.Y * sizeof(FColor);
			int64 InRawSize = ByteLength;
			int32 InWidth = BakedSize.X;
			int32 InHeight = BakedSize.Y;
			ERGBFormat InRGBFormat = ERGBFormat::BGRA;
			int32 InBitDepth = 8;
			EImageFormat InCompressionFormat = EImageFormat::PNG;
			int32 InCompressionQuality = 0;
			TArray64<uint8> OutCompressedData;

			IImageWrapperModule& ImageWrapperModule = FModuleManager::Get().LoadModuleChecked<IImageWrapperModule>(TEXT("ImageWrapper"));
			const TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(InCompressionFormat);

			if (ImageWrapper.IsValid() && ImageWrapper->SetRaw(InRawData, InRawSize, InWidth, InHeight, InRGBFormat, InBitDepth))
			{
				const TArray64<uint8>& CompressedData = ImageWrapper->GetCompressed(InCompressionQuality);

				FString FileName = Material->GetName() + TEXT("_") + Property.ToString() + TEXT("_") + FString::FromInt(MaterialIndex) + TEXT(".png");
				FString FilePath = ExportFolderPath + TEXT("/") + FileName;
				if (FFileHelper::SaveArrayToFile(CompressedData, *FilePath))
				{
					FbxFileTexture* lTexture = FbxFileTexture::Create(Scene, "EnvSamplerTex");
					lTexture->SetFileName(TCHAR_TO_UTF8(*FilePath)); //SetRelativeFileName does not seem to work?
					lTexture->SetTextureUse(FbxTexture::eStandard);
					lTexture->SetMappingType(FbxTexture::eUV);
					lTexture->ConnectDstProperty(FbxColorProperty);
				}
			}
		}

		bool GetInterchangeShadingModel(const UMaterialInterface* MaterialInterface, bool& bLambert)
		{
			using namespace FbxMaterialExportHelpers::InterchangeMaterialProcessHelpers;

			FString PathName = MaterialInterface->GetMaterial()->GetPathName();
			if (MaterialInterface->IsA(UMaterialInstance::StaticClass()))
			{
				if (PathName == LambertSurfaceMaterialPath
					|| PathName == PhongSurfaceMaterialPath)
				{
					bLambert = PathName == LambertSurfaceMaterialPath;

					return true;
				}
			}
			else
			{
				//Acquire the Material's expressions and check if any matches the MF_PhongToMetalRoughness MaterialFunctionCall:
				TConstArrayView<TObjectPtr<UMaterialExpression>> MaterialExpressions = MaterialInterface->GetMaterial()->GetExpressions();

				bool MFPhongUsed = false;
				for (const TObjectPtr<UMaterialExpression>& MaterialExpression : MaterialExpressions)
				{
					if (UMaterialExpressionMaterialFunctionCall* FunctionCallExpression = Cast<UMaterialExpressionMaterialFunctionCall>(MaterialExpression))
					{
						if (FunctionCallExpression->MaterialFunction)
						{
							FString MF_PathName = FunctionCallExpression->MaterialFunction->GetPathName();

							if (MF_PhongToMetalRoughnessPathName == MF_PathName)
							{
								MFPhongUsed = true;
								break;
							}
						}
					}
				}

				bLambert = !MFPhongUsed;

				return MFPhongUsed || (MaterialInterface->AssetImportData.IsA(UInterchangeAssetImportData::StaticClass()));
			}

			return false;
		}

		void ProcessInterchangeMaterials(UMaterialInterface* MaterialInterface, fbxsdk::FbxScene* Scene, fbxsdk::FbxSurfaceMaterial* FbxMaterial)
		{
			using namespace FbxMaterialExportHelpers::InterchangeMaterialProcessHelpers;

			FString PathName = MaterialInterface->GetMaterial()->GetPathName();

			if (MaterialInterface->IsA(UMaterialInstance::StaticClass()))
			{
				if (PathName == LambertSurfaceMaterialPath
					|| PathName == PhongSurfaceMaterialPath)
				{

					bool bSpecularToBeBaked = false; //also indicates to bake the BaseColor and Metallic
					if (PathName == PhongSurfaceMaterialPath)
					{
						//SpecularColor
						//TEXT("SpecularColor");
						//TEXT("SpecularColorMap");
						//TEXT("SpecularColorMapWeight");
						if (!HandleMaterialProperty(MaterialInterface, Scene, FbxMaterial, TEXT("SpecularColor"), MP_Specular, FbxSurfaceMaterial::sSpecular, FbxSurfaceMaterial::sSpecularFactor))
						{
							bSpecularToBeBaked = true;
						}

						//Roughness
						//TEXT("Shininess");
						//TEXT("ShininessMap");
						//TEXT("ShininessMapWeight");
						HandleMaterialProperty(MaterialInterface, Scene, FbxMaterial, TEXT("Shininess"), MP_Roughness, FbxSurfaceMaterial::sShininess, "NoFactorForThisProperty", false);

					}

					//Diffuse
					//TEXT("DiffuseColor");
					//TEXT("DiffuseColorMap");
					//TEXT("DiffuseColorMapWeight");
					if (!bSpecularToBeBaked) HandleMaterialProperty(MaterialInterface, Scene, FbxMaterial, TEXT("DiffuseColor"), MP_BaseColor, FbxSurfaceMaterial::sDiffuse, FbxSurfaceMaterial::sDiffuseFactor);

					//Emissive
					//TEXT("EmissiveColorMap");
					//TEXT("EmissiveColorMapWeight");
					//TEXT("EmissiveColor");
					HandleMaterialProperty(MaterialInterface, Scene, FbxMaterial, TEXT("EmissiveColor"), MP_EmissiveColor, FbxSurfaceMaterial::sEmissive, FbxSurfaceMaterial::sEmissiveFactor);

					//Normal
					//TEXT("NormalMap");
					//TEXT("NormalMapWeight");
					HandleMaterialProperty(MaterialInterface, Scene, FbxMaterial, TEXT("Normal"), MP_Normal, FbxSurfaceMaterial::sNormalMap, "NoFactorForThisProperty");

					//AmbientOcclusion is NOT AmbientColor, FbxSurfaceMaterail supports AmbientColor!
					// Meaning Interchange's Lambert and Phong Surface Materials have no support for AmbientColor
					//TEXT("AmbientOcclusionMap");
					//TEXT("AmbientOcclusionMapWeight");
					//HandleProperty(TEXT("AmbientOcclusion"), MP_AmbientOcclusion, FbxSurfaceMaterial::sAmbient, FbxSurfaceMaterial::sAmbientFactor);
				}
			}
			else //!MaterialInstance (==Material)
			{
				//Acquire the Material's expressions and check if any matches the MF_PhongToMetalRoughness MaterialFunctionCall:
				UMaterialExpressionMaterialFunctionCall* ImportedMF_PhongToMetalRoughness = GetInterchangeMFPongToMetalRoughness(MaterialInterface);

				if (ImportedMF_PhongToMetalRoughness)
				{
					TArrayView<FExpressionInput*> InputsView = ImportedMF_PhongToMetalRoughness->GetInputsView();

					for (FExpressionInput* Input : InputsView)
					{
						if (!Input || !Input->Expression || !Input->IsConnected())
						{
							continue;
						}

						FString Name = Input->InputName.ToString();
						UMaterialExpression* InputExpression = Input->GetTracedInput().Expression;

						if (Name == TEXT("AmbientColor"))
						{
							//Always set the ambient to zero since we dont have ambient in unreal we want to avoid default value in DCCs
							((FbxSurfaceLambert*)FbxMaterial)->Ambient.Set(FbxDouble3(0.0, 0.0, 0.0));
							//Overwrite with Interchange values:
							HandleMaterialExpression(MaterialInterface, Scene, FbxMaterial, FbxSurfaceMaterial::sAmbient, FbxSurfaceMaterial::sAmbientFactor, InputExpression);
						}
						else if (Name == TEXT("DiffuseColor"))
						{
							HandleMaterialExpression(MaterialInterface, Scene, FbxMaterial, FbxSurfaceMaterial::sDiffuse, FbxSurfaceMaterial::sDiffuseFactor, InputExpression);
						}
						else if (Name == TEXT("SpecularColor"))
						{
							HandleMaterialExpression(MaterialInterface, Scene, FbxMaterial, FbxSurfaceMaterial::sSpecular, FbxSurfaceMaterial::sSpecularFactor, InputExpression);
						}
						else if (Name == TEXT("Shininess"))
						{
							HandleMaterialExpression(MaterialInterface, Scene, FbxMaterial, FbxSurfaceMaterial::sShininess, "NoFactorForThisProperty", InputExpression);
						}
					}
				}
				

				if (MaterialInterface->AssetImportData.IsA(UInterchangeAssetImportData::StaticClass()))
				{
					UMaterial* Material = MaterialInterface->GetMaterial();

					if (ImportedMF_PhongToMetalRoughness == nullptr)
					{
						//AmbientColor
						// Only supported via MF_PhongToMetalRoughness
						
						//Diffuse
						HandleExpressionInput(MaterialInterface, Scene, FbxMaterial, FbxSurfaceMaterial::sDiffuse, FbxSurfaceMaterial::sDiffuseFactor, Material->GetExpressionInputForProperty(MP_BaseColor));

						//Specular
						HandleExpressionInput(MaterialInterface, Scene, FbxMaterial, FbxSurfaceMaterial::sSpecular, FbxSurfaceMaterial::sSpecularFactor, Material->GetExpressionInputForProperty(MP_Specular));
						
						//Shininess
						// Only supported via MF_PhongToMetalRoughness
					}

					//Normal
					HandleExpressionInput(MaterialInterface, Scene, FbxMaterial, FbxSurfaceMaterial::sNormalMap, "NoFactorForThisProperty", Material->GetExpressionInputForProperty(MP_Normal));

					//Emissive
					HandleExpressionInput(MaterialInterface, Scene, FbxMaterial, FbxSurfaceMaterial::sEmissive, FbxSurfaceMaterial::sEmissiveFactor, Material->GetExpressionInputForProperty(MP_EmissiveColor));


					UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(MaterialInterface);
					EBlendMode BlendMode = MaterialInstance != nullptr && MaterialInstance->BasePropertyOverrides.bOverride_BlendMode ? MaterialInstance->BasePropertyOverrides.BlendMode : Material->BlendMode;

					if (BlendMode == BLEND_Translucent)
					{
						//Opacity
						HandleExpressionInput(MaterialInterface, Scene, FbxMaterial, FbxSurfaceMaterial::sTransparentColor, "NoFactorForThisProperty", Material->GetExpressionInputForProperty(MP_Opacity));

						//OpacityMask
						HandleExpressionInput(MaterialInterface, Scene, FbxMaterial, FbxSurfaceMaterial::sTransparencyFactor, "NoFactorForThisProperty", Material->GetExpressionInputForProperty(MP_OpacityMask));
					}
				}
			}
		}
	}
}