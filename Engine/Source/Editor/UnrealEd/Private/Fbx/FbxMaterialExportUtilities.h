// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshDescription.h"
#include "SceneTypes.h"


class UModel;
class ABrush;
class UStaticMesh;
class UStaticMeshComponent;
class USkeletalMesh;
class USkeletalMeshComponent;
class FLightMap;
class FLightmapResourceCluster;
class UFbxExportOption;
struct FMaterialPropertyEx;
class UMaterialInterface;

namespace fbxsdk
{
	class FbxScene;
	class FbxSurfaceMaterial;
}


namespace UnFbx
{
	/*
	* Stores the necessary mesh information for Material Baking that requires vertex data
	*/
	struct FFbxMaterialBakingMeshData
	{
		FFbxMaterialBakingMeshData(); //Used for ULandscapes
		FFbxMaterialBakingMeshData(UModel* Model, ABrush* Actor = nullptr, int32 LODIndex = 0);
		FFbxMaterialBakingMeshData(const UStaticMesh* StaticMesh, const UStaticMeshComponent* StaticMeshComponent = nullptr, int32 LODIndex = 0);
		FFbxMaterialBakingMeshData(const USkeletalMesh* SkeletalMesh, const USkeletalMeshComponent* SkeletalMeshComponent = nullptr, int32 LODIndex = 0);

		bool bHasMeshData = false;
		FMeshDescription Description;

		FLightMapRef LightMap = nullptr;
		const FLightmapResourceCluster* LightMapResourceCluster = nullptr;
		int32 LightMapTexCoord = 0;

		int32 BakeUsingTexCoord = 0;

		//For FPrimitiveData:
		const UStaticMeshComponent* StaticMeshComponent = nullptr;
		const UStaticMesh* StaticMesh = nullptr;
		const USkeletalMesh* SkeletalMesh = nullptr;

		int32 LODIndex = 0;

		TArray<int32> GetSectionIndices(const int32& MaterialIndex) const;

		int32 GetUModelStaticMeshMaterialIndex(const UMaterialInterface* MaterialInterface) const; //Used for UModel
	};

	namespace FFbxMaterialExportUtilities
	{
		/*
		* Checks Material if it was Interchange imported, and if so, was it Lambert or Phong
		* Material presumed InterchangeImported when 1 of the following conditions are met:
		*		- MI -> PathName == Interchange.LambertSurfaceMaterialPath
		*		- MI -> PathName == Interchange.PhongSurfaceMaterialPath
		*		- M -> Has MF with path pointing to Interchange.MF_PhongToMetalRoughness
		*		- M -> AssetImportData == UInterchangeAssetImportData
		*/
		bool GetInterchangeShadingModel(const UMaterialInterface* MaterialInterface, bool& bLambert);


		/*
		* Bakes the Material Property. Saves it to png at the given folder path. Sets the Absolute path for the FbxMaterial for the given FbxPropertyName
		*/
		void BakeMaterialProperty(const UFbxExportOption* FbxExportOptions,
			fbxsdk::FbxScene* Scene, fbxsdk::FbxSurfaceMaterial* FbxMaterial, const char* FbxPropertyName,
			const FMaterialPropertyEx& Property, const UMaterialInterface* MaterialInterface, const int32& MaterialIndex,
			const FFbxMaterialBakingMeshData& MeshData,
			const FString& ExportFolderPath);

		/*
		* Checks and processes for Interchange Lambert/Phong Surface Materials and PhongToMaterialRoughness MaterialFunctions.
		* Returns Processed Fbx Property Names
		* Material presumed InterchangeImported when 1 of the following conditions are met:
		*		- MI -> PathName == Interchange.LambertSurfaceMaterialPath
		*		- MI -> PathName == Interchange.PhongSurfaceMaterialPath
		*		- M -> Has MF with path pointing to Interchange.MF_PhongToMetalRoughness
		*		- M -> AssetImportData == UInterchangeAssetImportData
		* Note: Possible deviation on Roundtripping:
		*				-ReflectionFactor is not used on import and is not stored so cannot be retrived on export.
		*				-AmbientColor is only supported for MF_PhongToMetalRoughness (Interchange's Phong and Lambert Surface Materials do not support it.)
		*/
		void ProcessInterchangeMaterials(UMaterialInterface* MaterialInterface, fbxsdk::FbxScene* Scene, fbxsdk::FbxSurfaceMaterial* FbxMaterial);
	}
}