// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Misc/PackageName.h"
#include "Materials/MaterialInterface.h"
#include "MaterialExpressionIO.h"
#include "Materials/Material.h"
#include "Factories/MaterialFactoryNew.h"
#include "Engine/Texture.h"
#include "Factories/TextureFactory.h"
#include "Engine/Texture2D.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Materials/MaterialExpressionTextureCoordinate.h"
#include "Materials/MaterialExpressionVectorParameter.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Factories/MaterialInstanceConstantFactoryNew.h"
#include "FbxImporter.h"
#include "ObjectTools.h"
#include "PackageTools.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "Misc/FbxErrors.h"
#include "AssetRegistry/ARFilter.h"
#include "Factories/MaterialImportHelpers.h"
#include "MaterialEditingLibrary.h"
#include "Engine/RendererSettings.h"

DEFINE_LOG_CATEGORY_STATIC(LogFbxMaterialImport, Log, All);

#define LOCTEXT_NAMESPACE "FbxMaterialImport"

using namespace UnFbx;

UTexture* UnFbx::FFbxImporter::ImportTexture(FbxFileTexture* FbxTexture, bool bSetupAsNormalMap)
{
	if (!FbxTexture || !CanImportClass(UTexture2D::StaticClass()))
	{
		return nullptr;
	}

	// create an unreal texture asset
	UTexture* UnrealTexture = NULL;
	FString AbsoluteFilename = UTF8_TO_TCHAR(FbxTexture->GetFileName());
	FString Extension = FPaths::GetExtension(AbsoluteFilename).ToLower();
	// name the texture with file name
	FString TextureName;
	if (FString* UniqueName = FbxTextureToUniqueNameMap.Find(FbxTexture))
	{
		TextureName = *UniqueName;
	}
	else
	{
		TextureName = FPaths::GetBaseFilename(AbsoluteFilename);
		TextureName = ObjectTools::SanitizeObjectName(TextureName);
	}

	// set where to place the textures
	FString BasePackageName = FPackageName::GetLongPackagePath(Parent->GetOutermost()->GetName()) / TextureName;
	BasePackageName = UPackageTools::SanitizePackageName(BasePackageName);

	UTexture* ExistingTexture = NULL;
	UPackage* TexturePackage = NULL;
	// First check if the asset already exists.
	{
		FString ObjectPath = BasePackageName + TEXT(".") + TextureName;
		ExistingTexture = LoadObject<UTexture>(NULL, *ObjectPath, nullptr, LOAD_Quiet | LOAD_NoWarn);
	}


	if (!ExistingTexture)
	{
		const FString Suffix(TEXT(""));

		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
		FString FinalPackageName;
		AssetToolsModule.Get().CreateUniqueAssetName(BasePackageName, Suffix, FinalPackageName, TextureName);

		TexturePackage = CreatePackage( *FinalPackageName);
	}
	else
	{
		TexturePackage = ExistingTexture->GetOutermost();
	}

	FString FinalFilePath;
	if (IFileManager::Get().FileExists(*AbsoluteFilename))
	{
		// try opening from absolute path
		FinalFilePath = AbsoluteFilename;
	}
	else if (IFileManager::Get().FileExists(*(FileBasePath / UTF8_TO_TCHAR(FbxTexture->GetRelativeFileName()))))
	{
		// try fbx file base path + relative path
		FinalFilePath = FileBasePath / UTF8_TO_TCHAR(FbxTexture->GetRelativeFileName());
	}
	else if (IFileManager::Get().FileExists(*(FileBasePath / AbsoluteFilename)))
	{
		// Some fbx files dont store the actual absolute filename as absolute and it is actually relative.  Try to get it relative to the FBX file we are importing
		FinalFilePath = FileBasePath / AbsoluteFilename;
	}
	else
	{
		UE_LOGF(LogFbxMaterialImport, Display, "Unable to find Texture file %ls", *AbsoluteFilename);
	}

	TArray<uint8> DataBinary;
	if (!FinalFilePath.IsEmpty())
	{
		FFileHelper::LoadFileToArray(DataBinary, *FinalFilePath);
	}

	if (DataBinary.Num()>0)
	{
		UE_LOGF(LogFbxMaterialImport, Verbose, "Loading texture file %ls",*FinalFilePath);
		const uint8* PtrTexture = DataBinary.GetData();
		auto TextureFact = NewObject<UTextureFactory>();
		TextureFact->AddToRoot();

		// save texture settings if texture exist
		TextureFact->SuppressImportOverwriteDialog();
		const TCHAR* TextureType = *Extension;
		const bool bTextureAssetAlreadyExists = FindObject<UTexture>(TexturePackage, *TextureName) != nullptr;

		// Unless the normal map setting is used during import, 
		//	the user has to manually hit "reimport" then "recompress now" button
		if ( bSetupAsNormalMap )
		{
			if (!ExistingTexture)
			{
				TextureFact->LODGroup = TEXTUREGROUP_WorldNormalMap;
				TextureFact->CompressionSettings = TC_Normalmap;
				TextureFact->bFlipNormalMapGreenChannel = ImportOptions->bInvertNormalMap;
			}
			else
			{
				UE_LOGF(LogFbxMaterialImport, Warning, "Manual texture reimport and recompression may be needed for %ls", *TextureName);
			}
		}

		UnrealTexture = (UTexture*)TextureFact->FactoryCreateBinary(
			UTexture2D::StaticClass(), TexturePackage, *TextureName, 
			RF_Standalone|RF_Public, NULL, TextureType, 
			PtrTexture, PtrTexture+DataBinary.Num(), GWarn );

		if ( UnrealTexture != NULL )
		{
			if ( !bTextureAssetAlreadyExists )
			{
				//This asset did not override any other asset during its creation, so it is considered as newly created.
				 CreatedObjects.Add(UnrealTexture);
			}

			//Make sure the AssetImportData point on the texture file and not on the fbx files since the factory point on the fbx file
			UnrealTexture->AssetImportData->Update(IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*FinalFilePath));

			// Notify the asset registry
			FAssetRegistryModule::AssetCreated(UnrealTexture);

			// Set the dirty flag so this package will get saved later
			TexturePackage->SetDirtyFlag(true);
		}
		TextureFact->RemoveFromRoot();
	}

	return UnrealTexture;
}

void UnFbx::FFbxImporter::ImportTexturesFromNode(FbxNode* Node)
{
	FbxProperty Property;
	int32 NbMat = Node->GetMaterialCount();

	// visit all materials
	int32 MaterialIndex;
	for (MaterialIndex = 0; MaterialIndex < NbMat; MaterialIndex++)
	{
		FbxSurfaceMaterial *Material = Node->GetMaterial(MaterialIndex);

		//go through all the possible textures
		if(Material)
		{
			int32 TextureIndex;
			FBXSDK_FOR_EACH_TEXTURE(TextureIndex)
			{
				Property = Material->FindProperty(FbxLayerElement::sTextureChannelNames[TextureIndex]);

				if( Property.IsValid() )
				{
					FbxTexture * lTexture= NULL;

					//Here we have to check if it's layered textures, or just textures:
					int32 LayeredTextureCount = Property.GetSrcObjectCount<FbxLayeredTexture>();
					FbxString PropertyName = Property.GetName();
					if(LayeredTextureCount > 0)
					{
						for(int32 LayerIndex=0; LayerIndex<LayeredTextureCount; ++LayerIndex)
						{
							FbxLayeredTexture *lLayeredTexture = Property.GetSrcObject<FbxLayeredTexture>(LayerIndex);
							int32 NbTextures = lLayeredTexture->GetSrcObjectCount<FbxTexture>();
							for(int32 TexIndex =0; TexIndex<NbTextures; ++TexIndex)
							{
								FbxFileTexture* Texture = lLayeredTexture->GetSrcObject<FbxFileTexture>(TexIndex);
								if(Texture)
								{
									ImportTexture(Texture, PropertyName == FbxSurfaceMaterial::sNormalMap || PropertyName == FbxSurfaceMaterial::sBump);
								}
							}
						}
					}
					else
					{
						//no layered texture simply get on the property
						int32 NbTextures = Property.GetSrcObjectCount<FbxTexture>();
						for(int32 TexIndex =0; TexIndex<NbTextures; ++TexIndex)
						{

							FbxFileTexture* Texture = Property.GetSrcObject<FbxFileTexture>(TexIndex);
							if(Texture)
							{
								ImportTexture(Texture, PropertyName == FbxSurfaceMaterial::sNormalMap || PropertyName == FbxSurfaceMaterial::sBump);
							}
						}
					}
				}
			}

		}//end if(Material)

	}// end for MaterialIndex
}

//-------------------------------------------------------------------------
//
//-------------------------------------------------------------------------

//Enable debug log of fbx material properties, this will log all material properties that are in the FBX file
#define DEBUG_LOG_FBX_MATERIAL_PROPERTIES 0
#if DEBUG_LOG_FBX_MATERIAL_PROPERTIES
void LogPropertyAndChild(FbxSurfaceMaterial& FbxMaterial, const FbxProperty &Property)
{
	FbxString PropertyName = Property.GetHierarchicalName();
	UE_LOGF(LogFbxMaterialImport, Display, "Property Name [%ls]", UTF8_TO_TCHAR(PropertyName.Buffer()));
	int32 TextureCount = Property.GetSrcObjectCount<FbxTexture>();
	if (TextureCount > 0)
	{
		for (int32 TextureIndex = 0; TextureIndex < TextureCount; ++TextureIndex)
		{
			FbxFileTexture* TextureObj = Property.GetSrcObject<FbxFileTexture>(TextureIndex);
			if (TextureObj != nullptr)
			{
				UE_LOGF(LogFbxMaterialImport, Display, "Texture Path [%ls]", UTF8_TO_TCHAR(TextureObj->GetFileName()));
			}
		}
	}
	const FbxProperty &NextProperty = FbxMaterial.GetNextProperty(Property);
	if (NextProperty.IsValid())
	{
		LogPropertyAndChild(FbxMaterial, NextProperty);
	}
}
#endif

bool UnFbx::FFbxImporter::CreateAndLinkExpressionForMaterialProperty(
							const FbxSurfaceMaterial& FbxMaterial,
							UMaterial* UnrealMaterial,
							const char* MaterialProperty ,
							FExpressionInput& MaterialInput, 
							bool bSetupAsNormalMap,
							TArray<FString>& UVSet,
							const FVector2D& Location)
{
	bool bCreated = false;
	FbxProperty FbxProperty = FbxMaterial.FindProperty( MaterialProperty );
	if( FbxProperty.IsValid() )
	{
		int32 UnsupportedTextureCount = FbxProperty.GetSrcObjectCount<FbxLayeredTexture>();
		UnsupportedTextureCount += FbxProperty.GetSrcObjectCount<FbxProceduralTexture>();
		if (UnsupportedTextureCount>0)
		{
			UE_LOGF(LogFbxMaterialImport, Warning,"Layered or procedural Textures are not supported (material %ls)",UTF8_TO_TCHAR(FbxMaterial.GetName()));
		}
		else
		{
			int32 TextureCount = FbxProperty.GetSrcObjectCount<FbxTexture>();
			if (TextureCount>0)
			{
				const bool bEnableVirtualTextureOpacityMask = GetDefault<URendererSettings>()->bEnableVirtualTextureOpacityMask;

				for(int32 TextureIndex =0; TextureIndex<TextureCount; ++TextureIndex)
				{
					FbxFileTexture* FbxTexture = FbxProperty.GetSrcObject<FbxFileTexture>(TextureIndex);

					// create an unreal texture asset
					UTexture* UnrealTexture = ImportTexture(FbxTexture, bSetupAsNormalMap);
				
					if (UnrealTexture)
					{
						float ScaleU = FbxTexture->GetScaleU();
						float ScaleV = FbxTexture->GetScaleV();
						bool bIsVirtualTexture = UnrealTexture->VirtualTextureStreaming;

						if (bIsVirtualTexture && MaterialProperty == FbxSurfaceMaterial::sTransparencyFactor && !bEnableVirtualTextureOpacityMask)
						{
							//Virtual textures are not supported in the OpacityMask slot, convert any textures back to a regular texture.
							//TODO, add a tracking of the materials created during the import so we can refresh here them if the TextureFactory actually found a existing asset instead of creating a new one.
							if (UTexture2D* UnrealTexture2D = Cast<UTexture2D>(UnrealTexture))
							{
								TArray<UTexture2D*> TexturesToConvert = { UnrealTexture2D };
								FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
								AssetToolsModule.Get().ConvertVirtualTextures(TexturesToConvert, true);
							}
							else
							{
								UnrealTexture->VirtualTextureStreaming = false;
							}
							bIsVirtualTexture = false;

							FString TextureName = UnrealTexture->GetName();
							UE_LOGF(LogFbxMaterialImport, Warning, "Texture %ls could not be imported as a Virtual Texture because the project renderer settings disable virtual textures in OpacityMask property (material %ls).", *TextureName, UTF8_TO_TCHAR(FbxMaterial.GetName()));
						}

						// and link it to the material 
						UMaterialExpressionTextureSample* UnrealTextureExpression = NewObject<UMaterialExpressionTextureSample>(UnrealMaterial);
						UnrealMaterial->GetExpressionCollection().AddExpression( UnrealTextureExpression );
						MaterialInput.Expression = UnrealTextureExpression;
						UnrealTextureExpression->Texture = UnrealTexture;
						UnrealTextureExpression->SamplerType = bSetupAsNormalMap ? 
							(bIsVirtualTexture ? SAMPLERTYPE_VirtualNormal : SAMPLERTYPE_Normal) :
							(bIsVirtualTexture ? SAMPLERTYPE_VirtualColor : SAMPLERTYPE_Color);

						UnrealTextureExpression->MaterialExpressionEditorX = FMath::TruncToInt(Location.X);
						UnrealTextureExpression->MaterialExpressionEditorY = FMath::TruncToInt(Location.Y);

						// add/find UVSet and set it to the texture
						FbxString UVSetName = FbxTexture->UVSet.Get();
						FString LocalUVSetName = UTF8_TO_TCHAR(UVSetName.Buffer());
						if (LocalUVSetName.IsEmpty())
						{
							LocalUVSetName = TEXT("UVmap_0");
						}
						int32 SetIndex = UVSet.Find(LocalUVSetName);
						if( (SetIndex != 0 && SetIndex != INDEX_NONE) || ScaleU != 1.0f || ScaleV != 1.0f )
						{
							// Create a texture coord node for the texture sample
							UMaterialExpressionTextureCoordinate* MyCoordExpression = NewObject<UMaterialExpressionTextureCoordinate>(UnrealMaterial);
							UnrealMaterial->GetExpressionCollection().AddExpression(MyCoordExpression);
							MyCoordExpression->CoordinateIndex = (SetIndex >= 0) ? SetIndex : 0;
							MyCoordExpression->UTiling = ScaleU;
							MyCoordExpression->VTiling = ScaleV;
							UnrealTextureExpression->Coordinates.Expression = MyCoordExpression;
							MyCoordExpression->MaterialExpressionEditorX = FMath::TruncToInt(Location.X-175);
							MyCoordExpression->MaterialExpressionEditorY = FMath::TruncToInt(Location.Y);

						}

						bCreated = true;
					}		
				}
			}

			if (MaterialInput.Expression)
			{
				TArray<FExpressionOutput> Outputs = MaterialInput.Expression->GetOutputs();
				FExpressionOutput* Output = Outputs.GetData();
				MaterialInput.Mask = Output->Mask;
				MaterialInput.MaskR = Output->MaskR;
				MaterialInput.MaskG = Output->MaskG;
				MaterialInput.MaskB = Output->MaskB;
				MaterialInput.MaskA = Output->MaskA;
			}
		}
	}

	return bCreated;
}

//-------------------------------------------------------------------------
//
//-------------------------------------------------------------------------
void UnFbx::FFbxImporter::FixupMaterial( const FbxSurfaceMaterial& FbxMaterial, UMaterial* UnrealMaterial )
{
	UMaterialEditorOnlyData* UnrealMaterialEditorOnly = UnrealMaterial->GetEditorOnlyData();
	// add a basic diffuse color if no texture is linked to diffuse
	if (UnrealMaterialEditorOnly->BaseColor.Expression == NULL)
	{
		FbxDouble3 DiffuseColor;
		
		UMaterialExpressionVectorParameter* MyColorExpression = NewObject<UMaterialExpressionVectorParameter>(UnrealMaterial);
		UnrealMaterial->GetExpressionCollection().AddExpression( MyColorExpression );
		UnrealMaterialEditorOnly->BaseColor.Expression = MyColorExpression;

		bool bFoundDiffuseColor = true;
		if( FbxMaterial.GetClassId().Is(FbxSurfacePhong::ClassId) )
		{
			DiffuseColor = ((FbxSurfacePhong&)(FbxMaterial)).Diffuse.Get();
		}
		else if( FbxMaterial.GetClassId().Is(FbxSurfaceLambert::ClassId) )
		{
			DiffuseColor = ((FbxSurfaceLambert&)(FbxMaterial)).Diffuse.Get();
		}
		else
		{
			bFoundDiffuseColor = false;
		}

		if( bFoundDiffuseColor )
		{
			MyColorExpression->DefaultValue.R = (float)(DiffuseColor[0]);
			MyColorExpression->DefaultValue.G = (float)(DiffuseColor[1]);
			MyColorExpression->DefaultValue.B = (float)(DiffuseColor[2]);
		}
		else
		{
			// use random color because there may be multiple materials, then they can be different 
			MyColorExpression->DefaultValue.R = 0.5f+(0.5f*FMath::Rand()) / static_cast<float>(RAND_MAX);
			MyColorExpression->DefaultValue.G = 0.5f+(0.5f*FMath::Rand()) / static_cast<float>(RAND_MAX);
			MyColorExpression->DefaultValue.B = 0.5f+(0.5f*FMath::Rand()) / static_cast<float>(RAND_MAX);
		}

		TArray<FExpressionOutput> Outputs = UnrealMaterialEditorOnly->BaseColor.Expression->GetOutputs();
		FExpressionOutput* Output = Outputs.GetData();
		UnrealMaterialEditorOnly->BaseColor.Mask = Output->Mask;
		UnrealMaterialEditorOnly->BaseColor.MaskR = Output->MaskR;
		UnrealMaterialEditorOnly->BaseColor.MaskG = Output->MaskG;
		UnrealMaterialEditorOnly->BaseColor.MaskB = Output->MaskB;
		UnrealMaterialEditorOnly->BaseColor.MaskA = Output->MaskA;
	}
}

//-------------------------------------------------------------------------
//
//-------------------------------------------------------------------------

FString UnFbx::FFbxImporter::GetMaterialFullName(const FbxSurfaceMaterial& FbxMaterial) const
{
	FString MaterialFullName = MakeName(FbxMaterial.GetName());

	if (MaterialFullName.Len() > 6)
	{
		int32 Offset = MaterialFullName.Find(TEXT("_SKIN"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
		if (Offset != INDEX_NONE)
		{
			// Chop off the material name so we are left with the number in _SKINXX
			FString SkinXXNumber = MaterialFullName.Right(MaterialFullName.Len() - (Offset + 1)).RightChop(4);

			if (SkinXXNumber.IsNumeric())
			{
				// remove the '_skinXX' suffix from the material name					
				MaterialFullName.LeftChopInline(MaterialFullName.Len() - Offset, EAllowShrinking::No);
			}
		}
	}
	else if (MaterialFullName.IsEmpty())
	{
		MaterialFullName = TEXT("UnnamedMaterial");
	}

	MaterialFullName = ObjectTools::SanitizeObjectName(MaterialFullName);

	return MaterialFullName;
}

FString UnFbx::FFbxImporter::GetMaterialBasePackageName(const FString& MaterialFullName) const
{
	FString BasePackageName = FPackageName::GetLongPackagePath(Parent->GetOutermost()->GetName());
	if (ImportOptions->MaterialBasePath != NAME_None)
	{
		BasePackageName = ImportOptions->MaterialBasePath.ToString();
	}
	else
	{
		BasePackageName += TEXT("/");
	}

	BasePackageName += MaterialFullName;
	BasePackageName = UPackageTools::SanitizePackageName(BasePackageName);

	return BasePackageName;
}


bool UnFbx::FFbxImporter::LinkMaterialProperty(
	const FbxSurfaceMaterial& FbxMaterial,
	UMaterialInstanceConstant* UnrealMaterial,
	const char* MaterialProperty,
	FName ParameterValue,
	bool bSetupAsNormalMap) 
{
	bool bCreated = false;
	FbxProperty FbxProperty = FbxMaterial.FindProperty(MaterialProperty);
	if (FbxProperty.IsValid())
	{
		int32 LayeredTextureCount = FbxProperty.GetSrcObjectCount<FbxLayeredTexture>();
		if (LayeredTextureCount > 0)
		{
			UE_LOGF(LogFbxMaterialImport, Warning, "Layered Textures are not supported (material %ls)", UTF8_TO_TCHAR(FbxMaterial.GetName()));
		}
		else
		{
			int32 TextureCount = FbxProperty.GetSrcObjectCount<FbxTexture>();
			if (TextureCount > 0)
			{
				for (int32 TextureIndex = 0; TextureIndex < TextureCount; ++TextureIndex)
				{
					FbxFileTexture* FbxTexture = FbxProperty.GetSrcObject<FbxFileTexture>(TextureIndex);

					// create an unreal texture asset
					UTexture* UnrealTexture = ImportTexture(FbxTexture, bSetupAsNormalMap);

					if (UnrealTexture)
					{
						UnrealMaterial->SetTextureParameterValueEditorOnly(ParameterValue, UnrealTexture);
						bCreated = true;
					}
				}
			}
		}
	}

	return bCreated;
}

bool CanUseMaterialWithInstance(const FbxSurfaceMaterial& FbxMaterial, const char* MaterialProperty, FString ParameterValueName, UMaterialInterface *BaseMaterial, TArray<FString>& UVSet) {
	FbxProperty FbxProperty = FbxMaterial.FindProperty(MaterialProperty);
	if (FbxProperty.IsValid())
	{
		int32 LayeredTextureCount = FbxProperty.GetSrcObjectCount<FbxLayeredTexture>();
		if (LayeredTextureCount == 0)
		{
			int32 TextureCount = FbxProperty.GetSrcObjectCount<FbxTexture>();
			if (TextureCount == 1)
			{
				// If we didnt specify a parameter to go with this property we can't use this as base instance
				if (ParameterValueName.IsEmpty())
				{
					return false;
				}

				if (FbxFileTexture* FbxTexture = FbxProperty.GetSrcObject<FbxFileTexture>(0))
				{
					float ScaleU = FbxTexture->GetScaleU();
					float ScaleV = FbxTexture->GetScaleV();
					FbxString UVSetName = FbxTexture->UVSet.Get();
					FString LocalUVSetName = UTF8_TO_TCHAR(UVSetName.Buffer());
					int32 SetIndex = UVSet.Find(LocalUVSetName);
					if ((SetIndex != 0 && SetIndex != INDEX_NONE) || ScaleU != 1.0f || ScaleV != 1.0f)
					{
						return false; // no support for custom uv with instanced yet
					}
				}
			}
			else if (TextureCount > 1)
			{
				return false; // no support for multiple textures
			}
		}
		else
		{
			return false; // no support for layered textures
		}
	}

	return true;
}

UMaterialInterface* UnFbx::FFbxImporter::FindExistingMaterialFromFbxMaterial(const FbxSurfaceMaterial& FbxMaterial, EMaterialSearchLocation MaterialSearchLocation)
{
	// Make sure we have a parent
	if (!ensure(Parent.IsValid()))
	{
		return nullptr;
	}

	if (UMaterialInterface** OverrideMaterial = ImportOptions->OverrideMaterials.Find(FbxMaterial.GetUniqueID()))
	{
		if (!ImportedMaterialData.IsAlreadyImported(FbxMaterial, FName(*(*OverrideMaterial)->GetPathName())))
		{
			// Add the material override to the list of imported materials
			ImportedMaterialData.AddImportedMaterial(FbxMaterial, **OverrideMaterial);
		}
		return *OverrideMaterial;
	}

	const FString MaterialFullName = GetMaterialFullName(FbxMaterial);
	const FString BasePackageName = GetMaterialBasePackageName(MaterialFullName);
	const FName ObjectPath = *(BasePackageName + TEXT(".") + MaterialFullName);

	// The material could already exist in the project
	UMaterialInterface* FoundMaterial = nullptr;
	if (ImportedMaterialData.IsAlreadyImported(FbxMaterial, ObjectPath))
	{
		FoundMaterial = ImportedMaterialData.GetUnrealMaterial(FbxMaterial);
	}
	else
	{
		FText Error;
		FoundMaterial = UMaterialImportHelpers::FindExistingMaterialFromSearchLocation(MaterialFullName, BasePackageName, MaterialSearchLocation, Error);

		if (!Error.IsEmpty())
		{
			AddTokenizedErrorMessage(
				FTokenizedMessage::Create(EMessageSeverity::Warning,
					FText::Format(LOCTEXT("FbxMaterialImport_MultipleMaterialsFound", "While importing '{0}': {1}"),
						FText::FromString(Parent->GetOutermost()->GetName()),
						Error)),
				FFbxErrors::Generic_LoadingSceneFailed);
		}
		
		if (FoundMaterial)
		{
			ImportedMaterialData.AddImportedMaterial(FbxMaterial, *FoundMaterial);
		}
	}

	return FoundMaterial;
}


UMaterialInterface* UnFbx::FFbxImporter::CreateUnrealMaterial(const FbxSurfaceMaterial& FbxMaterial, TArray<FString>& OutUVSets, bool bForSkeletalMesh)
{
	// Make sure we have a parent
	if ( !ensure(Parent.IsValid()) )
	{
		return nullptr;
	}

	// Check if we can use the specified base material to instance from it
	FBXImportOptions* FbxImportOptions = GetImportOptions();
	bool bCanInstance = false;
	if (FbxImportOptions->BaseMaterial)
	{
		// try to use the material as a base for the new material to instance from
		FbxProperty FbxDiffuseProperty = FbxMaterial.FindProperty(FbxSurfaceMaterial::sDiffuse);
		if (FbxDiffuseProperty.IsValid())
		{
			bCanInstance = CanUseMaterialWithInstance(FbxMaterial, FbxSurfaceMaterial::sDiffuse, FbxImportOptions->BaseDiffuseTextureName, FbxImportOptions->BaseMaterial, OutUVSets);
		}
		else
		{
			bCanInstance = !FbxImportOptions->BaseColorName.IsEmpty();
		}

		FbxProperty FbxEmissiveProperty = FbxMaterial.FindProperty(FbxSurfaceMaterial::sEmissive);
		if (FbxEmissiveProperty.IsValid())
		{
			bCanInstance &= CanUseMaterialWithInstance(FbxMaterial, FbxSurfaceMaterial::sEmissive, FbxImportOptions->BaseEmmisiveTextureName, FbxImportOptions->BaseMaterial, OutUVSets);
		}
		else
		{
			bCanInstance &= !FbxImportOptions->BaseEmissiveColorName.IsEmpty();
		}

		bCanInstance &= CanUseMaterialWithInstance(FbxMaterial, FbxSurfaceMaterial::sSpecular, FbxImportOptions->BaseSpecularTextureName, FbxImportOptions->BaseMaterial, OutUVSets);
		bCanInstance &= CanUseMaterialWithInstance(FbxMaterial, FbxSurfaceMaterial::sNormalMap, FbxImportOptions->BaseNormalTextureName, FbxImportOptions->BaseMaterial, OutUVSets);
		bCanInstance &= CanUseMaterialWithInstance(FbxMaterial, FbxSurfaceMaterial::sTransparentColor, FbxImportOptions->BaseOpacityTextureName, FbxImportOptions->BaseMaterial, OutUVSets);
	}

	//Make sure we can import the material class
	if ( bCanInstance )
	{
		if (!CanImportClass(UMaterialInstanceConstant::StaticClass()))
		{
			return nullptr;
		}
	}
	else if ( !CanImportClass(UMaterial::StaticClass()) )
	{
		return nullptr;
	}

	const FString BasePackageName = GetMaterialBasePackageName(GetMaterialFullName(FbxMaterial));
	const FString Suffix(TEXT(""));
	FString FinalPackageName;
	FString FinalMaterialName;
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	AssetToolsModule.Get().CreateUniqueAssetName(BasePackageName, Suffix, FinalPackageName, FinalMaterialName);

	UPackage* Package = CreatePackage(*FinalPackageName);
	UMaterialInterface* UnrealMaterialFinal = nullptr;
	if (bCanInstance)
	{
		auto MaterialInstanceFactory = NewObject<UMaterialInstanceConstantFactoryNew>();
		MaterialInstanceFactory->InitialParent = FbxImportOptions->BaseMaterial;
		UMaterialInstanceConstant* UnrealMaterialConstant = (UMaterialInstanceConstant*)MaterialInstanceFactory->FactoryCreateNew(UMaterialInstanceConstant::StaticClass(), Package, *FinalMaterialName, RF_Standalone | RF_Public, NULL, GWarn);
		if (UnrealMaterialConstant != NULL)
		{
			CreatedObjects.Add(UnrealMaterialConstant);
			UnrealMaterialFinal = UnrealMaterialConstant;
			// Notify the asset registry
			FAssetRegistryModule::AssetCreated(UnrealMaterialConstant);

			// Set the dirty flag so this package will get saved later
			Package->SetDirtyFlag(true);

			//UnrealMaterialConstant->SetParentEditorOnly(FbxImportOptions->BaseMaterial);


			// textures and properties
			bool bDiffuseTextureCreated = LinkMaterialProperty(FbxMaterial, UnrealMaterialConstant, FbxSurfaceMaterial::sDiffuse, FName(*FbxImportOptions->BaseDiffuseTextureName), false);
			bool bEmissiveTextureCreated = LinkMaterialProperty(FbxMaterial, UnrealMaterialConstant, FbxSurfaceMaterial::sEmissive, FName(*FbxImportOptions->BaseEmmisiveTextureName), false);
			bool bOpacityTextureCreated = LinkMaterialProperty(FbxMaterial, UnrealMaterialConstant, FbxSurfaceMaterial::sTransparentColor, FName(*FbxImportOptions->BaseOpacityTextureName), false);
			LinkMaterialProperty(FbxMaterial, UnrealMaterialConstant, FbxSurfaceMaterial::sSpecular, FName(*FbxImportOptions->BaseSpecularTextureName), false);
			if (!LinkMaterialProperty(FbxMaterial, UnrealMaterialConstant, FbxSurfaceMaterial::sNormalMap, FName(*FbxImportOptions->BaseNormalTextureName), true))
			{
				LinkMaterialProperty(FbxMaterial, UnrealMaterialConstant, FbxSurfaceMaterial::sBump, FName(*FbxImportOptions->BaseNormalTextureName), true); // no bump in unreal, use as normal map
			}

			// If we only have colors and its different from the base material
			if (!bDiffuseTextureCreated)
			{
				FbxDouble3 DiffuseColor;
				bool OverrideColor = false;

				if (FbxMaterial.GetClassId().Is(FbxSurfacePhong::ClassId))
				{
					DiffuseColor = ((FbxSurfacePhong&)(FbxMaterial)).Diffuse.Get();
					OverrideColor = true;
				}
				else if (FbxMaterial.GetClassId().Is(FbxSurfaceLambert::ClassId))
				{
					DiffuseColor = ((FbxSurfaceLambert&)(FbxMaterial)).Diffuse.Get();
					OverrideColor = true;
				}
				if(OverrideColor)
				{
					FLinearColor LinearColor((float)DiffuseColor[0], (float)DiffuseColor[1], (float)DiffuseColor[2]);
					FLinearColor CurrentLinearColor;
					if (UnrealMaterialConstant->GetVectorParameterValue(FName(*FbxImportOptions->BaseColorName), CurrentLinearColor))
					{
						//Alpha is not consider for diffuse color
						LinearColor.A = CurrentLinearColor.A;
						if (!CurrentLinearColor.Equals(LinearColor))
						{
							UnrealMaterialConstant->SetVectorParameterValueEditorOnly(FName(*FbxImportOptions->BaseColorName), LinearColor);
						}
					}
				}
			}
			if (!bEmissiveTextureCreated)
			{
				FbxDouble3 EmissiveColor;
				bool OverrideColor = false;

				if (FbxMaterial.GetClassId().Is(FbxSurfacePhong::ClassId))
				{
					EmissiveColor = ((FbxSurfacePhong&)(FbxMaterial)).Emissive.Get();
					OverrideColor = true;
				}
				else if (FbxMaterial.GetClassId().Is(FbxSurfaceLambert::ClassId))
				{
					EmissiveColor = ((FbxSurfaceLambert&)(FbxMaterial)).Emissive.Get();
					OverrideColor = true;
				}
				if (OverrideColor)
				{
					FLinearColor LinearColor((float)EmissiveColor[0], (float)EmissiveColor[1], (float)EmissiveColor[2]);
					FLinearColor CurrentLinearColor;
					if (UnrealMaterialConstant->GetVectorParameterValue(FName(*FbxImportOptions->BaseEmissiveColorName), CurrentLinearColor))
					{
						//Alpha is not consider for emissive color
						LinearColor.A = CurrentLinearColor.A;
						if (!CurrentLinearColor.Equals(LinearColor))
						{
							UnrealMaterialConstant->SetVectorParameterValueEditorOnly(FName(*FbxImportOptions->BaseEmissiveColorName), LinearColor);
						}
					}
				}
			}
			if (bOpacityTextureCreated)
			{
				FMaterialInstanceBasePropertyOverrides Overrides;
				Overrides.bOverride_BlendMode = true;
				Overrides.BlendMode = BLEND_Translucent;

				UnrealMaterialConstant->BasePropertyOverrides = Overrides;
			}
		}
	}
	else
	{
		// create an unreal material asset
		auto MaterialFactory = NewObject<UMaterialFactoryNew>();
	
		UMaterial* UnrealMaterial = (UMaterial*)MaterialFactory->FactoryCreateNew(
			UMaterial::StaticClass(), Package, *FinalMaterialName, RF_Standalone|RF_Public, NULL, GWarn );

		if (UnrealMaterial != NULL)
		{
			CreatedObjects.Add(UnrealMaterial);
			UnrealMaterialFinal = UnrealMaterial;
			// Notify the asset registry
			FAssetRegistryModule::AssetCreated(UnrealMaterial);

			if(bForSkeletalMesh)
			{
				UnrealMaterial->SetUsageByFlag(MATUSAGE_SkeletalMesh, true);
			}

			// Set the dirty flag so this package will get saved later
			Package->SetDirtyFlag(true);

			// textures and properties
#if DEBUG_LOG_FBX_MATERIAL_PROPERTIES
			const FbxProperty &FirstProperty = FbxMaterial.GetFirstProperty();
			if (FirstProperty.IsValid())
			{
				UE_LOGF(LogFbxMaterialImport, Display, "Creating Material [%ls]", UTF8_TO_TCHAR(FbxMaterial.GetName()));
				LogPropertyAndChild(FbxMaterial, FirstProperty);
				UE_LOGF(LogFbxMaterialImport, Display, "-------------------------------");
			}
#endif
			UMaterialEditorOnlyData* UnrealMaterialEditorOnly = UnrealMaterial->GetEditorOnlyData();
			CreateAndLinkExpressionForMaterialProperty(FbxMaterial, UnrealMaterial, FbxSurfaceMaterial::sDiffuse, UnrealMaterialEditorOnly->BaseColor, false, OutUVSets, FVector2D(240, -320));
			CreateAndLinkExpressionForMaterialProperty(FbxMaterial, UnrealMaterial, FbxSurfaceMaterial::sEmissive, UnrealMaterialEditorOnly->EmissiveColor, false, OutUVSets, FVector2D(240, -64));
			CreateAndLinkExpressionForMaterialProperty(FbxMaterial, UnrealMaterial, FbxSurfaceMaterial::sSpecular, UnrealMaterialEditorOnly->Specular, false, OutUVSets, FVector2D(240, -128));
			CreateAndLinkExpressionForMaterialProperty(FbxMaterial, UnrealMaterial, FbxSurfaceMaterial::sSpecularFactor, UnrealMaterialEditorOnly->Roughness, false, OutUVSets, FVector2D(240, -180));
			CreateAndLinkExpressionForMaterialProperty(FbxMaterial, UnrealMaterial, FbxSurfaceMaterial::sShininess, UnrealMaterialEditorOnly->Metallic, false, OutUVSets, FVector2D(240, -210));
			if (!CreateAndLinkExpressionForMaterialProperty(FbxMaterial, UnrealMaterial, FbxSurfaceMaterial::sNormalMap, UnrealMaterialEditorOnly->Normal, true, OutUVSets, FVector2D(240, 256)))
			{
				CreateAndLinkExpressionForMaterialProperty(FbxMaterial, UnrealMaterial, FbxSurfaceMaterial::sBump, UnrealMaterialEditorOnly->Normal, true, OutUVSets, FVector2D(240, 256)); // no bump in unreal, use as normal map
			}
			if (CreateAndLinkExpressionForMaterialProperty(FbxMaterial, UnrealMaterial, FbxSurfaceMaterial::sTransparentColor, UnrealMaterialEditorOnly->Opacity, false, OutUVSets, FVector2D(200, 256)))
			{
				UnrealMaterial->BlendMode = BLEND_Translucent;
				CreateAndLinkExpressionForMaterialProperty(FbxMaterial, UnrealMaterial, FbxSurfaceMaterial::sTransparencyFactor, UnrealMaterialEditorOnly->OpacityMask, false, OutUVSets, FVector2D(150, 256));

			}
			FixupMaterial(FbxMaterial, UnrealMaterial); // add random diffuse if none exists
			UMaterialEditingLibrary::LayoutMaterialExpressions(UnrealMaterial);
		}

		// compile shaders for PC (from UPrecompileShadersCommandlet::ProcessMaterial
		// and FMaterialEditor::UpdateOriginalMaterial)
	}

	if (UnrealMaterialFinal)
	{
		// let the material update itself if necessary
		UnrealMaterialFinal->PreEditChange(NULL);
		UnrealMaterialFinal->PostEditChange();
		ImportedMaterialData.AddImportedMaterial(FbxMaterial, *UnrealMaterialFinal);
	}
	
	return UnrealMaterialFinal;
}

void UnFbx::FFbxImporter::FindOrImportMaterialsFromNode(FbxNode* FbxNode, TArray<UMaterialInterface*>& OutMaterials, TArray<FString>& UVSets, bool bForSkeletalMesh)
{
	if (FbxMesh *MeshNode = FbxNode->GetMesh())
	{
		TSet<int32> UsedMaterialIndexes;
		
		for (int32 ElementMaterialIndex = 0; ElementMaterialIndex < MeshNode->GetElementMaterialCount(); ++ElementMaterialIndex)
		{
			FbxGeometryElementMaterial *ElementMaterial = MeshNode->GetElementMaterial(ElementMaterialIndex);
			switch (ElementMaterial->GetMappingMode())
			{
			case FbxLayerElement::eAllSame:
				{
					if (ElementMaterial->GetIndexArray().GetCount() > 0)
					{
						UsedMaterialIndexes.Add(ElementMaterial->GetIndexArray()[0]);
					}
				}
				break;
			case FbxLayerElement::eByPolygon:
				{
					for (int32 MaterialIndex = 0; MaterialIndex < ElementMaterial->GetIndexArray().GetCount(); ++MaterialIndex)
					{
						UsedMaterialIndexes.Add(ElementMaterial->GetIndexArray()[MaterialIndex]);
					}
				}
				break;
			}
		}

		for (int32 MaterialIndex = 0, MaterialCount = FbxNode->GetMaterialCount(); MaterialIndex < MaterialCount; ++MaterialIndex)
		{
			const FbxSurfaceMaterial *FbxMaterial = FbxNode->GetMaterial(MaterialIndex);
			UMaterialInterface* MaterialImported = nullptr;

			//Only create the material used by the mesh element material
			if (FbxMaterial && UsedMaterialIndexes.Contains(MaterialIndex))
			{
				if (UMaterialInterface* ExistingMaterial = FindExistingMaterialFromFbxMaterial(*FbxMaterial, ImportOptions->MaterialSearchLocation))
				{
					MaterialImported = ExistingMaterial;
				}
				else if (ImportOptions->bImportMaterials)
				{
					//Only create a new material if we are importing them and we could not find an existing one.
					MaterialImported = CreateUnrealMaterial(*FbxMaterial, UVSets, bForSkeletalMesh);
				}
			}

			//The fbxMaterial is not valid.
			OutMaterials.Add(MaterialImported);			
		}
	}
	else
	{
		//Could not import the materials, no mesh found.
		OutMaterials.AddDefaulted(FbxNode->GetMaterialCount());
	}
}


#undef LOCTEXT_NAMESPACE
