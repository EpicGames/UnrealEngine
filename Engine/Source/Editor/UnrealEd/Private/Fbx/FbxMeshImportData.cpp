// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/FbxMeshImportData.h"

#include "UObject/Object.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FbxMeshImportData)

UFbxMeshImportData::UFbxMeshImportData(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NormalImportMethod = FBXNIM_ComputeNormals;
	NormalGenerationMethod = EFBXNormalGenerationMethod::MikkTSpace;
	bComputeWeightedNormals = true;
	bBakePivotInVertex = false;
	bReorderMaterialToFbxOrder = true;
}

bool UFbxMeshImportData::CanEditChange(const FProperty* InProperty) const
{
	bool bMutable = Super::CanEditChange(InProperty);
	UObject* Outer = GetOuter();
	if(Outer && bMutable)
	{
		// Let the parent object handle the editability of our properties
		bMutable = Outer->CanEditChange(InProperty);
	}

	static FName NAME_NormalGenerationMethod("NormalGenerationMethod");
	static FName NAME_ComputeWeightedNormals("bComputeWeightedNormals");
	if( bMutable && (InProperty->GetFName() == NAME_NormalGenerationMethod || InProperty->GetFName() == NAME_ComputeWeightedNormals))
	{
		// Normal generation method is ignored if we are importing both tangents and normals
		return NormalImportMethod == FBXNIM_ImportNormalsAndTangents ? false : true;
	}


	return bMutable;
}
