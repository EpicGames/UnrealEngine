// Copyright Epic Games, Inc. All Rights Reserved.
#include "FbxOptionWindow.h"

#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailsViewArgs.h"
#include "Factories/FbxAnimSequenceImportData.h"
#include "Framework/Application/SlateApplication.h"
#include "IDetailsView.h"
#include "IDocumentation.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Math/Interval.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SPrimaryButton.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Types/SlateStructs.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "UObject/ObjectPtr.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FBXOption"

void SFbxOptionWindow::Construct(const FArguments& InArgs)
{
	ImportUI = InArgs._ImportUI;
	WidgetWindow = InArgs._WidgetWindow;
	bIsObjFormat = InArgs._IsObjFormat;

	check (ImportUI);
	
	TSharedPtr<SBox> ImportTypeDisplay;
	TSharedPtr<SHorizontalBox> FbxHeaderButtons;
	TSharedPtr<SBox> InspectorBox;
	this->ChildSlot
	[
		SNew(SBox)
		.MaxDesiredHeight(InArgs._MaxWindowHeight)
		.MaxDesiredWidth(InArgs._MaxWindowWidth)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SAssignNew(ImportTypeDisplay, SBox)
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SNew(SBorder)
				.Padding(FMargin(3))
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("Import_CurrentFileTitle", "Current Asset: "))
					]
					+SHorizontalBox::Slot()
					.Padding(5, 0, 0, 0)
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(InArgs._FullPath)
						.ToolTipText(InArgs._FullPath)
					]
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SAssignNew(InspectorBox, SBox)
				.MaxDesiredHeight(650.0f)
				.WidthOverride(400.0f)
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(2.0f)
				+ SUniformGridPanel::Slot(1, 0)
				[
					SAssignNew(ImportAllButton, SPrimaryButton)
					.Text(LOCTEXT("FbxOptionWindow_ImportAll", "Import All"))
					.ToolTipText(LOCTEXT("FbxOptionWindow_ImportAll_ToolTip", "Import all files with these same settings"))
					.IsEnabled(this, &SFbxOptionWindow::CanImport)
					.OnClicked(this, &SFbxOptionWindow::OnImportAll)
				]
				+ SUniformGridPanel::Slot(2, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("FbxOptionWindow_Import", "Import"))
					.IsEnabled(this, &SFbxOptionWindow::CanImport)
					.OnClicked(this, &SFbxOptionWindow::OnImport)
				]
				+ SUniformGridPanel::Slot(3, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("FbxOptionWindow_Cancel", "Cancel"))
					.ToolTipText(LOCTEXT("FbxOptionWindow_Cancel_ToolTip", "Cancels importing this FBX file"))
					.OnClicked(this, &SFbxOptionWindow::OnCancel)
				]
			]
		]
	];

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	InspectorBox->SetContent(DetailsView->AsShared());

	ImportTypeDisplay->SetContent(
		SNew(SBorder)
		.Padding(FMargin(3))
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SFbxOptionWindow::GetImportTypeDisplayText)
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				IDocumentation::Get()->CreateAnchor(FString("fbx-import-options-reference-in-unreal-engine"))
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			[
				SAssignNew(FbxHeaderButtons, SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(2.0f, 0.0f))
				[
					SNew(SButton)
					.Text(LOCTEXT("FbxOptionWindow_ResetOptions", "Reset to Default"))
					.OnClicked(this, &SFbxOptionWindow::OnResetToDefaultClick)
				]
			]
		]
	);

	DetailsView->SetObject(ImportUI);

	RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SFbxOptionWindow::SetFocusPostConstruct));

}

EActiveTimerReturnType SFbxOptionWindow::SetFocusPostConstruct(double InCurrentTime, float InDeltaTime)
{
	if (ImportAllButton.IsValid())
	{
		FSlateApplication::Get().SetKeyboardFocus(ImportAllButton, EFocusCause::SetDirectly);
	}

	return EActiveTimerReturnType::Stop;

}
FReply SFbxOptionWindow::OnResetToDefaultClick() const
{
	ImportUI->ResetToDefault();
	//Refresh the view to make sure the custom UI are updating correctly
	DetailsView->SetObject(ImportUI, true);
	return FReply::Handled();
}

FText SFbxOptionWindow::GetImportTypeDisplayText() const
{
	switch (ImportUI->MeshTypeToImport)
	{
	case EFBXImportType::FBXIT_Animation :
		return ImportUI->bIsReimport ? LOCTEXT("FbxOptionWindow_ReImportTypeAnim", "Reimport Animation") : LOCTEXT("FbxOptionWindow_ImportTypeAnim", "Import Animation");
	case EFBXImportType::FBXIT_SkeletalMesh:
		return ImportUI->bIsReimport ? LOCTEXT("FbxOptionWindow_ReImportTypeSK", "Reimport Skeletal Mesh") : LOCTEXT("FbxOptionWindow_ImportTypeSK", "Import Skeletal Mesh");
	case EFBXImportType::FBXIT_StaticMesh:
		return ImportUI->bIsReimport ? LOCTEXT("FbxOptionWindow_ReImportTypeSM", "Reimport Static Mesh") : LOCTEXT("FbxOptionWindow_ImportTypeSM", "Import Static Mesh");
	}
	return FText::GetEmpty();
}

bool SFbxOptionWindow::CanImport()  const
{
	// do test to see if we are ready to import
	if (ImportUI->MeshTypeToImport == FBXIT_Animation)
	{
		if (ImportUI->Skeleton == NULL || !ImportUI->bImportAnimations)
		{
			return false;
		}
	}

	if (ImportUI->AnimSequenceImportData->AnimationLength == FBXALIT_SetRange)
	{
		if (ImportUI->AnimSequenceImportData->FrameImportRange.Min > ImportUI->AnimSequenceImportData->FrameImportRange.Max)
		{
			return false;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
