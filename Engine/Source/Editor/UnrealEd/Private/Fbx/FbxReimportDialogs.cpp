// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Static mesh creation from FBX data.
	Largely based on StaticMeshEdit.cpp
=============================================================================*/

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "UObject/Object.h"
#include "UObject/GarbageCollection.h"
#include "UObject/Package.h"
#include "Misc/PackageName.h"
#include "Editor.h"
#include "ObjectTools.h"
#include "PackageTools.h"
#include "FbxImporter.h"
#include "Misc/FbxErrors.h"
#include "HAL/FileManager.h"
#include "Factories/FbxSceneImportFactory.h"

#include "AssetRegistry/AssetRegistryModule.h"

//Windows dialog popup
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IMainFrameModule.h"
#include "FbxCompareWindow.h"
#include "FbxMaterialConflictWindow.h"

//Meshes includes
#include "MeshUtilities.h"
#include "RawMesh.h"
#include "MaterialDomain.h"
#include "Materials/MaterialInterface.h"
#include "Materials/Material.h"
#include "GeomFitUtils.h"
#include "PhysicsAssetUtils.h"
#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/PhysicsAsset.h"

//Static mesh includes
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshSocket.h"
#include "StaticMeshResources.h"
#include "Factories/FbxStaticMeshImportData.h"

//Skeletal mesh includes
#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Components/SkinnedMeshComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "AnimEncoding.h"
#include "Engine/SkeletalMeshSocket.h"
#include "ClothingAsset.h"
#include "Factories/FbxSkeletalMeshImportData.h"
#include "Rendering/SkeletalMeshModel.h"



#define LOCTEXT_NAMESPACE "FbxPreviewReimport"

using namespace UnFbx;

struct FCreateCompFromFbxArg
{
	FString MeshName;
	bool IsStaticMesh;
	bool IsStaticHasLodGroup;
};

void RecursiveFillSkeletonData(ImportCompareHelper::FSkeletonTreeNode& ParentJoint, FCompJoint& ParentComp, int32 ParentIndex, TArray<FCompJoint>& Joints)
{
	for (int32 ChildIndex = 0; ChildIndex < ParentJoint.Childrens.Num(); ++ChildIndex)
	{
		int32 NewjointIndex = Joints.Num();
		ParentComp.ChildIndexes.Add(NewjointIndex);
		FCompJoint& NodeComp = Joints.AddDefaulted_GetRef();
		NodeComp.Name = ParentJoint.Childrens[ChildIndex].JointName;
		NodeComp.ParentIndex = ParentIndex;
		RecursiveFillSkeletonData(ParentJoint.Childrens[ChildIndex], NodeComp, NewjointIndex, Joints);
	}
}

void RecursiveCountSkeletonJoint(ImportCompareHelper::FSkeletonTreeNode& ParentJoint, int32& Count)
{
	for (int32 ChildIndex = 0; ChildIndex < ParentJoint.Childrens.Num(); ++ChildIndex)
	{
		Count++;
		RecursiveCountSkeletonJoint(ParentJoint.Childrens[ChildIndex], Count);
	}
}

void CreateCompFromImportCompareHelper(ImportCompareHelper::FSkeletonTreeNode& ResultAssetRoot, FCompMesh& ResultData)
{
	int32 Count = 1;
	RecursiveCountSkeletonJoint(ResultAssetRoot, Count);
	ResultData.CompSkeleton.Joints.Reserve(Count);

	FCompJoint& RootComp = ResultData.CompSkeleton.Joints.AddDefaulted_GetRef();
	RootComp.Name = ResultAssetRoot.JointName;
	RootComp.ParentIndex = INDEX_NONE;
	RecursiveFillSkeletonData(ResultAssetRoot, RootComp, 0, ResultData.CompSkeleton.Joints);
}

void FFbxImporter::ShowFbxSkeletonConflictWindow(USkeletalMesh* SkeletalMesh, USkeleton* Skeleton, ImportCompareHelper::FSkeletonCompareData& SkeletonCompareData)
{
	if (SkeletalMesh == nullptr)
	{
		return;
	}
	
	if (Skeleton == nullptr)
	{
		Skeleton = SkeletalMesh->GetSkeleton();
	}

	FCompMesh SourceData;
	FCompMesh ResultData;
	
	//Create the current data to compare from
	CreateCompFromImportCompareHelper(SkeletonCompareData.CurrentAssetRoot, SourceData);
	CreateCompFromImportCompareHelper(SkeletonCompareData.ResultAssetRoot, ResultData);

	TArray<TSharedPtr<FString>> AssetReferencingSkeleton;
	
	
	if(Skeleton != nullptr)
	{
		UObject* SelectedObject = Skeleton;
		if (SelectedObject)
		{
			FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
			const FName SelectedPackageName = SelectedObject->GetOutermost()->GetFName();
			//Get the Hard dependencies
			TArray<FName> HardDependencies;
			AssetRegistryModule.Get().GetReferencers(SelectedPackageName, HardDependencies, UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Hard);
			//Get the Soft dependencies
			TArray<FName> SoftDependencies;
			AssetRegistryModule.Get().GetReferencers(SelectedPackageName, SoftDependencies, UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Soft);
			//Compose the All dependencies array
			TArray<FName> AllDependencies = HardDependencies;
			AllDependencies += SoftDependencies;
			if (AllDependencies.Num() > 0)
			{
				for (const FName& AssetDependencyName : AllDependencies)
				{
					const FString PackageString = AssetDependencyName.ToString();
					const FSoftObjectPath FullAssetPath(FTopLevelAssetPath(*PackageString, *FPackageName::GetLongPackageAssetName(PackageString)));
					
					FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FullAssetPath);
					if (AssetData.GetClass() != nullptr)
					{
						TSharedPtr<FString> AssetReferencing = MakeShareable(new FString(AssetData.AssetClassPath.ToString() + TEXT(" ") + FullAssetPath.ToString()));
						AssetReferencingSkeleton.Add(AssetReferencing);
					}
				}
			}
		}
	}

	//Create the modal dialog window to let the user see the result of the compare
	TSharedPtr<SWindow> ParentWindow;

	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		ParentWindow = MainFrame.GetParentWindow();
	}

	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(NSLOCTEXT("UnrealEd", "FbxCompareWindowTitle", "Reimport Reports"))
		.AutoCenter(EAutoCenter::PreferredWorkArea)
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(600, 650))
		.MinWidth(600)
		.MinHeight(650);

	TSharedPtr<SFbxSkeltonConflictWindow> FbxCompareWindow;
	Window->SetContent
		(
			SAssignNew(FbxCompareWindow, SFbxSkeltonConflictWindow)
			.WidgetWindow(Window)
			.AssetReferencingSkeleton(&AssetReferencingSkeleton)
			.SourceData(&SourceData)
			.ResultData(&ResultData)
			.SourceObject(SkeletalMesh)
			.bIsPreviewConflict(true)
			);

	if (FbxCompareWindow->HasConflict())
	{
		// @todo: we can make this slow as showing progress bar later
		FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);
	}
}

template<typename TMaterialType>
void ResetMaterialSlot(const TArray<TMaterialType>& CurrentMaterial, TArray<TMaterialType>& ResultMaterial)
{
	// If "Reset Material Slot" is enable we want to change the material array to reflect the incoming FBX
	// But we want to try to keep material instance from the existing data, we will match the one that fit
	// but simply put the same index material instance on the one that do not match. Because we will fill
	// the material slot name, artist will be able to remap the material instance correctly
	for (int32 MaterialIndex = 0; MaterialIndex < ResultMaterial.Num(); ++MaterialIndex)
	{
		if (ResultMaterial[MaterialIndex].MaterialInterface == nullptr || ResultMaterial[MaterialIndex].MaterialInterface == UMaterial::GetDefaultMaterial(MD_Surface))
		{
			bool bFoundMatch = false;
			for (int32 ExistMaterialIndex = 0; ExistMaterialIndex < CurrentMaterial.Num(); ++ExistMaterialIndex)
			{
				if (CurrentMaterial[ExistMaterialIndex].ImportedMaterialSlotName == ResultMaterial[MaterialIndex].ImportedMaterialSlotName)
				{
					bFoundMatch = true;
					ResultMaterial[MaterialIndex].MaterialInterface = CurrentMaterial[ExistMaterialIndex].MaterialInterface;
				}
			}

			if (!bFoundMatch && CurrentMaterial.IsValidIndex(MaterialIndex))
			{
				ResultMaterial[MaterialIndex].MaterialInterface = CurrentMaterial[MaterialIndex].MaterialInterface;
			}
		}
	}
}

template<typename TMaterialType>
void FFbxImporter::PrepareAndShowMaterialConflictDialog(const TArray<TMaterialType>& CurrentMaterial, TArray<TMaterialType>& ResultMaterial, TArray<int32>& RemapMaterial, TArray<FName>& RemapMaterialName, bool bCanShowDialog, bool bIsPreviewDialog, bool bForceResetOnConflict, EFBXReimportDialogReturnOption& OutReturnOption)
{
	OutReturnOption = EFBXReimportDialogReturnOption::FBXRDRO_Ok;
	bool bHasSomeUnmatchedMaterial = false;
	for (int32 MaterialIndex = 0; MaterialIndex < ResultMaterial.Num(); ++MaterialIndex)
	{
		RemapMaterial[MaterialIndex] = MaterialIndex;
		RemapMaterialName[MaterialIndex] = ResultMaterial[MaterialIndex].ImportedMaterialSlotName;
		bool bFoundMatch = false;
		for (int32 ExistMaterialIndex = 0; ExistMaterialIndex < CurrentMaterial.Num(); ++ExistMaterialIndex)
		{
			if (CurrentMaterial[ExistMaterialIndex].ImportedMaterialSlotName == ResultMaterial[MaterialIndex].ImportedMaterialSlotName)
			{
				bFoundMatch = true;
				RemapMaterial[MaterialIndex] = ExistMaterialIndex;
				RemapMaterialName[MaterialIndex] = CurrentMaterial[ExistMaterialIndex].ImportedMaterialSlotName;
			}
		}
		if (!bFoundMatch)
		{
			RemapMaterial[MaterialIndex] = INDEX_NONE;
			RemapMaterialName[MaterialIndex] = NAME_None;
			bHasSomeUnmatchedMaterial = true;
		}
	}

	if (bHasSomeUnmatchedMaterial)
	{
		TArray<bool> AutoRemapMaterials;
		AutoRemapMaterials.AddZeroed(RemapMaterial.Num());
		//Do a weighted remap of the material names
		for (int32 ExistMaterialIndex = 0; ExistMaterialIndex < CurrentMaterial.Num(); ++ExistMaterialIndex)
		{
			if (RemapMaterial.Contains(ExistMaterialIndex))
			{
				//Already remapped
				continue;
			}
			//Lets have a minimum similarity to declare a match (under 15% it is not consider a match string)
			float BestWeight = 0.25f;
			int32 BestMaterialIndex = INDEX_NONE;
			for (int32 MaterialIndex = 0; MaterialIndex < ResultMaterial.Num(); ++MaterialIndex)
			{
				if (RemapMaterial[MaterialIndex] != INDEX_NONE)
				{
					continue;
				}
				float StringWeight = UnFbx::FFbxHelper::NameCompareWeight(CurrentMaterial[ExistMaterialIndex].ImportedMaterialSlotName.ToString(), ResultMaterial[MaterialIndex].ImportedMaterialSlotName.ToString());
				if (StringWeight > BestWeight)
				{
					BestWeight = StringWeight;
					BestMaterialIndex = MaterialIndex;
				}
			}
			if (RemapMaterial.IsValidIndex(BestMaterialIndex))
			{
				RemapMaterial[BestMaterialIndex] = ExistMaterialIndex;
				AutoRemapMaterials[BestMaterialIndex] = true;
			}
		}
		
		if (bForceResetOnConflict)
		{
			OutReturnOption = EFBXReimportDialogReturnOption::FBXRDRO_ResetToFbx;
		}
		else if (bCanShowDialog)
		{
			ShowFbxMaterialConflictWindow<TMaterialType>(CurrentMaterial, ResultMaterial, RemapMaterial, AutoRemapMaterials, OutReturnOption, bIsPreviewDialog);
		}

		if (OutReturnOption == EFBXReimportDialogReturnOption::FBXRDRO_ResetToFbx)
		{
			//Make identity remap because we reset to ResultMaterial
			for (int32 MaterialIndex = 0; MaterialIndex < ResultMaterial.Num(); ++MaterialIndex)
			{
				RemapMaterial[MaterialIndex] = MaterialIndex;
				RemapMaterialName[MaterialIndex] = ResultMaterial[MaterialIndex].ImportedMaterialSlotName;
			}
			ResetMaterialSlot(CurrentMaterial, ResultMaterial);
		}
		
	}
}

template<typename TMaterialType>
void FFbxImporter::ShowFbxMaterialConflictWindow(const TArray<TMaterialType>& InSourceMaterials, const TArray<TMaterialType>& InResultMaterials, TArray<int32>& RemapMaterials, TArray<bool>& AutoRemapMaterials, EFBXReimportDialogReturnOption& OutReturnOption, bool bIsPreviewConflict)
{
	TArray<FCompMaterial> SourceMaterials;
	TArray<FCompMaterial> ResultMaterials;

	SourceMaterials.Reserve(InSourceMaterials.Num());
	for (int32 MaterialIndex = 0; MaterialIndex < InSourceMaterials.Num(); ++MaterialIndex)
	{
		SourceMaterials.Add(FCompMaterial(InSourceMaterials[MaterialIndex].MaterialSlotName, InSourceMaterials[MaterialIndex].ImportedMaterialSlotName));
	}

	ResultMaterials.Reserve(InResultMaterials.Num());
	for (int32 MaterialIndex = 0; MaterialIndex < InResultMaterials.Num(); ++MaterialIndex)
	{
		ResultMaterials.Add(FCompMaterial(InResultMaterials[MaterialIndex].MaterialSlotName, InResultMaterials[MaterialIndex].ImportedMaterialSlotName));
	}

	//Create the modal dialog window to let the user see the result of the compare
	TSharedPtr<SWindow> ParentWindow;

	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		ParentWindow = MainFrame.GetParentWindow();
	}
	FText WindowTitle = bIsPreviewConflict ? NSLOCTEXT("UnrealEd", "FbxMaterialConflictOpionsTitlePreview", "Reimport Material Conflicts Preview") : NSLOCTEXT("UnrealEd", "FbxMaterialConflictOpionsTitle", "Reimport Material Conflicts Resolution");
	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(WindowTitle)
		.AutoCenter(EAutoCenter::PreferredWorkArea)
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(700, 350))
		.HasCloseButton(false)
		.MinWidth(700)
		.MinHeight(350);

	TSharedPtr<SFbxMaterialConflictWindow> FbxMaterialConflictWindow;
	Window->SetContent
	(
		SAssignNew(FbxMaterialConflictWindow, SFbxMaterialConflictWindow)
		.WidgetWindow(Window)
		.SourceMaterials(&SourceMaterials)
		.ResultMaterials(&ResultMaterials)
		.RemapMaterials(&RemapMaterials)
		.AutoRemapMaterials(&AutoRemapMaterials)
		.bIsPreviewConflict(bIsPreviewConflict)
	);

	// @todo: we can make this slow as showing progress bar later
	FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);

	OutReturnOption = FbxMaterialConflictWindow->GetReturnOption();
}

//Instantiate the template for the two possible material type
template void FFbxImporter::ShowFbxMaterialConflictWindow<FStaticMaterial>(const TArray<FStaticMaterial>& InSourceMaterials, const TArray<FStaticMaterial>& InResultMaterials, TArray<int32>& RemapMaterials, TArray<bool>& AutoRemapMaterials, EFBXReimportDialogReturnOption& OutReturnOption, bool bIsPreviewConflict);
template void FFbxImporter::ShowFbxMaterialConflictWindow<FSkeletalMaterial>(const TArray<FSkeletalMaterial>& InSourceMaterials, const TArray<FSkeletalMaterial>& InResultMaterials, TArray<int32>& RemapMaterials, TArray<bool>& AutoRemapMaterials, EFBXReimportDialogReturnOption& OutReturnOption, bool bIsPreviewConflict);

template void FFbxImporter::PrepareAndShowMaterialConflictDialog<FStaticMaterial>(const TArray<FStaticMaterial>& CurrentMaterial, TArray<FStaticMaterial>& ResultMaterial, TArray<int32>& RemapMaterial, TArray<FName>& RemapMaterialName, bool bCanShowDialog, bool bIsPreviewDialog, bool bForceResetOnConflict, EFBXReimportDialogReturnOption& OutReturnOption);
template void FFbxImporter::PrepareAndShowMaterialConflictDialog<FSkeletalMaterial>(const TArray<FSkeletalMaterial>& CurrentMaterial, TArray<FSkeletalMaterial>& ResultMaterial, TArray<int32>& RemapMaterial, TArray<FName>& RemapMaterialName, bool bCanShowDialog, bool bIsPreviewDialog, bool bForceResetOnConflict, EFBXReimportDialogReturnOption& OutReturnOption);

#undef LOCTEXT_NAMESPACE
