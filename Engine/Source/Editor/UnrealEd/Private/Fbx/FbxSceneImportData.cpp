// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/FbxSceneImportData.h"

#include "EditorFramework/AssetImportData.h"
#include "Factories/FbxSceneImportOptions.h"
#include "FbxImporter.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FbxSceneImportData)

UFbxSceneImportData::UFbxSceneImportData(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	SourceFbxFile.Empty();
	BluePrintFullName.Empty();
	SceneInfoSourceData = nullptr;
	bCreateFolderHierarchy = false;
	bForceFrontXAxis = false;
	bImportScene = true;
	HierarchyType = (int32)EFBXSceneOptionsCreateHierarchyType::FBXSOCHT_CreateLevelActors;
#endif
}

#if WITH_EDITORONLY_DATA
UnFbx::FBXImportOptions *JSONToFbxOption(TSharedPtr<FJsonValue> OptionJsonValue, FString &OptionName)
{
	const TSharedPtr<FJsonObject>& OptionObj = OptionJsonValue->AsObject();
	if (!OptionObj.IsValid())
	{
		return nullptr;
	}	

	if (!OptionObj->TryGetStringField(TEXT("OptionName"), OptionName))
	{
		return nullptr;
	}

	UnFbx::FBXImportOptions *Option = new UnFbx::FBXImportOptions();

	OptionObj->TryGetBoolField(TEXT("bImportScene"), Option->bImportScene);
	OptionObj->TryGetBoolField(TEXT("bImportMaterials"), Option->bImportMaterials);
	OptionObj->TryGetBoolField(TEXT("bInvertNormalMap"), Option->bInvertNormalMap);
	OptionObj->TryGetBoolField(TEXT("bImportTextures"), Option->bImportTextures);
	OptionObj->TryGetBoolField(TEXT("bImportLOD"), Option->bImportLOD);
	OptionObj->TryGetBoolField(TEXT("bUsedAsFullName"), Option->bUsedAsFullName);
	OptionObj->TryGetBoolField(TEXT("bConvertScene"), Option->bConvertScene);
	OptionObj->TryGetBoolField(TEXT("bForceFrontXAxis"), Option->bForceFrontXAxis);
	OptionObj->TryGetBoolField(TEXT("bConvertSceneUnit"), Option->bConvertSceneUnit);
	OptionObj->TryGetBoolField(TEXT("bRemoveNameSpace"), Option->bRemoveNameSpace);
	double X, Y, Z, Pitch, Yaw, Roll;
	const TSharedPtr<FJsonObject> *DataObj = nullptr;
	if (OptionObj->TryGetObjectField(TEXT("ImportTranslation"), DataObj))
	{
		(*DataObj)->TryGetNumberField(TEXT("X"), X);
		(*DataObj)->TryGetNumberField(TEXT("Y"), Y);
		(*DataObj)->TryGetNumberField(TEXT("Z"), Z);
		Option->ImportTranslation.Set((float)X, (float)Y, (float)Z);
	}
	if (OptionObj->TryGetObjectField(TEXT("ImportRotation"), DataObj))
	{
		(*DataObj)->TryGetNumberField(TEXT("P"), Pitch);
		(*DataObj)->TryGetNumberField(TEXT("Y"), Yaw);
		(*DataObj)->TryGetNumberField(TEXT("R"), Roll);
		Option->ImportRotation.Pitch = (float)Pitch;
		Option->ImportRotation.Yaw = (float)Yaw;
		Option->ImportRotation.Roll = (float)Roll;
	}
	double ImportUniformScale;
	if (OptionObj->TryGetNumberField(TEXT("ImportUniformScale"), ImportUniformScale))
	{
		Option->ImportUniformScale = (float)ImportUniformScale;
	}
	int32 NormalImportMethod;
	if (OptionObj->TryGetNumberField(TEXT("NormalImportMethod"), NormalImportMethod))
	{
		Option->NormalImportMethod = (EFBXNormalImportMethod)NormalImportMethod;
	}
	int32 NormalGenerationMethod;
	if (OptionObj->TryGetNumberField(TEXT("NormalGenerationMethod"), NormalGenerationMethod))
	{
		Option->NormalGenerationMethod = (EFBXNormalGenerationMethod::Type)NormalGenerationMethod;
	}
	OptionObj->TryGetBoolField(TEXT("bTransformVertexToAbsolute"), Option->bTransformVertexToAbsolute);
	OptionObj->TryGetBoolField(TEXT("bBakePivotInVertex"), Option->bBakePivotInVertex);
	OptionObj->TryGetBoolField(TEXT("bCombineToSingle"), Option->bCombineToSingle);
	int32 VertexColorImportOption;
	if (OptionObj->TryGetNumberField(TEXT("VertexColorImportOption"), VertexColorImportOption))
	{
		Option->VertexColorImportOption = (EVertexColorImportOption::Type)VertexColorImportOption;
	}
	if (OptionObj->TryGetObjectField(TEXT("VertexOverrideColor"), DataObj))
	{
		double R, G, B, A;
		(*DataObj)->TryGetNumberField(TEXT("R"), R);
		(*DataObj)->TryGetNumberField(TEXT("G"), G);
		(*DataObj)->TryGetNumberField(TEXT("B"), B);
		(*DataObj)->TryGetNumberField(TEXT("A"), A);
		Option->VertexOverrideColor.R = static_cast<uint8>(R);
		Option->VertexOverrideColor.G = static_cast<uint8>(G);
		Option->VertexOverrideColor.B = static_cast<uint8>(B);
		Option->VertexOverrideColor.A = static_cast<uint8>(A);
	}
	OptionObj->TryGetBoolField(TEXT("bRemoveDegenerates"), Option->bRemoveDegenerates);
	OptionObj->TryGetBoolField(TEXT("bBuildReversedIndexBuffer"), Option->bBuildReversedIndexBuffer);
	OptionObj->TryGetBoolField(TEXT("bGenerateLightmapUVs"), Option->bGenerateLightmapUVs);
	OptionObj->TryGetBoolField(TEXT("bOneConvexHullPerUCX"), Option->bOneConvexHullPerUCX);
	OptionObj->TryGetBoolField(TEXT("bAutoGenerateCollision"), Option->bAutoGenerateCollision);
	FString LODGroup;
	if (OptionObj->TryGetStringField(TEXT("StaticMeshLODGroup"), LODGroup))
	{
		Option->StaticMeshLODGroup = FName(*LODGroup);
	}
	OptionObj->TryGetBoolField(TEXT("bImportStaticMeshLODs"), Option->bImportStaticMeshLODs);

	//Skeletal mesh options
	OptionObj->TryGetBoolField(TEXT("bUpdateSkeletonReferencePose"), Option->bUpdateSkeletonReferencePose);
	//TODO support T0AsRefPose
	//OptionObj->TryGetBoolField(TEXT("bUseT0AsRefPose"), Option->bUseT0AsRefPose);
	Option->bUseT0AsRefPose = false;
	OptionObj->TryGetBoolField(TEXT("bPreserveSmoothingGroups"), Option->bPreserveSmoothingGroups);
	OptionObj->TryGetBoolField(TEXT("bKeepSectionsSeparate"), Option->bKeepSectionsSeparate);
	OptionObj->TryGetBoolField(TEXT("bImportMeshesInBoneHierarchy"), Option->bImportMeshesInBoneHierarchy);
	OptionObj->TryGetBoolField(TEXT("bImportMorphTargets"), Option->bImportMorph);
	OptionObj->TryGetBoolField(TEXT("bImportVertexAttributes"), Option->bImportVertexAttributes);
	
	if (OptionObj->TryGetObjectField(TEXT("OverlappingThresholds"), DataObj))
	{
		double Pos, NTB, UV, MorphPos;
		if ((*DataObj)->TryGetNumberField(TEXT("ThresholdPosition"), Pos))
		{
			Option->OverlappingThresholds.ThresholdPosition = (float)Pos;
		}
		if ((*DataObj)->TryGetNumberField(TEXT("ThresholdTangentNormal"), NTB))
		{
			Option->OverlappingThresholds.ThresholdTangentNormal = (float)NTB;
		}
		if ((*DataObj)->TryGetNumberField(TEXT("ThresholdUV"), UV))
		{
			Option->OverlappingThresholds.ThresholdUV = (float)UV;
		}
		if ((*DataObj)->TryGetNumberField(TEXT("MorphThresholdPosition"), MorphPos))
		{
			Option->OverlappingThresholds.MorphThresholdPosition = (float)MorphPos;
		}
	}

	FString MaterialBasePath;
	if (OptionObj->TryGetStringField(TEXT("MaterialBasePath"), MaterialBasePath))
	{
		Option->MaterialBasePath = FName(*MaterialBasePath);
	}
	return Option;
}

FString FbxOptionToJSON(FString OptionName, UnFbx::FBXImportOptions *Option)
{
	check(Option != nullptr);
	check(!OptionName.IsEmpty());

	FString JsonString = FString::Printf(TEXT("{ \"OptionName\" : \"%s\", \"bImportScene\" : \"%d\", \"bImportMaterials\" : \"%d\", \"bInvertNormalMap\" : \"%d\", \"bImportTextures\" : \"%d\", \"bImportLOD\" : \"%d\", \"bUsedAsFullName\" : \"%d\", \"bConvertScene\" : \"%d\", \"bForceFrontXAxis\" : \"%d\", \"bConvertSceneUnit\" : \"%d\", \"bRemoveNameSpace\" : \"%d\", "),
		*OptionName,
		Option->bImportScene ? 1 : 0,
		Option->bImportMaterials ? 1 : 0,
		Option->bInvertNormalMap ? 1 : 0,
		Option->bImportTextures ? 1 : 0,
		Option->bImportLOD ? 1 : 0,
		Option->bUsedAsFullName ? 1 : 0,
		Option->bConvertScene ? 1 : 0,
		Option->bForceFrontXAxis ? 1 : 0,
		Option->bConvertSceneUnit ? 1 : 0,
		Option->bRemoveNameSpace ? 1 : 0
		);

	JsonString += FString::Printf(TEXT("\"ImportTranslation\" : {\"X\" : \"%f\", \"Y\" : \"%f\", \"Z\" : \"%f\"}, \"ImportRotation\" : {\"P\" : \"%f\", \"Y\" : \"%f\", \"R\" : \"%f\"}, \"ImportUniformScale\" : \"%f\", "),
		Option->ImportTranslation.X,
		Option->ImportTranslation.Y,
		Option->ImportTranslation.Z,
		Option->ImportRotation.Pitch,
		Option->ImportRotation.Yaw,
		Option->ImportRotation.Roll,
		Option->ImportUniformScale
		);

	JsonString += FString::Printf(TEXT("\"NormalImportMethod\" : \"%d\", \"NormalGenerationMethod\" : \"%d\", \"bTransformVertexToAbsolute\" : \"%d\", \"bBakePivotInVertex\" : \"%d\", \"bCombineToSingle\" : \"%d\", \"VertexColorImportOption\" : \"%d\", \"VertexOverrideColor\" : {\"R\" : \"%d\", \"G\" : \"%d\", \"B\" : \"%d\", \"A\" : \"%d\" }, "),
		(int32)Option->NormalImportMethod,
		(int32)Option->NormalGenerationMethod,
		Option->bTransformVertexToAbsolute ? 1 : 0,
		Option->bBakePivotInVertex ? 1 : 0,
		Option->bCombineToSingle ? 1 : 0,
		(int32)Option->VertexColorImportOption,
		Option->VertexOverrideColor.R,
		Option->VertexOverrideColor.G,
		Option->VertexOverrideColor.B,
		Option->VertexOverrideColor.A
		);

	JsonString += FString::Printf(TEXT("\"bRemoveDegenerates\" : \"%d\", \"bBuildReversedIndexBuffer\" : \"%d\", \"bGenerateLightmapUVs\" : \"%d\", \"bOneConvexHullPerUCX\" : \"%d\", \"bAutoGenerateCollision\" : \"%d\", \"StaticMeshLODGroup\" : \"%s\", \"bImportStaticMeshLODs\" : \"%d\", "),
		Option->bRemoveDegenerates ? 1 : 0,
		Option->bBuildReversedIndexBuffer ? 1 : 0,
		Option->bGenerateLightmapUVs ? 1 : 0,
		Option->bOneConvexHullPerUCX ? 1 : 0,
		Option->bAutoGenerateCollision ? 1 : 0,
		*(Option->StaticMeshLODGroup.ToString()),
		Option->bImportStaticMeshLODs ? 1 : 0
		);

	JsonString += FString::Printf(TEXT("\"bUpdateSkeletonReferencePose\" : \"%d\", \"bUseT0AsRefPose\" : \"%d\", \"bPreserveSmoothingGroups\" : \"%d\", \"bKeepSectionsSeparate\" : \"%d\", \"bImportMeshesInBoneHierarchy\" : \"%d\", \"bImportMorphTargets\" : \"%d\", \"bImportVertexAttributes\" : \"%d\", \"OverlappingThresholds\" : {\"ThresholdPosition\" : \"%f\", \"ThresholdTangentNormal\" : \"%f\", \"ThresholdUV\" : \"%f\", \"MorphThresholdPosition\" : \"%f\"},"),
		Option->bUpdateSkeletonReferencePose ? 1 : 0,
		Option->bUseT0AsRefPose ? 1 : 0,
		Option->bPreserveSmoothingGroups ? 1 : 0,
		Option->bKeepSectionsSeparate ? 1 : 0,
		Option->bImportMeshesInBoneHierarchy ? 1 : 0,
		Option->bImportMorph ? 1 : 0,
		Option->bImportVertexAttributes ? 1 : 0,
		Option->OverlappingThresholds.ThresholdPosition,
		Option->OverlappingThresholds.ThresholdTangentNormal,
		Option->OverlappingThresholds.ThresholdUV,
		Option->OverlappingThresholds.MorphThresholdPosition
		);

	JsonString += FString::Printf(TEXT("\"MaterialBasePath\" : \"%s\"}"), *(Option->MaterialBasePath.ToString()));
	return JsonString;
}

FString FbxNodeInfoToJSON(const TSharedPtr<FFbxNodeInfo> NodeInfo)
{
	check(NodeInfo.IsValid());

	FString JsonString = FString::Printf(TEXT("{ \"NodeName\" : \"%s\", \"UniqueId\" : \"%llu\", \"NodeHierarchyPath\" : \"%s\", \"bImportNode\" : \"%d\", \"ParentUniqueId\" : \"%llu\", \"AttributeType\" : \"%s\", \"AttributeUniqueId\" : \"%llu\", \"Materials\" : ["),
		*(NodeInfo->NodeName),
		NodeInfo->UniqueId,
		*(NodeInfo->NodeHierarchyPath),
		NodeInfo->bImportNode ? 1 : 0,
		NodeInfo->ParentNodeInfo.IsValid() ? NodeInfo->ParentNodeInfo->UniqueId : 0,
		*(NodeInfo->AttributeType),
		NodeInfo->AttributeInfo.IsValid() ? NodeInfo->AttributeInfo->UniqueId : 0
		);

	int32 MaterialCount = NodeInfo->Materials.Num();
	for (TSharedPtr<FFbxMaterialInfo> Material : NodeInfo->Materials)
	{
		JsonString += FString::Printf(TEXT("{ \"Name\" : \"%s\", \"HierarchyPath\" : \"%s\", \"UniqueId\" : \"%llu\", \"bImportAttribute\" : \"%d\", \"OriginalImportPath\" : \"%s\", \"OriginalFullImportName\" : \"%s\", \"bOverridePath\" : \"%d\", \"OverrideImportPath\" : \"%s\", \"OverrideFullImportName\" : \"%s\" }"),
			*(Material->Name),
			*(Material->HierarchyPath),
			Material->UniqueId,
			Material->bImportAttribute ? 1 : 0,
			*(Material->OriginalImportPath),
			*(Material->OriginalFullImportName),
			Material->bOverridePath ? 1 : 0,
			*(Material->OverrideImportPath),
			*(Material->OverrideFullImportName)
			);
		MaterialCount--;
		if (MaterialCount > 0)
		{
			JsonString += FString::Printf(TEXT(", "));
		}
	}
	JsonString += FString::Printf(TEXT("] }"));
	return JsonString;
}

FString FbxMeshInfoToJSON(const TSharedPtr<FFbxMeshInfo> MeshInfo)
{
	check(MeshInfo.IsValid());

	FString JsonString = FString::Printf(TEXT("{ \"Name\" : \"%s\", \"UniqueId\" : \"%llu\", \"bImportAttribute\" : \"%d\", \"OptionName\" : \"%s\", \"bIsSkelMesh\" : \"%d\", \"OriginalImportPath\" : \"%s\", \"OriginalFullImportName\" : \"%s\", \"bOverridePath\" : \"%d\", \"OverrideImportPath\" : \"%s\", \"OverrideFullImportName\" : \"%s\", \"PivotNodeUid\" : \"%llu\", \"LODGroup\" : \"%s\", \"LODLevel\" : \"%d\", \"IsLod\" : \"%d\" }"),
		*(MeshInfo->Name),
		MeshInfo->UniqueId,
		MeshInfo->bImportAttribute ? 1 : 0,
		*(MeshInfo->OptionName),
		MeshInfo->bIsSkelMesh ? 1 : 0,
		*(MeshInfo->OriginalImportPath),
		*(MeshInfo->OriginalFullImportName),
		MeshInfo->bOverridePath ? 1 : 0,
		*(MeshInfo->OverrideImportPath),
		*(MeshInfo->OverrideFullImportName),
		MeshInfo->PivotNodeUid,
		*(MeshInfo->LODGroup),
		MeshInfo->LODLevel,
		MeshInfo->IsLod ? 1 : 0
		);
	return JsonString;
}

FString UFbxSceneImportData::ToJson() const
{
	if (!SceneInfoSourceData.IsValid())
	{
		return FString();
	}
	FString Json;
	Json.Reserve(1024);
	//data array start
	Json += TEXT("[ { ");
	Json += FString::Printf(TEXT("\"bImportScene\" : \"%d\", \"bCreateFolderHierarchy\" : \"%d\", \"HierarchyType\" : \"%d\", \"BluePrintFullName\" : \"%s\", \"bForceFrontXAxis\" : \"%d\" "),
		bImportScene ? 1 : 0,
		bCreateFolderHierarchy ? 1 : 0,
		HierarchyType,
		*BluePrintFullName,
		bForceFrontXAxis ? 1 : 0
		);
		
	if (NameOptionsMap.Num() > 0)
	{
		Json += TEXT(", \"FbxOptions\" : [");
		bool bFirstOption = true;
		for (auto kvp : NameOptionsMap)
		{
			if (!bFirstOption)
			{
				Json += TEXT(", ");
			}
			bFirstOption = false;
			Json += FbxOptionToJSON(kvp.Key, kvp.Value);
		}
		Json += TEXT(" ]");
	}
	//the meshes
	if (SceneInfoSourceData->MeshInfo.Num() > 0)
	{
		Json += TEXT(", \"MeshInfo\" : [");
		bool bFirstNode = true;
		for (const TSharedPtr<FFbxMeshInfo>& MeshInfo : SceneInfoSourceData->MeshInfo)
		{
			if (!MeshInfo.IsValid())
			{
				continue;
			}
			if (!bFirstNode)
			{
				Json += TEXT(", ");
			}
			bFirstNode = false;
			Json += FbxMeshInfoToJSON(MeshInfo);
		}
		//Close the MeshInfo array
		Json += TEXT(" ]");
	}

	//the hierarchy
	if (SceneInfoSourceData->HierarchyInfo.Num() > 0)
	{
		Json += TEXT(", \"Hierarchy\" : [");
		bool bFirstNode = true;
		for (TSharedPtr<FFbxNodeInfo> NodeInfo : SceneInfoSourceData->HierarchyInfo)
		{
			if (!NodeInfo.IsValid())
			{
				continue;
			}
			if (!bFirstNode)
			{
				Json += TEXT(", ");
			}
			bFirstNode = false;
			Json += FbxNodeInfoToJSON(NodeInfo);
		}
		//Close the hierarchy array
		Json += TEXT(" ]");
	}

	//Close the data array
	Json += TEXT(" } ]");
	return Json;
}

void UFbxSceneImportData::FromJson(FString InJsonString)
{
	//Allocate the SceneInfo result
	SceneInfoSourceData = MakeShareable(new FFbxSceneInfo());

	// Load json
	TSharedRef<TJsonReader<TCHAR>> Reader = FJsonStringReader::Create(MoveTemp(InJsonString));

	TArray<TSharedPtr<FJsonValue>> JSONSceneInfos;
	if (!FJsonSerializer::Deserialize(Reader, JSONSceneInfos))
	{
		//Cannot read the json file so return an empty array
		return;
	}

	//We should always have only one SceneInfo
	check(JSONSceneInfos.Num() == 1);

	for (const TSharedPtr<FJsonValue>& Value : JSONSceneInfos)
	{
		const TSharedPtr<FJsonObject>& SceneInfoObj = Value->AsObject();
		if (!SceneInfoObj.IsValid())
		{
			continue;
		}
		//Read General data
		SceneInfoObj->TryGetBoolField(TEXT("bImportScene"), bImportScene);
		SceneInfoObj->TryGetBoolField(TEXT("bCreateFolderHierarchy"), bCreateFolderHierarchy);
		SceneInfoObj->TryGetNumberField(TEXT("HierarchyType"), HierarchyType);
		SceneInfoObj->TryGetStringField(TEXT("BluePrintFullName"), BluePrintFullName);
		SceneInfoObj->TryGetBoolField(TEXT("bForceFrontXAxis"), bForceFrontXAxis);

		//Read Options
		const TArray<TSharedPtr<FJsonValue>>* JSONOptions;
		if (SceneInfoObj->TryGetArrayField(TEXT("FbxOptions"), JSONOptions))
		{
			for (const auto& OptionJsonValue : *JSONOptions)
			{
				FString OptionName;
				UnFbx::FBXImportOptions *Option = JSONToFbxOption(OptionJsonValue, OptionName);
				if (Option != nullptr)
				{
					NameOptionsMap.Add(OptionName, Option);
				}
			}
		}

		//Read Meshes
		const TArray<TSharedPtr<FJsonValue>>* JSONMeshes;
		if (SceneInfoObj->TryGetArrayField(TEXT("MeshInfo"), JSONMeshes))
		{
			for (const auto& MeshJsonValue : *JSONMeshes)
			{
				const TSharedPtr<FJsonObject>& MeshInfoObj = MeshJsonValue->AsObject();
				if (!MeshInfoObj.IsValid())
				{
					continue;
				}
				TSharedPtr<FFbxMeshInfo> MeshInfo = MakeShareable(new FFbxMeshInfo());

				if (!MeshInfoObj->TryGetStringField(TEXT("Name"), MeshInfo->Name)) continue;

				FString UniqueIDStr;
				if (!MeshInfoObj->TryGetStringField(TEXT("UniqueId"), UniqueIDStr)) continue;
				MeshInfo->UniqueId = FCString::Atoi64(*UniqueIDStr);

				MeshInfoObj->TryGetBoolField(TEXT("bImportAttribute"), MeshInfo->bImportAttribute);

				MeshInfoObj->TryGetStringField(TEXT("OptionName"), MeshInfo->OptionName);

				MeshInfoObj->TryGetBoolField(TEXT("bIsSkelMesh"), MeshInfo->bIsSkelMesh);

				MeshInfoObj->TryGetStringField(TEXT("OriginalImportPath"), MeshInfo->OriginalImportPath);

				MeshInfoObj->TryGetStringField(TEXT("OriginalFullImportName"), MeshInfo->OriginalFullImportName);

				MeshInfoObj->TryGetBoolField(TEXT("bOverridePath"), MeshInfo->bOverridePath);

				MeshInfoObj->TryGetStringField(TEXT("OverrideImportPath"), MeshInfo->OverrideImportPath);

				MeshInfoObj->TryGetStringField(TEXT("OverrideFullImportName"), MeshInfo->OverrideFullImportName);

				if (MeshInfoObj->TryGetStringField(TEXT("PivotNodeUid"), UniqueIDStr))
				{
					MeshInfo->PivotNodeUid = FCString::Atoi64(*UniqueIDStr);
				}

				MeshInfoObj->TryGetStringField(TEXT("LODGroup"), MeshInfo->LODGroup);

				MeshInfoObj->TryGetNumberField(TEXT("LODLevel"), MeshInfo->LODLevel);

				MeshInfoObj->TryGetBoolField(TEXT("IsLod"), MeshInfo->IsLod);

				SceneInfoSourceData->MeshInfo.Add(MeshInfo);
			}
		}

		//Read Hierarchy
		const TArray<TSharedPtr<FJsonValue>>* JSONHierarchyNodes;
		if (SceneInfoObj->TryGetArrayField(TEXT("Hierarchy"), JSONHierarchyNodes))
		{
			for (const auto& NodeJsonValue : *JSONHierarchyNodes)
			{
				const TSharedPtr<FJsonObject>& NodeInfoObj = NodeJsonValue->AsObject();
				if (!NodeInfoObj.IsValid())
				{
					continue;
				}
				TSharedPtr<FFbxNodeInfo> NodeInfo = MakeShareable(new FFbxNodeInfo());

				if (!NodeInfoObj->TryGetStringField(TEXT("NodeName"), NodeInfo->NodeName)) continue;

				NodeInfoObj->TryGetStringField(TEXT("NodeHierarchyPath"), NodeInfo->NodeHierarchyPath);

				FString UniqueIDStr;
				if (!NodeInfoObj->TryGetStringField(TEXT("UniqueId"), UniqueIDStr)) continue;
				NodeInfo->UniqueId = FCString::Atoi64(*UniqueIDStr);

				NodeInfoObj->TryGetBoolField(TEXT("bImportNode"), NodeInfo->bImportNode);

				//Find the parent
				if (NodeInfoObj->TryGetStringField(TEXT("ParentUniqueId"), UniqueIDStr))
				{
					uint64 ParentUniqueId = FCString::Atoi64(*UniqueIDStr);
					if (ParentUniqueId != 0)
					{
						//Find the parent
						for (TSharedPtr<FFbxNodeInfo> SearchNodeInfo : SceneInfoSourceData->HierarchyInfo)
						{
							if (SearchNodeInfo->UniqueId == ParentUniqueId)
							{
								NodeInfo->ParentNodeInfo = SearchNodeInfo;
								SearchNodeInfo->Childrens.Add(NodeInfo);
								break;
							}
						}
					}
				}

				NodeInfoObj->TryGetStringField(TEXT("AttributeType"), NodeInfo->AttributeType);

				if (NodeInfoObj->TryGetStringField(TEXT("AttributeUniqueId"), UniqueIDStr))
				{
					NodeInfo->AttributeUniqueId = FCString::Atoi64(*UniqueIDStr);
					if (NodeInfo->AttributeUniqueId != 0)
					{
						//Find the attribute info
						for (TSharedPtr<FFbxMeshInfo> MeshInfo : SceneInfoSourceData->MeshInfo)
						{
							if (MeshInfo->UniqueId == NodeInfo->AttributeUniqueId)
							{
								NodeInfo->AttributeInfo = MeshInfo;
								break;
							}
						}
					}
				}

				//Materials
				const TArray<TSharedPtr<FJsonValue>>* JSONMaterials;
				if (NodeInfoObj->TryGetArrayField(TEXT("Materials"), JSONMaterials))
				{
					for (const auto& MaterialJsonValue : *JSONMaterials)
					{
						const TSharedPtr<FJsonObject>& MaterialObj = MaterialJsonValue->AsObject();
						if (!MaterialObj.IsValid())
						{
							continue;
						}

						TSharedPtr<FFbxMaterialInfo> MaterialInfo = MakeShareable(new FFbxMaterialInfo());

						if (!MaterialObj->TryGetStringField(TEXT("Name"), MaterialInfo->Name)) continue;

						MaterialObj->TryGetStringField(TEXT("HierarchyPath"), MaterialInfo->HierarchyPath);

						if (!MaterialObj->TryGetStringField(TEXT("UniqueId"), UniqueIDStr)) continue;
						MaterialInfo->UniqueId = FCString::Atoi64(*UniqueIDStr);

						MaterialObj->TryGetBoolField(TEXT("bImportAttribute"), MaterialInfo->bImportAttribute);

						MaterialObj->TryGetStringField(TEXT("OriginalImportPath"), MaterialInfo->OriginalImportPath);

						MaterialObj->TryGetStringField(TEXT("OriginalFullImportName"), MaterialInfo->OriginalFullImportName);

						MaterialObj->TryGetBoolField(TEXT("bOverridePath"), MaterialInfo->bOverridePath);

						MaterialObj->TryGetStringField(TEXT("OverrideImportPath"), MaterialInfo->OverrideImportPath);

						MaterialObj->TryGetStringField(TEXT("OverrideFullImportName"), MaterialInfo->OverrideFullImportName);

						NodeInfo->Materials.Add(MaterialInfo);
					}
				}

				SceneInfoSourceData->HierarchyInfo.Add(NodeInfo);
			}
		}
	}
}

void UFbxSceneImportData::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UFbxSceneImportData::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	FAssetImportInfo AssetImportInfo;
	FAssetImportInfo::FSourceFile SourceFile(SourceFbxFile);
	AssetImportInfo.Insert(SourceFile);
	Context.AddTag(FAssetRegistryTag(SourceFileTagName(), AssetImportInfo.ToJson(), FAssetRegistryTag::TT_Hidden));
	Super::GetAssetRegistryTags(Context);
}

void UFbxSceneImportData::Serialize(FArchive& Ar)
{
	if (!Ar.IsFilterEditorOnly())
	{
		FString Json;
		if (Ar.IsLoading())
		{
			Ar << Json;
			FromJson(MoveTemp(Json));
		}
		else if (Ar.IsSaving())
		{
			Json = ToJson();
			Ar << Json;
		}
	}

	Super::Serialize(Ar);
}
#endif
