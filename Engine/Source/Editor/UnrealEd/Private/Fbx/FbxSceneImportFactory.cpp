// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/FbxSceneImportFactory.h"
#include "Misc/CoreMisc.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "Misc/FeedbackContext.h"
#include "Modules/ModuleManager.h"
#include "StaticMeshResources.h"
#include "UObject/Package.h"
#include "Misc/PackageName.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "Engine/EngineTypes.h"
#include "Components/SceneComponent.h"
#include "MaterialDomain.h"
#include "Materials/MaterialInterface.h"
#include "Components/StaticMeshComponent.h"
#include "ActorFactories/ActorFactoryEmptyActor.h"
#include "Engine/SkeletalMesh.h"
#include "Components/SkeletalMeshComponent.h"
#include "Materials/Material.h"
#include "Engine/Texture.h"
#include "Factories/FbxFactory.h"
#include "Factories/FbxAnimSequenceImportData.h"
#include "Factories/FbxSkeletalMeshImportData.h"
#include "Factories/FbxStaticMeshImportData.h"
#include "Factories/FbxTextureImportData.h"
#include "Factories/FbxSceneImportData.h"
#include "Factories/FbxSceneImportOptions.h"
#include "Factories/FbxSceneImportOptionsSkeletalMesh.h"
#include "Factories/FbxSceneImportOptionsStaticMesh.h"
#include "EngineGlobals.h"
#include "Camera/CameraComponent.h"
#include "Components/SpotLightComponent.h"
#include "Components/PointLightComponent.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"
#include "Components/DirectionalLightComponent.h"
#include "AssetRegistry/AssetData.h"
#include "Editor.h"
#include "FileHelpers.h"
#include "CineCameraComponent.h"
#include "Rendering/SkeletalMeshModel.h"

#include "AssetSelection.h"

#include "Logging/TokenizedMessage.h"
#include "FbxImporter.h"
#include "Misc/FbxErrors.h"
#include "AssetRegistry/AssetRegistryModule.h"

#include "Fbx/SSceneImportNodeTreeView.h"
#include "SFbxSceneOptionWindow.h"
#include "Interfaces/IMainFrameModule.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "PackageTools.h"

#include "Kismet2/KismetEditorUtilities.h"
#include "Math/UnitConversion.h"

#include "HAL/FileManager.h"
#include "LODUtilities.h"
#include "ComponentReregisterContext.h"
#include "Animation/Skeleton.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FbxSceneImportFactory)

#define LOCTEXT_NAMESPACE "FBXSceneImportFactory"

using namespace UnFbx;

//////////////////////////////////////////////////////////////////////////
// TODO list
// -. Set the combineMesh to true when importing a group of staticmesh
// -. Export correctly skeletal mesh, in fbxreview the skeletal mesh is not
//    playing is animation.
// -. Create some tests
// -. Support for camera
// -. Support for light

//Initialize static default option name
FString UFbxSceneImportFactory::DefaultOptionName = FString(TEXT("Default"));

FbxNode* FindFbxNodeById(UnFbx::FFbxImporter* FbxImporter, FbxNode *CurrentNode, uint64 UniqueId)
{
	if (CurrentNode == nullptr)
	{
		CurrentNode = FbxImporter->Scene->GetRootNode();
	}
	if (CurrentNode->GetUniqueID() == UniqueId)
	{
		return CurrentNode;
	}
	for (int32 ChildIndex = 0; ChildIndex < CurrentNode->GetChildCount(); ++ChildIndex)
	{
		FbxNode *FoundNode = FindFbxNodeById(FbxImporter, CurrentNode->GetChild(ChildIndex), UniqueId);
		if (FoundNode != nullptr)
		{
			return FoundNode;
		}
	}
	return nullptr;
}

bool GetFbxSceneImportOptions(UnFbx::FFbxImporter* FbxImporter
	, TSharedPtr<FFbxSceneInfo> SceneInfoPtr
	, UnFbx::FBXImportOptions *GlobalImportSettings
	, UFbxSceneImportOptions *SceneImportOptions
	, UFbxSceneImportOptionsStaticMesh *StaticMeshImportData
	, ImportOptionsNameMap &NameOptionsMap
	, UFbxSceneImportOptionsSkeletalMesh *SkeletalMeshImportData
	, FString Path)
{
	//Make sure we don't put the global transform into the vertex position of the mesh
	GlobalImportSettings->bTransformVertexToAbsolute = false;
	//Avoid combining meshes
	GlobalImportSettings->bCombineToSingle = false;
	//Use the full name (avoid creating one) to let us retrieve node transform and attachment later
	GlobalImportSettings->bUsedAsFullName = true;
	//Make sure we import the textures
	GlobalImportSettings->bImportTextures = true;
	//Make sure Material get imported
	GlobalImportSettings->bImportMaterials = true;
	//Make sure skeletal mesh sections with the same material are combined (and not kept separate)
	GlobalImportSettings->bKeepSectionsSeparate = false;
	//TODO support T0AsRefPose
	GlobalImportSettings->bUseT0AsRefPose = false;

	GlobalImportSettings->ImportTranslation = FVector(0);
	GlobalImportSettings->ImportRotation = FRotator(0);
	GlobalImportSettings->ImportUniformScale = 1.0f;
	GlobalImportSettings->DistanceFieldResolutionScale = 1.0f;

	GlobalImportSettings->bConvertScene = true;
	GlobalImportSettings->bConvertSceneUnit = true;

	GlobalImportSettings->bBakePivotInVertex = SceneImportOptions->bBakePivotInVertex;
	GlobalImportSettings->bInvertNormalMap = SceneImportOptions->bInvertNormalMaps;

	//TODO this options will be set by the fbxscene UI in the material options tab, it also should be save/load from config file
	//Prefix materials package name to put all material under Material folder (this avoid name clash with meshes)
	GlobalImportSettings->MaterialBasePath = NAME_None;

	GlobalImportSettings->OverrideMaterials.Reset();

	// Don't show the import options in unattended mode
	if (!GIsRunningUnattendedScript)
	{
		TSharedPtr<SWindow> ParentWindow;
		if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
		{
			IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
			ParentWindow = MainFrame.GetParentWindow();
		}
		TSharedRef<SWindow> Window = SNew(SWindow)
			.ClientSize(FVector2D(820.f, 650.f))
			.Title(NSLOCTEXT("UnrealEd", "FBXSceneImportOpionsTitle", "FBX Scene Import Options"));
		TSharedPtr<SFbxSceneOptionWindow> FbxSceneOptionWindow;

		Window->SetContent
		(
			SAssignNew(FbxSceneOptionWindow, SFbxSceneOptionWindow)
			.SceneInfo(SceneInfoPtr)
			.GlobalImportSettings(GlobalImportSettings)
			.SceneImportOptionsDisplay(SceneImportOptions)
			.SceneImportOptionsStaticMeshDisplay(StaticMeshImportData)
			.OverrideNameOptionsMap(&NameOptionsMap)
			.SceneImportOptionsSkeletalMeshDisplay(SkeletalMeshImportData)
			.OwnerWindow(Window)
			.FullPath(Path)
		);

		FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);

		if (!FbxSceneOptionWindow->ShouldImport())
		{
			return false;
		}
	}
	else
	{
		//Copy static and skeletalmesh options
		SFbxSceneOptionWindow::CopyStaticMeshOptionsToFbxOptions(GlobalImportSettings, StaticMeshImportData);
		SFbxSceneOptionWindow::CopySkeletalMeshOptionsToFbxOptions(GlobalImportSettings, SkeletalMeshImportData);
		NameOptionsMap.Add(UFbxSceneImportFactory::DefaultOptionName, GlobalImportSettings);
	}

	//setup all options
	GlobalImportSettings->bForceFrontXAxis = SceneImportOptions->bForceFrontXAxis;
	GlobalImportSettings->bBakePivotInVertex = SceneImportOptions->bBakePivotInVertex;
	GlobalImportSettings->bImportStaticMeshLODs = SceneImportOptions->bImportStaticMeshLODs;
	GlobalImportSettings->bImportSkeletalMeshLODs = SceneImportOptions->bImportSkeletalMeshLODs;
	GlobalImportSettings->bInvertNormalMap = SceneImportOptions->bInvertNormalMaps;
	GlobalImportSettings->ImportTranslation = SceneImportOptions->ImportTranslation;
	GlobalImportSettings->ImportRotation = SceneImportOptions->ImportRotation;
	GlobalImportSettings->ImportUniformScale = SceneImportOptions->ImportUniformScale;

	//Set the override material into the options
	for (TSharedPtr<FFbxNodeInfo> NodeInfo : SceneInfoPtr->HierarchyInfo)
	{
		for (TSharedPtr<FFbxMaterialInfo> Material : NodeInfo->Materials)
		{
			if (!GlobalImportSettings->OverrideMaterials.Contains(Material->UniqueId))
			{
				//If user dont want to import a material we have to replace it by the default material
				if (!Material->bImportAttribute)
				{
					UMaterial* DefaultMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
					if (DefaultMaterial != nullptr)
					{
						GlobalImportSettings->OverrideMaterials.Add(Material->UniqueId, DefaultMaterial);
					}
				}
				else if (Material->bOverridePath)
				{
					UMaterialInterface *UnrealMaterial = static_cast<UMaterialInterface*>(Material->GetContentObject());
					if (UnrealMaterial != nullptr)
					{
						GlobalImportSettings->OverrideMaterials.Add(Material->UniqueId, UnrealMaterial);
					}
				}
			}
		}
	}

	//Save the import options in ini config file
	SceneImportOptions->SaveConfig();

	//Save the Default setting copy them in the UObject and save them
	SFbxSceneOptionWindow::CopyFbxOptionsToStaticMeshOptions(GlobalImportSettings, StaticMeshImportData);
	StaticMeshImportData->SaveConfig();

	SFbxSceneOptionWindow::CopyFbxOptionsToSkeletalMeshOptions(GlobalImportSettings, SkeletalMeshImportData);
	SkeletalMeshImportData->SaveConfig();

	return true;
}

bool IsEmptyAttribute(FString AttributeType)
{
	return (AttributeType.Compare("eNull") == 0 || AttributeType.Compare("eUnknown") == 0);
}

static void ExtractPropertyTextures(FbxSurfaceMaterial *FbxMaterial, TSharedPtr<FFbxMaterialInfo> MaterialInfo, const char *MaterialProperty, TMap<uint64, TSharedPtr<FFbxTextureInfo>> ExtractedTextures)
{
	FbxProperty FbxProperty = FbxMaterial->FindProperty(MaterialProperty);
	if (FbxProperty.IsValid())
	{
		int32 LayeredTextureCount = FbxProperty.GetSrcObjectCount<FbxLayeredTexture>();
		if (LayeredTextureCount == 0)
		{
			int32 TextureCount = FbxProperty.GetSrcObjectCount<FbxFileTexture>();
			if (TextureCount > 0)
			{
				for (int32 TextureIndex = 0; TextureIndex < TextureCount; ++TextureIndex)
				{
					FbxFileTexture* FbxTexture = FbxProperty.GetSrcObject<FbxFileTexture>(TextureIndex);
					TSharedPtr<FFbxTextureInfo> TextureInfo = nullptr;
					if (ExtractedTextures.Contains(FbxTexture->GetUniqueID()))
					{
						TextureInfo = *(ExtractedTextures.Find(FbxTexture->GetUniqueID()));
					}
					else
					{
						TextureInfo = MakeShareable(new FFbxTextureInfo());
						TextureInfo->Name = UTF8_TO_TCHAR(FbxTexture->GetName());
						TextureInfo->UniqueId = FbxTexture->GetUniqueID();
						TextureInfo->TexturePath = UTF8_TO_TCHAR(FbxTexture->GetFileName());
						ExtractedTextures.Add(TextureInfo->UniqueId, TextureInfo);
					}
					//Add the texture
					MaterialInfo->Textures.Add(TextureInfo);
				}
			}
		}
	}
}

static void ExtractMaterialInfoFromNode(UnFbx::FFbxImporter* FbxImporter, FbxNode *Node, TSharedPtr<FFbxSceneInfo> SceneInfoPtr, TMap<uint64, TSharedPtr<FFbxMaterialInfo>> ExtractedMaterials, TMap<uint64, TSharedPtr<FFbxTextureInfo>> ExtractedTextures, FString CurrentHierarchyPath)
{
	TSharedPtr<FFbxNodeInfo> FoundNode = nullptr;
	for (TSharedPtr<FFbxNodeInfo> Nodeinfo : SceneInfoPtr->HierarchyInfo)
	{
		if (Nodeinfo->UniqueId == Node->GetUniqueID())
		{
			FoundNode = Nodeinfo;
		}
	}
	if (FoundNode.IsValid())
	{
		if (!CurrentHierarchyPath.IsEmpty())
		{
			CurrentHierarchyPath += TEXT("/");
		}
		CurrentHierarchyPath += FoundNode->NodeName;

		for (int MaterialIndex = 0; MaterialIndex < Node->GetMaterialCount(); ++MaterialIndex)
		{
			FbxSurfaceMaterial *FbxMaterial = Node->GetMaterial(MaterialIndex);
			TSharedPtr<FFbxMaterialInfo> MaterialInfo = nullptr;
			if (ExtractedMaterials.Contains(FbxMaterial->GetUniqueID()))
			{
				MaterialInfo = *(ExtractedMaterials.Find(FbxMaterial->GetUniqueID()));
			}
			else
			{
				MaterialInfo = MakeShareable(new FFbxMaterialInfo());
				MaterialInfo->HierarchyPath = CurrentHierarchyPath;
				MaterialInfo->UniqueId = FbxMaterial->GetUniqueID();
				MaterialInfo->Name = UTF8_TO_TCHAR(FbxMaterial->GetName());
				TCHAR IllegalCharacters[7] = { '/', '\\', ' ', '`', '\t', '\r', '\n' };
				bool DisplayInvalidNameError = false;
				FString OldMaterialName = MaterialInfo->Name;
				for (TCHAR IllegalCharacter : IllegalCharacters)
				{
					TCHAR IllegalChar[2];
					IllegalChar[0] = IllegalCharacter;
					IllegalChar[1] = '\0';
					if (MaterialInfo->Name.Contains(&IllegalChar[0]))
					{
						MaterialInfo->Name = MaterialInfo->Name.Replace(&IllegalChar[0], TEXT("_"));
						DisplayInvalidNameError = true;
					}
				}
				if (DisplayInvalidNameError)
				{
					FbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("FoundInvalidCharacterInMaterialName", "Found invalid character in a material name. Original name: {0} New name: {1}"), FText::FromString(OldMaterialName), FText::FromString(MaterialInfo->Name))), FFbxErrors::Generic_InvalidCharacterInName);
				}
				ExtractPropertyTextures(FbxMaterial, MaterialInfo, FbxSurfaceMaterial::sDiffuse, ExtractedTextures);
				ExtractPropertyTextures(FbxMaterial, MaterialInfo, FbxSurfaceMaterial::sEmissive, ExtractedTextures);
				ExtractPropertyTextures(FbxMaterial, MaterialInfo, FbxSurfaceMaterial::sSpecular, ExtractedTextures);
				//ExtractPropertyTextures(FbxMaterial, MaterialInfo, FbxSurfaceMaterial::sSpecularFactor, ExtractedTextures);
				//ExtractPropertyTextures(FbxMaterial, MaterialInfo, FbxSurfaceMaterial::sShininess, ExtractedTextures);
				ExtractPropertyTextures(FbxMaterial, MaterialInfo, FbxSurfaceMaterial::sNormalMap, ExtractedTextures);
				ExtractPropertyTextures(FbxMaterial, MaterialInfo, FbxSurfaceMaterial::sBump, ExtractedTextures);
				ExtractPropertyTextures(FbxMaterial, MaterialInfo, FbxSurfaceMaterial::sTransparentColor, ExtractedTextures);
				ExtractPropertyTextures(FbxMaterial, MaterialInfo, FbxSurfaceMaterial::sTransparencyFactor, ExtractedTextures);
				ExtractedMaterials.Add(MaterialInfo->UniqueId, MaterialInfo);
			}
			//Add the Material to the node
			FoundNode->Materials.AddUnique(MaterialInfo);
		}
	}
	for (int ChildIndex = 0; ChildIndex < Node->GetChildCount(); ++ChildIndex)
	{
		FbxNode *ChildNode = Node->GetChild(ChildIndex);
		ExtractMaterialInfoFromNode(FbxImporter, ChildNode, SceneInfoPtr, ExtractedMaterials, ExtractedTextures, CurrentHierarchyPath);
	}
}

void UFbxSceneImportFactory::ExtractMaterialInfo(void* FbxImporterVoid, TSharedPtr<FFbxSceneInfo> SceneInfoPtr)
{
	UnFbx::FFbxImporter* FbxImporter = (UnFbx::FFbxImporter*)FbxImporterVoid;
	TMap<uint64, TSharedPtr<FFbxMaterialInfo>> ExtractedMaterials;
	TMap<uint64, TSharedPtr<FFbxTextureInfo>> ExtractedTextures;
	FbxNode* RootNode = FbxImporter->Scene->GetRootNode();
	FString CurrentHierarchyPath = TEXT("");
	ExtractMaterialInfoFromNode(FbxImporter, RootNode, SceneInfoPtr, ExtractedMaterials, ExtractedTextures, CurrentHierarchyPath);
}

bool IsPartOfSkeletonHierarchy(const TMap<uint64, const UnFbx::FbxNodeInfo*> &NodeInfoMap, const UnFbx::FbxNodeInfo& NodeInfo)
{
	FString AttributeType = NodeInfo.AttributeType;
	if (AttributeType.Compare(TEXT("eSkeleton")) == 0)
	{
		return true;
	}
	if (NodeInfoMap.Contains(NodeInfo.ParentUniqueId))
	{
		const UnFbx::FbxNodeInfo *ParentNodeInfo = *NodeInfoMap.Find(NodeInfo.ParentUniqueId);
		return IsPartOfSkeletonHierarchy(NodeInfoMap, *ParentNodeInfo);
	}
	return false;
}

void FetchFbxCameraInScene(UnFbx::FFbxImporter *FbxImporter, FbxNode* ParentNode, TSharedPtr<FFbxSceneInfo> SceneInfoPtr)
{
	if (ParentNode->GetNodeAttribute() && ParentNode->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eCamera)
	{
		FbxCamera* CameraAttribute = (FbxCamera*)ParentNode->GetNodeAttribute();
		if (CameraAttribute != nullptr && CameraAttribute->GetNode() != nullptr && !SceneInfoPtr->CameraInfo.Contains(CameraAttribute->GetUniqueID()))
		{
			FbxNode* CameraNode = CameraAttribute->GetNode();
			TSharedPtr<FFbxCameraInfo> CameraInfo = MakeShareable(new FFbxCameraInfo());

			if (CameraAttribute->GetName()[0] != '\0')
			{
				CameraInfo->Name = FbxImporter->MakeName(CameraAttribute->GetName());
			}
			else
			{
				CameraInfo->Name = FbxImporter->MakeString(CameraNode ? CameraNode->GetName() : "None");
			}
			CameraInfo->UniqueId = CameraAttribute->GetUniqueID();

			FbxDouble FieldOfView;
			FbxDouble FocalLength;

			if (CameraAttribute->GetApertureMode() == FbxCamera::eFocalLength)
			{
				FocalLength = CameraAttribute->FocalLength.Get();
				FieldOfView = CameraAttribute->ComputeFieldOfView(FocalLength);
			}
			else
			{
				FieldOfView = CameraAttribute->FieldOfView.Get();
				FocalLength = CameraAttribute->ComputeFocalLength(FieldOfView);
			}

			CameraInfo->AspectWidth				= static_cast<float>(CameraAttribute->AspectWidth.Get());
			CameraInfo->AspectHeight			= static_cast<float>(CameraAttribute->AspectHeight.Get());
			CameraInfo->NearPlane				= static_cast<float>(CameraAttribute->NearPlane.Get());
			CameraInfo->FarPlane				= static_cast<float>(CameraAttribute->FarPlane.Get());
			CameraInfo->ProjectionPerspective	= CameraAttribute->ProjectionType.Get() == FbxCamera::ePerspective;
			CameraInfo->OrthoZoom				= static_cast<float>(CameraAttribute->OrthoZoom.Get());
			CameraInfo->FieldOfView				= static_cast<float>(FieldOfView);
			CameraInfo->FocalLength				= static_cast<float>(FocalLength);
			CameraInfo->ApertureWidth			= static_cast<float>(CameraAttribute->GetApertureWidth());
			CameraInfo->ApertureHeight			= static_cast<float>(CameraAttribute->GetApertureHeight());
			SceneInfoPtr->CameraInfo.Add(CameraInfo->UniqueId, CameraInfo);
		}
	}
	for (int i = 0; i < ParentNode->GetChildCount(); ++i)
	{
		FbxNode* Child = ParentNode->GetChild(i);
		FetchFbxCameraInScene(FbxImporter, Child, SceneInfoPtr);
	}
}


void FetchFbxLightInScene(UnFbx::FFbxImporter *FbxImporter, FbxNode* ParentNode, TSharedPtr<FFbxSceneInfo> SceneInfoPtr)
{
	if (ParentNode->GetNodeAttribute() && ParentNode->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLight)
	{
		FbxLight* LightAttribute = (FbxLight*)ParentNode->GetNodeAttribute();
		if (LightAttribute != nullptr && LightAttribute->GetNode() != nullptr && !SceneInfoPtr->LightInfo.Contains(LightAttribute->GetUniqueID()))
		{
			FbxNode* LightNode = LightAttribute->GetNode();
			TSharedPtr<FFbxLightInfo> LightInfo = MakeShareable(new FFbxLightInfo());

			if (LightAttribute->GetName()[0] != '\0')
			{
				LightInfo->Name = FbxImporter->MakeName(LightAttribute->GetName());
			}
			else
			{
				LightInfo->Name = FbxImporter->MakeString(LightNode ? LightNode->GetName() : "None");
			}
			LightInfo->UniqueId = LightAttribute->GetUniqueID();
			switch (LightAttribute->LightType.Get())
			{
			case FbxLight::ePoint:
				LightInfo->Type = 0;
				break;
			case FbxLight::eDirectional:
				LightInfo->Type = 1;
				break;
			case FbxLight::eSpot:
				LightInfo->Type = 2;
				break;
			case FbxLight::eArea:
				LightInfo->Type = 3;
				break;
			case FbxLight::eVolume:
				LightInfo->Type = 4;
				break;
			}
			LightInfo->Color = FFbxDataConverter::ConvertColor(LightAttribute->Color);
			LightInfo->Intensity = (float)(LightAttribute->Intensity.Get());
			switch (LightAttribute->DecayType.Get())
			{
			case FbxLight::EDecayType::eNone:
				LightInfo->Decay = 0;
				break;
			case FbxLight::EDecayType::eLinear:
				LightInfo->Decay = 1;
				break;
			case FbxLight::EDecayType::eQuadratic:
				LightInfo->Decay = 2;
				break;
			case FbxLight::EDecayType::eCubic:
				LightInfo->Decay = 3;
				break;
			}
			LightInfo->CastLight = LightAttribute->CastLight.Get();
			LightInfo->CastShadow = LightAttribute->CastShadows.Get();
			LightInfo->ShadowColor = FFbxDataConverter::ConvertColor(LightAttribute->ShadowColor);

			LightInfo->InnerAngle = (float)(LightAttribute->InnerAngle.Get());
			LightInfo->OuterAngle = (float)(LightAttribute->OuterAngle.Get());
			LightInfo->Fog = (float)(LightAttribute->Fog.Get());
			LightInfo->DecayStart = (float)(LightAttribute->DecayStart.Get());
			LightInfo->EnableNearAttenuation = LightAttribute->EnableNearAttenuation.Get();
			LightInfo->NearAttenuationStart = (float)(LightAttribute->NearAttenuationStart.Get());
			LightInfo->NearAttenuationEnd = (float)(LightAttribute->NearAttenuationEnd.Get());
			LightInfo->EnableFarAttenuation = LightAttribute->EnableFarAttenuation.Get();
			LightInfo->FarAttenuationStart = (float)(LightAttribute->FarAttenuationStart.Get());
			LightInfo->FarAttenuationEnd = (float)(LightAttribute->FarAttenuationEnd.Get());
			SceneInfoPtr->LightInfo.Add(LightInfo->UniqueId, LightInfo);
		}
	}
	for (int i = 0; i < ParentNode->GetChildCount(); ++i)
	{
		FbxNode* Child = ParentNode->GetChild(i);
		FetchFbxLightInScene(FbxImporter, Child, SceneInfoPtr);
	}
}

// TODO we should replace the old UnFbx:: data by the new data that use shared pointer.
// For now we convert the old structure to the new one
TSharedPtr<FFbxSceneInfo> UFbxSceneImportFactory::ConvertSceneInfo(void* VoidFbxImporter, void* VoidFbxSceneInfo)
{
	UnFbx::FFbxImporter *FbxImporter = (UnFbx::FFbxImporter*)VoidFbxImporter;
	UnFbx::FbxSceneInfo &SceneInfo = *((UnFbx::FbxSceneInfo*)VoidFbxSceneInfo);
	TSharedPtr<FFbxSceneInfo> SceneInfoPtr = MakeShareable(new FFbxSceneInfo());
	SceneInfoPtr->NonSkinnedMeshNum = SceneInfo.NonSkinnedMeshNum;
	SceneInfoPtr->SkinnedMeshNum = SceneInfo.SkinnedMeshNum;
	SceneInfoPtr->TotalGeometryNum = SceneInfo.TotalGeometryNum;
	SceneInfoPtr->TotalMaterialNum = SceneInfo.TotalMaterialNum;
	SceneInfoPtr->TotalTextureNum = SceneInfo.TotalTextureNum;
	SceneInfoPtr->bHasAnimation = SceneInfo.bHasAnimation;
	SceneInfoPtr->FrameRate = SceneInfo.FrameRate;
	SceneInfoPtr->TotalTime = SceneInfo.TotalTime;

	//Get the valid skeletal mesh from the fbx file and store it in the map
	TMap<uint64, FbxMesh*> ValidSkeletalMesh;
	FbxNode* RootNodeToImport = FbxImporter->Scene->GetRootNode();
	TArray< TArray<FbxNode*>* > SkelMeshArray;
	UnFbx::FBXImportOptions* FbxImportOptionsPtr = FbxImporter->GetImportOptions();
	bool OldValueImportMeshesInBoneHierarchy = FbxImportOptionsPtr->bImportMeshesInBoneHierarchy;
	FbxImportOptionsPtr->bImportMeshesInBoneHierarchy = true;
	FbxImporter->FillFbxSkelMeshArrayInScene(RootNodeToImport, SkelMeshArray, false, false, true);
	FbxImportOptionsPtr->bImportMeshesInBoneHierarchy = OldValueImportMeshesInBoneHierarchy;

	for (int32 i = 0; i < SkelMeshArray.Num(); i++)
	{
		TArray<FbxNode*> NodeArray = *SkelMeshArray[i];
		if (NodeArray.Num() < 1)
			continue;
		FbxNode* RootNodeArrayNode = NodeArray[0];
		if (RootNodeArrayNode->GetNodeAttribute() && RootNodeArrayNode->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup)
		{
			RootNodeArrayNode = FbxImporter->FindLODGroupNode(RootNodeArrayNode, 0);
		}
		FbxMesh* Mesh = RootNodeArrayNode->GetMesh();
		if (Mesh != nullptr)
		{
			ValidSkeletalMesh.Add(Mesh->GetUniqueID(), Mesh);
			for (FbxNode* SkelMeshNode : NodeArray)
			{
				uint64 MeshNodeID = SkelMeshNode->GetUniqueID();
				FbxMesh* SkeletalMeshAttribute = SkelMeshNode->GetMesh();
				
				if (SkeletalMeshAttribute != nullptr)
					MeshNodeID = SkeletalMeshAttribute->GetUniqueID();

				for (UnFbx::FbxMeshInfo &MeshInfo : SceneInfo.MeshInfo)
				{
					if (MeshInfo.UniqueId == MeshNodeID)
					{
						//We have either a skeletal mesh or a rigid mesh
						MeshInfo.bIsSkelMesh = true;
						break;
					}
				}
			}
		}
	}

	

	for (const UnFbx::FbxMeshInfo& MeshInfo : SceneInfo.MeshInfo)
	{
		//Add the skeletal mesh if its a valid one
		if (MeshInfo.bIsSkelMesh && !ValidSkeletalMesh.Contains(MeshInfo.UniqueId))
		{
			continue;
		}
		TSharedPtr<FFbxMeshInfo> MeshInfoPtr = MakeShareable(new FFbxMeshInfo());
		MeshInfoPtr->FaceNum = MeshInfo.FaceNum;
		MeshInfoPtr->VertexNum = MeshInfo.VertexNum;
		MeshInfoPtr->bTriangulated = MeshInfo.bTriangulated;
		MeshInfoPtr->MaterialNum = MeshInfo.MaterialNum;
		MeshInfoPtr->bIsSkelMesh = MeshInfo.bIsSkelMesh;
		MeshInfoPtr->SkeletonRoot = MeshInfo.SkeletonRoot;
		MeshInfoPtr->SkeletonElemNum = MeshInfo.SkeletonElemNum;
		MeshInfoPtr->LODGroup = MeshInfo.LODGroup;
		MeshInfoPtr->LODLevel = MeshInfo.LODLevel;
		MeshInfoPtr->MorphNum = MeshInfo.MorphNum;
		MeshInfoPtr->Name = MeshInfo.Name;
		MeshInfoPtr->UniqueId = MeshInfo.UniqueId;
		MeshInfoPtr->OptionName = DefaultOptionName;

		MeshInfoPtr->IsLod = MeshInfoPtr->LODLevel > 0;
		MeshInfoPtr->IsCollision = MeshInfoPtr->Name.Contains(TEXT("UCX"), ESearchCase::IgnoreCase)
			|| MeshInfoPtr->Name.Contains(TEXT("UBX"), ESearchCase::IgnoreCase)
			|| MeshInfoPtr->Name.Contains(TEXT("MCDCX"), ESearchCase::IgnoreCase)
			|| MeshInfoPtr->Name.Contains(TEXT("USP"), ESearchCase::IgnoreCase)
			|| MeshInfoPtr->Name.Contains(TEXT("UCP"), ESearchCase::IgnoreCase);

		SceneInfoPtr->MeshInfo.Add(MeshInfoPtr);
	}
	
	//Find all light and camera in the scene
	FetchFbxCameraInScene(FbxImporter, RootNodeToImport, SceneInfoPtr);
	FetchFbxLightInScene(FbxImporter, RootNodeToImport, SceneInfoPtr);

	TMap<uint64, const UnFbx::FbxNodeInfo*> NodeInfoMap;
	for (const UnFbx::FbxNodeInfo& NodeInfo : SceneInfo.HierarchyInfo)
	{
		NodeInfoMap.Add(NodeInfo.UniqueId, &NodeInfo);
		if (IsPartOfSkeletonHierarchy(NodeInfoMap, NodeInfo))
		{
			continue;
		}

		TSharedPtr<FFbxNodeInfo> NodeInfoPtr = MakeShareable(new FFbxNodeInfo());
		NodeInfoPtr->NodeName = NodeInfo.ObjectName;
		NodeInfoPtr->UniqueId = NodeInfo.UniqueId;
		NodeInfoPtr->AttributeType = NodeInfo.AttributeType;
		NodeInfoPtr->AttributeUniqueId = NodeInfo.AttributeUniqueId;

		//Find the parent
		NodeInfoPtr->ParentNodeInfo = nullptr;
		for (TSharedPtr<FFbxNodeInfo> ParentPtr : SceneInfoPtr->HierarchyInfo)
		{
			if (ParentPtr->UniqueId == NodeInfo.ParentUniqueId)
			{
				NodeInfoPtr->ParentNodeInfo = ParentPtr;
				ParentPtr->Childrens.Add(NodeInfoPtr);
				break;
			}
		}

		//Find the attribute info
		NodeInfoPtr->AttributeInfo = nullptr;
		if (NodeInfoPtr->AttributeType.Compare(TEXT("eMesh")) == 0)
		{
			for (TSharedPtr<FFbxMeshInfo> AttributePtr : SceneInfoPtr->MeshInfo)
			{
				if (AttributePtr->UniqueId == NodeInfo.AttributeUniqueId)
				{
					NodeInfoPtr->AttributeInfo = AttributePtr;
					break;
				}
			}
		}

		//Set the transform
		NodeInfoPtr->Transform = FTransform::Identity;
		FbxVector4 NewLocalT = NodeInfo.Transform.GetT();
		FbxVector4 NewLocalS = NodeInfo.Transform.GetS();
		FbxQuaternion NewLocalQ = NodeInfo.Transform.GetQ();
		NodeInfoPtr->Transform.SetTranslation(UnFbx::FFbxDataConverter::ConvertPos(NewLocalT));
		NodeInfoPtr->Transform.SetScale3D(UnFbx::FFbxDataConverter::ConvertScale(NewLocalS));
		NodeInfoPtr->Transform.SetRotation(UnFbx::FFbxDataConverter::ConvertRotToQuat(NewLocalQ));
		NodeInfoPtr->PivotRotation = UnFbx::FFbxDataConverter::ConvertPos(NodeInfo.RotationPivot);
		NodeInfoPtr->PivotScaling = UnFbx::FFbxDataConverter::ConvertPos(NodeInfo.ScalePivot);
		
		//Set the attribute pivot dictionary
		if (NodeInfoPtr->AttributeInfo.IsValid())
		{
			if (NodeInfoPtr->AttributeInfo->NodeReferencePivots.Contains(NodeInfoPtr->PivotRotation))
			{
				TArray<uint64>*NodeUidArray = NodeInfoPtr->AttributeInfo->NodeReferencePivots.Find(NodeInfoPtr->PivotRotation);
				check(NodeUidArray);
				NodeUidArray->Add(NodeInfoPtr->UniqueId);
			}
			else
			{
				TArray<uint64> NodeUidArray;
				NodeUidArray.Add(NodeInfoPtr->UniqueId);
				NodeInfoPtr->AttributeInfo->NodeReferencePivots.Add(NodeInfoPtr->PivotRotation, NodeUidArray);

			}
			if (NodeInfoPtr->AttributeInfo->PivotNodeUid == INVALID_UNIQUE_ID)
			{
				NodeInfoPtr->AttributeInfo->PivotNodeUid = NodeInfoPtr->UniqueId;
				NodeInfoPtr->AttributeInfo->PivotNodeName = NodeInfoPtr->NodeName;
			}
		}

		if (SceneInfoPtr->LightInfo.Contains(NodeInfoPtr->AttributeUniqueId))
		{
			//Add the z rotation of 90 degree locally for every light. Light direction differ from fbx to unreal 
			FRotator LightRotator(0.0f, 90.0f, 0.0f);
			FTransform LightTransform = FTransform(LightRotator);
			NodeInfoPtr->Transform = LightTransform * NodeInfoPtr->Transform;
		}
		else if (SceneInfoPtr->CameraInfo.Contains(NodeInfoPtr->AttributeUniqueId))
		{
			//Add a roll of -90 degree locally for every cameras. Camera up vector differ from fbx to unreal
			FRotator LightRotator(0.0f, 0.0f, -90.0f);
			FTransform LightTransform = FTransform(LightRotator);
			//Remove the scale of the node holding a camera (the mesh is provide by the engine and can be different in size)
			NodeInfoPtr->Transform.SetScale3D(FVector(1.0f));
			NodeInfoPtr->Transform = LightTransform * NodeInfoPtr->Transform;
		}

		//by default we import all node
		NodeInfoPtr->bImportNode = true;

		//Add the node to the hierarchy
		SceneInfoPtr->HierarchyInfo.Add(NodeInfoPtr);
	}

	for (TSharedPtr<FFbxNodeInfo> NodeInfo : SceneInfoPtr->HierarchyInfo)
	{
		if (NodeInfo->AttributeType.Compare(TEXT("eLODGroup")) == 0)
		{
			for (TSharedPtr<FFbxNodeInfo> ChildNodeInfo : NodeInfo->Childrens)
			{
				if (ChildNodeInfo->AttributeType.Compare(TEXT("eMesh")) != 0)
				{
					//We don't import under LOD group other stuff then the mesh
					ChildNodeInfo->bImportNode = false;
				}
			}
		}
	}
	return SceneInfoPtr;
}

UClass *FFbxMeshInfo::GetType()
{
	if (bIsSkelMesh)
		return USkeletalMesh::StaticClass();
	return UStaticMesh::StaticClass();
}

UClass *FFbxTextureInfo::GetType()
{
	return UTexture::StaticClass();
}

UClass *FFbxMaterialInfo::GetType()
{
	return UMaterial::StaticClass();
}

UPackage *FFbxAttributeInfo::GetContentPackage()
{
	if (!IsContentObjectUpToDate)
	{
		//Update the Object, this will update the ContentPackage and set the IsContentUpToDate state
		GetContentObject();
	}
	return ContentPackage;
}

UObject *FFbxAttributeInfo::GetContentObject()
{
	if (IsContentObjectUpToDate)
		return ContentObject;
	ContentPackage = nullptr;
	ContentObject = nullptr;
	FString ImportPath = UPackageTools::SanitizePackageName(GetImportPath());
	FString AssetName = GetFullImportName();
	if (!ImportPath.IsEmpty())
	{
		ContentPackage = LoadPackage(nullptr, *ImportPath, LOAD_Verify | LOAD_NoWarn);
	}

	if (ContentPackage != nullptr)
	{
		ContentPackage->FullyLoad();
	}
	ContentObject = FindFirstObject<UObject>(*AssetName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("FFbxAttributeInfo::GetContentObject()"));
	if (ContentObject != nullptr)
	{
		if (ContentObject->HasAnyFlags(RF_Transient) || !IsValid(ContentObject))
		{
			ContentObject = nullptr;
		}
		else if (ContentPackage == nullptr)
		{
			//If we are able to find the object but not to load the package, this mean that the package is a new created package that is not save yet
			ContentPackage = ContentObject->GetOutermost();
		}
	}

	IsContentObjectUpToDate = true;
	return ContentObject;
}

UFbxSceneImportFactory::UFbxSceneImportFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UWorld::StaticClass();
	Formats.Add(TEXT("fbx;Fbx Scene"));
	Formats.Add(TEXT("obj;OBJ Scene"));

	bCreateNew = false;
	bText = false;
	bEditorImport = true;
	Path = "";
	ImportWasCancel = false;

	SceneImportOptions = CreateDefaultSubobject<UFbxSceneImportOptions>(TEXT("SceneImportOptions"), true);
	SceneImportOptions->SetFlags(RF_Transactional);
	SceneImportOptionsStaticMesh = CreateDefaultSubobject<UFbxSceneImportOptionsStaticMesh>(TEXT("SceneImportOptionsStaticMesh"), true);
	SceneImportOptionsStaticMesh->SetFlags(RF_Transactional);
	SceneImportOptionsSkeletalMesh = CreateDefaultSubobject<UFbxSceneImportOptionsSkeletalMesh>(TEXT("SceneImportOptionsSkeletalMesh"), true);
	SceneImportOptionsSkeletalMesh->SetFlags(RF_Transactional);

	StaticMeshImportData = CreateDefaultSubobject<UFbxStaticMeshImportData>(TEXT("StaticMeshImportData"), true);
	StaticMeshImportData->SetFlags(RF_Transactional);
	SkeletalMeshImportData = CreateDefaultSubobject<UFbxSkeletalMeshImportData>(TEXT("SkeletalMeshImportData"), true);
	SkeletalMeshImportData->SetFlags(RF_Transactional);
	AnimSequenceImportData = CreateDefaultSubobject<UFbxAnimSequenceImportData>(TEXT("AnimSequenceImportData"), true);
	AnimSequenceImportData->SetFlags(RF_Transactional);
	TextureImportData = CreateDefaultSubobject<UFbxTextureImportData>(TEXT("TextureImportData"), true);
	TextureImportData->SetFlags(RF_Transactional);

	ReimportData = nullptr;
}

void UFbxSceneImportFactory::FillSceneHierarchyPath(TSharedPtr<FFbxSceneInfo> SceneInfo)
{
	//Set the hierarchy path for every node this data will be use by the reimport
	for (FbxNodeInfoPtr NodeInfo : SceneInfo->HierarchyInfo)
	{
		FString NodeTreePath = NodeInfo->NodeName;
		FbxNodeInfoPtr CurrentNode = NodeInfo->ParentNodeInfo;
		while (CurrentNode.IsValid())
		{
			NodeTreePath += TEXT(".");
			NodeTreePath += CurrentNode->NodeName;
			CurrentNode = CurrentNode->ParentNodeInfo;
		}
		NodeInfo->NodeHierarchyPath = NodeTreePath;
	}
}

UFbxSceneImportData* CreateReImportAsset(const FString &PackagePath, const FString &FbxImportFileName, UFbxSceneImportOptions* SceneImportOptions, TSharedPtr<FFbxSceneInfo> SceneInfo, ImportOptionsNameMap &NameOptionsMap)
{
	//Create or use existing package
	//The data must have the name of the import file to support drag drop reimport
	FString FilenameBase = FPaths::GetBaseFilename(FbxImportFileName);
	FString FbxReImportPkgName = PackagePath + TEXT("/") + FilenameBase;
	FbxReImportPkgName = UPackageTools::SanitizePackageName(FbxReImportPkgName);
	FString AssetName = FilenameBase;
	AssetName = UPackageTools::SanitizePackageName(AssetName);
	UPackage* Pkg = CreatePackage( *FbxReImportPkgName);
	if (!ensure(Pkg))
	{
		//TODO log an import warning stipulate that there is no re-import asset created
		return nullptr;
	}
	Pkg->FullyLoad();

	FbxReImportPkgName = FPackageName::GetLongPackageAssetName(Pkg->GetOutermost()->GetName());
	//Save the re-import data asset
	UFbxSceneImportData *ReImportAsset = NewObject<UFbxSceneImportData>(Pkg, NAME_None, RF_Public | RF_Standalone);
	FString NewUniqueName = AssetName;
	if (!ReImportAsset->Rename(*NewUniqueName, nullptr, REN_Test))
	{
		NewUniqueName = MakeUniqueObjectName(ReImportAsset, UFbxSceneImportData::StaticClass(), FName(*AssetName)).ToString();
	}
	ReImportAsset->Rename(*NewUniqueName, nullptr, REN_DontCreateRedirectors);
	ReImportAsset->SceneInfoSourceData = SceneInfo;
	//Copy the options map
	for (auto kvp : NameOptionsMap)
	{
		ReImportAsset->NameOptionsMap.Add(kvp.Key, kvp.Value);
	}
	
	ReImportAsset->SourceFbxFile = FPaths::ConvertRelativePathToFull(FbxImportFileName);
	ReImportAsset->bCreateFolderHierarchy = SceneImportOptions->bCreateContentFolderHierarchy;
	ReImportAsset->bForceFrontXAxis = SceneImportOptions->bForceFrontXAxis;
	ReImportAsset->HierarchyType = (int32)SceneImportOptions->HierarchyType;
	return ReImportAsset;
}

UObject* UFbxSceneImportFactory::FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled)
{
	// This function performs shortcut to call FactoryCreateBinary without loading a file to array.
	FString FileExtension = FPaths::GetExtension(Filename);
	
	if (!IFileManager::Get().FileExists(*Filename))
	{
		UE_LOGF(LogFbx, Error, "Failed to load file '%ls'", *Filename)
		return nullptr;
	}
	
	ParseParms(Parms);
	
	const uint8* Buffer = nullptr;
	const uint8* BufferEnd = nullptr;
	return FactoryCreateBinary(InClass, InParent, InName, Flags, nullptr, *FileExtension, Buffer, BufferEnd, Warn, bOutOperationCanceled);
}

UObject* UFbxSceneImportFactory::FactoryCreateBinary
(
	UClass*				Class,
	UObject*			InParent,
	FName				Name,
	EObjectFlags		Flags,
	UObject*			Context,
	const TCHAR*		Type,
	const uint8*&		Buffer,
	const uint8*		BufferEnd,
	FFeedbackContext*	Warn,
	bool& bOutOperationCanceled
	)
{
	UObject* ReturnObject = FactoryCreateBinary(Class, InParent, Name, Flags, Context, Type, Buffer, BufferEnd, Warn);
	bOutOperationCanceled = ImportWasCancel;
	ImportWasCancel = false;
	return ReturnObject;
}

bool UFbxSceneImportFactory::FactoryCanImport(const FString& Filename)
{
	const FString Extension = FPaths::GetExtension(Filename);

	if (Extension == TEXT("fbx") || Extension == TEXT("obj"))
	{
		return true;
	}
	return false;
}

TArray<FString> UFbxSceneImportFactory::GetFormats() const
{
	return UFbxFactory::GetFbxFormats(this);
}

TSharedPtr<FFbxNodeInfo> GetNodeInfoPtrById(TArray<TSharedPtr<FFbxNodeInfo>> &HierarchyInfo, uint64 SearchId)
{
	for (TSharedPtr<FFbxNodeInfo> NodeInfoPtr : HierarchyInfo)
	{
		if (NodeInfoPtr->UniqueId == SearchId)
		{
			return NodeInfoPtr;
		}
	}
	return nullptr;
}

void UFbxSceneImportFactory::ChangeFrontAxis(void* VoidFbxImporter, void* VoidSceneInfo, TSharedPtr<FFbxSceneInfo> SceneInfoPtr)
{
	UnFbx::FFbxImporter* FbxImporter = (UnFbx::FFbxImporter*)VoidFbxImporter;
	UnFbx::FbxSceneInfo* SceneInfo = (UnFbx::FbxSceneInfo*)VoidSceneInfo;

	FbxImporter->ConvertScene();
	//Adjust the root node with the new apply scene conversion
	FbxNode* RootNode = FbxImporter->Scene->GetRootNode();
	if (SceneInfo->HierarchyInfo.Num() > 0)
	{
		//Set the fbx data
		UnFbx::FbxNodeInfo &RootNodeInfo = SceneInfo->HierarchyInfo[0];
		check(RootNodeInfo.UniqueId == RootNode->GetUniqueID());
		RootNodeInfo.Transform = RootNode->EvaluateGlobalTransform();
		//Set the UE data
		TSharedPtr<FFbxNodeInfo> RootNodeInfoPtr = GetNodeInfoPtrById(SceneInfoPtr->HierarchyInfo, RootNodeInfo.UniqueId);
		if (RootNodeInfoPtr.IsValid())
		{
			RootNodeInfoPtr->Transform = FTransform::Identity;
			FbxVector4 NewLocalT = RootNodeInfo.Transform.GetT();
			FbxVector4 NewLocalS = RootNodeInfo.Transform.GetS();
			FbxQuaternion NewLocalQ = RootNodeInfo.Transform.GetQ();
			RootNodeInfoPtr->Transform.SetTranslation(UnFbx::FFbxDataConverter::ConvertPos(NewLocalT));
			RootNodeInfoPtr->Transform.SetScale3D(UnFbx::FFbxDataConverter::ConvertScale(NewLocalS));
			RootNodeInfoPtr->Transform.SetRotation(UnFbx::FFbxDataConverter::ConvertRotToQuat(NewLocalQ));
			for (int32 NodeIndex = 1; NodeIndex < SceneInfo->HierarchyInfo.Num(); ++NodeIndex)
			{
				UnFbx::FbxNodeInfo &LocalNodeInfo = SceneInfo->HierarchyInfo[NodeIndex];
				FbxNode *RealFbxNode = FindFbxNodeById(FbxImporter, nullptr, LocalNodeInfo.UniqueId);
				if (!RealFbxNode)
				{
					continue;
				}

				LocalNodeInfo.Transform = RealFbxNode->EvaluateLocalTransform();
				TSharedPtr<FFbxNodeInfo> LocalNodeInfoPtr = GetNodeInfoPtrById(SceneInfoPtr->HierarchyInfo, LocalNodeInfo.UniqueId);
				if (LocalNodeInfoPtr.IsValid())
				{
					LocalNodeInfoPtr->Transform = FTransform::Identity;
					NewLocalT = LocalNodeInfo.Transform.GetT();
					NewLocalS = LocalNodeInfo.Transform.GetS();
					NewLocalQ = LocalNodeInfo.Transform.GetQ();
					LocalNodeInfoPtr->Transform.SetTranslation(UnFbx::FFbxDataConverter::ConvertPos(NewLocalT));
					LocalNodeInfoPtr->Transform.SetScale3D(UnFbx::FFbxDataConverter::ConvertScale(NewLocalS));
					LocalNodeInfoPtr->Transform.SetRotation(UnFbx::FFbxDataConverter::ConvertRotToQuat(NewLocalQ));

					FString AttributeType = LocalNodeInfo.AttributeType;
					if (AttributeType.Compare(TEXT("eLight")) == 0)
					{
						//Add the z rotation of 90 degree locally for every light. Light direction differ from fbx to unreal 
						FRotator LightRotator(0.0f, 90.0f, 0.0f);
						FTransform LightTransform = FTransform(LightRotator);
						LocalNodeInfoPtr->Transform = LightTransform * LocalNodeInfoPtr->Transform;
					}
					if (AttributeType.Compare(TEXT("eCamera")) == 0)
					{
						//Add a roll of -90 degree locally for every cameras. Camera up vector differ from fbx to unreal
						FRotator CameraRotator(0.0f, 0.0f, -90.0f);
						FTransform CameraTransform = FTransform(CameraRotator);
						//Remove the scale of the node holding a camera (the mesh is provide by the engine and can be different in size)
						LocalNodeInfoPtr->Transform.SetScale3D(FVector(1.0f));
						LocalNodeInfoPtr->Transform = CameraTransform * LocalNodeInfoPtr->Transform;
					}
				}
			}
		}
	}
}

UObject* UFbxSceneImportFactory::FactoryCreateBinary
(
UClass*				Class,
UObject*			InParent,
FName				Name,
EObjectFlags		Flags,
UObject*			Context,
const TCHAR*		Type,
const uint8*&		Buffer,
const uint8*		BufferEnd,
FFeedbackContext*	Warn
)
{
	if (InParent == nullptr)
	{
		return nullptr;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UFbxSceneImportFactory::FactoryCreateBinary);

	NameOptionsMap.Reset();
	UWorld* World = GWorld;
	ULevel* CurrentLevel = World->GetCurrentLevel();
	//We will call other factory store the filename value since UFactory::CurrentFilename is static
	FString FbxImportFileName = UFactory::CurrentFilename;
	// Unselect all actors.
	GEditor->SelectNone(false, false);

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, Class, InParent, Name, Type);
	
	// logger for all error/warnings
	// this one prints all messages that are stored in FFbxImporter
	UnFbx::FFbxImporter* FbxImporter = UnFbx::FFbxImporter::GetInstance();

	UnFbx::FFbxLoggerSetter Logger(FbxImporter);

	Warn->BeginSlowTask(NSLOCTEXT("FbxSceneFactory", "BeginImportingFbxSceneTask", "Importing FBX scene"), true);
	
	GlobalImportSettings = FbxImporter->GetImportOptions();
	UnFbx::FBXImportOptions::ResetOptions(GlobalImportSettings);
	
	//Always convert the scene
	GlobalImportSettings->bConvertScene = true;
	GlobalImportSettings->bConvertSceneUnit = true;

	//Set the import option in importscene mode
	GlobalImportSettings->bImportScene = true;
	bool OriginalForceFrontXAxis = GlobalImportSettings->bForceFrontXAxis;
	//Read the fbx and store the hierarchy's information so we can reuse it after importing all the model in the fbx file
	if (!FbxImporter->ImportFromFile(*FbxImportFileName, Type, true))
	{
		// Log the error message and fail the import.
		Warn->Log(ELogVerbosity::Error, FbxImporter->GetErrorMessage());
		FbxImporter->ReleaseScene();
		FbxImporter = nullptr;
		Warn->EndSlowTask();
		GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, World);
		return nullptr;
	}

	//Make sure the Skeleton is null and not garbage, as we are importing the skeletalmesh for the first time we do not need any skeleton
	GlobalImportSettings->SkeletonForAnimation = nullptr;
	GlobalImportSettings->PhysicsAsset = nullptr;

	FString PackageName = "";
	InParent->GetName(PackageName);
	Path = FPaths::GetPath(PackageName);

	UnFbx::FbxSceneInfo SceneInfo;
	//Read the scene and found all instance with their scene information.
	FbxImporter->GetSceneInfo(FbxImportFileName, SceneInfo, true);
	
	GlobalImportSettingsReference = new UnFbx::FBXImportOptions();
	SFbxSceneOptionWindow::CopyFbxOptionsToFbxOptions(GlobalImportSettings, GlobalImportSettingsReference);

	//Convert old structure to the new scene export structure
	TSharedPtr<FFbxSceneInfo> SceneInfoPtr = ConvertSceneInfo(FbxImporter, &SceneInfo);
	
	//Get import material info
	ExtractMaterialInfo(FbxImporter, SceneInfoPtr);

	if(!GetFbxSceneImportOptions(FbxImporter
		, SceneInfoPtr
		, GlobalImportSettingsReference
		, SceneImportOptions
		, SceneImportOptionsStaticMesh
		, NameOptionsMap
		, SceneImportOptionsSkeletalMesh
		, Path))
	{
		//User cancel the scene import
		ImportWasCancel = true;
		FbxImporter->ReleaseScene();
		FbxImporter = nullptr;
		GlobalImportSettings = nullptr;
		GlobalImportSettingsReference = nullptr;
		Warn->EndSlowTask();
		GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, World);
		return nullptr;
	}


	SFbxSceneOptionWindow::CopyFbxOptionsToFbxOptions(GlobalImportSettingsReference, GlobalImportSettings);

	//Convert the scene to the correct axis system. Option like force front X or ConvertScene affect the scene conversion
	//We need to get the new convert transform
	if(OriginalForceFrontXAxis != GlobalImportSettings->bForceFrontXAxis)
	{
		ChangeFrontAxis(FbxImporter, &SceneInfo, SceneInfoPtr);
	}

	FillSceneHierarchyPath(SceneInfoPtr);

	ReimportData = CreateReImportAsset(Path, FbxImportFileName, SceneImportOptions, SceneInfoPtr, NameOptionsMap);
	if (ReimportData == nullptr)
	{
		//Cannot save the reimport data
		const FText CreateReimportDataFailed = LOCTEXT("CreateReimportDataFailed", "Failed to create the re import data asset, which will make impossible the re import of this scene.\nLook in the logs to see the reason.\nPress Ok to continue or Cancel to abort the import process");
		if (FMessageDialog::Open(EAppMsgType::OkCancel, CreateReimportDataFailed) == EAppReturnType::Cancel)
		{
			//User cancel the scene import
			ImportWasCancel = true;
			FbxImporter->ReleaseScene();
			FbxImporter = nullptr;
			GlobalImportSettings = nullptr;
			Warn->EndSlowTask();
			GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, World);
			return nullptr;
		}
	}

	//We are a scene import set the flag for the reimport factory for both static mesh and skeletal mesh
	StaticMeshImportData->bImportAsScene = true;
	StaticMeshImportData->FbxSceneImportDataReference = ReimportData;

	SkeletalMeshImportData->bImportAsScene = true;
	SkeletalMeshImportData->FbxSceneImportDataReference = ReimportData;

	AnimSequenceImportData->bImportAsScene = true;
	AnimSequenceImportData->FbxSceneImportDataReference = ReimportData;

	//Get the scene root node
	FbxNode* RootNodeToImport = FbxImporter->Scene->GetRootNode();
	
	// For animation and static mesh we assume there is at lease one interesting node by default
	int32 InterestingNodeCount = 1;

	AllNewAssets.Empty();

	int32 NodeIndex = 0;

	//////////////////////////////////////////////////////////////////////////
	//IMPORT ALL SKELETAL MESH
	ImportAllSkeletalMesh(RootNodeToImport, FbxImporter, Flags, NodeIndex, InterestingNodeCount, SceneInfoPtr);
	//////////////////////////////////////////////////////////////////////////

	//////////////////////////////////////////////////////////////////////////
	// IMPORT ALL STATIC MESH
	ImportAllStaticMesh(RootNodeToImport, FbxImporter, Flags, NodeIndex, InterestingNodeCount, SceneInfoPtr);
	//////////////////////////////////////////////////////////////////////////

	UObject *ReturnObject = nullptr;
	for (auto ItAsset = AllNewAssets.CreateIterator(); ItAsset; ++ItAsset)
	{
		UObject *AssetObject = ItAsset.Value();
		if (AssetObject)
		{
			if (ReturnObject == nullptr)
			{
				//Set the first import object as the return object to prevent false error from the caller of this factory
				ReturnObject = AssetObject;
			}
			if (AssetObject->IsA(UStaticMesh::StaticClass()) || AssetObject->IsA(USkeletalMesh::StaticClass()))
			{
				//Mark the mesh as modified so the render will draw the mesh correctly
				AssetObject->Modify();
				AssetObject->PostEditChange();
			}
		}
	}

	//////////////////////////////////////////////////////////////////////////
	// CREATE AND PLACE ACTOR
	// Instantiate all the scene hierarchy in the current level with link to previous created objects
	// go through the hierarchy and instantiate actor in the current level	
	switch (SceneImportOptions->HierarchyType)
	{
		case EFBXSceneOptionsCreateHierarchyType::FBXSOCHT_CreateLevelActors:
		{
			CreateLevelActorHierarchy(SceneInfoPtr);
		}
		break;
		case EFBXSceneOptionsCreateHierarchyType::FBXSOCHT_CreateActorComponents:
		case EFBXSceneOptionsCreateHierarchyType::FBXSOCHT_CreateBlueprint:
		{
			AActor *HierarchyActor = CreateActorComponentsHierarchy(SceneInfoPtr);
			//If the user want to export to a BP replace the container actor with a BP link
			if (SceneImportOptions->HierarchyType == EFBXSceneOptionsCreateHierarchyType::FBXSOCHT_CreateBlueprint && HierarchyActor != nullptr)
			{
				//The location+name of the BP is the user select content path + fbx base filename
				FString FullnameBP = Path + TEXT("/FbxScene_") + FPaths::GetBaseFilename(UFactory::CurrentFilename);
				FullnameBP = UPackageTools::SanitizePackageName(FullnameBP);
				FString AssetName = TEXT("FbxScene_") + FPaths::GetBaseFilename(UFactory::CurrentFilename);
				UPackage *Pkg = CreatePackageForNode(FullnameBP, AssetName);

				//Create the blueprint from the actor and replace the actor with an instance of the created blueprint
				FKismetEditorUtilities::FCreateBlueprintFromActorParams Params;
				Params.bReplaceActor = true;
				Params.bKeepMobility = true;
				UBlueprint* SceneBlueprint = FKismetEditorUtilities::CreateBlueprintFromActor(Pkg->GetName(), HierarchyActor, Params);

				if (SceneBlueprint != nullptr && ReimportData != nullptr)
				{
					//let the scene blueprint be the return object for this import
					ReturnObject = SceneBlueprint;
					//Set the blueprint path name in the re import scene data asset, this will allow re import to find the original import blueprint
					ReimportData->BluePrintFullName = SceneBlueprint->GetPathName();
				}
				GEngine->BroadcastLevelActorListChanged();
			}
		}
		break;
	}
	
	//If there is no content asset create return the fbx scene import data
	//This can happen if we only import actor in the scene like lights and camera
	if (ReturnObject == nullptr)
	{
		ReturnObject = ReimportData;
	}
	//Release the FbxImporter 
	FbxImporter->ReleaseScene();
	FbxImporter = nullptr;
	GlobalImportSettings = nullptr;
	GlobalImportSettingsReference = nullptr;

	ReimportData = nullptr;

	Warn->EndSlowTask();
	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, World);

	return ReturnObject;
}

bool UFbxSceneImportFactory::SetStaticMeshComponentOverrideMaterial(UStaticMeshComponent* StaticMeshComponent, TSharedPtr<FFbxNodeInfo> NodeInfo)
{
	bool bOverrideMaterial = false;
	UStaticMesh *StaticMesh = StaticMeshComponent->GetStaticMesh();
	if (StaticMesh->GetStaticMaterials().Num() == NodeInfo->Materials.Num())
	{
		for (int32 MaterialIndex = 0; MaterialIndex < NodeInfo->Materials.Num(); ++MaterialIndex)
		{
			TSharedPtr<FFbxMaterialInfo> MaterialInfo = NodeInfo->Materials[MaterialIndex];
			UMaterialInterface *MaterialInterface = Cast<UMaterialInterface>(MaterialInfo->GetContentObject());
			if (MaterialInterface != nullptr && StaticMesh->GetMaterial(MaterialIndex) != MaterialInterface)
			{
				bOverrideMaterial = true;
				break;
			}
		}
		if (bOverrideMaterial)
		{
			for (int32 MaterialIndex = 0; MaterialIndex < NodeInfo->Materials.Num(); ++MaterialIndex)
			{
				TSharedPtr<FFbxMaterialInfo> MaterialInfo = NodeInfo->Materials[MaterialIndex];
				UMaterialInterface *MaterialInterface = Cast<UMaterialInterface>(MaterialInfo->GetContentObject());
				if (MaterialInterface != nullptr && StaticMesh->GetMaterial(MaterialIndex) != MaterialInterface)
				{
					StaticMeshComponent->SetMaterial(MaterialIndex, MaterialInterface);
				}
			}
		}
	}
	return bOverrideMaterial;
}

USceneComponent *CreateCameraComponent(AActor *ParentActor, TSharedPtr<FFbxCameraInfo> CameraInfo)
{
	UCineCameraComponent *CameraComponent = NewObject<UCineCameraComponent>(ParentActor, *(CameraInfo->Name));
	CameraComponent->SetProjectionMode(CameraInfo->ProjectionPerspective ? ECameraProjectionMode::Perspective : ECameraProjectionMode::Orthographic);
	CameraComponent->SetAspectRatio(CameraInfo->AspectWidth / CameraInfo->AspectHeight);
	CameraComponent->SetOrthoNearClipPlane(CameraInfo->NearPlane);
	CameraComponent->SetOrthoFarClipPlane(CameraInfo->FarPlane);
	CameraComponent->SetOrthoWidth(CameraInfo->AspectWidth);
	CameraComponent->SetFieldOfView(CameraInfo->FieldOfView);
	CameraComponent->Filmback.SensorWidth = FUnitConversion::Convert(CameraInfo->ApertureWidth, EUnit::Inches, EUnit::Millimeters);
	CameraComponent->Filmback.SensorHeight = FUnitConversion::Convert(CameraInfo->ApertureHeight, EUnit::Inches, EUnit::Millimeters);
	CameraComponent->LensSettings.MaxFocalLength = CameraInfo->FocalLength;
	CameraComponent->LensSettings.MinFocalLength = CameraInfo->FocalLength;
	CameraComponent->FocusSettings.FocusMethod = ECameraFocusMethod::DoNotOverride;

	return CameraComponent;
}

USceneComponent *CreateLightComponent(AActor *ParentActor, TSharedPtr<FFbxLightInfo> LightInfo)
{
	ULightComponent *LightComponent = nullptr;
	switch (LightInfo->Type)
	{
	case 0:
	{
		//Point light
		UPointLightComponent *PointLightComponent = NewObject<UPointLightComponent>(ParentActor, *(LightInfo->Name));
		PointLightComponent->SetAttenuationRadius(LightInfo->EnableFarAttenuation ? LightInfo->FarAttenuationEnd : 16384.0f);
		LightComponent = static_cast<ULightComponent*>(PointLightComponent);
		LightComponent->SetIntensity(LightComponent->Intensity * LightInfo->Intensity / 100.0f);
	}
		break;
	case 1:
	{
		//Directional light
		UDirectionalLightComponent *DirectionalLightComponent = NewObject<UDirectionalLightComponent>(ParentActor, *(LightInfo->Name));
		LightComponent = static_cast<ULightComponent*>(DirectionalLightComponent);
		//We cannot convert fbx value to unreal value so we kept the default object value
		LightComponent->SetIntensity(LightComponent->Intensity * LightInfo->Intensity / 100.0f);
	}
		break;
	case 2:
	{
		//Spot light
		USpotLightComponent *SpotLightComponent = NewObject<USpotLightComponent>(ParentActor, *(LightInfo->Name));
		SpotLightComponent->SetInnerConeAngle(LightInfo->InnerAngle/2.0f);
		SpotLightComponent->SetOuterConeAngle(LightInfo->OuterAngle/2.0f);
		SpotLightComponent->SetAttenuationRadius(LightInfo->EnableFarAttenuation ? LightInfo->FarAttenuationEnd : 16384.0f);
		LightComponent = static_cast<ULightComponent*>(SpotLightComponent);
		
		LightComponent->SetIntensity(LightComponent->Intensity * LightInfo->Intensity / 100.0f);
	}
		break;
	case 3:
	case 4:
		return nullptr;
	}
	LightComponent->SetLightColor(LightInfo->Color);
	LightComponent->SetCastShadows(LightInfo->CastShadow);
	return LightComponent;
}

FVector GetParentPivotAccumulation(TSharedPtr<FFbxNodeInfo> NodeInfo, TSharedPtr<FFbxSceneInfo> SceneInfoPtr, FTransform &RootTransform)
{
	TArray<TSharedPtr<FFbxNodeInfo>> ParentHierarchy;
	FVector PivotAccumulation(0.0f);
	TSharedPtr<FFbxNodeInfo> ParentNodeInfo = NodeInfo->ParentNodeInfo;
	while (ParentNodeInfo.IsValid())
	{
		ParentHierarchy.Insert(ParentNodeInfo, 0);
		ParentNodeInfo = ParentNodeInfo->ParentNodeInfo;
	}
	FTransform CurrentGlobalMatrix;
	for (TSharedPtr<FFbxNodeInfo> ParentNode : ParentHierarchy)
	{
		FTransform LocalTransformAdjusted = ParentNode->Transform;
		FVector PivotLocation(0.0f);
		if (ParentNode->AttributeInfo.IsValid())
		{
			for (TSharedPtr<FFbxNodeInfo> NodeInfoIter : SceneInfoPtr->HierarchyInfo)
			{
				if (NodeInfoIter->UniqueId == ParentNode->AttributeInfo->PivotNodeUid)
				{
					PivotLocation = NodeInfoIter->PivotRotation;
					break;
				}
			}
		}
		FTransform LocalTransform = ParentNode->Transform;
		if (!PivotLocation.IsNearlyZero())
		{
			FTransform ParentPivotTransform;
			ParentPivotTransform.SetLocation(PivotLocation);
			FTransform AlmostNextCurrentTransform = LocalTransform *CurrentGlobalMatrix;
			//Get the final matrix with pivot
			LocalTransform = ParentPivotTransform * LocalTransform;
			CurrentGlobalMatrix = LocalTransform * CurrentGlobalMatrix;
			ParentPivotTransform = CurrentGlobalMatrix * AlmostNextCurrentTransform.Inverse();
			PivotAccumulation = ParentPivotTransform.GetLocation();
		}
		else
		{
			CurrentGlobalMatrix = LocalTransform * CurrentGlobalMatrix;
		}
	}
	return PivotAccumulation;
}

void UFbxSceneImportFactory::CreateLevelActorHierarchy(TSharedPtr<FFbxSceneInfo> SceneInfoPtr)
{
	EComponentMobility::Type MobilityType = SceneImportOptions->bImportAsDynamic ? EComponentMobility::Movable : EComponentMobility::Static;
	TMap<uint64, AActor *> NewActorNameMap;
	FTransform RootTransform = FTransform::Identity;
	bool bSelectActor = true;
	//////////////////////////////////////////////////////////////////////////
	// iterate the whole hierarchy and create all actors
	for (TSharedPtr<FFbxNodeInfo>  NodeInfo : SceneInfoPtr->HierarchyInfo)
	{
		if (NodeInfo->NodeName.Compare("RootNode") == 0)
		{
			RootTransform = NodeInfo->Transform;
			continue;
		}

		//Export only the node that are mark for export
		if (!NodeInfo->bImportNode)
		{
			continue;
		}
		
		TSharedPtr<FFbxNodeInfo> LODParentNodeInfo = nullptr;
		if (NodeInfo->AttributeType.Compare(TEXT("eMesh")) == 0)
		{
			LODParentNodeInfo = FFbxSceneInfo::RecursiveFindLODParentNode(NodeInfo);
		}

		//Find the asset that link with this node attribute
		UObject *AssetToPlace = (NodeInfo->AttributeInfo.IsValid() && AllNewAssets.Contains(NodeInfo->AttributeInfo)) ? AllNewAssets[NodeInfo->AttributeInfo] : nullptr;
		
		bool IsSkeletalMesh = false;
		//create actor
		AActor *PlacedActor = nullptr;
		if (AssetToPlace != nullptr)
		{
			//Create an actor from the asset
			//default flag is RF_Transactional;
			PlacedActor = FActorFactoryAssetProxy::AddActorForAsset(AssetToPlace, bSelectActor);
			
			//Set the actor override material
			if (PlacedActor->IsA(AStaticMeshActor::StaticClass()))
			{
				UStaticMeshComponent *StaticMeshComponent = Cast<UStaticMeshComponent>(PlacedActor->GetComponentByClass(UStaticMeshComponent::StaticClass()));
				SetStaticMeshComponentOverrideMaterial(StaticMeshComponent, NodeInfo);
			}
			IsSkeletalMesh = (AssetToPlace->GetClass() == USkeletalMesh::StaticClass());
		}
		else if (IsEmptyAttribute(NodeInfo->AttributeType) || NodeInfo->AttributeType.Compare("eMesh") == 0 || NodeInfo->AttributeUniqueId != INVALID_UNIQUE_ID)
		{
			if (NodeInfo->AttributeType.Compare("eMesh") == 0)
			{
				bool bIsSubSkeletalMesh = true;
				for (TSharedPtr<FFbxMeshInfo> MeshInfo : SceneInfoPtr->MeshInfo)
				{
					if (!MeshInfo->bIsSkelMesh && NodeInfo->AttributeUniqueId == MeshInfo->UniqueId)
					{
						bIsSubSkeletalMesh = false;
						break;
					}
				}
				if (bIsSubSkeletalMesh == true)
				{
					continue;
				}
			}
			//Create an empty actor if the node is an empty attribute or the attribute is a mesh(static mesh or skeletal mesh) that was not export
			UActorFactory* Factory = GEditor->FindActorFactoryByClass(UActorFactoryEmptyActor::StaticClass());
			FAssetData EmptyActorAssetData = FAssetData(Factory->GetDefaultActorClass(FAssetData()));
			//This is a group create an empty actor that just have a transform
			UObject* EmptyActorAsset = EmptyActorAssetData.GetAsset();
			//Place an empty actor
			PlacedActor = FActorFactoryAssetProxy::AddActorForAsset(EmptyActorAsset, bSelectActor);
			USceneComponent* RootComponent = nullptr;
			if(NodeInfo->AttributeType.Compare("eLight") == 0)
			{
				TSharedPtr<FFbxLightInfo> LightInfo = *SceneInfoPtr->LightInfo.Find(NodeInfo->AttributeUniqueId);
				RootComponent = CreateLightComponent(PlacedActor, LightInfo);
			}
			else if (NodeInfo->AttributeType.Compare("eCamera") == 0)
			{
				TSharedPtr<FFbxCameraInfo> CameraInfo = *SceneInfoPtr->CameraInfo.Find(NodeInfo->AttributeUniqueId);
				RootComponent = CreateCameraComponent(PlacedActor, CameraInfo);
			}
			
			if(RootComponent == nullptr)
			{
				if (LODParentNodeInfo.IsValid())
				{
					//This is not LOD index 0, don't export the transform. Lod 0 should have an asset to place
					continue;
				}
				RootComponent = NewObject<USceneComponent>(PlacedActor, USceneComponent::GetDefaultSceneRootVariableName());
			}
			RootComponent->Mobility = MobilityType;
			RootComponent->bVisualizeComponent = true;
			PlacedActor->SetRootComponent(RootComponent);
			PlacedActor->AddInstanceComponent(RootComponent);
			RootComponent->RegisterComponent();
		}
		else
		{
			//TODO log which fbx attribute we cannot create an actor from
		}

		if (PlacedActor != nullptr)
		{
			PlacedActor->SetFlags(RF_Transactional);
			//Rename the actor correctly
			//When importing a scene we don't want to change the actor name even if there is similar label already existing
			PlacedActor->SetActorLabel(NodeInfo->NodeName);

			USceneComponent* RootComponent = PlacedActor->GetRootComponent();
			if (RootComponent)
			{
				RootComponent->SetFlags(RF_Transactional);
				//Set the mobility
				RootComponent->Mobility = MobilityType;
				//Map the new actor name with the old name in case the name is changing
				NewActorNameMap.Add(NodeInfo->UniqueId, PlacedActor);
				uint64 ParentUniqueId = NodeInfo->ParentNodeInfo.IsValid() ? NodeInfo->ParentNodeInfo->UniqueId : 0;
				if (LODParentNodeInfo.IsValid())
				{
					ParentUniqueId = LODParentNodeInfo->UniqueId;
				}
				AActor *ParentActor = nullptr;
				//If there is a parent we must set the parent actor
				if (NewActorNameMap.Contains(ParentUniqueId))
				{
					ParentActor = *NewActorNameMap.Find(ParentUniqueId);
					if (ParentActor != nullptr)
					{
						USceneComponent* ParentRootComponent = ParentActor->GetRootComponent();
						if (ParentRootComponent)
						{
							if (GEditor->CanParentActors(ParentActor, PlacedActor))
							{
								GEditor->ParentActors(ParentActor, PlacedActor, NAME_None);
							}
						}
					}
				}
				//Find the pivot location
				FVector PivotLocation(0.0f);
				FVector ParentPivotAccumulation(0.0f);
				if (!IsSkeletalMesh && GlobalImportSettings->bBakePivotInVertex)
				{
					ParentPivotAccumulation -= GetParentPivotAccumulation(NodeInfo, SceneInfoPtr, RootTransform);
					if (NodeInfo->AttributeInfo.IsValid() && NodeInfo->AttributeInfo->PivotNodeUid != INVALID_UNIQUE_ID)
					{
						for (TSharedPtr<FFbxNodeInfo> NodeInfoIter : SceneInfoPtr->HierarchyInfo)
						{
							if (NodeInfoIter->UniqueId == NodeInfo->AttributeInfo->PivotNodeUid)
							{
								PivotLocation = NodeInfoIter->PivotRotation;
								break;
							}
						}
					}
				}

				//Apply the hierarchy local transform to the root component
				ApplyTransformToComponent(RootComponent, &(NodeInfo->Transform), ParentActor == nullptr ? &RootTransform : nullptr, PivotLocation, ParentPivotAccumulation);
				//Notify people that the component get created/changed
				RootComponent->PostEditChange();
			}
		}
		//We select only the first actor
		bSelectActor = false;
	}
	// End of iteration of the hierarchy
	//////////////////////////////////////////////////////////////////////////
}

AActor *UFbxSceneImportFactory::CreateActorComponentsHierarchy(TSharedPtr<FFbxSceneInfo> SceneInfoPtr)
{
	FString FbxImportFileName = UFactory::CurrentFilename;
	UBlueprint *NewBluePrintActor = nullptr;
	AActor *RootActorContainer = nullptr;
	FString FilenameBase = FbxImportFileName.IsEmpty() ? TEXT("TransientToBlueprintActor") : FPaths::GetBaseFilename(FbxImportFileName);
	USceneComponent *ActorRootComponent = nullptr;
	TMap<uint64, USceneComponent *> NewSceneComponentNameMap;
	EComponentMobility::Type MobilityType = SceneImportOptions->bImportAsDynamic ? EComponentMobility::Movable : EComponentMobility::Static;
	
	//////////////////////////////////////////////////////////////////////////
	//Create the Actor where to put components in
	UActorFactory* Factory = GEditor->FindActorFactoryByClass(UActorFactoryEmptyActor::StaticClass());
	FAssetData EmptyActorAssetData = FAssetData(Factory->GetDefaultActorClass(FAssetData()));
	//This is a group create an empty actor that just have a transform
	UObject* EmptyActorAsset = EmptyActorAssetData.GetAsset();
	//Place an empty actor
	RootActorContainer = FActorFactoryAssetProxy::AddActorForAsset(EmptyActorAsset, false);
	check(RootActorContainer != nullptr);
	ActorRootComponent = NewObject<USceneComponent>(RootActorContainer, USceneComponent::GetDefaultSceneRootVariableName());
	check(ActorRootComponent != nullptr);
	ActorRootComponent->Mobility = MobilityType;
	ActorRootComponent->bVisualizeComponent = true;
	RootActorContainer->SetRootComponent(ActorRootComponent);
	RootActorContainer->AddInstanceComponent(ActorRootComponent);
	ActorRootComponent->RegisterComponent();
	RootActorContainer->SetActorLabel(FilenameBase);
	RootActorContainer->SetFlags(RF_Transactional);
	ActorRootComponent->SetFlags(RF_Transactional);

	//////////////////////////////////////////////////////////////////////////
	// iterate the whole hierarchy and create all component
	FTransform RootTransform = FTransform::Identity;
	for (TSharedPtr<FFbxNodeInfo> NodeInfo : SceneInfoPtr->HierarchyInfo)
	{
		//Set the root transform if its the root node and skip the node
		//The root transform will be use for every node under the root node
		if (NodeInfo->NodeName.Compare("RootNode") == 0)
		{
			RootTransform = NodeInfo->Transform;
			continue;
		}
		//Export only the node that are mark for export
		if (!NodeInfo->bImportNode)
		{
			continue;
		}

		TSharedPtr<FFbxNodeInfo> LODParentNodeInfo = nullptr;
		if (NodeInfo->AttributeType.Compare(TEXT("eMesh")) == 0)
		{
			LODParentNodeInfo = FFbxSceneInfo::RecursiveFindLODParentNode(NodeInfo);
		}
		//Find the asset that link with this node attribute
		UObject *AssetToPlace = (NodeInfo->AttributeInfo.IsValid() && AllNewAssets.Contains(NodeInfo->AttributeInfo)) ? AllNewAssets[NodeInfo->AttributeInfo] : nullptr;

		bool IsSkeletalMesh = false;
		//Create the component where the type depend on the asset point by the component
		//In case there is no asset we create a SceneComponent
		USceneComponent* SceneComponent = nullptr;
		if (AssetToPlace != nullptr)
		{
			if (AssetToPlace->GetClass() == UStaticMesh::StaticClass())
			{
				//Component will be rename later
				UStaticMeshComponent* StaticMeshComponent = NewObject<UStaticMeshComponent>(RootActorContainer, NAME_None);
				StaticMeshComponent->SetStaticMesh(Cast<UStaticMesh>(AssetToPlace));
				StaticMeshComponent->DepthPriorityGroup = SDPG_World;
				SetStaticMeshComponentOverrideMaterial(StaticMeshComponent, NodeInfo);
				SceneComponent = StaticMeshComponent;
				SceneComponent->Mobility = MobilityType;
			}
			else if (AssetToPlace->GetClass() == USkeletalMesh::StaticClass())
			{
				//Component will be rename later
				USkeletalMeshComponent* SkeletalMeshComponent = NewObject<USkeletalMeshComponent>(RootActorContainer, NAME_None);
				SkeletalMeshComponent->SetSkeletalMesh(Cast<USkeletalMesh>(AssetToPlace));
				SkeletalMeshComponent->DepthPriorityGroup = SDPG_World;
				SceneComponent = SkeletalMeshComponent;
				SceneComponent->Mobility = MobilityType;
				IsSkeletalMesh = true;
			}
		}
		else if (IsEmptyAttribute(NodeInfo->AttributeType) || NodeInfo->AttributeType.Compare("eMesh") == 0 || NodeInfo->AttributeUniqueId != INVALID_UNIQUE_ID)
		{
			if (NodeInfo->AttributeType.Compare("eMesh") == 0)
			{
				bool bIsSubSkeletalMesh = true;
				for (TSharedPtr<FFbxMeshInfo> MeshInfo : SceneInfoPtr->MeshInfo)
				{
					if (!MeshInfo->bIsSkelMesh && NodeInfo->AttributeUniqueId == MeshInfo->UniqueId)
					{
						bIsSubSkeletalMesh = false;
						break;
					}
				}
				if (bIsSubSkeletalMesh == true)
				{
					continue;
				}
			}

			if (NodeInfo->AttributeType.Compare("eLight") == 0 && SceneInfoPtr->LightInfo.Contains(NodeInfo->AttributeUniqueId))
			{
				TSharedPtr<FFbxLightInfo> LightInfo = *SceneInfoPtr->LightInfo.Find(NodeInfo->AttributeUniqueId);
				SceneComponent = CreateLightComponent(RootActorContainer, LightInfo);
			}
			else if (NodeInfo->AttributeType.Compare("eCamera") == 0 && SceneInfoPtr->CameraInfo.Contains(NodeInfo->AttributeUniqueId))
			{
				TSharedPtr<FFbxCameraInfo> CameraInfo = *SceneInfoPtr->CameraInfo.Find(NodeInfo->AttributeUniqueId);
				SceneComponent = CreateCameraComponent(RootActorContainer, CameraInfo);
			}
			
			if(SceneComponent == nullptr)
			{
				if (LODParentNodeInfo.IsValid())
				{
					//This is not LOD index 0, don't export the transform. Lod 0 should have an asset to place
					continue;
				}
				SceneComponent = NewObject<USceneComponent>(RootActorContainer, NAME_None);
			}
			//Component will be rename later
			SceneComponent->Mobility = MobilityType;
		}
		else
		{
			continue;
		}

		//Make sure undo/redo is working
		SceneComponent->SetFlags(RF_Transactional);

		//////////////////////////////////////////////////////////////////////////
		//Make sure scenecomponent name are unique in the hierarchy of the outer
		FString NewUniqueName = NodeInfo->NodeName;
		if (!SceneComponent->Rename(*NewUniqueName, nullptr, REN_Test))
		{
			NewUniqueName = MakeUniqueObjectName(RootActorContainer, USceneComponent::StaticClass(), FName(*NodeInfo->NodeName)).ToString();
		}
		SceneComponent->Rename(*NewUniqueName, nullptr, REN_DontCreateRedirectors);

		//Add the component to the owner actor and register it
		RootActorContainer->AddInstanceComponent(SceneComponent);
		SceneComponent->RegisterComponent();

		//Add the component to the temporary map so we can retrieve it later when we search for parent
		NewSceneComponentNameMap.Add(NodeInfo->UniqueId, SceneComponent);

		//Find the parent component by unique ID and attach(as child) the newly created scenecomponent
		//Attach the component to the rootcomponent if we dont find any parent component
		uint64 ParentUniqueId = NodeInfo->ParentNodeInfo.IsValid() ? NodeInfo->ParentNodeInfo->UniqueId : 0;
		if (LODParentNodeInfo.IsValid())
		{
			ParentUniqueId = LODParentNodeInfo->UniqueId;
		}
		USceneComponent *ParentRootComponent = nullptr;
		if (NewSceneComponentNameMap.Contains(ParentUniqueId))
		{
			ParentRootComponent = *NewSceneComponentNameMap.Find(ParentUniqueId);
			if (ParentRootComponent != nullptr)
			{
				SceneComponent->AttachToComponent(ParentRootComponent, FAttachmentTransformRules::KeepWorldTransform);
			}
		}
		else
		{
			SceneComponent->AttachToComponent(ActorRootComponent, FAttachmentTransformRules::KeepWorldTransform);
		}

		//Find the pivot location
		FVector PivotLocation(0.0f);
		FVector ParentPivotAccumulation(0.0f);
		if (!IsSkeletalMesh && GlobalImportSettings->bBakePivotInVertex)
		{
			ParentPivotAccumulation -= GetParentPivotAccumulation(NodeInfo, SceneInfoPtr, RootTransform);
			if (NodeInfo->AttributeInfo.IsValid() && NodeInfo->AttributeInfo->PivotNodeUid != INVALID_UNIQUE_ID)
			{
				for (TSharedPtr<FFbxNodeInfo> NodeInfoIter : SceneInfoPtr->HierarchyInfo)
				{
					if (NodeInfoIter->UniqueId == NodeInfo->AttributeInfo->PivotNodeUid)
					{
						PivotLocation = NodeInfoIter->PivotRotation;
						break;
					}
				}
			}
		}

		//Apply the local transform to the scene component
		ApplyTransformToComponent(SceneComponent, &(NodeInfo->Transform), ParentRootComponent != nullptr ? nullptr : &RootTransform, PivotLocation, ParentPivotAccumulation);
		//Notify people that the component get created/changed
		SceneComponent->PostEditChange();
	}
	// End of iteration of the hierarchy
	//////////////////////////////////////////////////////////////////////////

	return RootActorContainer;
}

void UFbxSceneImportFactory::ApplyTransformToComponent(USceneComponent *SceneComponent, FTransform *LocalTransform, FTransform *PreMultiplyTransform, FVector &PivotLocation, FVector &ParentPivotAccumulation)
{
	check(SceneComponent);
	check(LocalTransform);
	FTransform LocalTransformAdjusted = (*LocalTransform);
	if (GlobalImportSettings->bBakePivotInVertex && (!PivotLocation.IsNearlyZero() || !ParentPivotAccumulation.IsNearlyZero()))
	{
		FTransform PivotTransform;
		PivotTransform.SetLocation(ParentPivotAccumulation);
		LocalTransformAdjusted = LocalTransformAdjusted * PivotTransform;
		PivotTransform.SetIdentity();
		PivotTransform.SetLocation(PivotLocation);
		LocalTransformAdjusted = PivotTransform * LocalTransformAdjusted;
	}
	//In case there is no parent we must multiply the root transform
	if (PreMultiplyTransform != nullptr)
	{
		FTransform OutTransform = FTransform::Identity;
		FTransform::Multiply(&OutTransform, &LocalTransformAdjusted, PreMultiplyTransform);
		SceneComponent->SetRelativeTransform(OutTransform);
	}
	else
	{
		SceneComponent->SetRelativeTransform(LocalTransformAdjusted);
	}
}
void UFbxSceneImportFactory::ApplyMeshInfoFbxOptions(TSharedPtr<FFbxMeshInfo> MeshInfo)
{
	if (!MeshInfo.IsValid())
	{
		//Use the default options
		SFbxSceneOptionWindow::CopyFbxOptionsToFbxOptions(GlobalImportSettingsReference, GlobalImportSettings);
		SFbxSceneOptionWindow::CopyFbxOptionsToSkeletalMeshOptions(GlobalImportSettingsReference, SceneImportOptionsSkeletalMesh);
		SFbxSceneOptionWindow::CopyFbxOptionsToStaticMeshOptions(GlobalImportSettingsReference, SceneImportOptionsStaticMesh);
	}
	else
	{
		UnFbx::FBXImportOptions* OverrideImportSettings = GetOptionsFromName(MeshInfo->OptionName);
		if (OverrideImportSettings != nullptr)
		{
			//Use the override options
			SFbxSceneOptionWindow::CopyFbxOptionsToFbxOptions(OverrideImportSettings, GlobalImportSettings);
			SFbxSceneOptionWindow::CopyFbxOptionsToSkeletalMeshOptions(OverrideImportSettings, SceneImportOptionsSkeletalMesh);
			SFbxSceneOptionWindow::CopyFbxOptionsToStaticMeshOptions(OverrideImportSettings, SceneImportOptionsStaticMesh);
		}
		else
		{
			//Use the default options if we found no options
			SFbxSceneOptionWindow::CopyFbxOptionsToFbxOptions(GlobalImportSettingsReference, GlobalImportSettings);
			SFbxSceneOptionWindow::CopyFbxOptionsToSkeletalMeshOptions(GlobalImportSettingsReference, SceneImportOptionsSkeletalMesh);
			SFbxSceneOptionWindow::CopyFbxOptionsToStaticMeshOptions(GlobalImportSettingsReference, SceneImportOptionsStaticMesh);
		}
	}
	SceneImportOptionsSkeletalMesh->FillSkeletalMeshInmportData(SkeletalMeshImportData, AnimSequenceImportData, SceneImportOptions);
	SceneImportOptionsStaticMesh->FillStaticMeshInmportData(StaticMeshImportData, SceneImportOptions);
}

UObject* UFbxSceneImportFactory::ImportOneSkeletalMesh(void* VoidRootNodeToImport, void* VoidFbxImporter, TSharedPtr<FFbxSceneInfo> SceneInfo, EObjectFlags Flags, TArray<void*> &VoidNodeArray, int32 &TotalNumNodes)
{
	FbxNode *RootNodeToImport = (FbxNode *)VoidRootNodeToImport;
	UnFbx::FFbxImporter* FbxImporter = (UnFbx::FFbxImporter*)VoidFbxImporter;
	TArray<FbxNode*> NodeArray;
	for (void *VoidNode : VoidNodeArray)
	{
		FbxNode* Node = (FbxNode*)VoidNode;
		NodeArray.Add(Node);
	}
	UObject* NewObject = nullptr;
	UPackage* Pkg = nullptr;
	TotalNumNodes += NodeArray.Num();
	TSharedPtr<FFbxNodeInfo> RootNodeInfo;
	if (TotalNumNodes > 0)
	{
		FbxNode* RootNodeArrayNode = NodeArray[0];
		if (RootNodeArrayNode->GetNodeAttribute() && RootNodeArrayNode->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup)
		{
			//In case we have a LOD group we must have only one node in the array
			check(NodeArray.Num() == 1);
			RootNodeArrayNode = FbxImporter->FindLODGroupNode(RootNodeArrayNode, 0);
			if (RootNodeArrayNode == nullptr)
			{
				return nullptr;
			}
		}
		if (!FindSceneNodeInfo(SceneInfo, RootNodeArrayNode->GetUniqueID(), RootNodeInfo))
		{
			return nullptr;
		}
		if (!RootNodeInfo->AttributeInfo.IsValid() || RootNodeInfo->AttributeInfo->GetType() != USkeletalMesh::StaticClass() || !RootNodeInfo->AttributeInfo->bImportAttribute)
		{
			return nullptr;
		}
	}
	if (!RootNodeInfo.IsValid())
	{
		return nullptr;
	}
	//Set the options
	//Apply the correct fbx options
	TSharedPtr<FFbxMeshInfo> MeshInfo = StaticCastSharedPtr<FFbxMeshInfo>(RootNodeInfo->AttributeInfo);

	ApplyMeshInfoFbxOptions(MeshInfo);
	//TODO support bBakePivotInVertex
	bool Old_bBakePivotInVertex = GlobalImportSettings->bBakePivotInVertex;
	GlobalImportSettings->bBakePivotInVertex = false;
	GlobalImportSettings->bImportBoneTracks = true;
	//if (GlobalImportSettings->bBakePivotInVertex && RootNodeInfo->AttributeInfo->PivotNodeUid == INVALID_UNIQUE_ID)
	//{
		//GlobalImportSettings->bBakePivotInVertex = false;
	//}

	// check if there is LODGroup for this skeletal mesh
	int32 MaxLODLevel = 1;
	for (int32 j = 0; j < NodeArray.Num(); j++)
	{
		FbxNode* Node = NodeArray[j];
		if (Node->GetNodeAttribute() && Node->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup)
		{
			// get max LODgroup level
			if (MaxLODLevel < Node->GetChildCount())
			{
				MaxLODLevel = Node->GetChildCount();
			}
		}
	}

	//The skeletalmesh will be set after we import the LOD 0 since it is not created yet.
	FScopedSkeletalMeshPostEditChange ScopedPostEditChange(nullptr);

	int32 LODIndex;
	for (LODIndex = 0; LODIndex < MaxLODLevel; LODIndex++)
	{
		TArray<FbxNode*> SkelMeshNodeArray;
		for (int32 j = 0; j < NodeArray.Num(); j++)
		{
			FbxNode* Node = NodeArray[j];
			if (Node->GetNodeAttribute() && Node->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup)
			{
				TArray<FbxNode*> NodeInLod;
				if (Node->GetChildCount() > LODIndex)
				{
					FbxImporter->FindAllLODGroupNode(NodeInLod, Node, LODIndex);
				}
				else // in less some LODGroups have less level, use the last level
				{
					FbxImporter->FindAllLODGroupNode(NodeInLod, Node, Node->GetChildCount() - 1);
				}
				for (FbxNode *MeshNode : NodeInLod)
				{
					SkelMeshNodeArray.Add(MeshNode);
				}
			}
			else
			{
				SkelMeshNodeArray.Add(Node);
			}
		}

		//Make sure to bake the pivot the user choose to bake
		TArray<FbxNode*> SkelMeshNodePivotArray;
		bool bUseSkelMeshNodePivotArray = false;
		if (GlobalImportSettings->bBakePivotInVertex)
		{
			for (FbxNode *SkelMeshNode : SkelMeshNodeArray)
			{
				TSharedPtr<FFbxNodeInfo> ExportNodeInfo;
				if (FindSceneNodeInfo(SceneInfo, SkelMeshNode->GetUniqueID(), ExportNodeInfo))
				{
					if (ExportNodeInfo->AttributeInfo.IsValid())
					{
						FbxNode *NodePivot = FindFbxNodeById(FbxImporter, nullptr, ExportNodeInfo->AttributeInfo->PivotNodeUid);
						if (NodePivot != nullptr)
						{
							SkelMeshNodePivotArray.Add(NodePivot);
							bUseSkelMeshNodePivotArray = true;
							continue;
						}
					}
				}
				SkelMeshNodePivotArray.Add(SkelMeshNode);
			}
		}
		FSkeletalMeshImportData OutData;
		bool bMapMorphTargetToTimeZero = false;
		if (LODIndex == 0 && SkelMeshNodeArray.Num() != 0)
		{
			FName OutputName = FbxImporter->MakeNameForMesh(SkelMeshNodeArray[0]->GetName(), SkelMeshNodeArray[0]);
			FString PackageName = Path + TEXT("/") + OutputName.ToString();
			FString SkeletalMeshName;
			Pkg = CreatePackageForNode(PackageName, SkeletalMeshName);
			if (Pkg == nullptr)
				break;
			RootNodeInfo->AttributeInfo->SetOriginalImportPath(PackageName);
			FName SkeletalMeshFName = FName(*SkeletalMeshName);

			//Import the skeletal mesh
			UnFbx::FFbxImporter::FImportSkeletalMeshArgs ImportSkeletalMeshArgs;
			ImportSkeletalMeshArgs.InParent = Pkg;
			ImportSkeletalMeshArgs.NodeArray = bUseSkelMeshNodePivotArray ? SkelMeshNodePivotArray : SkelMeshNodeArray;
			ImportSkeletalMeshArgs.Name = SkeletalMeshFName;
			ImportSkeletalMeshArgs.Flags = Flags;
			ImportSkeletalMeshArgs.TemplateImportData = SkeletalMeshImportData;
			ImportSkeletalMeshArgs.LodIndex = LODIndex;
			ImportSkeletalMeshArgs.OutData = &OutData;

			USkeletalMesh* NewMesh = FbxImporter->ImportSkeletalMesh( ImportSkeletalMeshArgs );
			NewObject = NewMesh;
			if (NewMesh)
			{
				ScopedPostEditChange.SetSkeletalMesh(NewMesh);
				TSharedPtr<FFbxNodeInfo> SkelMeshNodeInfo;
				if (FindSceneNodeInfo(SceneInfo, SkelMeshNodeArray[0]->GetUniqueID(), SkelMeshNodeInfo) && SkelMeshNodeInfo.IsValid() && SkelMeshNodeInfo->AttributeInfo.IsValid())
				{
					AllNewAssets.Add(SkelMeshNodeInfo->AttributeInfo, NewObject);
				}
				if (GlobalImportSettings->bImportAnimations)
				{
					// We need to remove all scaling from the root node before we set up animation data.
					// Othewise some of the global transform calculations will be incorrect.
					FbxImporter->RemoveTransformSettingsFromFbxNode(RootNodeToImport, SkeletalMeshImportData);
					FbxImporter->SetupAnimationDataFromMesh(NewMesh, Pkg, SkelMeshNodeArray, AnimSequenceImportData, OutputName.ToString());

					// Reapply the transforms for the rest of the import
					FbxImporter->ApplyTransformSettingsToFbxNode(RootNodeToImport, SkeletalMeshImportData);
				}

				//Set the data in the node info
				RootNodeInfo->AttributeInfo->SetOriginalImportPath(PackageName);
				RootNodeInfo->AttributeInfo->SetOriginalFullImportName(NewObject->GetPathName());

				bMapMorphTargetToTimeZero = ImportSkeletalMeshArgs.bMapMorphTargetToTimeZero;
			}
		}
		else if (NewObject && GlobalImportSettings->bImportSkeletalMeshLODs) // the base skeletal mesh is imported successfully
		{
			USkeletalMesh* BaseSkeletalMesh = Cast<USkeletalMesh>(NewObject);
			if ((bUseSkelMeshNodePivotArray ? SkelMeshNodePivotArray : SkelMeshNodeArray)[0]->GetMesh() == nullptr)
			{
				FSkeletalMeshUpdateContext UpdateContext;
				UpdateContext.SkeletalMesh = BaseSkeletalMesh;
				//Add a autogenerated LOD to the BaseSkeletalMesh
				FSkeletalMeshLODInfo& LODInfo = BaseSkeletalMesh->AddLODInfo();
				LODInfo.ReductionSettings.NumOfTrianglesPercentage = FMath::Pow(0.5f, (float)(LODIndex));
				LODInfo.ReductionSettings.BaseLOD = 0;
				LODInfo.bImportWithBaseMesh = true;
				LODInfo.SourceImportFilename = FString(TEXT(""));
				FLODUtilities::SimplifySkeletalMeshLOD(UpdateContext, LODIndex, GetTargetPlatformManagerRef().GetRunningTargetPlatform(), false);
			}
			else
			{
				FName LODObjectName = NAME_None;
				//Import skeletal mesh LOD
				UnFbx::FFbxImporter::FImportSkeletalMeshArgs ImportSkeletalMeshArgs;
				ImportSkeletalMeshArgs.InParent = BaseSkeletalMesh->GetOutermost();
				ImportSkeletalMeshArgs.NodeArray = bUseSkelMeshNodePivotArray ? SkelMeshNodePivotArray : SkelMeshNodeArray;
				ImportSkeletalMeshArgs.Name = LODObjectName;
				ImportSkeletalMeshArgs.Flags = RF_Transient;
				ImportSkeletalMeshArgs.TemplateImportData = SkeletalMeshImportData;
				ImportSkeletalMeshArgs.LodIndex = LODIndex;
				ImportSkeletalMeshArgs.OutData = &OutData;

				USkeletalMesh *LODObject = FbxImporter->ImportSkeletalMesh(ImportSkeletalMeshArgs);
				bool bImportSucceeded = FbxImporter->ImportSkeletalMeshLOD(LODObject, BaseSkeletalMesh, LODIndex);
				if (!bImportSucceeded)
				{
					FbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("FailedToImport_SkeletalMeshLOD", "Failed to import Skeletal mesh LOD.")), FFbxErrors::SkeletalMesh_LOD_FailedToImport);
				}
				else
				{
					bMapMorphTargetToTimeZero = ImportSkeletalMeshArgs.bMapMorphTargetToTimeZero;
				}
			}
		}

		// import morph target
		if (NewObject && SkeletalMeshImportData->bImportMorphTargets)
		{
			if (Pkg == nullptr)
				continue;
			
			USkeletalMesh *NewSkelMesh = Cast<USkeletalMesh>(NewObject);
			if ((GlobalImportSettings->bImportSkeletalMeshLODs || LODIndex == 0) &&
				GlobalImportSettings->bImportMorph &&
				NewSkelMesh &&
				NewSkelMesh->GetImportedModel() &&
				NewSkelMesh->GetImportedModel()->LODModels.IsValidIndex(LODIndex))
			{
				// TODO: Disable material importing when importing morph targets
				FbxImporter->ImportFbxMorphTarget(SkelMeshNodeArray, NewSkelMesh, LODIndex, OutData, bMapMorphTargetToTimeZero);
			}
		}
	}
	
	//Put back the options
	GlobalImportSettings->bBakePivotInVertex = Old_bBakePivotInVertex;

	//FScopedSkeletalMeshPostEditChange will call post edit change when going out of scope
	return NewObject;
}

void UFbxSceneImportFactory::ImportAllSkeletalMesh(void* VoidRootNodeToImport, void* VoidFbxImporter, EObjectFlags Flags, int32& NodeIndex, int32& InterestingNodeCount, TSharedPtr<FFbxSceneInfo> SceneInfo)
{
	UnFbx::FFbxImporter* FbxImporter = (UnFbx::FFbxImporter*)VoidFbxImporter;
	
	if (!FbxImporter->CanImportClass(USkeletalMesh::StaticClass()) || !FbxImporter->CanImportClass(USkeleton::StaticClass()))
	{
		return;
	}

	FbxNode *RootNodeToImport = (FbxNode *)VoidRootNodeToImport;
	InterestingNodeCount = 1;
	TArray< TArray<FbxNode*>* > SkelMeshArray;
	FbxImporter->FillFbxSkelMeshArrayInScene(RootNodeToImport, SkelMeshArray, false, false, true);
	InterestingNodeCount = SkelMeshArray.Num();

	int32 TotalNumNodes = 0;

	for (int32 i = 0; i < SkelMeshArray.Num(); i++)
	{
		TArray<FbxNode*> NodeArray = *SkelMeshArray[i];
		TArray<void*> VoidNodeArray;
		for (FbxNode *Node : NodeArray)
		{
			void* VoidNode = (void*)Node;
			VoidNodeArray.Add(VoidNode);
		}
		UObject* NewObject = ImportOneSkeletalMesh(VoidRootNodeToImport, VoidFbxImporter, SceneInfo, Flags, VoidNodeArray, TotalNumNodes);
		if (NewObject)
		{
			NodeIndex++;
		}
	}

	for (int32 i = 0; i < SkelMeshArray.Num(); i++)
	{
		delete SkelMeshArray[i];
	}

	// if total nodes we found is 0, we didn't find anything. 
	if (SkelMeshArray.Num() > 0 && TotalNumNodes == 0)
	{
		FbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("FailedToImport_NoMeshFoundOnRoot", "Could not find any valid mesh on the root hierarchy. If you have mesh in the sub hierarchy, please enable option of [Import Meshes In Bone Hierarchy] when import.")),
			FFbxErrors::SkeletalMesh_NoMeshFoundOnRoot);
	}
}

void UFbxSceneImportFactory::ImportAllStaticMesh(void* VoidRootNodeToImport, void* VoidFbxImporter, EObjectFlags Flags, int32& NodeIndex, int32& InterestingNodeCount, TSharedPtr<FFbxSceneInfo> SceneInfo)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UFbxSceneImportFactory::ImportAllStaticMesh);

	UnFbx::FFbxImporter* FbxImporter = (UnFbx::FFbxImporter*)VoidFbxImporter;

	if (!FbxImporter->CanImportClass(UStaticMesh::StaticClass()))
	{
		return;
	}

	FbxNode *RootNodeToImport = (FbxNode *)VoidRootNodeToImport;
	
	//Copy default options to StaticMeshImportData
	SFbxSceneOptionWindow::CopyFbxOptionsToStaticMeshOptions(GlobalImportSettingsReference, SceneImportOptionsStaticMesh);
	SceneImportOptionsStaticMesh->FillStaticMeshInmportData(StaticMeshImportData, SceneImportOptions);

	FbxImporter->ApplyTransformSettingsToFbxNode(RootNodeToImport, StaticMeshImportData);

	// count meshes in lod groups if we don't care about importing LODs
	int32 NumLODGroups = 0;
	bool bCountLODGroupMeshes = !GlobalImportSettingsReference->bImportStaticMeshLODs;
	InterestingNodeCount = FbxImporter->GetFbxMeshCount(RootNodeToImport, bCountLODGroupMeshes, NumLODGroups);

	int32 ImportedMeshCount = 0;
	UStaticMesh* NewStaticMesh = nullptr;
	UObject* Object = RecursiveImportNode(FbxImporter, RootNodeToImport, Flags, NodeIndex, InterestingNodeCount, SceneInfo, Path);

	NewStaticMesh = Cast<UStaticMesh>(Object);

	// Make sure to notify the asset registry of all assets created.
	for (auto AssetItKvp = AllNewAssets.CreateIterator(); AssetItKvp; ++AssetItKvp)
	{
		UObject* Asset = AssetItKvp.Value();
		FAssetRegistryModule::AssetCreated(Asset);
		Asset->MarkPackageDirty();
	}
	ImportedMeshCount = AllNewAssets.Num();
	if (ImportedMeshCount == 1 && NewStaticMesh)
	{
		FbxImporter->ImportStaticMeshGlobalSockets(NewStaticMesh);
	}
}

// @todo document
UObject* UFbxSceneImportFactory::RecursiveImportNode(void* VoidFbxImporter, void* VoidNode, EObjectFlags Flags, int32& NodeIndex, int32 Total, TSharedPtr<FFbxSceneInfo>  SceneInfo, FString PackagePath)
{
	UObject* FirstBaseObject = nullptr;
	TSharedPtr<FFbxNodeInfo> OutNodeInfo;
	UnFbx::FFbxImporter* FFbxImporter = (UnFbx::FFbxImporter*)VoidFbxImporter;

	FbxNode* Node = (FbxNode*)VoidNode;

	if (Node->GetNodeAttribute() && Node->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup && Node->GetChildCount() > 0)
	{
		//Find the deepest mesh child for the first LOD
		TArray<FbxNode*> AllNodeInLod;
		FFbxImporter->FindAllLODGroupNode(AllNodeInLod, Node, 0);
		UObject* NewObject = nullptr;
		//Combine LOD group
		TArray<void*> TmpVoidArray;
		if (AllNodeInLod.Num() > 0)
		{
			for (FbxNode* LodNode : AllNodeInLod)
			{
				TmpVoidArray.Add(LodNode);
			}
			NewObject = ImportANode(VoidFbxImporter, TmpVoidArray, Flags, NodeIndex, SceneInfo, OutNodeInfo, PackagePath, Total);
		}
		
		if(NewObject)
		{
			//We should always have a valid attribute if we just create a new asset
			check(OutNodeInfo.IsValid() && OutNodeInfo->AttributeInfo.IsValid());

			AllNewAssets.Add(OutNodeInfo->AttributeInfo, NewObject);
			if (GlobalImportSettingsReference->bImportStaticMeshLODs)
			{
				//We use ImportedLodIndex in case there is an empty LOD (FindAllLODGroupNode do not find geometry for the LOD)
				int32 ImportedLodIndex = 1;
				// import LOD meshes
				for (int32 LODIndex = 1; LODIndex < Node->GetChildCount(); LODIndex++)
				{
					if (LODIndex >= MAX_STATIC_MESH_LODS)
					{
						FFbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(
							LOCTEXT("ImporterLimits_MaximumStaticMeshLODReach", "Reached the maximum number of LODs for a Static Mesh({0}) - discarding {1} LOD meshes."), FText::AsNumber(MAX_STATIC_MESH_LODS), FText::AsNumber(Node->GetChildCount() - MAX_STATIC_MESH_LODS))
						), FFbxErrors::Generic_Mesh_TooManyLODs);
						break;
					}
					AllNodeInLod.Empty();
					FFbxImporter->FindAllLODGroupNode(AllNodeInLod, Node, LODIndex);
					if (AllNodeInLod.Num() > 0)
					{
						if (AllNodeInLod[0]->GetMesh() == nullptr)
						{
							UStaticMesh *BaseStaticMesh = Cast<UStaticMesh>(NewObject);
							FFbxImporter->AddStaticMeshSourceModelGeneratedLOD(BaseStaticMesh, LODIndex);
						}
						else
						{
							TmpVoidArray.Empty();
							for (FbxNode* LodNode : AllNodeInLod)
							{
								TmpVoidArray.Add(LodNode);
							}
							ImportANode(VoidFbxImporter, TmpVoidArray, Flags, NodeIndex, SceneInfo, OutNodeInfo, PackagePath, Total, NewObject, LODIndex);
						}
					}
				}
			}
			UStaticMesh *NewStaticMesh = Cast<UStaticMesh>(NewObject);
			if (NewStaticMesh != nullptr)
			{
				//Build the staticmesh
				FFbxImporter->FindAllLODGroupNode(AllNodeInLod, Node, 0);
				FFbxImporter->PostImportStaticMesh(NewStaticMesh, AllNodeInLod);
			}
		}
	}
	else
	{
		if (Node->GetMesh() && Node->GetMesh()->GetPolygonVertexCount() > 0)
		{
			TArray<void*> TmpVoidArray;
			TmpVoidArray.Add(Node);
			FirstBaseObject = ImportANode(VoidFbxImporter, TmpVoidArray, Flags, NodeIndex, SceneInfo, OutNodeInfo, PackagePath, Total);

			if (FirstBaseObject)
			{
				//We should always have a valid attribute if we just create a new asset
				check(OutNodeInfo.IsValid() && OutNodeInfo->AttributeInfo.IsValid());

				UStaticMesh *NewStaticMesh = Cast<UStaticMesh>(FirstBaseObject);
				if (NewStaticMesh != nullptr)
				{
					//Build the staticmesh
					TArray<FbxNode*> AllNodeInLod;
					AllNodeInLod.Add(Node);
					FFbxImporter->PostImportStaticMesh(NewStaticMesh, AllNodeInLod);
				}

				AllNewAssets.Add(OutNodeInfo->AttributeInfo, FirstBaseObject);
			}
		}
		
		if (SceneImportOptions->bCreateContentFolderHierarchy)
		{
			FString NodeName = FString(FFbxImporter->MakeName(Node->GetName()));
			if (NodeName.Compare("RootNode") != 0)
			{
				PackagePath += TEXT("/") + NodeName;
			}
		}

		for (int32 ChildIndex = 0; ChildIndex < Node->GetChildCount(); ++ChildIndex)
		{
			UObject* SubObject = RecursiveImportNode(VoidFbxImporter, Node->GetChild(ChildIndex), Flags, NodeIndex, Total, SceneInfo, PackagePath);
			if (FirstBaseObject == nullptr)
			{
				FirstBaseObject = SubObject;
			}
		}
	}

	return FirstBaseObject;
}

// @todo document
UObject* UFbxSceneImportFactory::ImportANode(void* VoidFbxImporter, TArray<void*> &VoidNodes, EObjectFlags Flags, int32& NodeIndex, TSharedPtr<FFbxSceneInfo> SceneInfo, TSharedPtr<FFbxNodeInfo> &OutNodeInfo, FString PackagePath, int32 Total, UObject* InMesh, int LODIndex)
{
	UnFbx::FFbxImporter* FFbxImporter = (UnFbx::FFbxImporter*)VoidFbxImporter;
	TArray<FbxNode*> Nodes;
	for (void* VoidNode : VoidNodes)
	{
		Nodes.Add((FbxNode*)VoidNode);
	}
	check(Nodes.Num() > 0 && Nodes[0] != nullptr);
	FString ParentName;
	if (Nodes[0]->GetParent() != nullptr)
	{
		ParentName = FFbxImporter->MakeName(Nodes[0]->GetParent()->GetName());
	}
	else
	{
		ParentName.Empty();
	}
	
	FString NodeName = FFbxImporter->MakeName(Nodes[0]->GetName());
	//Find the scene node info in the hierarchy
	if (!FindSceneNodeInfo(SceneInfo, Nodes[0]->GetUniqueID(), OutNodeInfo) || !OutNodeInfo->AttributeInfo.IsValid())
	{
		//We cannot instantiate this asset if its not part of the hierarchy
		return nullptr;
	}

	if(OutNodeInfo->AttributeInfo->GetType() != UStaticMesh::StaticClass() || !OutNodeInfo->AttributeInfo->bImportAttribute)
	{
		//export only static mesh or the user specify to not import this mesh
		return nullptr;
	}

	//Check if the Mesh was already import
	if (AllNewAssets.Contains(OutNodeInfo->AttributeInfo))
	{
		return AllNewAssets[OutNodeInfo->AttributeInfo];
	}

	UObject* NewObject = nullptr;
	// skip collision models
	if (NodeName.Contains(TEXT("UCX"), ESearchCase::IgnoreCase) || NodeName.Contains(TEXT("MCDCX"), ESearchCase::IgnoreCase) ||
		NodeName.Contains(TEXT("UBX"), ESearchCase::IgnoreCase) || NodeName.Contains(TEXT("USP"), ESearchCase::IgnoreCase) || NodeName.Contains(TEXT("UCP"), ESearchCase::IgnoreCase))
	{
		return nullptr;
	}

	//Create a package for this node
	FString PackageName = PackagePath + TEXT("/") + OutNodeInfo->AttributeInfo->Name;
	FString StaticMeshName;
	UPackage* Pkg = CreatePackageForNode(PackageName, StaticMeshName);
	if (Pkg == nullptr)
		return nullptr;

	//Apply the correct fbx options
	TSharedPtr<FFbxMeshInfo> MeshInfo = StaticCastSharedPtr<FFbxMeshInfo>(OutNodeInfo->AttributeInfo);
	
	ApplyMeshInfoFbxOptions(MeshInfo);
	bool Old_bBakePivotInVertex = GlobalImportSettings->bBakePivotInVertex;
	if (GlobalImportSettings->bBakePivotInVertex && OutNodeInfo->AttributeInfo->PivotNodeUid == INVALID_UNIQUE_ID)
	{
		GlobalImportSettings->bBakePivotInVertex = false;
	}
	FName StaticMeshFName = FName(*(OutNodeInfo->AttributeInfo->Name));
	//Make sure to bake the pivot the user choose to bake
	if (GlobalImportSettings->bBakePivotInVertex && Nodes.Num() == 1)
	{
		FbxNode *NodePivot = FindFbxNodeById(FFbxImporter, nullptr, OutNodeInfo->AttributeInfo->PivotNodeUid);
		if (NodePivot != nullptr)
		{
			Nodes[0] = NodePivot;
		}
	}
	
	NewObject = FFbxImporter->ImportStaticMeshAsSingle(Pkg, Nodes, StaticMeshFName, Flags, StaticMeshImportData, Cast<UStaticMesh>(InMesh), LODIndex);

	OutNodeInfo->AttributeInfo->SetOriginalImportPath(PackageName);

	if (NewObject)
	{
		OutNodeInfo->AttributeInfo->SetOriginalFullImportName(NewObject->GetPathName());

		NodeIndex++;
		FFormatNamedArguments Args;
		Args.Add(TEXT("NodeIndex"), NodeIndex);
		Args.Add(TEXT("ArrayLength"), Total);
		GWarn->StatusUpdate(NodeIndex, Total, FText::Format(NSLOCTEXT("UnrealEd", "Importingf", "Importing ({NodeIndex} of {ArrayLength})"), Args));
	}
	else
	{
		Pkg->SetDirtyFlag(false);
		Pkg->RemoveFromRoot();
		Pkg->ConditionalBeginDestroy();
	}

	// Destroy Fbx mesh to save memory.
	for (int32 Index = 0; Index < Nodes.Num(); Index++)
	{
		FbxMesh* Mesh = Nodes[Index]->GetMesh();
		Mesh->Destroy(true);
	}

	GlobalImportSettings->bBakePivotInVertex = Old_bBakePivotInVertex;
	return NewObject;
}

UnFbx::FBXImportOptions *UFbxSceneImportFactory::GetOptionsFromName(FString OptionsName)
{
	for (auto kvp : NameOptionsMap)
	{
		if (kvp.Key.Compare(OptionsName) == 0)
		{
			return kvp.Value;
		}
	}
	return nullptr;
}

bool UFbxSceneImportFactory::FindSceneNodeInfo(TSharedPtr<FFbxSceneInfo> SceneInfo, uint64 NodeInfoUniqueId, TSharedPtr<FFbxNodeInfo> &OutNodeInfo)
{
	for (auto NodeIt = SceneInfo->HierarchyInfo.CreateIterator(); NodeIt; ++NodeIt)
	{
		if (NodeInfoUniqueId == (*NodeIt)->UniqueId)
		{
			OutNodeInfo = (*NodeIt);
			return true;
		}
	}
	return false;
}

UPackage *UFbxSceneImportFactory::CreatePackageForNode(FString PackageName, FString &StaticMeshName)
{
	FString PackageNameOfficial = UPackageTools::SanitizePackageName(PackageName);
	// We can not create assets that share the name of a map file in the same location
	if (FEditorFileUtils::IsMapPackageAsset(PackageNameOfficial))
	{
		return nullptr;
	}
	bool IsPkgExist = FPackageName::DoesPackageExist(PackageNameOfficial);
	if (!IsPkgExist)
	{
		IsPkgExist = FindObject<UPackage>(nullptr, *PackageNameOfficial) != nullptr;
	}
	int32 tryCount = 1;
	while (IsPkgExist)
	{
		PackageNameOfficial = PackageName;
		PackageNameOfficial += TEXT("_");
		PackageNameOfficial += FString::FromInt(tryCount++);
		PackageNameOfficial = UPackageTools::SanitizePackageName(PackageNameOfficial);
		IsPkgExist = FPackageName::DoesPackageExist(PackageNameOfficial);
		if (!IsPkgExist)
		{
			IsPkgExist = FindObject<UPackage>(nullptr, *PackageNameOfficial) != nullptr;
		}
	}
	UPackage* Pkg = CreatePackage( *PackageNameOfficial);
	if (!ensure(Pkg))
	{
		return nullptr;
	}
	Pkg->FullyLoad();

	StaticMeshName = FPackageName::GetLongPackageAssetName(Pkg->GetOutermost()->GetName());
	return Pkg;
}

#undef LOCTEXT_NAMESPACE
