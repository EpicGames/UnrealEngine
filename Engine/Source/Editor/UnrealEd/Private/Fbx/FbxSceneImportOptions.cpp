// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/FbxSceneImportOptions.h"

#include "Math/MathFwd.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FbxSceneImportOptions)


UFbxSceneImportOptions::UFbxSceneImportOptions(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bTransformVertexToAbsolute = false;
	bBakePivotInVertex = false;
	bCreateContentFolderHierarchy = false;
	bImportAsDynamic = false;
	HierarchyType = EFBXSceneOptionsCreateHierarchyType::FBXSOCHT_CreateBlueprint;
	bForceFrontXAxis = false;
	bImportStaticMeshLODs = false;
	bImportSkeletalMeshLODs = false;
	bInvertNormalMaps = false;
	ImportTranslation = FVector(0);
	ImportRotation = FRotator(0);
	ImportUniformScale = 1.0f;
}

