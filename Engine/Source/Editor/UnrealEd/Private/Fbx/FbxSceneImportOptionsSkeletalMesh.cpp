// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/FbxSceneImportOptionsSkeletalMesh.h"

#include "Factories/FbxSceneImportOptions.h"
#include "Factories/FbxSkeletalMeshImportData.h"
#include "Math/Rotator.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Misc/AssertionMacros.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FbxSceneImportOptionsSkeletalMesh)


UFbxSceneImportOptionsSkeletalMesh::UFbxSceneImportOptionsSkeletalMesh(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bUpdateSkeletonReferencePose(false)
	, bCreatePhysicsAsset(false)
	, bUseT0AsRefPose(false)
	, bPreserveSmoothingGroups(false)
	, bKeepSectionsSeparate(false)
	, bImportMeshesInBoneHierarchy(true)
	, bImportMorphTargets(false)
	, bImportVertexAttributes(false)
	, ThresholdPosition(THRESH_POINTS_ARE_SAME)
	, ThresholdTangentNormal(THRESH_NORMALS_ARE_SAME)
	, ThresholdUV(THRESH_UVS_ARE_SAME)
	, MorphThresholdPosition(THRESH_POINTS_ARE_NEAR)
	, bImportAnimations(true)
	, AnimationLength(EFBXAnimationLengthImportType::FBXALIT_AnimatedKey)
	, FrameImportRange(0, 0)
	, bUseDefaultSampleRate(false)
	, CustomSampleRate(0)
	, bImportCustomAttribute(true)
	, bDeleteExistingCustomAttributeCurves(false)
	, bDeleteExistingNonCurveCustomAttributes(false)
	, bPreserveLocalTransform(false)
	, bDeleteExistingMorphTargetCurves(false)
{
}

void UFbxSceneImportOptionsSkeletalMesh::FillSkeletalMeshInmportData(UFbxSkeletalMeshImportData* SkeletalMeshImportData, UFbxAnimSequenceImportData* AnimSequenceImportData, UFbxSceneImportOptions* SceneImportOptions)
{
	check(SkeletalMeshImportData != nullptr);
	SkeletalMeshImportData->bImportMeshesInBoneHierarchy = bImportMeshesInBoneHierarchy;
	SkeletalMeshImportData->bImportMorphTargets = bImportMorphTargets;
	SkeletalMeshImportData->bImportVertexAttributes = bImportVertexAttributes;

	SkeletalMeshImportData->ThresholdPosition = ThresholdPosition;
	SkeletalMeshImportData->ThresholdTangentNormal = ThresholdTangentNormal;
	SkeletalMeshImportData->ThresholdUV = ThresholdUV;
	SkeletalMeshImportData->MorphThresholdPosition = MorphThresholdPosition;
	SkeletalMeshImportData->bPreserveSmoothingGroups = bPreserveSmoothingGroups;
	SkeletalMeshImportData->bKeepSectionsSeparate = bKeepSectionsSeparate;
	SkeletalMeshImportData->bUpdateSkeletonReferencePose = bUpdateSkeletonReferencePose;
	SkeletalMeshImportData->bUseT0AsRefPose = bUseT0AsRefPose;

	SkeletalMeshImportData->bImportMeshLODs = SceneImportOptions->bImportSkeletalMeshLODs;
	SkeletalMeshImportData->ImportTranslation = SceneImportOptions->ImportTranslation;
	SkeletalMeshImportData->ImportRotation = SceneImportOptions->ImportRotation;
	SkeletalMeshImportData->ImportUniformScale = SceneImportOptions->ImportUniformScale;
	SkeletalMeshImportData->bTransformVertexToAbsolute = SceneImportOptions->bTransformVertexToAbsolute;
	SkeletalMeshImportData->bBakePivotInVertex = SceneImportOptions->bBakePivotInVertex;
	
	SkeletalMeshImportData->bImportAsScene = true;

	AnimSequenceImportData->bImportMeshesInBoneHierarchy = bImportMeshesInBoneHierarchy;
	AnimSequenceImportData->AnimationLength = AnimationLength;
	AnimSequenceImportData->bDeleteExistingMorphTargetCurves = bDeleteExistingMorphTargetCurves;
	AnimSequenceImportData->bImportCustomAttribute = bImportCustomAttribute;
	AnimSequenceImportData->bDeleteExistingCustomAttributeCurves = bDeleteExistingCustomAttributeCurves;
	AnimSequenceImportData->bDeleteExistingNonCurveCustomAttributes = bDeleteExistingNonCurveCustomAttributes;
	AnimSequenceImportData->bPreserveLocalTransform = bPreserveLocalTransform;
	AnimSequenceImportData->bUseDefaultSampleRate = bUseDefaultSampleRate;
	AnimSequenceImportData->CustomSampleRate = CustomSampleRate;
	AnimSequenceImportData->FrameImportRange = FrameImportRange;

	AnimSequenceImportData->bImportAsScene = true;
}
