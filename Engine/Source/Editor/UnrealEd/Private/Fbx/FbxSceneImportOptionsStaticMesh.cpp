// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/FbxSceneImportOptionsStaticMesh.h"

#include "Containers/EnumAsByte.h"
#include "Factories/FbxMeshImportData.h"
#include "Factories/FbxSceneImportOptions.h"
#include "Factories/FbxStaticMeshImportData.h"
#include "Math/MathFwd.h"
#include "Math/Rotator.h"
#include "Math/Vector.h"
#include "Misc/AssertionMacros.h"
#include "UObject/UnrealNames.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FbxSceneImportOptionsStaticMesh)


UFbxSceneImportOptionsStaticMesh::UFbxSceneImportOptionsStaticMesh(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, StaticMeshLODGroup(NAME_None)
	, bAutoGenerateCollision(true)
	, bRemoveDegenerates(true)
	, bBuildReversedIndexBuffer(true)
	, bGenerateLightmapUVs(true)
	, bOneConvexHullPerUCX(true)
{
	VertexOverrideColor = FColor(255, 255, 255, 255);
	NormalImportMethod = EFBXSceneNormalImportMethod::FBXSceneNIM_ComputeNormals;
	NormalGenerationMethod = EFBXSceneNormalGenerationMethod::MikkTSpace;
}

void UFbxSceneImportOptionsStaticMesh::FillStaticMeshInmportData(UFbxStaticMeshImportData* StaticMeshImportData, UFbxSceneImportOptions* SceneImportOptions)
{
	check(StaticMeshImportData != nullptr);
	check(SceneImportOptions != nullptr);
	StaticMeshImportData->bAutoGenerateCollision = bAutoGenerateCollision;
	StaticMeshImportData->bBuildReversedIndexBuffer = bBuildReversedIndexBuffer;
	StaticMeshImportData->bGenerateLightmapUVs = bGenerateLightmapUVs;
	StaticMeshImportData->bOneConvexHullPerUCX = bOneConvexHullPerUCX;
	StaticMeshImportData->bRemoveDegenerates = bRemoveDegenerates;
	StaticMeshImportData->StaticMeshLODGroup = StaticMeshLODGroup;
	switch (VertexColorImportOption)
	{
	case EFbxSceneVertexColorImportOption::Ignore:
		StaticMeshImportData->VertexColorImportOption = EVertexColorImportOption::Ignore;
		break;
	case EFbxSceneVertexColorImportOption::Override:
		StaticMeshImportData->VertexColorImportOption = EVertexColorImportOption::Override;
		break;
	case EFbxSceneVertexColorImportOption::Replace:
		StaticMeshImportData->VertexColorImportOption = EVertexColorImportOption::Replace;
		break;
	}
	StaticMeshImportData->VertexOverrideColor = VertexOverrideColor;

	//Scene general options
	StaticMeshImportData->bImportMeshLODs = SceneImportOptions->bImportStaticMeshLODs;
	StaticMeshImportData->ImportTranslation = SceneImportOptions->ImportTranslation;
	StaticMeshImportData->ImportRotation = SceneImportOptions->ImportRotation;
	StaticMeshImportData->ImportUniformScale = SceneImportOptions->ImportUniformScale;
	StaticMeshImportData->bTransformVertexToAbsolute = SceneImportOptions->bTransformVertexToAbsolute;
	StaticMeshImportData->bBakePivotInVertex = SceneImportOptions->bBakePivotInVertex;

	StaticMeshImportData->bImportAsScene = true;
}
