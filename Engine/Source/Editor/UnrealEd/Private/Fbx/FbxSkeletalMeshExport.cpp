// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Implementation of Skeletal Mesh export related functionality from FbxExporter
=============================================================================*/

#include "CoreMinimal.h"
#include "Animation/Skeleton.h"
#include "BoneWeights.h"
#include "GPUSkinPublicDefs.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Animation/AnimSequence.h"
#include "Rendering/SkeletalMeshModel.h"

#include "FbxExporter.h"
#include "Exporters/FbxExportOption.h"
#include "UObject/MetaData.h"

#include "FbxMaterialExportUtilities.h"

DEFINE_LOG_CATEGORY_STATIC(LogFbxSkeletalMeshExport, Log, All);

namespace UnFbx
{

/**
 * Adds FBX skeleton nodes to the FbxScene based on the skeleton in the given USkeletalMesh, and fills
 * the given array with the nodes created
 */
FbxNode* FFbxExporter::CreateSkeleton(const USkeletalMesh* SkelMesh, TArray<FbxNode*>& BoneNodes)
{
	const FReferenceSkeleton& RefSkeleton= SkelMesh->GetRefSkeleton();

	if(RefSkeleton.GetRawBoneNum() == 0)
	{
		return nullptr;
	}

	// Create a list of the nodes we create for each bone, so that children can 
	// later look up their parent
	BoneNodes.Reserve(RefSkeleton.GetRawBoneNum());

	for(int32 BoneIndex = 0; BoneIndex < RefSkeleton.GetRawBoneNum(); ++BoneIndex)
	{
		const FMeshBoneInfo& CurrentBone = RefSkeleton.GetRefBoneInfo()[BoneIndex];
		const FTransform& BoneTransform = RefSkeleton.GetRefBonePose()[BoneIndex];

		FbxString BoneName = Converter.ConvertToFbxString(CurrentBone.ExportName);


		// Create the node's attributes
		FbxSkeleton* SkeletonAttribute = FbxSkeleton::Create(Scene, BoneName.Buffer());
		if(BoneIndex)
		{
			SkeletonAttribute->SetSkeletonType(FbxSkeleton::eLimbNode);
			//SkeletonAttribute->Size.Set(1.0);
		}
		else
		{
			SkeletonAttribute->SetSkeletonType(FbxSkeleton::eRoot);
			//SkeletonAttribute->Size.Set(1.0);
		}
		

		// Create the node
		FbxNode* BoneNode = FbxNode::Create(Scene, BoneName.Buffer());
		BoneNode->SetNodeAttribute(SkeletonAttribute);

		// Set the bone node's local orientation
		FVector UnrealRotation = BoneTransform.GetRotation().Euler();
		FbxVector4 LocalPos = Converter.ConvertToFbxPos(BoneTransform.GetTranslation());
		FbxVector4 LocalRot = Converter.ConvertToFbxRot(UnrealRotation);
		FbxVector4 LocalScale = Converter.ConvertToFbxScale(BoneTransform.GetScale3D());

		BoneNode->LclTranslation.Set(LocalPos);
		BoneNode->LclRotation.Set(LocalRot);
		BoneNode->LclScaling.Set(LocalScale);


		// If this is not the root bone, attach it to its parent
		if(BoneIndex)
		{
			BoneNodes[CurrentBone.ParentIndex]->AddChild(BoneNode);
		}


		// Add the node to the list of nodes, in bone order
		BoneNodes.Push(BoneNode);
	}

	return BoneNodes[0];
}

void FFbxExporter::GetSkeleton(FbxNode* RootNode, TArray<FbxNode*>& BoneNodes)
{
	if (RootNode->GetSkeleton())
	{
		BoneNodes.Add(RootNode);
	}

	for (int32 ChildIndex=0; ChildIndex<RootNode->GetChildCount(); ++ChildIndex)
	{
		GetSkeleton(RootNode->GetChild(ChildIndex), BoneNodes);
	}
}

/**
 * Adds an Fbx Mesh to the FBX scene based on the data in the given FSkeletalMeshLODModel
 */
FbxNode* FFbxExporter::CreateMesh(const USkeletalMesh* SkelMesh, const TCHAR* MeshName, int32 LODIndex, const UAnimSequence* AnimSeq /*= nullptr*/, const TArray<UMaterialInterface*>* OverrideMaterials /*= nullptr*/)
{
	const FSkeletalMeshModel* SkelMeshResource = SkelMesh->GetImportedModel();
	if (!SkelMeshResource->LODModels.IsValidIndex(LODIndex))
	{
		//Return an empty node
		return FbxNode::Create(Scene, TCHAR_TO_UTF8(MeshName));
	}

	const FSkeletalMeshLODModel& SourceModel = SkelMeshResource->LODModels[LODIndex];
	const int32 VertexCount = SourceModel.GetNumNonClothingVertices();

	// Verify the integrity of the mesh.
	if (VertexCount == 0) return nullptr;

	// Copy all the vertex data from the various chunks to a single buffer.
	// Makes the rest of the code in this function cleaner and easier to maintain.  
	TArray<FSoftSkinVertex> Vertices;
	SourceModel.GetNonClothVertices(Vertices);
	if (Vertices.Num() != VertexCount) return nullptr;

	FbxMesh* Mesh = FbxMesh::Create(Scene, TCHAR_TO_UTF8(MeshName));

	// Create and fill in the vertex position data source.
	Mesh->InitControlPoints(VertexCount);
	FbxVector4* ControlPoints = Mesh->GetControlPoints();
	for (int32 VertIndex = 0; VertIndex < VertexCount; ++VertIndex)
	{
		FVector Position			= (FVector)Vertices[VertIndex].Position;
		ControlPoints[VertIndex]	= Converter.ConvertToFbxPos(Position);
	}

	// Create Layer 0 to hold the normals
	FbxLayer* LayerZero = Mesh->GetLayer(0);
	if (LayerZero == nullptr)
	{
		Mesh->CreateLayer();
		LayerZero = Mesh->GetLayer(0);
	}

	// Create and fill in the per-face-vertex normal data source.
	// We extract the Z-tangent and drop the X/Y-tangents which are also stored in the render mesh.
	FbxLayerElementNormal* LayerElementNormal= FbxLayerElementNormal::Create(Mesh, "");

	LayerElementNormal->SetMappingMode(FbxLayerElement::eByControlPoint);
	// Set the normal values for every control point.
	LayerElementNormal->SetReferenceMode(FbxLayerElement::eDirect);

	for (int32 VertIndex = 0; VertIndex < VertexCount; ++VertIndex)
	{
		FVector Normal			= (FVector4)Vertices[VertIndex].TangentZ;
		FbxVector4 FbxNormal	= Converter.ConvertToFbxPos(Normal);

		LayerElementNormal->GetDirectArray().Add(FbxNormal);
	}

	LayerZero->SetNormals(LayerElementNormal);


	// Create and fill in the per-face-vertex texture coordinate data source(s).
	// Create UV for Diffuse channel.
	const int32 TexCoordSourceCount = SourceModel.NumTexCoords;
	TCHAR UVChannelName[32];
	for (int32 TexCoordSourceIndex = 0; TexCoordSourceIndex < TexCoordSourceCount; ++TexCoordSourceIndex)
	{
		FbxLayer* Layer = Mesh->GetLayer(TexCoordSourceIndex);
		if (Layer == nullptr)
		{
			Mesh->CreateLayer();
			Layer = Mesh->GetLayer(TexCoordSourceIndex);
		}

		if (TexCoordSourceIndex == 1)
		{
			FCString::Sprintf(UVChannelName, TEXT("LightMapUV"));
		}
		else
		{
			FCString::Sprintf(UVChannelName, TEXT("DiffuseUV"));
		}

		FbxLayerElementUV* UVDiffuseLayer = FbxLayerElementUV::Create(Mesh, TCHAR_TO_UTF8(UVChannelName));
		UVDiffuseLayer->SetMappingMode(FbxLayerElement::eByControlPoint);
		UVDiffuseLayer->SetReferenceMode(FbxLayerElement::eDirect);

		// Create the texture coordinate data source.
		for (int32 TexCoordIndex = 0; TexCoordIndex < VertexCount; ++TexCoordIndex)
		{
			const FVector2D& TexCoord = FVector2D(Vertices[TexCoordIndex].UVs[TexCoordSourceIndex]);
			UVDiffuseLayer->GetDirectArray().Add(FbxVector2(TexCoord.X, -TexCoord.Y + 1.0));
		}

		Layer->SetUVs(UVDiffuseLayer, FbxLayerElement::eTextureDiffuse);
	}

	FbxLayerElementMaterial* MatLayer = FbxLayerElementMaterial::Create(Mesh, "");
	MatLayer->SetMappingMode(FbxLayerElement::eByPolygon);
	MatLayer->SetReferenceMode(FbxLayerElement::eIndexToDirect);
	LayerZero->SetMaterials(MatLayer);


	// Create the per-material polygons sets.
	int32 SectionCount = SourceModel.Sections.Num();
	int32 ClothSectionVertexRemoveOffset = 0;
	TArray<TPair<uint32, uint32>> VertexIndexOffsetPairArray{ TPair<uint32, uint32>(0,0) };
	for (int32 SectionIndex = 0; SectionIndex < SectionCount; ++SectionIndex)
	{
		const FSkelMeshSection& Section = SourceModel.Sections[SectionIndex];
		if (Section.HasClothingData())
		{
			ClothSectionVertexRemoveOffset += Section.GetNumVertices();
			VertexIndexOffsetPairArray.Emplace(Section.BaseVertexIndex, ClothSectionVertexRemoveOffset);
			continue;
		}
		int32 MatIndex = Section.MaterialIndex;

		// Static meshes contain one triangle list per element.
		int32 TriangleCount = Section.NumTriangles;

		// Copy over the index buffer into the FBX polygons set.
		for (int32 TriangleIndex = 0; TriangleIndex < TriangleCount; ++TriangleIndex)
		{
			Mesh->BeginPolygon(MatIndex);
			for (int32 PointIndex = 0; PointIndex < 3; PointIndex++)
			{
				int32 VertexPositionIndex = SourceModel.IndexBuffer[Section.BaseIndex + ((TriangleIndex * 3) + PointIndex)] - ClothSectionVertexRemoveOffset;
				check(VertexPositionIndex >= 0);
				Mesh->AddPolygon(VertexPositionIndex);
			}
			Mesh->EndPolygon();
		}
	}

	if (GetExportOptions()->VertexColor)
	{
		// Create and fill in the vertex color data source.
		FbxLayerElementVertexColor* VertexColor = FbxLayerElementVertexColor::Create(Mesh, "");
		VertexColor->SetMappingMode(FbxLayerElement::eByControlPoint);
		VertexColor->SetReferenceMode(FbxLayerElement::eDirect);
		FbxLayerElementArrayTemplate<FbxColor>& VertexColorArray = VertexColor->GetDirectArray();
		LayerZero->SetVertexColors(VertexColor);

		for (int32 VertIndex = 0; VertIndex < VertexCount; ++VertIndex)
		{
			FLinearColor VertColor = Vertices[VertIndex].Color.ReinterpretAsLinear();
				VertexColorArray.Add(FbxColor(VertColor.R, VertColor.G, VertColor.B, VertColor.A));
		}
	}


	if (GetExportOptions()->bExportMorphTargets && SkelMesh->GetSkeleton()) //The skeleton can be null if this is a destructible mesh.
	{
		TMap<FName, FbxAnimCurve*> BlendShapeCurvesMap;

		if (SkelMesh->GetMorphTargets().Num())
		{
			// The original BlendShape Name was not saved during import, so we need to come up with a new one.
			const FString BlendShapeName(SkelMesh->GetName() + TEXT("_blendShapes"));
			FbxBlendShape* BlendShape = FbxBlendShape::Create(Mesh, TCHAR_TO_UTF8(*BlendShapeName));
			bool bHasBadMorphTarget = false;

			for (UMorphTarget* MorphTarget : SkelMesh->GetMorphTargets())
			{
				int32 DeformerIndex = Mesh->AddDeformer(BlendShape);
				FbxBlendShapeChannel* BlendShapeChannel = FbxBlendShapeChannel::Create(BlendShape, TCHAR_TO_UTF8(*MorphTarget->GetName()));

				if (BlendShape->AddBlendShapeChannel(BlendShapeChannel))
				{
					FbxShape* Shape = FbxShape::Create(BlendShapeChannel, TCHAR_TO_UTF8(*MorphTarget->GetName()));
					Shape->InitControlPoints(VertexCount);
					FbxVector4* ShapeControlPoints = Shape->GetControlPoints();

					// Replicate the base mesh in the shape control points to set up the data.
					for (int32 VertIndex = 0; VertIndex < VertexCount; ++VertIndex)
					{
						FVector Position = static_cast<FVector>(Vertices[VertIndex].Position);
						ShapeControlPoints[VertIndex] = Converter.ConvertToFbxPos(Position);
					}
				
					for (const FMorphTargetDelta& CurrentDelta: MorphTarget->GetMorphTargetDeltas(LODIndex))
					{
						// Apply the morph target deltas to the control points.
						uint32 RemappedSourceIndex = CurrentDelta.SourceIdx;

						if (VertexIndexOffsetPairArray.Num() > 1)
						{
							//If the skeletal mesh contains clothing we need to remap the morph target index too.
							int32 UpperBoundIndex = Algo::UpperBoundBy(VertexIndexOffsetPairArray, RemappedSourceIndex,
								[](const auto& CurrentPair) { return CurrentPair.Key; }); //Value functor
							RemappedSourceIndex -= VertexIndexOffsetPairArray[UpperBoundIndex - 1].Value;
						}

						if ( RemappedSourceIndex < static_cast<uint32>( VertexCount ) )
						{
							ShapeControlPoints[RemappedSourceIndex] = Converter.ConvertToFbxPos(FVector(Vertices[RemappedSourceIndex].Position + CurrentDelta.PositionDelta));
						}
						else
						{
							bHasBadMorphTarget = true;
						}
					}

					BlendShapeChannel->AddTargetShape(Shape);
					FName MorphTargetName = MorphTarget->GetFName();
					const FCurveMetaData* CurveMetaData = SkelMesh->GetSkeleton()->GetCurveMetaData(MorphTargetName);
					if (AnimSeq && CurveMetaData && CurveMetaData->Type.bMorphtarget)
					{
						FbxAnimCurve* AnimCurve = Mesh->GetShapeChannel(DeformerIndex, BlendShape->GetBlendShapeChannelCount() - 1, AnimLayer, true);
						BlendShapeCurvesMap.Add(MorphTargetName, AnimCurve);
					}
				}
			}

			if ( bHasBadMorphTarget )
			{
				UE_LOGF( LogFbx, Warning, "Encountered corrupted morphtarget(s) during export of SkeletalMesh %ls, bad vertices were ignored.", *SkelMesh->GetName() );
			}
		}

		if(AnimSeq && BlendShapeCurvesMap.Num() > 0)
		{
			ExportCustomAnimCurvesToFbx(BlendShapeCurvesMap, AnimSeq,
				FFrameTime(0),													// Start frame to export
				FFrameTime(AnimSeq->GetDataModel()->GetNumberOfFrames()),		// Final frame to export
				1.f,												// Frame rate scale
				0.f,												// FBX StartTime
				100.f);		// ValueScale, for some reason we need to scale BlendShape curves by a factor of 100.
		}
	}

	FbxNode* MeshNode = FbxNode::Create(Scene, TCHAR_TO_UTF8(MeshName));
	MeshNode->SetNodeAttribute(Mesh);

	// Add the materials for the mesh
	const TArray<FSkeletalMaterial>& SkelMeshMaterials = SkelMesh->GetMaterials();
	int32 MaterialCount = SkelMeshMaterials.Num();

	//TODO: move the init of MaterialBakingMeshData up a callstack step:
	FFbxMaterialBakingMeshData MaterialBakingMeshData(SkelMesh, nullptr, 0);

	for(int32 MaterialIndex = 0; MaterialIndex < MaterialCount; ++MaterialIndex)
	{
		UMaterialInterface* MatInterface = nullptr;

		if (OverrideMaterials && OverrideMaterials->IsValidIndex(MaterialIndex))
		{
			MatInterface = (*OverrideMaterials)[MaterialIndex];
		}
		else
		{
			MatInterface = SkelMeshMaterials[MaterialIndex].MaterialInterface;
		}

		FbxSurfaceMaterial* FbxMaterial = nullptr;
		if (LODIndex == 0)
		{
			if (MatInterface)
			{
				FbxMaterial = ExportMaterial(MatInterface, MaterialIndex, MaterialBakingMeshData);
			}
		}
		else if(MatInterface)
		{
			TMap<int32, FbxSurfaceMaterial*>* MaterialIndexToFbxMaterials = FbxMaterials.Find(MatInterface);
			if (MaterialIndexToFbxMaterials && MaterialIndexToFbxMaterials->Find(MaterialIndex))
			{
				if (FbxSurfaceMaterial** FbxMaterialPtr = MaterialIndexToFbxMaterials->Find(MaterialIndex))
				{
					FbxMaterial = *FbxMaterialPtr;
				}
			}
			
		}

		if(!FbxMaterial)
		{
			// Note: The vertex data relies on there being a set number of Materials.  
			// If you try to add the same material again it will not be added, so create a 
			// default material with a unique name to ensure the proper number of materials

			TCHAR NewMaterialName[MAX_SPRINTF] = TEXT("");
			FCString::Sprintf(NewMaterialName, TEXT("Fbx Default Material %i"), MaterialIndex);

			FbxMaterial = FbxSurfaceLambert::Create(Scene, TCHAR_TO_UTF8(NewMaterialName));
			((FbxSurfaceLambert*)FbxMaterial)->Diffuse.Set(FbxDouble3(0.72, 0.72, 0.72));
		}

		MeshNode->AddMaterial(FbxMaterial);
	}

	int32 SavedMaterialCount = MeshNode->GetMaterialCount();
	check(SavedMaterialCount == MaterialCount);

	return MeshNode;
}


/**
 * Adds Fbx Clusters necessary to skin a skeletal mesh to the bones in the BoneNodes list
 */
void FFbxExporter::BindMeshToSkeleton(const USkeletalMesh* SkelMesh, FbxNode* MeshRootNode, TArray<FbxNode*>& BoneNodes, int32 LODIndex)
{
	const FSkeletalMeshModel* SkelMeshResource = SkelMesh->GetImportedModel();
	if (!SkelMeshResource->LODModels.IsValidIndex(LODIndex))
	{
		//We cannot bind the LOD if its not valid
		return;
	}
	const FSkeletalMeshLODModel& SourceModel = SkelMeshResource->LODModels[LODIndex];
	const int32 VertexCount = SourceModel.NumVertices;

	FbxAMatrix MeshMatrix;

	FbxScene* lScene = MeshRootNode->GetScene();
	if( lScene ) 
	{
		MeshMatrix = MeshRootNode->EvaluateGlobalTransform();
	}
	
	FbxGeometry* MeshAttribute = (FbxGeometry*) MeshRootNode->GetNodeAttribute();
	FbxSkin* Skin = FbxSkin::Create(Scene, "");
	
	const int32 BoneCount = BoneNodes.Num();
	for(int32 BoneIndex = 0; BoneIndex < BoneCount; ++BoneIndex)
	{
		FbxNode* BoneNode = BoneNodes[BoneIndex];

		// Create the deforming cluster
		FbxCluster *CurrentCluster = FbxCluster::Create(Scene,"");
		CurrentCluster->SetLink(BoneNode);
		CurrentCluster->SetLinkMode(FbxCluster::eTotalOne);

		// Add all the vertices that are weighted to the current skeletal bone to the cluster
		// NOTE: the bone influence indices contained in the vertex data are based on a per-chunk
		// list of verts.  The convert the chunk bone index to the mesh bone index, the chunk's
		// boneMap is needed
		int32 VertIndex = 0;
		const int32 SectionCount = SourceModel.Sections.Num();
		for(int32 SectionIndex = 0; SectionIndex < SectionCount; ++SectionIndex)
		{
			const FSkelMeshSection& Section = SourceModel.Sections[SectionIndex];
			if (Section.HasClothingData())
			{
				continue;
			}

			for(int32 SoftIndex = 0; SoftIndex < Section.SoftVertices.Num(); ++SoftIndex)
			{
				const FSoftSkinVertex& Vert = Section.SoftVertices[SoftIndex];

				for(int32 InfluenceIndex = 0; InfluenceIndex < MAX_TOTAL_INFLUENCES; ++InfluenceIndex)
				{
					const int32 InfluenceBone		= Section.BoneMap[ Vert.InfluenceBones[InfluenceIndex] ];
					const float InfluenceWeight	= Vert.InfluenceWeights[InfluenceIndex] / UE::AnimationCore::MaxRawBoneWeightFloat;

					if(InfluenceBone == BoneIndex && InfluenceWeight > 0.f)
					{
						CurrentCluster->AddControlPointIndex(VertIndex, InfluenceWeight);
					}
				}

				++VertIndex;
			}
		}

		// Now we have the Patch and the skeleton correctly positioned,
		// set the Transform and TransformLink matrix accordingly.
		CurrentCluster->SetTransformMatrix(MeshMatrix);

		FbxAMatrix LinkMatrix;
		if( lScene )
		{
			LinkMatrix = BoneNode->EvaluateGlobalTransform();
		}

		CurrentCluster->SetTransformLinkMatrix(LinkMatrix);

		// Add the clusters to the mesh by creating a skin and adding those clusters to that skin.
		Skin->AddCluster(CurrentCluster);
	}

	// Add the skin to the mesh after the clusters have been added
	MeshAttribute->AddDeformer(Skin);
}


// Add the specified node to the node array. Also, add recursively
// all the parent node of the specified node to the array.
void AddNodeRecursively(FbxArray<FbxNode*>& pNodeArray, FbxNode* pNode)
{
	if (pNode)
	{
		AddNodeRecursively(pNodeArray, pNode->GetParent());

		if (pNodeArray.Find(pNode) == -1)
		{
			// Node not in the list, add it
			pNodeArray.Add(pNode);
		}
	}
}


/**
 * Add a bind pose to the scene based on the FbxMesh and skinning settings of the given node
 */
void FFbxExporter::CreateBindPose(FbxNode* MeshRootNode)
{
	if (!MeshRootNode)
	{
		return;
	}

	// In the bind pose, we must store all the link's global matrix at the time of the bind.
	// Plus, we must store all the parent(s) global matrix of a link, even if they are not
	// themselves deforming any model.

	// Create a bind pose with the link list

	FbxArray<FbxNode*> lClusteredFbxNodes;
	int                       i, j;

	if (MeshRootNode->GetNodeAttribute())
	{
		int lSkinCount=0;
		int lClusterCount=0;
		switch (MeshRootNode->GetNodeAttribute()->GetAttributeType())
		{
		case FbxNodeAttribute::eMesh:
		case FbxNodeAttribute::eNurbs:
		case FbxNodeAttribute::ePatch:

			lSkinCount = ((FbxGeometry*)MeshRootNode->GetNodeAttribute())->GetDeformerCount(FbxDeformer::eSkin);
			//Go through all the skins and count them
			//then go through each skin and get their cluster count
			for(i=0; i<lSkinCount; ++i)
			{
				FbxSkin *lSkin=(FbxSkin*)((FbxGeometry*)MeshRootNode->GetNodeAttribute())->GetDeformer(i, FbxDeformer::eSkin);
				lClusterCount+=lSkin->GetClusterCount();
			}
			break;
		}
		//if we found some clusters we must add the node
		if (lClusterCount)
		{
			//Again, go through all the skins get each cluster link and add them
			for (i=0; i<lSkinCount; ++i)
			{
				FbxSkin *lSkin=(FbxSkin*)((FbxGeometry*)MeshRootNode->GetNodeAttribute())->GetDeformer(i, FbxDeformer::eSkin);
				lClusterCount=lSkin->GetClusterCount();
				for (j=0; j<lClusterCount; ++j)
				{
					FbxNode* lClusterNode = lSkin->GetCluster(j)->GetLink();
					AddNodeRecursively(lClusteredFbxNodes, lClusterNode);
				}

			}

			// Add the patch to the pose
			lClusteredFbxNodes.Add(MeshRootNode);
		}
	}

	// Now create a bind pose with the link list
	if (lClusteredFbxNodes.GetCount())
	{
		// A pose must be named. Arbitrarily use the name of the patch node.
		FbxPose* lPose = FbxPose::Create(Scene, MeshRootNode->GetName());

		// default pose type is rest pose, so we need to set the type as bind pose
		lPose->SetIsBindPose(true);

		for (i=0; i<lClusteredFbxNodes.GetCount(); i++)
		{
			FbxNode*  lKFbxNode   = lClusteredFbxNodes.GetAt(i);
			FbxMatrix lBindMatrix = lKFbxNode->EvaluateGlobalTransform();

			lPose->Add(lKFbxNode, lBindMatrix);
		}

		// Add the pose to the scene
		Scene->AddPose(lPose);
	}
}

void FFbxExporter::ExportSkeletalMeshComponent(USkeletalMeshComponent* SkelMeshComp, const TCHAR* MeshName, FbxNode* ActorRootNode, INodeNameAdapter& NodeNameAdapter, bool bSaveAnimSeq)
{
	if (SkelMeshComp && SkelMeshComp->GetSkeletalMeshAsset())
	{
		UAnimSequence* AnimSeq = (bSaveAnimSeq && SkelMeshComp->GetAnimationMode() == EAnimationMode::AnimationSingleNode) ? 
			Cast<UAnimSequence>(SkelMeshComp->AnimationData.AnimToPlay) : nullptr;
		FbxNode* SkeletonRootNode = ExportSkeletalMeshToFbx(SkelMeshComp->GetSkeletalMeshAsset(), AnimSeq, MeshName, ActorRootNode, &ToRawPtrTArrayUnsafe(SkelMeshComp->OverrideMaterials));
		if(SkeletonRootNode)
		{
			FbxSkeletonRoots.Add(SkelMeshComp, SkeletonRootNode);
			NodeNameAdapter.AddFbxNode(SkelMeshComp, SkeletonRootNode);
		}
	}
}

void ExportObjectMetadataToBones(const UObject* ObjectToExport, const TArray<FbxNode*>& Nodes)
{
	if (!ObjectToExport || Nodes.Num() == 0)
	{
		return;
	}

	// Retrieve the metadata map without creating it
	const TMap<FName, FString>* MetadataMap = FMetaData::GetMapForObject(ObjectToExport);
	if (MetadataMap)
	{
		// Map the nodes to their names for fast access
		TMap<FString, FbxNode*> NameToNode;
		for (FbxNode* Node : Nodes)
		{
			NameToNode.Add(FString(Node->GetName()), Node);
		}

		static const FString MetadataPrefix(FBX_METADATA_PREFIX);
		for (const auto& MetadataIt : *MetadataMap)
		{
			// Export object metadata tags that are prefixed as FBX custom user-defined properties
			// Remove the prefix since it's for Unreal use only (and '.' is considered an invalid character for user property names in DCC like Maya)
			FString TagAsString = MetadataIt.Key.ToString();
			if (TagAsString.RemoveFromStart(MetadataPrefix))
			{
				// Extract the node name from the metadata tag
				FString NodeName;
				int32 CharPos = INDEX_NONE;
				if (TagAsString.FindChar(TEXT('.'), CharPos))
				{
					NodeName = TagAsString.Left(CharPos);

					// The remaining part is the actual metadata tag
					TagAsString.RightChopInline(CharPos + 1, EAllowShrinking::No); // exclude the period
				}

				// Try to attach the metadata to its associated node by name
				FbxNode** Node = NameToNode.Find(NodeName);
				if (Node)
				{
					if (MetadataIt.Value == TEXT("true") || MetadataIt.Value == TEXT("false"))
					{
						FbxProperty Property = FbxProperty::Create(*Node, FbxBoolDT, TCHAR_TO_UTF8(*TagAsString));
						FbxBool ValueBool = MetadataIt.Value == TEXT("true") ? true : false;

						Property.Set(ValueBool);
						Property.ModifyFlag(FbxPropertyFlags::eUserDefined, true);
					}
					else
					{
						FbxProperty Property = FbxProperty::Create(*Node, FbxStringDT, TCHAR_TO_UTF8(*TagAsString));
						FbxString ValueString(TCHAR_TO_UTF8(*MetadataIt.Value));

						Property.Set(ValueString);
						Property.ModifyFlag(FbxPropertyFlags::eUserDefined, true);
					}
				}
			}
		}
	}
}

/**
 * Add the given skeletal mesh to the Fbx scene in preparation for exporting.  Makes all new nodes a child of the given node
 */
FbxNode* FFbxExporter::ExportSkeletalMeshToFbx(const USkeletalMesh* SkeletalMesh, const UAnimSequence* AnimSeq, const TCHAR* MeshName, FbxNode* ActorRootNode, const TArray<UMaterialInterface*>* OverrideMaterials /*= nullptr*/)
{
	if(AnimSeq)
	{
		return ExportAnimSequence(AnimSeq, SkeletalMesh, GetExportOptions()->bExportPreviewMesh, MeshName, ActorRootNode, OverrideMaterials);

	}
	else
	{
		//Create a temporary node attach to the scene root.
		//This will allow us to do the binding without the scene transform (non uniform scale is not supported when binding the skeleton)
		//We then detach from the temp node and attach to the parent and remove the temp node
		FString FbxNodeName = FGuid::NewGuid().ToString(EGuidFormats::Digits);
		FbxNode* TmpNodeNoTransform = FbxNode::Create(Scene, TCHAR_TO_UTF8(*FbxNodeName));
		Scene->GetRootNode()->AddChild(TmpNodeNoTransform);

		TArray<FbxNode*> BoneNodes;

		// Add the skeleton to the scene
		FbxNode* SkeletonRootNode = CreateSkeleton(SkeletalMesh, BoneNodes);
		if(SkeletonRootNode)
		{
			TmpNodeNoTransform->AddChild(SkeletonRootNode);
		}

		FbxNode* MeshRootNode = nullptr;
		if (GetExportOptions()->LevelOfDetail && SkeletalMesh->GetLODNum() > 1)
		{
			FString LodGroup_MeshName = FString(MeshName) + TEXT("_LodGroup");
			MeshRootNode = FbxNode::Create(Scene, TCHAR_TO_UTF8(*LodGroup_MeshName));
			TmpNodeNoTransform->AddChild(MeshRootNode);
			LodGroup_MeshName = FString(MeshName) + TEXT("_LodGroupAttribute");
			FbxLODGroup *FbxLodGroupAttribute = FbxLODGroup::Create(Scene, TCHAR_TO_UTF8(*LodGroup_MeshName));
			MeshRootNode->AddNodeAttribute(FbxLodGroupAttribute);

			FbxLodGroupAttribute->ThresholdsUsedAsPercentage = true;
			//Export an Fbx Mesh Node for every LOD and child them to the fbx node (LOD Group)
			for (int CurrentLodIndex = 0; CurrentLodIndex < SkeletalMesh->GetLODNum(); ++CurrentLodIndex)
			{
				FString FbxLODNodeName = FString(MeshName) + TEXT("_LOD") + FString::FromInt(CurrentLodIndex);
				if (CurrentLodIndex + 1 < SkeletalMesh->GetLODNum())
				{
					//Convert the screen size to a threshold, it is just to be sure that we set some threshold, there is no way to convert this precisely
					double LodScreenSize = (double)(10.0f / SkeletalMesh->GetLODInfo(CurrentLodIndex)->ScreenSize.Default);
					FbxLodGroupAttribute->AddThreshold(LodScreenSize);
				}
				const UAnimSequence* NullAnimSeq = nullptr;
				FbxNode* FbxActorLOD = CreateMesh(SkeletalMesh, *FbxLODNodeName, CurrentLodIndex, NullAnimSeq, OverrideMaterials);
				if (FbxActorLOD)
				{
					MeshRootNode->AddChild(FbxActorLOD);
					if (SkeletonRootNode)
					{
						// Bind the mesh to the skeleton
						BindMeshToSkeleton(SkeletalMesh, FbxActorLOD, BoneNodes, CurrentLodIndex);
						// Add the bind pose
						CreateBindPose(FbxActorLOD);
					}
				}
			}
		}
		else
		{
			const int32 LODIndex = 0;
			const UAnimSequence* NullAnimSeq = nullptr;
			MeshRootNode = CreateMesh(SkeletalMesh, MeshName, LODIndex, NullAnimSeq, OverrideMaterials);
			if (MeshRootNode)
			{
				TmpNodeNoTransform->AddChild(MeshRootNode);
				if (SkeletonRootNode)
				{
					// Bind the mesh to the skeleton
					BindMeshToSkeleton(SkeletalMesh, MeshRootNode, BoneNodes, 0);

					// Add the bind pose
					CreateBindPose(MeshRootNode);
				}
			}
		}

		if (SkeletonRootNode)
		{
			TmpNodeNoTransform->RemoveChild(SkeletonRootNode);
			ActorRootNode->AddChild(SkeletonRootNode);
		}

		ExportObjectMetadataToBones(SkeletalMesh->GetSkeleton(), BoneNodes);

		if (MeshRootNode)
		{
			TmpNodeNoTransform->RemoveChild(MeshRootNode);
			ActorRootNode->AddChild(MeshRootNode);
			ExportObjectMetadata(SkeletalMesh, MeshRootNode);
		}

		Scene->GetRootNode()->RemoveChild(TmpNodeNoTransform);
		Scene->RemoveNode(TmpNodeNoTransform);
		return SkeletonRootNode;
	}
}

} // namespace UnFbx
