// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Skeletal mesh creation from FBX data.
	Largely based on SkeletalMeshImport.cpp
=============================================================================*/

#include "CoreMinimal.h"
#include "EngineDefines.h"
#include "Misc/MessageDialog.h"
#include "Containers/IndirectArray.h"
#include "Containers/UnrealString.h"
#include "Stats/Stats.h"
#include "Async/AsyncWork.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FeedbackContext.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/PackageName.h"
#include "Animation/Skeleton.h"
#include "Materials/Material.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkeletalMeshLODSettings.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Components/SkinnedMeshComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "AnimEncoding.h"
#include "Factories/Factory.h"
#include "Factories/FbxSkeletalMeshImportData.h"
#include "Animation/AnimationSettings.h"
#include "Animation/MorphTarget.h"
#include "PhysicsAssetUtils.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Misc/ScopedSlowTask.h"

#include "ImportUtils/SkeletalMeshImportUtils.h"
#include "ImportUtils/SkelImport.h"
#include "Logging/TokenizedMessage.h"
#include "FbxImporter.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetNotifications.h"

#include "ObjectTools.h"

#include "MeshUtilities.h"
#include "OverlappingCorners.h"

#include "IMessageLogListing.h"
#include "MessageLogModule.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "ComponentReregisterContext.h"

#include "Misc/FbxErrors.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "Engine/SkeletalMeshSocket.h"
#include "ClothingAsset.h"
#include "LODUtilities.h"
#include "IMeshBuilderModule.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Misc/CoreMisc.h"
#include "Preferences/PersonaOptions.h"

#define LOCTEXT_NAMESPACE "FBXImpoter"

using namespace UnFbx;

// Get the geometry deformation local to a node. It is never inherited by the
// children.
FbxAMatrix GetGeometry(FbxNode* pNode) 
{
	FbxVector4 lT, lR, lS;
	FbxAMatrix lGeometry;

	lT = pNode->GetGeometricTranslation(FbxNode::eSourcePivot);
	lR = pNode->GetGeometricRotation(FbxNode::eSourcePivot);
	lS = pNode->GetGeometricScaling(FbxNode::eSourcePivot);

	lGeometry.SetT(lT);
	lGeometry.SetR(lR);
	lGeometry.SetS(lS);

	return lGeometry;
}

// Scale all the elements of a matrix.
void MatrixScale(FbxAMatrix& pMatrix, double pValue)
{
	int32 i,j;

	for (i = 0; i < 4; i++)
	{
		for (j = 0; j < 4; j++)
		{
			pMatrix[i][j] *= pValue;
		}
	}
}

// Add a value to all the elements in the diagonal of the matrix.
void MatrixAddToDiagonal(FbxAMatrix& pMatrix, double pValue)
{
	pMatrix[0][0] += pValue;
	pMatrix[1][1] += pValue;
	pMatrix[2][2] += pValue;
	pMatrix[3][3] += pValue;
}


// Sum two matrices element by element.
void MatrixAdd(FbxAMatrix& pDstMatrix, FbxAMatrix& pSrcMatrix)
{
	int32 i,j;

	for (i = 0; i < 4; i++)
	{
		for (j = 0; j < 4; j++)
		{
			pDstMatrix[i][j] += pSrcMatrix[i][j];
		}
	}
}

/** Helper struct for the mesh component vert position octree */
struct FSkeletalMeshVertPosOctreeSemantics
{
	enum { MaxElementsPerLeaf = 16 };
	enum { MinInclusiveElementsPerNode = 7 };
	enum { MaxNodeDepth = 12 };

	typedef TInlineAllocator<MaxElementsPerLeaf> ElementAllocator;

	/**
	 * Get the bounding box of the provided octree element. In this case, the box
	 * is merely the point specified by the element.
	 *
	 * @param	Element	Octree element to get the bounding box for
	 *
	 * @return	Bounding box of the provided octree element
	 */
	FORCEINLINE static FBoxCenterAndExtent GetBoundingBox(const FSoftSkinVertex& Element)
	{
		return FBoxCenterAndExtent((FVector)Element.Position, FVector::ZeroVector);
	}

	/**
	 * Determine if two octree elements are equal
	 *
	 * @param	A	First octree element to check
	 * @param	B	Second octree element to check
	 *
	 * @return	true if both octree elements are equal, false if they are not
	 */
	FORCEINLINE static bool AreElementsEqual(const FSoftSkinVertex& A, const FSoftSkinVertex& B)
	{
		return (A.Position == B.Position && A.UVs[0] == B.UVs[0]);
	}

	/** Ignored for this implementation */
	FORCEINLINE static void SetElementId(const FSoftSkinVertex& Element, FOctreeElementId2 Id)
	{
	}
};
typedef TOctree2<FSoftSkinVertex, FSkeletalMeshVertPosOctreeSemantics> TSKCVertPosOctree;

void RemapSkeletalMeshVertexColorToImportData(const USkeletalMesh* SkeletalMesh, const int32 LODIndex, FSkeletalMeshImportData* SkelMeshImportData)
{
	//Make sure we have all the source data we need to do the remap
	if (!SkeletalMesh->GetImportedModel() || !SkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(LODIndex) || !SkeletalMesh->GetHasVertexColors())
	{
		return;
	}

	// Find the extents formed by the cached vertex positions in order to optimize the octree used later
	FBox Bounds(ForceInitToZero);
	SkelMeshImportData->bHasVertexColors = true;

	int32 WedgeNumber = SkelMeshImportData->Wedges.Num();
	for (int32 WedgeIndex = 0; WedgeIndex < WedgeNumber; ++WedgeIndex)
	{
		SkeletalMeshImportData::FVertex& Wedge = SkelMeshImportData->Wedges[WedgeIndex];
		const FVector& Position = (FVector)SkelMeshImportData->Points[Wedge.VertexIndex];
		Bounds += Position;
	}

	TArray<FSoftSkinVertex> Vertices;
	SkeletalMesh->GetImportedModel()->LODModels[LODIndex].GetVertices(Vertices);
	for (int32 SkinVertexIndex = 0; SkinVertexIndex < Vertices.Num(); ++SkinVertexIndex)
	{
		const FSoftSkinVertex& SkinVertex = Vertices[SkinVertexIndex];
		Bounds += (FVector)SkinVertex.Position;
	}

	TSKCVertPosOctree VertPosOctree(Bounds.GetCenter(), Bounds.GetExtent().GetMax());

	// Add each old vertex to the octree
	for (int32 SkinVertexIndex = 0; SkinVertexIndex < Vertices.Num(); ++SkinVertexIndex)
	{
		const FSoftSkinVertex& SkinVertex = Vertices[SkinVertexIndex];
		VertPosOctree.AddElement(SkinVertex);
	}

	TMap<int32, FVector3f> WedgeIndexToNormal;
	WedgeIndexToNormal.Reserve(WedgeNumber);
	for (int32 FaceIndex = 0; FaceIndex < SkelMeshImportData->Faces.Num(); ++FaceIndex)
	{
		const SkeletalMeshImportData::FTriangle& Triangle = SkelMeshImportData->Faces[FaceIndex];
		for (int32 Corner = 0; Corner < 3; ++Corner)
		{
			WedgeIndexToNormal.Add(Triangle.WedgeIndex[Corner], Triangle.TangentZ[Corner]);
		}
	}

	// Iterate over each new vertex position, attempting to find the old vertex it is closest to, applying
	// the color of the old vertex to the new position if possible.
	for (int32 WedgeIndex = 0; WedgeIndex < WedgeNumber; ++WedgeIndex)
	{
		SkeletalMeshImportData::FVertex& Wedge = SkelMeshImportData->Wedges[WedgeIndex];
		const FVector& Position = (FVector)SkelMeshImportData->Points[Wedge.VertexIndex];
		const FVector2f UV = Wedge.UVs[0];
		const FVector3f& Normal = WedgeIndexToNormal.FindChecked(WedgeIndex);

		TArray<FSoftSkinVertex> PointsToConsider;
		VertPosOctree.FindNearbyElements(Position, [&PointsToConsider](const FSoftSkinVertex& Vertex)
		{
			PointsToConsider.Add(Vertex);
		});

		if (PointsToConsider.Num() > 0)
		{
			//Get the closest position
			float MaxNormalDot = -MAX_FLT;
			float MinUVDistance = MAX_FLT;
			int32 MatchIndex = INDEX_NONE;
			for (int32 ConsiderationIndex = 0; ConsiderationIndex < PointsToConsider.Num(); ++ConsiderationIndex)
			{
				const FSoftSkinVertex& SkinVertex = PointsToConsider[ConsiderationIndex];
				const FVector2f& SkinVertexUV = SkinVertex.UVs[0];
				const float UVDistanceSqr = FVector2f::DistSquared(UV, SkinVertexUV);
				if (UVDistanceSqr < MinUVDistance)
				{
					MinUVDistance = FMath::Min(MinUVDistance, UVDistanceSqr);
					MatchIndex = ConsiderationIndex;
					MaxNormalDot = Normal | SkinVertex.TangentZ;
				}
				else if (FMath::IsNearlyEqual(UVDistanceSqr, MinUVDistance, KINDA_SMALL_NUMBER))
				{
					//This case is useful when we have hard edge that shared vertice, somtime not all the shared wedge have the same paint color
					//Think about a cube where each face have different vertex color.
					float NormalDot = Normal | SkinVertex.TangentZ;
					if (NormalDot > MaxNormalDot)
					{
						MaxNormalDot = NormalDot;
						MatchIndex = ConsiderationIndex;
					}
				}
			}
			if (PointsToConsider.IsValidIndex(MatchIndex))
			{
				Wedge.Color = PointsToConsider[MatchIndex].Color;
			}
		}
	}
}

// Utility function for creating a unique material import name based on the node's initial name and its material/section index
FString MakeNodeMaterialIndexName(FbxNode* Node, int32 MaterialIndex)
{
	FString NodeName = FFbxImporter::MakeName(Node->GetInitialName());
	NodeName.Append("_");
	NodeName.AppendInt(MaterialIndex);

	return NodeName;
}

void FFbxImporter::SkinControlPointsToPose(FSkeletalMeshImportData& ImportData, FbxMesh* FbxMesh, FbxShape* FbxShape, bool bUseT0 )
{
	FbxTime poseTime = FBXSDK_TIME_INFINITE;
	if(bUseT0)
	{
		poseTime = 0;
	}

	int32 VertexCount = FbxMesh->GetControlPointsCount();

	// Create a copy of the vertex array to receive vertex deformations.
	FbxVector4* VertexArray = new FbxVector4[VertexCount];

	// If a shape is provided, then it is the morphed version of the mesh.
	// So we want to deform that instead of the base mesh vertices
	if (FbxShape)
	{
		check(FbxShape->GetControlPointsCount() == VertexCount);
		FMemory::Memcpy(VertexArray, FbxShape->GetControlPoints(), VertexCount * sizeof(FbxVector4));
	}
	else
	{
		FMemory::Memcpy(VertexArray, FbxMesh->GetControlPoints(), VertexCount * sizeof(FbxVector4));
	}																	 


	int32 ClusterCount = 0;
	int32 SkinCount = FbxMesh->GetDeformerCount(FbxDeformer::eSkin);
	for( int32 i=0; i< SkinCount; i++)
	{
		ClusterCount += ((FbxSkin *)(FbxMesh->GetDeformer(i, FbxDeformer::eSkin)))->GetClusterCount();
	}
	
	// Deform the vertex array with the links contained in the mesh.
	if (ClusterCount)
	{
		FbxAMatrix MeshMatrix = ComputeTotalMatrix(FbxMesh->GetNode());
		// All the links must have the same link mode.
		FbxCluster::ELinkMode lClusterMode = ((FbxSkin*)FbxMesh->GetDeformer(0, FbxDeformer::eSkin))->GetCluster(0)->GetLinkMode();

		int32 i, j;
		int32 lClusterCount=0;

		int32 lSkinCount = FbxMesh->GetDeformerCount(FbxDeformer::eSkin);

		TArray<FbxAMatrix> ClusterDeformations;
		ClusterDeformations.AddZeroed( VertexCount );

		TArray<double> ClusterWeights;
		ClusterWeights.AddZeroed( VertexCount );
	

		if (lClusterMode == FbxCluster::eAdditive)
		{
			for (i = 0; i < VertexCount; i++)
			{
				ClusterDeformations[i].SetIdentity();
			}
		}

		for ( i=0; i<lSkinCount; ++i)
		{
			lClusterCount =( (FbxSkin *)FbxMesh->GetDeformer(i, FbxDeformer::eSkin))->GetClusterCount();
			for (j=0; j<lClusterCount; ++j)
			{
				FbxCluster* Cluster =((FbxSkin *) FbxMesh->GetDeformer(i, FbxDeformer::eSkin))->GetCluster(j);
				if (!Cluster->GetLink())
					continue;
					
				FbxNode* Link = Cluster->GetLink();
				
				FbxAMatrix lReferenceGlobalInitPosition;
				FbxAMatrix lReferenceGlobalCurrentPosition;
				FbxAMatrix lClusterGlobalInitPosition;
				FbxAMatrix lClusterGlobalCurrentPosition;
				FbxAMatrix lReferenceGeometry;
				FbxAMatrix lClusterGeometry;

				FbxAMatrix lClusterRelativeInitPosition;
				FbxAMatrix lClusterRelativeCurrentPositionInverse;
				FbxAMatrix lVertexTransformMatrix;

				if (lClusterMode == FbxCluster::eAdditive && Cluster->GetAssociateModel())
				{
					Cluster->GetTransformAssociateModelMatrix(lReferenceGlobalInitPosition);
					lReferenceGlobalCurrentPosition = Scene->GetAnimationEvaluator()->GetNodeGlobalTransform(Cluster->GetAssociateModel(), poseTime);
					// Geometric transform of the model
					lReferenceGeometry = GetGeometry(Cluster->GetAssociateModel());
					lReferenceGlobalCurrentPosition *= lReferenceGeometry;
				}
				else
				{
					Cluster->GetTransformMatrix(lReferenceGlobalInitPosition);
					lReferenceGlobalCurrentPosition = MeshMatrix; //pGlobalPosition;
					// Multiply lReferenceGlobalInitPosition by Geometric Transformation
					lReferenceGeometry = GetGeometry(FbxMesh->GetNode());
					lReferenceGlobalInitPosition *= lReferenceGeometry;
				}
				// Get the link initial global position and the link current global position.
				Cluster->GetTransformLinkMatrix(lClusterGlobalInitPosition);
				lClusterGlobalCurrentPosition = Link->GetScene()->GetAnimationEvaluator()->GetNodeGlobalTransform(Link, poseTime);

				// Compute the initial position of the link relative to the reference.
				lClusterRelativeInitPosition = lClusterGlobalInitPosition.Inverse() * lReferenceGlobalInitPosition;

				// Compute the current position of the link relative to the reference.
				lClusterRelativeCurrentPositionInverse = lReferenceGlobalCurrentPosition.Inverse() * lClusterGlobalCurrentPosition;

				// Compute the shift of the link relative to the reference.
				lVertexTransformMatrix = lClusterRelativeCurrentPositionInverse * lClusterRelativeInitPosition;

				int32 k;
				int32 lVertexIndexCount = Cluster->GetControlPointIndicesCount();

				for (k = 0; k < lVertexIndexCount; ++k) 
				{
					int32 lIndex = Cluster->GetControlPointIndices()[k];

					// Sometimes, the mesh can have less points than at the time of the skinning
					// because a smooth operator was active when skinning but has been deactivated during export.
					if (lIndex >= VertexCount)
					{
						continue;
					}

					double lWeight = Cluster->GetControlPointWeights()[k];

					if (lWeight == 0.0)
					{
						continue;
					}

					// Compute the influence of the link on the vertex.
					FbxAMatrix lInfluence = lVertexTransformMatrix;
					MatrixScale(lInfluence, lWeight);

					if (lClusterMode == FbxCluster::eAdditive)
					{
						// Multiply with to the product of the deformations on the vertex.
						MatrixAddToDiagonal(lInfluence, 1.0 - lWeight);
						ClusterDeformations[lIndex] = lInfluence * ClusterDeformations[lIndex];

						// Set the link to 1.0 just to know this vertex is influenced by a link.
						ClusterWeights[lIndex] = 1.0;
					}
					else // lLinkMode == KFbxLink::eNORMALIZE || lLinkMode == KFbxLink::eTOTAL1
					{
						// Add to the sum of the deformations on the vertex.
						MatrixAdd(ClusterDeformations[lIndex], lInfluence);

						// Add to the sum of weights to either normalize or complete the vertex.
						ClusterWeights[lIndex] += lWeight;
					}

				}
			}
		}
		
		for (i = 0; i < VertexCount; i++) 
		{
			FbxVector4 lSrcVertex = VertexArray[i];
			FbxVector4& lDstVertex = VertexArray[i];
			double lWeight = ClusterWeights[i];

			// Deform the vertex if there was at least a link with an influence on the vertex,
			if (lWeight != 0.0) 
			{
				lDstVertex = ClusterDeformations[i].MultT(lSrcVertex);

				if (lClusterMode == FbxCluster::eNormalize)
				{
					// In the normalized link mode, a vertex is always totally influenced by the links. 
					lDstVertex /= lWeight;
				}
				else if (lClusterMode == FbxCluster::eTotalOne)
				{
					// In the total 1 link mode, a vertex can be partially influenced by the links. 
					lSrcVertex *= (1.0 - lWeight);
					lDstVertex += lSrcVertex;
				}
			} 
		}
		
		{
			FbxAMatrix TotalMatrixForNormal = MeshMatrix.Inverse().Transpose();

			FbxLayer* BaseLayer = FbxMesh->GetLayer(0);
			FbxLayerElementNormal* LayerElementNormal = BaseLayer->GetNormals();
			FbxLayerElementTangent* LayerElementTangent = BaseLayer->GetTangents();
			FbxLayerElementBinormal* LayerElementBinormal = BaseLayer->GetBinormals();

			bool OddNegativeScale = IsOddNegativeScale(MeshMatrix);

			FbxLayerElement::EReferenceMode NormalReferenceMode(LayerElementNormal ? LayerElementNormal->GetReferenceMode() : FbxLayerElement::eDirect);
			FbxLayerElement::EMappingMode NormalMappingMode(LayerElementNormal ? LayerElementNormal->GetMappingMode() : FbxLayerElement::eByControlPoint);
			
			FbxLayerElement::EReferenceMode TangentReferenceMode(LayerElementTangent ? LayerElementTangent->GetReferenceMode() : FbxLayerElement::eDirect);
			FbxLayerElement::EMappingMode TangentMappingMode(LayerElementTangent ? LayerElementTangent->GetMappingMode() : FbxLayerElement::eByControlPoint);

			FbxLayerElement::EReferenceMode BinormalReferenceMode(LayerElementBinormal ? LayerElementBinormal->GetReferenceMode() : FbxLayerElement::eDirect);
			FbxLayerElement::EMappingMode BinormalMappingMode(LayerElementBinormal ? LayerElementBinormal->GetMappingMode() : FbxLayerElement::eByControlPoint);

			const int32 TriangleCount = FbxMesh->GetPolygonCount();
			const int32 ExistingTriangleCount = ImportData.Faces.Num();
			const int32 StartTriangleIndex = ExistingTriangleCount - TriangleCount;
			for (int32 TriangleIndex = 0; TriangleIndex < TriangleCount; ++TriangleIndex)
			{
				SkeletalMeshImportData::FTriangle& Triangle = ImportData.Faces[StartTriangleIndex + TriangleIndex];

				for (int32 VertexIndex = 0; VertexIndex < 3; ++VertexIndex)
				{
					int32 TmpIndex = TriangleIndex * 3 + VertexIndex;

					int32 UnrealVertexIndex = OddNegativeScale ? 2 - VertexIndex : VertexIndex;

					int32 ControlPointIndex = FbxMesh->GetPolygonVertex(TriangleIndex, VertexIndex);

					int32 NormalMapIndex = (NormalMappingMode == FbxLayerElement::eByControlPoint) ? ControlPointIndex : TmpIndex;
					int32 NormalValueIndex = (NormalReferenceMode == FbxLayerElement::eDirect) ? NormalMapIndex : LayerElementNormal->GetIndexArray().GetAt(NormalMapIndex);

					FbxAMatrix ClusterDeformation = ClusterDeformations[ControlPointIndex].Inverse().Transpose();

					if (ImportData.bHasNormals)
					{
						if (ImportData.bHasTangents)
						{
							if (LayerElementTangent)
							{
								int32 TangentMapIndex = (TangentMappingMode == FbxLayerElement::eByControlPoint) ? ControlPointIndex : TmpIndex;
								int32 TangentValueIndex = (TangentReferenceMode == FbxLayerElement::eDirect) ? TangentMapIndex : LayerElementTangent->GetIndexArray().GetAt(TangentMapIndex);

								FbxVector4 TangentX = LayerElementTangent->GetDirectArray().GetAt(TangentValueIndex);
								TangentX = ClusterDeformation.MultT(TangentX);
								TangentX = TotalMatrixForNormal.MultT(TangentX);
								Triangle.TangentX[UnrealVertexIndex] = (FVector3f)Converter.ConvertDir(TangentX);
								Triangle.TangentX[UnrealVertexIndex].Normalize();
							}
							else
							{
								Triangle.TangentX[UnrealVertexIndex] = FVector3f();
							}

							if (LayerElementBinormal)
							{
								int32 BinormalMapIndex = (BinormalMappingMode == FbxLayerElement::eByControlPoint) ? ControlPointIndex : TmpIndex;
								int32 BinormalValueIndex = (BinormalReferenceMode == FbxLayerElement::eDirect) ? BinormalMapIndex : LayerElementBinormal->GetIndexArray().GetAt(BinormalMapIndex);

								FbxVector4 TangentY = LayerElementBinormal->GetDirectArray().GetAt(BinormalValueIndex);
								TangentY = ClusterDeformation.MultT(TangentY);
								TangentY = TotalMatrixForNormal.MultT(TangentY);
								Triangle.TangentY[UnrealVertexIndex] = (FVector3f)Converter.ConvertDir(TangentY);
								Triangle.TangentY[UnrealVertexIndex].Normalize();
							}
							else
							{
								Triangle.TangentY[UnrealVertexIndex] = FVector3f();
							}
						}

						if (LayerElementNormal)
						{
							FbxVector4 TangentZ = LayerElementNormal->GetDirectArray().GetAt(NormalValueIndex);
							TangentZ = ClusterDeformation.MultT(TangentZ);
							TangentZ = TotalMatrixForNormal.MultT(TangentZ);
							Triangle.TangentZ[UnrealVertexIndex] = (FVector3f)Converter.ConvertDir(TangentZ);
							Triangle.TangentZ[UnrealVertexIndex].Normalize();
						}
						else
						{
							Triangle.TangentZ[UnrealVertexIndex] = FVector3f();
						}
					}
				}
			}
		}

		// change the vertex position
		int32 ExistPointNum = ImportData.Points.Num();
		int32 StartPointIndex = ExistPointNum - VertexCount;
		for(int32 ControlPointsIndex = 0 ; ControlPointsIndex < VertexCount ;ControlPointsIndex++ )
		{
			ImportData.Points[ControlPointsIndex+StartPointIndex] = (FVector3f)Converter.ConvertPos(MeshMatrix.MultT(VertexArray[ControlPointsIndex]));
		}
		
	}
	
	delete[] VertexArray;
}

// 3 "ProcessImportMesh..." functions outputing Unreal data from a filled FSkeletalMeshBinaryImport
// and a handfull of other minor stuff needed by these 
// Fully taken from SkeletalMeshImport.cpp


struct tFaceRecord
{
	int32 FaceIndex;
	int32 HoekIndex;
	int32 WedgeIndex;
	uint32 SmoothFlags;
	uint32 FanFlags;
};

struct VertsFans
{	
	TArray<tFaceRecord> FaceRecord;
	int32 FanGroupCount;
};

struct tInfluences
{
	TArray<int32> RawInfIndices;
};

struct tWedgeList
{
	TArray<int32> WedgeList;
};

struct tFaceSet
{
	TArray<int32> Faces;
};

// Check whether faces have at least two vertices in common. These must be POINTS - don't care about wedges.
bool UnFbx::FFbxImporter::FacesAreSmoothlyConnected( FSkeletalMeshImportData &ImportData, int32 Face1, int32 Face2 )
{

	//if( ( Face1 >= Thing->SkinData.Faces.Num()) || ( Face2 >= Thing->SkinData.Faces.Num()) ) return false;

	if( Face1 == Face2 )
	{
		return true;
	}

	// Smoothing groups match at least one bit in binary AND ?
	if( ( ImportData.Faces[Face1].SmoothingGroups & ImportData.Faces[Face2].SmoothingGroups ) == 0  ) 
	{
		return false;
	}

	int32 VertMatches = 0;
	for( int32 i=0; i<3; i++)
	{
		int32 Point1 = ImportData.Wedges[ ImportData.Faces[Face1].WedgeIndex[i] ].VertexIndex;

		for( int32 j=0; j<3; j++)
		{
			int32 Point2 = ImportData.Wedges[ ImportData.Faces[Face2].WedgeIndex[j] ].VertexIndex;
			if( Point2 == Point1 )
			{
				VertMatches ++;
			}
		}
	}

	return ( VertMatches >= 2 );
}

int32	UnFbx::FFbxImporter::DoUnSmoothVerts(FSkeletalMeshImportData &ImportData, bool bDuplicateUnSmoothWedges)
{
	//
	// Connectivity: triangles with non-matching smoothing groups will be physically split.
	//
	// -> Splitting involves: the UV+material-contaning vertex AND the 3d point.
	//
	// -> Tally smoothing groups for each and every (textured) vertex.
	//
	// -> Collapse: 
	// -> start from a vertex and all its adjacent triangles - go over
	// each triangle - if any connecting one (sharing more than one vertex) gives a smoothing match,
	// accumulate it. Then IF more than one resulting section, 
	// ensure each boundary 'vert' is split _if not already_ to give each smoothing group
	// independence from all others.
	//

	int32 DuplicatedVertCount = 0;
	int32 RemappedHoeks = 0;

	int32 TotalSmoothMatches = 0;
	int32 TotalConnexChex = 0;

	// Link _all_ faces to vertices.	
	TArray<VertsFans>  Fans;
	TArray<tInfluences> PointInfluences;	
	TArray<tWedgeList>  PointWedges;

	Fans.AddZeroed( ImportData.Points.Num() );//Fans.AddExactZeroed(			Thing->SkinData.Points.Num() );
	PointInfluences.AddZeroed( ImportData.Points.Num() );//PointInfluences.AddExactZeroed( Thing->SkinData.Points.Num() );
	PointWedges.AddZeroed( ImportData.Points.Num() );//PointWedges.AddExactZeroed(	 Thing->SkinData.Points.Num() );

	// Existing points map 1:1
	ImportData.PointToRawMap.AddUninitialized( ImportData.Points.Num() );
	for(int32 i=0; i<ImportData.Points.Num(); i++)
	{
		ImportData.PointToRawMap[i] = i;
	}

	for(int32 i=0; i< ImportData.Influences.Num(); i++)
	{
		if (PointInfluences.Num() <= ImportData.Influences[i].VertexIndex)
		{
			PointInfluences.AddZeroed(ImportData.Influences[i].VertexIndex - PointInfluences.Num() + 1);
		}
		PointInfluences[ImportData.Influences[i].VertexIndex ].RawInfIndices.Add( i );
	}

	for(int32 i=0; i< ImportData.Wedges.Num(); i++)
	{
		if (uint32(PointWedges.Num()) <= ImportData.Wedges[i].VertexIndex)
		{
			PointWedges.AddZeroed(ImportData.Wedges[i].VertexIndex - PointWedges.Num() + 1);
		}

		PointWedges[ImportData.Wedges[i].VertexIndex ].WedgeList.Add( i );
	}

	for(int32 f=0; f< ImportData.Faces.Num(); f++ )
	{
		// For each face, add a pointer to that face into the Fans[vertex].
		for( int32 i=0; i<3; i++)
		{
			int32 WedgeIndex = ImportData.Faces[f].WedgeIndex[i];			
			int32 PointIndex = ImportData.Wedges[ WedgeIndex ].VertexIndex;
			tFaceRecord NewFR;

			NewFR.FaceIndex = f;
			NewFR.HoekIndex = i;			
			NewFR.WedgeIndex = WedgeIndex; // This face touches the point courtesy of Wedges[Wedgeindex].
			NewFR.SmoothFlags = ImportData.Faces[f].SmoothingGroups;
			NewFR.FanFlags = 0;
			Fans[ PointIndex ].FaceRecord.Add( NewFR );
			Fans[ PointIndex ].FanGroupCount = 0;
		}		
	}

	// Investigate connectivity and assign common group numbers (1..+) to the fans' individual FanFlags.
	for( int32 p=0; p< Fans.Num(); p++) // The fan of faces for each 3d point 'p'.
	{
		// All faces connecting.
		if( Fans[p].FaceRecord.Num() > 0 ) 
		{		
			int32 FacesProcessed = 0;
			TArray<tFaceSet> FaceSets; // Sets with indices INTO FANS, not into face array.			

			// Digest all faces connected to this vertex (p) into one or more smooth sets. only need to check 
			// all faces MINUS one..
			while( FacesProcessed < Fans[p].FaceRecord.Num()  )
			{
				// One loop per group. For the current ThisFaceIndex, tally all truly connected ones
				// and put them in a new TArray. Once no more can be connected, stop.

				int32 NewSetIndex = FaceSets.Num(); // 0 to start
				FaceSets.AddZeroed(1);						// first one will be just ThisFaceIndex.

				// Find the first non-processed face. There will be at least one.
				int32 ThisFaceFanIndex = 0;
				{
					int32 SearchIndex = 0;
					while( Fans[p].FaceRecord[SearchIndex].FanFlags == -1 ) // -1 indicates already  processed. 
					{
						SearchIndex++;
					}
					ThisFaceFanIndex = SearchIndex; //Fans[p].FaceRecord[SearchIndex].FaceIndex; 
				}

				// Initial face.
				FaceSets[ NewSetIndex ].Faces.Add( ThisFaceFanIndex );   // Add the unprocessed Face index to the "local smoothing group" [NewSetIndex].
				Fans[p].FaceRecord[ThisFaceFanIndex].FanFlags = -1;			  // Mark as processed.
				FacesProcessed++; 

				// Find all faces connected to this face, and if there's any
				// smoothing group matches, put it in current face set and mark it as processed;
				// until no more match. 
				int32 NewMatches = 0;
				do
				{
					NewMatches = 0;
					// Go over all current faces in this faceset and set if the FaceRecord (local smoothing groups) has any matches.
					// there will be at least one face already in this faceset - the first face in the fan.
					for( int32 n=0; n< FaceSets[NewSetIndex].Faces.Num(); n++)
					{				
						int32 HookFaceIdx = Fans[p].FaceRecord[ FaceSets[NewSetIndex].Faces[n] ].FaceIndex;

						//Go over the fan looking for matches.
						for( int32 s=0; s< Fans[p].FaceRecord.Num(); s++)
						{
							// Skip if same face, skip if face already processed.
							if( ( HookFaceIdx != Fans[p].FaceRecord[s].FaceIndex )  && ( Fans[p].FaceRecord[s].FanFlags != -1  ))
							{
								TotalConnexChex++;
								// Process if connected with more than one vertex, AND smooth..
								if( FacesAreSmoothlyConnected( ImportData, HookFaceIdx, Fans[p].FaceRecord[s].FaceIndex ) )
								{									
									TotalSmoothMatches++;
									Fans[p].FaceRecord[s].FanFlags = -1; // Mark as processed.
									FacesProcessed++;
									// Add 
									FaceSets[NewSetIndex].Faces.Add( s ); // Store FAN index of this face index into smoothing group's faces. 
									// Tally
									NewMatches++;
								}
							} // not the same...
						}// all faces in fan
					} // all faces in FaceSet
				}while( NewMatches );	

			}// Repeat until all faces processed.

			// For the new non-initialized  face sets, 
			// Create a new point, influences, and uv-vertex(-ices) for all individual FanFlag groups with an index of 2+ and also remap
			// the face's vertex into those new ones.
			if( FaceSets.Num() > 1 )
			{
				for( int32 f=1; f<FaceSets.Num(); f++ )
				{				
					check(ImportData.Points.Num() == ImportData.PointToRawMap.Num());

					// We duplicate the current vertex. (3d point)
					int32 NewPointIndex = ImportData.Points.Num();
					ImportData.Points.AddUninitialized();
					ImportData.Points[NewPointIndex] = ImportData.Points[p] ;

					ImportData.PointToRawMap.AddUninitialized();
					ImportData.PointToRawMap[NewPointIndex] = p;
					
					DuplicatedVertCount++;

					// Duplicate all related weights.
					for( int32 t=0; t< PointInfluences[p].RawInfIndices.Num(); t++ )
					{
						// Add new weight
						int32 NewWeightIndex = ImportData.Influences.Num();
						ImportData.Influences.AddUninitialized();
						ImportData.Influences[NewWeightIndex] = ImportData.Influences[ PointInfluences[p].RawInfIndices[t] ];
						ImportData.Influences[NewWeightIndex].VertexIndex = NewPointIndex;
					}

					// Duplicate any and all Wedges associated with it; and all Faces' wedges involved.					
					for( int32 w=0; w< PointWedges[p].WedgeList.Num(); w++)
					{
						int32 OldWedgeIndex = PointWedges[p].WedgeList[w];
						int32 NewWedgeIndex = ImportData.Wedges.Num();

						if( bDuplicateUnSmoothWedges )
						{
							ImportData.Wedges.AddUninitialized();
							ImportData.Wedges[NewWedgeIndex] = ImportData.Wedges[ OldWedgeIndex ];
							ImportData.Wedges[ NewWedgeIndex ].VertexIndex = NewPointIndex; 

							//  Update relevant face's Wedges. Inelegant: just check all associated faces for every new wedge.
							for( int32 s=0; s< FaceSets[f].Faces.Num(); s++)
							{
								int32 FanIndex = FaceSets[f].Faces[s];
								if( Fans[p].FaceRecord[ FanIndex ].WedgeIndex == OldWedgeIndex )
								{
									// Update just the right one for this face (HoekIndex!) 
									ImportData.Faces[ Fans[p].FaceRecord[ FanIndex].FaceIndex ].WedgeIndex[ Fans[p].FaceRecord[ FanIndex ].HoekIndex ] = NewWedgeIndex;
									RemappedHoeks++;
								}
							}
						}
						else
						{
							ImportData.Wedges[OldWedgeIndex].VertexIndex = NewPointIndex; 
						}
					}
				}
			} //  if FaceSets.Num(). -> duplicate stuff
		}//	while( FacesProcessed < Fans[p].FaceRecord.Num() )
	} // Fans for each 3d point

	check(ImportData.Points.Num() == ImportData.PointToRawMap.Num());

	return DuplicatedVertCount; 
}

bool UnFbx::FFbxImporter::IsUnrealBone(FbxNode* Link)
{
	FbxNodeAttribute* Attr = Link->GetNodeAttribute();
	if (Attr)
	{
		FbxNodeAttribute::EType AttrType = Attr->GetAttributeType();
		if ( AttrType == FbxNodeAttribute::eSkeleton ||
			AttrType == FbxNodeAttribute::eMesh ||
			AttrType == FbxNodeAttribute::eNull )
		{
			return !IsUnrealTransformAttribute(Link);
		}
	}
	
	return false;
}

bool UnFbx::FFbxImporter::IsUnrealTransformAttribute(FbxNode* Link)
{
	FbxNodeAttribute* Attr = Link->GetNodeAttribute();
	if (Attr)
	{
		FbxNodeAttribute::EType AttrType = Attr->GetAttributeType();
		if (AttrType == FbxNodeAttribute::eNull)
		{
			FString AttributeName = FSkeletalMeshImportData::FixupBoneName(MakeName(Link->GetName()));

			for (const FString& TransformAttributeName : UAnimationSettings::Get()->TransformAttributeNames)
			{
				if (AttributeName.MatchesWildcard(TransformAttributeName, ESearchCase::IgnoreCase))
				{
					return true;
				}
			}
		}
	}

	return false;
}

void UnFbx::FFbxImporter::RecursiveBuildSkeleton(FbxNode* Link, TArray<FbxNode*>& OutSortedLinks)
{
	if (IsUnrealBone(Link))
	{
		OutSortedLinks.Add(Link);
		int32 ChildIndex;
		for (ChildIndex=0; ChildIndex<Link->GetChildCount(); ChildIndex++)
		{
			RecursiveBuildSkeleton(Link->GetChild(ChildIndex),OutSortedLinks);
		}
	}
	else if (IsUnrealTransformAttribute(Link))
	{
		OutSortedLinks.Add(Link);
	}
}

void UnFbx::FFbxImporter::BuildSkeletonSystem(TArray<FbxCluster*>& ClusterArray, TArray<FbxNode*>& OutSortedLinks)
{
	FbxNode* Link;
	TArray<FbxNode*> RootLinks;
	int32 ClusterIndex;
	for (ClusterIndex = 0; ClusterIndex < ClusterArray.Num(); ClusterIndex++)
	{
		Link = ClusterArray[ClusterIndex]->GetLink();
		if (Link)
		{
			Link = GetRootSkeleton(Link);
			int32 LinkIndex;
			for (LinkIndex = 0; LinkIndex < RootLinks.Num(); LinkIndex++)
			{
				if (Link == RootLinks[LinkIndex])
				{
					break;
				}
			}

			// this link is a new root, add it
			if (LinkIndex == RootLinks.Num())
			{
				RootLinks.Add(Link);
			}
		}
	}

	for (int32 LinkIndex=0; LinkIndex<RootLinks.Num(); LinkIndex++)
	{
		RecursiveBuildSkeleton(RootLinks[LinkIndex], OutSortedLinks);
	}
}

bool UnFbx::FFbxImporter::RetrievePoseFromBindPose(const TArray<FbxNode*>& NodeArray, FbxArray<FbxPose*> & PoseArray) const
{
	const int32 PoseCount = Scene->GetPoseCount();
	for(int32 PoseIndex = 0; PoseIndex < PoseCount; PoseIndex++)
	{
		FbxPose* CurrentPose = Scene->GetPose(PoseIndex);

		// current pose is bind pose, 
		if(CurrentPose && CurrentPose->IsBindPose())
		{
			// IsValidBindPose doesn't work reliably
			// It checks all the parent chain(regardless root given), and if the parent doesn't have correct bind pose, it fails
			// It causes more false positive issues than the real issue we have to worry about
			// If you'd like to try this, set CHECK_VALID_BIND_POSE to 1, and try the error message
			// when Autodesk fixes this bug, then we might be able to re-open this
			FString PoseName = CurrentPose->GetName();
			// all error report status
			FbxStatus Status;

			// it does not make any difference of checking with different node
			// it is possible pose 0 -> node array 2, but isValidBindPose function returns true even with node array 0
			for(auto Current : NodeArray)
			{
				FString CurrentName = Current->GetName();
				NodeList pMissingAncestors, pMissingDeformers, pMissingDeformersAncestors, pWrongMatrices;

				if(CurrentPose->IsValidBindPoseVerbose(Current, pMissingAncestors, pMissingDeformers, pMissingDeformersAncestors, pWrongMatrices, 0.0001, &Status))
				{
					PoseArray.Add(CurrentPose);
					UE_LOGF(LogFbx, Log, "Valid bind pose for Pose (%ls) - %ls", *PoseName, *FString(Current->GetName()));
					break;
				}
				else
				{
					// first try to fix up
					// add missing ancestors
					for(int i = 0; i < pMissingAncestors.GetCount(); i++)
					{
						FbxAMatrix mat = pMissingAncestors.GetAt(i)->EvaluateGlobalTransform(FBXSDK_TIME_ZERO);
						CurrentPose->Add(pMissingAncestors.GetAt(i), mat);
					}

					pMissingAncestors.Clear();
					pMissingDeformers.Clear();
					pMissingDeformersAncestors.Clear();
					pWrongMatrices.Clear();

					// check it again
					if(CurrentPose->IsValidBindPose(Current))
					{
						PoseArray.Add(CurrentPose);
						UE_LOGF(LogFbx, Log, "Valid bind pose for Pose (%ls) - %ls", *PoseName, *FString(Current->GetName()));
						break;
					}
					else
					{
						// first try to find parent who is null group and see if you can try test it again
						FbxNode * ParentNode = Current->GetParent();
						while(ParentNode)
						{
							FbxNodeAttribute* Attr = ParentNode->GetNodeAttribute();
							if(Attr && Attr->GetAttributeType() == FbxNodeAttribute::eNull)
							{
								// found it 
								break;
							}

							// find next parent
							ParentNode = ParentNode->GetParent();
						}

						if(ParentNode && CurrentPose->IsValidBindPose(ParentNode))
						{
							PoseArray.Add(CurrentPose);
							UE_LOGF(LogFbx, Log, "Valid bind pose for Pose (%ls) - %ls", *PoseName, *FString(Current->GetName()));
							break;
						}
						else
						{
							FString ErrorString = Status.GetErrorString();
							if (!GIsAutomationTesting)
								UE_LOGF(LogFbx, Warning, "Not valid bind pose for Pose (%ls) - Node %ls : %ls", *PoseName, *FString(Current->GetName()), *ErrorString);
						}
					}
				}
			}
		}
	}

	return (PoseArray.Size() > 0);
}

bool UnFbx::FFbxImporter::ImportBones(TArray<FbxNode*>& NodeArray, FSkeletalMeshImportData &ImportData, UFbxSkeletalMeshImportData* TemplateData ,TArray<FbxNode*> &SortedLinks, bool& bOutDiffPose, bool bDisableMissingBindPoseWarning, bool & bUseTime0AsRefPose, FbxNode *SkeletalMeshNode, bool bIsReimport)
{
	bOutDiffPose = false;
	bool bIsARigidMesh = false;
	FbxNode* Link = NULL;
	FbxArray<FbxPose*> PoseArray;
	TArray<FbxCluster*> ClusterArray;
	
	bool AllowImportBoneErrorAndWarning = !(bIsReimport && ImportOptions->bImportAsSkeletalGeometry);
	if (NodeArray[0]->GetMesh()->GetDeformerCount(FbxDeformer::eSkin) == 0)
	{
		bIsARigidMesh = true;
		Link = NodeArray[0];
		RecursiveBuildSkeleton(GetRootSkeleton(Link),SortedLinks);
	}
	else
	{
		// get bindpose and clusters from FBX skeleton
		
		// let's put the elements to their bind pose! (and we restore them after
		// we have built the ClusterInformation.
		int32 Default_NbPoses = SdkManager->GetBindPoseCount(Scene);
		// If there are no BindPoses, the following will generate them.
		//SdkManager->CreateMissingBindPoses(Scene);

		//if we created missing bind poses, update the number of bind poses
		int32 NbPoses = SdkManager->GetBindPoseCount(Scene);

		if ( NbPoses != Default_NbPoses && AllowImportBoneErrorAndWarning)
		{
			AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, LOCTEXT("FbxSkeletaLMeshimport_SceneMissingBinding", "The imported scene has no initial binding position (Bind Pose) for the skin. The plug-in will compute one automatically. However, it may create unexpected results.")), FFbxErrors::SkeletalMesh_NoBindPoseInScene);
		}

		//
		// create the bones / skinning
		//

		for ( int32 i = 0; i < NodeArray.Num(); i++)
		{
			FbxMesh* FbxMesh = NodeArray[i]->GetMesh();
			const int32 SkinDeformerCount = FbxMesh->GetDeformerCount(FbxDeformer::eSkin);
			for (int32 DeformerIndex=0; DeformerIndex < SkinDeformerCount; DeformerIndex++)
			{
				FbxSkin* Skin = (FbxSkin*)FbxMesh->GetDeformer(DeformerIndex, FbxDeformer::eSkin);
				for ( int32 ClusterIndex = 0; ClusterIndex < Skin->GetClusterCount(); ClusterIndex++)
				{
					ClusterArray.Add(Skin->GetCluster(ClusterIndex));
				}
			}
		}

		if (ClusterArray.Num() == 0)
		{
			if (AllowImportBoneErrorAndWarning)
			{
				AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, LOCTEXT("FbxSkeletaLMeshimport_NoAssociatedCluster", "No associated clusters")), FFbxErrors::SkeletalMesh_NoAssociatedCluster);
			}
			return false;
		}

		// get bind pose
		if(RetrievePoseFromBindPose(NodeArray, PoseArray) == false)
		{
			if (!GIsAutomationTesting)
				UE_LOGF(LogFbx, Warning, "Getting valid bind pose failed. Try to recreate bind pose");
			// if failed, delete bind pose, and retry.
			const int32 PoseCount = Scene->GetPoseCount();
			for(int32 PoseIndex = PoseCount-1; PoseIndex >= 0; --PoseIndex)
			{
				FbxPose* CurrentPose = Scene->GetPose(PoseIndex);

				// current pose is bind pose, 
				if(CurrentPose && CurrentPose->IsBindPose())
				{
					Scene->RemovePose(PoseIndex);
					CurrentPose->Destroy();
				}
			}

			SdkManager->CreateMissingBindPoses(Scene);
			if ( RetrievePoseFromBindPose(NodeArray, PoseArray) == false)
			{
				if (!GIsAutomationTesting)
					UE_LOGF(LogFbx, Warning, "Recreating bind pose failed.");
			}
			else
			{
				if (!GIsAutomationTesting)
					UE_LOGF(LogFbx, Warning, "Recreating bind pose succeeded.");
			}
		}

		// recurse through skeleton and build ordered table
		BuildSkeletonSystem(ClusterArray, SortedLinks);
	}

	// error check
	// if no bond is found
	if(SortedLinks.Num() == 0)
	{
		if (AllowImportBoneErrorAndWarning)
		{
			AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("FbxSkeletaLMeshimport_NoBone", "'{0}' has no bones"), FText::FromString(NodeArray[0]->GetName()))), FFbxErrors::SkeletalMesh_NoBoneFound);
		}
		return false;
	}

	// if no bind pose is found
	if (!bUseTime0AsRefPose && PoseArray.GetCount() == 0)
	{
		// add to tokenized error message
		if (ImportOptions->bImportScene)
		{
			AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("FbxSkeletaLMeshimport_InvalidBindPose", "Skeletal Mesh '{0}' dont have a bind pose. Scene import do not support yet time 0 as bind pose, there will be no bind pose import"), FText::FromString(NodeArray[0]->GetName()))), FFbxErrors::SkeletalMesh_InvalidBindPose);
		}
		else
		{
			if (!GIsAutomationTesting)
			{
				AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, LOCTEXT("FbxSkeletaLMeshimport_MissingBindPose", "Could not find the bind pose.  It will use time 0 as bind pose.")), FFbxErrors::SkeletalMesh_InvalidBindPose);
			}
			bUseTime0AsRefPose = true;
		}
	}

	int32 LinkIndex;

	// Check for duplicate bone names and issue a warning if found
	for(LinkIndex = 0; LinkIndex < SortedLinks.Num(); ++LinkIndex)
	{
		Link = SortedLinks[LinkIndex];

		for(int32 AltLinkIndex = LinkIndex+1; AltLinkIndex < SortedLinks.Num(); ++AltLinkIndex)
		{
			FbxNode* AltLink = SortedLinks[AltLinkIndex];

			if(FCStringAnsi::Strcmp(Link->GetName(), AltLink->GetName()) == 0)
			{
				if (AllowImportBoneErrorAndWarning)
				{
					FString RawBoneName = UTF8_TO_TCHAR(Link->GetName());
					AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("Error_DuplicateBoneName", "Error, Could not import {0}.\nDuplicate bone name found ('{1}'). Each bone must have a unique name."),
						FText::FromString(NodeArray[0]->GetName()), FText::FromString(RawBoneName))), FFbxErrors::SkeletalMesh_DuplicateBones);
				}
				return false;
			}
		}
	}

	FbxArray<FbxAMatrix> GlobalsPerLink;
	GlobalsPerLink.Grow(SortedLinks.Num());
	GlobalsPerLink[0].SetIdentity();

	bool GlobalLinkFoundFlag;
	FbxVector4 LocalLinkT;
	FbxQuaternion LocalLinkQ;
	FbxVector4	LocalLinkS;

	bool bAnyLinksNotInBindPose = false;
	FString LinksWithoutBindPoses;

	int32 RootIdx = INDEX_NONE;

	for (LinkIndex=0; LinkIndex<SortedLinks.Num(); LinkIndex++)
	{
		// Add a bone for each FBX Link
		ImportData.RefBonesBinary.Emplace(SkeletalMeshImportData::FBone());
		Link = SortedLinks[LinkIndex];
		int32 ParentIndex = INDEX_NONE; // base value for root if no parent found

		if (LinkIndex == 0)
		{
			RootIdx = LinkIndex;
		}
		else
		{
			const FbxNode* LinkParent = Link->GetParent();
			// get the link parent index.
			for (int32 ll = 0; ll < LinkIndex; ++ll) // <LinkIndex because parent is guaranteed to be before child in sortedLink
			{
				FbxNode* Otherlink = SortedLinks[ll];
				if (Otherlink == LinkParent)
				{
					ParentIndex = ll;
					break;
				}
			}

			if (ParentIndex == INDEX_NONE)//We found another root inside the hierarchy, this is not supported
			{
				if (AllowImportBoneErrorAndWarning)
				{
					AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("MultipleRootsFound", "Multiple roots are found in the bone hierarchy. We only support single root bone.")), FFbxErrors::SkeletalMesh_MultipleRoots);
				}
				return false;
			}
		}

		GlobalLinkFoundFlag = false;
		if (!bIsARigidMesh) //skeletal mesh
		{
			// there are some links, they have no cluster, but in bindpose
			if (PoseArray.GetCount())
			{
				for (int32 PoseIndex = 0; PoseIndex < PoseArray.GetCount(); PoseIndex++)
				{
					int32 PoseLinkIndex = PoseArray[PoseIndex]->Find(Link);
					if (PoseLinkIndex>=0)
					{
						FbxMatrix NoneAffineMatrix = PoseArray[PoseIndex]->GetMatrix(PoseLinkIndex);
						FbxAMatrix Matrix = *(FbxAMatrix*)(double*)&NoneAffineMatrix;
						GlobalsPerLink[LinkIndex] = Matrix;
						GlobalLinkFoundFlag = true;
						break;
					}
				}
			}

			if (!GlobalLinkFoundFlag)
			{
				// since now we set use time 0 as ref pose this won't unlikely happen
				// but leaving it just in case it still has case where it's missing partial bind pose
				if(!bUseTime0AsRefPose && !bDisableMissingBindPoseWarning)
				{
					bAnyLinksNotInBindPose = true;
					LinksWithoutBindPoses +=  UTF8_TO_TCHAR(Link->GetName());
					LinksWithoutBindPoses +=  TEXT("  \n");
				}

				for (int32 ClusterIndex=0; ClusterIndex<ClusterArray.Num(); ClusterIndex++)
				{
					FbxCluster* Cluster = ClusterArray[ClusterIndex];
					if (Link == Cluster->GetLink())
					{
						Cluster->GetTransformLinkMatrix(GlobalsPerLink[LinkIndex]);
						GlobalLinkFoundFlag = true;
						break;
					}
				}
			}
		}

		if (!GlobalLinkFoundFlag)
		{
			GlobalsPerLink[LinkIndex] = Link->EvaluateGlobalTransform();
		}
		
		if (bUseTime0AsRefPose && !ImportOptions->bImportScene)
		{
			FbxAMatrix& T0Matrix = Scene->GetAnimationEvaluator()->GetNodeGlobalTransform(Link, 0);
			if (GlobalsPerLink[LinkIndex] != T0Matrix)
			{
				bOutDiffPose = true;
			}
			
			GlobalsPerLink[LinkIndex] = T0Matrix;
		}

		//Add the join orientation
		GlobalsPerLink[LinkIndex] = GlobalsPerLink[LinkIndex] * FFbxDataConverter::GetJointPostConversionMatrix();
		if (LinkIndex)
		{
			FbxAMatrix	Matrix;
			Matrix = GlobalsPerLink[ParentIndex].Inverse() * GlobalsPerLink[LinkIndex];
			LocalLinkT = Matrix.GetT();
			LocalLinkQ = Matrix.GetQ();
			LocalLinkS = Matrix.GetS();
		}
		else	// skeleton root
		{
			// for root, this is global coordinate
			LocalLinkT = GlobalsPerLink[LinkIndex].GetT();
			LocalLinkQ = GlobalsPerLink[LinkIndex].GetQ();
			LocalLinkS = GlobalsPerLink[LinkIndex].GetS();
		}
		
		// set bone
		SkeletalMeshImportData::FBone& Bone = ImportData.RefBonesBinary[LinkIndex];
		FString BoneName = MakeName( Link->GetName() );
		Bone.Name = BoneName;

		//Check for nan and for zero scale
		if (AllowImportBoneErrorAndWarning)
		{
			bool bFoundNan = false;
			bool bFoundZeroScale = false;
			for (int32 i = 0; i < 4; ++i)
			{
				if (i < 3)
				{
					if (FMath::IsNaN(LocalLinkT[i]) || FMath::IsNaN(LocalLinkS[i]))
					{
						bFoundNan = true;
					}
					if (FMath::IsNearlyZero(LocalLinkS[i]))
					{
						bFoundZeroScale = true;
					}
				}
				if (FMath::IsNaN(LocalLinkQ[i]))
				{
					bFoundNan = true;
				}
			}

			if (bFoundNan)
			{
				AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("BoneTransformNAN", "Found NAN value in bone transform. Bone name: [{0}]"), FText::FromString(BoneName))), FFbxErrors::SkeletalMesh_InvalidBindPose);
			}

			if (bFoundZeroScale)
			{
				AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("BoneTransformZeroScale", "Found zero scale value in bone transform. Bone name: [{0}]"), FText::FromString(BoneName))), FFbxErrors::SkeletalMesh_InvalidBindPose);
			}
		}

		SkeletalMeshImportData::FJointPos& JointMatrix = Bone.BonePos;
		FbxSkeleton* Skeleton = Link->GetSkeleton();
		if (Skeleton)
		{
			JointMatrix.Length = Converter.ConvertDist(Skeleton->LimbLength.Get());
			JointMatrix.XSize = Converter.ConvertDist(Skeleton->Size.Get());
			JointMatrix.YSize = Converter.ConvertDist(Skeleton->Size.Get());
			JointMatrix.ZSize = Converter.ConvertDist(Skeleton->Size.Get());
		}
		else
		{
			JointMatrix.Length = 1. ;
			JointMatrix.XSize = 100. ;
			JointMatrix.YSize = 100. ;
			JointMatrix.ZSize = 100. ;
		}

		// get the link parent and children
		Bone.ParentIndex = ParentIndex;
		Bone.NumChildren = 0;
		for (int32 ChildIndex=0; ChildIndex<Link->GetChildCount(); ChildIndex++)
		{
			FbxNode* Child = Link->GetChild(ChildIndex);
			if (IsUnrealBone(Child))
			{
				Bone.NumChildren++;
			}
		}

		JointMatrix.Transform.SetTranslation(FVector3f(Converter.ConvertPos(LocalLinkT)));
		JointMatrix.Transform.SetRotation(FQuat4f(Converter.ConvertRotToQuat(LocalLinkQ)));
		JointMatrix.Transform.SetScale3D(FVector3f(Converter.ConvertScale(LocalLinkS)));
	}
	
	//In case we do a scene import we need a relative to skeletal mesh transform instead of a global
	if (ImportOptions->bImportScene && !ImportOptions->bTransformVertexToAbsolute)
	{
		FbxAMatrix GlobalSkeletalNodeFbx = Scene->GetAnimationEvaluator()->GetNodeGlobalTransform(SkeletalMeshNode, 0);
		FTransform3f GlobalSkeletalNode;
		GlobalSkeletalNode.SetFromMatrix(FMatrix44f(Converter.ConvertMatrix(GlobalSkeletalNodeFbx.Inverse())));

		SkeletalMeshImportData::FBone& RootBone = ImportData.RefBonesBinary[RootIdx];
		FTransform3f& RootTransform = RootBone.BonePos.Transform;
		RootTransform.SetFromMatrix(RootTransform.ToMatrixWithScale() * GlobalSkeletalNode.ToMatrixWithScale());
	}

	if(TemplateData)
	{
		FbxAMatrix FbxAddedMatrix;
		BuildFbxMatrixForImportTransform(FbxAddedMatrix, TemplateData);
		FMatrix44f AddedMatrix = FMatrix44f(Converter.ConvertMatrix(FbxAddedMatrix));

		SkeletalMeshImportData::FBone& RootBone = ImportData.RefBonesBinary[RootIdx];
		FTransform3f& RootTransform = RootBone.BonePos.Transform;
		RootTransform.SetFromMatrix(RootTransform.ToMatrixWithScale() * AddedMatrix);
	}
	
	if(bAnyLinksNotInBindPose && AllowImportBoneErrorAndWarning && !GIsAutomationTesting)
	{
		AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("FbxSkeletaLMeshimport_BonesAreMissingFromBindPose", "The following bones are missing from the bind pose:\n{0}\nThis can happen for bones that are not vert weighted. If they are not in the correct orientation after importing,\nplease set the \"Use T0 as ref pose\" option or add them to the bind pose and reimport the skeletal mesh."), FText::FromString(LinksWithoutBindPoses))), FFbxErrors::SkeletalMesh_BonesAreMissingFromBindPose);
	}
	
	return true;
}

bool UnFbx::FFbxImporter::FillSkeletalMeshImportData(TArray<FbxNode*>& NodeArray, UFbxSkeletalMeshImportData* TemplateImportData, TArray<FbxShape*> *FbxShapeArray, FSkeletalMeshImportData* OutData, TArray<FbxNode*>& OutImportedSkeletonLinkNodes, TArray<FName> &LastImportedMaterialNames, const bool bIsReimport, const TMap<FVector3f, FColor>& ExistingVertexColorData, bool& bMapMorphTargetToTimeZero)
{
	if (NodeArray.Num() == 0)
	{
		return false;
	}

	FbxNode* Node = NodeArray[0];
	// find the mesh by its name
	FbxMesh* FbxMesh = Node->GetMesh();

	if (OutData == nullptr)
	{
		return false;
	}

	// Fill with data from buffer - contains the full .FBX file. 	
	FSkeletalMeshImportData* SkelMeshImportDataPtr = OutData;

	OutImportedSkeletonLinkNodes.Empty();
	TArray<FbxNode*>& SortedLinkArray = OutImportedSkeletonLinkNodes;
	FbxArray<FbxAMatrix> GlobalsPerLink;

	bool bDiffPose = false;
	bool bUseT0AsRefPose = ImportOptions->bUseT0AsRefPose;

	// Note: importing morph data causes additional passes through this function, so disable the warning dialogs
	// from popping up again on each additional pass.
	if (!ImportBones(NodeArray, *SkelMeshImportDataPtr, TemplateImportData, SortedLinkArray, bDiffPose, (FbxShapeArray != nullptr), bUseT0AsRefPose, Node, bIsReimport))
	{
		if (!(bIsReimport && ImportOptions->bImportAsSkeletalGeometry)) //Do not import bone if we import only the geometry and we are reimporting
		{
			return false;
		}
	}
	
	bMapMorphTargetToTimeZero = bUseT0AsRefPose & bDiffPose;

 	FbxNode* SceneRootNode = Scene->GetRootNode();
 	if(SceneRootNode && TemplateImportData)
 	{
 		ApplyTransformSettingsToFbxNode(SceneRootNode, TemplateImportData);
 	}

	// Create a list of all unique fbx materials.  This needs to be done as a separate pass before reading geometry
	// so that we know about all possible materials before assigning material indices to each triangle
	TArray<FbxSurfaceMaterial*> FbxMaterials;
	for (int32 NodeIndex = 0; NodeIndex < NodeArray.Num(); ++NodeIndex)
	{
		Node = NodeArray[NodeIndex];

		int32 MaterialCount = Node->GetMaterialCount();

		for (int32 MaterialIndex = 0; MaterialIndex < MaterialCount; ++MaterialIndex)
		{
			FbxSurfaceMaterial* FbxMaterial = Node->GetMaterial(MaterialIndex);

			// Store unique FbxNode materials resulting in sections being combined.
			if (!ImportOptions->bKeepSectionsSeparate)
			{
				if (!FbxMaterials.Contains(FbxMaterial))
				{
					FbxMaterials.Add(FbxMaterial);

					SkeletalMeshImportData::FMaterial NewMaterial;

					NewMaterial.MaterialImportName = MakeName(FbxMaterial->GetName());
					// Add an entry for each unique material
					SkelMeshImportDataPtr->Materials.Add(NewMaterial);
				}
			}
			else
			{
				// Store each FbxNode material regardless of duplicates to avoid sections being combined.
				FbxMaterials.Add(FbxMaterial);

				SkeletalMeshImportData::FMaterial NewMaterial;

				NewMaterial.MaterialImportName = MakeNodeMaterialIndexName(Node, MaterialIndex);
				// Add an entry for each unique material
				SkelMeshImportDataPtr->Materials.Add(NewMaterial);
			}
		}
	}

	for (int32 NodeIndex = 0; NodeIndex < NodeArray.Num(); ++NodeIndex)
	{
		Node = NodeArray[NodeIndex];
		FbxNode *RootNode = NodeArray[0];
		FbxMesh = Node->GetMesh();
		FbxSkin* Skin = (FbxSkin*)FbxMesh->GetDeformer(0, FbxDeformer::eSkin);
		FbxShape* FbxShape = nullptr;
		if (FbxShapeArray)
		{
			FbxShape = (*FbxShapeArray)[NodeIndex];
		}

		// NOTE: This function may invalidate FbxMesh and set it to point to a an updated version
		if (!FillSkelMeshImporterFromFbx( *SkelMeshImportDataPtr, FbxMesh, Skin, FbxShape, SortedLinkArray, FbxMaterials, RootNode, ExistingVertexColorData) )
		{
			return false;
		}

		if (bMapMorphTargetToTimeZero && !ImportOptions->bImportScene)
		{
			// deform skin vertex to the frame 0 from bind pose
			SkinControlPointsToPose(*SkelMeshImportDataPtr, FbxMesh, FbxShape, true);
		}
	}

	CleanUpUnusedMaterials(*SkelMeshImportDataPtr);

	for (SkeletalMeshImportData::FMaterial& MaterialInfo : SkelMeshImportDataPtr->Materials)
	{
		if (MaterialInfo.Material.IsValid())
		{
			MaterialInfo.Material->SetMaterialUsage(MATUSAGE_SkeletalMesh);
		}
	}

	if (LastImportedMaterialNames.Num() > 0)
	{
		SetMaterialOrderByName(*SkelMeshImportDataPtr, LastImportedMaterialNames);
	}
	else
	{
		// reorder material according to "SKinXX" in material name
		SetMaterialSkinXXOrder(*SkelMeshImportDataPtr);
	}

	if (ImportOptions->bPreserveSmoothingGroups)
	{
		bool bDuplicateUnSmoothWedges = false; //We deprecate legacy build
		DoUnSmoothVerts(*SkelMeshImportDataPtr, bDuplicateUnSmoothWedges);
	}
	else
	{
		SkelMeshImportDataPtr->PointToRawMap.AddUninitialized(SkelMeshImportDataPtr->Points.Num());
		for (int32 PointIdx = 0; PointIdx < SkelMeshImportDataPtr->Points.Num(); PointIdx++)
		{
			SkelMeshImportDataPtr->PointToRawMap[PointIdx] = PointIdx;
		}
	}

	return true;
}

bool UnFbx::FFbxImporter::ReplaceSkeletalMeshGeometryImportData(const USkeletalMesh* SkeletalMesh, FSkeletalMeshImportData* ImportData, int32 LodIndex)
{
	return FSkeletalMeshImportData::ReplaceSkeletalMeshGeometryImportData(SkeletalMesh, ImportData, LodIndex);
}

bool UnFbx::FFbxImporter::ReplaceSkeletalMeshSkinningImportData(const USkeletalMesh* SkeletalMesh, FSkeletalMeshImportData* ImportData, int32 LodIndex)
{
	return FSkeletalMeshImportData::ReplaceSkeletalMeshRigImportData(SkeletalMesh, ImportData, LodIndex);
}

bool UnFbx::FFbxImporter::FillSkeletalMeshImportPoints(FSkeletalMeshImportData* OutData, FbxNode* RootNode, FbxNode* Node, FbxShape* FbxShape)
{
	FbxMesh* FbxMesh = Node->GetMesh();

	// Extract the name.
	FString MeshName = MakeName(FbxMesh->GetName());
	if (MeshName.IsEmpty())
	{
		MeshName = MakeName(Node->GetName());
	}

	// Add the mesh info
	OutData->MeshInfos.AddDefaulted();
	SkeletalMeshImportData::FMeshInfo& MeshInfo = OutData->MeshInfos.Last();
	MeshInfo.Name = *MeshName;

	const int32 ControlPointsCount = FbxMesh->GetControlPointsCount();
	const int32 ExistPointNum = OutData->Points.Num();
	OutData->Points.AddUninitialized(ControlPointsCount);
	
	MeshInfo.StartImportedVertex = ExistPointNum;
	MeshInfo.NumVertices = ControlPointsCount;

	// Construct the matrices for the conversion from right handed to left handed system
	FbxAMatrix TotalMatrix = ComputeSkeletalMeshTotalMatrix(Node, RootNode);

	int32 ControlPointsIndex;
	bool bInvalidPositionFound = false;
	for( ControlPointsIndex = 0 ; ControlPointsIndex < ControlPointsCount ;ControlPointsIndex++ )
	{
		FbxVector4 Position;
		if (FbxShape)
		{
			Position = FbxShape->GetControlPoints()[ControlPointsIndex];
		}
		else
		{
			Position = FbxMesh->GetControlPoints()[ControlPointsIndex];
		}																	 

		FbxVector4 FinalPosition;
		FinalPosition = TotalMatrix.MultT(Position);
		FVector ConvertedPosition = Converter.ConvertPos(FinalPosition);

		// ensure user when this happens if attached to debugger
		if (!ensure(ConvertedPosition.ContainsNaN() == false))
		{
			if (!bInvalidPositionFound)
			{
				bInvalidPositionFound = true;
			}

			ConvertedPosition = FVector::ZeroVector;
		}

		OutData->Points[ ControlPointsIndex + ExistPointNum ] = (FVector3f)ConvertedPosition;
	}

	if (bInvalidPositionFound)
	{
		AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning,
			FText::Format(LOCTEXT("FbxSkeletaLMeshimport_InvalidPosition", "Invalid position (NaN or Inf) found from source position for mesh '{0}'. Please verify if the source asset contains valid position. "),
				FText::FromString(FbxMesh->GetName()))), FFbxErrors::SkeletalMesh_InvalidPosition);
	}

	return true;
}

bool UnFbx::FFbxImporter::GatherPointsForMorphTarget(FSkeletalMeshImportData* OutData, TArray<FbxNode*>& NodeArray, TArray< FbxShape* >* FbxShapeArray, TSet<uint32>& ModifiedPoints, const bool bMapMorphTargetToTimeZero)
{
	check(OutData);
	TArray<FVector3f> CompressPoints;
	CompressPoints.Reserve(OutData->Points.Num());
	FSkeletalMeshImportData NewImportData = *OutData;
	NewImportData.Points.Empty();

	FbxNode* const RootNode = NodeArray[0];

	for ( int32 NodeIndex = 0; NodeIndex < NodeArray.Num(); ++NodeIndex )
	{
		FbxNode* Node = NodeArray[NodeIndex];
		FbxMesh* FbxMesh = Node->GetMesh();

		FbxShape* FbxShape = nullptr;
		if (FbxShapeArray && FbxShapeArray->IsValidIndex(NodeIndex))
		{
			FbxShape = (*FbxShapeArray)[NodeIndex];
		}

		FillSkeletalMeshImportPoints( &NewImportData, RootNode, Node, FbxShape );

		if (bMapMorphTargetToTimeZero && !ImportOptions->bImportScene)
		{
			// deform skin vertex to the frame 0 from bind pose
			SkinControlPointsToPose(NewImportData, FbxMesh, FbxShape, true);
		}
	}

	for ( int32 PointIdx = 0; PointIdx < OutData->Points.Num(); ++PointIdx )
	{
		int32 OriginalPointIdx = OutData->PointToRawMap[ PointIdx ];
		//Rebuild the data with only the modified point
		if ( ( NewImportData.Points[ OriginalPointIdx ] - OutData->Points[ PointIdx ] ).SizeSquared() > FMath::Square(THRESH_POINTS_ARE_SAME) )
		{
			ModifiedPoints.Add( PointIdx );
			CompressPoints.Add(NewImportData.Points[OriginalPointIdx]);
		}
	}
	OutData->Points = CompressPoints;
	return true;
}

void UnFbx::FFbxImporter::FillLastImportMaterialNames(TArray<FName> &LastImportedMaterialNames, USkeletalMesh* BaseSkelMesh, TArray<FName> *OrderedMaterialNames)
{
	if (OrderedMaterialNames == nullptr && BaseSkelMesh)
	{
		int32 NoneNameCount = 0;
		for (const FSkeletalMaterial &Material : BaseSkelMesh->GetMaterials())
		{
			if (Material.ImportedMaterialSlotName == NAME_None)
			{
				NoneNameCount++;
			}
			LastImportedMaterialNames.Add(Material.ImportedMaterialSlotName);
		}
		if (NoneNameCount >= LastImportedMaterialNames.Num())
		{
			LastImportedMaterialNames.Empty();
		}
	}
	else if (OrderedMaterialNames)
	{
		//Copy the ordered material name parameter
		LastImportedMaterialNames = (*OrderedMaterialNames);
	}

	//If the imported model is using skinxx workflow just empty LastImportedMaterialNames array
	if (LastImportedMaterialNames.Num() > 0)
	{
		int32 SkinXXNameCount = 0;
		for (FName MaterialName : LastImportedMaterialNames)
		{
			if (MaterialName == NAME_None)
			{
				continue;
			}
			FString ImportedMaterialName = MaterialName.ToString();
			int32 Offset = ImportedMaterialName.Find(TEXT("_SKIN"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
			if (Offset != INDEX_NONE)
			{
				FString SkinXXNumber = ImportedMaterialName.Right(ImportedMaterialName.Len() - (Offset + 1)).RightChop(4);

				if (SkinXXNumber.IsNumeric())
				{
					SkinXXNameCount++;
				}
			}
		}
		//If we have some skinxx suffixe we don't use the name to reorder
		if (SkinXXNameCount == LastImportedMaterialNames.Num())
		{
			LastImportedMaterialNames.Empty();
		}
	}
}

//Small helper macro helping us to do an early return when the import is canceled.
#define EARLY_RETURN_ON_CANCEL(bForce, CancelOperation) \
	if(bForce || (ImportOptions->bIsImportCancelable && SlowTask.ShouldCancel())) \
	{ \
		CancelOperation(); \
		return nullptr; \
	}

USkeletalMesh* UnFbx::FFbxImporter::ImportSkeletalMesh(FImportSkeletalMeshArgs &ImportSkeletalMeshArgs)
{
	if (ImportSkeletalMeshArgs.NodeArray.Num() == 0 || !CanImportClass(USkeletalMesh::StaticClass()))
	{
		return nullptr;
	}
	FFbxScopedOperation ScopedImportOperation(this);

	FScopedSlowTask SlowTask(1);
	SlowTask.EnterProgressFrame(1);
	USkeletalMesh* SkeletalMesh = nullptr;

	// We canceled the import, let the factory delete the assets.
	auto CancelCleanup = [&]()
		{
			bImportOperationCanceled = true;
		};

	// The import failed, we are marking the imported meshes for deletion in the next GC.
	auto FailureCleanup = [&]()
		{
			if (SkeletalMesh)
			{
				SkeletalMesh->ClearFlags(RF_Standalone);
				SkeletalMesh->Rename(NULL, GetTransientPackage(), REN_DontCreateRedirectors);
			}
		};

	EARLY_RETURN_ON_CANCEL(false, CancelCleanup);

	int32 SafeLODIndex = ImportSkeletalMeshArgs.LodIndex < 0 ? 0 : ImportSkeletalMeshArgs.LodIndex;
	FbxNode* Node = ImportSkeletalMeshArgs.NodeArray[0];
	// find the mesh by its name
	FbxMesh* FbxMesh = Node->GetMesh();

	if( !FbxMesh )
	{
		AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("FbxSkeletaLMeshimport_NodeInvalidSkeletalMesh", "Fbx node: '{0}' is not a valid skeletal mesh"), FText::FromString(Node->GetName()))), FFbxErrors::Generic_Mesh_MeshNotFound);
		return nullptr;
	}

	//Make sure the render thread is done
	FlushRenderingCommands();

	// warning for missing smoothing group info
	CheckSmoothingInfo(FbxMesh);

	Parent = ImportSkeletalMeshArgs.InParent;
	
	USkeletalMesh* ExistingSkelMesh = nullptr;
	if ( !ImportSkeletalMeshArgs.FbxShapeArray  )
	{
		UObject* ExistingObject = StaticFindObjectFast(UObject::StaticClass(), ImportSkeletalMeshArgs.InParent, ImportSkeletalMeshArgs.Name, EFindObjectFlags::None, RF_NoFlags, EInternalObjectFlags::Garbage);
		ExistingSkelMesh = Cast<USkeletalMesh>(ExistingObject);

		if (!ExistingSkelMesh && ExistingObject)
		{
			AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("FbxSkeletaLMeshimport_OverlappingName", "Same name but different class: '{0}' already exists"), FText::FromString(ExistingObject->GetName()))), FFbxErrors::Generic_SameNameAssetExists);
			return NULL;
		}
	}

	TMap<FVector3f, FColor> ExistingVertexColorData;
	if (!ExistingSkelMesh)
	{
		// When we are not re-importing we want to create the mesh here to be sure there is no material
		// or texture that will be create with the same name
		SkeletalMesh = NewObject<USkeletalMesh>(ImportSkeletalMeshArgs.InParent, ImportSkeletalMeshArgs.Name, ImportSkeletalMeshArgs.Flags);

		//This should not happen
		if (!ensure(SkeletalMesh))
		{
			return nullptr;
		}

		CreatedObjects.Add(SkeletalMesh);
	}
	else
	{
		ExistingVertexColorData = ExistingSkelMesh->GetVertexColorData();
	}

	FSkeletalMeshImportData TempData;
	// Fill with data from buffer - contains the full .FBX file. 	
	FSkeletalMeshImportData* SkelMeshImportDataPtr = &TempData;
	if( ImportSkeletalMeshArgs.OutData )
	{
		SkelMeshImportDataPtr = ImportSkeletalMeshArgs.OutData;
	}
	
	TArray<FName> LastImportedMaterialNames;
	FillLastImportMaterialNames(LastImportedMaterialNames, ExistingSkelMesh, ImportSkeletalMeshArgs.OrderedMaterialNames);

	//////////////////////////////////////////////////////////////////////////
	// We must do a maximum of fail test before backing up the data since the backup is destructive on the existing skeletal mesh.
	// See the comment later when we call the following function (SaveExistingSkelMeshData)
	TArray<FbxNode*> ImportedSkeletonLinkNodes;
	if (FillSkeletalMeshImportData(ImportSkeletalMeshArgs.NodeArray, ImportSkeletalMeshArgs.TemplateImportData, ImportSkeletalMeshArgs.FbxShapeArray, SkelMeshImportDataPtr, ImportedSkeletonLinkNodes, LastImportedMaterialNames, ExistingSkelMesh != nullptr, ExistingVertexColorData, ImportSkeletalMeshArgs.bMapMorphTargetToTimeZero) == false)
	{
		AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, LOCTEXT("FbxSkeletaLMeshimport_FillupImportData", "Get Import Data has failed.")), FFbxErrors::SkeletalMesh_FillImportDataFailed);
		EARLY_RETURN_ON_CANCEL(true, FailureCleanup);
	}

	EARLY_RETURN_ON_CANCEL(false, CancelCleanup);

	//When reimporting we must verify the import skeleton is valid before applying any modification to the skeletalmesh. For sure if we reimport only the geometry we do not need to do that
	if (ExistingSkelMesh && !ImportOptions->bImportAsSkeletalGeometry)
	{
		const TArray <SkeletalMeshImportData::FBone>& RefBonesBinary = SkelMeshImportDataPtr->RefBonesBinary;
		int32 BoneNumber = RefBonesBinary.Num();
		TArray<FString> UniqueBoneNames;
		UniqueBoneNames.Reserve(BoneNumber);
		for (int32 BoneIndex = 0; BoneIndex < BoneNumber; BoneIndex++)
		{
			const SkeletalMeshImportData::FBone& BinaryBone = RefBonesBinary[BoneIndex];
			const FString BoneName = FSkeletalMeshImportData::FixupBoneName(BinaryBone.Name);
			//FString == operator and <= or >= are all case insensitive
			//We want to check with case insensitive so this is perfect to use contains. No need to iterate
			if (UniqueBoneNames.Contains(BoneName))
			{
				UnFbx::FFbxImporter* FFbxImporter = UnFbx::FFbxImporter::GetInstance();
				FFbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("SkeletonHasDuplicateBones", "Skeleton has non-unique bone names.\nBone named '{0}' encountered more than once."), FText::FromString(BoneName))), FFbxErrors::SkeletalMesh_DuplicateBones);
				if (SkeletalMesh)
				{
					SkeletalMesh->ClearFlags(RF_Standalone);
					SkeletalMesh->Rename(NULL, GetTransientPackage(), REN_DontCreateRedirectors);
				}
				return nullptr;
			}
			UniqueBoneNames.Add(BoneName);
		}
	}

	//Stack the PostEditChange call, it will call post edit change when it will go out of scope
	FScopedSkeletalMeshPostEditChange ScopedPostEditChange(ExistingSkelMesh);

	//Adjust the import data from the import options
	if (ExistingSkelMesh != nullptr)
	{
		if (ImportOptions->bImportAsSkeletalSkinning)
		{
			//Replace geometry import data by original existing skel mesh geometry data
			ReplaceSkeletalMeshGeometryImportData(ExistingSkelMesh, SkelMeshImportDataPtr, SafeLODIndex);
		}
		else if (ImportOptions->bImportAsSkeletalGeometry)
		{
			//Replace skinning import data by original existing skel mesh skinning data
			ReplaceSkeletalMeshSkinningImportData(ExistingSkelMesh, SkelMeshImportDataPtr, SafeLODIndex);
		}

		//Reuse the vertex color in case we are not re-importing weights only and there is some vertexcolor in the existing skeleMesh
		if (ExistingSkelMesh->GetHasVertexColors() && ImportOptions->VertexColorImportOption == EVertexColorImportOption::Ignore)
		{
			RemapSkeletalMeshVertexColorToImportData(ExistingSkelMesh, SafeLODIndex, SkelMeshImportDataPtr);
		}
	}

	

	// Create initial bounding box based on expanded version of reference pose for meshes without physics assets. Can be overridden by artist.
	FBox3f BoundingBox(SkelMeshImportDataPtr->Points.GetData(), SkelMeshImportDataPtr->Points.Num());
	const FVector3f BoundingBoxSize = BoundingBox.GetSize();

	if (SkelMeshImportDataPtr->Points.Num() > 2 && BoundingBoxSize.X < THRESH_POINTS_ARE_SAME && BoundingBoxSize.Y < THRESH_POINTS_ARE_SAME && BoundingBoxSize.Z < THRESH_POINTS_ARE_SAME)
	{
		AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("FbxSkeletaLMeshimport_ErrorMeshTooSmall", "Cannot import this mesh, the bounding box of this mesh is smaller than the supported threshold[{0}]."), FText::FromString(FString::Printf(TEXT("%f"), THRESH_POINTS_ARE_SAME)))), FFbxErrors::SkeletalMesh_FillImportDataFailed);
		EARLY_RETURN_ON_CANCEL(true, FailureCleanup);
	}

	FSkeletalMeshBuildSettings BuildOptions;
	//Make sure the build option change in the re-import ui is reconduct
	BuildOptions.bUseFullPrecisionUVs = false;
	BuildOptions.bUseBackwardsCompatibleF16TruncUVs = false;
	BuildOptions.bUseHighPrecisionTangentBasis = false;
	BuildOptions.bRecomputeNormals = !ImportOptions->ShouldImportNormals() || !SkelMeshImportDataPtr->bHasNormals;
	BuildOptions.bRecomputeTangents = !ImportOptions->ShouldImportTangents() || !SkelMeshImportDataPtr->bHasTangents;
	BuildOptions.bUseMikkTSpace = (ImportOptions->NormalGenerationMethod == EFBXNormalGenerationMethod::MikkTSpace) && (!ImportOptions->ShouldImportNormals() || !ImportOptions->ShouldImportTangents());
	BuildOptions.bComputeWeightedNormals = ImportOptions->bComputeWeightedNormals;
	BuildOptions.bRemoveDegenerates = ImportOptions->bRemoveDegenerates;
	BuildOptions.ThresholdPosition = ImportOptions->OverlappingThresholds.ThresholdPosition;
	BuildOptions.ThresholdTangentNormal = ImportOptions->OverlappingThresholds.ThresholdTangentNormal;
	BuildOptions.ThresholdUV = ImportOptions->OverlappingThresholds.ThresholdUV;
	BuildOptions.MorphThresholdPosition = ImportOptions->OverlappingThresholds.MorphThresholdPosition;
	
	TSharedPtr<FExistingSkelMeshData> ExistSkelMeshDataPtr;
	if (ExistingSkelMesh)
	{
		ExistingSkelMesh->PreEditChange(NULL);
		//We update the build settings before saving the asset data we want to restore after the re-import
		int32 SafeReimportLODIndex = ImportSkeletalMeshArgs.LodIndex < 0 ? 0 : ImportSkeletalMeshArgs.LodIndex;
		FSkeletalMeshLODInfo* LODInfoPtr = ExistingSkelMesh->GetLODInfo(SafeReimportLODIndex);
		if (LODInfoPtr)
		{
			// Full precision UV and High precision tangent cannot be change in the re-import options, it must not be change from the original data.
			BuildOptions.bUseFullPrecisionUVs = LODInfoPtr->BuildSettings.bUseFullPrecisionUVs;
			BuildOptions.bUseHighPrecisionTangentBasis = LODInfoPtr->BuildSettings.bUseHighPrecisionTangentBasis;
			BuildOptions.bUseBackwardsCompatibleF16TruncUVs = LODInfoPtr->BuildSettings.bUseBackwardsCompatibleF16TruncUVs;

			//Copy all the build option to reflect any change in the setting using the re-import UI
			LODInfoPtr->BuildSettings = BuildOptions;
		}

		// Before calling SaveExistingSkelMeshData() we must remove the existing fbx metadata as we don't want to restore those.
		RemoveFBXMetaData(ExistingSkelMesh);

		//The backup of the skeletal mesh data empty the LOD array in the ImportedResource of the skeletal mesh
		//If the import fail after this step the editor can crash when updating the bone later since the LODModel will not exist anymore
		ExistSkelMeshDataPtr = SkeletalMeshImportUtils::SaveExistingSkelMeshData(ExistingSkelMesh, !ImportOptions->bImportMaterials, ImportSkeletalMeshArgs.LodIndex);
	}

	if (SkeletalMesh == nullptr)
	{
		if (!ensure(ExistingSkelMesh != nullptr))
		{
			return nullptr;
		}

		SkeletalMesh = ExistingSkelMesh;

		for (int32 LODIndex = 0, LODCount = SkeletalMesh->GetLODNum(); LODIndex < LODCount; ++LODIndex)
		{
			SkeletalMesh->ClearMeshDescriptionAndBulkData(LODIndex);
		}

		FSkeletalMeshModel* ImportedResource = SkeletalMesh->GetImportedModel();
		ImportedResource->EmptyOriginalReductionSourceMeshData();
		ImportedResource->LODModels.Empty();

		SkeletalMesh->SetNumSourceModels(0);
		SkeletalMesh->GetMaterials().Empty();
		SkeletalMesh->GetRefSkeleton().Empty();
		SkeletalMesh->SetSkeleton(nullptr);
		SkeletalMesh->SetPhysicsAsset(nullptr);
		SkeletalMesh->UnregisterAllMorphTarget();

		SkeletalMesh->ReleaseResources();
	}


	SkeletalMesh->PreEditChange(NULL);
	//Dirty the DDC Key for any imported Skeletal Mesh
	SkeletalMesh->InvalidateDeriveDataCacheGUID();

	FSkeletalMeshModel *ImportedResource = SkeletalMesh->GetImportedModel();
	check(ImportedResource->LODModels.Num() == 0);
	ImportedResource->LODModels.Empty();
	ImportedResource->LODModels.Add(new FSkeletalMeshLODModel());
	const int32 ImportLODModelIndex = 0;
	FSkeletalMeshLODModel& LODModel = ImportedResource->LODModels[ImportLODModelIndex];

	// process materials from import data
	SkeletalMeshImportUtils::ProcessImportMeshMaterials(SkeletalMesh->GetMaterials(), *SkelMeshImportDataPtr);

	// process reference skeleton from import data
	int32 SkeletalDepth = 0;
	USkeleton* ExistingSkeleton = ExistSkelMeshDataPtr ? ExistSkelMeshDataPtr->ExistingSkeleton : ImportOptions->SkeletonForAnimation;
	if (!SkeletalMeshImportUtils::ProcessImportMeshSkeleton(ExistingSkeleton, SkeletalMesh->GetRefSkeleton(), SkeletalDepth, *SkelMeshImportDataPtr))
	{
		EARLY_RETURN_ON_CANCEL(true, FailureCleanup);
	}

	EARLY_RETURN_ON_CANCEL(false, CancelCleanup);

	if (!GIsAutomationTesting)
	{
		UE_LOGF(LogFbx, Log, "Bones digested - %i  Depth of hierarchy - %i", SkeletalMesh->GetRefSkeleton().GetNum(), SkeletalDepth);
	}

	// process bone influences from import data
	SkeletalMeshImportUtils::ProcessImportMeshInfluences(*SkelMeshImportDataPtr, SkeletalMesh->GetPathName());

	// Clean out all LODs.
	SkeletalMesh->SetNumSourceModels(0);
	
	FSkeletalMeshLODInfo& NewLODInfo = SkeletalMesh->AddLODInfo();
	NewLODInfo.ReductionSettings.NumOfTrianglesPercentage = 1.0f;
	NewLODInfo.ReductionSettings.NumOfVertPercentage = 1.0f;
	NewLODInfo.ReductionSettings.MaxDeviationPercentage = 0.0f;
	NewLODInfo.LODHysteresis = 0.02f;

	//Store the original fbx import data the SkelMeshImportDataPtr should not be modified after this
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SkeletalMesh->SaveLODImportedData(ImportLODModelIndex, *SkelMeshImportDataPtr);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	SkeletalMesh->SetImportedBounds(FBoxSphereBounds((FBox)BoundingBox));

	// Store whether or not this mesh has vertex colors
	SkeletalMesh->SetHasVertexColors(SkelMeshImportDataPtr->bHasVertexColors);
	SkeletalMesh->SetVertexColorGuid(SkeletalMesh->GetHasVertexColors() ? FGuid::NewGuid() : FGuid());

	// Pass the number of texture coordinate sets to the LODModel.  Ensure there is at least one UV coord
	LODModel.NumTexCoords = FMath::Max<uint32>(1, SkelMeshImportDataPtr->NumTexCoords);

	if(ImportSkeletalMeshArgs.bCreateRenderData )
	{
		TArray<FVector3f> LODPoints;
		TArray<SkeletalMeshImportData::FMeshWedge> LODWedges;
		TArray<SkeletalMeshImportData::FMeshFace> LODFaces;
		TArray<SkeletalMeshImportData::FVertInfluence> LODInfluences;
		TArray<int32> LODPointToRawMap;
		SkelMeshImportDataPtr->CopyLODImportData(LODPoints,LODWedges,LODFaces,LODInfluences,LODPointToRawMap);

		bool bUseLegacyMeshUtilities = false;
		bool bBuildSuccess = false;
		if (bUseLegacyMeshUtilities)
		{
			IMeshUtilities::MeshBuildOptions LegacyBuildOptions;
			LegacyBuildOptions.FillOptions(BuildOptions);
			LegacyBuildOptions.TargetPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();

			IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshUtilities>("MeshUtilities");

			TArray<FText> WarningMessages;
			TArray<FName> WarningNames;
			// Create actual rendering data.
			bBuildSuccess = MeshUtilities.BuildSkeletalMesh(ImportedResource->LODModels[ImportLODModelIndex], SkeletalMesh->GetPathName(), SkeletalMesh->GetRefSkeleton(), LODInfluences, LODWedges, LODFaces, LODPoints, LODPointToRawMap, LegacyBuildOptions, &WarningMessages, &WarningNames);

			//Cache the vertex/triangle count in the InlineReductionCacheData so we can know if the LODModel need reduction or not.
			TArray<FInlineReductionCacheData>& InlineReductionCacheDatas = ImportedResource->InlineReductionCacheDatas;
			if (!InlineReductionCacheDatas.IsValidIndex(ImportLODModelIndex))
			{
				InlineReductionCacheDatas.AddDefaulted((ImportLODModelIndex + 1) - InlineReductionCacheDatas.Num());
			}
			if (ensure(InlineReductionCacheDatas.IsValidIndex(ImportLODModelIndex)))
			{
				InlineReductionCacheDatas[ImportLODModelIndex].SetCacheGeometryInfo(ImportedResource->LODModels[ImportLODModelIndex]);
			}

			// temporary hack of message/names, should be one token or a struct
			if (WarningMessages.Num() > 0 && WarningNames.Num() == WarningMessages.Num())
			{
				EMessageSeverity::Type MessageSeverity = bBuildSuccess ? EMessageSeverity::Warning : EMessageSeverity::Error;

				for (int32 MessageIdx = 0; MessageIdx < WarningMessages.Num(); ++MessageIdx)
				{
					AddTokenizedErrorMessage(FTokenizedMessage::Create(MessageSeverity, WarningMessages[MessageIdx]), WarningNames[MessageIdx]);
				}
			}
		}
		else
		{
			//The imported LOD is always 0 here, the LOD custom import will import the LOD alone(in a temporary skeletalmesh) and add it to the base skeletal mesh later
			check(SkeletalMesh->GetLODInfo(ImportLODModelIndex) != nullptr);
			//Set the build options
			SkeletalMesh->GetLODInfo(ImportLODModelIndex)->BuildSettings = BuildOptions;
			//New MeshDescription build process
			IMeshBuilderModule& MeshBuilderModule = IMeshBuilderModule::GetForRunningPlatform();
			//We must build the LODModel so we can restore properly the mesh, but we do not have to regenerate LODs
			const bool bRegenDepLODs = false;
			FSkeletalMeshBuildParameters SkeletalMeshBuildParameters(SkeletalMesh, GetTargetPlatformManagerRef().GetRunningTargetPlatform(), ImportLODModelIndex, bRegenDepLODs);
			// JIRA UE-250536: Change of API to IMeshBuilderModule::BuildSkeletalMesh now requires a rendering resource
			// Solution: Allocate an uninitialized rendering resource then release it after the call to IMeshBuilderModule::BuildSkeletalMesh
			//			 Although the legacy importer does not support Nanite for skeletal mesh, the call succeeds even if Nanite is manually set to true
			//			 Hopefully, this solution will last until the legacy FBX importer is deprecated.
			SkeletalMesh->AllocateResourceForRendering();
			bBuildSuccess = MeshBuilderModule.BuildSkeletalMesh(*SkeletalMesh->GetResourceForRendering(), SkeletalMeshBuildParameters);
			SkeletalMesh->ReleaseResources();
		}

		if( !bBuildSuccess )
		{
			SkeletalMesh->MarkAsGarbage();
			return NULL;
		}
		
		// Presize the per-section shadow casting array with the number of sections in the imported LOD.
		const int32 NumSections = LODModel.Sections.Num();

		//Get the last fbx file data need for reimport
		if (ImportSkeletalMeshArgs.ImportMaterialOriginalNameData)
		{
			for (FSkeletalMaterial SkeletalMaterial : SkeletalMesh->GetMaterials())
			{
				ImportSkeletalMeshArgs.ImportMaterialOriginalNameData->Add(SkeletalMaterial.ImportedMaterialSlotName);
			}
		}
		if (ImportSkeletalMeshArgs.ImportMeshSectionsData)
		{
			const TArray<FSkeletalMaterial>& Materials = SkeletalMesh->GetMaterials();
			for (int32 SectionIndex = 0; SectionIndex < ImportedResource->LODModels[ImportLODModelIndex].Sections.Num(); ++SectionIndex)
			{
				const FSkelMeshSection &SkelMeshSection = ImportedResource->LODModels[ImportLODModelIndex].Sections[SectionIndex];
				int32 MaterialIndex = SkelMeshSection.MaterialIndex;
				if (SkeletalMesh->GetLODInfo(0)->LODMaterialMap.IsValidIndex(SectionIndex) && SkeletalMesh->GetLODInfo(0)->LODMaterialMap[SectionIndex] != INDEX_NONE)
				{
					MaterialIndex = SkeletalMesh->GetLODInfo(0)->LODMaterialMap[SectionIndex];
				}

				if (ensure(Materials.IsValidIndex(MaterialIndex)))
				{
					ImportSkeletalMeshArgs.ImportMeshSectionsData->SectionOriginalMaterialName.Add(Materials[MaterialIndex].ImportedMaterialSlotName);
				}
			}
		}

		// Store the current file path and timestamp for re-import purposes
		UFbxSkeletalMeshImportData* ImportData = UFbxSkeletalMeshImportData::GetImportDataForSkeletalMesh(SkeletalMesh, ImportSkeletalMeshArgs.TemplateImportData);

		if (ExistingSkelMesh)
		{
			//In case of a re-import we update only the type we re-import
			SkeletalMesh->GetAssetImportData()->AddFileName(UFactory::GetCurrentFilename(), ImportOptions->bImportAsSkeletalGeometry ? 1 : ImportOptions->bImportAsSkeletalSkinning ? 2 : 0);
		}
		else
		{
			//When we create a new skeletal mesh asset we create 1 or 3 source files. 1 in case we import all the content, 3 in case we import geo or skinning
			int32 SourceFileIndex = ImportOptions->bImportAsSkeletalGeometry ? 1 : ImportOptions->bImportAsSkeletalSkinning ? 2 : 0;
			//Always add the base sourcefile
			SkeletalMesh->GetAssetImportData()->AddFileName(UFactory::GetCurrentFilename(), 0, NSSkeletalMeshSourceFileLabels::GeoAndSkinningText().ToString());
			if (SourceFileIndex != 0)
			{
				//If the user import geo or skinning, we add both entries to allow reimport of them
				SkeletalMesh->GetAssetImportData()->AddFileName(UFactory::GetCurrentFilename(), 1, NSSkeletalMeshSourceFileLabels::GeometryText().ToString());
				SkeletalMesh->GetAssetImportData()->AddFileName(UFactory::GetCurrentFilename(), 2, NSSkeletalMeshSourceFileLabels::SkinningText().ToString());
			}
		}

		if (ExistSkelMeshDataPtr)
		{
			SkeletalMeshImportUtils::RestoreExistingSkelMeshData(ExistSkelMeshDataPtr, SkeletalMesh, ImportSkeletalMeshArgs.LodIndex, ImportOptions->bCanShowDialog, ImportOptions->bImportAsSkeletalSkinning, ImportOptions->bResetToFbxOnMaterialConflict);

			SkeletalMesh->GetRefBasesInvMatrix().Reset();
		}

		SkeletalMesh->CalculateInvRefMatrices();

		EARLY_RETURN_ON_CANCEL(false, CancelCleanup);

		if ((!SkeletalMesh->GetResourceForRendering() || !SkeletalMesh->GetResourceForRendering()->LODRenderData.IsValidIndex(0)) && ImportOptions->bCreatePhysicsAsset && CanImportClass(UPhysicsAsset::StaticClass()))
		{
			//We need to have a valid render data to create physic asset
			SkeletalMesh->Build();
		}
		SkeletalMesh->MarkPackageDirty();
	}

	if(ImportSkeletalMeshArgs.LodIndex == 0)
	{
		// see if we have skeleton set up
		// if creating skeleton, create skeleton
		USkeleton* Skeleton = ImportOptions->SkeletonForAnimation;
		if (Skeleton == NULL)
		{
			FString ObjectName = FString::Printf(TEXT("%s_Skeleton"), *SkeletalMesh->GetName());
			Skeleton = CreateAsset<USkeleton>(ImportSkeletalMeshArgs.InParent->GetName(), ObjectName, true);
			if (!Skeleton)
			{
				// same object exists, try to see if it's skeleton, if so, load
				Skeleton = LoadObject<USkeleton>(ImportSkeletalMeshArgs.InParent, *ObjectName);

				// if not skeleton, we're done, we can't create skeleton with same name
				// @todo in the future, we'll allow them to rename
				if (!Skeleton)
				{
					AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("FbxSkeletaLMeshimport_SkeletonRecreateError", "'{0}' already exists. It fails to recreate it."), FText::FromString(ObjectName))), FFbxErrors::SkeletalMesh_SkeletonRecreateError);
					return SkeletalMesh;
				}
			}
			else
			{
				CreatedObjects.Add(Skeleton);
			}
		}

		// merge bones to the selected skeleton
		if ((!ImportOptions->bImportAsSkeletalSkinning || ExistingSkelMesh == nullptr) && !Skeleton->MergeAllBonesToBoneTree( SkeletalMesh ) )
		{
			// We should only show the skeleton save toast once, not as many times as we have nodes to import
			bool bToastSaveMessage = false;
			if(bFirstMesh || (LastMergeBonesChoice != EAppReturnType::NoAll && LastMergeBonesChoice != EAppReturnType::YesAll))
			{
				LastMergeBonesChoice = FMessageDialog::Open(EAppMsgType::YesNoYesAllNoAllCancel,
					LOCTEXT("SkeletonFailed_BoneMerge", "FAILED TO MERGE BONES:\n\n This could happen if significant hierarchical changes have been made\ne.g. inserting a bone between nodes.\nWould you like to regenerate the Skeleton from this mesh?\n\n***WARNING: THIS MAY INVALIDATE OR REQUIRE RECOMPRESSION OF ANIMATION DATA.***\n"));
				bToastSaveMessage = true;
			}
			
			if(LastMergeBonesChoice == EAppReturnType::Cancel)
			{
				// User wants to cancel further importing
				bImportOperationCanceled = true;
				return nullptr;
			}

			if (LastMergeBonesChoice == EAppReturnType::Yes || LastMergeBonesChoice == EAppReturnType::YesAll) 
			{
				if ( Skeleton->RecreateBoneTree( SkeletalMesh ) && bToastSaveMessage)
				{
					// @todo: this is a lot of message box but this requires user input and it can be very annoying to miss
					// make sure to go through all skeletalmesh and merge them also to recreate the issue. 
					if (FMessageDialog::Open(EAppMsgType::YesNo,
						LOCTEXT("Skeleton_ReAddAllMeshes", "Would you like to merge all SkeletalMeshes using this skeleton to ensure all bones are merged? This will require to load those SkeletalMeshes.")) == EAppReturnType::Yes)
					{
						FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
						TArray<FAssetData> SkeletalMeshAssetData;
						
						FARFilter ARFilter;
						ARFilter.ClassPaths.Add(USkeletalMesh::StaticClass()->GetClassPathName());
						ARFilter.TagsAndValues.Add(TEXT("Skeleton"), FAssetData(Skeleton).GetExportTextName());

						IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
						if (AssetRegistry.GetAssets(ARFilter, SkeletalMeshAssetData))
						{
							// look through all skeletalmeshes that uses this skeleton
							for (int32 AssetId = 0; AssetId < SkeletalMeshAssetData.Num(); ++AssetId)
							{
								FAssetData& CurAssetData = SkeletalMeshAssetData[AssetId];
								const USkeletalMesh* ExtraSkeletalMesh = Cast<USkeletalMesh>(CurAssetData.GetAsset());
								if (SkeletalMesh != ExtraSkeletalMesh && IsValid(ExtraSkeletalMesh))
								{
									// merge still can fail, then print message box
									if (Skeleton->MergeAllBonesToBoneTree(ExtraSkeletalMesh) == false)
									{
										// print warning
										FMessageDialog::Open(EAppMsgType::Ok,
											FText::Format(LOCTEXT("SkeletonRegenError_RemergingBones", "Failed to merge SkeletalMesh '{0}'."), FText::FromString(ExtraSkeletalMesh->GetName())));
									}
								}
							}
						}
					}

					FAssetNotifications::SkeletonNeedsToBeSaved(Skeleton);
				}
			}
		}
		else
		{
			// ask if they'd like to update their position form this mesh
			if ( ImportOptions->SkeletonForAnimation && ImportOptions->bUpdateSkeletonReferencePose )
			{
				Skeleton->UpdateReferencePoseFromMesh(SkeletalMesh);
				FAssetNotifications::SkeletonNeedsToBeSaved(Skeleton);
			}
		}

		EARLY_RETURN_ON_CANCEL(false, CancelCleanup);

		if (SkeletalMesh->GetSkeleton() != Skeleton)
		{
			SkeletalMesh->SetSkeleton(Skeleton);
			SkeletalMesh->MarkPackageDirty();
		}
		// Create PhysicsAsset if requested and if physics asset is null
		// We create the physic asset after we create the skeleton since we need the skeleton to correctly build it
		if (ImportOptions->bCreatePhysicsAsset)
		{
			if (SkeletalMesh->GetPhysicsAsset() == NULL && CanImportClass(UPhysicsAsset::StaticClass()))
			{
				FString ObjectName = FString::Printf(TEXT("%s_PhysicsAsset"), *SkeletalMesh->GetName());
				UPhysicsAsset * NewPhysicsAsset = CreateAsset<UPhysicsAsset>(ImportSkeletalMeshArgs.InParent->GetName(), ObjectName, true);
				if (!NewPhysicsAsset)
				{
					AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("FbxSkeletaLMeshimport_CouldNotCreatePhysicsAsset", "Could not create Physics Asset ('{0}') for '{1}'"), FText::FromString(ObjectName), FText::FromString(SkeletalMesh->GetName()))), FFbxErrors::SkeletalMesh_FailedToCreatePhyscisAsset);
				}
				else
				{
					FPhysAssetCreateParams NewBodyData;
					FText CreationErrorMessage;
					CreatedObjects.Add(NewPhysicsAsset);
					bool bSuccess = FPhysicsAssetUtils::CreateFromSkeletalMesh(NewPhysicsAsset, SkeletalMesh, NewBodyData, CreationErrorMessage);
					if (!bSuccess)
					{
						AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, CreationErrorMessage), FFbxErrors::SkeletalMesh_FailedToCreatePhyscisAsset);
						// delete the asset since we could not have create physics asset
						TArray<UObject*> ObjectsToDelete;
						ObjectsToDelete.Add(NewPhysicsAsset);
						ObjectTools::DeleteObjects(ObjectsToDelete, false);
					}
				}
			}
		}
		// if physics asset is selected
		else if (ImportOptions->PhysicsAsset)
		{
			SkeletalMesh->SetPhysicsAsset(ImportOptions->PhysicsAsset);
		}
	}

	// Import mesh metadata to SkeletalMesh
	for (FbxNode* MeshNode : ImportSkeletalMeshArgs.NodeArray)
	{
		ImportNodeCustomProperties(SkeletalMesh, MeshNode, true);
	}

	// Import bone metadata to Skeleton
	for (FbxNode* SkeletonNode : ImportedSkeletonLinkNodes)
	{
		ImportNodeCustomProperties(SkeletalMesh->GetSkeleton(), SkeletonNode, true);
	}

	return SkeletalMesh;
}
#undef EARLY_RETURN_ON_CANCEL

void UnFbx::FFbxImporter::UpdateSkeletalMeshImportData(USkeletalMesh *SkeletalMesh, UFbxSkeletalMeshImportData* SkeletalMeshImportData, int32 SpecificLod, TArray<FName> *ImportMaterialOriginalNameData, TArray<FImportMeshLodSectionsData> *ImportMeshLodData)
{
	if (SkeletalMesh == nullptr)
	{
		return;
	}

	UFbxSkeletalMeshImportData* ImportData = Cast<UFbxSkeletalMeshImportData>(SkeletalMesh->GetAssetImportData());
	if (!ImportData && SkeletalMeshImportData)
	{
		ImportData = UFbxSkeletalMeshImportData::GetImportDataForSkeletalMesh(SkeletalMesh, SkeletalMeshImportData);
	}
	if (ImportData)
	{
		//Set the last content type import
		ImportData->LastImportContentType = ImportData->ImportContentType;

		ImportData->ImportMaterialOriginalNameData.Empty();
		if (ImportMaterialOriginalNameData && ImportMeshLodData)
		{
			if (SpecificLod == INDEX_NONE && ImportMeshLodData->Num() == SkeletalMesh->GetLODNum())
			{
				//Copy the material array
				ImportData->ImportMaterialOriginalNameData = (*ImportMaterialOriginalNameData);

				ImportData->ImportMeshLodData.Empty();
				for (const FImportMeshLodSectionsData &ImportMeshLodSectionsData : (*ImportMeshLodData))
				{
					ImportData->ImportMeshLodData.Add(ImportMeshLodSectionsData);
				}
			}
			else
			{
				for (FName MaterialImportNameLOD : (*ImportMaterialOriginalNameData))
				{
					bool bFoundMaterial = false;
					for (FName MaterialImportName : ImportData->ImportMaterialOriginalNameData)
					{
						if (MaterialImportNameLOD == MaterialImportName)
						{
							bFoundMaterial = true;
							break;
						}
					}
					if (!bFoundMaterial)
					{
						//Add the LOD material at the end of the original array
						ImportData->ImportMaterialOriginalNameData.Add(MaterialImportNameLOD);
					}
				}

				if (SpecificLod == INDEX_NONE)
				{
					for (int32 UpdateLodIndex = 0; UpdateLodIndex < (*ImportMeshLodData).Num(); ++UpdateLodIndex)
					{
						if (ImportData->ImportMeshLodData.Num() <= UpdateLodIndex)
						{
							ImportData->ImportMeshLodData.AddZeroed();
						}
						ImportData->ImportMeshLodData[UpdateLodIndex] = (*ImportMeshLodData)[UpdateLodIndex];
					}
				}
				else
				{
					if (ImportData->ImportMeshLodData.Num() <= SpecificLod)
					{
						ImportData->ImportMeshLodData.AddZeroed(1 + SpecificLod - ImportData->ImportMeshLodData.Num());
					}
					ImportData->ImportMeshLodData[SpecificLod] = (*ImportMeshLodData)[0];
				}
			}
		}
		else
		{
			//This is not a re-import or we reimport an old asset containing no data
			//In this case we update from the skeletal mesh import
			ImportData->ImportMaterialOriginalNameData.Empty();
			ImportData->ImportMeshLodData.Empty();
			for (const FSkeletalMaterial &Material : SkeletalMesh->GetMaterials())
			{
				ImportData->ImportMaterialOriginalNameData.Add(Material.ImportedMaterialSlotName);
			}
			FSkeletalMeshModel* ImportedResource = SkeletalMesh->GetImportedModel();
			for (int32 LODResoureceIndex = 0; LODResoureceIndex < ImportedResource->LODModels.Num(); ++LODResoureceIndex)
			{
				ImportData->ImportMeshLodData.AddZeroed();
				const FSkeletalMeshLODInfo& LODInfo = *(SkeletalMesh->GetLODInfo(LODResoureceIndex));
				const FSkeletalMeshLODModel& LODModel = ImportedResource->LODModels[LODResoureceIndex];
				int32 NumSections = LODModel.Sections.Num();
				for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
				{
					int32 MaterialLodSectionIndex = LODModel.Sections[SectionIndex].MaterialIndex;
					//Is this LOD use the LODMaterialMap override
					if (LODInfo.LODMaterialMap.IsValidIndex(SectionIndex) && LODInfo.LODMaterialMap[SectionIndex] != INDEX_NONE)
					{
						MaterialLodSectionIndex = LODInfo.LODMaterialMap[SectionIndex];
					}

					if (ImportData->ImportMaterialOriginalNameData.IsValidIndex(MaterialLodSectionIndex))
					{
						ImportData->ImportMeshLodData[LODResoureceIndex].SectionOriginalMaterialName.Add(ImportData->ImportMaterialOriginalNameData[MaterialLodSectionIndex]);
					}
					else
					{
						ImportData->ImportMeshLodData[LODResoureceIndex].SectionOriginalMaterialName.Add(TEXT("InvalidMaterialIndex"));
					}
				}
			}
		}

		// Note: For 36.00, the forward axis is either Y or X. A more complete solution will be implemented later.
		SkeletalMesh->SetForwardAxis(ImportData->bConvertScene ? (ImportData->bForceFrontXAxis ? EAxis::X : EAxis::Y) : EAxis::Y);
	}
}


UObject* UnFbx::FFbxImporter::CreateAssetOfClass(UClass* AssetClass, FString ParentPackageName, FString ObjectName, bool bAllowReplace)
{
	// See if this sequence already exists.
	UObject* 	ParentPkg = CreatePackage( *ParentPackageName);
	FString 	ParentPath = FString::Printf(TEXT("%s/%s"), *FPackageName::GetLongPackagePath(*ParentPackageName), *ObjectName);
	UObject* 	Parent = CreatePackage( *ParentPath);
	// See if an object with this name exists
	UObject* Object = LoadObject<UObject>(Parent, *ObjectName, NULL, LOAD_NoWarn | LOAD_Quiet, NULL);

	// if object with same name but different class exists, warn user
	if ((Object != NULL) && (Object->GetClass() != AssetClass))
	{
		UnFbx::FFbxImporter* FFbxImporter = UnFbx::FFbxImporter::GetInstance();
		FFbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("Error_AssetExist", "Asset with same name exists. Can't overwrite another asset")), FFbxErrors::Generic_SameNameAssetExists);
		return NULL;
	}

	// if object with same name exists, warn user
	if ( Object != NULL && !bAllowReplace )
	{
		// until we have proper error message handling so we don't ask every time, but once, I'm disabling it. 
		// 		if ( EAppReturnType::Yes != FMessageDialog::Open( EAppMsgType::YesNo, LocalizeSecure(NSLOCTEXT("UnrealEd", "Error_AssetExistAsk", "Asset with the name (`~) exists. Would you like to overwrite?").ToString(), *ParentPath) ) ) 
		// 		{
		// 			return NULL;
		// 		}
		UnFbx::FFbxImporter* FFbxImporter = UnFbx::FFbxImporter::GetInstance();
		FFbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("FbxSkeletaLMeshimport_SameNameExist", "Asset with the name ('{0}') exists. Overwriting..."), FText::FromString(ParentPath))), FFbxErrors::Generic_SameNameAssetOverriding);
	}

	if (Object == NULL)
	{
		// add it to the set
		// do not add to the set, now create independent asset
		Object = NewObject<UObject>(Parent, AssetClass, *ObjectName, RF_Public | RF_Standalone);
		Object->MarkPackageDirty();
		// Notify the asset registry
		FAssetRegistryModule::AssetCreated(Object);
	}

	return Object;
}

void UnFbx::FFbxImporter::SetupAnimationDataFromMesh(USkeletalMesh* SkeletalMesh, UObject* InParent, TArray<FbxNode*>& NodeArray, UFbxAnimSequenceImportData* TemplateImportData, const FString& Name)
{
	if (!CanImportClass(UAnimSequence::StaticClass()))
	{
		return;
	}

	USkeleton* Skeleton = SkeletalMesh->GetSkeleton();

	if (Scene->GetSrcObjectCount<FbxAnimStack>() > 0)
	{
		if ( ensure ( Skeleton != NULL ) )
		{
			TArray<FbxNode*> FBXMeshNodeArray;
			FbxNode* SkeletonRoot = FindFBXMeshesByBone(SkeletalMesh->GetSkeleton()->GetReferenceSkeleton().GetBoneName(0), true, FBXMeshNodeArray);

			if (SkeletonRoot != nullptr)
			{
				TArray<FbxNode*> SortedLinks;
				RecursiveBuildSkeleton(SkeletonRoot, SortedLinks);

				// when importing animation from SkeletalMesh, add new Group Anim, a lot of times they're same name
				UPackage * OuterPackage = InParent->GetOutermost();
				FString AnimName = (ImportOptions->AnimationName != "") ? ImportOptions->AnimationName : Name + TEXT("_Anim");
				// give animouter as outer
				ImportAnimations(Skeleton, OuterPackage, SortedLinks, AnimName, TemplateImportData, NodeArray);
			}
			else
			{
				//Cannot import animations if the skeleton do not match
				AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("FbxSkeletaLMeshimport_SkeletonNotMatching_no_anim_import", "Specified Skeleton '{0}' do not match fbx imported skeleton. Cannot import animations with this skeleton."), FText::FromName(Skeleton->GetFName()) )), FFbxErrors::Animation_InvalidData);
			}
		}
	}
}

USkeletalMesh* UnFbx::FFbxImporter::ReimportSkeletalMesh(USkeletalMesh* Mesh, UFbxSkeletalMeshImportData* TemplateImportData, uint64 SkeletalMeshFbxUID, TArray<FbxNode*> *OutSkeletalMeshArray)
{
	FFbxScopedOperation ScopedImportOperation(this);
	
	if ( !ensure(Mesh) )
	{
		// You need a mesh in order to reimport
		return NULL;
	}

	if ( !ensure(TemplateImportData) )
	{
		// You need import data in order to reimport
		return NULL;
	}

	TArray<FbxNode*>* FbxNodes = NULL;
	USkeletalMesh* NewMesh = NULL;

	bool Old_ImportRigidMesh = ImportOptions->bImportRigidMesh;
	bool Old_ImportMaterials = ImportOptions->bImportMaterials;
	bool Old_ImportTextures = ImportOptions->bImportTextures;
	bool Old_ImportAnimations = ImportOptions->bImportAnimations;

	auto CleanUpImportOptionsData = [&]()
	{
		ImportOptions->bImportRigidMesh = Old_ImportRigidMesh;
		ImportOptions->bImportMaterials = Old_ImportMaterials;
		ImportOptions->bImportTextures = Old_ImportTextures;
		ImportOptions->bImportAnimations = Old_ImportAnimations;
	};

	// support to update rigid animation mesh
	ImportOptions->bImportRigidMesh = true;

	UFbxSkeletalMeshImportData* SKImportData = Cast<UFbxSkeletalMeshImportData>(Mesh->GetAssetImportData());
	if (SKImportData)
	{
		constexpr int32 LODIndex = 0;
		FSkeletalMeshLODInfo* LODInfo = Mesh->GetLODInfo(LODIndex);
		if(LODInfo && Mesh->GetImportedModel() && Mesh->GetImportedModel()->LODModels.IsValidIndex(LODIndex))
		{
			if (!Mesh->HasMeshDescription(LODIndex))
			{
				//Set the build settings
				LODInfo->BuildSettings.bComputeWeightedNormals = SKImportData->bComputeWeightedNormals;
				LODInfo->BuildSettings.bRecomputeNormals = SKImportData->NormalImportMethod == EFBXNormalImportMethod::FBXNIM_ComputeNormals;
				LODInfo->BuildSettings.bRecomputeTangents = SKImportData->NormalImportMethod != EFBXNormalImportMethod::FBXNIM_ImportNormalsAndTangents;
				LODInfo->BuildSettings.bRemoveDegenerates = true;
				LODInfo->BuildSettings.bUseMikkTSpace = SKImportData->NormalGenerationMethod == EFBXNormalGenerationMethod::MikkTSpace;
				LODInfo->BuildSettings.ThresholdPosition = SKImportData->ThresholdPosition;
				LODInfo->BuildSettings.ThresholdTangentNormal = SKImportData->ThresholdTangentNormal;
				LODInfo->BuildSettings.ThresholdUV = SKImportData->ThresholdUV;
				LODInfo->BuildSettings.MorphThresholdPosition = SKImportData->MorphThresholdPosition;
			}
		}
	}

	// get meshes in Fbx file
	//the function also fill the collision models, so we can update collision models correctly
	TArray< TArray<FbxNode*>* > FbxSkelMeshArray;
	FillFbxSkelMeshArrayInScene(Scene->GetRootNode(), FbxSkelMeshArray, false, ImportOptions->bImportAsSkeletalGeometry || ImportOptions->bImportAsSkeletalSkinning, ImportOptions->bImportScene);

	if(SkeletalMeshFbxUID != 0xFFFFFFFFFFFFFFFF)
	{
		//Scene reimport know which skeletal mesh we want to reimport
		for (TArray<FbxNode*>*SkeletalMeshNodes : FbxSkelMeshArray)
		{
			if (SkeletalMeshNodes->Num() > 0)
			{
				FbxNode *Node = (*SkeletalMeshNodes)[0];
				FbxNode *SkeletalMeshNode = Node;
				if (Node->GetNodeAttribute() && Node->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup)
				{
					TArray<FbxNode*> NodeInLod;
					FindAllLODGroupNode(NodeInLod, Node, 0);
					for (FbxNode *MeshNode : NodeInLod)
					{
						if (MeshNode != nullptr && MeshNode->GetNodeAttribute() && MeshNode->GetNodeAttribute()->GetUniqueID() == SkeletalMeshFbxUID)
						{
							FbxNodes = SkeletalMeshNodes;
							if (OutSkeletalMeshArray != nullptr)
							{
								for (FbxNode *NodeReimport : (*SkeletalMeshNodes))
								{
									OutSkeletalMeshArray->Add(NodeReimport);
								}
							}
							break;
						}
					}
				}
				else
				{
					if (SkeletalMeshNode != nullptr && SkeletalMeshNode->GetNodeAttribute() && SkeletalMeshNode->GetNodeAttribute()->GetUniqueID() == SkeletalMeshFbxUID)
					{
						FbxNodes = SkeletalMeshNodes;
						if (OutSkeletalMeshArray != nullptr)
						{
							for (FbxNode *NodeReimport : (*SkeletalMeshNodes))
							{
								OutSkeletalMeshArray->Add(NodeReimport);
							}
						}
						break;
					}
				}
			}
			if (FbxNodes != nullptr)
				break;
		}
	}
	else
	{
		// if there is only one mesh, use it without name checking 
		// (because the "Used As Full Name" option enables users name the Unreal mesh by themselves
		if (FbxSkelMeshArray.Num() == 1)
		{
			FbxNodes = FbxSkelMeshArray[0];
		}
		else if (FbxSkelMeshArray.Num() > 1)
		{
			//Search a set corresponding
			for (TArray<FbxNode*>*SkeletalMeshNodes : FbxSkelMeshArray)
			{
				FbxNode* Node = GetMeshNodesFromName(Mesh->GetName(), *SkeletalMeshNodes);
				if (Node != nullptr)
				{
					FbxNodes = SkeletalMeshNodes;
					break;
				}
			}
			if (FbxNodes == nullptr)
			{
				//Re import the first skeletalmesh found in the fbx file
				FbxNodes = FbxSkelMeshArray[0];
			}
		}
	}
	
	if (FbxNodes)
	{
		//Posteditchange will be call at the end of the reimport which will reallocate the render ressource
		FScopedSkeletalMeshPostEditChange ScopedPostEditChange(Mesh);

		FScopedSlowTask SlowTask(100.0f, NSLOCTEXT("SkeletalMeshReimportFactory", "SkeletalMeshReimportTask", "Reimporting skeletal mesh..."));
		SlowTask.MakeDialog();

		//The fbx file is valid we can reimport the skeletal mesh
		Mesh->PreEditChange(nullptr);

		// Unbind any existing clothing assets before we reimport the geometry
		TArray<ClothingAssetUtils::FClothingAssetMeshBinding> ClothingBindings;
		FLODUtilities::UnbindClothingAndBackup(Mesh, ClothingBindings);

		//Reimport must force a new guid
		Mesh->InvalidateDeriveDataCacheGUID();

		for (int32 LODCounter = 1; LODCounter < Mesh->GetLODNum(); ++LODCounter)
		{
			//Reset all import with base mesh flags
			Mesh->GetLODInfo(LODCounter)->bImportWithBaseMesh = false;
		}
		//Empty the morph target before re-importing, it will prevent to have old data that can point on random vertex
		//Do not empty the morph if we import weights only
		if (Mesh->GetMorphTargets().Num() > 0 && !ImportOptions->bImportAsSkeletalSkinning)
		{
			Mesh->UnregisterAllMorphTarget();
		}

		// set import options, how about others?
		if (!ImportOptions->bImportScene)
		{
			ImportOptions->bImportMaterials = false;
			ImportOptions->bImportTextures = false;
		}
		//In case of a scene reimport animations are reimport later so its ok to hardcode animation to false here
		ImportOptions->bImportAnimations = false;
		// check if there is LODGroup for this skeletal mesh
		int32 MaxLODLevel = 1;
		int32 NumPrevLODs = Mesh->GetLODNum();

		for (int32 j = 0; j < (*FbxNodes).Num(); j++)
		{
			FbxNode* Node = (*FbxNodes)[j];
			if (Node->GetNodeAttribute() && Node->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup)
			{
				// get max LODgroup level
				if (MaxLODLevel < Node->GetChildCount())
				{
					MaxLODLevel = Node->GetChildCount();
				}
			}
		}

		//Original fbx data storage
		TArray<FName> ImportMaterialOriginalNameData;
		TArray<FImportMeshLodSectionsData> ImportMeshLodData;

		int32 LODIndex;
		int32 SuccessfulLodIndex = 0;
		float ProgressStep = static_cast<float>(90.0f / MaxLODLevel);
		for (LODIndex = 0; LODIndex < MaxLODLevel; LODIndex++)
		{
			SlowTask.EnterProgressFrame(ProgressStep);
			int32 ImportedSuccessfulLodIndex = INDEX_NONE;
			TArray<FbxNode*> SkelMeshNodeArray;
			for (int32 j = 0; j < (*FbxNodes).Num(); j++)
			{
				FbxNode* Node = (*FbxNodes)[j];
				if (Node->GetNodeAttribute() && Node->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup)
				{
					TArray<FbxNode*> NodeInLod;
					if (Node->GetChildCount() > LODIndex)
					{
						FindAllLODGroupNode(NodeInLod, Node, LODIndex);
					}
					else // in less some LODGroups have less level, use the last level
					{
						FindAllLODGroupNode(NodeInLod, Node, Node->GetChildCount() - 1);
					}
					for (FbxNode *MeshNode : NodeInLod)
					{
						SkelMeshNodeArray.Add(MeshNode);
					}
				}
				else
				{
					SkelMeshNodeArray.Add(Node);
				}
			}
			FSkeletalMeshImportData OutData;
			bool bMapMorphTargetToTimeZero = false;
			if (LODIndex == 0)
			{
				ImportMeshLodData.AddZeroed();
				UnFbx::FFbxImporter::FImportSkeletalMeshArgs ImportSkeletalMeshArgs;
				ImportSkeletalMeshArgs.InParent = Mesh->GetOuter();
				ImportSkeletalMeshArgs.NodeArray = SkelMeshNodeArray;
				ImportSkeletalMeshArgs.Name = *Mesh->GetName();
				ImportSkeletalMeshArgs.Flags = RF_Public | RF_Standalone;
				ImportSkeletalMeshArgs.TemplateImportData = TemplateImportData;
				ImportSkeletalMeshArgs.LodIndex = LODIndex;
				ImportSkeletalMeshArgs.ImportMaterialOriginalNameData = &ImportMaterialOriginalNameData;
				ImportSkeletalMeshArgs.ImportMeshSectionsData = &ImportMeshLodData[0];
				ImportSkeletalMeshArgs.OutData = &OutData;

				NewMesh = ImportSkeletalMesh( ImportSkeletalMeshArgs );
				if (bImportOperationCanceled)
				{
					// User cancel, clean up and return
					NewMesh = nullptr;
					break;
				}
				if (NewMesh)
				{
					bMapMorphTargetToTimeZero = ImportSkeletalMeshArgs.bMapMorphTargetToTimeZero;
					ImportedSuccessfulLodIndex = SuccessfulLodIndex;
					SuccessfulLodIndex++;
				}
			}
			else if (NewMesh && ImportOptions->bImportSkeletalMeshLODs && SkelMeshNodeArray[0]->GetMesh() == nullptr)
			{
				FSkeletalMeshUpdateContext UpdateContext;
				UpdateContext.SkeletalMesh = NewMesh;
				//Add a autogenerated LOD to the BaseSkeletalMesh
				if(SuccessfulLodIndex >= NewMesh->GetLODNum())
				{
					NewMesh->AddLODInfo();
					check(SuccessfulLodIndex == NewMesh->GetLODNum() - 1);
				}
				ImportMeshLodData.AddZeroed();
				FSkeletalMeshLODInfo* LODInfo = NewMesh->GetLODInfo(SuccessfulLodIndex);
				LODInfo->ReductionSettings.NumOfTrianglesPercentage = FMath::Pow(0.5f, (float)(SuccessfulLodIndex));
				LODInfo->ReductionSettings.BaseLOD = 0;
				FLODUtilities::SimplifySkeletalMeshLOD(UpdateContext, SuccessfulLodIndex, GetTargetPlatformManagerRef().GetRunningTargetPlatform(), false);
				LODInfo->bImportWithBaseMesh = true;
				LODInfo->SourceImportFilename = FString(TEXT(""));
				ImportedSuccessfulLodIndex = SuccessfulLodIndex;
				SuccessfulLodIndex++;
			}
			else if (NewMesh && ImportOptions->bImportSkeletalMeshLODs) // the base skeletal mesh is imported successfully
			{
				TArray<FName> ImportMaterialOriginalNameDataLOD;
				ImportMeshLodData.AddZeroed();
				USkeletalMesh* BaseSkeletalMesh = NewMesh;
				
				UnFbx::FFbxImporter::FImportSkeletalMeshArgs ImportSkeletalMeshArgs;
				ImportSkeletalMeshArgs.InParent = NewMesh->GetOutermost();
				ImportSkeletalMeshArgs.NodeArray = SkelMeshNodeArray;
				ImportSkeletalMeshArgs.Name = NAME_None;
				ImportSkeletalMeshArgs.Flags = RF_Transient;
				ImportSkeletalMeshArgs.TemplateImportData = TemplateImportData;
				ImportSkeletalMeshArgs.LodIndex = SuccessfulLodIndex;
				ImportSkeletalMeshArgs.ImportMaterialOriginalNameData = &ImportMaterialOriginalNameDataLOD;
				ImportSkeletalMeshArgs.ImportMeshSectionsData = &ImportMeshLodData[SuccessfulLodIndex];
				ImportSkeletalMeshArgs.OutData = &OutData;

				UObject *LODObject = ImportSkeletalMesh( ImportSkeletalMeshArgs );
				bool bImportSucceeded = !bImportOperationCanceled && ImportSkeletalMeshLOD( Cast<USkeletalMesh>(LODObject), BaseSkeletalMesh, SuccessfulLodIndex, TemplateImportData);

				if (bImportSucceeded)
				{
					FSkeletalMeshLODInfo* LODInfo = BaseSkeletalMesh->GetLODInfo(SuccessfulLodIndex);
					LODInfo->bImportWithBaseMesh = true;
					LODInfo->SourceImportFilename = FString(TEXT(""));
					bMapMorphTargetToTimeZero = ImportSkeletalMeshArgs.bMapMorphTargetToTimeZero;
					ImportedSuccessfulLodIndex = SuccessfulLodIndex;
					SuccessfulLodIndex++;
					for (FName MaterialImportNameLOD : ImportMaterialOriginalNameDataLOD)
					{
						bool bFoundMaterial = false;
						for (FName MaterialImportName : ImportMaterialOriginalNameData)
						{
							if (MaterialImportNameLOD == MaterialImportName)
							{
								bFoundMaterial = true;
								break;
							}
						}
						if (!bFoundMaterial)
						{
							//Add the LOD material at the end of the original array
							ImportMaterialOriginalNameData.Add(MaterialImportNameLOD);
						}
					}
				}
				else
				{
					AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("FailedToImport_SkeletalMeshLOD", "Failed to import Skeletal mesh LOD.")), FFbxErrors::SkeletalMesh_LOD_FailedToImport);
				}
				
			}

			if (NewMesh)
			{
				if ((ImportOptions->bImportSkeletalMeshLODs || LODIndex == 0) &&
					ImportOptions->bImportMorph &&
					ImportedSuccessfulLodIndex != INDEX_NONE &&
					NewMesh->GetImportedModel() &&
					NewMesh->GetImportedModel()->LODModels.IsValidIndex(ImportedSuccessfulLodIndex))
				{
					ImportFbxMorphTarget(SkelMeshNodeArray, NewMesh, ImportedSuccessfulLodIndex, OutData, bMapMorphTargetToTimeZero);
				}
			}
		}

		auto RebindClothing = [&ClothingBindings](USkeletalMesh* SkelMesh)
		{
			FSkeletalMeshModel* MeshResource = SkelMesh->GetImportedModel();
			if (MeshResource && ClothingBindings.Num() > 0)
			{
				FLODUtilities::RestoreClothingFromBackup(SkelMesh, ClothingBindings);
			}
		};


		if (!bImportOperationCanceled && NewMesh)
		{
			RebindClothing(NewMesh);

			//Update the import data so we can re-import correctly
			UpdateSkeletalMeshImportData(NewMesh, TemplateImportData, INDEX_NONE, &ImportMaterialOriginalNameData, &ImportMeshLodData);
		}
		else if(bImportOperationCanceled && Mesh != nullptr)
		{
			//Operation was canceled
			//rebind the clothing on the original mesh
			RebindClothing(Mesh);
		}
		SlowTask.EnterProgressFrame(10.0f);
	}
	else
	{
		// no mesh found in the FBX file
		AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("FbxSkeletaLMeshimport_NoFBXMeshMatch", "No FBX mesh matches the Unreal mesh '{0}'."), FText::FromString(Mesh->GetName()))), FFbxErrors::Generic_Mesh_MeshNotFound);
	}

	CleanUpImportOptionsData();

	return NewMesh;
}

void UnFbx::FFbxImporter::SetMaterialSkinXXOrder(FSkeletalMeshImportData& ImportData)
{
	TArray<int32> MaterialIndexToSkinIndex;
	TMap<int32, int32> SkinIndexToMaterialIndex;
	TArray<int32> MissingSkinSuffixMaterial;
	TMap<int32, int32> SkinIndexGreaterThenMaterialArraySize;
	{
		int32 MaterialCount = ImportData.Materials.Num();

		bool bNeedsReorder = false;
		for (int32 MaterialIndex = 0; MaterialIndex < MaterialCount; ++MaterialIndex)
		{
			// get skin index
			FString MatName = ImportData.Materials[MaterialIndex].MaterialImportName;

			if (MatName.Len() > 6)
			{
				int32 Offset = MatName.Find(TEXT("_SKIN"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
				if (Offset != INDEX_NONE)
				{
					// Chop off the material name so we are left with the number in _SKINXX
					FString SkinXXNumber = MatName.Right(MatName.Len() - (Offset + 1)).RightChop(4);

					if (SkinXXNumber.IsNumeric())
					{
						bNeedsReorder = true;
						int32 TmpIndex = FPlatformString::Atoi(*SkinXXNumber);
						if (TmpIndex < MaterialCount)
						{
							SkinIndexToMaterialIndex.Add(TmpIndex, MaterialIndex);
						}
						else
						{
							SkinIndexGreaterThenMaterialArraySize.Add(TmpIndex, MaterialIndex);
						}
						
					}
				}
				else
				{
					MissingSkinSuffixMaterial.Add(MaterialIndex);
				}
			}
			else
			{
				MissingSkinSuffixMaterial.Add(MaterialIndex);
			}
		}

		if (bNeedsReorder && MissingSkinSuffixMaterial.Num() > 0)
		{
			AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("FbxSkeletaLMeshimport_Skinxx_missing", "Cannot mix skinxx suffix materials with no skinxx material, mesh section order will not be right.")), FFbxErrors::Generic_Mesh_SkinxxNameError);
			return;
		}

		//Add greater then material array skinxx at the end sorted by integer the index will be remap correctly in the case of a LOD import
		if (SkinIndexGreaterThenMaterialArraySize.Num() > 0)
		{
			int32 MaxAvailableKey = SkinIndexToMaterialIndex.Num();
			for (int32 AvailableKey = 0; AvailableKey < MaxAvailableKey; ++AvailableKey)
			{
				if (SkinIndexToMaterialIndex.Contains(AvailableKey))
					continue;

				TMap<int32, int32> TempSkinIndexToMaterialIndex;
				for (auto KvpSkinToMat : SkinIndexToMaterialIndex)
				{
					if (KvpSkinToMat.Key > AvailableKey)
					{
						TempSkinIndexToMaterialIndex.Add(KvpSkinToMat.Key - 1, KvpSkinToMat.Value);
					}
					else
					{
						TempSkinIndexToMaterialIndex.Add(KvpSkinToMat.Key, KvpSkinToMat.Value);
					}
				}
				//move all the later key of the array to fill the available index
				SkinIndexToMaterialIndex = TempSkinIndexToMaterialIndex;
				AvailableKey--; //We need to retest the same index it can be empty
			}
			//Reorder the array
			SkinIndexGreaterThenMaterialArraySize.KeySort(TLess<int32>());
			for (auto Kvp : SkinIndexGreaterThenMaterialArraySize)
			{
				SkinIndexToMaterialIndex.Add(SkinIndexToMaterialIndex.Num(), Kvp.Value);
			}
		}

		//Fill the array MaterialIndexToSkinIndex so we order material by _skinXX order
		//This ensure we support skinxx suffixe that are not increment by one like _skin00, skin_01, skin_03, skin_04, skin_08... 
		for (auto kvp : SkinIndexToMaterialIndex)
		{
			int32 MatIndexToInsert = 0;
			for (MatIndexToInsert = 0; MatIndexToInsert < MaterialIndexToSkinIndex.Num(); ++MatIndexToInsert)
			{
				if (*(SkinIndexToMaterialIndex.Find(MaterialIndexToSkinIndex[MatIndexToInsert])) >= kvp.Value)
				{
					break;
				}
			}
			MaterialIndexToSkinIndex.Insert(kvp.Key, MatIndexToInsert);
		}

		if (bNeedsReorder)
		{
			// re-order the materials
			TArray< SkeletalMeshImportData::FMaterial > ExistingMatList = ImportData.Materials;
			for (int32 MissingIndex : MissingSkinSuffixMaterial)
			{
				MaterialIndexToSkinIndex.Insert(MaterialIndexToSkinIndex.Num(), MissingIndex);
			}
			for (int32 MaterialIndex = 0; MaterialIndex < MaterialCount; ++MaterialIndex)
			{
				if (MaterialIndex < MaterialIndexToSkinIndex.Num())
				{
					int32 NewIndex = MaterialIndexToSkinIndex[MaterialIndex];
					if (ExistingMatList.IsValidIndex(NewIndex))
					{
						ImportData.Materials[NewIndex] = ExistingMatList[MaterialIndex];
					}
				}
			}

			// remapping the material index for each triangle
			int32 FaceNum = ImportData.Faces.Num();
			for (int32 TriangleIndex = 0; TriangleIndex < FaceNum; TriangleIndex++)
			{
				SkeletalMeshImportData::FTriangle& Triangle = ImportData.Faces[TriangleIndex];
				if (Triangle.MatIndex < MaterialIndexToSkinIndex.Num())
				{
					Triangle.MatIndex = MaterialIndexToSkinIndex[Triangle.MatIndex];
				}
			}
		}
	}
}

void UnFbx::FFbxImporter::SetMaterialOrderByName(FSkeletalMeshImportData& ImportData, TArray<FName> LastImportedMaterialNames)
{
	TArray<int32> MaterialIndexToNameIndex;
	TMap<int32, int32> NameIndexToMaterialIndex;
	TArray<int32> MissingNameSuffixMaterial;
	TMap<int32, int32> NameIndexGreaterThenMaterialArraySize;
	{
		int32 MaterialCount = ImportData.Materials.Num();
		int32 MaxMaterialOrderedCount = 0;
		bool bNeedsReorder = false;
		for (int32 MaterialIndex = 0; MaterialIndex < MaterialCount; ++MaterialIndex)
		{
			FName MatName = FName(*(ImportData.Materials[MaterialIndex].MaterialImportName));
			bool bFoundValidName = false;
			for (int32 OrderedIndex = 0; OrderedIndex < LastImportedMaterialNames.Num(); ++OrderedIndex)
			{
				FName OrderedMaterialName = LastImportedMaterialNames[OrderedIndex];
				if (OrderedMaterialName == NAME_None)
				{
					continue;
				}
				if (OrderedMaterialName == MatName)
				{
					if (OrderedIndex < MaterialCount)
					{
						MaxMaterialOrderedCount = FMath::Max(MaxMaterialOrderedCount, OrderedIndex+1);
						NameIndexToMaterialIndex.Add(OrderedIndex, MaterialIndex);
					}
					else
					{
						NameIndexGreaterThenMaterialArraySize.Add(OrderedIndex, MaterialIndex);
					}
					bFoundValidName = true;
					bNeedsReorder = true;
					break;
				}
			}
			if (!bFoundValidName)
			{
				MissingNameSuffixMaterial.Add(MaterialIndex);
				MaxMaterialOrderedCount = FMath::Max(MaxMaterialOrderedCount, MaterialIndex+1);
			}
		}

		if (bNeedsReorder && MissingNameSuffixMaterial.Num() > 0)
		{
			//Add the missing name material at the end to not disturb the existing order
			TArray<int32> OrderedListMissing;
			OrderedListMissing.AddZeroed(MaxMaterialOrderedCount);
			for (auto Kvp : NameIndexToMaterialIndex)
			{
				OrderedListMissing[Kvp.Key] = -1;
			}
			for (int32 OrderedListMissingIndex = 0; OrderedListMissingIndex < OrderedListMissing.Num(); ++OrderedListMissingIndex)
			{
				if (MissingNameSuffixMaterial.Num() <= 0)
				{
					break;
				}

				if (OrderedListMissing[OrderedListMissingIndex] != 0)
					continue;

				NameIndexToMaterialIndex.Add(OrderedListMissingIndex, MissingNameSuffixMaterial.Pop());
			}
		}

		//Add greater then material array slot index at the end sorted by integer the index will be remap correctly in the case of a LOD import
		if (NameIndexGreaterThenMaterialArraySize.Num() > 0)
		{
			int32 MaxAvailableKey = NameIndexToMaterialIndex.Num();
			for (int32 AvailableKey = 0; AvailableKey < MaxAvailableKey; ++AvailableKey)
			{
				if (NameIndexToMaterialIndex.Contains(AvailableKey))
					continue;

				TMap<int32, int32> TempSkinIndexToMaterialIndex;
				for (auto KvpSkinToMat : NameIndexToMaterialIndex)
				{
					if (KvpSkinToMat.Key > AvailableKey)
					{
						TempSkinIndexToMaterialIndex.Add(KvpSkinToMat.Key - 1, KvpSkinToMat.Value);
					}
					else
					{
						TempSkinIndexToMaterialIndex.Add(KvpSkinToMat.Key, KvpSkinToMat.Value);
					}
				}
				//move all the later key of the array to fill the available index
				NameIndexToMaterialIndex = TempSkinIndexToMaterialIndex;
				AvailableKey--; //We need to retest the same index it can be empty
			}
			//Reorder the array
			NameIndexGreaterThenMaterialArraySize.KeySort(TLess<int32>());
			for (auto Kvp : NameIndexGreaterThenMaterialArraySize)
			{
				NameIndexToMaterialIndex.Add(NameIndexToMaterialIndex.Num(), Kvp.Value);
			}
		}

		//Fill the array MaterialIndexToNameIndex so we order material by ordered index
		for (auto kvp : NameIndexToMaterialIndex)
		{
			int32 MatIndexToInsert = 0;
			for (MatIndexToInsert = 0; MatIndexToInsert < MaterialIndexToNameIndex.Num(); ++MatIndexToInsert)
			{
				if (*(NameIndexToMaterialIndex.Find(MaterialIndexToNameIndex[MatIndexToInsert])) >= kvp.Value)
				{
					break;
				}
			}
			MaterialIndexToNameIndex.Insert(kvp.Key, MatIndexToInsert);
		}

		if (bNeedsReorder)
		{
			// re-order the materials
			TArray< SkeletalMeshImportData::FMaterial > ExistingMatList = ImportData.Materials;
			
			for (int32 MaterialIndex = 0; MaterialIndex < MaterialCount; ++MaterialIndex)
			{
				if (MaterialIndex < MaterialIndexToNameIndex.Num())
				{
					int32 NewIndex = MaterialIndexToNameIndex[MaterialIndex];
					if (ExistingMatList.IsValidIndex(NewIndex))
					{
						ImportData.Materials[NewIndex] = ExistingMatList[MaterialIndex];
					}
				}
			}

			// remapping the material index for each triangle
			int32 FaceNum = ImportData.Faces.Num();
			for (int32 TriangleIndex = 0; TriangleIndex < FaceNum; TriangleIndex++)
			{
				SkeletalMeshImportData::FTriangle& Triangle = ImportData.Faces[TriangleIndex];
				if (Triangle.MatIndex < MaterialIndexToNameIndex.Num())
				{
					Triangle.MatIndex = MaterialIndexToNameIndex[Triangle.MatIndex];
				}
			}
		}
	}
}

void UnFbx::FFbxImporter::CleanUpUnusedMaterials(FSkeletalMeshImportData& ImportData)
{
	if (ImportData.Materials.Num() <= 0)
	{
		return;
	}

	TArray< SkeletalMeshImportData::FMaterial > ExistingMatList = ImportData.Materials;

	TArray<uint8> UsedMaterialIndex;
	// Find all material that are use by the mesh faces
	int32 FaceNum = ImportData.Faces.Num();
	for (int32 TriangleIndex = 0; TriangleIndex < FaceNum; TriangleIndex++)
	{
		SkeletalMeshImportData::FTriangle& Triangle = ImportData.Faces[TriangleIndex];
		UsedMaterialIndex.AddUnique(Triangle.MatIndex);
	}
	//Remove any unused material.
	if (UsedMaterialIndex.Num() < ExistingMatList.Num())
	{
		TArray<int32> RemapIndex;
		TArray< SkeletalMeshImportData::FMaterial > &NewMatList = ImportData.Materials;
		NewMatList.Empty();
		for (int32 ExistingMatIndex = 0; ExistingMatIndex < ExistingMatList.Num(); ++ExistingMatIndex)
		{
			if (UsedMaterialIndex.Contains((uint8)ExistingMatIndex))
			{
				RemapIndex.Add(NewMatList.Add(ExistingMatList[ExistingMatIndex]));
			}
			else
			{
				RemapIndex.Add(INDEX_NONE);
			}
		}
		ImportData.MaxMaterialIndex = 0;
		//Remap the face material index
		for (int32 TriangleIndex = 0; TriangleIndex < FaceNum; TriangleIndex++)
		{
			SkeletalMeshImportData::FTriangle& Triangle = ImportData.Faces[TriangleIndex];
			check(RemapIndex[Triangle.MatIndex] != INDEX_NONE);
			Triangle.MatIndex = RemapIndex[Triangle.MatIndex];
			ImportData.MaxMaterialIndex = FMath::Max<uint32>(ImportData.MaxMaterialIndex, Triangle.MatIndex);
		}
	}
}

bool UnFbx::FFbxImporter::FillSkelMeshImporterFromFbx( FSkeletalMeshImportData& ImportData, FbxMesh*& Mesh, FbxSkin* Skin, FbxShape* FbxShape, TArray<FbxNode*> &SortedLinks, const TArray<FbxSurfaceMaterial*>& FbxMaterials, FbxNode *RootNode, const TMap<FVector3f, FColor>& ExistingVertexColorData)
{
	FbxNode* Node = Mesh->GetNode();

	//remove the bad polygons before getting any data from mesh
	Mesh->RemoveBadPolygons();

	//Get the base layer of the mesh
	FbxLayer* BaseLayer = Mesh->GetLayer(0);
	if (BaseLayer == NULL)
	{
		AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("FbxSkeletaLMeshimport_NoGeometry", "There is no geometry information in mesh")), FFbxErrors::Generic_Mesh_NoGeometry);
		return false;
	}

	// Do some checks before proceeding, check to make sure the number of bones does not exceed the maximum supported
	if(SortedLinks.Num() > MAX_BONES)
	{
		AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("FbxSkeletaLMeshimport_ExceedsMaxBoneCount", "'{0}' mesh has '{1}' bones which exceeds the maximum allowed bone count of {2}."), FText::FromString(Node->GetName()), FText::AsNumber(SortedLinks.Num()), FText::AsNumber(MAX_BONES))), FFbxErrors::SkeletalMesh_ExceedsMaxBoneCount);
		return false;
	}

	//
	//	store the UVs in arrays for fast access in the later looping of triangles 
	//
	// mapping from UVSets to Fbx LayerElementUV
	// Fbx UVSets may be duplicated, remove the duplicated UVSets in the mapping 
	int32 LayerCount = Mesh->GetLayerCount();
	TArray<FString> UVSets;

	UVSets.Empty();
	if (LayerCount > 0)
	{
		int32 UVLayerIndex;
		for (UVLayerIndex = 0; UVLayerIndex<LayerCount; UVLayerIndex++)
		{
			FbxLayer* lLayer = Mesh->GetLayer(UVLayerIndex);
			int32 UVSetCount = lLayer->GetUVSetCount();
			if(UVSetCount)
			{
				FbxArray<FbxLayerElementUV const*> EleUVs = lLayer->GetUVSets();
				for (int32 UVIndex = 0; UVIndex<UVSetCount; UVIndex++)
				{
					FbxLayerElementUV const* ElementUV = EleUVs[UVIndex];
					if (ElementUV)
					{
						const char* UVSetName = ElementUV->GetName();
						FString LocalUVSetName = UTF8_TO_TCHAR(UVSetName);
						if (LocalUVSetName.IsEmpty())
						{
							LocalUVSetName = TEXT("UVmap_") + FString::FromInt(UVLayerIndex);
						}
						UVSets.AddUnique(LocalUVSetName);
					}
				}
			}
		}
	}

	// If the the UV sets are named using the following format (UVChannel_X; where X ranges from 1 to 4)
	// we will re-order them based on these names.  Any UV sets that do not follow this naming convention
	// will be slotted into available spaces.
	if( UVSets.Num() > 0 )
	{
		for(int32 ChannelNumIdx = 0; ChannelNumIdx < 4; ChannelNumIdx++)
		{
			FString ChannelName = FString::Printf( TEXT("UVChannel_%d"), ChannelNumIdx+1 );
			int32 SetIdx = UVSets.Find( ChannelName );

			// If the specially formatted UVSet name appears in the list and it is in the wrong spot,
			// we will swap it into the correct spot.
			if( SetIdx != INDEX_NONE && SetIdx != ChannelNumIdx )
			{
				// If we are going to swap to a position that is outside the bounds of the
				// array, then we pad out to that spot with empty data.
				for(int32 ArrSize = UVSets.Num(); ArrSize < ChannelNumIdx+1; ArrSize++)
				{
					UVSets.Add( FString(TEXT("")) );
				}
				//Swap the entry into the appropriate spot.
				UVSets.Swap( SetIdx, ChannelNumIdx );
			}
		}
	}


	TArray<UMaterialInterface*> Materials;
	const bool bForSkeletalMesh = true;

	FindOrImportMaterialsFromNode(Node, Materials, UVSets, bForSkeletalMesh);
	if (!ImportOptions->bImportMaterials && ImportOptions->bImportTextures)
	{
		//If we are not importing any new material, we might still want to import new textures.
		ImportTexturesFromNode(Node);
	}

	// Maps local mesh material index to global material index
	const int32 MaterialCount = Node->GetMaterialCount();
	TArray<int32> MaterialMapping;
	MaterialMapping.AddUninitialized(MaterialCount);

	for( int32 MaterialIndex = 0; MaterialIndex < MaterialCount; ++MaterialIndex )
	{
		FbxSurfaceMaterial* FbxMaterial = Node->GetMaterial( MaterialIndex );

		int32 ExistingMatIndex = INDEX_NONE;

		// Default behavior finds the first material in FbxMaterials that matches the node's materials.
		// This results in combining nodes/sections that use the same material.
		if (!ImportOptions->bKeepSectionsSeparate)
		{
			FbxMaterials.Find(FbxMaterial, ExistingMatIndex);
		}
		else
		{
			// Otherwise find the material in FbxMaterials specific to that node's section/surface
			for (int32 FbxMatIndex = 0; FbxMatIndex < FbxMaterials.Num(); FbxMatIndex++)
			{
				FString NodeMaterialIndexName = MakeNodeMaterialIndexName(Node, MaterialIndex);
				bool MaterialImportNameMatches = ImportData.Materials[FbxMatIndex].MaterialImportName == NodeMaterialIndexName;
				bool MaterialInitialNameMatches = FbxMaterials[FbxMatIndex]->GetInitialName() == FbxMaterial->GetInitialName();

				if (MaterialImportNameMatches && MaterialInitialNameMatches)
				{
					ExistingMatIndex = FbxMatIndex;
					break;
				}
			}
		}

		if (ExistingMatIndex != INDEX_NONE)
		{
			// Reuse existing material
			MaterialMapping[MaterialIndex] = ExistingMatIndex;

			if (!ImportOptions->bKeepSectionsSeparate)
			{
				if (Materials.IsValidIndex(MaterialIndex))
				{
					ImportData.Materials[ExistingMatIndex].Material = Materials[MaterialIndex];
				}
			}
			else
			{
				if (Materials.IsValidIndex(MaterialIndex) && ImportData.Materials.IsValidIndex(ExistingMatIndex))
				{
					ImportData.Materials[ExistingMatIndex].MaterialImportName = MakeNodeMaterialIndexName(Node, MaterialIndex);
					ImportData.Materials[ExistingMatIndex].Material = Materials[MaterialIndex];
				}
			}
		}
		else
		{
			MaterialMapping[MaterialIndex] = 0;
		}
	}

	if( LayerCount > 0 && ImportOptions->bPreserveSmoothingGroups )
	{
		// Check and see if the smooothing data is valid.  If not generate it from the normals
		BaseLayer = Mesh->GetLayer(0);
		if( BaseLayer )
		{
			const FbxLayerElementSmoothing* SmoothingLayer = BaseLayer->GetSmoothing();

			if( SmoothingLayer )
			{
				bool bValidSmoothingData = false;
				FbxLayerElementArrayTemplate<int32>& Array = SmoothingLayer->GetDirectArray();
				for( int32 SmoothingIndex = 0; SmoothingIndex < Array.GetCount(); ++SmoothingIndex )
				{
					if( Array[SmoothingIndex] != 0 )
					{
						bValidSmoothingData = true;
						break;
					}
				}

				if( !bValidSmoothingData && Mesh->GetPolygonVertexCount() > 0 )
				{
					GeometryConverter->ComputeEdgeSmoothingFromNormals(Mesh);
				}
			}
		}
	}

	// Must do this before triangulating the mesh due to an FBX bug in TriangulateMeshAdvance
	int32 LayerSmoothingCount = Mesh->GetLayerCount(FbxLayerElement::eSmoothing);
	for(int32 i = 0; i < LayerSmoothingCount; i++)
	{
		FbxLayerElementSmoothing const* SmoothingInfo = Mesh->GetLayer(i)->GetSmoothing();
		if (SmoothingInfo && SmoothingInfo->GetMappingMode() != FbxLayerElement::eByPolygon)
		{
			GeometryConverter->ComputePolygonSmoothingFromEdgeSmoothing(Mesh, i);
		}
	}

	//
	// Convert data format to unreal-compatible
	//

	if (!Mesh->IsTriangleMesh())
	{
		UE_LOGF(LogFbx, Log, "Triangulating skeletal mesh %ls", UTF8_TO_TCHAR(Node->GetName()));
		
		const bool bReplace = true;
		FbxNodeAttribute* ConvertedNode = GeometryConverter->Triangulate(Mesh, bReplace);
		if( ConvertedNode != NULL && ConvertedNode->GetAttributeType() == FbxNodeAttribute::eMesh )
		{
			Mesh = ConvertedNode->GetNode()->GetMesh();
		}
		else
		{
			AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("FbxSkeletaLMeshimport_TriangulatingFailed", "Unable to triangulate mesh '{0}'. Check detail for Ouput Log."), FText::FromString(Node->GetName()))), FFbxErrors::Generic_Mesh_TriangulationFailed);
			return false;
		}
	}
	
	// renew the base layer
	BaseLayer = Mesh->GetLayer(0);
	Skin = (FbxSkin*)static_cast<FbxGeometry*>(Mesh)->GetDeformer(0, FbxDeformer::eSkin);

	//
	//	store the UVs in arrays for fast access in the later looping of triangles 
	//
	uint32 UniqueUVCount = UVSets.Num();
	FbxLayerElementUV** LayerElementUV = NULL;
	FbxLayerElement::EReferenceMode* UVReferenceMode = NULL;
	FbxLayerElement::EMappingMode* UVMappingMode = NULL;
	if (UniqueUVCount > 0)
	{
		LayerElementUV = new FbxLayerElementUV*[UniqueUVCount];
		UVReferenceMode = new FbxLayerElement::EReferenceMode[UniqueUVCount];
		UVMappingMode = new FbxLayerElement::EMappingMode[UniqueUVCount];
	}
	else
	{
		AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("FbxSkeletaLMeshimport_NoUVSet", "Mesh '{0}' has no UV set. Creating a default set."), FText::FromString(Node->GetName()))), FFbxErrors::SkeletalMesh_NoUVSet);
	}
	
	LayerCount = Mesh->GetLayerCount();
	for (uint32 UVIndex = 0; UVIndex < UniqueUVCount; UVIndex++)
	{
		LayerElementUV[UVIndex] = NULL;
		for (int32 UVLayerIndex = 0; UVLayerIndex<LayerCount; UVLayerIndex++)
		{
			FbxLayer* lLayer = Mesh->GetLayer(UVLayerIndex);
			int32 UVSetCount = lLayer->GetUVSetCount();
			if(UVSetCount)
			{
				FbxArray<FbxLayerElementUV const*> EleUVs = lLayer->GetUVSets();
				for (int32 FbxUVIndex = 0; FbxUVIndex<UVSetCount; FbxUVIndex++)
				{
					FbxLayerElementUV const* ElementUV = EleUVs[FbxUVIndex];
					if (ElementUV)
					{
						const char* UVSetName = ElementUV->GetName();
						FString LocalUVSetName = UTF8_TO_TCHAR(UVSetName);
						if (LocalUVSetName.IsEmpty())
						{
							LocalUVSetName = TEXT("UVmap_") + FString::FromInt(UVLayerIndex);
						}
						if (LocalUVSetName == UVSets[UVIndex])
						{
							LayerElementUV[UVIndex] = const_cast<FbxLayerElementUV*>(ElementUV);
							UVReferenceMode[UVIndex] = LayerElementUV[UVIndex]->GetReferenceMode();
							UVMappingMode[UVIndex] = LayerElementUV[UVIndex]->GetMappingMode();
							break;
						}
					}
				}
			}
		}
	}

	//
	// get the smoothing group layer
	//
	bool bSmoothingAvailable = false;

	FbxLayerElementSmoothing const* SmoothingInfo = BaseLayer->GetSmoothing();
	FbxLayerElement::EReferenceMode SmoothingReferenceMode(FbxLayerElement::eDirect);
	FbxLayerElement::EMappingMode SmoothingMappingMode(FbxLayerElement::eByEdge);
	if (SmoothingInfo)
	{
 		if( SmoothingInfo->GetMappingMode() == FbxLayerElement::eByEdge )
		{
			if (!GeometryConverter->ComputePolygonSmoothingFromEdgeSmoothing(Mesh))
			{
				AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("FbxSkeletaLMeshimport_ConvertSmoothingGroupFailed", "Unable to fully convert the smoothing groups for mesh '{0}'"), FText::FromString(Mesh->GetName()))), FFbxErrors::Generic_Mesh_ConvertSmoothingGroupFailed);
				bSmoothingAvailable = false;
			}
			else
			{
				//After using the geometry converter we always have to get the Layer and the smoothing info
				BaseLayer = Mesh->GetLayer(0);
				SmoothingInfo = BaseLayer->GetSmoothing();
			}
		}

		if( SmoothingInfo->GetMappingMode() == FbxLayerElement::eByPolygon )
		{
			bSmoothingAvailable = true;
		}


		SmoothingReferenceMode = SmoothingInfo->GetReferenceMode();
		SmoothingMappingMode = SmoothingInfo->GetMappingMode();
	}


	//
	//	get the "material index" layer
	//
	FbxLayerElementMaterial* LayerElementMaterial = BaseLayer->GetMaterials();
	FbxLayerElement::EMappingMode MaterialMappingMode = LayerElementMaterial ? 
		LayerElementMaterial->GetMappingMode() : FbxLayerElement::eByPolygon;

	UniqueUVCount = FMath::Min<uint32>( UniqueUVCount, MAX_TEXCOORDS );

	// One UV set is required but only import up to MAX_TEXCOORDS number of uv layers
	ImportData.NumTexCoords = FMath::Max<uint32>( ImportData.NumTexCoords, UniqueUVCount );

	//
	// get the first vertex color layer
	//
	
	FbxLayerElementVertexColor* LayerElementVertexColor = BaseLayer->GetVertexColors();
	FbxLayerElement::EReferenceMode VertexColorReferenceMode(FbxLayerElement::eDirect);
	FbxLayerElement::EMappingMode VertexColorMappingMode(FbxLayerElement::eByControlPoint);
	if (LayerElementVertexColor && ImportOptions->VertexColorImportOption == EVertexColorImportOption::Replace)
	{
		VertexColorReferenceMode = LayerElementVertexColor->GetReferenceMode();
		VertexColorMappingMode = LayerElementVertexColor->GetMappingMode();
		ImportData.bHasVertexColors = true;
	}
	else if(ImportOptions->VertexColorImportOption == EVertexColorImportOption::Override)
	{
		ImportData.bHasVertexColors = true;
	}
	else if (ImportOptions->VertexColorImportOption == EVertexColorImportOption::Ignore && ExistingVertexColorData.Num() != 0)
	{
		ImportData.bHasVertexColors = true;
	}

	//
	// get the first normal layer
	//
	FbxLayerElementNormal* LayerElementNormal = BaseLayer->GetNormals();
	FbxLayerElementTangent* LayerElementTangent = BaseLayer->GetTangents();
	FbxLayerElementBinormal* LayerElementBinormal = BaseLayer->GetBinormals();

	//whether there is normal, tangent and binormal data in this mesh
	bool bHasNormalInformation = LayerElementNormal != NULL;
	bool bHasTangentInformation = LayerElementTangent != NULL && LayerElementBinormal != NULL;

	ImportData.bHasNormals = bHasNormalInformation;
	ImportData.bHasTangents = bHasTangentInformation;

	FbxLayerElement::EReferenceMode NormalReferenceMode(FbxLayerElement::eDirect);
	FbxLayerElement::EMappingMode NormalMappingMode(FbxLayerElement::eByControlPoint);
	if (LayerElementNormal)
	{
		NormalReferenceMode = LayerElementNormal->GetReferenceMode();
		NormalMappingMode = LayerElementNormal->GetMappingMode();
	}

	FbxLayerElement::EReferenceMode TangentReferenceMode(FbxLayerElement::eDirect);
	FbxLayerElement::EMappingMode TangentMappingMode(FbxLayerElement::eByControlPoint);
	if (LayerElementTangent)
	{
		TangentReferenceMode = LayerElementTangent->GetReferenceMode();
		TangentMappingMode = LayerElementTangent->GetMappingMode();
	}

	FbxLayerElement::EReferenceMode BinormalReferenceMode(FbxLayerElement::eDirect);
	FbxLayerElement::EMappingMode BinormalMappingMode(FbxLayerElement::eByControlPoint);
	if (LayerElementBinormal)
	{
		BinormalReferenceMode = LayerElementBinormal->GetReferenceMode();
		BinormalMappingMode = LayerElementBinormal->GetMappingMode();
	}

	//
	// create the points / wedges / faces
	//
	int32 ControlPointsCount = Mesh->GetControlPointsCount();
	int32 ExistPointNum = ImportData.Points.Num();
	FillSkeletalMeshImportPoints( &ImportData, RootNode, Node, FbxShape );

	// Construct the matrices for the conversion from right handed to left handed system
	FbxAMatrix TotalMatrix;
	FbxAMatrix TotalMatrixForNormal;
	TotalMatrix = ComputeSkeletalMeshTotalMatrix(Node, RootNode);
	TotalMatrixForNormal = TotalMatrix.Inverse();
	TotalMatrixForNormal = TotalMatrixForNormal.Transpose();

	bool OddNegativeScale = IsOddNegativeScale(TotalMatrix);
	
	int32 TriangleCount = Mesh->GetPolygonCount();
	int32 ExistFaceNum = ImportData.Faces.Num();
	ImportData.Faces.AddUninitialized( TriangleCount );
	int32 ExistWedgesNum = ImportData.Wedges.Num();
	SkeletalMeshImportData::FVertex TmpWedges[3];

	bool bUnsupportedSmoothingGroupErrorDisplayed = false;
	bool bFaceMaterialIndexInconsistencyErrorDisplayed = false;
	for( int32 TriangleIndex = ExistFaceNum, LocalIndex = 0 ; TriangleIndex < ExistFaceNum+TriangleCount ; TriangleIndex++, LocalIndex++ )
	{

		SkeletalMeshImportData::FTriangle& Triangle = ImportData.Faces[TriangleIndex];

		//
		// smoothing mask
		//
		// set the face smoothing by default. It could be any number, but not zero
		Triangle.SmoothingGroups = 255; 
		if ( bSmoothingAvailable)
		{
			if (SmoothingInfo)
			{
				if (SmoothingMappingMode == FbxLayerElement::eByPolygon)
				{
					int32 lSmoothingIndex = (SmoothingReferenceMode == FbxLayerElement::eDirect) ? LocalIndex : SmoothingInfo->GetIndexArray().GetAt(LocalIndex); //-V516 PVS 7.41 bug
					Triangle.SmoothingGroups = SmoothingInfo->GetDirectArray().GetAt(lSmoothingIndex);
				}
				else if(!bUnsupportedSmoothingGroupErrorDisplayed)
				{
					AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("FbxSkeletaLMeshimport_Unsupportingsmoothinggroup", "Unsupported Smoothing group mapping mode on mesh '{0}'"), FText::FromString(Mesh->GetName()))), FFbxErrors::Generic_Mesh_UnsupportingSmoothingGroup);
					bUnsupportedSmoothingGroupErrorDisplayed = true;
				}
			}
		}

		for (int32 VertexIndex=0; VertexIndex<3; VertexIndex++)
		{
			// If there are odd number negative scale, invert the vertex order for triangles
			int32 UnrealVertexIndex = OddNegativeScale ? 2 - VertexIndex : VertexIndex;

			int32 ControlPointIndex = Mesh->GetPolygonVertex(LocalIndex, VertexIndex);
			//
			// normals, tangents and binormals
			//
			if( bHasNormalInformation )
			{
				const int32 TmpIndex = LocalIndex*3 + VertexIndex;
				// normals may have different reference and mapping mode than tangents and binormals
				const int32 NormalMapIndex = (NormalMappingMode == FbxLayerElement::eByControlPoint) ?
										ControlPointIndex : TmpIndex;
				const int32 NormalValueIndex = (NormalReferenceMode == FbxLayerElement::eDirect) ? //-V516 PVS 7.41 bug
										NormalMapIndex : LayerElementNormal->GetIndexArray().GetAt(NormalMapIndex);

				// tangents and binormals share the same reference, mapping mode and index array
				FbxVector4 TempValue;

				if( bHasTangentInformation )
				{
					const int32 TangentMapIndex = (TangentMappingMode == FbxLayerElement::eByControlPoint) ? ControlPointIndex : TmpIndex;
					const int32 TangentValueIndex = (TangentReferenceMode == FbxLayerElement::eDirect) ? TangentMapIndex : LayerElementTangent->GetIndexArray().GetAt(TangentMapIndex); //-V516 PVS 7.41 bug
					TempValue = LayerElementTangent->GetDirectArray().GetAt(TangentValueIndex);
					TempValue = TotalMatrixForNormal.MultT(TempValue);
					Triangle.TangentX[ UnrealVertexIndex ] = (FVector3f)Converter.ConvertDir(TempValue);
					Triangle.TangentX[ UnrealVertexIndex ].Normalize();

					const int32 BinormalMapIndex = (BinormalMappingMode == FbxLayerElement::eByControlPoint) ? ControlPointIndex : TmpIndex;
					const int32 BinormalValueIndex = (BinormalReferenceMode == FbxLayerElement::eDirect) ? BinormalMapIndex : LayerElementBinormal->GetIndexArray().GetAt(BinormalMapIndex); //-V516 PVS 7.41 bug
					TempValue = LayerElementBinormal->GetDirectArray().GetAt(BinormalValueIndex);
					TempValue = TotalMatrixForNormal.MultT(TempValue);
					Triangle.TangentY[ UnrealVertexIndex ] = (FVector3f)-Converter.ConvertDir(TempValue);
					Triangle.TangentY[ UnrealVertexIndex ].Normalize();
				}

				TempValue = LayerElementNormal->GetDirectArray().GetAt(NormalValueIndex);
				TempValue = TotalMatrixForNormal.MultT(TempValue);
				Triangle.TangentZ[ UnrealVertexIndex ] = (FVector3f)Converter.ConvertDir(TempValue);
				Triangle.TangentZ[ UnrealVertexIndex ].Normalize();
			
			}
			else
			{
				for( int32 NormalIndex = 0; NormalIndex < 3; ++NormalIndex )
				{
					Triangle.TangentX[ NormalIndex ] = FVector3f::ZeroVector;
					Triangle.TangentY[ NormalIndex ] = FVector3f::ZeroVector;
					Triangle.TangentZ[ NormalIndex ] = FVector3f::ZeroVector;
				}
			}
		}
		
		//
		// material index
		//
		Triangle.MatIndex = 0; // default value
		if (MaterialCount>0)
		{
			if (LayerElementMaterial)
			{
				switch(MaterialMappingMode)
				{
				// material index is stored in the IndexArray, not the DirectArray (which is irrelevant with 2009.1)
				case FbxLayerElement::eAllSame:
					{	
						Triangle.MatIndex = MaterialMapping[ LayerElementMaterial->GetIndexArray().GetAt(0) ];
					}
					break;
				case FbxLayerElement::eByPolygon:
					{	
						int32 Index = LayerElementMaterial->GetIndexArray().GetAt(LocalIndex);							
						if (!MaterialMapping.IsValidIndex(Index))
						{
							if (!bFaceMaterialIndexInconsistencyErrorDisplayed)
							{
								AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, LOCTEXT("FbxSkeletaLMeshimport_MaterialIndexInconsistency", "Face material index inconsistency - forcing to 0")), FFbxErrors::Generic_Mesh_MaterialIndexInconsistency);
								bFaceMaterialIndexInconsistencyErrorDisplayed = true;
							}
						}
						else
						{
							Triangle.MatIndex = MaterialMapping[Index];
						}
					}
					break;
				}
			}

			// When import morph, we don't check the material index 
			// because we don't import material for morph, so the ImportData.Materials contains zero material
			if ( !FbxShape && (Triangle.MatIndex < 0 ||  Triangle.MatIndex >= FbxMaterials.Num() ) )
			{
				if (!bFaceMaterialIndexInconsistencyErrorDisplayed)
				{
					AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, LOCTEXT("FbxSkeletaLMeshimport_MaterialIndexInconsistency", "Face material index inconsistency - forcing to 0")), FFbxErrors::Generic_Mesh_MaterialIndexInconsistency);
					bFaceMaterialIndexInconsistencyErrorDisplayed = true;
				}
				Triangle.MatIndex = 0;
			}
		}
		ImportData.MaxMaterialIndex = FMath::Max<uint32>( ImportData.MaxMaterialIndex, Triangle.MatIndex );

		Triangle.AuxMatIndex = 0;
		for (int32 VertexIndex=0; VertexIndex<3; VertexIndex++)
		{
			// If there are odd number negative scale, invert the vertex order for triangles
			int32 UnrealVertexIndex = OddNegativeScale ? 2 - VertexIndex : VertexIndex;

			TmpWedges[UnrealVertexIndex].MatIndex = Triangle.MatIndex;
			TmpWedges[UnrealVertexIndex].VertexIndex = ExistPointNum + Mesh->GetPolygonVertex(LocalIndex,VertexIndex);
			// Initialize all colors to white.
			TmpWedges[UnrealVertexIndex].Color = FColor::White;
		}

		//
		// uvs
		//
		uint32 UVLayerIndex;
		// Some FBX meshes can have no UV sets, so also check the UniqueUVCount
		for ( UVLayerIndex = 0; UVLayerIndex< UniqueUVCount; UVLayerIndex++ )
		{
			// ensure the layer has data
			if (LayerElementUV[UVLayerIndex] != NULL) 
			{
				// Get each UV from the layer
				for (int32 VertexIndex=0;VertexIndex<3;VertexIndex++)
				{
					// If there are odd number negative scale, invert the vertex order for triangles
					int32 UnrealVertexIndex = OddNegativeScale ? 2 - VertexIndex : VertexIndex;

					int32 lControlPointIndex = Mesh->GetPolygonVertex(LocalIndex, VertexIndex);
					int32 UVMapIndex = (UVMappingMode[UVLayerIndex] == FbxLayerElement::eByControlPoint) ? 
							lControlPointIndex : LocalIndex*3+VertexIndex;
					int32 UVIndex = (UVReferenceMode[UVLayerIndex] == FbxLayerElement::eDirect) ? 
							UVMapIndex : LayerElementUV[UVLayerIndex]->GetIndexArray().GetAt(UVMapIndex);
					FbxVector2	UVVector = LayerElementUV[UVLayerIndex]->GetDirectArray().GetAt(UVIndex);

					TmpWedges[UnrealVertexIndex].UVs[ UVLayerIndex ].X = static_cast<float>(UVVector[0]);
					TmpWedges[UnrealVertexIndex].UVs[ UVLayerIndex ].Y = 1.f - static_cast<float>(UVVector[1]);
				}
			}
			else if( UVLayerIndex == 0 )
			{
				// Set all UV's to zero.  If we are here the mesh had no UV sets so we only need to do this for the
				// first UV set which always exists.

				for (int32 VertexIndex=0; VertexIndex<3; VertexIndex++)
				{
					TmpWedges[VertexIndex].UVs[UVLayerIndex].X = 0.0f;
					TmpWedges[VertexIndex].UVs[UVLayerIndex].Y = 0.0f;
				}
			}
		}

		// Read vertex colors if they exist.
		if( LayerElementVertexColor && ImportOptions->VertexColorImportOption == EVertexColorImportOption::Replace)
		{
			auto ConvertFbxColorToFColor = [](const FbxColor& VertexColor)
				{
					// Clamp manually to avoid wrapping around because of truncation for values outside of the 0.0 to 1.0 range
					return FColor(uint8(255.f * FMath::Clamp(VertexColor.mRed, 0.0, 1.0)),
						uint8(255.f * FMath::Clamp(VertexColor.mGreen, 0.0, 1.0)),
						uint8(255.f * FMath::Clamp(VertexColor.mBlue, 0.0, 1.0)),
						uint8(255.f * FMath::Clamp(VertexColor.mAlpha, 0.0, 1.0)));
				};

			switch(VertexColorMappingMode)
			{
			case FbxLayerElement::eByControlPoint:
				{
					for (int32 VertexIndex=0;VertexIndex<3;VertexIndex++)
					{
						int32 UnrealVertexIndex = OddNegativeScale ? 2 - VertexIndex : VertexIndex;

						FbxColor VertexColor = (VertexColorReferenceMode == FbxLayerElement::eDirect) //-V516 PVS 7.41 bug
							?	LayerElementVertexColor->GetDirectArray().GetAt(Mesh->GetPolygonVertex(LocalIndex,VertexIndex))
							:	LayerElementVertexColor->GetDirectArray().GetAt(LayerElementVertexColor->GetIndexArray().GetAt(Mesh->GetPolygonVertex(LocalIndex,VertexIndex)));

						
						TmpWedges[UnrealVertexIndex].Color = ConvertFbxColorToFColor(VertexColor);
					}
				}
				break;
			case FbxLayerElement::eByPolygonVertex:
				{	
					for (int32 VertexIndex=0;VertexIndex<3;VertexIndex++)
					{
						int32 UnrealVertexIndex = OddNegativeScale ? 2 - VertexIndex : VertexIndex;

						FbxColor VertexColor = (VertexColorReferenceMode == FbxLayerElement::eDirect) //-V516 PVS 7.41 bug
							?	LayerElementVertexColor->GetDirectArray().GetAt(LocalIndex*3+VertexIndex)
							:	LayerElementVertexColor->GetDirectArray().GetAt(LayerElementVertexColor->GetIndexArray().GetAt(LocalIndex*3+VertexIndex));

						TmpWedges[UnrealVertexIndex].Color = ConvertFbxColorToFColor(VertexColor);
					}
				}
				break;
			}
		}
		else if (ImportOptions->VertexColorImportOption == EVertexColorImportOption::Override)
		{
			for (int32 VertexIndex = 0; VertexIndex < 3; VertexIndex++)
			{
				int32 UnrealVertexIndex = OddNegativeScale ? 2 - VertexIndex : VertexIndex;
				TmpWedges[UnrealVertexIndex].Color = ImportOptions->VertexOverrideColor;
			}
		}
		else if (ImportOptions->VertexColorImportOption == EVertexColorImportOption::Ignore && ImportData.bHasVertexColors)
		{
			for (int32 VertexIndex = 0; VertexIndex < 3; VertexIndex++)
			{
				const FVector3f& VertexPosition = ImportData.Points[TmpWedges[VertexIndex].VertexIndex];
				const FColor* PaintedColor = ExistingVertexColorData.Find(VertexPosition);
				
				// try to match this wedge current vertex with one that existed in the previous mesh.
				// This is a find in a tmap which uses a fast hash table lookup.
				if (PaintedColor)
				{
					TmpWedges[VertexIndex].Color = *PaintedColor;
				}
			}
		}
		
		//
		// basic wedges matching : 3 unique per face. TODO Can we do better ?
		//
		for (int32 VertexIndex=0; VertexIndex<3; VertexIndex++)
		{
			int32 w;
			
			w = ImportData.Wedges.AddUninitialized();
			ImportData.Wedges[w].VertexIndex = TmpWedges[VertexIndex].VertexIndex;
			ImportData.Wedges[w].MatIndex = TmpWedges[VertexIndex].MatIndex;
			ImportData.Wedges[w].Color = TmpWedges[VertexIndex].Color;
			ImportData.Wedges[w].Reserved = 0;
			FMemory::Memcpy( ImportData.Wedges[w].UVs, TmpWedges[VertexIndex].UVs, sizeof(FVector2f)*MAX_TEXCOORDS );
			
			Triangle.WedgeIndex[VertexIndex] = w;
		}
		
	}
	
	// now we can work on a per-cluster basis with good ordering
	if (Skin) // skeletal mesh
	{
		// create influences for each cluster
		for (int32 ClusterIndex = 0; ClusterIndex < Skin->GetClusterCount(); ClusterIndex++)
		{
			FbxCluster* Cluster = Skin->GetCluster(ClusterIndex);
			// When Maya plug-in exports rigid binding, it will generate "CompensationCluster" for each ancestor links.
			// FBX writes these "CompensationCluster" out. The CompensationCluster also has weight 1 for vertices.
			// Unreal importer should skip these clusters.
			if(!Cluster || (FCStringAnsi::Strcmp(Cluster->GetUserDataID(), "Maya_ClusterHint") == 0 && FCStringAnsi::Strcmp(Cluster->GetUserData(), "CompensationCluster")  == 0))
			{
				continue;
			}
			
			FbxNode* Link = Cluster->GetLink();
			// find the bone index
			int32 BoneIndex = -1;
			SortedLinks.Find(Link, BoneIndex);

			//	get the vertex indices
			int32 ControlPointIndicesCount = Cluster->GetControlPointIndicesCount();
			int32* ControlPointIndices = Cluster->GetControlPointIndices();
			double* Weights = Cluster->GetControlPointWeights();

			//	for each vertex index in the cluster
			for (int32 ControlPointIndex = 0; ControlPointIndex < ControlPointIndicesCount; ++ControlPointIndex) 
			{
				ImportData.Influences.AddUninitialized();
				ImportData.Influences.Last().BoneIndex = BoneIndex;
				ImportData.Influences.Last().Weight = static_cast<float>(Weights[ControlPointIndex]);
				ImportData.Influences.Last().VertexIndex = ExistPointNum + ControlPointIndices[ControlPointIndex];
			}
		}
	}
	else // for rigid mesh
	{
		// find the bone index, the bone is the node itself
		int32 BoneIndex = -1;
		SortedLinks.Find(Node, BoneIndex);
		
		if (BoneIndex == -1 && ImportOptions->bImportAsSkeletalGeometry && SortedLinks.Num() > 0)
		{
			//When we import geometry only, we want to hook all the influence to a particular bone
			BoneIndex = 0;
		}

		//	for each vertex in the mesh
		for (int32 ControlPointIndex = 0; ControlPointIndex < ControlPointsCount; ++ControlPointIndex) 
		{
			ImportData.Influences.AddUninitialized();
			ImportData.Influences.Last().BoneIndex = BoneIndex;
			ImportData.Influences.Last().Weight = 1.0;
			ImportData.Influences.Last().VertexIndex = ExistPointNum + ControlPointIndex;
		}
	}

	//
	// Get the vertex attribute layers from all layers, even the first layer which may be used as a vertex color layer.
	// Currently we're only interested in alpha-only layers, since those are the only layer types the engine
	// currently exposes for vertex attributes. Internally we can store 1-4 components, but there's no tooling for that
	// 2-4 channels as of yet.
	//
	struct FNamedVertexAttribute : SkeletalMeshImportData::FVertexAttribute
	{
		FNamedVertexAttribute(FString&& InAttributeName, TArray<float>&& InAttributeValues, const int32 InComponentCount)
			: FVertexAttribute(MoveTemp(InAttributeValues), InComponentCount)
			, AttributeName(InAttributeName)
		{}
			
		FString AttributeName;
	};

	if (ImportOptions->bImportVertexAttributes)
	{
		TArray<FNamedVertexAttribute> NamedVertexAttributes;

		for (int32 LayerIndex = 0; LayerIndex < LayerCount; LayerIndex++)
		{
			FbxLayerElementVertexColor* LayerElementVertexAttribute = Mesh->GetLayer(LayerIndex)->GetVertexColors();
			if (!LayerElementVertexAttribute)
			{
				continue;
			}

			// Check if this is an alpha-only attribute, by ensuring the RGB values are all zero, otherwise skip.
			bool bIsValidAttribute = true;
			const FbxLayerElementArrayTemplate<FbxColor>& AttributeValues = LayerElementVertexAttribute->GetDirectArray();
			for (int32 Index = 0; Index < AttributeValues.GetCount(); Index++)
			{
				// We do an exact comparison, since that's how empty channels would be represented in the FBX file.
				const FbxColor& Value = AttributeValues.GetAt(Index); 
				if (Value.mRed != 0.0 || Value.mGreen != 0.0 || Value.mBlue != 0.0)
				{
					bIsValidAttribute = false;
					break;
				}
			}

			// We can only do attributes that are mapped per-vertex.
			if (!bIsValidAttribute)
			{
				continue;
			}

			const int32 AttributeComponentCount = 1;	// Number of component values per vertex. See comment above. 
			TArray<float> AttributeComponentValues;
			
			switch(LayerElementVertexAttribute->GetMappingMode())
			{
			case FbxLayerElement::eByControlPoint:
				{
					AttributeComponentValues.AddZeroed(ControlPointsCount);

					if (LayerElementVertexAttribute->GetReferenceMode() == FbxLayerElement::eDirect)
					{
						for (int32 Index = 0; Index < AttributeValues.GetCount(); Index++)
						{
							AttributeComponentValues[Index] = AttributeValues.GetAt(Index).mAlpha;
						}
					}
					else // LayerElementVertexAttribute->GetReferenceMode() == FbxLayerElement::eIndexToDirect
					{
						const FbxLayerElementArrayTemplate<int>& IndexArray = LayerElementVertexAttribute->GetIndexArray();
						for (int32 Index = 0; Index < IndexArray.GetCount(); Index++)
						{
							AttributeComponentValues[Index] = AttributeValues.GetAt(IndexArray[Index]).mAlpha;
						}
					}
				}
				break;
			case FbxLayerElement::eByPolygonVertex:
				{
					// Vertex attributes are stored per-vertex, not per-vertex instance. To work around this we average
					// together values that share a vertex.
					TArray<int32> SharedVertexCount;
					SharedVertexCount.AddZeroed(ControlPointsCount);
					AttributeComponentValues.AddZeroed(ControlPointsCount);

					const FbxLayerElementArrayTemplate<int>* IndexArray = nullptr;
					if (LayerElementVertexAttribute->GetReferenceMode() == FbxLayerElement::eIndexToDirect)
					{
						IndexArray = &LayerElementVertexAttribute->GetIndexArray();
					}

					const int* PolygonControlPointIndexes = Mesh->GetPolygonVertices();
					
					for(int32 TriangleIndex = 0; TriangleIndex < TriangleCount; TriangleIndex++)
					{
						for (int32 InnerIndex = 0; InnerIndex < 3; InnerIndex++)
						{
							const int32 PolygonVertexIndex = TriangleIndex * 3 + InnerIndex;;
							const int32 PointIndex = PolygonControlPointIndexes[PolygonVertexIndex];

							AttributeComponentValues[PointIndex] +=
									AttributeValues.GetAt(IndexArray ? IndexArray->GetAt(PolygonVertexIndex) : PolygonVertexIndex).mAlpha;
							SharedVertexCount[PointIndex]++;
						}					
					}

					for (int32 PointIndex = 0; PointIndex < ControlPointsCount; PointIndex++)
					{
						if (SharedVertexCount[PointIndex] > 1)
						{
							AttributeComponentValues[PointIndex] /= static_cast<float>(SharedVertexCount[PointIndex]);
						}
					}
				}
				break;
			default:
				break;
			}

			if (!AttributeComponentValues.IsEmpty())
			{
				FString AttributeName(UTF8_TO_TCHAR(LayerElementVertexAttribute->GetName()));
				NamedVertexAttributes.Emplace(MoveTemp(AttributeName), MoveTemp(AttributeComponentValues), AttributeComponentCount);
			}
		}

		//
		// Add in the attributes that we received, and pad any non-matching existing attributes with zero values to
		// match the new point count.
		//
		if (ExistFaceNum)
		{
			// TODO: In the future, once we support attributes with non-unity component counts, we need to decide how to deal
			// with attributes that share the same name but differing component counts. Two options:
			// 1) We create a new attribute with a new name that differs from the existing name. Preferably with the component
			//    count somehow embedded in the name so that we can deal with appending again.
			// 2) Widen the existing attribute's component count to match the new one (or vice versa, widening the new attribute
			//    so that we can append).
			
			const int32 VertexCount = Mesh->GetControlPointsCount();
			
			for (int32 ExistingAttributeIndex = 0; ExistingAttributeIndex < ImportData.VertexAttributes.Num(); ExistingAttributeIndex++)
			{
				const FString& ExistingAttributeName = ImportData.VertexAttributeNames[ExistingAttributeIndex];
				const int32 ExistingAttributeComponentCount = ImportData.VertexAttributes[ExistingAttributeIndex].ComponentCount;
				
				int32 NewAttributeIndex = NamedVertexAttributes.IndexOfByPredicate(
					[ExistingAttributeName](const FNamedVertexAttribute& InAttribute)
					{
						return InAttribute.AttributeName == ExistingAttributeName && ensure(InAttribute.ComponentCount == 1);
					});

				if (ExistingAttributeComponentCount != 1)
				{
					// For now, if an existing attribute has a non-unity component count, we don't append the new attribute values
					// on top, just pad with zeroes. 
					const FText ErrorMsg = FText::Format(LOCTEXT("FbxSkeletalMeshimport_AttributeComponentCountMismatch", "Existing attribute '{0}' has more than one component. Ignoring imported attribute of same name."),
							FText::FromString(ExistingAttributeName)); 
					AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, ErrorMsg), FFbxErrors::SkeletalMesh_AttributeComponentCountMismatch);
					
					NewAttributeIndex = INDEX_NONE;
				}

				SkeletalMeshImportData::FVertexAttribute& ExistingAttribute = ImportData.VertexAttributes[ExistingAttributeIndex];
				if (NewAttributeIndex != INDEX_NONE)
				{
					ExistingAttribute.AttributeValues.Append(NamedVertexAttributes[NewAttributeIndex].AttributeValues);
					NamedVertexAttributes.RemoveAt(NewAttributeIndex);
				}
				else
				{
					ExistingAttribute.AttributeValues.AddZeroed(VertexCount * ExistingAttribute.ComponentCount);
				}
			}

			// Any remaining attributes we add + padding for existing points.
			for (FNamedVertexAttribute& NamedVertexAttribute: NamedVertexAttributes)
			{
				ImportData.VertexAttributeNames.Emplace(MoveTemp(NamedVertexAttribute.AttributeName));
				
				SkeletalMeshImportData::FVertexAttribute& NewAttribute = ImportData.VertexAttributes.AddDefaulted_GetRef();
				NewAttribute.ComponentCount = NamedVertexAttribute.ComponentCount;
				NewAttribute.AttributeValues.AddZeroed(ExistPointNum * NamedVertexAttribute.ComponentCount);
				NewAttribute.AttributeValues.Append(NamedVertexAttribute.AttributeValues);
			}
		}
		else
		{
			for (FNamedVertexAttribute& NamedVertexAttribute: NamedVertexAttributes)
			{
				ImportData.VertexAttributes.Emplace(MoveTemp(NamedVertexAttribute.AttributeValues), NamedVertexAttribute.ComponentCount);
				ImportData.VertexAttributeNames.Emplace(MoveTemp(NamedVertexAttribute.AttributeName));
			}
		}
	}

	//
	// clean up
	//
	if (LayerElementUV)
	{
		delete[] LayerElementUV;
	}
	if (UVReferenceMode)
	{
		delete[] UVReferenceMode;
	}
	if (UVMappingMode)
	{
		delete[] UVMappingMode;
	}
	
	return true;
}

void UnFbx::FFbxImporter::InsertNewLODToBaseSkeletalMesh(USkeletalMesh* InSkeletalMesh, USkeletalMesh* BaseSkeletalMesh, int32 DesiredLOD, UFbxSkeletalMeshImportData* TemplateImportData)
{
	//Scope a post edit change
	FScopedSkeletalMeshPostEditChange ScopedPostEditChange(BaseSkeletalMesh);

	FSkeletalMeshModel* ImportedResource = InSkeletalMesh->GetImportedModel();
	FSkeletalMeshModel* DestImportedResource = BaseSkeletalMesh->GetImportedModel();

	FSkeletalMeshLODModel& NewLODModel = ImportedResource->LODModels[0];


	//Fill the data we need to recover the user section material slot assignation
	TArray<FName> ExistingMeshSectionSlotNames;
	TArray<FName> OriginalImportMeshSectionSlotNames;
	UFbxSkeletalMeshImportData* ImportData = nullptr;
	bool HasReimportData = (TemplateImportData != nullptr && DesiredLOD != 0 && BaseSkeletalMesh->GetLODNum() > DesiredLOD);
	if (HasReimportData)
	{
		ImportData = UFbxSkeletalMeshImportData::GetImportDataForSkeletalMesh(BaseSkeletalMesh, TemplateImportData);
		HasReimportData = (ImportData != nullptr && ImportData->ImportMeshLodData.Num() > DesiredLOD);
		if (HasReimportData)
		{
			const FImportMeshLodSectionsData& OriginalImportMeshLodSectionsData = ImportData->ImportMeshLodData[DesiredLOD];
			const FSkeletalMeshLODInfo &ExistingSkelMeshLodInfo = *(BaseSkeletalMesh->GetLODInfo(DesiredLOD));
			//Restore the section changes from the old import data
			for (int32 SectionIndex = 0; SectionIndex < ExistingSkelMeshLodInfo.LODMaterialMap.Num(); SectionIndex++)
			{
				if (ExistingSkelMeshLodInfo.LODMaterialMap.Num() <= SectionIndex || OriginalImportMeshLodSectionsData.SectionOriginalMaterialName.Num() <= SectionIndex)
				{
					break;
				}

				//Get the current skelmesh section slot import name
				int32 ExistRemapMaterialIndex = ExistingSkelMeshLodInfo.LODMaterialMap[SectionIndex];
				if (!BaseSkeletalMesh->GetMaterials().IsValidIndex(ExistRemapMaterialIndex))
				{
					//If we have an INDEX_NONE it mean we have to use the section material index. If we have a bad value we will use the section material index if possible
					if (DestImportedResource->LODModels.IsValidIndex(DesiredLOD) && DestImportedResource->LODModels[DesiredLOD].Sections.IsValidIndex(SectionIndex))
					{
						ExistRemapMaterialIndex = DestImportedResource->LODModels[DesiredLOD].Sections[SectionIndex].MaterialIndex;
					}
					else
					{
						//If the material is wrong log and pick the zero index
						UE_LOGF(LogFbx, Display, "Insert LOD, invalid material rematch: LOD [%d] Section[%d]- Skeletalmesh:[%ls]", DesiredLOD, SectionIndex, *BaseSkeletalMesh->GetFullName());
						ExistRemapMaterialIndex = 0;
					}
				}
				check(BaseSkeletalMesh->GetMaterials().IsValidIndex(ExistRemapMaterialIndex));

				ExistingMeshSectionSlotNames.Add(BaseSkeletalMesh->GetMaterials()[ExistRemapMaterialIndex].ImportedMaterialSlotName);

				//Get the Last imported skelmesh section slot import name
				OriginalImportMeshSectionSlotNames.Add(OriginalImportMeshLodSectionsData.SectionOriginalMaterialName[SectionIndex]);
			}

			if (DestImportedResource->LODModels.IsValidIndex(DesiredLOD))
			{
				const FSkeletalMeshLODModel& ExistLODModel = DestImportedResource->LODModels[DesiredLOD];
				for (int32 NewSectionIndex = 0; NewSectionIndex < NewLODModel.Sections.Num(); NewSectionIndex++)
				{
					FSkelMeshSection& NewSection = NewLODModel.Sections[NewSectionIndex];
					int32 MatIdx = NewSection.MaterialIndex;
					for (int32 ExistSectionIndex = 0; ExistSectionIndex < ExistLODModel.Sections.Num(); ExistSectionIndex++)
					{
						if (OriginalImportMeshSectionSlotNames.IsValidIndex(ExistSectionIndex) && OriginalImportMeshSectionSlotNames[ExistSectionIndex] == InSkeletalMesh->GetMaterials()[MatIdx].ImportedMaterialSlotName)
						{
							const FSkelMeshSection& ExistSection = ExistLODModel.Sections[ExistSectionIndex];
							//We found a match, restore the data
							NewSection.bCastShadow = ExistSection.bCastShadow;
							NewSection.bVisibleInRayTracing = ExistSection.bVisibleInRayTracing;
							NewSection.bRecomputeTangent = ExistSection.bRecomputeTangent;
							NewSection.RecomputeTangentsVertexMaskChannel = ExistSection.RecomputeTangentsVertexMaskChannel;
							NewSection.bDisabled = ExistSection.bDisabled;
							NewSection.GenerateUpToLodIndex = ExistSection.GenerateUpToLodIndex;
							bool bBoneChunkedSection = NewSection.ChunkedParentSectionIndex >= 0;
							int32 ParentOriginalSectionIndex = NewSection.OriginalDataSectionIndex;
							if (!bBoneChunkedSection)
							{
								//Set the new Parent Index
								FSkelMeshSourceSectionUserData& UserSectionData = NewLODModel.UserSectionsData.FindOrAdd(ParentOriginalSectionIndex);
								UserSectionData.bDisabled = NewSection.bDisabled;
								UserSectionData.bCastShadow = NewSection.bCastShadow;
								UserSectionData.bVisibleInRayTracing = NewSection.bVisibleInRayTracing;
								UserSectionData.bRecomputeTangent = NewSection.bRecomputeTangent;					
								UserSectionData.RecomputeTangentsVertexMaskChannel = NewSection.RecomputeTangentsVertexMaskChannel;
								UserSectionData.GenerateUpToLodIndex = NewSection.GenerateUpToLodIndex;
								//The cloth will be rebind later after the reimport is done
							}
							break;
						}
					}
				}
			}
		}
	}

	// If we want to add this as a new LOD to this mesh - add to LODModels/LODInfo array.
	if (DesiredLOD == DestImportedResource->LODModels.Num())
	{
		DestImportedResource->LODModels.Add(new FSkeletalMeshLODModel());

		// Add element to LODInfo array.
		BaseSkeletalMesh->AddLODInfo();
		check(BaseSkeletalMesh->GetLODNum() == DestImportedResource->LODModels.Num());
	}

	// Set up LODMaterialMap to number of materials in new mesh.
	FSkeletalMeshLODInfo& LODInfo = *(BaseSkeletalMesh->GetLODInfo(DesiredLOD));

	//Copy the build settings
	if(InSkeletalMesh->GetLODInfo(0))
	{
		const FSkeletalMeshLODInfo& ImportedLODInfo = *(InSkeletalMesh->GetLODInfo(0));
		LODInfo.BuildSettings = ImportedLODInfo.BuildSettings;
	}
	
	TArray<FSkeletalMaterial>& BaseMaterials = BaseSkeletalMesh->GetMaterials();
	LODInfo.LODMaterialMap.Empty();
	// Now set up the material mapping array.
	for (int32 SectionIndex = 0; SectionIndex < NewLODModel.Sections.Num(); SectionIndex++)
	{
		int32 MatIdx = NewLODModel.Sections[SectionIndex].MaterialIndex;
		// Try and find the auto-assigned material in the array.
		int32 LODMatIndex = INDEX_NONE;
		//First try to match by name
		for (int32 BaseMaterialIndex = 0; BaseMaterialIndex < BaseMaterials.Num(); ++BaseMaterialIndex)
		{
			const FSkeletalMaterial& SkeletalMaterial = BaseMaterials[BaseMaterialIndex];
			if (SkeletalMaterial.ImportedMaterialSlotName != NAME_None && SkeletalMaterial.ImportedMaterialSlotName == InSkeletalMesh->GetMaterials()[MatIdx].ImportedMaterialSlotName)
			{
				LODMatIndex = BaseMaterialIndex;
				break;
			}
		}

		// If we dont have a match, add a new entry to the material list.
		if (LODMatIndex == INDEX_NONE)
		{
			LODMatIndex = BaseMaterials.Add(InSkeletalMesh->GetMaterials()[MatIdx]);
		}

		LODInfo.LODMaterialMap.Add(LODMatIndex);
	}

	// Release all resources before replacing the model
	BaseSkeletalMesh->PreEditChange(nullptr);

	// Assign new FSkeletalMeshLODModel to desired slot in selected skeletal mesh.
	FSkeletalMeshLODModel::CopyStructure(&(DestImportedResource->LODModels[DesiredLOD]), &NewLODModel);
	//Copy the import data into the base skeletalmesh for the imported LOD
	FMeshDescription SourceMeshDescription;
	if (InSkeletalMesh->CloneMeshDescription(0, SourceMeshDescription))
	{
		BaseSkeletalMesh->CreateMeshDescription(DesiredLOD, MoveTemp(SourceMeshDescription));
		BaseSkeletalMesh->CommitMeshDescription(DesiredLOD);
	}
	

	// If this LOD had been generated previously by automatic mesh reduction, clear that flag.
	LODInfo.bHasBeenSimplified = false;
	if (BaseSkeletalMesh->GetLODSettings() == nullptr || !BaseSkeletalMesh->GetLODSettings()->HasValidSettings() || BaseSkeletalMesh->GetLODSettings()->GetNumberOfSettings() <= DesiredLOD)
	{
		//Make sure any custom LOD have correct settings (no reduce)
		LODInfo.ReductionSettings.NumOfTrianglesPercentage = 1.0f;
		LODInfo.ReductionSettings.MaxNumOfTriangles = MAX_uint32;
		LODInfo.ReductionSettings.MaxNumOfTrianglesPercentage = MAX_uint32;
		LODInfo.ReductionSettings.NumOfVertPercentage = 1.0f;
		LODInfo.ReductionSettings.MaxNumOfVerts = MAX_uint32;
		LODInfo.ReductionSettings.MaxNumOfVertsPercentage = MAX_uint32;
		LODInfo.ReductionSettings.MaxDeviationPercentage = 0.0f;
	}

	//Set back the user data
	if (HasReimportData && BaseSkeletalMesh->GetLODNum() > DesiredLOD)
	{
		FSkeletalMeshLODInfo &NewSkelMeshLodInfo = *(BaseSkeletalMesh->GetLODInfo(DesiredLOD));
		//Restore the section changes from the old import data
		for (int32 SectionIndex = 0; SectionIndex < NewLODModel.Sections.Num(); SectionIndex++)
		{
			if (ExistingMeshSectionSlotNames.Num() <= SectionIndex || OriginalImportMeshSectionSlotNames.Num() <= SectionIndex)
			{
				break;
			}
			//Get the current skelmesh section slot import name
			FName ExistMeshSectionSlotName = ExistingMeshSectionSlotNames[SectionIndex];
			//Get the new skelmesh section slot import name
			int32 NewRemapMaterialIndex = NewSkelMeshLodInfo.LODMaterialMap[SectionIndex];
			if (!BaseMaterials.IsValidIndex(NewRemapMaterialIndex))
			{
				int32 RematchMaterialIndex = NewSkelMeshLodInfo.LODMaterialMap[SectionIndex];
				if (RematchMaterialIndex == INDEX_NONE)
				{
					RematchMaterialIndex = NewLODModel.Sections[SectionIndex].MaterialIndex;
				}
				NewSkelMeshLodInfo.LODMaterialMap[SectionIndex] = FMath::Clamp<int32>(RematchMaterialIndex, 0, BaseMaterials.Num()-1);
				continue;
			}
			FName NewMeshSectionSlotName = BaseMaterials[NewRemapMaterialIndex].ImportedMaterialSlotName;
			//Get the Last imported skelmesh section slot import name
			FName OriginalImportMeshSectionSlotName = OriginalImportMeshSectionSlotNames[SectionIndex];

			if (OriginalImportMeshSectionSlotName == NewMeshSectionSlotName && ExistMeshSectionSlotName != OriginalImportMeshSectionSlotName)
			{
				//The last import slot name match the New import slot name, but the Exist slot name is different then the last import slot name.
				//This mean the user has change the section assign slot and the fbx file did not change it
				//Override the new section material index to use the one that the user set
				for (int32 RemapMaterialIndex = 0; RemapMaterialIndex < BaseMaterials.Num(); ++RemapMaterialIndex)
				{
					const FSkeletalMaterial &NewSectionMaterial = BaseMaterials[RemapMaterialIndex];
					if (NewSectionMaterial.ImportedMaterialSlotName == ExistMeshSectionSlotName)
					{
						NewSkelMeshLodInfo.LODMaterialMap[SectionIndex] = RemapMaterialIndex;
						break;
					}
				}
			}
		}
	}
}

bool UnFbx::FFbxImporter::ImportSkeletalMeshLOD(USkeletalMesh* InSkeletalMesh, USkeletalMesh* BaseSkeletalMesh, int32 DesiredLOD, UFbxSkeletalMeshImportData* TemplateImportData)
{
	check(InSkeletalMesh);
	check(BaseSkeletalMesh);

	FSkeletalMeshModel* ImportedResource = InSkeletalMesh->GetImportedModel();
	FSkeletalMeshModel* DestImportedResource = BaseSkeletalMesh->GetImportedModel();

	// Now we copy the base FSkeletalMeshLODModel from the imported skeletal mesh as the new LOD in the selected mesh.
	check(ImportedResource->LODModels.Num() == 1);

	// Names of root bones must match.
	// If the names of root bones don't match, the LOD Mesh does not share skeleton with base Mesh. 
	if (InSkeletalMesh->GetRefSkeleton().GetBoneName(0) != BaseSkeletalMesh->GetRefSkeleton().GetBoneName(0) && !ImportOptions->bImportAsSkeletalGeometry)
	{
		AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("LODRootNameIncorrect", "Root bone in LOD is '{0}' instead of '{1}'.\nImport failed."),
			FText::FromName(InSkeletalMesh->GetRefSkeleton().GetBoneName(0)), FText::FromName(BaseSkeletalMesh->GetRefSkeleton().GetBoneName(0)))), FFbxErrors::SkeletalMesh_LOD_RootNameIncorrect);

		return false;
	}
	bool bApplyBaseSkinning = false;
	// We do some checking here that for every bone in the mesh we just imported, it's in our base ref skeleton, and the parent is the same.
	for (int32 i = 0; i < InSkeletalMesh->GetRefSkeleton().GetRawBoneNum(); i++)
	{
		int32 LODBoneIndex = i;
		FName LODBoneName = InSkeletalMesh->GetRefSkeleton().GetBoneName(LODBoneIndex);
		int32 BaseBoneIndex = BaseSkeletalMesh->GetRefSkeleton().FindBoneIndex(LODBoneName);
		if (BaseBoneIndex == INDEX_NONE)
		{
			if (!ImportOptions->bImportAsSkeletalGeometry)
			{
				// If we could not find the bone from this LOD in base mesh - we fail.
				AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("LODBoneDoesNotMatch", "Bone '{0}' not found in base SkeletalMesh '{1}'.\nImport failed."),
					FText::FromName(LODBoneName), FText::FromString(BaseSkeletalMesh->GetName()))), FFbxErrors::SkeletalMesh_LOD_BonesDoNotMatch);
				return false;
			}

			bApplyBaseSkinning = true;
			break;
		}

		if (i > 0)
		{
			int32 LODParentIndex = InSkeletalMesh->GetRefSkeleton().GetParentIndex(LODBoneIndex);
			FName LODParentName = InSkeletalMesh->GetRefSkeleton().GetBoneName(LODParentIndex);

			int32 BaseParentIndex = BaseSkeletalMesh->GetRefSkeleton().GetParentIndex(BaseBoneIndex);
			FName BaseParentName = BaseSkeletalMesh->GetRefSkeleton().GetBoneName(BaseParentIndex);

			if (LODParentName != BaseParentName)
			{
				if (!ImportOptions->bImportAsSkeletalGeometry)
				{
					// If bone has different parents, display an error and don't allow import.
					AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("LODBoneHasIncorrectParent", "Bone '{0}' in LOD has parent '{1}' instead of '{2}'"),
						FText::FromName(LODBoneName), FText::FromName(LODParentName), FText::FromName(BaseParentName))), FFbxErrors::SkeletalMesh_LOD_IncorrectParent);
					return false;
				}
				bApplyBaseSkinning = true;
				break;
			}
		}
	}

	FScopedSkeletalMeshPostEditChange ScopedPostEditChange(BaseSkeletalMesh);

	//Skeleton do not match and we are importing skeletal geometry only so apply the base LOD Skinning
	if (bApplyBaseSkinning)
	{
		check(ImportOptions->bImportAsSkeletalGeometry);
		//Apply the base LOD skinning to the new LOD geometry, DestImportedResource here is the base lod and is the source for the apply skinning.
		//The Dest here mean we merge the just import LOD to the base LodModel.
		SkeletalMeshImportUtils::ApplySkinning(InSkeletalMesh, DestImportedResource->LODModels[0], ImportedResource->LODModels[0]);

		//Excluded bones will be remove when we rebuild the asset
		//IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshUtilities>("MeshUtilities");
		//MeshUtilities.RemoveBonesFromMesh(BaseSkeletalMesh, DesiredLOD, NULL);
	}

	FSkeletalMeshLODModel& NewLODModel = ImportedResource->LODModels[0];

	// Enforce LODs having only single-influence vertices.
	bool bCheckSingleInfluence;
	GConfig->GetBool(TEXT("ImportSetting"), TEXT("CheckSingleInfluenceLOD"), bCheckSingleInfluence, GEditorIni);
	if (bCheckSingleInfluence &&
		DesiredLOD > 0)
	{
		for (int32 SectionIndex = 0; SectionIndex < NewLODModel.Sections.Num(); SectionIndex++)
		{
			if (NewLODModel.Sections[SectionIndex].SoftVertices.Num() > 0)
			{
				AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText(LOCTEXT("LODHasSoftVertices", "Warning: The mesh LOD you are importing has some vertices with more than one influence."))), FFbxErrors::SkeletalMesh_LOD_HasSoftVerts);
			}
		}
	}

	// If this LOD is going to be the lowest one, we check all bones we have sockets on are present in it.
	if (DesiredLOD == DestImportedResource->LODModels.Num() ||
		DesiredLOD == DestImportedResource->LODModels.Num() - 1)
	{
		const TArray<USkeletalMeshSocket*>& Sockets = BaseSkeletalMesh->GetMeshOnlySocketList();

		for (int32 i = 0; i < Sockets.Num(); i++)
		{
			// Find bone index the socket is attached to.
			USkeletalMeshSocket* Socket = Sockets[i];
			int32 SocketBoneIndex = InSkeletalMesh->GetRefSkeleton().FindBoneIndex(Socket->BoneName);

			// If this LOD does not contain the socket bone, abort import.
			if (SocketBoneIndex == INDEX_NONE)
			{
				AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("LODMissingSocketBone", "This LOD is missing bone '{0}' used by socket '{1}'.\nAborting import."),
					FText::FromName(Socket->BoneName), FText::FromName(Socket->SocketName))), FFbxErrors::SkeletalMesh_LOD_MissingSocketBone);

				return false;
			}
		}
	}

	if (!bApplyBaseSkinning)
	{
		//The imported LOD is always in LOD 0 of the InSkeletalMesh
		const int32 SourceLODIndex = 0;
		if(InSkeletalMesh->HasMeshDescription(SourceLODIndex))
		{
			// Fix up the imported data bone indexes
			FSkeletalMeshImportData LODImportData;
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			InSkeletalMesh->LoadLODImportedData(SourceLODIndex, LODImportData);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			const int32 LODImportDataBoneNumber = LODImportData.RefBonesBinary.Num();
			//We want to create a remap array so we can fix all influence easily
			TArray<int32> ImportDataBoneRemap;
			ImportDataBoneRemap.AddZeroed(LODImportDataBoneNumber);
			//We generate a new RefBonesBinary array to replace the existing one
			TArray<SkeletalMeshImportData::FBone> RemapedRefBonesBinary;
			RemapedRefBonesBinary.AddZeroed(BaseSkeletalMesh->GetRefSkeleton().GetNum());
			for (int32 ImportBoneIndex = 0; ImportBoneIndex < LODImportDataBoneNumber; ++ImportBoneIndex)
			{
				SkeletalMeshImportData::FBone& ImportedBone = LODImportData.RefBonesBinary[ImportBoneIndex];
				int32 LODBoneIndex = ImportBoneIndex;
				FName LODBoneName = FName(*FSkeletalMeshImportData::FixupBoneName(ImportedBone.Name));
				int32 BaseBoneIndex = BaseSkeletalMesh->GetRefSkeleton().FindBoneIndex(LODBoneName);
				ImportDataBoneRemap[ImportBoneIndex] = BaseBoneIndex;
				if (BaseBoneIndex != INDEX_NONE)
				{
					RemapedRefBonesBinary[BaseBoneIndex] = ImportedBone;
					if(RemapedRefBonesBinary[BaseBoneIndex].ParentIndex != INDEX_NONE)
					{
						RemapedRefBonesBinary[BaseBoneIndex].ParentIndex = ImportDataBoneRemap[RemapedRefBonesBinary[BaseBoneIndex].ParentIndex];
					}
				}
			}
			//Copy the new RefBonesBinary over the existing one
			LODImportData.RefBonesBinary = RemapedRefBonesBinary;
		
			//Fix the influences
			bool bNeedShrinking = false;
			const int32 InfluenceNumber = LODImportData.Influences.Num();
			for (int32 InfluenceIndex = InfluenceNumber-1; InfluenceIndex >= 0; --InfluenceIndex)
			{
				SkeletalMeshImportData::FRawBoneInfluence& Influence = LODImportData.Influences[InfluenceIndex];
				Influence.BoneIndex = ImportDataBoneRemap.IsValidIndex(Influence.BoneIndex) ? ImportDataBoneRemap[Influence.BoneIndex] : INDEX_NONE;
				if (Influence.BoneIndex == INDEX_NONE)
				{
					const int32 DeleteCount = 1;
					LODImportData.Influences.RemoveAt(InfluenceIndex, DeleteCount, EAllowShrinking::No);
					bNeedShrinking = true;
				}
			}
			//Shrink the array if we have deleted at least one entry
			if(bNeedShrinking)
			{
				LODImportData.Influences.Shrink();
			}
			//Save the fix up remap bone index
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			InSkeletalMesh->SaveLODImportedData(SourceLODIndex, LODImportData);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}

		// Fix up the ActiveBoneIndices array.
		for (int32 ActiveIndex = 0; ActiveIndex < NewLODModel.ActiveBoneIndices.Num(); ActiveIndex++)
		{
			int32 LODBoneIndex = NewLODModel.ActiveBoneIndices[ActiveIndex];
			FName LODBoneName = InSkeletalMesh->GetRefSkeleton().GetBoneName(LODBoneIndex);
			int32 BaseBoneIndex = BaseSkeletalMesh->GetRefSkeleton().FindBoneIndex(LODBoneName);
			NewLODModel.ActiveBoneIndices[ActiveIndex] = BaseBoneIndex;
		}

		// Fix up the chunk BoneMaps.
		for (int32 SectionIndex = 0; SectionIndex < NewLODModel.Sections.Num(); SectionIndex++)
		{
			FSkelMeshSection& Section = NewLODModel.Sections[SectionIndex];
			for (int32 BoneMapIndex = 0; BoneMapIndex < Section.BoneMap.Num(); BoneMapIndex++)
			{
				int32 LODBoneIndex = Section.BoneMap[BoneMapIndex];
				FName LODBoneName = InSkeletalMesh->GetRefSkeleton().GetBoneName(LODBoneIndex);
				int32 BaseBoneIndex = BaseSkeletalMesh->GetRefSkeleton().FindBoneIndex(LODBoneName);
				Section.BoneMap[BoneMapIndex] = BaseBoneIndex;
			}
		}

		// Create the RequiredBones array in the LODModel from the ref skeleton.
		for (int32 RequiredBoneIndex = 0; RequiredBoneIndex < NewLODModel.RequiredBones.Num(); RequiredBoneIndex++)
		{
			FName LODBoneName = InSkeletalMesh->GetRefSkeleton().GetBoneName(NewLODModel.RequiredBones[RequiredBoneIndex]);
			int32 BaseBoneIndex = BaseSkeletalMesh->GetRefSkeleton().FindBoneIndex(LODBoneName);
			if (BaseBoneIndex != INDEX_NONE)
			{
				NewLODModel.RequiredBones[RequiredBoneIndex] = BaseBoneIndex;
			}
			else
			{
				NewLODModel.RequiredBones.RemoveAt(RequiredBoneIndex--);
			}
		}

		// Also sort the RequiredBones array to be strictly increasing.
		NewLODModel.RequiredBones.Sort();
		BaseSkeletalMesh->GetRefSkeleton().EnsureParentsExistAndSort(NewLODModel.ActiveBoneIndices);
	}
	// To be extra-nice, we apply the difference between the root transform of the meshes to the verts.
	FMatrix44f LODToBaseTransform = FMatrix44f(InSkeletalMesh->GetRefPoseMatrix(0).InverseFast() * BaseSkeletalMesh->GetRefPoseMatrix(0));

	for (int32 SectionIndex = 0; SectionIndex < NewLODModel.Sections.Num(); SectionIndex++)
	{
		FSkelMeshSection& Section = NewLODModel.Sections[SectionIndex];

		// Fix up soft verts.
		for (int32 i = 0; i < Section.SoftVertices.Num(); i++)
		{
			Section.SoftVertices[i].Position = LODToBaseTransform.TransformPosition(Section.SoftVertices[i].Position);
			Section.SoftVertices[i].TangentX = LODToBaseTransform.TransformVector(Section.SoftVertices[i].TangentX);
			Section.SoftVertices[i].TangentY = LODToBaseTransform.TransformVector(Section.SoftVertices[i].TangentY);
			Section.SoftVertices[i].TangentZ = LODToBaseTransform.TransformVector(Section.SoftVertices[i].TangentZ);
		}
	}

	//Restore the LOD section data in case this LOD was reimport and some material match
	if (DestImportedResource->LODModels.IsValidIndex(DesiredLOD) && ImportedResource->LODModels.IsValidIndex(0))
	{
		const TArray<FSkelMeshSection>& ExistingSections = DestImportedResource->LODModels[DesiredLOD].Sections;
		const FSkeletalMeshLODInfo& ExistingInfo = *(BaseSkeletalMesh->GetLODInfo(DesiredLOD));

		TArray<FSkelMeshSection>& ImportedSections = ImportedResource->LODModels[0].Sections;
		const FSkeletalMeshLODInfo& ImportedInfo = *(InSkeletalMesh->GetLODInfo(0));
		
		auto GetImportMaterialSlotName = [](const USkeletalMesh* SkelMesh, const FSkelMeshSection& Section, int32 SectionIndex, const FSkeletalMeshLODInfo& Info, int32& OutMaterialIndex)->FName
		{
			const TArray<FSkeletalMaterial>& MeshMaterials = SkelMesh->GetMaterials();
			check(MeshMaterials.Num() > 0);
			OutMaterialIndex = Section.MaterialIndex;
			if (Info.LODMaterialMap.IsValidIndex(SectionIndex) && Info.LODMaterialMap[SectionIndex] != INDEX_NONE)
			{
				OutMaterialIndex = Info.LODMaterialMap[SectionIndex];
			}
			FName ImportedMaterialSlotName = NAME_None;
			if (MeshMaterials.IsValidIndex(OutMaterialIndex))
			{
				ImportedMaterialSlotName = MeshMaterials[OutMaterialIndex].ImportedMaterialSlotName;
			}
			else
			{
				ImportedMaterialSlotName = MeshMaterials[0].ImportedMaterialSlotName;
				OutMaterialIndex = 0;
			}
			return ImportedMaterialSlotName;
		};

		for(int32 ExistingSectionIndex = 0; ExistingSectionIndex < ExistingSections.Num(); ++ExistingSectionIndex)
		{
			const FSkelMeshSection& ExistingSection = ExistingSections[ExistingSectionIndex];
			int32 ExistingMaterialIndex = 0;
			FName ExistingImportedMaterialSlotName = GetImportMaterialSlotName(BaseSkeletalMesh, ExistingSection, ExistingSectionIndex, ExistingInfo, ExistingMaterialIndex);

			for(int32 ImportedSectionIndex = 0; ImportedSectionIndex < ImportedSections.Num(); ++ImportedSectionIndex)
			{
				FSkelMeshSection& ImportedSection = ImportedSections[ImportedSectionIndex];
				int32 ImportedMaterialIndex = 0;
				FName ImportedImportedMaterialSlotName = GetImportMaterialSlotName(InSkeletalMesh, ImportedSection, ImportedSectionIndex, ImportedInfo, ImportedMaterialIndex);
				if (ExistingImportedMaterialSlotName != NAME_None)
				{
					if (ImportedImportedMaterialSlotName == ExistingImportedMaterialSlotName)
					{
						//Set the value and exit
						ImportedSection.bCastShadow = ExistingSection.bCastShadow;
						ImportedSection.bVisibleInRayTracing = ExistingSection.bVisibleInRayTracing;
						ImportedSection.bRecomputeTangent = ExistingSection.bRecomputeTangent;
						ImportedSection.RecomputeTangentsVertexMaskChannel = ExistingSection.RecomputeTangentsVertexMaskChannel;
						break;
					}
				}
				else if(InSkeletalMesh->GetMaterials()[ImportedMaterialIndex] == BaseSkeletalMesh->GetMaterials()[ExistingMaterialIndex]) //Use material slot compare to match in case the name is none
				{
					//Set the value and exit
					ImportedSection.bCastShadow = ExistingSection.bCastShadow;
					ImportedSection.bVisibleInRayTracing = ExistingSection.bVisibleInRayTracing;
					ImportedSection.bRecomputeTangent = ExistingSection.bRecomputeTangent;
					ImportedSection.RecomputeTangentsVertexMaskChannel = ExistingSection.RecomputeTangentsVertexMaskChannel;
					break;
				}
			}
		}
	}

	InsertNewLODToBaseSkeletalMesh(InSkeletalMesh, BaseSkeletalMesh, DesiredLOD, TemplateImportData);

	return true;
}


void UnFbx::FFbxImporter::ImportMorphTargetsInternal( TArray<FbxNode*>& SkelMeshNodeArray, USkeletalMesh* BaseSkelMesh, int32 LODIndex, FSkeletalMeshImportData &BaseImportData, const bool bMapMorphTargetToTimeZero)
{
	FbxString ShapeNodeName;
	TMap<FString, TArray<FbxShape*>> ShapeNameToShapeArray;
	FScopedSlowTask ImportMorphTargetSlowTask(0, NSLOCTEXT("FbxImporter", "BeginGeneratingMorphModelsTask", "Generating Morph Models"), true);

	// For each morph in FBX geometries, we create one morph target for the Unreal skeletal mesh
	for (int32 NodeIndex = 0; NodeIndex < SkelMeshNodeArray.Num(); NodeIndex++)
	{
		FbxGeometry* Geometry = (FbxGeometry*)SkelMeshNodeArray[NodeIndex]->GetNodeAttribute();
		if (Geometry)
		{
			const int32 BlendShapeDeformerCount = Geometry->GetDeformerCount(FbxDeformer::eBlendShape);

			/************************************************************************/
			/* collect all the shapes                                               */
			/************************************************************************/
			for(int32 BlendShapeIndex = 0; BlendShapeIndex<BlendShapeDeformerCount; ++BlendShapeIndex)
			{
				FbxBlendShape* BlendShape = (FbxBlendShape*)Geometry->GetDeformer(BlendShapeIndex, FbxDeformer::eBlendShape);
				const int32 BlendShapeChannelCount = BlendShape->GetBlendShapeChannelCount();

				FString BlendShapeName = MakeName(BlendShape->GetName());

				// see below where this is used for explanation...
				const bool bMightBeBadMAXFile = (BlendShapeName == FString("Morpher"));

				for(int32 ChannelIndex = 0; ChannelIndex<BlendShapeChannelCount; ++ChannelIndex)
				{
					FbxBlendShapeChannel* Channel = BlendShape->GetBlendShapeChannel(ChannelIndex);
					if(Channel)
					{
						//Find which shape should we use according to the weight.
						const int32 CurrentChannelShapeCount = Channel->GetTargetShapeCount();
						
						FString ChannelName = MakeName(Channel->GetName());

						// Maya adds the name of the blendshape and an underscore to the front of the channel name, so remove it
						if(ChannelName.StartsWith(BlendShapeName))
						{
							ChannelName.RightInline(ChannelName.Len() - (BlendShapeName.Len()+1), EAllowShrinking::No);
						}

						for(int32 ShapeIndex = 0; ShapeIndex<CurrentChannelShapeCount; ++ShapeIndex)
						{
							FbxShape* Shape = Channel->GetTargetShape(ShapeIndex);

							FString ShapeName;
							if( CurrentChannelShapeCount > 1 )
							{
								ShapeName = MakeName(Shape->GetName());
							}
							else
							{
								if (bMightBeBadMAXFile)
								{
									ShapeName = MakeName(Shape->GetName());
								}
								else
								{
									// Maya concatenates the number of the shape to the end of its name, so instead use the name of the channel
									ShapeName = ChannelName;
								}
							}

							TArray<FbxShape*> & ShapeArray = ShapeNameToShapeArray.FindOrAdd(ShapeName);
							if (ShapeArray.Num() == 0)
							{
								ShapeArray.AddZeroed(SkelMeshNodeArray.Num());
							}

							ShapeArray[NodeIndex] = Shape;
						}
					}
				}
			}
		}
	} // for NodeIndex

	for (auto Iter = ShapeNameToShapeArray.CreateIterator(); Iter && !bImportOperationCanceled; ++Iter)
	{
		FString ShapeName = Iter.Key();
		TArray< FbxShape* >& ShapeArray = Iter.Value();
		FSkeletalMeshImportData ShapeImportData;
		BaseImportData.CopyDataNeedByMorphTargetImport(ShapeImportData);

		//Store the rebuild morph data into the base import data, this will allow us to rebuild the morph data in case the mesh is rebuild and the vertex count change because of options (max bone per section, normals compute...)
		int32 MorphTargetIndex;
		if (BaseImportData.MorphTargetNames.Find(ShapeName, MorphTargetIndex))
		{
			BaseImportData.MorphTargetNames.RemoveAt(MorphTargetIndex);
			BaseImportData.MorphTargetModifiedPoints.RemoveAt(MorphTargetIndex);
			BaseImportData.MorphTargets.RemoveAt(MorphTargetIndex);
		}
		BaseImportData.MorphTargetNames.Add(ShapeName);
		TSet<uint32>& ModifiedPoints = BaseImportData.MorphTargetModifiedPoints.AddDefaulted_GetRef();
		GatherPointsForMorphTarget(&ShapeImportData, SkelMeshNodeArray, &ShapeArray, ModifiedPoints, bMapMorphTargetToTimeZero);
		//We do not need this data anymore empty it so we reduce the size of what we save into memory
		ShapeImportData.PointToRawMap.Empty();
		BaseImportData.MorphTargets.Add(ShapeImportData);
		check(BaseImportData.MorphTargetNames.Num() == BaseImportData.MorphTargets.Num() && BaseImportData.MorphTargetNames.Num() == BaseImportData.MorphTargetModifiedPoints.Num());

		// Ensure that we have curve metadata for this morph target (either skeleton or mesh)
		FName CurveName = *ShapeName;
		if(ImportOptions->bAddCurveMetadataToSkeleton)
		{
			if(USkeleton* Skeleton = BaseSkelMesh->GetSkeleton())
			{
				Skeleton->AddCurveMetaData(CurveName);

				// Ensure we have a morph flag set
				FCurveMetaData* CurveMetaData = Skeleton->GetCurveMetaData(CurveName);
				check(CurveMetaData);
				CurveMetaData->Type.bMorphtarget = true;
			}
		}
		else
		{
			UAnimCurveMetaData* AnimCurveMetaData = BaseSkelMesh->GetAssetUserData<UAnimCurveMetaData>();
			if(AnimCurveMetaData == nullptr)
			{
				AnimCurveMetaData = NewObject<UAnimCurveMetaData>(BaseSkelMesh, NAME_None, RF_Transactional);
				BaseSkelMesh->AddAssetUserData(AnimCurveMetaData);
			}

			AnimCurveMetaData->AddCurveMetaData(CurveName);

			// Ensure we have a morph flag set
			FCurveMetaData* CurveMetaData = AnimCurveMetaData->GetCurveMetaData(CurveName);
			check(CurveMetaData);
			CurveMetaData->Type.bMorphtarget = true;
		}

		if (ImportOptions->bIsImportCancelable && ImportMorphTargetSlowTask.ShouldCancel())
		{
			bImportOperationCanceled = true;
			break;
		}
	}

	if (BaseSkelMesh->GetImportedModel() && BaseSkelMesh->GetImportedModel()->LODModels.IsValidIndex(LODIndex))
	{
		//If we can build the skeletal mesh there is no need to build the morph target now, all the necessary build morph target data was copied before.
		if (!BaseSkelMesh->HasMeshDescription(LODIndex))
		{
			//Build MorphTargets
			FLODUtilities::BuildMorphTargets(
				BaseSkelMesh,
				BaseImportData,
				LODIndex,
				ImportOptions->ShouldImportNormals(),
				ImportOptions->ShouldImportTangents(),
				(ImportOptions->NormalGenerationMethod == EFBXNormalGenerationMethod::MikkTSpace),
				ImportOptions->OverlappingThresholds
				);
		}
	}
}

// Import Morph target
void UnFbx::FFbxImporter::ImportFbxMorphTarget(TArray<FbxNode*> &SkelMeshNodeArray, USkeletalMesh* BaseSkelMesh, int32 LODIndex, FSkeletalMeshImportData &BaseSkeletalMeshImportData, const bool bMapMorphTargetToTimeZero)
{
	if (!CanImportClass(UMorphTarget::StaticClass()))
	{
		return;
	}

	//Stack the PostEditChange call, it will call post edit change when it will go out of scope
	FScopedSkeletalMeshPostEditChange ScopedPostEditChange(BaseSkelMesh);
	FFbxScopedOperation ScopedImportOperation(this);

	bool bHasMorph = false;
	int32 NodeIndex;
	// check if there are morph in this geometry
	for (NodeIndex = 0; NodeIndex < SkelMeshNodeArray.Num(); NodeIndex++)
	{
		FbxGeometry* Geometry = (FbxGeometry*)SkelMeshNodeArray[NodeIndex]->GetNodeAttribute();
		if (Geometry)
		{
			bHasMorph = Geometry->GetDeformerCount(FbxDeformer::eBlendShape) > 0;
			if (bHasMorph)
			{
				break;
			}
		}
	}
	
	if (bHasMorph)
	{
		ImportMorphTargetsInternal( SkelMeshNodeArray, BaseSkelMesh, LODIndex, BaseSkeletalMeshImportData, bMapMorphTargetToTimeZero);
		//Save the rawMesh
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		BaseSkelMesh->SaveLODImportedData(LODIndex, BaseSkeletalMeshImportData);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

void UnFbx::FFbxImporter::AddTokenizedErrorMessage(TSharedRef<FTokenizedMessage> Error, FName FbxErrorName)
{
	// check to see if Logger exists, this way, we guarantee only prints to FBX import
	// when we meant to print
	if (Logger)
	{
		Logger->TokenizedErrorMessages.Add(Error);

		if(FbxErrorName != NAME_None)
		{
			Error->AddToken(FFbxErrorToken::Create(FbxErrorName));
		}
	}
	else
	{
		// if not found, use normal log
		UE_LOGF(LogFbx, Warning, "%ls", *(Error->ToText().ToString()));
	}
}

void UnFbx::FFbxImporter::ClearTokenizedErrorMessages()
{
	if(Logger)
	{
		Logger->TokenizedErrorMessages.Empty();
	}
}

void UnFbx::FFbxImporter::FlushToTokenizedErrorMessage(EMessageSeverity::Type Severity)
{
	if (!ErrorMessage.IsEmpty())
	{
		AddTokenizedErrorMessage(FTokenizedMessage::Create(Severity, FText::Format(FText::FromString("{0}"), FText::FromString(ErrorMessage))), NAME_None);
	}
}

void UnFbx::FFbxImporter::SetLogger(class FFbxLogger * InLogger)
{
	// this should be only called by top level functions
	// if you set it you can't set it again. Otherwise, you'll lose all log information
	check(Logger == NULL);
	Logger = InLogger;
}

// just in case if DeleteScene/CleanUp is getting called too late
void UnFbx::FFbxImporter::ClearLogger()
{
	Logger = NULL;
}

FFbxLogger::FFbxLogger()
{
}

FFbxLogger::~FFbxLogger()
{
	bool ShowLogMessage = !ShowLogMessageOnlyIfError;
	if (ShowLogMessageOnlyIfError)
	{
		for (TSharedRef<FTokenizedMessage> TokenMessage : TokenizedErrorMessages)
		{
			if (TokenMessage->GetSeverity() == EMessageSeverity::Error)
			{
				ShowLogMessage = true;
				break;
			}
		}
	}

	//Always clear the old message after an import or re-import
	const TCHAR* LogTitle = TEXT("FBXImport");
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	TSharedPtr<class IMessageLogListing> LogListing = MessageLogModule.GetLogListing(LogTitle);
	LogListing->SetLabel(FText::FromString("FBX Import"));
	LogListing->ClearMessages();

	if(TokenizedErrorMessages.Num() > 0)
	{
		LogListing->AddMessages(TokenizedErrorMessages);
		if (ShowLogMessage)
		{
			MessageLogModule.OpenMessageLog(LogTitle);
		}
	}
}

UnFbx::FFbxScopedOperation::FFbxScopedOperation(FFbxImporter* FbxImporter) : Importer(FbxImporter)
{
	check(Importer);

	if (Importer->ImportOperationStack++ == 0)
	{
		Importer->bImportOperationCanceled = false;
	}
}

UnFbx::FFbxScopedOperation::~FFbxScopedOperation()
{
	check(Importer->ImportOperationStack > 0);
	Importer->ImportOperationStack--;
}
#undef LOCTEXT_NAMESPACE
