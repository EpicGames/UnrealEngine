// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/FbxSkeletalMeshImportData.h"
#include "Engine/SkeletalMesh.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/Package.h"
#include "UObject/EditorObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FbxSkeletalMeshImportData)

UFbxSkeletalMeshImportData::UFbxSkeletalMeshImportData(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bImportMeshesInBoneHierarchy(true)
{
	bTransformVertexToAbsolute = true;
	bBakePivotInVertex = false;
	VertexColorImportOption = EVertexColorImportOption::Replace;
	LastImportContentType = EFBXImportContentType::FBXICT_All;
	bKeepSectionsSeparate = false;
}

void UFbxSkeletalMeshImportData::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);

	//bComputeWeightedNormals default is true so old asset should have set to false.
	if (Ar.CustomVer(FEditorObjectVersion::GUID) < FEditorObjectVersion::ComputeWeightedNormals)
	{
		//Old asset are not using weighted normals
		bComputeWeightedNormals = false;
	}
}

UFbxSkeletalMeshImportData* UFbxSkeletalMeshImportData::GetImportDataForSkeletalMesh(USkeletalMesh* SkeletalMesh, UFbxSkeletalMeshImportData* TemplateForCreation)
{
	check(SkeletalMesh);
	
	UFbxSkeletalMeshImportData* ImportData = Cast<UFbxSkeletalMeshImportData>(SkeletalMesh->GetAssetImportData());
	if ( !ImportData )
	{
		ImportData = NewObject<UFbxSkeletalMeshImportData>(SkeletalMesh, NAME_None, RF_NoFlags, TemplateForCreation);

		// Try to preserve the source file data if possible
		if ( SkeletalMesh->GetAssetImportData() != NULL )
		{
			ImportData->SourceData = SkeletalMesh->GetAssetImportData()->SourceData;
		}

		SkeletalMesh->SetAssetImportData(ImportData);
	}

	return ImportData;
}

bool UFbxSkeletalMeshImportData::CanEditChange(const FProperty* InProperty) const
{
	bool bMutable = Super::CanEditChange(InProperty);
	UObject* Outer = GetOuter();
	if(Outer && bMutable)
	{
		// Let the FbxImportUi object handle the editability of our properties
		bMutable = Outer->CanEditChange(InProperty);
	}
	return bMutable;
}

bool UFbxSkeletalMeshImportData::GetImportContentFilename(FString& OutFilename, FString& OutFilenameLabel) const
{
	if (SourceData.SourceFiles.Num() < 1)
	{
		OutFilename = FString();
		OutFilenameLabel = FString();
		return false;
	}
	int32 SourceIndex = ImportContentType == EFBXImportContentType::FBXICT_All ? 0 : ImportContentType == EFBXImportContentType::FBXICT_Geometry ? 1 : 2;
	if (SourceData.SourceFiles.Num() > SourceIndex)
	{
		OutFilename = ResolveImportFilename(SourceData.SourceFiles[SourceIndex].RelativeFilename);
		OutFilenameLabel = SourceData.SourceFiles[SourceIndex].DisplayLabelName;
		return true;
	}
	OutFilename = ResolveImportFilename(SourceData.SourceFiles[0].RelativeFilename);
	OutFilenameLabel = SourceData.SourceFiles[0].DisplayLabelName;
	return true;
}

void UFbxSkeletalMeshImportData::AppendAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::AppendAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UFbxSkeletalMeshImportData::AppendAssetRegistryTags(FAssetRegistryTagsContext Context)
{
	auto EFBXImportContentTypeToString = [](const EFBXImportContentType value)-> FString
	{
		switch (value)
		{
		case EFBXImportContentType::FBXICT_All:
			return TEXT("FBXICT_All");
		case EFBXImportContentType::FBXICT_Geometry:
			return TEXT("FBXICT_Geometry");
		case EFBXImportContentType::FBXICT_SkinningWeights:
			return TEXT("FBXICT_SkinningWeights");
		case EFBXImportContentType::FBXICT_MAX:
			return TEXT("FBXICT_MAX");
		}
		return TEXT("FBXICT_All");
	};

	FString EnumString = EFBXImportContentTypeToString(LastImportContentType);
	Context.AddTag(FAssetRegistryTag("LastImportContentType", EnumString, FAssetRegistryTag::TT_Hidden));

	Super::AppendAssetRegistryTags(Context);
}
