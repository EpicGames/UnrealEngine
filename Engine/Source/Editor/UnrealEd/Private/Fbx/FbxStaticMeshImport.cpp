// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Static mesh creation from FBX data.
	Largely based on StaticMeshEdit.cpp
=============================================================================*/

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "UObject/Object.h"
#include "UObject/GarbageCollection.h"
#include "UObject/MetaData.h"
#include "UObject/Package.h"
#include "Misc/PackageName.h"
#include "MaterialDomain.h"
#include "Materials/MaterialInterface.h"
#include "Materials/Material.h"
#include "Factories/Factory.h"
#include "Factories/FbxSceneImportFactory.h"
#include "Factories/FbxStaticMeshImportData.h"
#include "Engine/StaticMesh.h"
#include "Engine/Polys.h"
#include "Engine/StaticMeshSocket.h"
#include "Editor.h"
#include "MeshBudgetProjectSettings.h"
#include "Modules/ModuleManager.h"
#include "Subsystems/AssetEditorSubsystem.h"

#include "StaticMeshResources.h"
#include "ObjectTools.h"
#include "PackageTools.h"
#include "Logging/TokenizedMessage.h"
#include "FbxImporter.h"
#include "GeomFitUtils.h"
#include "ImportUtils/StaticMeshImportUtils.h"
#include "InterchangeProjectSettings.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Misc/FbxErrors.h"
#include "Misc/ScopedSlowTask.h"
#include "PhysicsEngine/BodySetup.h"
#include "StaticMeshAttributes.h"
#include "IMeshBuilderModule.h"
#include "Settings/EditorExperimentalSettings.h"

#define LOCTEXT_NAMESPACE "FbxStaticMeshImport"

static const int32 LARGE_MESH_MATERIAL_INDEX_THRESHOLD = 64;

using namespace UnFbx;


struct FRestoreReimportData
{
	UObject* EditorObject = nullptr;
	UObject* DupObject = nullptr;
	FString OriginalName;
	FString OriginalPackageName;
	TMap<FName, FString> ObjectMetaData;

	FRestoreReimportData()
	{
		//Unsupported default constructor
		check(false);
	}

	FRestoreReimportData(UStaticMesh* Mesh)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FRestoreReimportData);
		if (!ensure(Mesh))
		{
			return;
		}
		EditorObject = Mesh;
		OriginalName = Mesh->GetName();
		TMap<FName, FString>* ObjectMetaDataPtr = Mesh->GetPackage()->GetMetaData().GetMapForObject(Mesh);
		if (ObjectMetaDataPtr && ObjectMetaDataPtr->Num() > 0)
		{
			ObjectMetaData = *ObjectMetaDataPtr;
		}
		OriginalPackageName = Mesh->GetOutermost()->GetName();
		DupObject = StaticDuplicateObject(Mesh, GetTransientPackage());
		DupObject->AddToRoot();
	}

	void RestoreMesh(UnFbx::FFbxImporter* FbxImporter)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RestoreMesh);
		if (!ensure(EditorObject && DupObject))
		{
			return;
		}

		if(ensure(FbxImporter))
		{
			FbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("RestoreMesh_FailReimportMesh_RestoreMessage", "Fail to reimport mesh {0}. Restoring the original mesh"), FText::FromString(OriginalPackageName))), FFbxErrors::Generic_ImportingNewObjectFailed);
		}

		UPackage* Package = nullptr;
		//We have to close any staticmesh editor using this asset
		int32 EditorCloseCount = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(EditorObject);
		//Restore the duplicate asset
		OriginalPackageName = UPackageTools::SanitizePackageName(OriginalPackageName);
		Package = CreatePackage( *OriginalPackageName);
		UObject* ExistingObject = StaticFindObject(UStaticMesh::StaticClass(), Package, *OriginalName, EFindObjectFlags::ExactClass);
		if (ExistingObject)
		{
			//Some reimport path did not trash the package in case there is a fail (i.e. scene reimport)
			//Rename the original mesh and trash it
			ExistingObject->Rename(nullptr, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
			ExistingObject->MarkAsGarbage();
		}
		
		//Rename the dup object
		DupObject->Rename(*OriginalName, Package, REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
		if (ObjectMetaData.Num() > 0)
		{
			//Package was recreate so restore the metadata
			FMetaData& PackageMetaData = DupObject->GetPackage()->GetMetaData();
			PackageMetaData.SetObjectValues(DupObject, MoveTemp(ObjectMetaData));
		}
		//Since all loaded package are add to root, we have to remove the dup from the root
		DupObject->RemoveFromRoot();
		if (EditorCloseCount > 0)
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(DupObject);
		}
	}

	void CleanupDuplicateMesh()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CleanupDuplicateMesh);
		if(DupObject)
		{
			DupObject->RemoveFromRoot();
			DupObject->MarkAsGarbage();
			DupObject = nullptr;
		}
	}
};

static FbxString GetNodeNameWithoutNamespace( const FbxString& NodeName )
{
	// Namespaces are marked with colons so find the last colon which will mark the start of the actual name
	int32 LastNamespceIndex = NodeName.ReverseFind(':');

	if( LastNamespceIndex == -1 )
	{
		// No namespace
		return NodeName;
	}
	else
	{
		// chop off the namespace
		return NodeName.Right( NodeName.GetLen() - (LastNamespceIndex + 1) );
	}
}

void CreateTokenizedErrorForDegeneratedPart(UnFbx::FFbxImporter* FbxImporter, const FString MeshName, const FString NodeName)
{
	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("MeshName"), FText::FromString(MeshName));
	Arguments.Add(TEXT("PartName"), FText::FromString(NodeName));
	FText ErrorMsg = LOCTEXT("MeshHasNoRenderableTriangles", "Mesh name: [{MeshName}] part name: [{PartName}]  could not be created because all of its polygons are degenerate.");
	FbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(ErrorMsg, Arguments)), FFbxErrors::StaticMesh_AllTrianglesDegenerate);
}

//-------------------------------------------------------------------------
//
//-------------------------------------------------------------------------
UStaticMesh* UnFbx::FFbxImporter::ImportStaticMesh(UObject* InParent, FbxNode* Node, const FName& Name, EObjectFlags Flags, UFbxStaticMeshImportData* ImportData, UStaticMesh* InStaticMesh, int LODIndex, const FExistingStaticMeshData* ExistMeshDataPtr)
{
	TArray<FbxNode*> MeshNodeArray;
	
	if ( !Node->GetMesh())
	{
		return NULL;
	}
	
	MeshNodeArray.Add(Node);
	return ImportStaticMeshAsSingle(InParent, MeshNodeArray, Name, Flags, ImportData, InStaticMesh, LODIndex, ExistMeshDataPtr);
}

// Wraps some common code useful for multiple fbx import code path
struct FFBXUVs
{
	// constructor
	FFBXUVs(UnFbx::FFbxImporter* FbxImporter, FbxMesh* Mesh)
		: UniqueUVCount(0)
	{
		check(Mesh);

		//
		//	store the UVs in arrays for fast access in the later looping of triangles 
		//
		// mapping from UVSets to Fbx LayerElementUV
		// Fbx UVSets may be duplicated, remove the duplicated UVSets in the mapping 
		int32 LayerCount = Mesh->GetLayerCount();
		if (LayerCount > 0)
		{
			int32 UVLayerIndex;
			for (UVLayerIndex = 0; UVLayerIndex<LayerCount; UVLayerIndex++)
			{
				FbxLayer* lLayer = Mesh->GetLayer(UVLayerIndex);
				int UVSetCount = lLayer->GetUVSetCount();
				if(UVSetCount)
				{
					FbxArray<FbxLayerElementUV const*> EleUVs = lLayer->GetUVSets();
					for (int UVIndex = 0; UVIndex<UVSetCount; UVIndex++)
					{
						FbxLayerElementUV const* ElementUV = EleUVs[UVIndex];
						if (ElementUV)
						{
							const char* UVSetName = ElementUV->GetName();
							FString LocalUVSetName = UTF8_TO_TCHAR(UVSetName);
							if (LocalUVSetName.IsEmpty())
							{
								LocalUVSetName = TEXT("UVmap_") + FString::FromInt(UVLayerIndex);
							}

							UVSets.AddUnique(LocalUVSetName);
						}
					}
				}
			}
		}


		// If the the UV sets are named using the following format (UVChannel_X; where X ranges from 1 to 4)
		// we will re-order them based on these names, provided there are enough UV channels in total to do so.
		// Any UV sets that do not follow this naming convention will be slotted into available spaces.
		//
		// Examples:
		// {UVChannel_3, UVChannel_1, RandomName} will be reordered to {UVChannel_1, RandomName, UVChannel_3}.
		// {UVChannel_3, UVChannel_1} will be reordered to {UVChannel?1, UVChannel_3}
		//     - note that UVChannel_3 cannot be placed in the appropriate slot as there are not enough sets.
		if(UVSets.Num())
		{
			const int32 MaxPossibleUVIndex = FMath::Min(UVSets.Num(), 4);
			for(int32 ChannelNumIdx = 0; ChannelNumIdx < MaxPossibleUVIndex; ChannelNumIdx++)
			{
				FString ChannelName = FString::Printf( TEXT("UVChannel_%d"), ChannelNumIdx+1 );
				int32 SetIdx = UVSets.Find( ChannelName );

				// If the specially formatted UVSet name appears in the list and it is in the wrong spot,
				// we will swap it into the correct spot, provided there are enough UV channels
				if( SetIdx != INDEX_NONE && SetIdx != ChannelNumIdx )
				{
					//Swap the entry into the appropriate spot.
					UVSets.Swap( SetIdx, ChannelNumIdx );
				}
			}
		}

	}

	void Phase2(UnFbx::FFbxImporter* FbxImporter, FbxMesh* Mesh)
	{
		//
		//	store the UVs in arrays for fast access in the later looping of triangles 
		//
		UniqueUVCount = UVSets.Num();
		if (UniqueUVCount > 0)
		{
			LayerElementUV.AddZeroed(UniqueUVCount);
			UVReferenceMode.AddZeroed(UniqueUVCount);
			UVMappingMode.AddZeroed(UniqueUVCount);
		}
		for (int32 UVIndex = 0; UVIndex < UniqueUVCount; UVIndex++)
		{
			LayerElementUV[UVIndex] = NULL;
			for (int32 UVLayerIndex = 0, LayerCount = Mesh->GetLayerCount(); UVLayerIndex < LayerCount; UVLayerIndex++)
			{
				FbxLayer* lLayer = Mesh->GetLayer(UVLayerIndex);
				int UVSetCount = lLayer->GetUVSetCount();
				if(UVSetCount)
				{
					FbxArray<FbxLayerElementUV const*> EleUVs = lLayer->GetUVSets();
					for (int32 FbxUVIndex = 0; FbxUVIndex<UVSetCount; FbxUVIndex++)
					{
						FbxLayerElementUV const* ElementUV = EleUVs[FbxUVIndex];
						if (ElementUV)
						{
							const char* UVSetName = ElementUV->GetName();
							FString LocalUVSetName = UTF8_TO_TCHAR(UVSetName);
							if (LocalUVSetName.IsEmpty())
							{
								LocalUVSetName = TEXT("UVmap_") + FString::FromInt(UVLayerIndex);
							}
							if (LocalUVSetName == UVSets[UVIndex])
							{
								LayerElementUV[UVIndex] = ElementUV;
								UVReferenceMode[UVIndex] = ElementUV->GetReferenceMode();
								UVMappingMode[UVIndex] = ElementUV->GetMappingMode();
								break;
							}
						}
					}
				}
			}
		}

		if (UniqueUVCount > MAX_MESH_TEXTURE_COORDS_MD)
		{
			FbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("Error_TooMuchUVChannel", "Reached the maximum number of UV Channels for a Static Mesh({0}) - discarding {1} UV Channels"), FText::AsNumber(MAX_MESH_TEXTURE_COORDS_MD), FText::AsNumber(UniqueUVCount-MAX_MESH_TEXTURE_COORDS_MD))), FFbxErrors::Generic_Mesh_TooMuchUVChannels);
		}

		UniqueUVCount = FMath::Min<int32>(UniqueUVCount, MAX_MESH_TEXTURE_COORDS_MD);
	}

	int32 FindLightUVIndex() const
	{
		// See if any of our UV set entry names match LightMapUV.
		for(int32 UVSetIdx = 0; UVSetIdx < UVSets.Num(); UVSetIdx++)
		{
			if( UVSets[UVSetIdx] == TEXT("LightMapUV"))
			{
				return UVSetIdx;
			}
		}

		// not found
		return INDEX_NONE;
	}
	
	// @param FaceCornerIndex usually TriangleIndex * 3 + CornerIndex but more complicated for mixed n-gons
	int32 ComputeUVIndex(int32 UVLayerIndex, int32 lControlPointIndex, int32 FaceCornerIndex) const
	{
		int32 UVMapIndex = (UVMappingMode[UVLayerIndex] == FbxLayerElement::eByControlPoint) ? lControlPointIndex : FaceCornerIndex;
						
		int32 Ret;

		if(UVReferenceMode[UVLayerIndex] == FbxLayerElement::eDirect)
		{
			Ret = UVMapIndex;
		}
		else
		{
			FbxLayerElementArrayTemplate<int>& Array = LayerElementUV[UVLayerIndex]->GetIndexArray();
			Ret = Array.GetAt(UVMapIndex);
		}

		return Ret;
	}

	// todo: is that needed? could the dtor do it?
	void Cleanup()
	{
		//
		// clean up.  This needs to happen before the mesh is destroyed
		//
		LayerElementUV.Empty();
		UVReferenceMode.Empty();
		UVMappingMode.Empty();
	}

	TArray<FString> UVSets;
	TArray<FbxLayerElementUV const*> LayerElementUV;
	TArray<FbxLayerElement::EReferenceMode> UVReferenceMode;
	TArray<FbxLayerElement::EMappingMode> UVMappingMode;
	int32 UniqueUVCount;
};


float UnFbx::FFbxImporter::GetPointComparisonThreshold() const
{
	return (ImportOptions->bRemoveDegenerates && !ImportOptions->bBuildNanite) ? THRESH_POINTS_ARE_SAME : 0.0f;
}

float UnFbx::FFbxImporter::GetTriangleAreaThreshold() const
{
	return (ImportOptions->bRemoveDegenerates && !ImportOptions->bBuildNanite) ? SMALL_NUMBER : 0.0f;
}

bool UnFbx::FFbxImporter::BuildStaticMeshFromGeometry(FbxNode* Node, UStaticMesh* StaticMesh, TArray<FFbxMaterial>& MeshMaterials, int32 LODIndex,
	EVertexColorImportOption::Type VertexColorImportOption, const TMap<FVector3f, FColor>& ExistingVertexColorData, const FColor& VertexOverrideColor)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FFbxImporter::BuildStaticMeshFromGeometry);
	FFbxScopedOperation ScopedImportOperation(this);

	//Only cancel the operation if we are creating a new asset, as we don't support existing asset restoration yet.
	check(StaticMesh->IsSourceModelValid(LODIndex));
	FbxMesh* Mesh = Node->GetMesh();
	FString FbxNodeName = UTF8_TO_TCHAR(Node->GetName());
	
	FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription(LODIndex);
	//The mesh description should have been created before calling BuildStaticMeshFromGeometry
	check(MeshDescription);
	FStaticMeshAttributes Attributes(*MeshDescription);

	//Get the base layer of the mesh
	FbxLayer* BaseLayer = Mesh->GetLayer(0);
	if (BaseLayer == NULL)
	{
		AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("Error_NoGeometryInMesh", "There is no geometry information in mesh '{0}'"), FText::FromString(Mesh->GetName()))), FFbxErrors::Generic_Mesh_NoGeometry);
		return false;
	}

	FFBXUVs FBXUVs(this, Mesh);
	int32 FBXNamedLightMapCoordinateIndex = FBXUVs.FindLightUVIndex();
	if (FBXNamedLightMapCoordinateIndex != INDEX_NONE)
	{
		StaticMesh->SetLightMapCoordinateIndex(FBXNamedLightMapCoordinateIndex);
	}
	
	//Get the default hard/smooth edges from the project settings
	const UInterchangeProjectSettings* InterchangeProjectSettings = GetDefault<UInterchangeProjectSettings>();
	const bool bStaticMeshUseSmoothEdgesIfSmoothingInformationIsMissing = InterchangeProjectSettings->bStaticMeshUseSmoothEdgesIfSmoothingInformationIsMissing;

	//
	// create materials
	//
	int32 MaterialCount = 0;
	int32 MaterialIndexOffset = 0;
	TArray<UMaterialInterface*> FbxMeshMaterials;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CreateMaterials);
		
		
		const bool bForSkeletalMesh = false;
		
		FindOrImportMaterialsFromNode(Node, FbxMeshMaterials, FBXUVs.UVSets, bForSkeletalMesh);
		if (!ImportOptions->bImportMaterials && ImportOptions->bImportTextures)
		{
			//If we are not importing any new material, we might still want to import new textures.
			ImportTexturesFromNode(Node);
		}

		MaterialCount = Node->GetMaterialCount();
		check(FbxMeshMaterials.Num() == MaterialCount);
	
		// Used later to offset the material indices on the raw triangle data
		MaterialIndexOffset = MeshMaterials.Num();

		for (int32 MaterialIndex=0; MaterialIndex<MaterialCount; MaterialIndex++)
		{
			FFbxMaterial* NewMaterial = new(MeshMaterials) FFbxMaterial;
			FbxSurfaceMaterial *FbxMaterial = Node->GetMaterial(MaterialIndex);
			NewMaterial->FbxMaterial = FbxMaterial;
			
			if (FbxMeshMaterials[MaterialIndex])
			{
				NewMaterial->Material = FbxMeshMaterials[MaterialIndex];
			}
			else
			{
				NewMaterial->Material = UMaterial::GetDefaultMaterial(MD_Surface);
			}
		}

		if ( MaterialCount == 0 )
		{
			UMaterial* DefaultMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
			check(DefaultMaterial);
			FFbxMaterial* NewMaterial = new(MeshMaterials) FFbxMaterial;
			NewMaterial->Material = DefaultMaterial;
			NewMaterial->FbxMaterial = NULL;
			MaterialCount = 1;
		}
	}

	//
	// Convert data format to unreal-compatible
	//

	// Must do this before triangulating the mesh due to an FBX bug in TriangulateMeshAdvance
	int32 LayerSmoothingCount = Mesh->GetLayerCount(FbxLayerElement::eSmoothing);
	for(int32 i = 0; i < LayerSmoothingCount; i++)
	{
		FbxLayerElementSmoothing const* SmoothingInfo = Mesh->GetLayer(0)->GetSmoothing();
		if (SmoothingInfo && SmoothingInfo->GetMappingMode() != FbxLayerElement::eByPolygon)
		{
			GeometryConverter->ComputePolygonSmoothingFromEdgeSmoothing (Mesh, i);
		}
	}

	if (!Mesh->IsTriangleMesh())
	{
		if(!GIsAutomationTesting)
		UE_LOGF(LogFbx, Display, "Triangulating static mesh %ls", *FbxNodeName);

		const bool bReplace = true;
		FbxNodeAttribute* ConvertedNode = GeometryConverter->Triangulate(Mesh, bReplace);

		if( ConvertedNode != NULL && ConvertedNode->GetAttributeType() == FbxNodeAttribute::eMesh )
		{
			Mesh = (fbxsdk::FbxMesh*)ConvertedNode;
		}
		else
		{
			AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("Error_FailedToTriangulateWithOmission", "Unable to triangulate mesh '{0}': it will be omitted."), FText::FromString(FbxNodeName))), FFbxErrors::Generic_Mesh_TriangulationFailed);
			return false;
		}
	}
	
	// renew the base layer
	BaseLayer = Mesh->GetLayer(0);

	//
	//	get the "material index" layer.  Do this AFTER the triangulation step as that may reorder material indices
	//
	FbxLayerElementMaterial* LayerElementMaterial = BaseLayer->GetMaterials();
	FbxLayerElement::EMappingMode MaterialMappingMode = LayerElementMaterial ? 
		LayerElementMaterial->GetMappingMode() : FbxLayerElement::eByPolygon;

	//	todo second phase UV, ok to put in first phase?
	FBXUVs.Phase2(this, Mesh);

	//
	// get the smoothing group layer
	//
	bool bSmoothingAvailable = false;

	FbxLayerElementSmoothing* SmoothingInfo = BaseLayer->GetSmoothing();
	FbxLayerElement::EReferenceMode SmoothingReferenceMode(FbxLayerElement::eDirect);
	FbxLayerElement::EMappingMode SmoothingMappingMode(FbxLayerElement::eByEdge);
	if (SmoothingInfo)
	{
		if (SmoothingInfo->GetMappingMode() == FbxLayerElement::eByPolygon)
		{
			//Convert the base layer to edge smoothing
			GeometryConverter->ComputeEdgeSmoothingFromPolygonSmoothing(Mesh, 0);
			BaseLayer = Mesh->GetLayer(0);
			SmoothingInfo = BaseLayer->GetSmoothing();
		}

		if (SmoothingInfo->GetMappingMode() == FbxLayerElement::eByEdge)
		{
			bSmoothingAvailable = true;
		}

		SmoothingReferenceMode = SmoothingInfo->GetReferenceMode();
		SmoothingMappingMode = SmoothingInfo->GetMappingMode();
	}

	//
	// get the first vertex color layer
	//
	FbxLayerElementVertexColor* LayerElementVertexColor = BaseLayer->GetVertexColors();
	FbxLayerElement::EReferenceMode VertexColorReferenceMode(FbxLayerElement::eDirect);
	FbxLayerElement::EMappingMode VertexColorMappingMode(FbxLayerElement::eByControlPoint);
	if (LayerElementVertexColor)
	{
		VertexColorReferenceMode = LayerElementVertexColor->GetReferenceMode();
		VertexColorMappingMode = LayerElementVertexColor->GetMappingMode();
	}

	//
	// get the first normal layer
	//
	FbxLayerElementNormal* LayerElementNormal = BaseLayer->GetNormals();
	FbxLayerElementTangent* LayerElementTangent = BaseLayer->GetTangents();
	FbxLayerElementBinormal* LayerElementBinormal = BaseLayer->GetBinormals();

	//whether there is normal, tangent and binormal data in this mesh
	bool bHasNTBInformation = LayerElementNormal && LayerElementTangent && LayerElementBinormal;

	FbxLayerElement::EReferenceMode NormalReferenceMode(FbxLayerElement::eDirect);
	FbxLayerElement::EMappingMode NormalMappingMode(FbxLayerElement::eByControlPoint);
	if (LayerElementNormal)
	{
		NormalReferenceMode = LayerElementNormal->GetReferenceMode();
		NormalMappingMode = LayerElementNormal->GetMappingMode();
	}

	FbxLayerElement::EReferenceMode TangentReferenceMode(FbxLayerElement::eDirect);
	FbxLayerElement::EMappingMode TangentMappingMode(FbxLayerElement::eByControlPoint);
	if (LayerElementTangent)
	{
		TangentReferenceMode = LayerElementTangent->GetReferenceMode();
		TangentMappingMode = LayerElementTangent->GetMappingMode();
	}

	FbxLayerElement::EReferenceMode BinormalReferenceMode(FbxLayerElement::eDirect);
	FbxLayerElement::EMappingMode BinormalMappingMode(FbxLayerElement::eByControlPoint);
	if (LayerElementBinormal)
	{
		BinormalReferenceMode = LayerElementBinormal->GetReferenceMode();
		BinormalMappingMode = LayerElementBinormal->GetMappingMode();
	}

	//
	// build collision
	//
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(BuildCollision);

		FbxString NodeName;
		if (const FbxMap<FbxString, FbxString>::RecordType* OriginalNodeNameKeyValuePair = NodeUniqueNameToOriginalNameMap.Find(Node->GetName()))
		{
			NodeName = GetNodeNameWithoutNamespace(OriginalNodeNameKeyValuePair->GetValue());
		}
		else
		{
			NodeName = GetNodeNameWithoutNamespace(Node->GetName());
		}

		bool bImportedCollision = ImportCollisionModels(StaticMesh, NodeName);

		//If we import a collision or we "generate one and remove the degenerates triangles" we will automatically set the section collision boolean.
		bool bEnableCollision = bImportedCollision || (GBuildStaticMeshCollision && LODIndex == 0 && ImportOptions->bRemoveDegenerates);
		for(int32 SectionIndex=MaterialIndexOffset; SectionIndex<MaterialIndexOffset+MaterialCount; SectionIndex++)
		{
			if (StaticMesh->GetSectionInfoMap().IsValidSection(LODIndex, SectionIndex))
			{
				FMeshSectionInfo Info = StaticMesh->GetSectionInfoMap().Get(LODIndex, SectionIndex);
		
				Info.bEnableCollision = bEnableCollision;
				//Make sure LOD greater then 0 copy the LOD 0 sections collision flags
				if (LODIndex != 0)
				{
					//Match the material slot index
					for (int32 LodZeroSectionIndex = 0; LodZeroSectionIndex < StaticMesh->GetSectionInfoMap().GetSectionNumber(0); ++LodZeroSectionIndex)
					{
						FMeshSectionInfo InfoLodZero = StaticMesh->GetSectionInfoMap().Get(0, LodZeroSectionIndex);
						if (InfoLodZero.MaterialIndex == Info.MaterialIndex)
						{
							Info.bEnableCollision = InfoLodZero.bEnableCollision;
							Info.bCastShadow = InfoLodZero.bCastShadow;
							break;
						}
					}
				}

				StaticMesh->GetSectionInfoMap().Set(LODIndex, SectionIndex, Info);
			}
		}
	}

	//
	// build un-mesh triangles
	//
	bool bHasNonDegeneratePolygons = false;
	{
		MeshDescription->SuspendVertexInstanceIndexing();
		MeshDescription->SuspendEdgeIndexing();
		MeshDescription->SuspendPolygonIndexing();
		MeshDescription->SuspendPolygonGroupIndexing();
		MeshDescription->SuspendUVIndexing();

		TRACE_CPUPROFILER_EVENT_SCOPE(BuildTriangles);

		// Construct the matrices for the conversion from right handed to left handed system
		FbxAMatrix TotalMatrix;
		FbxAMatrix TotalMatrixForNormal;
		TotalMatrix = ComputeTotalMatrix(Node);
		TotalMatrixForNormal = TotalMatrix.Inverse();
		TotalMatrixForNormal = TotalMatrixForNormal.Transpose();
		int32 PolygonCount = Mesh->GetPolygonCount();

		if(PolygonCount == 0)
		{
			AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("Error_NoPolygonFoundInMesh", "No polygon were found on mesh  '{0}'"), FText::FromString(Mesh->GetName()))), FFbxErrors::StaticMesh_NoTriangles);
			return false;
		}

		int32 VertexCount = Mesh->GetControlPointsCount();
		bool OddNegativeScale = IsOddNegativeScale(TotalMatrix);

		TVertexAttributesRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
		TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
		TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
		TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();
		TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors();
		TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();
		TEdgeAttributesRef<bool> EdgeHardnesses = Attributes.GetEdgeHardnesses();
		TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();

		int32 VertexOffset = MeshDescription->Vertices().Num();
		int32 VertexInstanceOffset = MeshDescription->VertexInstances().Num();
		int32 PolygonOffset = MeshDescription->Polygons().Num();

		// The below code expects Num() to be equivalent to GetArraySize(), i.e. that all added elements are appended, not inserted into existing gaps
		check(VertexOffset == MeshDescription->Vertices().GetArraySize());
		check(VertexInstanceOffset == MeshDescription->VertexInstances().GetArraySize());
		check(PolygonOffset == MeshDescription->Polygons().GetArraySize());

		TMap<int32, FPolygonGroupID> PolygonGroupMapping;
		
		//Ensure the polygon groups are create in the same order has the imported materials order
		for (int32 FbxMeshMaterialIndex = 0; FbxMeshMaterialIndex < FbxMeshMaterials.Num(); ++FbxMeshMaterialIndex)
		{
			int32 RealMaterialIndex = FbxMeshMaterialIndex + MaterialIndexOffset;
			if (!PolygonGroupMapping.Contains(RealMaterialIndex))
			{
				UMaterialInterface* Material = MeshMaterials.IsValidIndex(RealMaterialIndex) ? MeshMaterials[RealMaterialIndex].Material : UMaterial::GetDefaultMaterial(MD_Surface);
				FName ImportedMaterialSlotName = MeshMaterials.IsValidIndex(RealMaterialIndex) ? FName(*MeshMaterials[RealMaterialIndex].GetName()) : (Material != nullptr ? FName(*Material->GetName()) : NAME_None);
				FPolygonGroupID ExistingPolygonGroup = INDEX_NONE;
				for (const FPolygonGroupID PolygonGroupID : MeshDescription->PolygonGroups().GetElementIDs())
				{
					if (PolygonGroupImportedMaterialSlotNames[PolygonGroupID] == ImportedMaterialSlotName)
					{
						ExistingPolygonGroup = PolygonGroupID;
						break;
					}
				}
				if (ExistingPolygonGroup == INDEX_NONE)
				{
					ExistingPolygonGroup = MeshDescription->CreatePolygonGroup();
					PolygonGroupImportedMaterialSlotNames[ExistingPolygonGroup] = ImportedMaterialSlotName;
				}
				PolygonGroupMapping.Add(RealMaterialIndex, ExistingPolygonGroup);
			}
		}


		// When importing multiple mesh pieces to the same static mesh.  Ensure each mesh piece has the same number of Uv's
		int32 ExistingUVCount = VertexInstanceUVs.GetNumChannels();

		int32 NumUVs = FMath::Max(FBXUVs.UniqueUVCount, ExistingUVCount);
		NumUVs = FMath::Min<int32>(MAX_MESH_TEXTURE_COORDS_MD, NumUVs);
		// At least one UV set must exist.  
		// @todo: this needn't be true; we should be able to handle zero UV channels
		NumUVs = FMath::Max(1, NumUVs);

		//Make sure all Vertex instance have the correct number of UVs
		VertexInstanceUVs.SetNumChannels( NumUVs );
		MeshDescription->SetNumUVChannels(NumUVs);

		TArray<int32> UVOffsets;
		UVOffsets.SetNumUninitialized(NumUVs);
		for (int UVChannel = 0; UVChannel < NumUVs; UVChannel++)
		{
			UVOffsets[UVChannel] = MeshDescription->UVs(UVChannel).GetArraySize();
		}

		//Fill the vertex array
		MeshDescription->ReserveNewVertices(VertexCount);
		for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
		{
			int32 RealVertexIndex = VertexOffset + VertexIndex;
			FbxVector4 FbxPosition = Mesh->GetControlPoints()[VertexIndex];
			FbxPosition = TotalMatrix.MultT(FbxPosition);
			FVector3f VertexPosition = (FVector3f)Converter.ConvertPos(FbxPosition);
			
			FVertexID AddedVertexId = MeshDescription->CreateVertex();
			VertexPositions[AddedVertexId] = VertexPosition;
			if (AddedVertexId.GetValue() != RealVertexIndex)
			{
				AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("Error_CannotCreateVertex", "Cannot create valid vertex for mesh '{0}'"), FText::FromString(Mesh->GetName()))), FFbxErrors::StaticMesh_BuildError);
				return false;
			}
		}
		
		// Fill the UV arrays
		for (int32 UVLayerIndex = 0; UVLayerIndex < FBXUVs.UniqueUVCount; UVLayerIndex++)
		{
			check(FBXUVs.LayerElementUV[UVLayerIndex]);
			if (FBXUVs.LayerElementUV[UVLayerIndex] != nullptr)
			{
				int32 UVCount = FBXUVs.LayerElementUV[UVLayerIndex]->GetDirectArray().GetCount();
				TUVAttributesRef<FVector2f> UVCoordinates = MeshDescription->UVAttributes(UVLayerIndex).GetAttributesRef<FVector2f>(MeshAttribute::UV::UVCoordinate);
				MeshDescription->ReserveNewUVs(UVCount, UVLayerIndex);
				for (int32 UVIndex = 0; UVIndex < UVCount; UVIndex++)
				{
					FUVID UVID = MeshDescription->CreateUV(UVLayerIndex);
					FbxVector2 UVVector = FBXUVs.LayerElementUV[UVLayerIndex]->GetDirectArray().GetAt(UVIndex);
					UVCoordinates[UVID] = FVector2f(static_cast<float>(UVVector[0]), 1.0f - static_cast<float>(UVVector[1]));	// flip the Y of UVs for DirectX
				}
			}
		}

		TMap<uint64, int32> RemapEdgeID;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(BuildMeshEdgeVertices);
			Mesh->BeginGetMeshEdgeVertices();

			//Fill the edge array
			int32 FbxEdgeCount = Mesh->GetMeshEdgeCount();
			RemapEdgeID.Reserve(FbxEdgeCount*2);
			for (int32 FbxEdgeIndex = 0; FbxEdgeIndex < FbxEdgeCount; ++FbxEdgeIndex)
			{
				int32 EdgeStartVertexIndex = -1;
				int32 EdgeEndVertexIndex = -1;
				Mesh->GetMeshEdgeVertices(FbxEdgeIndex, EdgeStartVertexIndex, EdgeEndVertexIndex);
				// Skip invalid edges, i.e. one of the ends is invalid, or degenerated ones
				if(EdgeStartVertexIndex == -1 || EdgeEndVertexIndex == -1 || EdgeStartVertexIndex == EdgeEndVertexIndex)
				{
					UE_LOGF(LogFbx, Warning, "Skipping invalid edge on mesh %ls", *FString(Mesh->GetName()));
					continue;
				}
				FVertexID EdgeVertexStart(EdgeStartVertexIndex + VertexOffset);
				check(MeshDescription->Vertices().IsValid(EdgeVertexStart));
				FVertexID EdgeVertexEnd(EdgeEndVertexIndex + VertexOffset);
				check(MeshDescription->Vertices().IsValid(EdgeVertexEnd));
				uint64 CompactedKey = (((uint64)EdgeVertexStart.GetValue()) << 32) | ((uint64)EdgeVertexEnd.GetValue());
				RemapEdgeID.Add(CompactedKey, FbxEdgeIndex);
				//Add the other edge side
				CompactedKey = (((uint64)EdgeVertexEnd.GetValue()) << 32) | ((uint64)EdgeVertexStart.GetValue());
				RemapEdgeID.Add(CompactedKey, FbxEdgeIndex);
			}
			//Call this after all GetMeshEdgeIndexForPolygon call this is for optimization purpose.
			Mesh->EndGetMeshEdgeVertices();
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(BuildMeshEdgeIndexForPolygon);

			// Compute and reserve memory to be used for vertex instances
			{
				int32 TotalVertexCount = 0;
				for (int32 PolygonIndex = 0; PolygonIndex < PolygonCount; PolygonIndex++)
				{
					TotalVertexCount += Mesh->GetPolygonSize(PolygonIndex);
				}

				MeshDescription->ReserveNewPolygons(PolygonCount);
				MeshDescription->ReserveNewVertexInstances(TotalVertexCount);
				MeshDescription->ReserveNewEdges(Mesh->GetMeshEdgeCount());
			}

			bool  bBeginGetMeshEdgeIndexForPolygonCalled   = false;
			bool  bBeginGetMeshEdgeIndexForPolygonRequired = true;
			int32 CurrentVertexInstanceIndex = 0;
			int32 SkippedVertexInstance = 0;

			// keep those for all iterations to avoid heap allocations
			TArray<FVertexInstanceID> CornerInstanceIDs;
			TArray<FVertexID> CornerVerticesIDs;
			TArray<FVector3f, TInlineAllocator<3>> P;

			//Create a slowtask with a granularity of ~5% to avoid entering progress for every polygon.
			const int32 SlowTaskNumberOfUpdates = 20;
			const int32 SlowTaskNumberOfWorkUnitsBetweenUpdates = PolygonCount >= SlowTaskNumberOfUpdates ? PolygonCount / SlowTaskNumberOfUpdates : 1;
			int32 SlowTaskUpdateCounter = 0;
			FScopedSlowTask ProcessingPolygonsSlowTask(PolygonCount % SlowTaskNumberOfUpdates == 0 
				? SlowTaskNumberOfUpdates 
				: SlowTaskNumberOfUpdates + FMath::CeilToInt((PolygonCount % SlowTaskNumberOfUpdates) / (float)SlowTaskNumberOfWorkUnitsBetweenUpdates));
			ProcessingPolygonsSlowTask.MakeDialog();

			bool bFaceMaterialIndexInconsistencyErrorDisplayed = false;
			bool bUnsupportedSmoothingGroupErrorDisplayed = false;
			bool bCorruptedMsgDone = false;
			//Polygons
			for (int32 PolygonIndex = 0; PolygonIndex < PolygonCount; PolygonIndex++)
			{
				if (SlowTaskUpdateCounter-- <= 0)
				{
					SlowTaskUpdateCounter += SlowTaskNumberOfWorkUnitsBetweenUpdates;
					ProcessingPolygonsSlowTask.EnterProgressFrame(1);
					//Only cancel the operation if we are creating a new asset, as we don't support existing asset restoration yet.
					if (ImportOptions->bIsImportCancelable && ProcessingPolygonsSlowTask.ShouldCancel())
					{
						Mesh->EndGetMeshEdgeIndexForPolygon();
						bImportOperationCanceled = true;
						return false;
					}
				}

				int32 PolygonVertexCount = Mesh->GetPolygonSize(PolygonIndex);
				//Verify if the polygon is degenerate, in this case do not add them
				{
					float ComparisonThreshold = GetTriangleAreaThreshold();
					P.Reset();
					P.AddUninitialized(PolygonVertexCount);
					bool bAllCornerValid = true;
					for (int32 CornerIndex = 0; CornerIndex < PolygonVertexCount; CornerIndex++)
					{
						const int32 ControlPointIndex = Mesh->GetPolygonVertex(PolygonIndex, CornerIndex);
						const FVertexID VertexID(VertexOffset + ControlPointIndex);
						if (!MeshDescription->Vertices().IsValid(VertexID))
						{
							bAllCornerValid = false;
							break;
						}
						P[CornerIndex] = VertexPositions[VertexID];
					}
					if (!bAllCornerValid)
					{
						if (!bCorruptedMsgDone)
						{
							AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("Error_CorruptedFbxPolygon", "Corrupted triangle for the mesh '{0}'"), FText::FromString(Mesh->GetName()))), FFbxErrors::StaticMesh_BuildError);
							bCorruptedMsgDone = true;
						}
						SkippedVertexInstance += PolygonVertexCount;
						continue;
					}
					check(P.Num() > 2); //triangle is the smallest polygon we can have
					const FVector3f Normal = ((P[1] - P[2]) ^ (P[0] - P[2])).GetSafeNormal(ComparisonThreshold);
					//Check for degenerated polygons, avoid NAN
					if (Normal.IsNearlyZero(ComparisonThreshold) || Normal.ContainsNaN())
					{
						SkippedVertexInstance += PolygonVertexCount;
						continue;
					}
				}

				int32 RealPolygonIndex = PolygonOffset + PolygonIndex;
				CornerInstanceIDs.Reset();
				CornerInstanceIDs.AddUninitialized(PolygonVertexCount);
				CornerVerticesIDs.Reset();
				CornerVerticesIDs.AddUninitialized(PolygonVertexCount);
				for (int32 CornerIndex = 0; CornerIndex < PolygonVertexCount; CornerIndex++)
				{
					int32 VertexInstanceIndex = VertexInstanceOffset + CurrentVertexInstanceIndex + CornerIndex;
					int32 RealFbxVertexIndex = SkippedVertexInstance + CurrentVertexInstanceIndex + CornerIndex;
					const FVertexInstanceID VertexInstanceID(VertexInstanceIndex);
					CornerInstanceIDs[CornerIndex] = VertexInstanceID;
					const int32 ControlPointIndex = Mesh->GetPolygonVertex(PolygonIndex, CornerIndex);
					const FVertexID VertexID(VertexOffset + ControlPointIndex);
					const FVector3f& VertexPosition = VertexPositions[VertexID];
					CornerVerticesIDs[CornerIndex] = VertexID;

					FVertexInstanceID AddedVertexInstanceId = MeshDescription->CreateVertexInstance(VertexID);
				
					//Make sure the Added vertex instance ID is matching the expected vertex instance ID
					check(AddedVertexInstanceId == VertexInstanceID);
				
					if (AddedVertexInstanceId.GetValue() != VertexInstanceIndex)
					{
						AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("Error_CannotCreateVertexInstance", "Cannot create valid vertex instance for mesh '{0}'"), FText::FromString(Mesh->GetName()))), FFbxErrors::StaticMesh_BuildError);
						return false;
					}

					//UVs attributes
					for (int32 UVLayerIndex = 0; UVLayerIndex < FBXUVs.UniqueUVCount; UVLayerIndex++)
					{
						FVector2f FinalUVVector(0.0f, 0.0f);
						if (FBXUVs.LayerElementUV[UVLayerIndex] != NULL)
						{
							int UVMapIndex = (FBXUVs.UVMappingMode[UVLayerIndex] == FbxLayerElement::eByControlPoint) ? ControlPointIndex : RealFbxVertexIndex;
							int32 UVIndex = (FBXUVs.UVReferenceMode[UVLayerIndex] == FbxLayerElement::eDirect) ?
								UVMapIndex : FBXUVs.LayerElementUV[UVLayerIndex]->GetIndexArray().GetAt(UVMapIndex);

							FbxVector2	UVVector = FBXUVs.LayerElementUV[UVLayerIndex]->GetDirectArray().GetAt(UVIndex);
							FinalUVVector.X = static_cast<float>(UVVector[0]);
							FinalUVVector.Y = 1.f - static_cast<float>(UVVector[1]);   //flip the Y of UVs for DirectX
						}
						VertexInstanceUVs.Set(AddedVertexInstanceId, UVLayerIndex, FinalUVVector);
					}

					//Color attribute
					if (VertexColorImportOption == EVertexColorImportOption::Replace)
					{
						if (LayerElementVertexColor)
						{
							int32 VertexColorMappingIndex = (VertexColorMappingMode == FbxLayerElement::eByControlPoint) ?
								Mesh->GetPolygonVertex(PolygonIndex, CornerIndex) : (RealFbxVertexIndex);

							int32 VectorColorIndex = (VertexColorReferenceMode == FbxLayerElement::eDirect) ?
								VertexColorMappingIndex : LayerElementVertexColor->GetIndexArray().GetAt(VertexColorMappingIndex);

							FbxColor VertexColor = LayerElementVertexColor->GetDirectArray().GetAt(VectorColorIndex);

							// Clamp manually to avoid wrapping around because of truncation for values outside of the 0.0 to 1.0 range
							FColor VertexInstanceColor(
								uint8(255.f * FMath::Clamp(VertexColor.mRed, 0.0, 1.0)),
								uint8(255.f * FMath::Clamp(VertexColor.mGreen, 0.0, 1.0)),
								uint8(255.f * FMath::Clamp(VertexColor.mBlue, 0.0, 1.0)),
								uint8(255.f * FMath::Clamp(VertexColor.mAlpha, 0.0, 1.0))
							);
							VertexInstanceColors[AddedVertexInstanceId] = FVector4f(FLinearColor(VertexInstanceColor));
						}
					}
					else if (VertexColorImportOption == EVertexColorImportOption::Ignore)
					{
						// try to match this triangles current vertex with one that existed in the previous mesh.
						// This is a find in a tmap which uses a fast hash table lookup.
						const FColor* PaintedColor = ExistingVertexColorData.Find(VertexPosition);
						if (PaintedColor)
						{
							// A matching color for this vertex was found
							VertexInstanceColors[AddedVertexInstanceId] = FVector4f(FLinearColor(*PaintedColor));
						}
					}
					else
					{
						// set the triangle's vertex color to a constant override
						check(VertexColorImportOption == EVertexColorImportOption::Override);
						VertexInstanceColors[AddedVertexInstanceId] = FVector4f(FLinearColor(VertexOverrideColor));
					}

					if (LayerElementNormal)
					{
						//normals may have different reference and mapping mode than tangents and binormals
						int NormalMapIndex = (NormalMappingMode == FbxLayerElement::eByControlPoint) ?
							ControlPointIndex : RealFbxVertexIndex;
						int NormalValueIndex = (NormalReferenceMode == FbxLayerElement::eDirect) ?
							NormalMapIndex : LayerElementNormal->GetIndexArray().GetAt(NormalMapIndex);
					
						FbxVector4 TempValue = LayerElementNormal->GetDirectArray().GetAt(NormalValueIndex);
						TempValue = TotalMatrixForNormal.MultT(TempValue);
						FVector3f TangentZ = (FVector3f)Converter.ConvertDir(TempValue);
						VertexInstanceNormals[AddedVertexInstanceId] = TangentZ.GetSafeNormal();
						//tangents and binormals share the same reference, mapping mode and index array
						if (bHasNTBInformation)
						{
							int TangentMapIndex = (TangentMappingMode == FbxLayerElement::eByControlPoint) ?
								ControlPointIndex : RealFbxVertexIndex;
							int TangentValueIndex = (TangentReferenceMode == FbxLayerElement::eDirect) ?
								TangentMapIndex : LayerElementTangent->GetIndexArray().GetAt(TangentMapIndex);

							TempValue = LayerElementTangent->GetDirectArray().GetAt(TangentValueIndex);
							TempValue = TotalMatrixForNormal.MultT(TempValue);
							FVector3f TangentX = (FVector3f)Converter.ConvertDir(TempValue);
							VertexInstanceTangents[AddedVertexInstanceId] = TangentX.GetSafeNormal();

							int BinormalMapIndex = (BinormalMappingMode == FbxLayerElement::eByControlPoint) ?
								ControlPointIndex : RealFbxVertexIndex;
							int BinormalValueIndex = (BinormalReferenceMode == FbxLayerElement::eDirect) ?
								BinormalMapIndex : LayerElementBinormal->GetIndexArray().GetAt(BinormalMapIndex);

							TempValue = LayerElementBinormal->GetDirectArray().GetAt(BinormalValueIndex);
							TempValue = TotalMatrixForNormal.MultT(TempValue);
							FVector3f TangentY = (FVector3f)-Converter.ConvertDir(TempValue);
							VertexInstanceBinormalSigns[AddedVertexInstanceId] = GetBasisDeterminantSign((FVector)TangentX.GetSafeNormal(), (FVector)TangentY.GetSafeNormal(), (FVector)TangentZ.GetSafeNormal());
						}
					}
				}
			
				// Check if the polygon just discovered is non-degenerate if we haven't found one yet
				//TODO check all polygon vertex, not just the first 3 vertex
				if (!bHasNonDegeneratePolygons)
				{
					const float PointComparisonThreshold = GetPointComparisonThreshold();
					FVector3f VertexPosition[3];
					VertexPosition[0] = VertexPositions[CornerVerticesIDs[0]];
					VertexPosition[1] = VertexPositions[CornerVerticesIDs[1]];
					VertexPosition[2] = VertexPositions[CornerVerticesIDs[2]];
					if (!( VertexPosition[0].Equals(VertexPosition[1], PointComparisonThreshold)
						|| VertexPosition[0].Equals(VertexPosition[2], PointComparisonThreshold)
						|| VertexPosition[1].Equals(VertexPosition[2], PointComparisonThreshold)))
					{
						bHasNonDegeneratePolygons = true;
					}
				}

				//
				// material index
				//
				int32 MaterialIndex = 0;
				if (MaterialCount > 0)
				{
					if (LayerElementMaterial)
					{
						switch (MaterialMappingMode)
						{
							// material index is stored in the IndexArray, not the DirectArray (which is irrelevant with 2009.1)
						case FbxLayerElement::eAllSame:
						{
							MaterialIndex = LayerElementMaterial->GetIndexArray().GetAt(0);
						}
						break;
						case FbxLayerElement::eByPolygon:
						{
							MaterialIndex = LayerElementMaterial->GetIndexArray().GetAt(PolygonIndex);
						}
						break;
						}
					}
				}

				if (MaterialIndex >= MaterialCount || MaterialIndex < 0)
				{
					if (!bFaceMaterialIndexInconsistencyErrorDisplayed)
					{
						AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, LOCTEXT("Error_MaterialIndexInconsistency", "Face material index inconsistency - forcing to 0")), FFbxErrors::Generic_Mesh_MaterialIndexInconsistency);
						bFaceMaterialIndexInconsistencyErrorDisplayed = true;
					}
					MaterialIndex = 0;
				}

				//Create a polygon with the 3 vertex instances Add it to the material group
				int32 RealMaterialIndex = MaterialIndexOffset + MaterialIndex;
				if (!PolygonGroupMapping.Contains(RealMaterialIndex))
				{
					UMaterialInterface* Material = MeshMaterials.IsValidIndex(RealMaterialIndex) ? MeshMaterials[RealMaterialIndex].Material : UMaterial::GetDefaultMaterial(MD_Surface);
					FName ImportedMaterialSlotName = MeshMaterials.IsValidIndex(RealMaterialIndex) ? FName(*MeshMaterials[RealMaterialIndex].GetName()) : (Material != nullptr ? FName(*Material->GetName()) : NAME_None);
					FPolygonGroupID ExistingPolygonGroup = INDEX_NONE;
					for (const FPolygonGroupID PolygonGroupID : MeshDescription->PolygonGroups().GetElementIDs())
					{
						if (PolygonGroupImportedMaterialSlotNames[PolygonGroupID] == ImportedMaterialSlotName)
						{
							ExistingPolygonGroup = PolygonGroupID;
							break;
						}
					}
					if (ExistingPolygonGroup == INDEX_NONE)
					{
						ExistingPolygonGroup = MeshDescription->CreatePolygonGroup();
						PolygonGroupImportedMaterialSlotNames[ExistingPolygonGroup] = ImportedMaterialSlotName;
					}
					PolygonGroupMapping.Add(RealMaterialIndex, ExistingPolygonGroup);
				}

				// Create polygon edges
				{
					// Add the edges of this polygon
					for (uint32 PolygonEdgeNumber = 0; PolygonEdgeNumber < (uint32)PolygonVertexCount; ++PolygonEdgeNumber)
					{
						//Find the matching edge ID
						uint32 CornerIndices[2];
						CornerIndices[0] = (PolygonEdgeNumber + 0) % PolygonVertexCount;
						CornerIndices[1] = (PolygonEdgeNumber + 1) % PolygonVertexCount;

						FVertexID EdgeVertexIDs[2];
						EdgeVertexIDs[0] = CornerVerticesIDs[CornerIndices[0]];
						EdgeVertexIDs[1] = CornerVerticesIDs[CornerIndices[1]];

						FEdgeID MatchEdgeId = MeshDescription->GetVertexPairEdge(EdgeVertexIDs[0], EdgeVertexIDs[1]);
						if (MatchEdgeId == INDEX_NONE)
						{
							MatchEdgeId = MeshDescription->CreateEdge(EdgeVertexIDs[0], EdgeVertexIDs[1]);
						}

						//RawMesh do not have edges, so by ordering the edge with the triangle construction we can ensure back and forth conversion with RawMesh
						//When raw mesh will be completely remove we can create the edges right after the vertex creation.
						int32 EdgeIndex = INDEX_NONE;
						uint64 CompactedKey = (((uint64)EdgeVertexIDs[0].GetValue()) << 32) | ((uint64)EdgeVertexIDs[1].GetValue());
						if (RemapEdgeID.Contains(CompactedKey))
						{
							EdgeIndex = RemapEdgeID[CompactedKey];
						}
						else
						{
							// Call BeginGetMeshEdgeIndexForPolygon lazily only if we enter the code path calling GetMeshEdgeIndexForPolygon
							if (bBeginGetMeshEdgeIndexForPolygonRequired)
							{
								//Call this before all GetMeshEdgeIndexForPolygon for optimization purpose.
								//But do not spend time precomputing stuff if the mesh has no edge since 
								//GetMeshEdgeIndexForPolygon is going to always returns -1 anyway without going into the slow path.
								if (Mesh->GetMeshEdgeCount() > 0)
								{
									Mesh->BeginGetMeshEdgeIndexForPolygon();
									bBeginGetMeshEdgeIndexForPolygonCalled = true;
								}
								bBeginGetMeshEdgeIndexForPolygonRequired = false;
							}

							EdgeIndex = Mesh->GetMeshEdgeIndexForPolygon(PolygonIndex, PolygonEdgeNumber);
						}
					
						if (!EdgeHardnesses[MatchEdgeId])
						{
							if (bSmoothingAvailable && SmoothingInfo)
							{
								if (SmoothingMappingMode == FbxLayerElement::eByEdge)
								{
									int32 lSmoothingIndex = (SmoothingReferenceMode == FbxLayerElement::eDirect) ? EdgeIndex : SmoothingInfo->GetIndexArray().GetAt(EdgeIndex);
									//Set the hard edges
									EdgeHardnesses[MatchEdgeId] = (SmoothingInfo->GetDirectArray().GetAt(lSmoothingIndex) == 0);
								}
								else if(!bUnsupportedSmoothingGroupErrorDisplayed)
								{
									AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("Error_UnsupportedSmoothingGroup", "Unsupported Smoothing group mapping mode on mesh  '{0}'"), FText::FromString(Mesh->GetName()))), FFbxErrors::Generic_Mesh_UnsupportingSmoothingGroup);
									bUnsupportedSmoothingGroupErrorDisplayed = true;
								}
							}
							else
							{
								//When there is no smoothing group we set all edge to hard (faceted mesh) or false depending on the project settings
								EdgeHardnesses[MatchEdgeId] = !bStaticMeshUseSmoothEdgesIfSmoothingInformationIsMissing;
							}
						}
					}
				}

				FPolygonGroupID PolygonGroupID = PolygonGroupMapping[RealMaterialIndex];
				// Insert a triangle into the mesh
				// @note: we only ever import triangulated meshes currently. This could easily change as we have the infrastructure set up for arbitrary ngons.
				// However, I think either we need to triangulate in the exact same way as Maya/Max if we do that.
				check(CornerInstanceIDs.Num() == 3);
				check(PolygonVertexCount == 3);
				TArray<FEdgeID> NewEdgeIDs;
				const FTriangleID NewTriangleID = MeshDescription->CreateTriangle(PolygonGroupID, CornerInstanceIDs, &NewEdgeIDs);
				check(NewEdgeIDs.Num() == 0);

				for (int32 UVLayerIndex = 0; UVLayerIndex < FBXUVs.UniqueUVCount; UVLayerIndex++)
				{
					FUVID UVIDs[3] = {INDEX_NONE, INDEX_NONE, INDEX_NONE};
					if (FBXUVs.LayerElementUV[UVLayerIndex] != NULL)
					{
						int32 UVDirectArrayCount = FBXUVs.LayerElementUV[UVLayerIndex]->GetDirectArray().GetCount();
						int32 UVIndexArrayCount = FBXUVs.LayerElementUV[UVLayerIndex]->GetIndexArray().GetCount();

						for (int32 VertexIndex = 0; VertexIndex < 3; ++VertexIndex)
						{
							int UVMapIndex = (FBXUVs.UVMappingMode[UVLayerIndex] == FbxLayerElement::eByControlPoint)
								? Mesh->GetPolygonVertex(PolygonIndex, VertexIndex)
								: SkippedVertexInstance + CurrentVertexInstanceIndex + VertexIndex;
							int32 UVIndex = (FBXUVs.UVReferenceMode[UVLayerIndex] == FbxLayerElement::eDirect)
								? UVMapIndex
								: (UVMapIndex < UVIndexArrayCount ? FBXUVs.LayerElementUV[UVLayerIndex]->GetIndexArray().GetAt(UVMapIndex) : -1);

							if (UVIndex >= FBXUVs.LayerElementUV[UVLayerIndex]->GetDirectArray().GetCount())
							{
								UVIndex = -1;
							}

							if (UVIndex != -1)
							{
								UVIDs[VertexIndex] = UVIndex + UVOffsets[UVLayerIndex];

								check(MeshDescription->VertexInstanceAttributes().GetAttribute<FVector2f>(CornerInstanceIDs[VertexIndex], MeshAttribute::VertexInstance::TextureCoordinate, UVLayerIndex) ==
									MeshDescription->UVAttributes(UVLayerIndex).GetAttribute<FVector2f>(UVIndex + UVOffsets[UVLayerIndex], MeshAttribute::UV::UVCoordinate));
							}
							else
							{
								// TODO: what does it mean to have a UV index of -1?
								// Investigate this case more carefully and handle it properly.
							}
						}
					}

					MeshDescription->SetTriangleUVIndices(NewTriangleID, UVIDs, UVLayerIndex);
				}

				CurrentVertexInstanceIndex += PolygonVertexCount;
			}

			//Call this after all GetMeshEdgeIndexForPolygon call this is for optimization purpose.
			if (bBeginGetMeshEdgeIndexForPolygonCalled)
			{
				Mesh->EndGetMeshEdgeIndexForPolygon();
			}

			MeshDescription->ResumeVertexInstanceIndexing();
			MeshDescription->ResumeEdgeIndexing();
			MeshDescription->ResumePolygonIndexing();
			MeshDescription->ResumePolygonGroupIndexing();
			MeshDescription->ResumeUVIndexing();

			if (SkippedVertexInstance > 0)
			{
				check(MeshDescription->Triangles().Num() == MeshDescription->Triangles().GetArraySize());
			}
		}
	}

	TArray<FPolygonGroupID> EmptyPolygonGroups;
	for (const FPolygonGroupID PolygonGroupID : MeshDescription->PolygonGroups().GetElementIDs())
	{
		if (MeshDescription->GetNumPolygonGroupTriangles(PolygonGroupID) == 0)
		{
			EmptyPolygonGroups.Add(PolygonGroupID);
		}
	}
	if (EmptyPolygonGroups.Num() > 0)
	{
		for (const FPolygonGroupID PolygonGroupID : EmptyPolygonGroups)
		{
			MeshDescription->DeletePolygonGroup(PolygonGroupID);
		}
		FElementIDRemappings OutRemappings;
		MeshDescription->Compact(OutRemappings);
	}

	// needed?
	FBXUVs.Cleanup();

	if (!bHasNonDegeneratePolygons)
	{
		CreateTokenizedErrorForDegeneratedPart(this, StaticMesh->GetName(), FbxNodeName);
	}

	bool bIsValidMesh = bHasNonDegeneratePolygons;

	return bIsValidMesh;
}

UStaticMesh* UnFbx::FFbxImporter::ReimportSceneStaticMesh(uint64 FbxNodeUniqueId, uint64 FbxUniqueId, UStaticMesh* Mesh, UFbxStaticMeshImportData* TemplateImportData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FFbxImporter::ReimportSceneStaticMesh);
	FFbxScopedOperation FbxScopedImportOperation(this);

	FRestoreReimportData RestoreData(Mesh);

	TArray<FbxNode*> FbxMeshArray;
	UStaticMesh* FirstBaseMesh = NULL;
	FbxNode* Node = NULL;

	// get meshes in Fbx file
	//the function also fill the collision models, so we can update collision models correctly
	FillFbxMeshArray(Scene->GetRootNode(), FbxMeshArray, this);

	if (FbxMeshArray.Num() < 1)
	{
		AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("Error_NoFBXMeshAttributeFound", "No FBX attribute mesh found when reimport scene static mesh '{0}'. The FBX file contain no static mesh."), FText::FromString(Mesh->GetName()))), FFbxErrors::Generic_Mesh_MeshNotFound);
		return Mesh;
	}
	else
	{
		//Find the first node using the mesh attribute with the unique ID
		for (FbxNode *MeshNode : FbxMeshArray)
		{
			if (FbxNodeUniqueId == INVALID_UNIQUE_ID || ImportOptions->bBakePivotInVertex == false)
			{
				if (FbxUniqueId == MeshNode->GetMesh()->GetUniqueID())
				{
					Node = MeshNode;
					break;
				}
			}
			else
			{
				if (FbxNodeUniqueId == MeshNode->GetUniqueID() && FbxUniqueId == MeshNode->GetMesh()->GetUniqueID())
				{
					Node = MeshNode;
					break;
				}
			}
		}
	}

	if (!Node)
	{
		//Cannot find the staticmesh name in the fbx scene file
		AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("Error_NoFBXMeshNameFound", "No FBX attribute mesh with the same name was found when reimport scene static mesh '{0}'."), FText::FromString(Mesh->GetName()))), FFbxErrors::Generic_Mesh_MeshNotFound);
		return Mesh;
	}

	TSharedPtr<FExistingStaticMeshData> ExistMeshDataPtr = StaticMeshImportUtils::SaveExistingStaticMeshData(Mesh, ImportOptions, INDEX_NONE);

	if (Node)
	{
		FbxNode* NodeParent = RecursiveFindParentLodGroup(Node->GetParent());

		// if the Fbx mesh is a part of LODGroup, update LOD
		if (NodeParent && NodeParent->GetNodeAttribute() && NodeParent->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup)
		{
			TArray<UStaticMesh*> BaseMeshes;
			TArray<FbxNode*> AllNodeInLod;
			FindAllLODGroupNode(AllNodeInLod, NodeParent, 0);
			FirstBaseMesh = ImportStaticMeshAsSingle(Mesh->GetOutermost(), AllNodeInLod, *Mesh->GetName(), RF_Public | RF_Standalone, TemplateImportData, Mesh, 0, ExistMeshDataPtr.Get());
			//If we have a valid LOD group name we don't want to re-import LODs since they will be automatically generate by the LODGroup reduce settings
			if(FirstBaseMesh && Mesh->GetLODGroup() == NAME_None)
			{
				// import LOD meshes
				for (int32 LODIndex = 1; LODIndex < NodeParent->GetChildCount(); LODIndex++)
				{
					if (LODIndex >= MAX_STATIC_MESH_LODS)
					{
						AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(
							LOCTEXT("ImporterLimits_MaximumStaticMeshLODReach_ReimportSceneStaticMesh", "Reached the maximum number of LODs for a Static Mesh({0}) - discarding {1} LOD meshes.")
								, FText::AsNumber(MAX_STATIC_MESH_LODS), FText::AsNumber(NodeParent->GetChildCount() - MAX_STATIC_MESH_LODS))
						), FFbxErrors::Generic_Mesh_TooManyLODs);
						break;
					}
					AllNodeInLod.Empty();
					FindAllLODGroupNode(AllNodeInLod, NodeParent, LODIndex);
					if (AllNodeInLod.Num() > 0)
					{
						if (AllNodeInLod[0]->GetMesh() == nullptr)
						{
							AddStaticMeshSourceModelGeneratedLOD(FirstBaseMesh, LODIndex);
						}
						else
						{
							//For LOD we don't pass the ExistMeshDataPtr
							ImportStaticMeshAsSingle(Mesh->GetOutermost(), AllNodeInLod, *Mesh->GetName(), RF_Public | RF_Standalone, TemplateImportData, FirstBaseMesh, LODIndex, nullptr);
							if (FirstBaseMesh->IsSourceModelValid(LODIndex))
							{
								FirstBaseMesh->GetSourceModel(LODIndex).bImportWithBaseMesh = true;
							}
						}
					}
				}
			}
			if (FirstBaseMesh != nullptr)
			{
				FindAllLODGroupNode(AllNodeInLod, NodeParent, 0);
				PostImportStaticMesh(FirstBaseMesh, AllNodeInLod);
			}
		}
		else
		{
			FirstBaseMesh = ImportStaticMesh(Mesh->GetOutermost(), Node, *Mesh->GetName(), RF_Public | RF_Standalone, TemplateImportData, Mesh, 0, ExistMeshDataPtr.Get());
			if (FirstBaseMesh != nullptr)
			{
				TArray<FbxNode*> AllNodeInLod;
				AllNodeInLod.Add(Node);
				PostImportStaticMesh(FirstBaseMesh, AllNodeInLod);
			}
		}
	}
	else
	{
		// no FBX mesh match, maybe the Unreal mesh is imported from multiple FBX mesh (enable option "Import As Single")
		if (FbxMeshArray.Num() > 0)
		{
			FirstBaseMesh = ImportStaticMeshAsSingle(Mesh->GetOutermost(), FbxMeshArray, *Mesh->GetName(), RF_Public | RF_Standalone, TemplateImportData, Mesh, 0, ExistMeshDataPtr.Get());
			if (FirstBaseMesh != nullptr)
			{
				PostImportStaticMesh(FirstBaseMesh, FbxMeshArray);
			}
		}
		else // no mesh found in the FBX file
		{
			AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("Error_NoFBXMeshFound", "No FBX mesh found when reimport Unreal mesh '{0}'. The FBX file is crashed."), FText::FromString(Mesh->GetName()))), FFbxErrors::Generic_Mesh_MeshNotFound);
		}
	}

	if(FirstBaseMesh)
	{
	//Don't restore materials when reimporting scene
		StaticMeshImportUtils::RestoreExistingMeshData(ExistMeshDataPtr, FirstBaseMesh, INDEX_NONE, false, ImportOptions->bResetToFbxOnMaterialConflict);
		RestoreData.CleanupDuplicateMesh();
	}
	else
	{
		RestoreData.RestoreMesh(this);
	}

	return FirstBaseMesh;
}

void UnFbx::FFbxImporter::AddStaticMeshSourceModelGeneratedLOD(UStaticMesh* StaticMesh, int32 LODIndex)
{
	//Add a Lod generated model
	while (StaticMesh->GetNumSourceModels() <= LODIndex)
	{
		StaticMesh->AddSourceModel();
	}

	FStaticMeshSourceModel& ThisSourceModel = StaticMesh->GetSourceModel(LODIndex);
	if (LODIndex - 1 > 0 && StaticMesh->IsReductionActive(LODIndex - 1))
	{
		const FStaticMeshSourceModel& PrevSourceModel = StaticMesh->GetSourceModel(LODIndex - 1);
		if (PrevSourceModel.ReductionSettings.PercentTriangles < 1.0f)
		{
			ThisSourceModel.ReductionSettings.PercentTriangles = PrevSourceModel.ReductionSettings.PercentTriangles * 0.5f;
		}
		else if (PrevSourceModel.ReductionSettings.MaxDeviation > 0.0f)
		{
			ThisSourceModel.ReductionSettings.MaxDeviation = PrevSourceModel.ReductionSettings.MaxDeviation + 1.0f;
		}
	}
	else
	{
		ThisSourceModel.ReductionSettings.PercentTriangles = FMath::Pow(0.5f, (float)LODIndex);
	}
}

UStaticMesh* UnFbx::FFbxImporter::ReimportStaticMesh(UStaticMesh* Mesh, UFbxStaticMeshImportData* TemplateImportData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FFbxImporter::ReimportStaticMesh);
	FFbxScopedOperation FbxScopedImportOperation(this);

	FRestoreReimportData RestoreData(Mesh);

	TArray<FbxNode*> FbxMeshArray;
	FbxNode* Node = NULL;
	UStaticMesh* NewMesh = NULL;

	// get meshes in Fbx file
	bool bImportStaticMeshLODs = ImportOptions->bImportStaticMeshLODs;
	bool bCombineMeshes = ImportOptions->bCombineToSingle;
	bool bCombineMeshesLOD = false;
	TArray<TArray<FbxNode*>> FbxMeshesLod;

	if (bCombineMeshes && !bImportStaticMeshLODs)
	{
		//the function also fill the collision models, so we can update collision models correctly
		FillFbxMeshArray(Scene->GetRootNode(), FbxMeshArray, this);
	}
	else
	{
		// count meshes in lod groups if we dont care about importing LODs
		bool bCountLODGroupMeshes = !bImportStaticMeshLODs && bCombineMeshes;
		int32 NumLODGroups = 0;
		GetFbxMeshCount(Scene->GetRootNode(), bCountLODGroupMeshes, NumLODGroups);
		// if there were LODs in the file, do not combine meshes even if requested
		if (bImportStaticMeshLODs && bCombineMeshes && NumLODGroups > 0)
		{
			TArray<FbxNode*> FbxLodGroups;
			
			FillFbxMeshAndLODGroupArray(Scene->GetRootNode(), FbxLodGroups, FbxMeshArray);
			FbxMeshesLod.Add(FbxMeshArray);
			for (FbxNode* LODGroup : FbxLodGroups)
			{
				if (LODGroup->GetNodeAttribute() && LODGroup->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup && LODGroup->GetChildCount() > 0)
				{
					for (int32 GroupLodIndex = 0; GroupLodIndex < LODGroup->GetChildCount(); ++GroupLodIndex)
					{
						if (GroupLodIndex >= MAX_STATIC_MESH_LODS)
						{
							AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(
								LOCTEXT("ImporterLimits_MaximumStaticMeshLODReach", "Reached the maximum number of LODs for a Static Mesh({0}) - discarding {1} LOD meshes."), FText::AsNumber(MAX_STATIC_MESH_LODS), FText::AsNumber(LODGroup->GetChildCount() - MAX_STATIC_MESH_LODS))
							), FFbxErrors::Generic_Mesh_TooManyLODs);
							break;
						}
						TArray<FbxNode*> AllNodeInLod;
						FindAllLODGroupNode(AllNodeInLod, LODGroup, GroupLodIndex);
						if (AllNodeInLod.Num() > 0)
						{
							if (FbxMeshesLod.Num() <= GroupLodIndex)
							{
								FbxMeshesLod.Add(AllNodeInLod);
							}
							else
							{
								TArray<FbxNode*> &LODGroupArray = FbxMeshesLod[GroupLodIndex];
								for (FbxNode* NodeToAdd : AllNodeInLod)
								{
									LODGroupArray.Add(NodeToAdd);
								}
							}
						}
					}
				}
			}
			bCombineMeshesLOD = true;
			bCombineMeshes = false;
			//Set the first LOD
			FbxMeshArray = FbxMeshesLod[0];
		}
		else
		{
			FillFbxMeshArray(Scene->GetRootNode(), FbxMeshArray, this);
		}
	}
	
	// if there is only one mesh, use it without name checking 
	// (because the "Used As Full Name" option enables users name the Unreal mesh by themselves
	if (!bCombineMeshesLOD && FbxMeshArray.Num() == 1)
	{
		Node = FbxMeshArray[0];
	}
	else if(!bCombineMeshes && !bCombineMeshesLOD)
	{
		Node = GetMeshNodesFromName(Mesh->GetName(), FbxMeshArray);
	}

	// If there is no match it may be because an LOD group was imported where
	// the mesh name does not match the file name. This is actually the common case.
	if (!bCombineMeshesLOD && !Node && FbxMeshArray.IsValidIndex(0))
	{
		FbxNode* BaseLODNode = FbxMeshArray[0];
		
		FbxNode* NodeParent = BaseLODNode ? RecursiveFindParentLodGroup(BaseLODNode->GetParent()) : nullptr;
		if (NodeParent && NodeParent->GetNodeAttribute() && NodeParent->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup)
		{
			// Reimport the entire LOD chain.
			Node = BaseLODNode;
		}
	}
	
	ImportOptions->bImportMaterials = false;
	ImportOptions->bImportTextures = false;

	// Before calling SaveExistingStaticMeshData() we must remove the existing fbx metadata as we don't want to restore those.
	RemoveFBXMetaData(Mesh);
	TSharedPtr<FExistingStaticMeshData> ExistMeshDataPtr = StaticMeshImportUtils::SaveExistingStaticMeshData(Mesh, ImportOptions, INDEX_NONE);

	TArray<int32> ReimportLodList;
	if (bCombineMeshesLOD)
	{
		TArray<FbxNode*> LodZeroNodes;
		//Import the LOD root
		if (FbxMeshesLod.Num() > 0)
		{
			TArray<FbxNode*> &LODMeshesArray = FbxMeshesLod[0];
			LodZeroNodes = FbxMeshesLod[0];
			NewMesh = ImportStaticMeshAsSingle(Mesh->GetOuter(), LODMeshesArray, *Mesh->GetName(), RF_Public | RF_Standalone, TemplateImportData, Mesh, 0, ExistMeshDataPtr.Get());
			ReimportLodList.Add(0);
		}
		//Import all LODs
		for (int32 LODIndex = 1; LODIndex < FbxMeshesLod.Num(); ++LODIndex)
		{
			if (LODIndex >= MAX_STATIC_MESH_LODS)
			{
				AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(
					LOCTEXT("ImporterLimits_MaximumStaticMeshLODReach", "Reached the maximum number of LODs for a Static Mesh({0}) - discarding {1} LOD meshes."), FText::AsNumber(MAX_STATIC_MESH_LODS), FText::AsNumber(FbxMeshesLod.Num() - MAX_STATIC_MESH_LODS))
				), FFbxErrors::Generic_Mesh_TooManyLODs);
				break;
			}

			TArray<FbxNode*> &LODMeshesArray = FbxMeshesLod[LODIndex];

			if (LODMeshesArray[0]->GetMesh() == nullptr)
			{
				AddStaticMeshSourceModelGeneratedLOD(NewMesh, LODIndex);
			}
			else
			{
				ImportStaticMeshAsSingle(Mesh->GetOuter(), LODMeshesArray, *Mesh->GetName(), RF_Public | RF_Standalone, TemplateImportData, NewMesh, LODIndex, nullptr);
				ReimportLodList.Add(LODIndex);
				if (NewMesh && NewMesh->IsSourceModelValid(LODIndex))
				{
					NewMesh->GetSourceModel(LODIndex).bImportWithBaseMesh = true;
				}
			}
		}
		if (NewMesh != nullptr)
		{
			PostImportStaticMesh(NewMesh, LodZeroNodes);
		}
	}
	else if (Node)
	{
		FbxNode* NodeParent = RecursiveFindParentLodGroup(Node->GetParent());

		TArray<FbxNode*> LodZeroNodes;
		// if the Fbx mesh is a part of LODGroup, update LOD
		if (NodeParent && NodeParent->GetNodeAttribute() && NodeParent->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup)
		{
			TArray<FbxNode*> AllNodeInLod;
			FindAllLODGroupNode(AllNodeInLod, NodeParent, 0);
			if (AllNodeInLod.Num() > 0)
			{
				LodZeroNodes = AllNodeInLod;
				NewMesh = ImportStaticMeshAsSingle(Mesh->GetOuter(), AllNodeInLod, *Mesh->GetName(), RF_Public | RF_Standalone, TemplateImportData, Mesh, 0, ExistMeshDataPtr.Get());
				ReimportLodList.Add(0);
			}

			//If we have a valid LOD group name we don't want to re-import LODs since they will be automatically generate by the LODGroup reduce settings
			if (NewMesh && ImportOptions->bImportStaticMeshLODs && Mesh->GetLODGroup() == NAME_None)
			{
				// import LOD meshes
				for (int32 LODIndex = 1; LODIndex < NodeParent->GetChildCount(); LODIndex++)
				{
					if (LODIndex >= MAX_STATIC_MESH_LODS)
					{
						AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(
							LOCTEXT("ImporterLimits_MaximumStaticMeshLODReach", "Reached the maximum number of LODs for a Static Mesh({0}) - discarding {1} LOD meshes."), FText::AsNumber(MAX_STATIC_MESH_LODS), FText::AsNumber(NodeParent->GetChildCount() - MAX_STATIC_MESH_LODS))
						), FFbxErrors::Generic_Mesh_TooManyLODs);
						break;
					}

					AllNodeInLod.Empty();
					FindAllLODGroupNode(AllNodeInLod, NodeParent, LODIndex);
					if (AllNodeInLod.Num() > 0)
					{
						if (AllNodeInLod[0]->GetMesh() == nullptr)
						{
							AddStaticMeshSourceModelGeneratedLOD(NewMesh, LODIndex);
						}
						else
						{
							//For LOD we don't pass the ExistMeshDataPtr
							ImportStaticMeshAsSingle(Mesh->GetOuter(), AllNodeInLod, *Mesh->GetName(), RF_Public | RF_Standalone, TemplateImportData, NewMesh, LODIndex, nullptr);
							ReimportLodList.Add(LODIndex);
							if (NewMesh->IsSourceModelValid(LODIndex))
							{
								NewMesh->GetSourceModel(LODIndex).bImportWithBaseMesh = true;
							}
						}
					}
				}
			}
		}
		else
		{
			LodZeroNodes.Add(Node);
			NewMesh = ImportStaticMesh(Mesh->GetOuter(), Node, *Mesh->GetName(), RF_Public|RF_Standalone, TemplateImportData, Mesh, 0, ExistMeshDataPtr.Get());
			ReimportLodList.Add(0);
		}

		if (NewMesh != nullptr)
		{
			PostImportStaticMesh(NewMesh, LodZeroNodes);
		}
		
	}
	else
	{
		// no FBX mesh match, maybe the Unreal mesh is imported from multiple FBX mesh (enable option "Import As Single")
		if (FbxMeshArray.Num() > 0)
		{
			NewMesh = ImportStaticMeshAsSingle(Mesh->GetOuter(), FbxMeshArray, *Mesh->GetName(), RF_Public|RF_Standalone, TemplateImportData, Mesh, 0, ExistMeshDataPtr.Get());
			ReimportLodList.Add(0);
			if (NewMesh != nullptr)
			{
				PostImportStaticMesh(NewMesh, FbxMeshArray);
			}
		}
		else // no mesh found in the FBX file
		{
			AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("Error_NoFBXMeshFound", "No FBX mesh found when reimport Unreal mesh '{0}'. The FBX file is crashed."), FText::FromString(Mesh->GetName()))), FFbxErrors::Generic_Mesh_MeshNotFound);
		}
	}
	if (NewMesh != nullptr)
	{
		StaticMeshImportUtils::UpdateSomeLodsImportMeshData(NewMesh, &ReimportLodList);
		StaticMeshImportUtils::RestoreExistingMeshData(ExistMeshDataPtr, NewMesh, INDEX_NONE, ImportOptions->bCanShowDialog, ImportOptions->bResetToFbxOnMaterialConflict);
		RestoreData.CleanupDuplicateMesh();
	}
	else
	{
		RestoreData.RestoreMesh(this);
	}
	return NewMesh;
}

void UnFbx::FFbxImporter::VerifyGeometry(UStaticMesh* StaticMesh)
{
	// Calculate bounding box to check if too small
	{
		FVector Center, Extents;
		ComputeBoundingBox(StaticMesh, Center, Extents);

		if (Extents.GetAbsMax() < 5.f)
		{
			AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, LOCTEXT("Prompt_MeshVerySmall", "Warning: The imported mesh is very small. This is most likely an issue with the units used when exporting to FBX.")), FFbxErrors::Generic_Mesh_SmallGeometry);
		}
	}
}

UStaticMesh* UnFbx::FFbxImporter::ImportStaticMeshAsSingle(UObject* InParent, TArray<FbxNode*>& MeshNodeArray, const FName InName, EObjectFlags Flags, UFbxStaticMeshImportData* TemplateImportData, UStaticMesh* InStaticMesh, int LODIndex, const FExistingStaticMeshData* ExistMeshDataPtr)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FbxImporter::ImportStaticMeshAsSingle);
	FFbxScopedOperation FbxScopedImportOperation(this);
	bool bBuildStatus = true;

	// Make sure rendering is done - so we are not changing data being used by collision drawing.
	FlushRenderingCommands();

	if (MeshNodeArray.Num() == 0)
	{
		return NULL;
	}
	
	// Count the number of verts
	int32 NumVerts = 0;
	for (int32 MeshIndex = 0; MeshIndex < MeshNodeArray.Num(); MeshIndex++ )
	{
		FbxNode* Node = MeshNodeArray[MeshIndex];
		FbxMesh* FbxMesh = Node->GetMesh();

		if (FbxMesh)
		{
			NumVerts += FbxMesh->GetControlPointsCount();

			// If not combining meshes, reset the vert count between meshes
			if (!ImportOptions->bCombineToSingle)
			{
				NumVerts = 0;
			}
		}
	}

	Parent = InParent;
	
	FString MeshName = ObjectTools::SanitizeObjectName(InName.ToString());

	// warning for missing smoothing group info
	CheckSmoothingInfo(MeshNodeArray[0]->GetMesh());

	// Parent package to place new meshes
	UPackage* Package = NULL;
	if (ImportOptions->bImportScene && InParent != nullptr && InParent->IsA(UPackage::StaticClass()))
	{
		Package = StaticCast<UPackage*>(InParent);
	}

	// create empty mesh
	UStaticMesh*	StaticMesh = NULL;
	UStaticMesh* ExistingMesh = NULL;
	UObject* ExistingObject = NULL;

	// A mapping of vertex positions to their color in the existing static mesh
	TMap<FVector3f, FColor>		ExistingVertexColorData;

	EVertexColorImportOption::Type VertexColorImportOption = ImportOptions->VertexColorImportOption;
	FString NewPackageName;

	if( InStaticMesh == NULL || LODIndex == 0 )
	{
		// Create a package for each mesh
		if (Package == nullptr)
		{
			if (Parent != nullptr && Parent->GetOutermost() != nullptr)
			{
				NewPackageName = FPackageName::GetLongPackagePath(Parent->GetOutermost()->GetName()) + TEXT("/") + MeshName;
			}
			else
			{
				AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("ImportStaticMeshAsSingle", "Invalid Parent package when importing {0}.\nThe asset will not be imported."), FText::FromString(MeshName))), FFbxErrors::Generic_ImportingNewObjectFailed);
				return nullptr;
			}
			NewPackageName = UPackageTools::SanitizePackageName(NewPackageName);
			Package = CreatePackage( *NewPackageName);
		}
		Package->FullyLoad();

		ExistingMesh = FindObject<UStaticMesh>( Package, *MeshName );
		ExistingObject = FindObject<UObject>( Package, *MeshName );		
	}

	if (ExistingMesh)
	{
		ExistingMesh->GetVertexColorData(ExistingVertexColorData);

		// Free any RHI resources for existing mesh before we re-create in place.
		ExistingMesh->PreEditChange(NULL);

		//When we do a reimport the nanite static mesh import setting should not affect the asset since its a "reimportrestrict" option
		//In that case we override the import setting
		ImportOptions->bBuildNanite = ExistingMesh->IsNaniteEnabled();
	}
	else if (ExistingObject)
	{
		// Replacing an object.  Here we go!
		// Delete the existing object
		bool bDeleteSucceeded = ObjectTools::DeleteSingleObject( ExistingObject );

		if (bDeleteSucceeded)
		{
			// Force GC so we can cleanly create a new asset (and not do an 'in place' replacement)
			CollectGarbage( GARBAGE_COLLECTION_KEEPFLAGS );

			// Create a package for each mesh
			Package = CreatePackage( *NewPackageName);

			// Require the parent because it will have been invalidated from the garbage collection
			Parent = Package;
		}
		else
		{
			// failed to delete
			AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("ContentBrowser_CannotDeleteReferenced", "{0} wasn't created.\n\nThe asset is referenced by other content."), FText::FromString(MeshName))), FFbxErrors::Generic_CannotDeleteReferenced);
			return NULL;
		}
	}
	
	if( InStaticMesh != NULL && LODIndex > 0 )
	{
		StaticMesh = InStaticMesh;
		//Only cancel the operation if we are creating a new asset, as we don't support existing asset restoration yet.
		ImportOptions->bIsImportCancelable = false;
	}
	else
	{
		StaticMesh = NewObject<UStaticMesh>(Package, FName(*MeshName), Flags | RF_Public);
		CreatedObjects.Add(StaticMesh);
	}

	if (StaticMesh->GetNumSourceModels() < LODIndex+1)
	{
		// Add one LOD 
		StaticMesh->AddSourceModel();
		
		if (StaticMesh->GetNumSourceModels() < LODIndex+1)
		{
			LODIndex = StaticMesh->GetNumSourceModels() - 1;
		}
	}

	FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription(LODIndex);
	if (MeshDescription == nullptr)
	{
		MeshDescription = StaticMesh->CreateMeshDescription(LODIndex);
		check(MeshDescription != nullptr);

		UStaticMesh::FCommitMeshDescriptionParams Params;
		Params.bUseHashAsGuid = true;
		StaticMesh->CommitMeshDescription(LODIndex, Params);

		//Make sure an imported mesh do not get reduce if there was no mesh data before reimport.
		//In this case we have a generated LOD convert to a custom LOD
		FStaticMeshSourceModel& SrcModel = StaticMesh->GetSourceModel(LODIndex);
		SrcModel.ResetReductionSetting();
	}
	else if (InStaticMesh != NULL && LODIndex > 0)
	{
		// clear out the old mesh data
		MeshDescription->Empty();
	}

	FStaticMeshAttributes Attributes(*MeshDescription);

	// make sure it has a new lighting guid
	StaticMesh->SetLightingGuid();

	// Set it to use textured lightmaps. Note that Build Lighting will do the error-checking (texcoordindex exists for all LODs, etc).
	StaticMesh->SetLightMapResolution(64);
	StaticMesh->SetLightMapCoordinateIndex(1);

	float SqrBoundingBoxThreshold = THRESH_POINTS_ARE_NEAR * THRESH_POINTS_ARE_NEAR;

	bool bAllDegenerated = true;
	TArray<FFbxMaterial> MeshMaterials;
	int32 NodeFailCount = 0;
	
	FScopedSlowTask BuildMeshSlowTask(MeshNodeArray.Num(), FText::Format(LOCTEXT("FbxStaticMeshBuildGeometryTask", "Importing Static Mesh Geometries {0} of {1}."), FText::AsNumber(0), FText::AsNumber(MeshNodeArray.Num())));
	BuildMeshSlowTask.MakeDialog();
	for (int32 MeshIndex = 0; MeshIndex < MeshNodeArray.Num(); MeshIndex++)
	{
		BuildMeshSlowTask.EnterProgressFrame(1, FText::Format(LOCTEXT("FbxStaticMeshBuildGeometryTaskProgress", "Importing Static Mesh Geometry {0} of {1}."), FText::AsNumber(MeshIndex+1), FText::AsNumber(MeshNodeArray.Num())));
		FbxNode* Node = MeshNodeArray[MeshIndex];
		if (Node->GetMesh())
		{
			Node->GetMesh()->ComputeBBox();
			
			FbxVector4 GlobalScale = Node->EvaluateGlobalTransform(0).GetS();
			FbxVector4 BBoxMax = FbxVector4(Node->GetMesh()->BBoxMax.Get()) * GlobalScale;
			FbxVector4 BBoxMin = FbxVector4(Node->GetMesh()->BBoxMin.Get()) * GlobalScale;
			FbxVector4 BoxExtend = BBoxMax - BBoxMin;
			double SqrSize = BoxExtend.SquareLength();
			//If the bounding box of the mesh part is smaller then the position threshold, the part is consider degenerated and will be skip.
			if (SqrSize > SqrBoundingBoxThreshold)
			{
				bAllDegenerated = false;

				if(ImportOptions->bIsImportCancelable && BuildMeshSlowTask.ShouldCancel())
				{
					bImportOperationCanceled = true; 
					bBuildStatus = false;
					break;
				}

				if (!BuildStaticMeshFromGeometry(Node, StaticMesh, MeshMaterials, LODIndex,
					VertexColorImportOption, ExistingVertexColorData, ImportOptions->VertexOverrideColor))
				{
					// If this FBX node failed to build, make a note and continue.
					// We should only fail the mesh/LOD import if every node failed.
					NodeFailCount++;
				}
			}
			else
			{
				CreateTokenizedErrorForDegeneratedPart(this, StaticMesh->GetName(), UTF8_TO_TCHAR(Node->GetName()));
				NodeFailCount++;
			}
		}
	}

	// If every node in the list failed to build, this counts as a failure to import the entire mesh/LOD.
	if (NodeFailCount == MeshNodeArray.Num() || bAllDegenerated)
	{
		bBuildStatus = false;
	}

	if (bBuildStatus)
	{
		TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();
		int32 FirstOpenUVChannel = VertexInstanceUVs.GetNumChannels() >= MAX_MESH_TEXTURE_COORDS_MD ? 1 : VertexInstanceUVs.GetNumChannels();
		TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();

		TArray<FStaticMaterial> MaterialToAdd;
		for (const FPolygonGroupID PolygonGroupID : MeshDescription->PolygonGroups().GetElementIDs())
		{
			const FName& ImportedMaterialSlotName = PolygonGroupImportedMaterialSlotNames[PolygonGroupID];
			const FString ImportedMaterialSlotNameString = ImportedMaterialSlotName.ToString();
			const FName MaterialSlotName = ImportedMaterialSlotName;
			int32 MaterialIndex = INDEX_NONE;
			for (int32 FbxMaterialIndex = 0; FbxMaterialIndex < MeshMaterials.Num(); ++FbxMaterialIndex)
			{
				FFbxMaterial& FbxMaterial = MeshMaterials[FbxMaterialIndex];
				if (FbxMaterial.GetName().Equals(ImportedMaterialSlotNameString))
				{
					MaterialIndex = FbxMaterialIndex;
					break;
				}
			}
			if (MaterialIndex == INDEX_NONE)
			{
				MaterialIndex = PolygonGroupID.GetValue();
			}
			UMaterialInterface* Material = MeshMaterials.IsValidIndex(MaterialIndex) ? MeshMaterials[MaterialIndex].Material : UMaterial::GetDefaultMaterial(MD_Surface);
			FStaticMaterial StaticMaterial(Material, MaterialSlotName, ImportedMaterialSlotName);
			MaterialToAdd.Add(StaticMaterial);
		}

		{
			//Insert the new materials in the static mesh
			//The build function will search for imported slot name to find the appropriate slot
			for (int32 MaterialToAddIndex = 0; MaterialToAddIndex < MaterialToAdd.Num(); ++MaterialToAddIndex)
			{
				const FStaticMaterial& CandidateMaterial = MaterialToAdd[MaterialToAddIndex];
				bool FoundExistingMaterial = false;
				//Found matching existing material
				for (int32 StaticMeshMaterialIndex = 0; StaticMeshMaterialIndex < StaticMesh->GetStaticMaterials().Num(); ++StaticMeshMaterialIndex)
				{
					const FStaticMaterial& StaticMeshMaterial = StaticMesh->GetStaticMaterials()[StaticMeshMaterialIndex];
					if (StaticMeshMaterial.ImportedMaterialSlotName == CandidateMaterial.ImportedMaterialSlotName)
					{
						FoundExistingMaterial = true;
						break;
					}
				}
				if (!FoundExistingMaterial)
				{
					StaticMesh->GetStaticMaterials().Add(CandidateMaterial);
				}
			}
			
			//Set the Section Info Map to fit the real StaticMaterials array
			int32 SectionIndex = 0;
			for (const FPolygonGroupID PolygonGroupID : MeshDescription->PolygonGroups().GetElementIDs())
			{
				const FName& ImportedMaterialSlotName = PolygonGroupImportedMaterialSlotNames[PolygonGroupID];
				int32 MaterialIndex = INDEX_NONE;
				for (int32 FbxMaterialIndex = 0; FbxMaterialIndex < StaticMesh->GetStaticMaterials().Num(); ++FbxMaterialIndex)
				{
					FName& StaticMaterialName = StaticMesh->GetStaticMaterials()[FbxMaterialIndex].ImportedMaterialSlotName;
					if (StaticMaterialName == ImportedMaterialSlotName)
					{
						MaterialIndex = FbxMaterialIndex;
						break;
					}
				}
				if (MaterialIndex == INDEX_NONE)
				{
					if (LODIndex > 0 && ExistMeshDataPtr != nullptr)
					{
						//Do not add Material slot when reimporting a LOD just use the index found in the fbx if valid or use the last MaterialSlot index
						MaterialIndex = StaticMesh->GetStaticMaterials().Num() - 1;
					}
					else
					{
						MaterialIndex = PolygonGroupID.GetValue();
					}
				}
				FMeshSectionInfo Info = StaticMesh->GetSectionInfoMap().Get(LODIndex, SectionIndex);
				Info.MaterialIndex = MaterialIndex;
				StaticMesh->GetSectionInfoMap().Remove(LODIndex, SectionIndex);
				StaticMesh->GetSectionInfoMap().Set(LODIndex, SectionIndex, Info);
				SectionIndex++;
			}
		}

		//Set the original mesh description to be able to do non destructive reduce
		UStaticMesh::FCommitMeshDescriptionParams Params;
		Params.bUseHashAsGuid = true;
		StaticMesh->CommitMeshDescription(LODIndex, Params);

		// Setup default LOD settings based on the selected LOD group.
		if (LODIndex == 0)
		{
			ITargetPlatform* CurrentPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
			check(CurrentPlatform);
			const FStaticMeshLODGroup& LODGroup = CurrentPlatform->GetStaticMeshLODSettings().GetLODGroup(ImportOptions->StaticMeshLODGroup);
			int32 NumLODs = LODGroup.GetDefaultNumLODs();
			while (StaticMesh->GetNumSourceModels() < NumLODs)
			{
				StaticMesh->AddSourceModel();
			}
			for (int32 ModelLODIndex = 0; ModelLODIndex < NumLODs; ++ModelLODIndex)
			{
				StaticMesh->GetSourceModel(ModelLODIndex).ReductionSettings = LODGroup.GetDefaultSettings(ModelLODIndex);
			}
			StaticMesh->SetLightMapResolution(LODGroup.GetDefaultLightMapResolution());

			//@third party BEGIN SIMPLYGON
			/* ImportData->Update(UFactory::GetCurrentFilename());
			Developer Note: Update method above computed Hash internally. Hash is calculated based on the file size.
			Doing this for CAD files with thousands of components hugely increases the time.
			The following method uses a precomputed hash (once per file). Huge time savings.
			*/
			UFbxStaticMeshImportData* ImportData = UFbxStaticMeshImportData::GetImportDataForStaticMesh(StaticMesh, TemplateImportData);
			FString FactoryCurrentFileName = UFactory::GetCurrentFilename();
			if (!FactoryCurrentFileName.IsEmpty())
			{
				//The factory is instantiate only when importing or re-importing the LOD 0
				//The LOD re-import is not using the factory so the static function UFactory::GetCurrentFilename()
				//will return the last fbx imported asset name or no name if there was no imported asset before.
				ImportData->Update(FactoryCurrentFileName, UFactory::GetFileHash());
			}
			//@third party END SIMPLYGON
		}

		// @todo This overrides restored values currently but we need to be able to import over the existing settings if the user chose to do so.
		FStaticMeshSourceModel& SrcModel = StaticMesh->GetSourceModel(LODIndex);
		SrcModel.BuildSettings.bRemoveDegenerates = ImportOptions->bRemoveDegenerates;
		SrcModel.BuildSettings.bBuildReversedIndexBuffer = ImportOptions->bBuildReversedIndexBuffer;
		SrcModel.BuildSettings.bRecomputeNormals = ImportOptions->NormalImportMethod == FBXNIM_ComputeNormals;
		SrcModel.BuildSettings.bRecomputeTangents = ImportOptions->NormalImportMethod != FBXNIM_ImportNormalsAndTangents;
		SrcModel.BuildSettings.bUseMikkTSpace = (ImportOptions->NormalGenerationMethod == EFBXNormalGenerationMethod::MikkTSpace) && (!ImportOptions->ShouldImportNormals() || !ImportOptions->ShouldImportTangents());
		SrcModel.BuildSettings.bComputeWeightedNormals = ImportOptions->bComputeWeightedNormals;

		StaticMesh->GetNaniteSettings().bEnabled = ImportOptions->bBuildNanite;
		SrcModel.BuildSettings.DistanceFieldResolutionScale = ImportOptions->DistanceFieldResolutionScale;

		if (ImportOptions->bGenerateLightmapUVs)
		{
			SrcModel.BuildSettings.bGenerateLightmapUVs = true;
			SrcModel.BuildSettings.DstLightmapIndex = FirstOpenUVChannel;
			StaticMesh->SetLightMapCoordinateIndex(FirstOpenUVChannel);
		}
		else
		{
			SrcModel.BuildSettings.bGenerateLightmapUVs = false;
		}

		//LODGroup should never change during a re-import or when we import a LOD > 0
		if (LODIndex == 0 && InStaticMesh == nullptr)
		{
			StaticMesh->GetLODGroup() = ImportOptions->StaticMeshLODGroup;
		}

		//Set the Imported version before calling the build
		//We set it here because the remap index is build in RestoreExistingMeshSettings call before the build
		StaticMesh->SetImportVersion(EImportStaticMeshVersion::LastVersion);

		if (ExistMeshDataPtr && InStaticMesh)
		{
			// Patch existing mesh settings to reflect new parameters from import settings
			FExistingStaticMeshData* MutableExistMeshDataPtr = const_cast<FExistingStaticMeshData*>(ExistMeshDataPtr);
			FMeshBuildSettings& ExistingBuildSettings = MutableExistMeshDataPtr->ExistingLODData[LODIndex].ExistingBuildSettings;
			ExistingBuildSettings.bRemoveDegenerates = SrcModel.BuildSettings.bRemoveDegenerates;
			ExistingBuildSettings.bBuildReversedIndexBuffer = SrcModel.BuildSettings.bBuildReversedIndexBuffer;
			ExistingBuildSettings.bRecomputeNormals = SrcModel.BuildSettings.bRecomputeNormals;
			ExistingBuildSettings.bRecomputeTangents = SrcModel.BuildSettings.bRecomputeTangents;
			ExistingBuildSettings.bUseMikkTSpace = SrcModel.BuildSettings.bUseMikkTSpace;
			ExistingBuildSettings.bComputeWeightedNormals = SrcModel.BuildSettings.bComputeWeightedNormals;
			ExistingBuildSettings.bGenerateLightmapUVs = SrcModel.BuildSettings.bGenerateLightmapUVs;
			ExistingBuildSettings.DstLightmapIndex = SrcModel.BuildSettings.DstLightmapIndex;
			MutableExistMeshDataPtr->ExistingLightMapCoordinateIndex = SrcModel.BuildSettings.DstLightmapIndex;
			MutableExistMeshDataPtr->ExistingNaniteSettings.bEnabled = StaticMesh->IsNaniteEnabled();

			StaticMeshImportUtils::RestoreExistingMeshSettings(ExistMeshDataPtr, InStaticMesh, StaticMesh->GetLODGroup() != NAME_None ? INDEX_NONE : LODIndex);
		}

		// The code to check for bad lightmap UVs doesn't scale well with number of triangles.
		// Skip it here because Lightmass will warn about it during a light build anyway.
		bool bWarnOnBadLightmapUVs = false;
		if (bWarnOnBadLightmapUVs)
		{
			TArray< FString > MissingUVSets;
			TArray< FString > BadUVSets;
			TArray< FString > ValidUVSets;
			UStaticMesh::CheckLightMapUVs(StaticMesh, MissingUVSets, BadUVSets, ValidUVSets);

			// NOTE: We don't care about missing UV sets here, just bad ones!
			if (BadUVSets.Num() > 0)
			{
				AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("Error_UVSetLayoutProblem", "Warning: The light map UV set for static mesh '{0}' appears to have layout problems.  Either the triangle UVs are overlapping one another or the UV are out of bounds (0.0 - 1.0 range.)"), FText::FromString(MeshName))), FFbxErrors::StaticMesh_UVSetLayoutProblem);
			}
		}
	}
	else
	{
		// If we couldn't build the static mesh, its package is invalid. We should reject it entirely to prevent issues from arising from trying to use it in the editor.
		if (!NewPackageName.IsEmpty())
		{
			Package->SetDirtyFlag(false);
			Package->RemoveFromRoot();
			Package->ConditionalBeginDestroy();
		}

		StaticMesh = NULL;
	}

	if (StaticMesh)
	{
		ImportStaticMeshLocalSockets(StaticMesh, MeshNodeArray);

		for (FbxNode* Node : MeshNodeArray)
		{
			ImportNodeCustomProperties(StaticMesh, Node);
		}
	}

	return StaticMesh;
}

void ReorderMaterialAfterImport(UStaticMesh* StaticMesh, TArray<FbxNode*>& MeshNodeArray, bool bAllowFbxReorder)
{
	if (StaticMesh == nullptr)
	{
		return;
	}
	TArray<FString> MeshMaterials;
	for (int32 MeshIndex = 0; MeshIndex < MeshNodeArray.Num(); MeshIndex++)
	{
		FbxNode* Node = MeshNodeArray[MeshIndex];
		if (Node->GetMesh())
		{
			int32 MaterialCount = Node->GetMaterialCount();

			for (int32 MaterialIndex = 0; MaterialIndex < MaterialCount; MaterialIndex++)
			{
				//Get the original fbx import name
				FbxSurfaceMaterial *FbxMaterial = Node->GetMaterial(MaterialIndex);
				FString FbxMaterialName = FbxMaterial ? ANSI_TO_TCHAR(FbxMaterial->GetName()) : TEXT("None");
				if (!MeshMaterials.Contains(FbxMaterialName))
				{
					MeshMaterials.Add(FbxMaterialName);
				}
			}
		}
	}

	//There is no material in the fbx node
	if (MeshMaterials.Num() < 1)
	{
		return;
	}

	//If there is some skinxx material name we will reorder the material to follow the skinxx workflow instead of the fbx order
	bool IsUsingSkinxxWorkflow = true;
	TArray<FString> MeshMaterialsSkinXX;
	MeshMaterialsSkinXX.AddZeroed(MeshMaterials.Num());
	for (int32 FbxMaterialIndex = 0; FbxMaterialIndex < MeshMaterials.Num(); ++FbxMaterialIndex)
	{
		const FString &FbxMaterialName = MeshMaterials[FbxMaterialIndex];
		//If we have all skinxx material name we have to re-order to skinxx workflow
		int32 Offset = FbxMaterialName.Find(TEXT("_SKIN"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
		if (Offset == INDEX_NONE)
		{
			IsUsingSkinxxWorkflow = false;
			MeshMaterialsSkinXX.Empty();
			break;
		}
		int32 SkinIndex = INDEX_NONE;
		// Chop off the material name so we are left with the number in _SKINXX
		FString SkinXXNumber = FbxMaterialName.Right(FbxMaterialName.Len() - (Offset + 1)).RightChop(4);
		if (SkinXXNumber.IsNumeric())
		{
			SkinIndex = FPlatformString::Atoi(*SkinXXNumber);
		}

		if (SkinIndex >= MeshMaterialsSkinXX.Num())
		{
			MeshMaterialsSkinXX.AddZeroed((SkinIndex + 1) - MeshMaterialsSkinXX.Num());
		}
		if (MeshMaterialsSkinXX.IsValidIndex(SkinIndex))
		{
			MeshMaterialsSkinXX[SkinIndex] = FbxMaterialName;
		}
		else
		{
			//Cannot reorder this item
			IsUsingSkinxxWorkflow = false;
			MeshMaterialsSkinXX.Empty();
			break;
		}
	}

	if (IsUsingSkinxxWorkflow)
	{
		//Shrink the array to valid entry, in case the skinxx has some hole like _skin[01, 02, 04, 05...]
		for (int32 FbxMaterialIndex = MeshMaterialsSkinXX.Num() - 1; FbxMaterialIndex >= 0; --FbxMaterialIndex)
		{
			const FString &FbxMaterial = MeshMaterialsSkinXX[FbxMaterialIndex];
			if (FbxMaterial.IsEmpty())
			{
				MeshMaterialsSkinXX.RemoveAt(FbxMaterialIndex);
			}
		}
		//Replace the fbx ordered materials by the skinxx ordered material
		MeshMaterials = MeshMaterialsSkinXX;
	}

	//Reorder the StaticMaterials array to reflect the order in the fbx file
	if (bAllowFbxReorder)
	{
		//So we make sure the order reflect the material ID in the DCCs
		FMeshSectionInfoMap OldSectionInfoMap = StaticMesh->GetSectionInfoMap();
		TArray<int32> FbxRemapMaterials;
		TArray<FStaticMaterial> NewStaticMaterials;
		for (int32 FbxMaterialIndex = 0; FbxMaterialIndex < MeshMaterials.Num(); ++FbxMaterialIndex)
		{
			const FString &FbxMaterial = MeshMaterials[FbxMaterialIndex];
			int32 FoundMaterialIndex = INDEX_NONE;
			for (int32 BuildMaterialIndex = 0; BuildMaterialIndex < StaticMesh->GetStaticMaterials().Num(); ++BuildMaterialIndex)
			{
				FStaticMaterial &BuildMaterial = StaticMesh->GetStaticMaterials()[BuildMaterialIndex];
				if (FbxMaterial.Compare(BuildMaterial.ImportedMaterialSlotName.ToString()) == 0)
				{
					FoundMaterialIndex = BuildMaterialIndex;
					break;
				}
			}

			if (FoundMaterialIndex != INDEX_NONE)
			{
				FbxRemapMaterials.Add(FoundMaterialIndex);
				NewStaticMaterials.Add(StaticMesh->GetStaticMaterials()[FoundMaterialIndex]);
			}
		}
		//Add the materials not used by the LOD 0 at the end of the array. The order here is irrelevant since it can be used by many LOD other then LOD 0 and in different order
		for (int32 BuildMaterialIndex = 0; BuildMaterialIndex < StaticMesh->GetStaticMaterials().Num(); ++BuildMaterialIndex)
		{
			const FStaticMaterial &StaticMaterial = StaticMesh->GetStaticMaterials()[BuildMaterialIndex];
			bool bFoundMaterial = false;
			for (const FStaticMaterial &BuildMaterial : NewStaticMaterials)
			{
				if (StaticMaterial == BuildMaterial)
				{
					bFoundMaterial = true;
					break;
				}
			}
			if (!bFoundMaterial)
			{
				FbxRemapMaterials.Add(BuildMaterialIndex);
				NewStaticMaterials.Add(StaticMaterial);
			}
		}

		StaticMesh->GetStaticMaterials().Empty();
		for (const FStaticMaterial &BuildMaterial : NewStaticMaterials)
		{
			StaticMesh->GetStaticMaterials().Add(BuildMaterial);
		}

		//Remap the material instance of the staticmaterial array and remap the material index of all sections
		for (int32 LODResoureceIndex = 0; LODResoureceIndex < StaticMesh->GetRenderData()->LODResources.Num(); ++LODResoureceIndex)
		{
			FStaticMeshLODResources& LOD = StaticMesh->GetRenderData()->LODResources[LODResoureceIndex];
			int32 NumSections = LOD.Sections.Num();
			for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
			{
				FMeshSectionInfo Info = OldSectionInfoMap.Get(LODResoureceIndex, SectionIndex);
				int32 RemapIndex = FbxRemapMaterials.Find(Info.MaterialIndex);
				if (StaticMesh->GetStaticMaterials().IsValidIndex(RemapIndex))
				{
					Info.MaterialIndex = RemapIndex;
					StaticMesh->GetSectionInfoMap().Set(LODResoureceIndex, SectionIndex, Info);
					StaticMesh->GetOriginalSectionInfoMap().Set(LODResoureceIndex, SectionIndex, Info);
				}
			}
		}
	}
}

void UnFbx::FFbxImporter::PostImportStaticMesh(UStaticMesh* StaticMesh, TArray<FbxNode*>& MeshNodeArray, int32 LODIndex)
{
	if (StaticMesh == nullptr)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FbxImporter::PostImportStaticMesh);

	// Build the staticmesh, we move the build here because we want to avoid building the staticmesh for every LOD
	// when we import the mesh.
	TArray<FText> BuildErrors;
	if (GIsAutomationTesting)
	{
		//Generate a random GUID to be sure it rebuild the asset
		StaticMesh->BuildCacheAutomationTestGuid = FGuid::NewGuid();
		//Avoid distance field calculation in automation test setting this to false is not suffisant since the condition OR with the CVar
		//But fbx automation test turn off the CVAR
		StaticMesh->bGenerateMeshDistanceField = false;
	}

	static const auto CVarDistanceField = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.GenerateMeshDistanceFields"));
	int32 OriginalCVarDistanceFieldValue = CVarDistanceField->GetValueOnGameThread();
	IConsoleVariable* CVarDistanceFieldInterface = IConsoleManager::Get().FindConsoleVariable(TEXT("r.GenerateMeshDistanceFields"));
	bool bOriginalGenerateMeshDistanceField = StaticMesh->bGenerateMeshDistanceField;
	
	//Prebuild the static mesh when we use LodGroup and we want to modify the LodNumber
	if (!ImportOptions->bImportScene)
	{
		//Set the minimum LOD
		if (ImportOptions->MinimumLodNumber > 0)
		{
			StaticMesh->SetMinLODIdx(ImportOptions->MinimumLodNumber);
		}

		//User specify a number of LOD.
		if (ImportOptions->LodNumber > 0)
		{
			//In case we plan to change the LodNumber we will build the static mesh 2 time
			//We have to disable the distance field calculation so it get calculated only during the second build
			bool bSpecifiedLodGroup = ImportOptions->StaticMeshLODGroup != NAME_None;
			if (bSpecifiedLodGroup)
			{
				//Avoid building the distance field when we prebuild
				if (OriginalCVarDistanceFieldValue != 0 && CVarDistanceFieldInterface)
				{
					//Hack we change the distance field user console variable to control the build, but we put back the value after the first build
					CVarDistanceFieldInterface->SetWithCurrentPriority(0);
				}
				StaticMesh->bGenerateMeshDistanceField = false;

				StaticMesh->Build(false, &BuildErrors);
				for (FText& Error : BuildErrors)
				{
					AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, Error), FFbxErrors::StaticMesh_BuildError);
				}

				StaticMesh->bGenerateMeshDistanceField = bOriginalGenerateMeshDistanceField;
				if (OriginalCVarDistanceFieldValue != 0 && CVarDistanceFieldInterface)
				{
					CVarDistanceFieldInterface->SetWithCurrentPriority(OriginalCVarDistanceFieldValue);
				}
			}

			//Set the Number of LODs, this has to be done after we build the specified LOD Group
			int32 LODCount = ImportOptions->LodNumber;
			if (LODCount < 0)
			{
				LODCount = 0;
			}
			if (LODCount > MAX_STATIC_MESH_LODS)
			{
				LODCount = MAX_STATIC_MESH_LODS;
			}

			StaticMesh->SetNumSourceModels(LODCount);
		}
	}

	StaticMesh->Build(false, &BuildErrors);
	for (FText& Error : BuildErrors)
	{
		AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, Error), FFbxErrors::StaticMesh_BuildError);
	}

	//Make sure the light map UVChannel is valid, this must be done after the build
	StaticMesh->EnforceLightmapRestrictions();

	//Set the specified LOD distances for every LODs we have to do this after the build in case there is a specified Lod Group
	if (!ImportOptions->bAutoComputeLodDistances && !ImportOptions->bImportScene)
	{
		StaticMesh->SetAutoComputeLODScreenSize(false);

		for (int32 LodIndex = 0; LodIndex < StaticMesh->GetNumSourceModels(); ++LodIndex)
		{
			FStaticMeshSourceModel& StaticMeshSourceModel = StaticMesh->GetSourceModel(LodIndex);
			StaticMeshSourceModel.ScreenSize = ImportOptions->LodDistances.IsValidIndex(LodIndex) ? ImportOptions->LodDistances[LodIndex] : 0.0f;
		}
	}

	// this is damage control. After build, we'd like to absolutely sure that 
	// all index is pointing correctly and they're all used. Otherwise we remove them
	FMeshSectionInfoMap TempOldSectionInfoMap = StaticMesh->GetSectionInfoMap();
	StaticMesh->GetSectionInfoMap().Clear();
	StaticMesh->GetOriginalSectionInfoMap().Clear();
	if (StaticMesh->GetRenderData())
	{
		// fix up section data
		for (int32 LODResoureceIndex = 0; LODResoureceIndex < StaticMesh->GetRenderData()->LODResources.Num(); ++LODResoureceIndex)
		{
			FStaticMeshLODResources& LOD = StaticMesh->GetRenderData()->LODResources[LODResoureceIndex];
			int32 NumSections = LOD.Sections.Num();
			for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
			{
				FMeshSectionInfo Info = TempOldSectionInfoMap.Get(LODResoureceIndex, SectionIndex);
				if (StaticMesh->GetStaticMaterials().IsValidIndex(Info.MaterialIndex))
				{
					StaticMesh->GetSectionInfoMap().Set(LODResoureceIndex, SectionIndex, Info);
					StaticMesh->GetOriginalSectionInfoMap().Set(LODResoureceIndex, SectionIndex, Info);
				}
			}
		}
	}

	//collision generation must be done after the build, this will ensure a valid BodySetup
	if (StaticMesh->GetCustomizedCollision() == false && ImportOptions->bAutoGenerateCollision && StaticMesh->GetBodySetup() && LODIndex == 0)
	{
		FKAggregateGeom & AggGeom = StaticMesh->GetBodySetup()->AggGeom;
		AggGeom.ConvexElems.Empty(1);	//if no custom collision is setup we just regenerate collision when reimport

		const int32 NumDirs = 18;
		TArray<FVector> Dirs;
		Dirs.AddUninitialized(NumDirs);
		for (int32 DirIdx = 0; DirIdx < NumDirs; ++DirIdx) { Dirs[DirIdx] = KDopDir18[DirIdx]; }
		GenerateKDopAsSimpleCollision(StaticMesh, Dirs);
	}

	// Refresh collision change back to staticmesh components, this must be done after the build.
	// Some derived components rely on data computed in the build for collision computation. For example,
	// USplineMeshComponent uses the static mesh's extended bounds in its convex collision computation.
	RefreshCollisionChangeComponentsOnly(*StaticMesh);

	//If there is less the 2 materials in the fbx file there is no need to reorder them
	//If we have import a LOD other then the base, the material array cannot be sorted, because only the base LOD reorder the material array
	if (LODIndex == 0 && StaticMesh->GetStaticMaterials().Num() > 1)
	{
		ReorderMaterialAfterImport(StaticMesh, MeshNodeArray, ImportOptions->bReorderMaterialToFbxOrder);
	}

#if WITH_EDITOR
	FMeshBudgetProjectSettingsUtils::SetLodGroupForStaticMesh(StaticMesh);
#endif
}

void UnFbx::FFbxImporter::UpdateStaticMeshImportData(UStaticMesh *StaticMesh, UFbxStaticMeshImportData* StaticMeshImportData)
{
	if (StaticMesh == nullptr || StaticMesh->GetRenderData() == nullptr)
	{
		return;
	}
	UFbxStaticMeshImportData* ImportData = Cast<UFbxStaticMeshImportData>(StaticMesh->GetAssetImportData());
	if (!ImportData && StaticMeshImportData)
	{
		ImportData = UFbxStaticMeshImportData::GetImportDataForStaticMesh(StaticMesh, StaticMeshImportData);
	}

	if (ImportData)
	{
		ImportData->ImportMaterialOriginalNameData.Empty();
		ImportData->ImportMeshLodData.Empty();

		for (const FStaticMaterial &Material : StaticMesh->GetStaticMaterials())
		{
			ImportData->ImportMaterialOriginalNameData.Add(Material.ImportedMaterialSlotName);
		}
		for (int32 LODResoureceIndex = 0; LODResoureceIndex < StaticMesh->GetRenderData()->LODResources.Num(); ++LODResoureceIndex)
		{
			ImportData->ImportMeshLodData.AddZeroed();
			FStaticMeshLODResources& LOD = StaticMesh->GetRenderData()->LODResources[LODResoureceIndex];
			int32 NumSections = LOD.Sections.Num();
			for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
			{
				int32 MaterialLodSectionIndex = LOD.Sections[SectionIndex].MaterialIndex;
				if (StaticMesh->GetSectionInfoMap().GetSectionNumber(LODResoureceIndex) > SectionIndex)
				{
					//In case we have a different ordering then the original fbx order use the sectioninfomap
					const FMeshSectionInfo &SectionInfo = StaticMesh->GetSectionInfoMap().Get(LODResoureceIndex, SectionIndex);
					MaterialLodSectionIndex = SectionInfo.MaterialIndex;
				}
				if (ImportData->ImportMaterialOriginalNameData.IsValidIndex(MaterialLodSectionIndex))
				{
					ImportData->ImportMeshLodData[LODResoureceIndex].SectionOriginalMaterialName.Add(ImportData->ImportMaterialOriginalNameData[MaterialLodSectionIndex]);
				}
				else
				{
					ImportData->ImportMeshLodData[LODResoureceIndex].SectionOriginalMaterialName.Add(TEXT("InvalidMaterialIndex"));
				}
			}
		}
	}
}

struct FbxSocketNode
{
	FName SocketName;
	FbxNode* Node;
};

static void FindMeshSockets( FbxNode* StartNode, TArray<FbxSocketNode>& OutFbxSocketNodes )
{
	if( !StartNode )
	{
		return;
	}

	static const FString SocketPrefix( TEXT("SOCKET_") );
	if( StartNode->GetNodeAttributeCount() > 0 )
	{
		// Find null attributes, they cold be sockets
		FbxNodeAttribute* Attribute = StartNode->GetNodeAttribute();

		if( Attribute != NULL && (Attribute->GetAttributeType() == FbxNodeAttribute::eNull || Attribute->GetAttributeType() == FbxNodeAttribute::eSkeleton))
		{
			// Is this prefixed correctly? If so it is a socket
			FString SocketName = UTF8_TO_TCHAR( StartNode->GetName() );
			if( SocketName.StartsWith( SocketPrefix ) )
			{
				// Remove the prefix from the name
				SocketName.RightChopInline( SocketPrefix.Len(), EAllowShrinking::No );

				FbxSocketNode NewNode;
				NewNode.Node = StartNode;
				NewNode.SocketName = *SocketName;
				OutFbxSocketNodes.Add( NewNode );
			}
		}
	}

	// Recursively examine all children
	for ( int32 ChildIndex=0; ChildIndex < StartNode->GetChildCount(); ++ChildIndex )
	{
		FindMeshSockets( StartNode->GetChild(ChildIndex), OutFbxSocketNodes );
	}
}

void UnFbx::FFbxImporter::ImportStaticMeshLocalSockets(UStaticMesh* StaticMesh, TArray<FbxNode*>& MeshNodeArray)
{
	check(MeshNodeArray.Num());
	FbxNode *MeshRootNode = MeshNodeArray[0];
	const FbxAMatrix &MeshTotalMatrix = ComputeTotalMatrix(MeshRootNode);
	TArray<FbxSocketNode> AllSocketNodes;
	for (FbxNode* RootNode : MeshNodeArray)
	{
		// Find all nodes that are sockets
		TArray<FbxSocketNode> SocketNodes;
		FindMeshSockets(RootNode, SocketNodes);
		for (int32 SocketIndex = 0; SocketIndex < SocketNodes.Num(); ++SocketIndex)
		{
			bool bFoundNewSocket = true;
			for (int32 AllSocketIndex = 0; AllSocketIndex < AllSocketNodes.Num(); ++AllSocketIndex)
			{
				if (AllSocketNodes[AllSocketIndex].SocketName == SocketNodes[SocketIndex].SocketName)
				{
					bFoundNewSocket = false;
					break;
				}
			}
			if (bFoundNewSocket)
			{
				AllSocketNodes.Add(SocketNodes[SocketIndex]);
			}
		}
	}
		
	// Create a UStaticMeshSocket for each fbx socket
	for (int32 SocketIndex = 0; SocketIndex < AllSocketNodes.Num(); ++SocketIndex)
	{
		FbxSocketNode& SocketNode = AllSocketNodes[SocketIndex];

		UStaticMeshSocket* Socket = StaticMesh->FindSocket(SocketNode.SocketName);
		if (!Socket)
		{
			// If the socket didn't exist create a new one now
			Socket = NewObject<UStaticMeshSocket>(StaticMesh);
			Socket->bSocketCreatedAtImport = true;
			check(Socket);

			Socket->SocketName = SocketNode.SocketName;
			StaticMesh->AddSocket(Socket);
		}

		if (Socket)
		{
			const FbxAMatrix& SocketMatrix = Scene->GetAnimationEvaluator()->GetNodeLocalTransform(SocketNode.Node);
			//Remove the axis conversion for the socket since its attach to a mesh containing this conversion.
			FbxAMatrix FinalSocketMatrix = (MeshTotalMatrix * SocketMatrix) * FFbxDataConverter::GetAxisConversionMatrixInv();
			FTransform SocketTransform;
			SocketTransform.SetTranslation(Converter.ConvertPos(FinalSocketMatrix.GetT()));
			SocketTransform.SetRotation(Converter.ConvertRotToQuat(FinalSocketMatrix.GetQ()));
			SocketTransform.SetScale3D(Converter.ConvertScale(FinalSocketMatrix.GetS()));

			Socket->RelativeLocation = SocketTransform.GetLocation();
			Socket->RelativeRotation = SocketTransform.GetRotation().Rotator();
			Socket->RelativeScale = SocketTransform.GetScale3D();
		}
	}
	// Delete mesh sockets that were removed from the import data
	for (int32 MeshSocketIx = 0; MeshSocketIx < StaticMesh->Sockets.Num(); ++MeshSocketIx)
	{
		bool Found = false;
		UStaticMeshSocket* MeshSocket = StaticMesh->Sockets[MeshSocketIx];
		//Do not remove socket that was not generated at import
		if (!MeshSocket->bSocketCreatedAtImport)
		{
			continue;
		}

		for (int32 FbxSocketIx = 0; FbxSocketIx < AllSocketNodes.Num(); FbxSocketIx++)
		{
			if (AllSocketNodes[FbxSocketIx].SocketName == MeshSocket->SocketName)
			{
				Found = true;
				break;
			}
		}
		if (!Found)
		{
			StaticMesh->Sockets.RemoveAt(MeshSocketIx);
			MeshSocketIx--;
		}
	}
}

void UnFbx::FFbxImporter::ImportStaticMeshGlobalSockets( UStaticMesh* StaticMesh )
{
	FbxNode* RootNode = Scene->GetRootNode();

	// Find all nodes that are sockets
	TArray<FbxSocketNode> SocketNodes;
	FindMeshSockets( RootNode, SocketNodes );

	// Create a UStaticMeshSocket for each fbx socket
	for( int32 SocketIndex = 0; SocketIndex < SocketNodes.Num(); ++SocketIndex )
	{
		FbxSocketNode& SocketNode = SocketNodes[ SocketIndex ];

		UStaticMeshSocket* Socket = StaticMesh->FindSocket( SocketNode.SocketName );
		if( !Socket )
		{
			// If the socket didn't exist create a new one now
			Socket = NewObject<UStaticMeshSocket>(StaticMesh);
			check(Socket);

			Socket->SocketName = SocketNode.SocketName;
			StaticMesh->AddSocket(Socket);
			//Remove the axis conversion for the socket since its attach to a mesh containing this conversion.
			const FbxAMatrix& SocketMatrix = Scene->GetAnimationEvaluator()->GetNodeGlobalTransform(SocketNode.Node) * FFbxDataConverter::GetAxisConversionMatrixInv();
			FTransform SocketTransform;
			SocketTransform.SetTranslation(Converter.ConvertPos(SocketMatrix.GetT()));
			SocketTransform.SetRotation(Converter.ConvertRotToQuat(SocketMatrix.GetQ()));
			SocketTransform.SetScale3D(Converter.ConvertScale(SocketMatrix.GetS()));

			Socket->RelativeLocation = SocketTransform.GetLocation();
			Socket->RelativeRotation = SocketTransform.GetRotation().Rotator();
			Socket->RelativeScale = SocketTransform.GetScale3D();

			Socket->bSocketCreatedAtImport = true;
		}
	}
	for (int32 MeshSocketIx = 0; MeshSocketIx < StaticMesh->Sockets.Num(); ++MeshSocketIx)
	{
		bool Found = false;
		UStaticMeshSocket* MeshSocket = StaticMesh->Sockets[MeshSocketIx];
		//Do not remove socket that was not generated at import
		if (!MeshSocket->bSocketCreatedAtImport)
		{
			continue;
		}

		for (int32 FbxSocketIx = 0; FbxSocketIx < SocketNodes.Num(); FbxSocketIx++)
		{
			if (SocketNodes[FbxSocketIx].SocketName == MeshSocket->SocketName)
			{
				Found = true;
				break;
			}
		}
		if (!Found)
		{
			StaticMesh->Sockets.RemoveAt(MeshSocketIx);
			MeshSocketIx--;
		}
	}
}


bool UnFbx::FFbxImporter::FillCollisionModelList(FbxNode* Node)
{
	FbxString NodeName = GetNodeNameWithoutNamespace( Node->GetName() );

	if ( NodeName.Find("UCX") != -1 || NodeName.Find("MCDCX") != -1 ||
		 NodeName.Find("UBX") != -1 || NodeName.Find("USP") != -1 || NodeName.Find("UCP") != -1)
	{
		// Get name of static mesh that the collision model connect to
		uint32 StartIndex = NodeName.Find('_') + 1;
		int32 TmpEndIndex = NodeName.Find('_', StartIndex);
		int32 EndIndex = TmpEndIndex;
		// Find the last '_' (underscore)
		while (TmpEndIndex >= 0)
		{
			EndIndex = TmpEndIndex;
			TmpEndIndex = NodeName.Find('_', EndIndex+1);
		}
		
		const int32 NumMeshNames = 2;
		FbxString MeshName[NumMeshNames];
		if ( EndIndex >= 0 )
		{
			// all characters between the first '_' and the last '_' are the FBX mesh name
			// convert the name to upper because we are case insensitive
			MeshName[0] = NodeName.Mid(StartIndex, EndIndex - StartIndex).Upper();
			
			// also add a version of the mesh name that includes what follows the last '_'
			// in case that's not a suffix but, instead, is part of the mesh name
			if (StartIndex < (int32)NodeName.GetLen())
			{            
				MeshName[1] = NodeName.Mid(StartIndex).Upper();
			}
		}
		else if (StartIndex < (int32)NodeName.GetLen())
		{            
			MeshName[0] = NodeName.Mid(StartIndex).Upper();
		}

		for (int32 NameIdx = 0; NameIdx < NumMeshNames; ++NameIdx)
		{
			if ((int32)MeshName[NameIdx].GetLen() > 0)
			{
				FbxMap<FbxString, TSharedPtr<FbxArray<FbxNode* > > >::RecordType const *Models = CollisionModels.Find(MeshName[NameIdx]);
				TSharedPtr< FbxArray<FbxNode* > > Record;
				if ( !Models )
				{
					Record = MakeShareable( new FbxArray<FbxNode*>() );
					CollisionModels.Insert(MeshName[NameIdx], Record);
				}
				else
				{
					Record = Models->GetValue();
				}

				//Unique add
				Record->AddUnique(Node);
			}
		}

		return true;
	}

	return false;
}


bool UnFbx::FFbxImporter::ImportCollisionModels(UStaticMesh* StaticMesh, const FbxString& InNodeName)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FFbxImporter::ImportCollisionModels);

	// find collision models
	bool bRemoveEmptyKey = false;
	FbxString EmptyKey;

	// convert the name to upper because we are case insensitive
	FbxMap<FbxString, TSharedPtr< FbxArray<FbxNode* > > >::RecordType const *Record = CollisionModels.Find(InNodeName.Upper());
	if ( !Record )
	{
		// compatible with old collision name format
		// if CollisionModels has only one entry and the key is ""
		if ( CollisionModels.GetSize() == 1 )
		{
			Record = CollisionModels.Find( EmptyKey );
		}
		if ( !Record ) 
		{
			return false;
		}
		else
		{
			bRemoveEmptyKey = true;
		}
	}

	TSharedPtr< FbxArray<FbxNode*> > Models = Record->GetValue();

	bool bAtLeastOneCollisionMeshImported = false;

	StaticMesh->CreateBodySetup();

	TArray<FVector3f>	CollisionVertices;
	TArray<int32>		CollisionFaceIdx;

	// construct collision model
	for (int32 ModelIndex=0; ModelIndex<Models->GetCount(); ModelIndex++)
	{
		FbxNode* Node = Models->GetAt(ModelIndex);
		FbxMesh* FbxMesh = Node->GetMesh();

		FbxMesh->RemoveBadPolygons();

		// Must do this before triangulating the mesh due to an FBX bug in TriangulateMeshAdvance
		int32 LayerSmoothingCount = FbxMesh->GetLayerCount(FbxLayerElement::eSmoothing);
		for(int32 LayerIndex = 0; LayerIndex < LayerSmoothingCount; LayerIndex++)
		{
			GeometryConverter->ComputePolygonSmoothingFromEdgeSmoothing (FbxMesh, LayerIndex);
		}

		if (!FbxMesh->IsTriangleMesh())
		{
			FString NodeName = MakeName(Node->GetName());
			UE_LOGF(LogFbx, Display, "Triangulating mesh %ls for collision model", *NodeName);

			const bool bReplace = true;
			FbxNodeAttribute* ConvertedNode = GeometryConverter->Triangulate(FbxMesh, bReplace); // not in place ! the old mesh is still there

			if( ConvertedNode != NULL && ConvertedNode->GetAttributeType() == FbxNodeAttribute::eMesh )
			{
				FbxMesh = (fbxsdk::FbxMesh*)ConvertedNode;
			}
			else
			{
				AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("Error_FailedToTriangulate", "Unable to triangulate mesh '{0}'"), FText::FromString(NodeName))), FFbxErrors::Generic_Mesh_TriangulationFailed);
				return false;
			}
		}

		int32 ControlPointsIndex;
		int32 ControlPointsCount = FbxMesh->GetControlPointsCount();
		FbxVector4* ControlPoints = FbxMesh->GetControlPoints();
		FbxAMatrix Matrix = ComputeTotalMatrix(Node);

		for ( ControlPointsIndex = 0; ControlPointsIndex < ControlPointsCount; ControlPointsIndex++ )
		{
			new(CollisionVertices)FVector3f(Converter.ConvertPos(Matrix.MultT(ControlPoints[ControlPointsIndex])));
		}

		int32 TriangleCount = FbxMesh->GetPolygonCount();
		int32 TriangleIndex;
		for ( TriangleIndex = 0 ; TriangleIndex < TriangleCount ; TriangleIndex++ )
		{
			new(CollisionFaceIdx)int32(FbxMesh->GetPolygonVertex(TriangleIndex,0));
			new(CollisionFaceIdx)int32(FbxMesh->GetPolygonVertex(TriangleIndex,1));
			new(CollisionFaceIdx)int32(FbxMesh->GetPolygonVertex(TriangleIndex,2));
		}

		TArray<FPoly> CollisionTriangles;

		// Make triangles
		for(int32 x = 0;x < CollisionFaceIdx.Num();x += 3)
		{
			FPoly*	Poly = new( CollisionTriangles ) FPoly();

			Poly->Init();

			new(Poly->Vertices) FVector3f( CollisionVertices[CollisionFaceIdx[x + 2]] );
			new(Poly->Vertices) FVector3f( CollisionVertices[CollisionFaceIdx[x + 1]] );
			new(Poly->Vertices) FVector3f( CollisionVertices[CollisionFaceIdx[x + 0]] );
			Poly->iLink = x / 3;

			Poly->CalcNormal(1);
		}

		// Construct geometry object
		FbxString ModelName(Node->GetName());
		if ( ModelName.Find("UCX") != -1 || ModelName.Find("MCDCX") != -1 )
		{
			if( !ImportOptions->bOneConvexHullPerUCX )
			{
				if (StaticMeshImportUtils::DecomposeUCXMesh(CollisionVertices, CollisionFaceIdx, StaticMesh->GetBodySetup()))
				{
					bAtLeastOneCollisionMeshImported = true;
				}
				else
				{
					FString CollisionModelName = UTF8_TO_TCHAR(ModelName.Buffer());
					AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("Error_DecomposingCollisionMesh", "Could not decompose collision mesh [{0}]."), FText::FromString(CollisionModelName))), FFbxErrors::StaticMesh_BuildError);
				}
			}
			else
			{
				FKAggregateGeom& AggGeo = StaticMesh->GetBodySetup()->AggGeom;

				// This function cooks the given data, so we cannot test for duplicates based on the position data
				// before we call it
				if (StaticMeshImportUtils::AddConvexGeomFromVertices(CollisionVertices, &AggGeo, ANSI_TO_TCHAR(Node->GetName())))
				{
					bAtLeastOneCollisionMeshImported = true;
					FKConvexElem& NewElem = AggGeo.ConvexElems.Last();

					// Now test the late element in the AggGeo list and remove it if its a duplicate
					for(int32 ElementIndex = 0; ElementIndex < AggGeo.ConvexElems.Num()-1; ++ElementIndex)
					{
						FKConvexElem& CurrentElem = AggGeo.ConvexElems[ElementIndex];
					
						if(CurrentElem.VertexData.Num() == NewElem.VertexData.Num())
						{
							bool bFoundDifference = false;
							for(int32 VertexIndex = 0; VertexIndex < NewElem.VertexData.Num(); ++VertexIndex)
							{
								if(CurrentElem.VertexData[VertexIndex] != NewElem.VertexData[VertexIndex])
								{
									bFoundDifference = true;
									break;
								}
							}

							if(!bFoundDifference)
							{
								// The new collision geo is a duplicate, delete it
								AggGeo.ConvexElems.RemoveAt(AggGeo.ConvexElems.Num()-1);
								break;
							}
						}
					}
				}
			}
		}
		else if ( ModelName.Find("UBX") != -1 )
		{
			FKAggregateGeom& AggGeo = StaticMesh->GetBodySetup()->AggGeom;

			if(StaticMeshImportUtils::AddBoxGeomFromTris(CollisionTriangles, &AggGeo, ANSI_TO_TCHAR(Node->GetName())))
			{
				bAtLeastOneCollisionMeshImported = true;
				FKBoxElem& NewElem = AggGeo.BoxElems.Last();

				// Now test the last element in the AggGeo list and remove it if its a duplicate
				for(int32 ElementIndex = 0; ElementIndex < AggGeo.BoxElems.Num()-1; ++ElementIndex)
				{
					FKBoxElem& CurrentElem = AggGeo.BoxElems[ElementIndex];

					if(	CurrentElem == NewElem )
					{
						// The new element is a duplicate, remove it
						AggGeo.BoxElems.RemoveAt(AggGeo.BoxElems.Num()-1);
						break;
					}
				}
			}
		}
		else if ( ModelName.Find("USP") != -1 )
		{
			FKAggregateGeom& AggGeo = StaticMesh->GetBodySetup()->AggGeom;

			if (StaticMeshImportUtils::AddSphereGeomFromVerts(CollisionVertices, &AggGeo, ANSI_TO_TCHAR(Node->GetName())))
			{
				bAtLeastOneCollisionMeshImported = true;
				FKSphereElem& NewElem = AggGeo.SphereElems.Last();

				// Now test the late element in the AggGeo list and remove it if its a duplicate
				for(int32 ElementIndex = 0; ElementIndex < AggGeo.SphereElems.Num()-1; ++ElementIndex)
				{
					FKSphereElem& CurrentElem = AggGeo.SphereElems[ElementIndex];

					if(	CurrentElem == NewElem )
					{
						// The new element is a duplicate, remove it
						AggGeo.SphereElems.RemoveAt(AggGeo.SphereElems.Num()-1);
						break;
					}
				}
			}
		}
		else if (ModelName.Find("UCP") != -1)
		{
			FKAggregateGeom& AggGeo = StaticMesh->GetBodySetup()->AggGeom;

			if (StaticMeshImportUtils::AddCapsuleGeomFromVerts(CollisionVertices, &AggGeo, ANSI_TO_TCHAR(Node->GetName())))
			{
				bAtLeastOneCollisionMeshImported = true;
				FKSphylElem& NewElem = AggGeo.SphylElems.Last();

			// Now test the late element in the AggGeo list and remove it if its a duplicate
				for (int32 ElementIndex = 0; ElementIndex < AggGeo.SphylElems.Num() - 1; ++ElementIndex)
				{
					FKSphylElem& CurrentElem = AggGeo.SphylElems[ElementIndex];
					if (CurrentElem == NewElem)
					{
						// The new element is a duplicate, remove it
						AggGeo.SphylElems.RemoveAt(AggGeo.SphylElems.Num() - 1);
						break;
					}
				}
			}
		}

		// Clear any cached rigid-body collision shapes for this body setup.
		StaticMesh->GetBodySetup()->ClearPhysicsMeshes();

		// Remove the empty key because we only use the model once for the first mesh
		if (bRemoveEmptyKey)
		{
			CollisionModels.Remove(EmptyKey);
		}

		CollisionVertices.Empty();
		CollisionFaceIdx.Empty();
	}

	if (bAtLeastOneCollisionMeshImported)
	{
		StaticMesh->SetCustomizedCollision(true);
	}

	// Create new GUID
	StaticMesh->GetBodySetup()->InvalidatePhysicsData();

	// Update collision on the static mesh
	// Note: do not refresh collision on components here, this must occur after the static mesh build.
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CreateStaticMeshNavCollision);
		StaticMesh->CreateNavCollision(true);
	}

	return bAtLeastOneCollisionMeshImported;
}
#undef LOCTEXT_NAMESPACE
