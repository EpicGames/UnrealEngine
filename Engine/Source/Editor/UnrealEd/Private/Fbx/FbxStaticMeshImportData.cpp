// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/FbxStaticMeshImportData.h"
#include "Engine/StaticMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FbxStaticMeshImportData)

UFbxStaticMeshImportData::UFbxStaticMeshImportData(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	StaticMeshLODGroup = NAME_None;
	bRemoveDegenerates = true;
	bBuildReversedIndexBuffer = true;
	bBuildNanite = false;
	bGenerateLightmapUVs = true;
	bOneConvexHullPerUCX = true;
	bAutoGenerateCollision = true;
	bTransformVertexToAbsolute = true;
	bBakePivotInVertex = false;
	VertexOverrideColor = FColor(255, 255, 255, 255);
	DistanceFieldResolutionScale = 1.0f;
}

UFbxStaticMeshImportData* UFbxStaticMeshImportData::GetImportDataForStaticMesh(UStaticMesh* StaticMesh, UFbxStaticMeshImportData* TemplateForCreation)
{
	check(StaticMesh);
	
	UFbxStaticMeshImportData* ImportData = Cast<UFbxStaticMeshImportData>(StaticMesh->GetAssetImportData());
	if ( !ImportData )
	{
		ImportData = NewObject<UFbxStaticMeshImportData>(StaticMesh, NAME_None, RF_NoFlags, TemplateForCreation);

		// Try to preserve the source file path if possible
		if ( StaticMesh->GetAssetImportData() != NULL )
		{
			ImportData->SourceData = StaticMesh->GetAssetImportData()->SourceData;
		}

		StaticMesh->SetAssetImportData(ImportData);
	}

	return ImportData;
}

bool UFbxStaticMeshImportData::CanEditChange(const FProperty* InProperty) const
{
	bool bMutable = Super::CanEditChange(InProperty);
	UObject* Outer = GetOuter();
	if(Outer && bMutable)
	{
		// Let the FbxImportUi object handle the editability of our properties
		bMutable = Outer->CanEditChange(InProperty);
	}
	return bMutable;
}
