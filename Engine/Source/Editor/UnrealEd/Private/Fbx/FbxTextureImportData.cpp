// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/FbxTextureImportData.h"

#include "UObject/Object.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FbxTextureImportData)

class FProperty;

UFbxTextureImportData::UFbxTextureImportData(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	
}

bool UFbxTextureImportData::CanEditChange(const FProperty* InProperty) const
{
	bool bMutable = Super::CanEditChange(InProperty);
	UObject* Outer = GetOuter();
	if(Outer && bMutable)
	{
		// Let the parent object handle the editability of our properties
		bMutable = Outer->CanEditChange(InProperty);
	}
	return bMutable;
}
