// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "FbxImporter.h"
#include "Logging/TokenizedMessage.h"

using namespace UnFbx;


FbxAMatrix FFbxDataConverter::JointPostConversionMatrix;
FbxAMatrix FFbxDataConverter::AxisConversionMatrix;
FbxAMatrix FFbxDataConverter::AxisConversionMatrixInv;

namespace UE::Fbx::Private
{
	void LogFinitErrorOnce()
	{
		static bool bLogged = false;
		if (!bLogged)
		{
			bLogged = true;
			UE_LOGF(LogFbx, Error, "Import mesh have some infinite value in the data.");
		}
	}

	void VerifyFiniteValue(float& Value)
	{
		if (!FMath::IsFinite(Value))
		{
			Value = 0.0f;
			LogFinitErrorOnce();
		}
	}

	void VerifyFiniteValue(FVector& Value)
	{
		if (Value.ContainsNaN())
		{
			Value.Set(0.0, 0.0, 0.0);
			LogFinitErrorOnce();
		}
	}

	void VerifyFiniteValue(FQuat& Value)
	{
		if (Value.ContainsNaN())
		{
			Value = FQuat::Identity;
			LogFinitErrorOnce();
		}
	}

	void VerifyFiniteValue(FMatrix& Value)
	{
		if (Value.ContainsNaN())
		{
			Value.SetIdentity();
			LogFinitErrorOnce();
		}
	}
} //ns UE::Fbx::Private

FVector FFbxDataConverter::ConvertPos(FbxVector4 Vector)
{
	FVector Out;
	Out[0] = Vector[0];
	// flip Y, then the right-handed axis system is converted to LHS
	Out[1] = -Vector[1];
	Out[2] = Vector[2];
	UE::Fbx::Private::VerifyFiniteValue(Out);
	return Out;
}



FVector FFbxDataConverter::ConvertDir(FbxVector4 Vector)
{
	FVector Out;
	Out[0] = Vector[0];
	Out[1] = -Vector[1];
	Out[2] = Vector[2];
	UE::Fbx::Private::VerifyFiniteValue(Out);
	return Out;
}

FRotator FFbxDataConverter::ConvertEuler( FbxDouble3 Euler )
{
	FVector UnrealEuler(Euler[0], -Euler[1], Euler[2]);
	UE::Fbx::Private::VerifyFiniteValue(UnrealEuler);
	return FRotator::MakeFromEuler(UnrealEuler);
}


FVector FFbxDataConverter::ConvertScale(FbxDouble3 Vector)
{
	FVector Out;
	Out[0] = Vector[0];
	Out[1] = Vector[1];
	Out[2] = Vector[2];
	UE::Fbx::Private::VerifyFiniteValue(Out);
	return Out;
}


FVector FFbxDataConverter::ConvertScale(FbxVector4 Vector)
{
	FVector Out;
	Out[0] = Vector[0];
	Out[1] = Vector[1];
	Out[2] = Vector[2];
	UE::Fbx::Private::VerifyFiniteValue(Out);
	return Out;
}

FRotator FFbxDataConverter::ConvertRotation(FbxQuaternion Quaternion)
{
	FRotator Out(ConvertRotToQuat(Quaternion));
	return Out;
}

//-------------------------------------------------------------------------
//
//-------------------------------------------------------------------------
FVector FFbxDataConverter::ConvertRotationToFVect(FbxQuaternion Quaternion, bool bInvertOrient)
{
	FQuat UnrealQuaternion = ConvertRotToQuat(Quaternion);
	FVector Euler;
	Euler = UnrealQuaternion.Euler();
	UE::Fbx::Private::VerifyFiniteValue(Euler);
	if (bInvertOrient)
	{
		Euler.Y = -Euler.Y;
		Euler.Z = 180.f+Euler.Z;
	}
	return Euler;
}

//-------------------------------------------------------------------------
//
//-------------------------------------------------------------------------
FQuat FFbxDataConverter::ConvertRotToQuat(FbxQuaternion Quaternion)
{
	FQuat UnrealQuat;
	UnrealQuat.X = Quaternion[0];
	UnrealQuat.Y = -Quaternion[1];
	UnrealQuat.Z = Quaternion[2];
	UnrealQuat.W = -Quaternion[3];
	UE::Fbx::Private::VerifyFiniteValue(UnrealQuat);
	
	return UnrealQuat;
}

//-------------------------------------------------------------------------
//
//-------------------------------------------------------------------------
float FFbxDataConverter::ConvertDist(FbxDouble Distance)
{
	float Out;
	Out = (float)Distance;
	UE::Fbx::Private::VerifyFiniteValue(Out);
	return Out;
}

FTransform FFbxDataConverter::ConvertTransform(FbxAMatrix Matrix)
{
	FTransform Out;

	FQuat Rotation = ConvertRotToQuat(Matrix.GetQ());
	FVector Origin = ConvertPos(Matrix.GetT());
	FVector Scale = ConvertScale(Matrix.GetS());

	Out.SetTranslation(Origin);
	Out.SetScale3D(Scale);
	Out.SetRotation(Rotation);

	return Out;
}

FMatrix FFbxDataConverter::ConvertMatrix(const FbxAMatrix& Matrix)
{
	FMatrix UEMatrix;

	for(int i=0; i<4; ++i)
	{
		const FbxVector4 Row = Matrix.GetRow(i);
		if(i==1)
		{
			UEMatrix.M[i][0] = -Row[0];
			UEMatrix.M[i][1] = Row[1];
			UEMatrix.M[i][2] = -Row[2];
			UEMatrix.M[i][3] = -Row[3];
		}
		else
		{
			UEMatrix.M[i][0] = Row[0];
			UEMatrix.M[i][1] = -Row[1];
			UEMatrix.M[i][2] = Row[2];
			UEMatrix.M[i][3] = Row[3];
		}
	}
	UE::Fbx::Private::VerifyFiniteValue(UEMatrix);
	return UEMatrix;
}

FbxAMatrix FFbxDataConverter::ConvertMatrix(const FMatrix& UEMatrix)
{
	FbxAMatrix FbxMatrix;

	for (int i = 0; i < 4; ++i)
	{
		FbxVector4 Row;
		if (i == 1)
		{
			Row[0] = -UEMatrix.M[i][0];
			Row[1] = UEMatrix.M[i][1];
			Row[2] = -UEMatrix.M[i][2];
			Row[3] = -UEMatrix.M[i][3];
		}
		else
		{
			Row[0] = UEMatrix.M[i][0];
			Row[1] = -UEMatrix.M[i][1];
			Row[2] = UEMatrix.M[i][2];
			Row[3] = UEMatrix.M[i][3];
		}
		FbxMatrix.SetRow(i, Row);
	}

	return FbxMatrix;
}

FColor FFbxDataConverter::ConvertColor(FbxDouble3 Color)
{
	//Fbx is in linear color space
	FColor SRGBColor = 
		FLinearColor(
			static_cast<float>(Color[0]),
			static_cast<float>(Color[1]),
			static_cast<float>(Color[2])
		).ToFColor(true);
	return SRGBColor;
}

FbxVector4 FFbxDataConverter::ConvertToFbxPos(FVector Vector)
{
	FbxVector4 Out;
	Out[0] = Vector[0];
	Out[1] = -Vector[1];
	Out[2] = Vector[2];
	
	return Out;
}

FbxVector4 FFbxDataConverter::ConvertToFbxRot(FVector Vector)
{
	FbxVector4 Out;
	Out[0] = Vector[0];
	Out[1] = -Vector[1];
	Out[2] = -Vector[2];

	return Out;
}

FbxVector4 FFbxDataConverter::ConvertToFbxScale(FVector Vector)
{
	FbxVector4 Out;
	Out[0] = Vector[0];
	Out[1] = Vector[1];
	Out[2] = Vector[2];

	return Out;
}

FbxDouble3 FFbxDataConverter::ConvertToFbxColor(FColor Color)
{
	//Fbx is in linear color space
	FLinearColor FbxLinearColor(Color);
	FbxDouble3 Out;
	Out[0] = FbxLinearColor.R;
	Out[1] = FbxLinearColor.G;
	Out[2] = FbxLinearColor.B;

	return Out;
}

FbxString FFbxDataConverter::ConvertToFbxString(FName Name)
{
	FbxString OutString;

	FString UnrealString;
	Name.ToString(UnrealString);

	OutString = TCHAR_TO_UTF8(*UnrealString);

	return OutString;
}

FbxString FFbxDataConverter::ConvertToFbxString(const FString& String)
{
	FbxString OutString;

	OutString = TCHAR_TO_UTF8(*String);

	return OutString;
}
