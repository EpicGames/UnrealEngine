// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/ReimportFbxSceneFactory.h"
#include "Animation/Skeleton.h"
#include "Misc/Paths.h"
#include "Misc/FeedbackContext.h"
#include "Modules/ModuleManager.h"
#include "Serialization/ObjectWriter.h"
#include "Serialization/ObjectReader.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "Components/SceneComponent.h"
#include "Engine/Blueprint.h"
#include "Components/StaticMeshComponent.h"
#include "Animation/AnimTypes.h"
#include "Engine/SkeletalMesh.h"
#include "Components/SkeletalMeshComponent.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Animation/AnimSequence.h"
#include "Factories/FbxAssetImportData.h"
#include "Factories/FbxAnimSequenceImportData.h"
#include "Factories/FbxSkeletalMeshImportData.h"
#include "Factories/FbxSceneImportData.h"
#include "Factories/FbxSceneImportOptions.h"
#include "Factories/FbxSceneImportOptionsSkeletalMesh.h"
#include "Factories/FbxSceneImportOptionsStaticMesh.h"
#include "Camera/CameraComponent.h"
#include "Components/PointLightComponent.h"
#include "Components/SpotLightComponent.h"
#include "Engine/StaticMesh.h"
#include "Components/DirectionalLightComponent.h"
#include "Engine/AssetUserData.h"
#include "FileHelpers.h"

#include "Logging/TokenizedMessage.h"
#include "FbxImporter.h"

#include "Misc/FbxErrors.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "PackageTools.h"

#include "SFbxSceneOptionWindow.h"
#include "Interfaces/IMainFrameModule.h"

#include "Kismet2/KismetEditorUtilities.h"
#include "Engine/SimpleConstructionScript.h"
#include "Engine/SCS_Node.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"

#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"

#include "ObjectTools.h"

#include "AI/Navigation/NavCollisionBase.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Editor.h"

#include "SSceneReimportNodeTreeView.h"
#include "SSceneBaseMeshListView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReimportFbxSceneFactory)

#define LOCTEXT_NAMESPACE "FBXSceneReImportFactory"

UFbxSceneImportData *GetFbxSceneImportData(UObject *Obj)
{
	UFbxSceneImportData* SceneImportData = nullptr;
	if (Obj->IsA(UFbxSceneImportData::StaticClass()))
	{
		//Reimport from the scene data
		SceneImportData = Cast<UFbxSceneImportData>(Obj);
	}
	else
	{
		UFbxAssetImportData *ImportData = nullptr;
		if (Obj->IsA(UStaticMesh::StaticClass()))
		{
			//Reimport from one of the static mesh
			UStaticMesh* Mesh = Cast<UStaticMesh>(Obj);
			if (Mesh != nullptr && Mesh->GetAssetImportData() != nullptr)
			{
				ImportData = Cast<UFbxAssetImportData>(Mesh->GetAssetImportData());
			}
		}
		else if (Obj->IsA(USkeletalMesh::StaticClass()))
		{
			//Reimport from one of the static mesh
			USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Obj);
			if (SkeletalMesh != nullptr && SkeletalMesh->GetAssetImportData() != nullptr)
			{
				ImportData = Cast<UFbxAssetImportData>(SkeletalMesh->GetAssetImportData());
			}
		}
		else if (Obj->IsA(UAnimSequence::StaticClass()))
		{
			//Reimport from one of the static mesh
			UAnimSequence* AnimSequence = Cast<UAnimSequence>(Obj);
			if (AnimSequence != nullptr && AnimSequence->AssetImportData != nullptr)
			{
				ImportData = Cast<UFbxAssetImportData>(AnimSequence->AssetImportData);
			}
		}
		//TODO: add all type the fbx scene import can create: material, texture, skeletal mesh, animation, ... 

		if (ImportData != nullptr)
		{
			SceneImportData = ImportData->bImportAsScene ? ToRawPtr(ImportData->FbxSceneImportDataReference) : nullptr;
		}
	}
	return SceneImportData;
}

UReimportFbxSceneFactory::UReimportFbxSceneFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = UFbxSceneImportData::StaticClass();

	bCreateNew = false;
	bText = false;

	ImportPriority = DefaultImportPriority - 1;
}

bool UReimportFbxSceneFactory::FactoryCanImport(const FString& Filename)
{
	// Return false, we are a reimport only factory
	return false;
}

bool UReimportFbxSceneFactory::CanReimport(UObject* Obj, TArray<FString>& OutFilenames)
{
	UFbxSceneImportData* SceneImportData = GetFbxSceneImportData(Obj);
	if (SceneImportData != nullptr)
	{
		OutFilenames.Add(SceneImportData->SourceFbxFile);
		return true;
	}
	return false;
}

void UReimportFbxSceneFactory::SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths)
{
	UFbxSceneImportData* SceneImportData = Cast<UFbxSceneImportData>(Obj);
	if (SceneImportData && ensure(NewReimportPaths.Num() == 1))
	{
		SceneImportData->SourceFbxFile = NewReimportPaths[0];
	}
}

void RecursivelyCreateOriginalPath(UnFbx::FFbxImporter* FbxImporter, TSharedPtr<FFbxNodeInfo> NodeInfo, FString AssetPath, TSet<uint64> &AssetPathDone)
{
	if (NodeInfo->AttributeInfo.IsValid() && !AssetPathDone.Contains(NodeInfo->AttributeInfo->UniqueId))
	{
		FString AssetName = AssetPath + TEXT("/") + NodeInfo->AttributeInfo->Name;
		NodeInfo->AttributeInfo->SetOriginalImportPath(AssetName);
		FString OriginalFullImportName = UPackageTools::SanitizePackageName(AssetName);
		OriginalFullImportName = OriginalFullImportName + TEXT(".") + UPackageTools::SanitizePackageName(NodeInfo->AttributeInfo->Name);
		NodeInfo->AttributeInfo->SetOriginalFullImportName(OriginalFullImportName);
		AssetPathDone.Add(NodeInfo->AttributeInfo->UniqueId);
	}
	if (NodeInfo->NodeName.Compare("RootNode") != 0)
	{
		AssetPath += TEXT("/") + NodeInfo->NodeName;
	}
	for (TSharedPtr<FFbxNodeInfo> Child : NodeInfo->Childrens)
	{
		RecursivelyCreateOriginalPath(FbxImporter, Child, AssetPath, AssetPathDone);
	}
}

void SetNodeInfoTypeChanged(TSharedPtr<FFbxNodeInfo> NodeInfoA, TSharedPtr<FFbxNodeInfo> NodeInfoB)
{
	if (NodeInfoA->AttributeInfo.IsValid())
	{
		//We found a match verify the type
		NodeInfoA->AttributeInfo->bOriginalTypeChanged = NodeInfoA->AttributeInfo.IsValid() != NodeInfoB->AttributeInfo.IsValid();
		if (!NodeInfoA->AttributeInfo->bOriginalTypeChanged && NodeInfoA->AttributeInfo.IsValid() && NodeInfoB->AttributeInfo.IsValid())
		{
			NodeInfoA->AttributeInfo->bOriginalTypeChanged = NodeInfoA->AttributeInfo->GetType() != NodeInfoB->AttributeInfo->GetType();
		}
		if (!NodeInfoA->AttributeInfo->bOriginalTypeChanged)
		{
			UObject *ContentObjectA = NodeInfoA->AttributeInfo->GetContentObject();
			if (ContentObjectA != nullptr)
			{
				if (!ContentObjectA->IsA(NodeInfoA->AttributeInfo->GetType()))
				{
					NodeInfoA->AttributeInfo->bOriginalTypeChanged = true;
				}
			}
		}
	}
}

void MarkAssetTypeChanged(TSharedPtr<FFbxSceneInfo> SceneInfoPtr, TSharedPtr<FFbxSceneInfo> SceneInfoOriginalPtr)
{
	//Set the node info
	for (TSharedPtr<FFbxNodeInfo> NodeInfo : SceneInfoPtr->HierarchyInfo)
	{
		for (TSharedPtr<FFbxNodeInfo> NodeInfoOriginal : SceneInfoOriginalPtr->HierarchyInfo)
		{
			if (NodeInfo->NodeHierarchyPath == NodeInfoOriginal->NodeHierarchyPath)
			{
				//Set the current
				SetNodeInfoTypeChanged(NodeInfo, NodeInfoOriginal);
				//Set the original
				SetNodeInfoTypeChanged(NodeInfoOriginal, NodeInfo);
			}
		}
	}
}

bool GetFbxSceneReImportOptions(UnFbx::FFbxImporter* FbxImporter
	, TSharedPtr<FFbxSceneInfo> SceneInfoPtr
	, TSharedPtr<FFbxSceneInfo> SceneInfoOriginalPtr
	, UnFbx::FBXImportOptions *GlobalImportSettings
	, UFbxSceneImportOptions *SceneImportOptions
	, UFbxSceneImportOptionsStaticMesh *StaticMeshImportData
	, UFbxSceneImportOptionsSkeletalMesh *SkeletalMeshImportData
	, ImportOptionsNameMap &NameOptionsMap
	, FbxSceneReimportStatusMap &MeshStatusMap
	, FbxSceneReimportStatusMap &NodeStatusMap
	, bool bCanReimportHierarchy
	, FString Path)
{
	//Make sure we don't put the global transform into the vertex position of the mesh
	GlobalImportSettings->bTransformVertexToAbsolute = false;
	//Avoid combining meshes
	GlobalImportSettings->bCombineToSingle = false;
	//Use the full name (avoid creating one) to let us retrieve node transform and attachment later
	GlobalImportSettings->bUsedAsFullName = true;
	//Make sure we import the textures
	GlobalImportSettings->bImportTextures = true;
	//Make sure Material get imported
	GlobalImportSettings->bImportMaterials = true;
	//TODO support T0AsRefPose
	GlobalImportSettings->bUseT0AsRefPose = false;
	//Make sure skeletal mesh sections with the same material are combined (and not kept separate)
	GlobalImportSettings->bKeepSectionsSeparate = false;
	//Make sure we do not mess with AutoComputeLodDistances when re-importing
	GlobalImportSettings->bAutoComputeLodDistances = true;
	GlobalImportSettings->LodNumber = 0;
	GlobalImportSettings->MinimumLodNumber = 0;

	GlobalImportSettings->ImportTranslation = FVector(0);
	GlobalImportSettings->ImportRotation = FRotator(0);
	GlobalImportSettings->ImportUniformScale = 1.0f;

	GlobalImportSettings->bConvertScene = true;
	GlobalImportSettings->bConvertSceneUnit = true;

	GlobalImportSettings->OverrideMaterials.Reset();

	if (!GIsRunningUnattendedScript)
	{
		TSharedPtr<SWindow> ParentWindow;
		if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
		{
			IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
			ParentWindow = MainFrame.GetParentWindow();
		}
		TSharedRef<SWindow> Window = SNew(SWindow)
			.ClientSize(FVector2D(800.f, 650.f))
			.Title(NSLOCTEXT("UnrealEd", "FBXSceneReimportOpionsTitle", "FBX Scene Reimport Options"));
		TSharedPtr<SFbxSceneOptionWindow> FbxSceneOptionWindow;

		//Make sure the display option show the save default options
		SFbxSceneOptionWindow::CopyFbxOptionsToStaticMeshOptions(GlobalImportSettings, StaticMeshImportData);
		SFbxSceneOptionWindow::CopyFbxOptionsToSkeletalMeshOptions(GlobalImportSettings, SkeletalMeshImportData);

		Window->SetContent
		(
			SAssignNew(FbxSceneOptionWindow, SFbxSceneOptionWindow)
			.SceneInfo(SceneInfoPtr)
			.SceneInfoOriginal(SceneInfoOriginalPtr)
			.SceneImportOptionsDisplay(SceneImportOptions)
			.SceneImportOptionsStaticMeshDisplay(StaticMeshImportData)
			.SceneImportOptionsSkeletalMeshDisplay(SkeletalMeshImportData)
			.OverrideNameOptionsMap(&NameOptionsMap)
			.MeshStatusMap(&MeshStatusMap)
			.CanReimportHierarchy(bCanReimportHierarchy)
			.NodeStatusMap(&NodeStatusMap)
			.GlobalImportSettings(GlobalImportSettings)
			.OwnerWindow(Window)
			.FullPath(Path)
		);

		FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);

		if (!FbxSceneOptionWindow->ShouldImport())
		{
			return false;
		}
	}
	else
	{
		//Fill the node and mesh status data so the reimport know what was added/removed/changed
		SFbxReimportSceneTreeView::FillNodeStatusMap(&NodeStatusMap, SceneInfoPtr, SceneInfoOriginalPtr);
		bool bFillSkeletalMeshStatusMap = false;
		SFbxSSceneBaseMeshListView::FillMeshStatusMap(&MeshStatusMap, SceneInfoPtr, SceneInfoOriginalPtr, bFillSkeletalMeshStatusMap);
		bFillSkeletalMeshStatusMap = true;
		SFbxSSceneBaseMeshListView::FillMeshStatusMap(&MeshStatusMap, SceneInfoPtr, SceneInfoOriginalPtr, bFillSkeletalMeshStatusMap);
	}

	//Set the bakepivot option in the SceneImportOptions
	SceneImportOptions->bBakePivotInVertex = GlobalImportSettings->bBakePivotInVertex;
	//setup all options
	GlobalImportSettings->bForceFrontXAxis = SceneImportOptions->bForceFrontXAxis;
	GlobalImportSettings->bImportStaticMeshLODs = SceneImportOptions->bImportStaticMeshLODs;
	GlobalImportSettings->bImportSkeletalMeshLODs = SceneImportOptions->bImportSkeletalMeshLODs;
	SceneImportOptions->bInvertNormalMaps = GlobalImportSettings->bInvertNormalMap;
	GlobalImportSettings->ImportTranslation = SceneImportOptions->ImportTranslation;
	GlobalImportSettings->ImportRotation = SceneImportOptions->ImportRotation;
	GlobalImportSettings->ImportUniformScale = SceneImportOptions->ImportUniformScale;

	//Set the override material into the options
	for (TSharedPtr<FFbxNodeInfo> NodeInfo : SceneInfoPtr->HierarchyInfo)
	{
		for (TSharedPtr<FFbxMaterialInfo> Material : NodeInfo->Materials)
		{
			if (!GlobalImportSettings->OverrideMaterials.Contains(Material->UniqueId))
			{
				//If user don't want to import a material we have to replace it by the default material
				if (!Material->bImportAttribute)
				{
					UMaterial* DefaultMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
					if (DefaultMaterial != nullptr)
					{
						GlobalImportSettings->OverrideMaterials.Add(Material->UniqueId, DefaultMaterial);
					}
				}
				else if (Material->bOverridePath)
				{
					UMaterialInterface *UnrealMaterial = static_cast<UMaterialInterface*>(Material->GetContentObject());
					if (UnrealMaterial != nullptr)
					{
						GlobalImportSettings->OverrideMaterials.Add(Material->UniqueId, UnrealMaterial);
					}
				}
			}
		}
	}

	SceneImportOptions->SaveConfig();

	//Save the Default setting copy them in the UObject and save them
	SFbxSceneOptionWindow::CopyFbxOptionsToStaticMeshOptions(GlobalImportSettings, StaticMeshImportData);
	StaticMeshImportData->SaveConfig();

	SFbxSceneOptionWindow::CopyFbxOptionsToSkeletalMeshOptions(GlobalImportSettings, SkeletalMeshImportData);
	SkeletalMeshImportData->SaveConfig();

	//Make sure default option set will not override fbx global setting by making copy of the real default options
	ImportOptionsNameMap TmpNameOptionsMap;
	for (auto kvp : NameOptionsMap)
	{
		UnFbx::FBXImportOptions *NewOptions = new UnFbx::FBXImportOptions();
		SFbxSceneOptionWindow::CopyFbxOptionsToFbxOptions(kvp.Value, NewOptions);
		TmpNameOptionsMap.Add(kvp.Key, NewOptions);
	}
	NameOptionsMap.Reset();
	for (auto kvp : TmpNameOptionsMap)
	{
		NameOptionsMap.Add(kvp.Key, kvp.Value);
	}
	return true;
}

EReimportResult::Type UReimportFbxSceneFactory::Reimport(UObject* Obj)
{
	ReimportData = GetFbxSceneImportData(Obj);
	if (!ReimportData)
	{
		return EReimportResult::Failed;
	}
	NameOptionsMap.Reset();

	//We will call other factory store the filename value since UFactory::CurrentFilename is static
	FbxImportFileName = ReimportData->SourceFbxFile;

	UnFbx::FFbxImporter* FbxImporter = UnFbx::FFbxImporter::GetInstance();
	UnFbx::FFbxLoggerSetter Logger(FbxImporter);
	GWarn->BeginSlowTask(NSLOCTEXT("FbxSceneReImportFactory", "BeginReImportingFbxSceneTask_ReadingFbxFile", "Re-importing FBX scene (reading fbx file)"), true);

	GlobalImportSettings = FbxImporter->GetImportOptions();
	UnFbx::FBXImportOptions::ResetOptions(GlobalImportSettings);

	//Fill the original options
	for (auto kvp : ReimportData->NameOptionsMap)
	{
		if (kvp.Key.Compare(DefaultOptionName) == 0)
		{
			//Save the default option to the fbx default import setting
			SFbxSceneOptionWindow::CopyFbxOptionsToFbxOptions(kvp.Value, GlobalImportSettings);
			NameOptionsMap.Add(kvp.Key, GlobalImportSettings);
		}
		else
		{
			NameOptionsMap.Add(kvp.Key, kvp.Value);
		}
	}

	//Always convert the scene
	GlobalImportSettings->bConvertScene = true;
	GlobalImportSettings->bConvertSceneUnit = true;
	GlobalImportSettings->bImportScene = ReimportData->bImportScene;
	if (ReimportData->NameOptionsMap.Contains(DefaultOptionName))
	{
		UnFbx::FBXImportOptions *DefaultOption = *(ReimportData->NameOptionsMap.Find(DefaultOptionName));
		GlobalImportSettings->bBakePivotInVertex = DefaultOption->bBakePivotInVertex;
		GlobalImportSettings->bInvertNormalMap = DefaultOption->bInvertNormalMap;
	}
	bool OriginalForceFrontXAxis = GlobalImportSettings->bForceFrontXAxis;
	//Read the fbx and store the hierarchy's information so we can reuse it after importing all the model in the fbx file
	if (!FbxImporter->ImportFromFile(*FbxImportFileName, FPaths::GetExtension(FbxImportFileName), true))
	{
		// Log the error message and fail the import.
		GWarn->Log(ELogVerbosity::Error, FbxImporter->GetErrorMessage());
		FbxImporter->ReleaseScene();
		FbxImporter = nullptr;
		GWarn->EndSlowTask();
		return EReimportResult::Failed;
	}

	//Make sure the Skeleton is null and not garbage, as we are importing the skeletalmesh for the first time we do not need any skeleton
	GlobalImportSettings->SkeletonForAnimation = nullptr;
	GlobalImportSettings->PhysicsAsset = nullptr;

	SFbxSceneOptionWindow::CopyFbxOptionsToStaticMeshOptions(GlobalImportSettings, SceneImportOptionsStaticMesh);
	SFbxSceneOptionWindow::CopyFbxOptionsToSkeletalMeshOptions(GlobalImportSettings, SceneImportOptionsSkeletalMesh);
	SceneImportOptions->bBakePivotInVertex = GlobalImportSettings->bBakePivotInVertex;
	SceneImportOptions->bTransformVertexToAbsolute = GlobalImportSettings->bTransformVertexToAbsolute;
	SceneImportOptions->bImportStaticMeshLODs = GlobalImportSettings->bImportStaticMeshLODs;
	SceneImportOptions->bImportSkeletalMeshLODs = GlobalImportSettings->bImportSkeletalMeshLODs;

	FString PackageName = "";
	Obj->GetOutermost()->GetName(PackageName);
	Path = FPaths::GetPath(PackageName);

	UnFbx::FbxSceneInfo SceneInfo;
	//Read the scene and found all instance with their scene information.
	FbxImporter->GetSceneInfo(FbxImportFileName, SceneInfo, true);

	//Convert old structure to the new scene export structure
	TSharedPtr<FFbxSceneInfo> SceneInfoPtr = ConvertSceneInfo(FbxImporter, &SceneInfo);
	//Get import material info
	ExtractMaterialInfo(FbxImporter, SceneInfoPtr);

	if (!ReimportData->bCreateFolderHierarchy)
	{
		for (TSharedPtr<FFbxMeshInfo> MeshInfo : SceneInfoPtr->MeshInfo)
		{
			FString AssetName = Path + TEXT("/") + MeshInfo->Name;
			MeshInfo->SetOriginalImportPath(AssetName);
			FString OriginalFullImportName = UPackageTools::SanitizePackageName(AssetName);
			OriginalFullImportName = OriginalFullImportName + TEXT(".") + UPackageTools::SanitizePackageName(MeshInfo->Name);
			MeshInfo->SetOriginalFullImportName(OriginalFullImportName);
		}
	}
	else
	{
		TSet<uint64> AssetPathDone;
		FString AssetPath = Path;
		for (TSharedPtr<FFbxNodeInfo> NodeInfo : SceneInfoPtr->HierarchyInfo)
		{
			//Iterate the hierarchy and build the original path
			RecursivelyCreateOriginalPath(FbxImporter, NodeInfo, AssetPath, AssetPathDone);
		}
	}

	FillSceneHierarchyPath(SceneInfoPtr);

	MarkAssetTypeChanged(SceneInfoPtr, ReimportData->SceneInfoSourceData);

	FbxSceneReimportStatusMap MeshStatusMap;
	FbxSceneReimportStatusMap NodeStatusMap;
	bool bCanReimportHierarchy = ReimportData->HierarchyType == (int32)EFBXSceneOptionsCreateHierarchyType::FBXSOCHT_CreateBlueprint && !ReimportData->BluePrintFullName.IsEmpty();

	SceneImportOptions->bForceFrontXAxis = GlobalImportSettings->bForceFrontXAxis;
	
	//We stop The slow task before showing the import option dialog
	//This prevent fight between slow task dialog and import option dialog (both are modal)
	GWarn->EndSlowTask();

	if (!GetFbxSceneReImportOptions(FbxImporter
		, SceneInfoPtr
		, ReimportData->SceneInfoSourceData
		, GlobalImportSettings
		, SceneImportOptions
		, SceneImportOptionsStaticMesh
		, SceneImportOptionsSkeletalMesh
		, NameOptionsMap
		, MeshStatusMap
		, NodeStatusMap
		, bCanReimportHierarchy
		, Path))
	{
		//User cancel the scene import
		FbxImporter->ReleaseScene();
		FbxImporter = nullptr;
		GlobalImportSettings = nullptr;
		return EReimportResult::Cancelled;
	}

	//Restart the slow task after the reimport dialog
	GWarn->BeginSlowTask(NSLOCTEXT("FbxSceneReImportFactory", "BeginReImportingFbxSceneTask_ReimportingScene", "Re-importing FBX scene (importing scene and assets)"), true);
	GlobalImportSettingsReference = new UnFbx::FBXImportOptions();
	SFbxSceneOptionWindow::CopyFbxOptionsToFbxOptions(GlobalImportSettings, GlobalImportSettingsReference);

	//Convert the scene to the correct axis system. Option like force front X
	//We need to get the new convert transform
	if (OriginalForceFrontXAxis != GlobalImportSettings->bForceFrontXAxis)
	{
		ChangeFrontAxis(FbxImporter, &SceneInfo, SceneInfoPtr);
	}

	//Overwrite the reimport asset data with the new data
	ReimportData->SceneInfoSourceData = SceneInfoPtr;
	ReimportData->SourceFbxFile = FPaths::ConvertRelativePathToFull(FbxImportFileName);
	ReimportData->bImportScene = GlobalImportSettingsReference->bImportScene;

	//Copy the options map
	ReimportData->NameOptionsMap.Reset();
	for (auto kvp : NameOptionsMap)
	{
		ReimportData->NameOptionsMap.Add(kvp.Key, kvp.Value);
	}

	StaticMeshImportData->bImportAsScene = true;
	StaticMeshImportData->FbxSceneImportDataReference = ReimportData;
	SkeletalMeshImportData->bImportAsScene = true;
	SkeletalMeshImportData->FbxSceneImportDataReference = ReimportData;

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	TArray<FAssetData> AssetDataToDelete;
	for (TSharedPtr<FFbxMeshInfo> MeshInfo : SceneInfoPtr->MeshInfo)
	{
		//Delete all the delete asset
		if (!MeshStatusMap.Contains(MeshInfo->OriginalImportPath) || MeshInfo->bOriginalTypeChanged)
		{
			continue;
		}
		EFbxSceneReimportStatusFlags MeshStatus = *(MeshStatusMap.Find(MeshInfo->OriginalImportPath));
		if ((MeshStatus & EFbxSceneReimportStatusFlags::Removed) == EFbxSceneReimportStatusFlags::None || (MeshStatus & EFbxSceneReimportStatusFlags::ReimportAsset) == EFbxSceneReimportStatusFlags::None)
		{
			continue;
		}
		//Make sure we load all package that will be deleted
		UPackage* PkgExist = MeshInfo->GetContentPackage();
		if (PkgExist == nullptr)
		{
			continue;
		}
		//Find the asset
		AssetDataToDelete.Add(AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath((MeshInfo->GetFullImportName()))));
	}

	FbxNode* RootNodeToImport = FbxImporter->Scene->GetRootNode();

	AllNewAssets.Empty();
	AssetToSyncContentBrowser.Empty();
	EReimportResult::Type ReimportResult = EReimportResult::Succeeded;
	//Reimport and add asset
	for (TSharedPtr<FFbxMeshInfo> MeshInfo : SceneInfoPtr->MeshInfo)
	{
		if (!MeshStatusMap.Contains(MeshInfo->OriginalImportPath))
		{
			continue;
		}
		EFbxSceneReimportStatusFlags MeshStatus = *(MeshStatusMap.Find(MeshInfo->OriginalImportPath));
		
		//Set the import status for the next reimport
		MeshInfo->bImportAttribute = (MeshStatus & EFbxSceneReimportStatusFlags::ReimportAsset) != EFbxSceneReimportStatusFlags::None;

		//Remove the mesh
		if ((MeshStatus & EFbxSceneReimportStatusFlags::Removed) != EFbxSceneReimportStatusFlags::None)
		{
			continue;
		}

		if ((MeshStatus & EFbxSceneReimportStatusFlags::ReimportAsset) == EFbxSceneReimportStatusFlags::None)
		{
			if (bCanReimportHierarchy &&
				(MeshStatus & EFbxSceneReimportStatusFlags::Same) != EFbxSceneReimportStatusFlags::None &&
				(MeshStatus & EFbxSceneReimportStatusFlags::FoundContentBrowserAsset) != EFbxSceneReimportStatusFlags::None &&
				!AllNewAssets.Contains(MeshInfo))
			{
				//Add the old asset in the allNewAsset array, it allow to kept the reference if there was one
				//Load the UObject associate with this MeshInfo
				UObject* Mesh = MeshInfo->GetContentObject();
				if (Mesh != nullptr)
				{
					if (MeshInfo->bIsSkelMesh ? Cast<USkeletalMesh>(Mesh) != nullptr : Cast<UStaticMesh>(Mesh) != nullptr)
					{
						AllNewAssets.Add(MeshInfo, Mesh);
					}
				}
			}
			continue;
		}

		if (((MeshStatus & EFbxSceneReimportStatusFlags::Same) != EFbxSceneReimportStatusFlags::None || (MeshStatus & EFbxSceneReimportStatusFlags::Added) != EFbxSceneReimportStatusFlags::None) &&
			(MeshStatus & EFbxSceneReimportStatusFlags::FoundContentBrowserAsset) != EFbxSceneReimportStatusFlags::None)
		{
			//Reimport over the old asset
			if (!MeshInfo->bIsSkelMesh)
			{
				ReimportResult = ReimportStaticMesh(FbxImporter, MeshInfo);
			}
			else
			{
				ReimportResult = ReimportSkeletalMesh(FbxImporter, MeshInfo);
			}
		}
		else if ((MeshStatus & EFbxSceneReimportStatusFlags::Added) != EFbxSceneReimportStatusFlags::None || (MeshStatus & EFbxSceneReimportStatusFlags::Same) != EFbxSceneReimportStatusFlags::None)
		{
			if (!MeshInfo->bIsSkelMesh)
			{
				ReimportResult = ImportStaticMesh(FbxImporter, MeshInfo, SceneInfoPtr);
			}
			else
			{
				ReimportResult = ImportSkeletalMesh(RootNodeToImport, FbxImporter, MeshInfo, SceneInfoPtr);
			}
		}
	}

	//Put back the default option in the static mesh import data, so next import will have those last import option
	SFbxSceneOptionWindow::CopyFbxOptionsToFbxOptions(GlobalImportSettingsReference, GlobalImportSettings);
	SFbxSceneOptionWindow::CopyFbxOptionsToStaticMeshOptions(GlobalImportSettingsReference, SceneImportOptionsStaticMesh);
	SceneImportOptionsStaticMesh->FillStaticMeshInmportData(StaticMeshImportData, SceneImportOptions);
	SceneImportOptionsStaticMesh->SaveConfig();

	SFbxSceneOptionWindow::CopyFbxOptionsToSkeletalMeshOptions(GlobalImportSettingsReference, SceneImportOptionsSkeletalMesh);
	SceneImportOptionsSkeletalMesh->FillSkeletalMeshInmportData(SkeletalMeshImportData, AnimSequenceImportData, SceneImportOptions);
	SceneImportOptionsSkeletalMesh->SaveConfig();

	//Update the blueprint
	UBlueprint *ReimportBlueprint = nullptr;
	if (bCanReimportHierarchy && GlobalImportSettingsReference->bImportScene)
	{
		ReimportBlueprint = UpdateOriginalBluePrint(ReimportData->BluePrintFullName, &NodeStatusMap, SceneInfoPtr, ReimportData->SceneInfoSourceData, AssetDataToDelete);
	}

	//Remove the deleted meshinfo node from the reimport data
	TArray<TSharedPtr<FFbxMeshInfo>> ToRemoveHierarchyNode;
	for (TSharedPtr<FFbxMeshInfo> MeshInfo : ReimportData->SceneInfoSourceData->MeshInfo)
	{
		if (MeshStatusMap.Contains(MeshInfo->OriginalImportPath))
		{
			EFbxSceneReimportStatusFlags MeshStatus = *(MeshStatusMap.Find(MeshInfo->OriginalImportPath));
			if ((MeshStatus & EFbxSceneReimportStatusFlags::Removed) != EFbxSceneReimportStatusFlags::None)
			{
				ToRemoveHierarchyNode.Add(MeshInfo);
			}
		}
	}
	for (TSharedPtr<FFbxMeshInfo> MeshInfo : ToRemoveHierarchyNode)
	{
		ReimportData->SceneInfoSourceData->MeshInfo.Remove(MeshInfo);
	}
	ReimportData->Modify();
	ReimportData->PostEditChange();
	
	//Make sure the content browser is in sync before we delete
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	ContentBrowserModule.Get().SyncBrowserToAssets(AssetToSyncContentBrowser);

	//No asset should have the pending kill flags before starting a delete operation, since the dependencies GC will store and restore UObject flags
	//And it will assert when restoring Pending kill flag.
	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

	if (AssetDataToDelete.Num() > 0)
	{
		bool AbortDelete = false;
		if (ReimportBlueprint != nullptr)
		{
			//Save the blueprint to avoid reference from the old blueprint
			FAssetData ReimportBlueprintAsset(ReimportBlueprint);
			TArray<UPackage*> Packages;
			Packages.Add(ReimportBlueprintAsset.GetPackage());
			FEditorFileUtils::PromptForCheckoutAndSave(Packages, false, false);

			//Make sure the Asset registry is up to date after the save
			TArray<FString> Paths;
			Paths.Add(ReimportBlueprintAsset.PackagePath.ToString());
			AssetRegistryModule.Get().ScanPathsSynchronous(Paths, true);
		}

		if (!AbortDelete)
		{
			//Delete the asset and use the normal dialog to make sure the user understands they will remove some content.
			//The user can decide to cancel the delete or not. This will not interrupt the reimport process.
			//The delete is done at the end because we want to remove the Blueprint reference before deleting the object.
			ObjectTools::DeleteAssets(AssetDataToDelete, !GIsRunningUnattendedScript);
		}
	}
	//Make sure the content browser is in sync
	ContentBrowserModule.Get().SyncBrowserToAssets(AssetToSyncContentBrowser);
	
	AllNewAssets.Empty();
	
	GlobalImportSettings = nullptr;
	GlobalImportSettingsReference = nullptr;

	FbxImporter->ReleaseScene();
	FbxImporter = nullptr;
	GWarn->EndSlowTask();
	return EReimportResult::Succeeded;
}

void UReimportFbxSceneFactory::ScriptReimportHelper(UObject* Obj)
{
	//Scripted reimport should be consider unattended
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);
	Reimport(Obj);
}

void UReimportFbxSceneFactory::RemoveChildNodeRecursively(USimpleConstructionScript* SimpleConstructionScript, USCS_Node* ScsNode)
{
	TArray<USCS_Node*> ChildNodes = ScsNode->GetChildNodes();
	for (USCS_Node* ChildNode : ChildNodes)
	{
		RemoveChildNodeRecursively(SimpleConstructionScript, ChildNode);
	}
	SimpleConstructionScript->RemoveNode(ScsNode);
}

void UReimportFbxSceneFactory::StoreImportedSpecializeComponentData(USceneComponent *SceneComponent, FSpecializeComponentData &SpecializeComponentData, UClass *CurrentNodeComponentClass)
{
	SpecializeComponentData.NodeTransform = SceneComponent->GetRelativeTransform();

	if (CurrentNodeComponentClass == UPointLightComponent::StaticClass())
	{
		UPointLightComponent *LightComponent = Cast<UPointLightComponent>(SceneComponent);
		SpecializeComponentData.LightColor = LightComponent->LightColor;
		SpecializeComponentData.LightIntensity = LightComponent->Intensity;
		SpecializeComponentData.FarAttenuation = LightComponent->AttenuationRadius;
		SpecializeComponentData.CastShadow = LightComponent->CastShadows;
	}
	else if (CurrentNodeComponentClass == UDirectionalLightComponent::StaticClass())
	{
		UDirectionalLightComponent *LightComponent = Cast<UDirectionalLightComponent>(SceneComponent);
		SpecializeComponentData.LightColor = LightComponent->LightColor;
		SpecializeComponentData.LightIntensity = LightComponent->Intensity;
		SpecializeComponentData.CastShadow = LightComponent->CastShadows;
	}
	else if (CurrentNodeComponentClass == USpotLightComponent::StaticClass())
	{
		USpotLightComponent *LightComponent = Cast<USpotLightComponent>(SceneComponent);
		SpecializeComponentData.LightColor = LightComponent->LightColor;
		SpecializeComponentData.LightIntensity = LightComponent->Intensity;
		SpecializeComponentData.InnerAngle = LightComponent->InnerConeAngle;
		SpecializeComponentData.OuterAngle = LightComponent->OuterConeAngle;
		SpecializeComponentData.FarAttenuation = LightComponent->AttenuationRadius;
		SpecializeComponentData.CastShadow = LightComponent->CastShadows;
	}
	else if (CurrentNodeComponentClass == UCameraComponent::StaticClass())
	{
		UCameraComponent *CameraComponent = Cast<UCameraComponent>(SceneComponent);
		SpecializeComponentData.ProjectionMode = CameraComponent->ProjectionMode;
		SpecializeComponentData.AspectRatio = CameraComponent->AspectRatio;
		SpecializeComponentData.OrthoNearPlane = CameraComponent->OrthoNearClipPlane;
		SpecializeComponentData.OrthoFarPlane = CameraComponent->OrthoFarClipPlane;
		SpecializeComponentData.OrthoWidth = CameraComponent->OrthoWidth;
	}
}

void UReimportFbxSceneFactory::RestoreImportedSpecializeComponentData(USceneComponent *SceneComponent, const FSpecializeComponentData &SpecializeComponentData, UClass *CurrentNodeComponentClass)
{
	SceneComponent->SetRelativeTransform(SpecializeComponentData.NodeTransform);

	if (CurrentNodeComponentClass == UPointLightComponent::StaticClass())
	{
		UPointLightComponent *LightComponent = Cast<UPointLightComponent>(SceneComponent);
		LightComponent->SetLightColor(SpecializeComponentData.LightColor);
		LightComponent->SetIntensity(SpecializeComponentData.LightIntensity);
		LightComponent->SetAttenuationRadius(SpecializeComponentData.FarAttenuation);
		LightComponent->SetCastShadows(SpecializeComponentData.CastShadow);
	}
	else if (CurrentNodeComponentClass == UDirectionalLightComponent::StaticClass())
	{
		UDirectionalLightComponent *LightComponent = Cast<UDirectionalLightComponent>(SceneComponent);
		LightComponent->SetLightColor(SpecializeComponentData.LightColor);
		LightComponent->SetIntensity(SpecializeComponentData.LightIntensity);
		LightComponent->SetCastShadows(SpecializeComponentData.CastShadow);
	}
	else if (CurrentNodeComponentClass == USpotLightComponent::StaticClass())
	{
		USpotLightComponent *LightComponent = Cast<USpotLightComponent>(SceneComponent);
		LightComponent->SetLightColor(SpecializeComponentData.LightColor);
		LightComponent->SetIntensity(SpecializeComponentData.LightIntensity);
		LightComponent->SetAttenuationRadius(SpecializeComponentData.FarAttenuation);
		LightComponent->SetCastShadows(SpecializeComponentData.CastShadow);
		LightComponent->SetInnerConeAngle(SpecializeComponentData.InnerAngle);
		LightComponent->SetOuterConeAngle(SpecializeComponentData.OuterAngle);
	}
	else if (CurrentNodeComponentClass == UCameraComponent::StaticClass())
	{
		UCameraComponent *CameraComponent = Cast<UCameraComponent>(SceneComponent);
		CameraComponent->SetProjectionMode(SpecializeComponentData.ProjectionMode);
		CameraComponent->SetAspectRatio(SpecializeComponentData.AspectRatio);
		CameraComponent->SetOrthoNearClipPlane(SpecializeComponentData.OrthoNearPlane);
		CameraComponent->SetOrthoFarClipPlane(SpecializeComponentData.OrthoFarPlane);
		CameraComponent->SetOrthoWidth(SpecializeComponentData.OrthoWidth);
	}
}

void UReimportFbxSceneFactory::RecursivelySetComponentProperties(USCS_Node* CurrentNode, const TArray<UActorComponent*>& ActorComponents, TArray<FString> ParentNames, bool IsDefaultSceneNode)
{
	UActorComponent *CurrentNodeActorComponent = CurrentNode->ComponentTemplate;
	if (!CurrentNodeActorComponent) //We need a component
		return;

	int32 IndexTemplateSuffixe = CurrentNodeActorComponent->GetName().Find(USimpleConstructionScript::ComponentTemplateNameSuffix);
	bool NameContainTemplateSuffixe = IndexTemplateSuffixe != INDEX_NONE;
	FString NodeName = CurrentNodeActorComponent->GetName();
	FString ReduceNodeName = NodeName;
	if (NameContainTemplateSuffixe)
	{
		ReduceNodeName.LeftInline(IndexTemplateSuffixe, EAllowShrinking::No);
	}

	USceneComponent *CurrentNodeSceneComponent = Cast<USceneComponent>(CurrentNodeActorComponent);
	UClass *CurrentNodeComponentClass = CurrentNodeActorComponent->GetClass();
	FString DefaultSceneRootVariableName = USceneComponent::GetDefaultSceneRootVariableName().ToString();
	for (UActorComponent *ActorComponent : ActorComponents)
	{
		TArray<FString> ComponentParentNames;
		FString ComponentName = ActorComponent->GetName();
		if (IsDefaultSceneNode)
		{
			if (!NodeName.StartsWith(ComponentName))
				continue;

			if (ReduceNodeName.Len() > ComponentName.Len() && (!ReduceNodeName.RightChop(ComponentName.Len()).IsNumeric()))
				continue;
		}

		if (NameContainTemplateSuffixe)
		{
			ComponentName += USimpleConstructionScript::ComponentTemplateNameSuffix;
		}
		USceneComponent *SceneComponent = Cast<USceneComponent>(ActorComponent);
		if (!SceneComponent) //We support only scene component
			continue;

		if (CurrentNodeComponentClass != SceneComponent->GetClass())
			continue;
		
		if (!IsDefaultSceneNode && NodeName.Compare(ComponentName) != 0)
			continue;

		USceneComponent *ParentComponent = SceneComponent->GetAttachParent();
		while (ParentComponent)
		{
			FString ComponentParentName = ParentComponent->GetName();
			if (NameContainTemplateSuffixe)
			{
				ComponentParentName += USimpleConstructionScript::ComponentTemplateNameSuffix;
			}
			ComponentParentNames.Insert(ComponentParentName, 0);
			ParentComponent = ParentComponent->GetAttachParent();
		}
		if (ComponentParentNames.Num() != ParentNames.Num())
		{
			continue;
		}
		bool ParentHierarchyDiffer = false;
		for (int32 SCSParentNameIndex = 0; SCSParentNameIndex < ParentNames.Num(); ++SCSParentNameIndex)
		{
			if (ParentNames[SCSParentNameIndex].Compare(ComponentParentNames[SCSParentNameIndex]) != 0)
			{
				ParentHierarchyDiffer = true;
				break;
			}
		}
		if (ParentHierarchyDiffer)
			continue;

		NodeName = ComponentName;
		
		bool bShouldSerializeProperty = true;
		//If the staticmesh or the skeletal mesh change, we don't want to keep the component value
		if (CurrentNodeComponentClass == UStaticMeshComponent::StaticClass())
		{
			UStaticMeshComponent *CurrentNodeMeshComponent = Cast<UStaticMeshComponent>(CurrentNodeSceneComponent);
			UStaticMeshComponent *MeshComponent = Cast<UStaticMeshComponent>(SceneComponent);
			if (CurrentNodeMeshComponent->GetStaticMesh() != MeshComponent->GetStaticMesh())
				bShouldSerializeProperty = false;
		}
		else if (CurrentNodeComponentClass == USkeletalMeshComponent::StaticClass())
		{
			USkeletalMeshComponent *CurrentNodeMeshComponent = Cast<USkeletalMeshComponent>(CurrentNodeSceneComponent);
			USkeletalMeshComponent *MeshComponent = Cast<USkeletalMeshComponent>(SceneComponent);
			if (CurrentNodeMeshComponent->GetSkeletalMeshAsset() != MeshComponent->GetSkeletalMeshAsset())
				bShouldSerializeProperty = false;
		}

		if (bShouldSerializeProperty)
		{
			//Store some component data we always re-import, this is some component data user will always loose is modification when re-importing
			//a blueprint hierarchy.
			FSpecializeComponentData SpecializeComponentData;
			StoreImportedSpecializeComponentData(SceneComponent, SpecializeComponentData, CurrentNodeComponentClass);

			//We have a match copy all component property from the scs node to the actor component
			TArray<uint8> Data;
			// Serialize the original property
			FObjectWriter Ar(CurrentNodeSceneComponent, Data);
			// Deserialize original value in the new component
			FObjectReader(SceneComponent, Data);
			
			//Update the component to world so we can restore the relative value of the transform
			SceneComponent->UpdateComponentToWorld();

			//Restore the re-import mandatory data
			RestoreImportedSpecializeComponentData(SceneComponent, SpecializeComponentData, CurrentNodeComponentClass);
		}

		//We found the node no need to go further
		break;
	}
	ParentNames.Add(NodeName);
	
	for (USCS_Node* ChildNode : CurrentNode->GetChildNodes())
	{
		RecursivelySetComponentProperties(ChildNode, ActorComponents, ParentNames, false);
	}
}

UBlueprint *UReimportFbxSceneFactory::UpdateOriginalBluePrint(FString &BluePrintFullName, void* VoidNodeStatusMapPtr, TSharedPtr<FFbxSceneInfo> SceneInfoPtr, TSharedPtr<FFbxSceneInfo> SceneInfoOriginalPtr, TArray<FAssetData> &AssetDataToDelete)
{
	if (!SceneInfoPtr.IsValid() || VoidNodeStatusMapPtr == nullptr || !SceneInfoOriginalPtr.IsValid() || BluePrintFullName.IsEmpty())
	{
		return nullptr;
	}

	FbxSceneReimportStatusMapPtr NodeStatusMapPtr = (FbxSceneReimportStatusMapPtr)VoidNodeStatusMapPtr;
	//Find the BluePrint
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	FAssetData BlueprintAssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(BluePrintFullName));

	UPackage* PkgExist = FindPackage(nullptr, *BlueprintAssetData.PackageName.ToString());
	if (PkgExist == nullptr)
	{
		PkgExist = LoadPackage(nullptr, *BlueprintAssetData.PackageName.ToString(), LOAD_Verify | LOAD_NoWarn);
		if (PkgExist == nullptr)
		{
			return nullptr;
		}
	}
	//Load the package before searching the asset
	PkgExist->FullyLoad();
	UBlueprint* BluePrint = FindObjectSafe<UBlueprint>(nullptr, *BluePrintFullName);
	if (BluePrint == nullptr)
	{
		return nullptr;
	}
	if (GEditor)
	{
		//Close all editor that edit this blueprint
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(BluePrint);
		//Make sure all editor selection are cleared
		GEditor->ResetAllSelectionSets();
	}
	//Set the import status for the next reimport
	for (TSharedPtr<FFbxNodeInfo> NodeInfo : SceneInfoPtr->HierarchyInfo)
	{
		if (!NodeStatusMapPtr->Contains(NodeInfo->NodeHierarchyPath))
			continue;
		EFbxSceneReimportStatusFlags NodeStatus = *(NodeStatusMapPtr->Find(NodeInfo->NodeHierarchyPath));
		NodeInfo->bImportNode = (NodeStatus & EFbxSceneReimportStatusFlags::ReimportAsset) != EFbxSceneReimportStatusFlags::None;
	}
	//Add back the component that was in delete state but no flag for reimport
	for (TSharedPtr<FFbxNodeInfo> OriginalNodeInfo : SceneInfoOriginalPtr->HierarchyInfo)
	{
		if (!NodeStatusMapPtr->Contains(OriginalNodeInfo->NodeHierarchyPath))
		{
			continue;
		}

		EFbxSceneReimportStatusFlags NodeStatus = *(NodeStatusMapPtr->Find(OriginalNodeInfo->NodeHierarchyPath));
		if (OriginalNodeInfo->bImportNode != true || (NodeStatus & EFbxSceneReimportStatusFlags::ReimportAsset) != EFbxSceneReimportStatusFlags::None)
		{
			continue;
		}

		//Clear the child
		OriginalNodeInfo->Childrens.Empty();

		//hook the node to the new hierarchy parent
		bool bFoundParent = false;
		if (OriginalNodeInfo->ParentNodeInfo.IsValid())
		{
			int32 InsertIndex = 0;
			for (TSharedPtr<FFbxNodeInfo> NodeInfo : SceneInfoPtr->HierarchyInfo)
			{
				InsertIndex++;
				if (NodeInfo->bImportNode && NodeInfo->NodeHierarchyPath.Compare(OriginalNodeInfo->ParentNodeInfo->NodeHierarchyPath) == 0)
				{
					OriginalNodeInfo->ParentNodeInfo = NodeInfo;
					NodeInfo->Childrens.Add(OriginalNodeInfo);
					SceneInfoPtr->HierarchyInfo.Insert(OriginalNodeInfo, InsertIndex);
					bFoundParent = true;
					break;
				}
			}
		}

		if (!bFoundParent)
		{
			//Insert after the root node
			OriginalNodeInfo->ParentNodeInfo = nullptr;
			SceneInfoPtr->HierarchyInfo.Insert(OriginalNodeInfo, 1);
		}
	}
	//Create a brand new actor with the correct component hierarchy then replace the existing blueprint
	//This function is using the bImportNode flag not the EFbxSceneReimportStatusFlags
	AActor *HierarchyActor = CreateActorComponentsHierarchy(SceneInfoPtr);
	if (HierarchyActor != nullptr)
	{
		//Modify the current blueprint to reflect the new actor
		//Clear all nodes by removing all root node
		TArray<USCS_Node*> BluePrintRootNodes = BluePrint->SimpleConstructionScript->GetRootNodes();
		//Save the property value of every node
		TArray<FString> ParentNames;
		for (USCS_Node* RootNode : BluePrintRootNodes)
		{
			RecursivelySetComponentProperties(RootNode, HierarchyActor->GetInstanceComponents(), ParentNames, true);
		}

		for(USCS_Node* RootNode : BluePrintRootNodes)
		{
			RemoveChildNodeRecursively(BluePrint->SimpleConstructionScript, RootNode);
		}
		//We want to avoid name reservation so we compile the blueprint after removing all node
		FKismetEditorUtilities::CompileBlueprint(BluePrint);

		//Create the new nodes from the hierarchy actor
		FKismetEditorUtilities::FAddComponentsToBlueprintParams Params;
		Params.bKeepMobility = true;
		FKismetEditorUtilities::AddComponentsToBlueprint(BluePrint, HierarchyActor->GetInstanceComponents(), Params);
		
		UWorld* World = HierarchyActor->GetWorld();
		World->DestroyActor(HierarchyActor);
		
		//Make sure we do not leave any pending kill assets
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

		GEngine->BroadcastLevelActorListChanged();

		FBlueprintEditorUtils::MarkBlueprintAsModified(BluePrint);
		FKismetEditorUtilities::CompileBlueprint(BluePrint);
		BluePrint->Modify();
		BluePrint->PostEditChange();
		AssetToSyncContentBrowser.Add(BluePrint);
		return BluePrint;
	}
	return nullptr;
}

EReimportResult::Type UReimportFbxSceneFactory::ImportSkeletalMesh(void* VoidRootNodeToImport, void* VoidFbxImporter, TSharedPtr<FFbxMeshInfo> MeshInfo, TSharedPtr<FFbxSceneInfo> SceneInfoPtr)
{
	UnFbx::FFbxImporter* FbxImporter = (UnFbx::FFbxImporter*)VoidFbxImporter;
	FbxNode *RootNodeToImport = (FbxNode *)VoidRootNodeToImport;
	//GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, UStaticMesh::StaticClass(), GWorld, FName(""), TEXT("fbx"));
	FbxNode *GeometryParentNode = nullptr;
	//Get the first parent geometry node
	for (int idx = 0; idx < FbxImporter->Scene->GetGeometryCount(); ++idx)
	{
		FbxGeometry *Geometry = FbxImporter->Scene->GetGeometry(idx);
		if (Geometry->GetUniqueID() == MeshInfo->UniqueId)
		{
			GeometryParentNode = Geometry->GetNode();
			break;
		}
	}
	if (GeometryParentNode == nullptr)
	{
		FbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(FText::FromString("Reimport Mesh {0} fail, the mesh dont have any parent node inside the fbx."), FText::FromString(MeshInfo->GetImportPath()))), FName(TEXT("Reimport Fbx Scene")));
		return EReimportResult::Failed;
	}

	FString PackageName = MeshInfo->GetImportPath();
	FString StaticMeshName;
	UPackage* Pkg = CreatePackageForNode(PackageName, StaticMeshName);
	if (Pkg == nullptr)
	{
		return EReimportResult::Failed;
	}
	ApplyMeshInfoFbxOptions(MeshInfo);
	
	//TODO support bBakePivotInVertex
	bool Old_bBakePivotInVertex = GlobalImportSettings->bBakePivotInVertex;
	GlobalImportSettings->bBakePivotInVertex = false;
	GlobalImportSettings->bImportBoneTracks = true;
	//if (GlobalImportSettings->bBakePivotInVertex && MeshInfo->PivotNodeUid == INVALID_UNIQUE_ID)
	//{
		//GlobalImportSettings->bBakePivotInVertex = false;
	//}

	TArray< TArray<FbxNode*>* > SkelMeshArray;
	FbxImporter->FillFbxSkelMeshArrayInScene(RootNodeToImport, SkelMeshArray, false, GlobalImportSettings->bImportAsSkeletalGeometry || GlobalImportSettings->bImportAsSkeletalSkinning, true);
	UObject* NewObject = nullptr;
	for (int32 i = 0; i < SkelMeshArray.Num(); i++)
	{
		TArray<FbxNode*> NodeArray = *SkelMeshArray[i];
		FbxNode* RootNodeArrayNode = NodeArray[0];
		TSharedPtr<FFbxNodeInfo> RootNodeInfo;
		if (!FindSceneNodeInfo(SceneInfoPtr, RootNodeArrayNode->GetUniqueID(), RootNodeInfo))
		{
			continue;
		}
		if (!RootNodeInfo->AttributeInfo.IsValid() || RootNodeInfo->AttributeInfo->GetType() != USkeletalMesh::StaticClass())
		{
			continue;
		}
		TSharedPtr<FFbxMeshInfo> RootMeshInfo = StaticCastSharedPtr<FFbxMeshInfo>(RootNodeInfo->AttributeInfo);
		if (!RootMeshInfo.IsValid() || RootMeshInfo->UniqueId != MeshInfo->UniqueId)
		{
			continue;
		}

		TArray<void*> VoidNodeArray;
		for (FbxNode *Node : NodeArray)
		{
			void* VoidNode = (void*)Node;
			VoidNodeArray.Add(VoidNode);
		}
		int32 TotalNumNodes = 0;
		
		NewObject = ImportOneSkeletalMesh(VoidRootNodeToImport, VoidFbxImporter, SceneInfoPtr, RF_Public | RF_Standalone, VoidNodeArray, TotalNumNodes);
	}

	GlobalImportSettings->bBakePivotInVertex = Old_bBakePivotInVertex;

	for (int32 i = 0; i < SkelMeshArray.Num(); i++)
	{
		delete SkelMeshArray[i];
	}

	if (NewObject == nullptr)
	{
		return EReimportResult::Failed;
	}
	AllNewAssets.Add(MeshInfo, NewObject);
	AssetToSyncContentBrowser.Add(NewObject);
	return EReimportResult::Succeeded;
}

EReimportResult::Type UReimportFbxSceneFactory::ImportStaticMesh(void* VoidFbxImporter, TSharedPtr<FFbxMeshInfo> MeshInfo, TSharedPtr<FFbxSceneInfo> SceneInfoPtr)
{
	UnFbx::FFbxImporter* FbxImporter = (UnFbx::FFbxImporter*)VoidFbxImporter;
	//GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, UStaticMesh::StaticClass(), GWorld, FName(""), TEXT("fbx"));
	FbxNode *GeometryParentNode = nullptr;
	//Get the first parent geometry node
	for (int idx = 0; idx < FbxImporter->Scene->GetGeometryCount(); ++idx)
	{
		FbxGeometry *Geometry = FbxImporter->Scene->GetGeometry(idx);
		if (Geometry->GetUniqueID() == MeshInfo->UniqueId)
		{
			GeometryParentNode = Geometry->GetNode();
			break;
		}
	}
	if (GeometryParentNode == nullptr)
	{
		FbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(FText::FromString("Reimport Mesh {0} fail, the mesh dont have any parent node inside the fbx."), FText::FromString(MeshInfo->GetImportPath()))), FName(TEXT("Reimport Fbx Scene")));
		return EReimportResult::Failed;
	}

	FString PackageName = MeshInfo->GetImportPath();
	FString StaticMeshName;
	UPackage* Pkg = CreatePackageForNode(PackageName, StaticMeshName);
	if (Pkg == nullptr)
	{
		return EReimportResult::Failed;
	}

	//Copy default options to StaticMeshImportData
	SFbxSceneOptionWindow::CopyFbxOptionsToStaticMeshOptions(GlobalImportSettingsReference, SceneImportOptionsStaticMesh);
	SceneImportOptionsStaticMesh->FillStaticMeshInmportData(StaticMeshImportData, SceneImportOptions);

	UnFbx::FBXImportOptions* OverrideImportSettings = GetOptionsFromName(MeshInfo->OptionName);
	if (OverrideImportSettings != nullptr)
	{
		SFbxSceneOptionWindow::CopyFbxOptionsToFbxOptions(OverrideImportSettings, GlobalImportSettings);
		SFbxSceneOptionWindow::CopyFbxOptionsToStaticMeshOptions(OverrideImportSettings, SceneImportOptionsStaticMesh);
	}
	else
	{
		SFbxSceneOptionWindow::CopyFbxOptionsToFbxOptions(GlobalImportSettingsReference, GlobalImportSettings);
		SFbxSceneOptionWindow::CopyFbxOptionsToStaticMeshOptions(GlobalImportSettingsReference, SceneImportOptionsStaticMesh);
	}
	SceneImportOptionsStaticMesh->FillStaticMeshInmportData(StaticMeshImportData, SceneImportOptions);
	//Override the pivot bake option
	if (GlobalImportSettings->bBakePivotInVertex && MeshInfo->PivotNodeUid == INVALID_UNIQUE_ID)
	{
		GlobalImportSettings->bBakePivotInVertex = false;
	}
	FName StaticMeshFName = FName(*(MeshInfo->Name));

	UStaticMesh *NewObject = nullptr;
	FbxNode* NodeParent = FbxImporter->RecursiveFindParentLodGroup(GeometryParentNode->GetParent());
	if (NodeParent && NodeParent->GetNodeAttribute() && NodeParent->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup)
	{
		TArray<UStaticMesh*> BaseMeshes;
		TArray<FbxNode*> AllNodeInLod;
		FbxImporter->FindAllLODGroupNode(AllNodeInLod, NodeParent, 0);
		NewObject = FbxImporter->ImportStaticMeshAsSingle(Pkg, AllNodeInLod, StaticMeshFName, RF_Public | RF_Standalone, StaticMeshImportData, nullptr, 0);
		if (NewObject)
		{
			// import LOD meshes
			for (int32 LODIndex = 1; LODIndex < NodeParent->GetChildCount(); LODIndex++)
			{
				AllNodeInLod.Empty();
				FbxImporter->FindAllLODGroupNode(AllNodeInLod, NodeParent, LODIndex);
				FbxImporter->ImportStaticMeshAsSingle(Pkg, AllNodeInLod, StaticMeshFName, RF_Public | RF_Standalone, StaticMeshImportData, NewObject, LODIndex);
			}
			FbxImporter->FindAllLODGroupNode(AllNodeInLod, NodeParent, 0);
			FbxImporter->PostImportStaticMesh(NewObject, AllNodeInLod);
		}
	}
	else
	{
		NewObject = FbxImporter->ImportStaticMesh(Pkg, GeometryParentNode, StaticMeshFName, RF_Public | RF_Standalone, StaticMeshImportData);
		if (NewObject != nullptr)
		{
			TArray<FbxNode*> AllNodeInLod;
			AllNodeInLod.Add(GeometryParentNode);
			FbxImporter->PostImportStaticMesh(NewObject, AllNodeInLod);
		}
	}
	if (NewObject == nullptr)
	{
		if (Pkg != nullptr)
		{
			Pkg->SetDirtyFlag(false);
			Pkg->RemoveFromRoot();
			Pkg->ConditionalBeginDestroy();
		}
		return EReimportResult::Failed;
	}
	else
	{

		//Mark any re-imported package dirty
		NewObject->MarkPackageDirty();
	}

	AllNewAssets.Add(MeshInfo, NewObject);
	AssetToSyncContentBrowser.Add(NewObject);
	return EReimportResult::Succeeded;
}

EReimportResult::Type UReimportFbxSceneFactory::ReimportSkeletalMesh(void* VoidFbxImporter, TSharedPtr<FFbxMeshInfo> MeshInfo)
{
	UnFbx::FFbxImporter* FbxImporter = (UnFbx::FFbxImporter*)VoidFbxImporter;
	//Find the UObject associate with this MeshInfo
	UPackage* PkgExist = MeshInfo->GetContentPackage();

	FString AssetName = MeshInfo->GetFullImportName();
	USkeletalMesh* Mesh = FindFirstObject<USkeletalMesh>(*AssetName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("ReimportSkeletalMesh"));
	if (Mesh == nullptr)
	{
		//We reimport only skeletal mesh here
		FbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(FText::FromString("Reimport Mesh {0} fail, the original skeletalmesh in the content browser cannot be load."), FText::FromString(MeshInfo->GetImportPath()))), FName(TEXT("Reimport Fbx Scene")));
		return EReimportResult::Failed;
	}

	const TArray<UAssetUserData*>* UserData = Mesh->GetAssetUserDataArray();
	TMap<UAssetUserData*, bool> UserDataCopy;
	if (UserData)
	{
		for (int32 Idx = 0; Idx < UserData->Num(); Idx++)
		{
			if ((*UserData)[Idx] != nullptr)
			{
				UAssetUserData* DupObject = (UAssetUserData*)StaticDuplicateObject((*UserData)[Idx], GetTransientPackage());
				bool bAddDupToRoot = !(DupObject->IsRooted());
				if (bAddDupToRoot)
				{
					DupObject->AddToRoot();
				}
				UserDataCopy.Add(DupObject, bAddDupToRoot);
			}
		}
	}

	ApplyMeshInfoFbxOptions(MeshInfo);
	//TODO support bBakePivotInVertex
	bool Old_bBakePivotInVertex = GlobalImportSettings->bBakePivotInVertex;
	GlobalImportSettings->bBakePivotInVertex = false;
	GlobalImportSettings->bImportBoneTracks = true;
	//if (GlobalImportSettings->bBakePivotInVertex && MeshInfo->PivotNodeUid == INVALID_UNIQUE_ID)
	//{
		//GlobalImportSettings->bBakePivotInVertex = false;
	//}
	TArray<FbxNode*> OutSkeletalMeshArray;
	EReimportResult::Type ReimportResult = EReimportResult::Succeeded;
	if (FbxImporter->ReimportSkeletalMesh(Mesh, SkeletalMeshImportData, MeshInfo->UniqueId, &OutSkeletalMeshArray))
	{
		Mesh->GetAssetImportData()->Update(FbxImportFileName);

		// Copy user data to newly created mesh
		for (auto Kvp : UserDataCopy)
		{
			UAssetUserData* UserDataObject = Kvp.Key;
			if (Kvp.Value)
			{
				//if the duplicated temporary UObject was add to root, we must remove it from the root
				UserDataObject->RemoveFromRoot();
			}
			UserDataObject->Rename(nullptr, Mesh, REN_DontCreateRedirectors | REN_DoNotDirty);
			Mesh->AddAssetUserData(UserDataObject);
		}

		// Try to find the outer package so we can dirty it up
		if (Mesh->GetOuter())
		{
			Mesh->GetOuter()->MarkPackageDirty();
		}
		else
		{
			Mesh->MarkPackageDirty();
		}
		AllNewAssets.Add(MeshInfo, Mesh);
		AssetToSyncContentBrowser.Add(Mesh);
		
		//TODO reimport all animation
		//1. Store all anim sequence reference that was originally import, for every skeletal mesh
		//2. On reimport match the existing one
		//3. Reimport matching animation
		if (GlobalImportSettings->bImportAnimations && ensure(Mesh->GetSkeleton()))
		{
			TArray<FbxNode*> FBXMeshNodeArray;
			FbxNode* SkeletonRoot = FbxImporter->FindFBXMeshesByBone(Mesh->GetSkeleton()->GetReferenceSkeleton().GetBoneName(0), true, FBXMeshNodeArray);

			FString AnimName = FbxImporter->MakeNameForMesh(FBXMeshNodeArray[0]->GetName(), FBXMeshNodeArray[0]).ToString();
			AnimName = (GlobalImportSettings->AnimationName != "") ? GlobalImportSettings->AnimationName : AnimName + TEXT("_Anim");

			TArray<FbxNode*> SortedLinks;
			FbxImporter->RecursiveBuildSkeleton(SkeletonRoot, SortedLinks);

			if (SortedLinks.Num() != 0)
			{
				//Find the number of take
				int32 ResampleRate = static_cast<int32>(DEFAULT_SAMPLERATE);
				if (GlobalImportSettings->bResample)
				{
					if(FbxImporter->ImportOptions->ResampleRate > 0)
					{
						ResampleRate = FbxImporter->ImportOptions->ResampleRate;
					}
					else
					{
						int32 BestResampleRate = FbxImporter->GetMaxSampleRate(SortedLinks);
						if (BestResampleRate > 0)
						{
							ResampleRate = BestResampleRate;
						}
					}
				}
				int32 ValidTakeCount = 0;
				int32 AnimStackCount = FbxImporter->Scene->GetSrcObjectCount<FbxAnimStack>();
				for (int32 AnimStackIndex = 0; AnimStackIndex < AnimStackCount; AnimStackIndex++)
				{
					FbxAnimStack* CurAnimStack = FbxImporter->Scene->GetSrcObject<FbxAnimStack>(AnimStackIndex);

					FbxTimeSpan AnimTimeSpan = FbxImporter->GetAnimationTimeSpan(SortedLinks[0], CurAnimStack);
					bool bValidAnimStack = FbxImporter->ValidateAnimStack(SortedLinks, FBXMeshNodeArray, CurAnimStack, ResampleRate, GlobalImportSettings->bImportMorph, GlobalImportSettings->bSnapToClosestFrameBoundary, AnimTimeSpan);
					// no animation
					if (!bValidAnimStack)
					{
						continue;
					}
					ValidTakeCount++;
				}

				if (ValidTakeCount > 0)
				{
					//Reimport all sequence (reimport existing and import new one)
					AnimStackCount = FbxImporter->Scene->GetSrcObjectCount<FbxAnimStack>();
					for (int32 AnimStackIndex = 0; AnimStackIndex < AnimStackCount; AnimStackIndex++)
					{
						FbxAnimStack* CurAnimStack = FbxImporter->Scene->GetSrcObject<FbxAnimStack>(AnimStackIndex);
						FString SequenceName = AnimName;
						if (ValidTakeCount > 1)
						{
							SequenceName += "_";
							SequenceName += UTF8_TO_TCHAR(CurAnimStack->GetName());
						}

						// See if this sequence already exists.
						SequenceName = ObjectTools::SanitizeObjectName(SequenceName);
						FString 	ParentPath = FString::Printf(TEXT("%s/%s"), *FPackageName::GetLongPackagePath(*Mesh->GetOutermost()->GetName()), *SequenceName);
						//See if the sequence exist
						UAnimSequence *DestSeq = nullptr;
						UPackage *ParentPackage = LoadPackage(nullptr, *ParentPath, LOAD_Verify | LOAD_NoWarn);
						if (ParentPackage != nullptr)
						{
							ParentPackage->FullyLoad();
						}

						UObject* Object = FindFirstObject<UObject>(*SequenceName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("ReimportSkeletalMesh"));
						if (Object != nullptr)
						{
							if (ParentPackage == nullptr)
							{
								ParentPackage = Object->GetOutermost();
								ParentPackage->FullyLoad();
							}
							//Cast into sequence
							DestSeq = Cast<UAnimSequence>(Object);
						}
						
						//Get the sequence timespan
						ResampleRate = static_cast<int32>(DEFAULT_SAMPLERATE);
						if (FbxImporter->ImportOptions->bResample)
						{
							if(FbxImporter->ImportOptions->ResampleRate > 0)
							{
								ResampleRate = FbxImporter->ImportOptions->ResampleRate;
							}
							else
							{
								int32 BestResampleRate = FbxImporter->GetMaxSampleRate(SortedLinks);
								if (BestResampleRate > 0)
								{
									ResampleRate = BestResampleRate;
								}
							}
						}
						FbxTimeSpan AnimTimeSpan = FbxImporter->GetAnimationTimeSpan(SortedLinks[0], CurAnimStack);

						if (DestSeq == nullptr)
						{
							//Import a new sequence
							ParentPackage = CreatePackage( *ParentPath);
							Object = LoadObject<UObject>(ParentPackage, *SequenceName, NULL, LOAD_None, NULL);
							DestSeq = Cast<UAnimSequence>(Object);
							if (Object && !DestSeq)
							{
								FbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("Error_AssetExist", "Asset with same name exists. Can't overwrite another asset")), FFbxErrors::Generic_SameNameAssetExists);
								continue; // Move on to next sequence...
							}
							// If not, create new one now.
							if (!DestSeq)
							{
								DestSeq = NewObject<UAnimSequence>(ParentPackage, *SequenceName, RF_Public | RF_Standalone);
								// Notify the asset registry
								FAssetRegistryModule::AssetCreated(DestSeq);
							}
							DestSeq->SetSkeleton(Mesh->GetSkeleton());
							// since to know full path, reimport will need to do same
							UFbxAnimSequenceImportData* ImportData = UFbxAnimSequenceImportData::GetImportDataForAnimSequence(DestSeq, AnimSequenceImportData);
							ImportData->Update(UFactory::CurrentFilename);
							FbxImporter->ImportAnimation(Mesh->GetSkeleton(), DestSeq, CurrentFilename, SortedLinks, FBXMeshNodeArray, CurAnimStack, ResampleRate, AnimTimeSpan, false);
						}
						else
						{
							//Reimport in a existing sequence
							if (FbxImporter->ValidateAnimStack(SortedLinks, FBXMeshNodeArray, CurAnimStack, ResampleRate, true, FbxImporter->ImportOptions->bSnapToClosestFrameBoundary, AnimTimeSpan))
							{
								FbxImporter->ImportAnimation(Mesh->GetSkeleton(), DestSeq, CurrentFilename, SortedLinks, FBXMeshNodeArray, CurAnimStack, ResampleRate, AnimTimeSpan, true);
							}
						}
					}
				}
			}
		}
	}
	else
	{
		ReimportResult = EReimportResult::Failed;
	}
	GlobalImportSettings->bBakePivotInVertex = Old_bBakePivotInVertex;
	return ReimportResult;
}

EReimportResult::Type UReimportFbxSceneFactory::ReimportStaticMesh(void* VoidFbxImporter, TSharedPtr<FFbxMeshInfo> MeshInfo)
{
	UnFbx::FFbxImporter* FbxImporter = (UnFbx::FFbxImporter*)VoidFbxImporter;
	//Load the UObject associate with this MeshInfo
	MeshInfo->GetContentPackage();

	FString AssetName = MeshInfo->GetFullImportName();
	UStaticMesh* Mesh = FindFirstObject<UStaticMesh>(*AssetName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("ReimportStaticMesh"));
	if (Mesh == nullptr)
	{
		//We reimport only static mesh here
		FbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(FText::FromString("Reimport Mesh {0} fail, the original staicmesh in the content browser cannot be load."), FText::FromString(MeshInfo->GetImportPath()))), FName(TEXT("Reimport Fbx Scene")));
		return EReimportResult::Failed;
	}
	ApplyMeshInfoFbxOptions(MeshInfo);
	//Override the pivot bake option
	bool Old_bBakePivotInVertex = GlobalImportSettings->bBakePivotInVertex;
	if (GlobalImportSettings->bBakePivotInVertex && MeshInfo->PivotNodeUid == INVALID_UNIQUE_ID)
	{
		GlobalImportSettings->bBakePivotInVertex = false;
	}
	EReimportResult::Type ReimportResult = EReimportResult::Succeeded;

	FbxImporter->ApplyTransformSettingsToFbxNode(FbxImporter->Scene->GetRootNode(), StaticMeshImportData);
	const TArray<UAssetUserData*>* UserData = Mesh->GetAssetUserDataArray();
	TMap<UAssetUserData*, bool> UserDataCopy;
	if (UserData)
	{
		for (int32 Idx = 0; Idx < UserData->Num(); Idx++)
		{
			if ((*UserData)[Idx] != nullptr)
			{
				UAssetUserData* DupObject = (UAssetUserData*)StaticDuplicateObject((*UserData)[Idx], GetTransientPackage());
				bool bAddDupToRoot = !(DupObject->IsRooted());
				if (bAddDupToRoot)
				{
					DupObject->AddToRoot();
				}
				UserDataCopy.Add(DupObject, bAddDupToRoot);
			}
		}
	}

	// preserve settings in navcollision subobject
	UNavCollisionBase* NavCollision = Mesh->GetNavCollision() ?
		(UNavCollisionBase*)StaticDuplicateObject(Mesh->GetNavCollision(), GetTransientPackage()) :
		nullptr;

	// preserve extended bound settings
	const FVector PositiveBoundsExtension = Mesh->GetPositiveBoundsExtension();
	const FVector NegativeBoundsExtension = Mesh->GetNegativeBoundsExtension();
	uint64 NodeInfoUid = INVALID_UNIQUE_ID;
	if (GlobalImportSettings->bBakePivotInVertex && MeshInfo->PivotNodeUid != INVALID_UNIQUE_ID)
	{
		NodeInfoUid = MeshInfo->PivotNodeUid;
	}

	Mesh = FbxImporter->ReimportSceneStaticMesh(NodeInfoUid, MeshInfo->UniqueId, Mesh, StaticMeshImportData);
	if (Mesh != nullptr)
	{
		//Put back the new mesh data since the reimport is putting back the original import data
		SceneImportOptionsStaticMesh->FillStaticMeshInmportData(StaticMeshImportData, SceneImportOptions);
		Mesh->SetAssetImportData(StaticMeshImportData);

		// Copy user data to newly created mesh
		for (auto Kvp : UserDataCopy)
		{
			UAssetUserData* UserDataObject = Kvp.Key;
			if (Kvp.Value)
			{
				//if the duplicated temporary UObject was add to root, we must remove it from the root
				UserDataObject->RemoveFromRoot();
			}
			UserDataObject->Rename(nullptr, Mesh, REN_DontCreateRedirectors | REN_DoNotDirty);
			Mesh->AddAssetUserData(UserDataObject);
		}

		if (NavCollision)
		{
			Mesh->SetNavCollision(NavCollision);
			NavCollision->Rename(nullptr, Mesh, REN_DontCreateRedirectors | REN_DoNotDirty);
		}

		// Restore bounds extension settings
		Mesh->SetPositiveBoundsExtension(PositiveBoundsExtension);
		Mesh->SetNegativeBoundsExtension(NegativeBoundsExtension);

		Mesh->GetAssetImportData()->Update(FbxImportFileName);

		// Try to find the outer package so we can dirty it up
		if (Mesh->GetOutermost())
		{
			Mesh->GetOutermost()->MarkPackageDirty();
		}
		else
		{
			Mesh->MarkPackageDirty();
		}
		AllNewAssets.Add(MeshInfo, Mesh);
		AssetToSyncContentBrowser.Add(Mesh);
	}
	else
	{
		ReimportResult = EReimportResult::Failed;
	}
	GlobalImportSettings->bBakePivotInVertex = Old_bBakePivotInVertex;
	return ReimportResult;
}

int32 UReimportFbxSceneFactory::GetPriority() const
{
	return ImportPriority;
}

#undef LOCTEXT_NAMESPACE
