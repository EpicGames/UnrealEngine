// Copyright Epic Games, Inc. All Rights Reserved.
#include "SFbxSceneOptionWindow.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Input/SCheckBox.h"
#include "Styling/AppStyle.h"
#include "Factories/FbxSceneImportOptions.h"
#include "Factories/FbxSceneImportOptionsSkeletalMesh.h"
#include "Factories/FbxSceneImportOptionsStaticMesh.h"
#include "Fbx/SSceneImportNodeTreeView.h"
#include "Fbx/SSceneImportStaticMeshListView.h"
#include "Fbx/SSceneReimportNodeTreeView.h"
#include "Fbx/SSceneSkeletalMeshListView.h"
#include "Fbx/SSceneReimportSkeletalMeshListView.h"
#include "Fbx/SSceneReimportStaticMeshListView.h"
#include "IDocumentation.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/STextComboBox.h"
#include "FbxImporter.h"
#include "Dialogs/DlgPickPath.h"

#define LOCTEXT_NAMESPACE "FBXOption"

SFbxSceneOptionWindow::SFbxSceneOptionWindow()
{
	SceneInfo = nullptr;
	SceneInfoOriginal = nullptr;
	MeshStatusMap = nullptr;
	GlobalImportSettings = nullptr;
	SceneImportOptionsDisplay = nullptr;
	SceneImportOptionsStaticMeshDisplay = nullptr;
	OverrideNameOptionsMap = nullptr;
	SceneImportOptionsSkeletalMeshDisplay = nullptr;
	OwnerWindow = nullptr;
	FbxSceneImportTabManager = nullptr;
	Layout = nullptr;
	bShouldImport = false;
	SceneTabTreeview = nullptr;
	SceneTabDetailsView = nullptr;
	SceneReimportTabDetailsView = nullptr;
	OverrideNameOptions.Empty();
	StaticMeshTabListView = nullptr;
	StaticMeshTabDetailsView = nullptr;
	SkeletalMeshTabListView = nullptr;
	SkeletalMeshTabDetailsView = nullptr;
	SceneReimportTreeview = nullptr;
	StaticMeshReimportListView = nullptr;
	StaticMeshReimportDetailsView = nullptr;
	MaterialsTabListView = nullptr;
	TexturesArray.Reset();
	MaterialBasePath.Empty();
}

SFbxSceneOptionWindow::~SFbxSceneOptionWindow()
{
	if (FbxSceneImportTabManager.IsValid())
	{
		FbxSceneImportTabManager->UnregisterAllTabSpawners();
	}
	SceneInfo = nullptr;
	SceneInfoOriginal = nullptr;
	MeshStatusMap = nullptr;
	GlobalImportSettings = nullptr;
	SceneImportOptionsDisplay = nullptr;
	SceneImportOptionsStaticMeshDisplay = nullptr;
	OverrideNameOptionsMap = nullptr;
	SceneImportOptionsSkeletalMeshDisplay = nullptr;
	OwnerWindow = nullptr;
	FbxSceneImportTabManager = nullptr;
	Layout = nullptr;
	bShouldImport = false;
	SceneTabTreeview = nullptr;
	SceneTabDetailsView = nullptr;
	SceneReimportTabDetailsView = nullptr;
	StaticMeshTabListView = nullptr;
	StaticMeshTabDetailsView = nullptr;
	SkeletalMeshTabListView = nullptr;
	SkeletalMeshTabDetailsView = nullptr;
	SceneReimportTreeview = nullptr;
	StaticMeshReimportListView = nullptr;
	StaticMeshReimportDetailsView = nullptr;
	MaterialsTabListView = nullptr;
	TexturesArray.Reset();
	MaterialBasePath.Empty();
}

TSharedRef<SDockTab> SFbxSceneOptionWindow::SpawnSceneTab(const FSpawnTabArgs& Args)
{
	//Create the treeview
	SceneTabTreeview = SNew(SFbxSceneTreeView)
		.SceneInfo(SceneInfo);

	TSharedPtr<SBox> InspectorBox;
	TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		.Label(LOCTEXT("WidgetFbxSceneActorTab", "Scene"))
		.ToolTipText(LOCTEXT("WidgetFbxSceneActorTabTextToolTip", "Switch to the scene tab."))
		[
			SNew(SSplitter)
			.Orientation(Orient_Horizontal)
			+ SSplitter::Slot()
			.Value(0.4f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.HAlign(HAlign_Left)
				.AutoHeight()
				[
					SNew(SUniformGridPanel)
					.SlotPadding(2.0f)
					+ SUniformGridPanel::Slot(0, 0)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SCheckBox)
							.HAlign(HAlign_Center)
							.OnCheckStateChanged(SceneTabTreeview.Get(), &SFbxSceneTreeView::OnToggleSelectAll)
						]
						+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						.Padding(0.0f, 3.0f, 6.0f, 3.0f)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("FbxOptionWindow_Scene_All", "All"))
						]
					]
					+ SUniformGridPanel::Slot(1, 0)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.Text(LOCTEXT("FbxOptionWindow_Scene_ExpandAll", "Expand All"))
						.OnClicked(SceneTabTreeview.Get(), &SFbxSceneTreeView::OnExpandAll)
					]
					+ SUniformGridPanel::Slot(2, 0)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.Text(LOCTEXT("FbxOptionWindow_Scene_CollapseAll", "Collapse All"))
						.OnClicked(SceneTabTreeview.Get(), &SFbxSceneTreeView::OnCollapseAll)
					]
				]
				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					SNew(SBox)
					[
						SceneTabTreeview.ToSharedRef()
					]
				]
			]
			+ SSplitter::Slot()
			.Value(0.6f)
			[
				SAssignNew(InspectorBox, SBox)
			]
		];
	//Prevent user to close the tab
	DockTab->SetCanCloseTab(SDockTab::FCanCloseTab::CreateRaw(this, &SFbxSceneOptionWindow::CanCloseTab));

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	SceneTabDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	InspectorBox->SetContent(SceneTabDetailsView->AsShared());
	SceneTabDetailsView->SetObject(SceneImportOptionsDisplay);
	SceneTabDetailsView->OnFinishedChangingProperties().AddSP(this, &SFbxSceneOptionWindow::OnFinishedChangingPropertiesSceneTabDetailView);
	return DockTab;
}

bool SFbxSceneOptionWindow::CanCloseTab()
{
	return false;
}

void SFbxSceneOptionWindow::OnFinishedChangingPropertiesSceneTabDetailView(const FPropertyChangedEvent& PropertyChangedEvent)
{
	if (!SceneInfoOriginal.IsValid())
	{
		MaterialsTabListView->SetCreateContentFolderHierarchy(SceneImportOptionsDisplay->bCreateContentFolderHierarchy);
		//Update the MaterialList
		MaterialsTabListView->UpdateMaterialBasePath();
	}
	//Set the Global Import setting
	GlobalImportSettings->bForceFrontXAxis = SceneImportOptionsDisplay->bForceFrontXAxis;
	GlobalImportSettings->bBakePivotInVertex = SceneImportOptionsDisplay->bBakePivotInVertex;
	GlobalImportSettings->bInvertNormalMap = SceneImportOptionsDisplay->bInvertNormalMaps;
	GlobalImportSettings->ImportTranslation = SceneImportOptionsDisplay->ImportTranslation;
	GlobalImportSettings->ImportRotation = SceneImportOptionsDisplay->ImportRotation;
	GlobalImportSettings->ImportUniformScale = SceneImportOptionsDisplay->ImportUniformScale;
}

TSharedRef<SDockTab> SFbxSceneOptionWindow::SpawnStaticMeshTab(const FSpawnTabArgs& Args)
{
	//Create the static mesh listview
	StaticMeshTabListView = SNew(SFbxSceneStaticMeshListView)
		.SceneInfo(SceneInfo)
		.GlobalImportSettings(GlobalImportSettings)
		.OverrideNameOptions(&OverrideNameOptions)
		.OverrideNameOptionsMap(OverrideNameOptionsMap)
		.SceneImportOptionsStaticMeshDisplay(SceneImportOptionsStaticMeshDisplay);

	TSharedPtr<SBox> InspectorBox;
	TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		.Label(LOCTEXT("WidgetFbxSceneStaticMeshTab", "Static Meshes"))
		.ToolTipText(LOCTEXT("WidgetFbxSceneStaticMeshTabTextToolTip", "Switch to the static meshes tab."))
		[
			SNew(SSplitter)
			.Orientation(Orient_Horizontal)
			+ SSplitter::Slot()
			.Value(0.4f)
			[
				SNew(SBox)
				[
					StaticMeshTabListView.ToSharedRef()
				]
			]
			+ SSplitter::Slot()
			.Value(0.6f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						StaticMeshTabListView->CreateOverrideOptionComboBox().ToSharedRef()
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.Text(LOCTEXT("FbxOptionWindow_SM_Select_asset_using", "Select Asset Using"))
						.OnClicked(StaticMeshTabListView.Get(), &SFbxSceneStaticMeshListView::OnSelectAssetUsing)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.Text(LOCTEXT("FbxOptionWindow_SM_CreateOverride", "Create Override"))
						.ToolTipText(LOCTEXT("FbxOptionWindow_SM_CreateOverrideTooltip", "Create Override to specify custom import options for some static meshes.\nTo assign options use context menu on static meshes."))
						.OnClicked(StaticMeshTabListView.Get(), &SFbxSceneStaticMeshListView::OnCreateOverrideOptions)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.Text(LOCTEXT("FbxOptionWindow_SM_Delete", "Delete"))
						.IsEnabled(StaticMeshTabListView.Get(), &SFbxSceneStaticMeshListView::CanDeleteOverride)
						.OnClicked(StaticMeshTabListView.Get(), &SFbxSceneStaticMeshListView::OnDeleteOverride)
					]
				]
				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					SAssignNew(InspectorBox, SBox)
				]
			]
		];

	//Prevent user to close the tab
	DockTab->SetCanCloseTab(SDockTab::FCanCloseTab::CreateRaw(this, &SFbxSceneOptionWindow::CanCloseTab));

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	StaticMeshTabDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	InspectorBox->SetContent(StaticMeshTabDetailsView->AsShared());
	StaticMeshTabDetailsView->SetObject(SceneImportOptionsStaticMeshDisplay);
	StaticMeshTabDetailsView->OnFinishedChangingProperties().AddSP(StaticMeshTabListView.Get(), &SFbxSceneStaticMeshListView::OnFinishedChangingProperties);
	return DockTab;
}

TSharedRef<SDockTab> SFbxSceneOptionWindow::SpawnSkeletalMeshReimportTab(const FSpawnTabArgs& Args)
{

	//Create the Skeletal mesh listview
	SkeletalMeshReimportListView = SNew(SFbxSceneSkeletalMeshReimportListView)
		.SceneInfo(SceneInfo)
		.SceneInfoOriginal(SceneInfoOriginal)
		.GlobalImportSettings(GlobalImportSettings)
		.OverrideNameOptions(&OverrideNameOptions)
		.OverrideNameOptionsMap(OverrideNameOptionsMap)
		.SceneImportOptionsSkeletalMeshDisplay(SceneImportOptionsSkeletalMeshDisplay)
		.MeshStatusMap(MeshStatusMap);

	TSharedPtr<SBox> InspectorBox;
	TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		.Label(LOCTEXT("WidgetFbxSceneReimportSkeletalMeshTab", "Skeletal Meshes"))
		.ToolTipText(LOCTEXT("WidgetFbxSceneReimportSkeletalMeshTabTextToolTip", "Switch to the reimport Skeletal meshes tab."))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.HAlign(HAlign_Left)
					.AutoHeight()
					[
						SNew(SUniformGridPanel)
						.SlotPadding(2.0f)
						+ SUniformGridPanel::Slot(0, 0)
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Left)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("FbxOptionWindow_Scene_Filters_Label", "Filters:"))
						]
						+ SUniformGridPanel::Slot(1, 0)
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
							[
								SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(SCheckBox)
									.HAlign(HAlign_Center)
									.OnCheckStateChanged(SkeletalMeshReimportListView.Get(), &SFbxSceneSkeletalMeshReimportListView::OnToggleFilterAddContent)
									.IsChecked(SkeletalMeshReimportListView.Get(), &SFbxSceneSkeletalMeshReimportListView::IsFilterAddContentChecked)
								]
								+ SHorizontalBox::Slot()
								.FillWidth(1.0f)
								.Padding(0.0f, 3.0f, 6.0f, 3.0f)
								.VAlign(VAlign_Center)
								[
									SNew(STextBlock)
									.Text(LOCTEXT("FbxOptionWindow_Scene_Reimport_Filter_Add_Content", "Add"))
								]
							]
						]
						+ SUniformGridPanel::Slot(2, 0)
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
							[
								SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(SCheckBox)
									.HAlign(HAlign_Center)
									.OnCheckStateChanged(SkeletalMeshReimportListView.Get(), &SFbxSceneSkeletalMeshReimportListView::OnToggleFilterDeleteContent)
									.IsChecked(SkeletalMeshReimportListView.Get(), &SFbxSceneSkeletalMeshReimportListView::IsFilterDeleteContentChecked)
								]
								+ SHorizontalBox::Slot()
								.FillWidth(1.0f)
								.Padding(0.0f, 3.0f, 6.0f, 3.0f)
								.VAlign(VAlign_Center)
								[
									SNew(STextBlock)
									.Text(LOCTEXT("FbxOptionWindow_Scene_Reimport_Filter_Delete_Content", "Delete"))
								]
							]
						]
						+ SUniformGridPanel::Slot(3, 0)
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
							[
								SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(SCheckBox)
									.HAlign(HAlign_Center)
									.OnCheckStateChanged(SkeletalMeshReimportListView.Get(), &SFbxSceneSkeletalMeshReimportListView::OnToggleFilterOverwriteContent)
									.IsChecked(SkeletalMeshReimportListView.Get(), &SFbxSceneSkeletalMeshReimportListView::IsFilterOverwriteContentChecked)
								]
								+ SHorizontalBox::Slot()
								.FillWidth(1.0f)
								.Padding(0.0f, 3.0f, 6.0f, 3.0f)
								.VAlign(VAlign_Center)
								[
									SNew(STextBlock)
									.Text(LOCTEXT("FbxOptionWindow_Scene_Reimport_Filter_Overwrite_Content", "Overwrite"))
								]
							]
						]
						+ SUniformGridPanel::Slot(4, 0)
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
							[
								SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(SCheckBox)
									.HAlign(HAlign_Center)
									.OnCheckStateChanged(SkeletalMeshReimportListView.Get(), &SFbxSceneSkeletalMeshReimportListView::OnToggleFilterDiff)
									.IsChecked(SkeletalMeshReimportListView.Get(), &SFbxSceneSkeletalMeshReimportListView::IsFilterDiffChecked)
								]
								+ SHorizontalBox::Slot()
								.FillWidth(1.0f)
								.Padding(0.0f, 3.0f, 6.0f, 3.0f)
								.VAlign(VAlign_Center)
								[
									SNew(STextBlock)
									.Text(LOCTEXT("FbxOptionWindow_Scene_Reimport_Filter_Diff", "Diff"))
									.ToolTipText(LOCTEXT("FbxOptionWindow_Scene_Reimport_Filter_Diff_Tooltip", "Show every reimport item that dont match between the original fbx and the new one."))
								]
							]
						]
					]
					+ SVerticalBox::Slot()
					.FillHeight(1.0f)
					[
						SNew(SSplitter)
						.Orientation(Orient_Vertical)
						+ SSplitter::Slot()
						.Value(0.4f)
						[
							SNew(SBox)
							[
								SkeletalMeshReimportListView.ToSharedRef()
							]
						]
						+ SSplitter::Slot()
						.Value(0.6f)
						[
							SNew(SVerticalBox)
							+ SVerticalBox::Slot()
							.AutoHeight()
							[
								SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SkeletalMeshReimportListView->CreateOverrideOptionComboBox().ToSharedRef()
								]
								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(SButton)
									.HAlign(HAlign_Center)
									.Text(LOCTEXT("FbxOptionWindow_SM_Select_asset_using", "Select Asset Using"))
									.OnClicked(SkeletalMeshReimportListView.Get(), &SFbxSceneSkeletalMeshReimportListView::OnSelectAssetUsing)
								]
								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(SButton)
									.Text(LOCTEXT("FbxOptionWindow_SKM_CreateOverride", "Create Override"))
									.ToolTipText(LOCTEXT("FbxOptionWindow_SKM_CreateOverrideTooltip", "Create Override to specify custom import options for some Skeletal meshes.\nTo assign options use context menu on Skeletal meshes."))
									.OnClicked(SkeletalMeshReimportListView.Get(), &SFbxSceneSkeletalMeshReimportListView::OnCreateOverrideOptions)
								]
								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(SButton)
									.HAlign(HAlign_Center)
									.Text(LOCTEXT("FbxOptionWindow_SM_Delete", "Delete"))
									.IsEnabled(SkeletalMeshReimportListView.Get(), &SFbxSceneSkeletalMeshReimportListView::CanDeleteOverride)
									.OnClicked(SkeletalMeshReimportListView.Get(), &SFbxSceneSkeletalMeshReimportListView::OnDeleteOverride)
								]
							]
							+ SVerticalBox::Slot()
							.FillHeight(1.0f)
							[
								SAssignNew(InspectorBox, SBox)
							]
						]
					]
				]
			]
		];
	
	//Prevent user to close the tab
	DockTab->SetCanCloseTab(SDockTab::FCanCloseTab::CreateRaw(this, &SFbxSceneOptionWindow::CanCloseTab));

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	SkeletalMeshReimportDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	InspectorBox->SetContent(SkeletalMeshReimportDetailsView->AsShared());
	SkeletalMeshReimportDetailsView->SetObject(SceneImportOptionsSkeletalMeshDisplay);
	SkeletalMeshReimportDetailsView->OnFinishedChangingProperties().AddSP(SkeletalMeshReimportListView.Get(), &SFbxSceneSkeletalMeshReimportListView::OnFinishedChangingProperties);
	return DockTab;
}

TSharedRef<SDockTab> SFbxSceneOptionWindow::SpawnSkeletalMeshTab(const FSpawnTabArgs& Args)
{
	//Create the static mesh listview
	SkeletalMeshTabListView = SNew(SFbxSceneSkeletalMeshListView)
		.SceneInfo(SceneInfo)
		.GlobalImportSettings(GlobalImportSettings)
		.OverrideNameOptions(&OverrideNameOptions)
		.OverrideNameOptionsMap(OverrideNameOptionsMap)
		.SceneImportOptionsSkeletalMeshDisplay(SceneImportOptionsSkeletalMeshDisplay);

	TSharedPtr<SBox> InspectorBox;
	TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		.Label(LOCTEXT("WidgetFbxSceneSkeletalMeshTab", "Skeletal Meshes"))
		.ToolTipText(LOCTEXT("WidgetFbxSceneSkeletalMeshTabTextToolTip", "Switch to the skeletal meshes tab."))
		[
			SNew(SSplitter)
			.Orientation(Orient_Horizontal)
			+ SSplitter::Slot()
			.Value(0.4f)
			[
				SNew(SBox)
				[
					SkeletalMeshTabListView.ToSharedRef()
				]
			]
			+ SSplitter::Slot()
			.Value(0.6f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SkeletalMeshTabListView->CreateOverrideOptionComboBox().ToSharedRef()
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.Text(LOCTEXT("FbxOptionWindow_SM_Select_asset_using", "Select Asset Using"))
						.OnClicked(SkeletalMeshTabListView.Get(), &SFbxSceneSkeletalMeshListView::OnSelectAssetUsing)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.Text(LOCTEXT("FbxOptionWindow_SM_CreateOverride", "Create Override"))
						.ToolTipText(LOCTEXT("FbxOptionWindow_SM_CreateOverrideTooltip", "Create Override to specify custom import options for some static meshes.\nTo assign options use context menu on static meshes."))
						.OnClicked(SkeletalMeshTabListView.Get(), &SFbxSceneSkeletalMeshListView::OnCreateOverrideOptions)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.Text(LOCTEXT("FbxOptionWindow_SM_Delete", "Delete"))
						.IsEnabled(SkeletalMeshTabListView.Get(), &SFbxSceneSkeletalMeshListView::CanDeleteOverride)
						.OnClicked(SkeletalMeshTabListView.Get(), &SFbxSceneSkeletalMeshListView::OnDeleteOverride)
					]
				]
				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					SAssignNew(InspectorBox, SBox)
				]
			]
		];
	
	//Prevent user to close the tab
	DockTab->SetCanCloseTab(SDockTab::FCanCloseTab::CreateRaw(this, &SFbxSceneOptionWindow::CanCloseTab));

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	SkeletalMeshTabDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	InspectorBox->SetContent(SkeletalMeshTabDetailsView->AsShared());
	SkeletalMeshTabDetailsView->SetObject(SceneImportOptionsSkeletalMeshDisplay);
	SkeletalMeshTabDetailsView->OnFinishedChangingProperties().AddSP(SkeletalMeshTabListView.Get(), &SFbxSceneSkeletalMeshListView::OnFinishedChangingProperties);
	return DockTab;
}

FText SFbxSceneOptionWindow::GetMaterialBasePath() const
{
	return FText::FromString(MaterialBasePath);
}

bool IsMaterialBasePathValid(FString MaterialBasePath)
{
	bool bInvalidMaterialPath = (MaterialBasePath.Contains(TEXT("//"), ESearchCase::CaseSensitive)) ||
								(MaterialBasePath.Len() < 2) ||
								!MaterialBasePath.StartsWith(TEXT("/")) ||
								!MaterialBasePath.EndsWith(TEXT("/")
								);
	return !bInvalidMaterialPath;
}

void SFbxSceneOptionWindow::OnMaterialBasePathCommited(const FText& InText, ETextCommit::Type InCommitType)
{
	MaterialBasePath = InText.ToString();

	bool bMaterialPathValid = IsMaterialBasePathValid(MaterialBasePath);
	//Commit only if all the rules are respected
	if (MaterialBasePath.Len() == 0)
	{
		GlobalImportSettings->MaterialBasePath = NAME_None;
		MaterialsTabListView->UpdateMaterialBasePath();
	}
	else if (bMaterialPathValid)
	{
		GlobalImportSettings->MaterialBasePath = FName(*MaterialBasePath);
		MaterialsTabListView->UpdateMaterialBasePath();
	}
}

FReply SFbxSceneOptionWindow::OnMaterialBasePathBrowse()
{
	TSharedRef<SDlgPickPath> PickContentPathDlg =
		SNew(SDlgPickPath)
		.Title(LOCTEXT("FbxChooseImportOverrideMaterialPath", "Choose Location path for importing all materials"));

	if (PickContentPathDlg->ShowModal() == EAppReturnType::Cancel)
	{
		return FReply::Handled();
	}
	MaterialBasePath = PickContentPathDlg->GetPath().ToString();
	
	if (MaterialBasePath.IsEmpty())
	{
		return FReply::Handled();
	}
	//Make sure it start and end with a slash
	if (!MaterialBasePath.EndsWith(TEXT("/")))
	{
		MaterialBasePath += TEXT("/");
	}
	if (!MaterialBasePath.StartsWith(TEXT("/")))
	{
		MaterialBasePath.InsertAt(0, TEXT("/"));
	}
	GlobalImportSettings->MaterialBasePath = FName(*MaterialBasePath);
	MaterialsTabListView->UpdateMaterialBasePath();

	return FReply::Handled();
}

FSlateColor SFbxSceneOptionWindow::GetMaterialBasePathTextColor() const
{
	if (MaterialBasePath.IsEmpty() || IsMaterialBasePathValid(MaterialBasePath))
	{
		return FSlateColor::UseForeground();
	}
	else
	{
		return FLinearColor(0.75f, 0.75f, 0.0f, 1.0f);
	}
}

TSharedRef<SDockTab> SFbxSceneOptionWindow::SpawnMaterialTab(const FSpawnTabArgs& Args)
{
	//Create the static mesh listview
	MaterialsTabListView = SNew(SFbxSceneMaterialsListView)
		.SceneInfo(SceneInfo)
		.SceneInfoOriginal(SceneInfoOriginal)
		.GlobalImportSettings(GlobalImportSettings)
		.TexturesArray(&TexturesArray)
		.FullPath(FullPath)
		.IsReimport(SceneInfoOriginal.IsValid())
		.CreateContentFolderHierarchy(SceneImportOptionsDisplay->bCreateContentFolderHierarchy != 0);

	TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		.Label(LOCTEXT("WidgetFbxSceneMaterialsTab", "Materials"))
		.ToolTipText(LOCTEXT("WidgetFbxSceneMaterialsTabTextToolTip", "Switch to the materials tab."))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Left)
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("FbxOptionWindow_Scene_Material_Prefix", "Material override base path: "))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(5.0f, 3.0f, 6.0f, 3.0f)
				.VAlign(VAlign_Center)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					[
						SNew(SEditableText)
						.SelectAllTextWhenFocused(true)
						.Text(this, &SFbxSceneOptionWindow::GetMaterialBasePath)
						.ToolTipText(LOCTEXT("FbxOptionWindow_Scene_MaterialBasePath_tooltip", "The override path must start and end by '/' use this to import all material to a different base path(i.e. /Game/Materials/)"))
						.OnTextCommitted(this, &SFbxSceneOptionWindow::OnMaterialBasePathCommited)
						.OnTextChanged(this, &SFbxSceneOptionWindow::OnMaterialBasePathCommited, ETextCommit::Default)
						.ColorAndOpacity(this, &SFbxSceneOptionWindow::GetMaterialBasePathTextColor)
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.Text(LOCTEXT("FbxOptionWindow_Scene_Material_Browse", "Browse..."))
					.ToolTipText(LOCTEXT("FbxOptionWindow_Scene_MaterialBasePath_Browse_tooltip", "Select a path where to save all materials"))
					.OnClicked(this, &SFbxSceneOptionWindow::OnMaterialBasePathBrowse)
				]
			]
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.FillHeight(1.0f)
					[
						MaterialsTabListView.ToSharedRef()
						/*SNew(SSplitter)
						.Orientation(Orient_Vertical)
						+ SSplitter::Slot()
						.Value(0.4f)
						[
							SNew(SBox)
							[
								MaterialsTabListView.ToSharedRef()
							]
						]
						+ SSplitter::Slot()
						.Value(0.6f)
						[
							SNew(SVerticalBox)
							+ SVerticalBox::Slot()
							.FillHeight(1.0f)
							[
								SNew(STextBlock)
								.Text(LOCTEXT("FbxOptionWindow_Scene_Texture_Slot", "Put the texture array here"))
							]
						]*/
					]
				]
			]
		];
	
	//Prevent user to close the tab
	DockTab->SetCanCloseTab(SDockTab::FCanCloseTab::CreateRaw(this, &SFbxSceneOptionWindow::CanCloseTab));

	return DockTab;
}

TSharedRef<SDockTab> SFbxSceneOptionWindow::SpawnSceneReimportTab(const FSpawnTabArgs& Args)
{
	//Create the treeview
	SceneReimportTreeview = SNew(SFbxReimportSceneTreeView)
		.SceneInfo(SceneInfo)
		.SceneInfoOriginal(SceneInfoOriginal)
		.NodeStatusMap(NodeStatusMap);
	
	TSharedPtr<SBox> InspectorBox;
	
	TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		.Label(LOCTEXT("WidgetFbxSceneActorTab", "Scene"))
		.ToolTipText(LOCTEXT("WidgetFbxSceneActorTabTextToolTip", "Switch to the scene tab."))
		[
			SNew(SSplitter)
			.Orientation(Orient_Horizontal)
			+ SSplitter::Slot()
			.Value(0.4f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.HAlign(HAlign_Left)
				.AutoHeight()
				[
					SNew(SUniformGridPanel)
					.SlotPadding(2.0f)
					+ SUniformGridPanel::Slot(0, 0)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SCheckBox)
							.HAlign(HAlign_Center)
							.OnCheckStateChanged(SceneReimportTreeview.Get(), &SFbxReimportSceneTreeView::OnToggleSelectAll)
						]
						+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						.Padding(0.0f, 3.0f, 6.0f, 3.0f)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("FbxOptionWindow_Scene_All", "All"))
						]
					]
					+ SUniformGridPanel::Slot(1, 0)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.Text(LOCTEXT("FbxOptionWindow_Scene_ExpandAll", "Expand All"))
						.OnClicked(SceneReimportTreeview.Get(), &SFbxReimportSceneTreeView::OnExpandAll)
					]
					+ SUniformGridPanel::Slot(2, 0)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.Text(LOCTEXT("FbxOptionWindow_Scene_CollapseAll", "Collapse All"))
						.OnClicked(SceneReimportTreeview.Get(), &SFbxReimportSceneTreeView::OnCollapseAll)
					]
				]
				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					SNew(SBox)
					[
						SceneReimportTreeview.ToSharedRef()
					]
				]
			]
			+ SSplitter::Slot()
			.Value(0.6f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.HAlign(HAlign_Left)
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Top)
					.HAlign(HAlign_Left)
					[
						SNew(SCheckBox)
						.HAlign(HAlign_Center)
						.OnCheckStateChanged(this, &SFbxSceneOptionWindow::OnToggleReimportHierarchy)
						.IsChecked(this, &SFbxSceneOptionWindow::IsReimportHierarchyChecked)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0.0f, 3.0f, 6.0f, 3.0f)
					.VAlign(VAlign_Top)
					.HAlign(HAlign_Left)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("FbxOptionWindow_Scene_Reimport_ImportHierarchy", "Reimport Hierarchy"))
						.ToolTipText(LOCTEXT("FbxOptionWindow_Scene_Reimport_ImportHierarchy_Tooltip", "If Check and the original import was done in a blueprint, the blueprint hierarchy will be revisited to include the fbx changes"))
					]
				]
				+ SVerticalBox::Slot()
				.HAlign(HAlign_Left)
				.AutoHeight()
				[
					SAssignNew(InspectorBox, SBox)
					.WidthOverride(1920.0f)
				]
			]
		];

	//Prevent user to close the tab
	DockTab->SetCanCloseTab(SDockTab::FCanCloseTab::CreateRaw(this, &SFbxSceneOptionWindow::CanCloseTab));

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	SceneReimportTabDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	InspectorBox->SetContent(SceneReimportTabDetailsView->AsShared());
	SceneReimportTabDetailsView->SetObject(SceneImportOptionsDisplay);
	SceneReimportTabDetailsView->OnFinishedChangingProperties().AddSP(this, &SFbxSceneOptionWindow::OnFinishedChangingPropertiesSceneTabDetailView);

	return DockTab;
}

void SFbxSceneOptionWindow::OnToggleReimportHierarchy(ECheckBoxState CheckType)
{
	if (GlobalImportSettings != nullptr)
	{
		GlobalImportSettings->bImportScene = CheckType == ECheckBoxState::Checked;
	}
}

ECheckBoxState SFbxSceneOptionWindow::IsReimportHierarchyChecked() const
{
	return  GlobalImportSettings->bImportScene ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SFbxSceneOptionWindow::OnToggleBakePivotInVertex(ECheckBoxState CheckType)
{
	if (GlobalImportSettings != nullptr)
	{
		GlobalImportSettings->bBakePivotInVertex = CheckType == ECheckBoxState::Checked;
	}
}

ECheckBoxState SFbxSceneOptionWindow::IsBakePivotInVertexChecked() const
{
	return  GlobalImportSettings->bBakePivotInVertex ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

TSharedRef<SDockTab> SFbxSceneOptionWindow::SpawnStaticMeshReimportTab(const FSpawnTabArgs& Args)
{
	//Create the static mesh listview
	StaticMeshReimportListView = SNew(SFbxSceneStaticMeshReimportListView)
		.SceneInfo(SceneInfo)
		.SceneInfoOriginal(SceneInfoOriginal)
		.GlobalImportSettings(GlobalImportSettings)
		.OverrideNameOptions(&OverrideNameOptions)
		.OverrideNameOptionsMap(OverrideNameOptionsMap)
		.SceneImportOptionsStaticMeshDisplay(SceneImportOptionsStaticMeshDisplay)
		.MeshStatusMap(MeshStatusMap);

	TSharedPtr<SBox> InspectorBox;
	TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		.Label(LOCTEXT("WidgetFbxSceneReimportStaticMeshTab", "Static Meshes"))
		.ToolTipText(LOCTEXT("WidgetFbxSceneReimportStaticMeshTabTextToolTip", "Switch to the reimport static meshes tab."))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.HAlign(HAlign_Left)
					.AutoHeight()
					[
						SNew(SUniformGridPanel)
						.SlotPadding(2.0f)
						+ SUniformGridPanel::Slot(0, 0)
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Left)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("FbxOptionWindow_Scene_Filters_Label", "Filters:"))
						]
						+ SUniformGridPanel::Slot(1, 0)
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
							[
								SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(SCheckBox)
									.HAlign(HAlign_Center)
									.OnCheckStateChanged(StaticMeshReimportListView.Get(), &SFbxSceneStaticMeshReimportListView::OnToggleFilterAddContent)
									.IsChecked(StaticMeshReimportListView.Get(), &SFbxSceneStaticMeshReimportListView::IsFilterAddContentChecked)
								]
								+ SHorizontalBox::Slot()
								.FillWidth(1.0f)
								.Padding(0.0f, 3.0f, 6.0f, 3.0f)
								.VAlign(VAlign_Center)
								[
									SNew(STextBlock)
									.Text(LOCTEXT("FbxOptionWindow_Scene_Reimport_Filter_Add_Content", "Add"))
								]
							]
						]
						+ SUniformGridPanel::Slot(2, 0)
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
							[
								SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(SCheckBox)
									.HAlign(HAlign_Center)
									.OnCheckStateChanged(StaticMeshReimportListView.Get(), &SFbxSceneStaticMeshReimportListView::OnToggleFilterDeleteContent)
									.IsChecked(StaticMeshReimportListView.Get(), &SFbxSceneStaticMeshReimportListView::IsFilterDeleteContentChecked)
								]
								+ SHorizontalBox::Slot()
								.FillWidth(1.0f)
								.Padding(0.0f, 3.0f, 6.0f, 3.0f)
								.VAlign(VAlign_Center)
								[
									SNew(STextBlock)
									.Text(LOCTEXT("FbxOptionWindow_Scene_Reimport_Filter_Delete_Content", "Delete"))
								]
							]
						]
						+ SUniformGridPanel::Slot(3, 0)
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
							[
								SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(SCheckBox)
									.HAlign(HAlign_Center)
									.OnCheckStateChanged(StaticMeshReimportListView.Get(), &SFbxSceneStaticMeshReimportListView::OnToggleFilterOverwriteContent)
									.IsChecked(StaticMeshReimportListView.Get(), &SFbxSceneStaticMeshReimportListView::IsFilterOverwriteContentChecked)
								]
								+ SHorizontalBox::Slot()
								.FillWidth(1.0f)
								.Padding(0.0f, 3.0f, 6.0f, 3.0f)
								.VAlign(VAlign_Center)
								[
									SNew(STextBlock)
									.Text(LOCTEXT("FbxOptionWindow_Scene_Reimport_Filter_Overwrite_Content", "Overwrite"))
								]
							]
						]
						+ SUniformGridPanel::Slot(4, 0)
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
							[
								SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(SCheckBox)
									.HAlign(HAlign_Center)
									.OnCheckStateChanged(StaticMeshReimportListView.Get(), &SFbxSceneStaticMeshReimportListView::OnToggleFilterDiff)
									.IsChecked(StaticMeshReimportListView.Get(), &SFbxSceneStaticMeshReimportListView::IsFilterDiffChecked)
								]
								+ SHorizontalBox::Slot()
								.FillWidth(1.0f)
								.Padding(0.0f, 3.0f, 6.0f, 3.0f)
								.VAlign(VAlign_Center)
								[
									SNew(STextBlock)
									.Text(LOCTEXT("FbxOptionWindow_Scene_Reimport_Filter_Diff", "Diff"))
									.ToolTipText(LOCTEXT("FbxOptionWindow_Scene_Reimport_Filter_Diff_Tooltip", "Show every reimport item that dont match between the original fbx and the new one."))
								]
							]
						]
					]
					+ SVerticalBox::Slot()
					.FillHeight(1.0f)
					[
						SNew(SSplitter)
						.Orientation(Orient_Vertical)
						+ SSplitter::Slot()
						.Value(0.4f)
						[
							SNew(SBox)
							[
								StaticMeshReimportListView.ToSharedRef()
							]
						]
						+ SSplitter::Slot()
						.Value(0.6f)
						[
							SNew(SVerticalBox)
							+ SVerticalBox::Slot()
							.AutoHeight()
							[
								SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									StaticMeshReimportListView->CreateOverrideOptionComboBox().ToSharedRef()
								]
								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(SButton)
									.HAlign(HAlign_Center)
									.Text(LOCTEXT("FbxOptionWindow_SM_Select_asset_using", "Select Asset Using"))
									.OnClicked(StaticMeshReimportListView.Get(), &SFbxSceneStaticMeshReimportListView::OnSelectAssetUsing)
								]
								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(SButton)
									.Text(LOCTEXT("FbxOptionWindow_SM_CreateOverride", "Create Override"))
									.ToolTipText(LOCTEXT("FbxOptionWindow_SM_CreateOverrideTooltip", "Create Override to specify custom import options for some static meshes.\nTo assign options use context menu on static meshes."))
									.OnClicked(StaticMeshReimportListView.Get(), &SFbxSceneStaticMeshReimportListView::OnCreateOverrideOptions)
								]
								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(SButton)
									.HAlign(HAlign_Center)
									.Text(LOCTEXT("FbxOptionWindow_SM_Delete", "Delete"))
									.IsEnabled(StaticMeshReimportListView.Get(), &SFbxSceneStaticMeshReimportListView::CanDeleteOverride)
									.OnClicked(StaticMeshReimportListView.Get(), &SFbxSceneStaticMeshReimportListView::OnDeleteOverride)
								]
							]
							+ SVerticalBox::Slot()
							.FillHeight(1.0f)
							[
								SAssignNew(InspectorBox, SBox)
							]
						]
					]
				]
			]
		];
	
	//Prevent user to close the tab
	DockTab->SetCanCloseTab(SDockTab::FCanCloseTab::CreateRaw(this, &SFbxSceneOptionWindow::CanCloseTab));

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	StaticMeshReimportDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	InspectorBox->SetContent(StaticMeshReimportDetailsView->AsShared());
	StaticMeshReimportDetailsView->SetObject(SceneImportOptionsStaticMeshDisplay);
	StaticMeshReimportDetailsView->OnFinishedChangingProperties().AddSP(StaticMeshReimportListView.Get(), &SFbxSceneStaticMeshReimportListView::OnFinishedChangingProperties);
	return DockTab;
}


TSharedPtr<SWidget> SFbxSceneOptionWindow::SpawnDockTab()
{
	return FbxSceneImportTabManager->RestoreFrom(Layout.ToSharedRef(), OwnerWindow.Pin()).ToSharedRef();
}

void SFbxSceneOptionWindow::InitAllTabs()
{
	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabRole(ETabRole::MajorTab);

	FbxSceneImportTabManager = FGlobalTabmanager::Get()->NewTabManager(DockTab);

	if (!SceneInfoOriginal.IsValid())
	{
		Layout = FTabManager::NewLayout("FbxSceneImportUI_Layout")
			->AddArea
			(
				FTabManager::NewPrimaryArea()
				->Split
				(
					FTabManager::NewStack()
					->AddTab("Scene", ETabState::OpenedTab)
					->AddTab("StaticMeshes", ETabState::OpenedTab)
					->AddTab("SkeletalMeshes", ETabState::OpenedTab)
					->AddTab("Materials", ETabState::OpenedTab)
				)
			);

		FbxSceneImportTabManager->RegisterTabSpawner("Scene", FOnSpawnTab::CreateSP(this, &SFbxSceneOptionWindow::SpawnSceneTab));
		FbxSceneImportTabManager->RegisterTabSpawner("StaticMeshes", FOnSpawnTab::CreateSP(this, &SFbxSceneOptionWindow::SpawnStaticMeshTab));
		FbxSceneImportTabManager->RegisterTabSpawner("SkeletalMeshes", FOnSpawnTab::CreateSP(this, &SFbxSceneOptionWindow::SpawnSkeletalMeshTab));
		FbxSceneImportTabManager->RegisterTabSpawner("Materials", FOnSpawnTab::CreateSP(this, &SFbxSceneOptionWindow::SpawnMaterialTab));
	}
	else
	{
		if (bCanReimportHierarchy)
		{
			Layout = FTabManager::NewLayout("FbxSceneImportUI_Layout")
				->AddArea
				(
					FTabManager::NewPrimaryArea()
					->Split
					(
						FTabManager::NewStack()
						->AddTab("SceneReImport", ETabState::OpenedTab)
						->AddTab("StaticMeshesReImport", ETabState::OpenedTab)
						->AddTab("SkeletalMeshesReImport", ETabState::OpenedTab)
						->AddTab("Materials", ETabState::OpenedTab)
					)
				);
			FbxSceneImportTabManager->RegisterTabSpawner("SceneReImport", FOnSpawnTab::CreateSP(this, &SFbxSceneOptionWindow::SpawnSceneReimportTab));
		}
		else
		{
			//Re import only the assets, the hierarchy cannot be re import.
			Layout = FTabManager::NewLayout("FbxSceneImportUI_Layout")
				->AddArea
				(
					FTabManager::NewPrimaryArea()
					->Split
					(
						FTabManager::NewStack()
						->AddTab("StaticMeshesReImport", ETabState::OpenedTab)
						->AddTab("SkeletalMeshesReImport", ETabState::OpenedTab)
						->AddTab("Materials", ETabState::OpenedTab)
					)
				);
		}
		FbxSceneImportTabManager->RegisterTabSpawner("StaticMeshesReimport", FOnSpawnTab::CreateSP(this, &SFbxSceneOptionWindow::SpawnStaticMeshReimportTab));
		FbxSceneImportTabManager->RegisterTabSpawner("SkeletalMeshesReimport", FOnSpawnTab::CreateSP(this, &SFbxSceneOptionWindow::SpawnSkeletalMeshReimportTab));
		FbxSceneImportTabManager->RegisterTabSpawner("Materials", FOnSpawnTab::CreateSP(this, &SFbxSceneOptionWindow::SpawnMaterialTab));
	}

	//Prevent Docking the tab outside of the dialog well
	FbxSceneImportTabManager->SetCanDoDragOperation(false);
}

void SFbxSceneOptionWindow::Construct(const FArguments& InArgs)
{
	SceneInfo = InArgs._SceneInfo;
	SceneInfoOriginal = InArgs._SceneInfoOriginal;
	MeshStatusMap = InArgs._MeshStatusMap;
	bCanReimportHierarchy = InArgs._CanReimportHierarchy;
	NodeStatusMap = InArgs._NodeStatusMap;
	GlobalImportSettings = InArgs._GlobalImportSettings;
	SceneImportOptionsDisplay = InArgs._SceneImportOptionsDisplay;
	SceneImportOptionsStaticMeshDisplay = InArgs._SceneImportOptionsStaticMeshDisplay;
	OverrideNameOptionsMap = InArgs._OverrideNameOptionsMap;
	SceneImportOptionsSkeletalMeshDisplay = InArgs._SceneImportOptionsSkeletalMeshDisplay;
	OwnerWindow = InArgs._OwnerWindow;
	FullPath = InArgs._FullPath;

	check(SceneInfo.IsValid());
	check(GlobalImportSettings != nullptr);
	check(SceneImportOptionsDisplay != nullptr);
	check(SceneImportOptionsStaticMeshDisplay != nullptr);
	check(SceneImportOptionsSkeletalMeshDisplay != nullptr);
	check(OverrideNameOptionsMap != nullptr);

	if (SceneInfoOriginal.IsValid())
	{
		check(MeshStatusMap != nullptr);
		check(NodeStatusMap);
	}

	check(OwnerWindow.IsValid());

	MaterialBasePath = (GlobalImportSettings->MaterialBasePath == NAME_None) ? TEXT("") : GlobalImportSettings->MaterialBasePath.ToString();

	InitAllTabs();

	FText SubmitText = SceneInfoOriginal.IsValid() ? LOCTEXT("FbxSceneOptionWindow_ReImport", "Reimport") : LOCTEXT("FbxSceneOptionWindow_Import", "Import");

	this->ChildSlot
	[
		SNew(SBorder)
		.Padding(FMargin(10.0f, 3.0f))
		.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SNew(SBorder)
				.Padding(FMargin(3))
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
						.Font(FAppStyle::GetFontStyle("CurveEd.LabelFont"))
						.Text(LOCTEXT("FbxSceneImport_CurrentPath", "Import Asset Path: "))
					]
					+SHorizontalBox::Slot()
					.Padding(5, 0, 0, 0)
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Font(FAppStyle::GetFontStyle("CurveEd.InfoFont"))
						.Text(FText::FromString(FullPath))
					]
				]
			]
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(2.0f)
			[
				SpawnDockTab().ToSharedRef()
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(2.0f)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(2.0f)
				+ SUniformGridPanel::Slot(0, 0)
				[
					IDocumentation::Get()->CreateAnchor(FString("fbx-import-options-reference-in-unreal-engine"))
				]
				+ SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(SubmitText)
					.IsEnabled(this, &SFbxSceneOptionWindow::CanImport)
					.OnClicked(this, &SFbxSceneOptionWindow::OnImport)
				]
				+ SUniformGridPanel::Slot(2, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("FbxOptionWindow_Cancel", "Cancel"))
					.ToolTipText(LOCTEXT("FbxOptionWindow_Cancel_ToolTip", "Cancels importing this FBX file"))
					.OnClicked(this, &SFbxSceneOptionWindow::OnCancel)
				]
			]
		]
	];

	//By default we want to see the Scene tab
	if (!SceneInfoOriginal.IsValid())
	{
		FbxSceneImportTabManager->TryInvokeTab(FTabId("Scene"));
	}
	else
	{
		if (bCanReimportHierarchy)
		{
			FbxSceneImportTabManager->TryInvokeTab(FTabId("SceneReImport"));
		}
		else
		{
			FbxSceneImportTabManager->TryInvokeTab(FTabId("StaticMeshesReimport"));
		}
	}
}

void SFbxSceneOptionWindow::CloseFbxSceneOption()
{
	//Free all resource before closing the window
	//Unregister spawn tab
	if (FbxSceneImportTabManager.IsValid())
	{
		FbxSceneImportTabManager->UnregisterAllTabSpawners();
		FbxSceneImportTabManager->CloseAllAreas();
	}

	//Clear scene tab resource
	SceneTabTreeview = nullptr;
	SceneTabDetailsView = nullptr;

	StaticMeshTabListView = nullptr;
	StaticMeshTabDetailsView = nullptr;

	SkeletalMeshTabListView = nullptr;

	SceneReimportTreeview = nullptr;
	SceneReimportTabDetailsView = nullptr;

	StaticMeshReimportListView = nullptr;
	StaticMeshReimportDetailsView = nullptr;

	MaterialsTabListView = nullptr;
	TexturesArray.Reset();
	MaterialBasePath.Empty();

	SceneInfo = nullptr;
	SceneInfoOriginal = nullptr;
	SceneImportOptionsDisplay = nullptr;
	GlobalImportSettings = nullptr;
	SceneImportOptionsDisplay = nullptr;
	SceneImportOptionsStaticMeshDisplay = nullptr;
	OverrideNameOptionsMap = nullptr;
	SceneImportOptionsSkeletalMeshDisplay = nullptr;

	MeshStatusMap = nullptr;
	NodeStatusMap = nullptr;

	if (OwnerWindow.IsValid())
	{
		//Close the window
		OwnerWindow.Pin()->RequestDestroyWindow();
	}
	OwnerWindow = nullptr;
	//The tab manager must be deleted only after we destroy the windows, because it can be access by a WeakPtr and it must be valid
	FbxSceneImportTabManager = nullptr;
	Layout = nullptr;
}

bool SFbxSceneOptionWindow::CanImport()  const
{
	return true;
}

void SFbxSceneOptionWindow::CopyFbxOptionsToFbxOptions(UnFbx::FBXImportOptions *SourceOptions, UnFbx::FBXImportOptions *DestinationOptions)
{
	if (ensure(SourceOptions && DestinationOptions))
	{
		*DestinationOptions = *SourceOptions;
	}
}

void SFbxSceneOptionWindow::CopyStaticMeshOptionsToFbxOptions(UnFbx::FBXImportOptions *ImportSettings, UFbxSceneImportOptionsStaticMesh* StaticMeshOptions)
{
	ImportSettings->bAutoGenerateCollision = StaticMeshOptions->bAutoGenerateCollision;
	ImportSettings->bBuildReversedIndexBuffer = StaticMeshOptions->bBuildReversedIndexBuffer;
	ImportSettings->bGenerateLightmapUVs = StaticMeshOptions->bGenerateLightmapUVs;
	ImportSettings->bOneConvexHullPerUCX = StaticMeshOptions->bOneConvexHullPerUCX;
	ImportSettings->bRemoveDegenerates = StaticMeshOptions->bRemoveDegenerates;
	ImportSettings->StaticMeshLODGroup = StaticMeshOptions->StaticMeshLODGroup;
	switch (StaticMeshOptions->VertexColorImportOption)
	{
	case EFbxSceneVertexColorImportOption::Replace:
		ImportSettings->VertexColorImportOption = EVertexColorImportOption::Type::Replace;
		break;
	case EFbxSceneVertexColorImportOption::Override:
		ImportSettings->VertexColorImportOption = EVertexColorImportOption::Type::Override;
		break;
	case EFbxSceneVertexColorImportOption::Ignore:
		ImportSettings->VertexColorImportOption = EVertexColorImportOption::Type::Ignore;
		break;
	default:
		ImportSettings->VertexColorImportOption = EVertexColorImportOption::Type::Replace;
	}
	ImportSettings->VertexOverrideColor = StaticMeshOptions->VertexOverrideColor;
	switch (StaticMeshOptions->NormalImportMethod)
	{
		case EFBXSceneNormalImportMethod::FBXSceneNIM_ComputeNormals:
			ImportSettings->NormalImportMethod = FBXNIM_ComputeNormals;
			break;
		case EFBXSceneNormalImportMethod::FBXSceneNIM_ImportNormals:
			ImportSettings->NormalImportMethod = FBXNIM_ImportNormals;
			break;
		case EFBXSceneNormalImportMethod::FBXSceneNIM_ImportNormalsAndTangents:
			ImportSettings->NormalImportMethod = FBXNIM_ImportNormalsAndTangents;
			break;
	}
	switch (StaticMeshOptions->NormalGenerationMethod)
	{
	case EFBXSceneNormalGenerationMethod::BuiltIn:
		ImportSettings->NormalGenerationMethod = EFBXNormalGenerationMethod::BuiltIn;
		break;
	case EFBXSceneNormalGenerationMethod::MikkTSpace:
		ImportSettings->NormalGenerationMethod = EFBXNormalGenerationMethod::MikkTSpace;
		break;
	}
}

void SFbxSceneOptionWindow::CopyFbxOptionsToStaticMeshOptions(UnFbx::FBXImportOptions *ImportSettings, UFbxSceneImportOptionsStaticMesh* StaticMeshOptions)
{
	StaticMeshOptions->bAutoGenerateCollision = ImportSettings->bAutoGenerateCollision;
	StaticMeshOptions->bBuildReversedIndexBuffer = ImportSettings->bBuildReversedIndexBuffer;
	StaticMeshOptions->bGenerateLightmapUVs = ImportSettings->bGenerateLightmapUVs;
	StaticMeshOptions->bOneConvexHullPerUCX = ImportSettings->bOneConvexHullPerUCX;
	StaticMeshOptions->bRemoveDegenerates = ImportSettings->bRemoveDegenerates;
	StaticMeshOptions->StaticMeshLODGroup = ImportSettings->StaticMeshLODGroup;
	switch (ImportSettings->VertexColorImportOption)
	{
	case EVertexColorImportOption::Type::Replace:
		StaticMeshOptions->VertexColorImportOption = EFbxSceneVertexColorImportOption::Replace;
		break;
	case EVertexColorImportOption::Type::Override:
		StaticMeshOptions->VertexColorImportOption = EFbxSceneVertexColorImportOption::Override;
		break;
	case EVertexColorImportOption::Type::Ignore:
		StaticMeshOptions->VertexColorImportOption = EFbxSceneVertexColorImportOption::Ignore;
		break;
	default:
		StaticMeshOptions->VertexColorImportOption = EFbxSceneVertexColorImportOption::Replace;
	}
	StaticMeshOptions->VertexOverrideColor = ImportSettings->VertexOverrideColor;
	switch (ImportSettings->NormalImportMethod)
	{
	case FBXNIM_ComputeNormals:
		StaticMeshOptions->NormalImportMethod = EFBXSceneNormalImportMethod::FBXSceneNIM_ComputeNormals;
		break;
	case FBXNIM_ImportNormals:
		StaticMeshOptions->NormalImportMethod = EFBXSceneNormalImportMethod::FBXSceneNIM_ImportNormals;
		break;
	case FBXNIM_ImportNormalsAndTangents:
		StaticMeshOptions->NormalImportMethod = EFBXSceneNormalImportMethod::FBXSceneNIM_ImportNormalsAndTangents;
		break;
	}
	switch (ImportSettings->NormalGenerationMethod)
	{
	case EFBXNormalGenerationMethod::BuiltIn:
		StaticMeshOptions->NormalGenerationMethod = EFBXSceneNormalGenerationMethod::BuiltIn;
		break;
	case EFBXNormalGenerationMethod::MikkTSpace:
		StaticMeshOptions->NormalGenerationMethod = EFBXSceneNormalGenerationMethod::MikkTSpace;
		break;
	}
}

void SFbxSceneOptionWindow::CopySkeletalMeshOptionsToFbxOptions(UnFbx::FBXImportOptions *ImportSettings, UFbxSceneImportOptionsSkeletalMesh* SkeletalMeshOptions)
{
	ImportSettings->bCreatePhysicsAsset = SkeletalMeshOptions->bCreatePhysicsAsset;
	ImportSettings->bImportMeshesInBoneHierarchy = SkeletalMeshOptions->bImportMeshesInBoneHierarchy;
	ImportSettings->bImportMorph = SkeletalMeshOptions->bImportMorphTargets;
	ImportSettings->bImportVertexAttributes = SkeletalMeshOptions->bImportVertexAttributes;
	ImportSettings->OverlappingThresholds.ThresholdPosition = SkeletalMeshOptions->ThresholdPosition;
	ImportSettings->OverlappingThresholds.ThresholdTangentNormal = SkeletalMeshOptions->ThresholdTangentNormal;
	ImportSettings->OverlappingThresholds.ThresholdUV = SkeletalMeshOptions->ThresholdUV;
	ImportSettings->OverlappingThresholds.MorphThresholdPosition = SkeletalMeshOptions->MorphThresholdPosition;
	ImportSettings->bPreserveSmoothingGroups = SkeletalMeshOptions->bPreserveSmoothingGroups;
	ImportSettings->bKeepSectionsSeparate = SkeletalMeshOptions->bKeepSectionsSeparate;
	ImportSettings->bUpdateSkeletonReferencePose = SkeletalMeshOptions->bUpdateSkeletonReferencePose;
	ImportSettings->bUseT0AsRefPose = SkeletalMeshOptions->bUseT0AsRefPose;

	ImportSettings->bImportAnimations = SkeletalMeshOptions->bImportAnimations;
	ImportSettings->AnimationLengthImportType = SkeletalMeshOptions->AnimationLength;
	ImportSettings->bDeleteExistingMorphTargetCurves = SkeletalMeshOptions->bDeleteExistingMorphTargetCurves;
	ImportSettings->bImportCustomAttribute = SkeletalMeshOptions->bImportCustomAttribute;
	ImportSettings->bDeleteExistingCustomAttributeCurves = SkeletalMeshOptions->bDeleteExistingCustomAttributeCurves;
	ImportSettings->bDeleteExistingNonCurveCustomAttributes = SkeletalMeshOptions->bDeleteExistingNonCurveCustomAttributes;	
	ImportSettings->bPreserveLocalTransform = SkeletalMeshOptions->bPreserveLocalTransform;
	ImportSettings->bResample = !SkeletalMeshOptions->bUseDefaultSampleRate;
	ImportSettings->ResampleRate = SkeletalMeshOptions->CustomSampleRate;
	ImportSettings->bSnapToClosestFrameBoundary = SkeletalMeshOptions->bSnapToClosestFrameBoundary;	
	ImportSettings->AnimationRange.X = SkeletalMeshOptions->FrameImportRange.Min;
	ImportSettings->AnimationRange.Y = SkeletalMeshOptions->FrameImportRange.Max;
}

void SFbxSceneOptionWindow::CopyFbxOptionsToSkeletalMeshOptions(UnFbx::FBXImportOptions *ImportSettings, class UFbxSceneImportOptionsSkeletalMesh* SkeletalMeshOptions)
{
	SkeletalMeshOptions->bCreatePhysicsAsset = ImportSettings->bCreatePhysicsAsset;
	SkeletalMeshOptions->bImportMeshesInBoneHierarchy = ImportSettings->bImportMeshesInBoneHierarchy;
	SkeletalMeshOptions->bImportMorphTargets = ImportSettings->bImportMorph;
	SkeletalMeshOptions->bImportVertexAttributes = ImportSettings->bImportVertexAttributes;
	SkeletalMeshOptions->ThresholdPosition = ImportSettings->OverlappingThresholds.ThresholdPosition;
	SkeletalMeshOptions->ThresholdTangentNormal = ImportSettings->OverlappingThresholds.ThresholdTangentNormal;
	SkeletalMeshOptions->ThresholdUV = ImportSettings->OverlappingThresholds.ThresholdUV;
	SkeletalMeshOptions->MorphThresholdPosition = ImportSettings->OverlappingThresholds.MorphThresholdPosition;
	SkeletalMeshOptions->bPreserveSmoothingGroups = ImportSettings->bPreserveSmoothingGroups;
	SkeletalMeshOptions->bKeepSectionsSeparate = ImportSettings->bKeepSectionsSeparate;
	SkeletalMeshOptions->bUpdateSkeletonReferencePose = ImportSettings->bUpdateSkeletonReferencePose;
	SkeletalMeshOptions->bUseT0AsRefPose = ImportSettings->bUseT0AsRefPose;

	SkeletalMeshOptions->bImportAnimations = ImportSettings->bImportAnimations;
	SkeletalMeshOptions->AnimationLength = ImportSettings->AnimationLengthImportType;
	SkeletalMeshOptions->bDeleteExistingMorphTargetCurves = ImportSettings->bDeleteExistingMorphTargetCurves;
	SkeletalMeshOptions->bImportCustomAttribute = ImportSettings->bImportCustomAttribute;
	SkeletalMeshOptions->bDeleteExistingCustomAttributeCurves = ImportSettings->bDeleteExistingCustomAttributeCurves;
	SkeletalMeshOptions->bDeleteExistingNonCurveCustomAttributes = ImportSettings->bDeleteExistingNonCurveCustomAttributes;	
	SkeletalMeshOptions->bPreserveLocalTransform = ImportSettings->bPreserveLocalTransform;
	SkeletalMeshOptions->bUseDefaultSampleRate = !ImportSettings->bResample;
	SkeletalMeshOptions->CustomSampleRate = ImportSettings->ResampleRate;
	SkeletalMeshOptions->FrameImportRange.Min = ImportSettings->AnimationRange.X;
	SkeletalMeshOptions->FrameImportRange.Max = ImportSettings->AnimationRange.Y;
}

#undef LOCTEXT_NAMESPACE
