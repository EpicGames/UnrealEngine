// Copyright Epic Games, Inc. All Rights Reserved.

#include "Fbx/SSceneBaseMeshListView.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Fbx/SSceneImportNodeTreeView.h"
#include "Fbx/SSceneImportStaticMeshListView.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SFbxSceneOptionWindow.h"
#include "FbxImporter.h"
#include "UObject/Package.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Input/STextEntryPopup.h"
#include "Factories/FbxSceneImportData.h"

#define LOCTEXT_NAMESPACE "SFbxSSceneBaseMeshListView"

SFbxSSceneBaseMeshListView::~SFbxSSceneBaseMeshListView()
{
	OverrideNameOptions = nullptr;
	OverrideNameOptionsMap = nullptr;
	OptionComboBox = nullptr;
	DefaultOptionNamePtr = nullptr;
}

void SFbxSSceneBaseMeshListView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	bool SelectDefaultOptions = false;
	//Check if the array of Name have change
	TSharedPtr<FString> CurrentSelectedOptionName = OptionComboBox->GetSelectedItem();
	if (!CurrentSelectedOptionName.IsValid())
	{
		SelectDefaultOptions = true;
	}
	else
	{
		SelectDefaultOptions = true;
		for (TSharedPtr<FString> OptionsName : (*OverrideNameOptions))
		{
			if (CurrentSelectedOptionName == OptionsName)
			{
				SelectDefaultOptions = false;
				break;
			}
		}
	}
	if (SelectDefaultOptions)
	{
		if (CurrentSelectedOptionName.IsValid())
		{
			//Reset all object using the current option to default option
			TArray<TSharedPtr<FFbxMeshInfo>> RemoveKeys;
			TArray<TSharedPtr<FFbxMeshInfo>> SceneMeshOverrideKeys;
			for (TSharedPtr<FFbxMeshInfo> MeshInfo : FbxMeshesArray)
			{
				if (CurrentSelectedOptionName.Get()->Compare(MeshInfo->OptionName) == 0)
				{
					MeshInfo->OptionName = UFbxSceneImportFactory::DefaultOptionName;
				}
			}
		}
		//We have to select the default the option was probably destroy by another tab
		OptionComboBox->SetSelectedItem(FindOptionNameFromName(UFbxSceneImportFactory::DefaultOptionName));
	}

	bool bFoundColumn = false;
	for (const SHeaderRow::FColumn &Column : HeaderRow->GetColumns())
	{
		if (Column.ColumnId == FbxSceneBaseListViewColumn::PivotColumnId)
		{
			bFoundColumn = true;
			break;
		}
	}
	if (GlobalImportSettings->bBakePivotInVertex && !bFoundColumn)
	{
		HeaderRow->AddColumn(SHeaderRow::Column(FbxSceneBaseListViewColumn::PivotColumnId)
		.FillWidth(150)
		.HAlignCell(EHorizontalAlignment::HAlign_Left)
		.DefaultLabel(LOCTEXT("PivotNameHeaderName", "Pivot Node")));
	}
	else if(!GlobalImportSettings->bBakePivotInVertex && bFoundColumn)
	{
		HeaderRow->RemoveColumn(FbxSceneBaseListViewColumn::PivotColumnId);
	}

	SListView::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

void SFbxSSceneBaseMeshListView::AddSelectionToImport()
{
	SetSelectionImportState(true);
}

void SFbxSSceneBaseMeshListView::RemoveSelectionFromImport()
{
	SetSelectionImportState(false);
}

void SFbxSSceneBaseMeshListView::SetSelectionImportState(bool MarkForImport)
{
	TArray<FbxMeshInfoPtr> SelectedFbxMeshes;
	GetSelectedItems(SelectedFbxMeshes);
	for (auto Item : SelectedFbxMeshes)
	{
		FbxMeshInfoPtr ItemPtr = Item;
		ItemPtr->bImportAttribute = MarkForImport;
	}
}

void SFbxSSceneBaseMeshListView::OnSelectionChanged(FbxMeshInfoPtr Item, ESelectInfo::Type SelectionType)
{
	//Change the option selection
	TArray<FbxMeshInfoPtr> SelectedFbxMeshes;
	GetSelectedItems(SelectedFbxMeshes);
	for (FbxMeshInfoPtr SelectItem : SelectedFbxMeshes)
	{
		for (auto kvp : *OverrideNameOptionsMap)
		{
			if (kvp.Key.Compare(SelectItem->OptionName) == 0)
			{
				OptionComboBox->SetSelectedItem(FindOptionNameFromName(kvp.Key));
				return;
			}
		}
	}
	//Select Default in case we don't have a valid OptionName
	OptionComboBox->SetSelectedItem(FindOptionNameFromName(UFbxSceneImportFactory::DefaultOptionName));
}

void SFbxSSceneBaseMeshListView::OnToggleSelectAll(ECheckBoxState CheckType)
{
	for (FbxMeshInfoPtr MeshInfo : FbxMeshesArray)
	{
		if (!MeshInfo->bOriginalTypeChanged)
		{
			MeshInfo->bImportAttribute = CheckType == ECheckBoxState::Checked;
		}
	}
}

void SFbxSSceneBaseMeshListView::AddBakePivotMenu(class FMenuBuilder& MenuBuilder)
{
	if (GlobalImportSettings->bBakePivotInVertex)
	{
		MenuBuilder.AddMenuSeparator();
		// Add a sub-menu for "Pivot"
		MenuBuilder.AddSubMenu(
			LOCTEXT("PivotBakeSubMenu", "Pivot Options"),
			LOCTEXT("PivotBakeSubMenu_ToolTip", "Choose which pivot to Bake from"),
			FNewMenuDelegate::CreateSP(this, &SFbxSceneStaticMeshListView::FillPivotContextMenu));
	}
}

void SFbxSSceneBaseMeshListView::FillPivotContextMenu(FMenuBuilder& MenuBuilder)
{
	TArray<FbxMeshInfoPtr> SelectedFbxMeshes;
	int32 SelectCount = GetSelectedItems(SelectedFbxMeshes);

	uint64 InvalidUid = INVALID_UNIQUE_ID;
	if (SelectedFbxMeshes.Num() == 1)
	{
		FbxMeshInfoPtr Item = SelectedFbxMeshes[0];
		if (Item->bOriginalTypeChanged)
			return;
		MenuBuilder.AddMenuEntry(Item->PivotNodeUid == INVALID_UNIQUE_ID ? LOCTEXT("ResetPivotBakeCurrent", "* No Pivot Bake") : LOCTEXT("ResetPivotBake", "No Pivot Bake"), FText(), FSlateIcon(), FUIAction(FExecuteAction::CreateSP(this, &SFbxSSceneBaseMeshListView::AssignToPivot, InvalidUid)));
		for (auto Kvp : Item->NodeReferencePivots)
		{
			//Create an entry for each pivot
			const FVector &PivotValue = Kvp.Key;
			const TArray<uint64> &NodeUids = Kvp.Value;
			bool IsCurrentPivotSelected = false;
			for (uint64 NodeUid : NodeUids)
			{
				if (Item->PivotNodeUid == NodeUid)
				{
					IsCurrentPivotSelected = true;
					break;
				}
			}
			FString MenuText = (IsCurrentPivotSelected ? TEXT("* Pivot: ") : TEXT("Pivot: ")) + PivotValue.ToCompactString();
			FString MenuTooltipText = IsCurrentPivotSelected ? LOCTEXT("PivotCurrentMenuItemTooltip", "This is the pivot that will be use to import this mesh. Node Number using this pivot: ").ToString() : LOCTEXT("PivotMenuItemTooltip", "Node Number using this pivot: ").ToString();
			MenuTooltipText.AppendInt(NodeUids.Num());
			MenuBuilder.AddMenuEntry(FText::FromString(*MenuText), FText::FromString(*MenuTooltipText), FSlateIcon(), FUIAction(FExecuteAction::CreateSP(this, &SFbxSSceneBaseMeshListView::AssignToPivot, NodeUids[0])));
		}
	}
	else
	{
		MenuBuilder.AddMenuEntry(LOCTEXT("ResetPivotBakeAll", "All No Pivot Bake"), FText(), FSlateIcon(), FUIAction(FExecuteAction::CreateSP(this, &SFbxSSceneBaseMeshListView::AssignToPivot, InvalidUid)));
	}
}

void SFbxSSceneBaseMeshListView::FillMeshStatusMap(FbxSceneReimportStatusMapPtr MeshStatusMap
	, TSharedPtr<FFbxSceneInfo> SceneInfo
	, TSharedPtr<FFbxSceneInfo> SceneInfoOriginal
	, bool bFillSkeletalMeshStatusMap
	, TArray<FbxMeshInfoPtr>* FilterFbxMeshesArrayPtr
	, TArray<FbxMeshInfoPtr>* FbxMeshesArrayPtr)
{
	//If FbxMeshesArrayPtr is null use the InternalFbxMeshesArray, we need it to found match between SceneInfo and SceneInfoOriginal
	TArray<FbxMeshInfoPtr> InternalFbxMeshesArray;
	TArray<FbxMeshInfoPtr>* ValidFbxMeshesArrayPtr = FbxMeshesArrayPtr ? FbxMeshesArrayPtr : &InternalFbxMeshesArray;

	auto AddToFilterFbxMeshesArrayPtr = [&FilterFbxMeshesArrayPtr](FbxMeshInfoPtr& MeshInfo)
	{
		if (FilterFbxMeshesArrayPtr)
		{
			FilterFbxMeshesArrayPtr->Add(MeshInfo);
		}
	};

	for (FbxMeshInfoPtr MeshInfo : SceneInfo->MeshInfo)
	{
		if (MeshInfo->bIsSkelMesh != bFillSkeletalMeshStatusMap || MeshInfo->IsLod || MeshInfo->IsCollision)
		{
			continue;
		}
		ValidFbxMeshesArrayPtr->Add(MeshInfo);
		AddToFilterFbxMeshesArrayPtr(MeshInfo);
		FbxMeshInfoPtr FoundMeshInfo = nullptr;
		for (FbxMeshInfoPtr OriginalMeshInfo : SceneInfoOriginal->MeshInfo)
		{
			if (OriginalMeshInfo->OriginalImportPath.Compare(MeshInfo->OriginalImportPath) == 0)
			{
				FoundMeshInfo = MeshInfo;
				break;
			}
		}
		if (!FoundMeshInfo.IsValid())
		{
			//have an added asset
			EFbxSceneReimportStatusFlags StatusFlag = EFbxSceneReimportStatusFlags::Added | EFbxSceneReimportStatusFlags::ReimportAsset;
			if (MeshInfo->GetContentObject() != nullptr)
			{
				StatusFlag |= EFbxSceneReimportStatusFlags::FoundContentBrowserAsset;
			}
			MeshStatusMap->Add(MeshInfo->OriginalImportPath, StatusFlag);
		}
	}

	for (FbxMeshInfoPtr OriginalMeshInfo : SceneInfoOriginal->MeshInfo)
	{
		if (OriginalMeshInfo->bIsSkelMesh != bFillSkeletalMeshStatusMap || OriginalMeshInfo->IsLod || OriginalMeshInfo->IsCollision)
		{
			continue;
		}
		FbxMeshInfoPtr FoundMeshInfo = nullptr;
		
		{
			TArray<FbxMeshInfoPtr>& FbxMeshesArray = *ValidFbxMeshesArrayPtr;
			for (FbxMeshInfoPtr MeshInfo : FbxMeshesArray)
			{
				if (OriginalMeshInfo->OriginalImportPath.Compare(MeshInfo->OriginalImportPath) == 0)
				{
					FoundMeshInfo = MeshInfo;
					//Add the override info to the new fbx meshinfo
					FoundMeshInfo->SetOverridePath(OriginalMeshInfo->bOverridePath);
					FoundMeshInfo->OverrideImportPath = OriginalMeshInfo->OverrideImportPath;
					FoundMeshInfo->OverrideFullImportName = OriginalMeshInfo->OverrideFullImportName;
					FoundMeshInfo->OptionName = OriginalMeshInfo->OptionName;
					break;
				}
			}
		}

		if (FoundMeshInfo.IsValid() && FoundMeshInfo->bOriginalTypeChanged)
		{
			//We dont reimport asset that have change their type
			EFbxSceneReimportStatusFlags StatusFlag = EFbxSceneReimportStatusFlags::None;
			MeshStatusMap->Add(FoundMeshInfo->OriginalImportPath, StatusFlag);
		}
		else if (FoundMeshInfo.IsValid())
		{
			//Set the old pivot information if we find one
			FbxNodeInfoPtr OriginalPivotNode = SFbxSSceneBaseMeshListView::FindNodeInfoByUid(OriginalMeshInfo->PivotNodeUid, SceneInfoOriginal);
			if (OriginalPivotNode.IsValid())
			{
				for (FbxNodeInfoPtr NodeInfo : SceneInfo->HierarchyInfo)
				{
					if (OriginalPivotNode->NodeHierarchyPath.Compare(NodeInfo->NodeHierarchyPath) == 0)
					{
						FoundMeshInfo->PivotNodeUid = NodeInfo->UniqueId;
						FoundMeshInfo->PivotNodeName = NodeInfo->NodeName;
						break;
					}
				}
			}

			//We have a match
			EFbxSceneReimportStatusFlags StatusFlag = EFbxSceneReimportStatusFlags::Same;
			if (OriginalMeshInfo->GetContentObject() != nullptr)
			{
				StatusFlag |= EFbxSceneReimportStatusFlags::FoundContentBrowserAsset;
			}
			if (OriginalMeshInfo->bImportAttribute)
			{
				StatusFlag |= EFbxSceneReimportStatusFlags::ReimportAsset;
			}
			MeshStatusMap->Add(FoundMeshInfo->OriginalImportPath, StatusFlag);

		}
		else
		{
			//we have a delete asset
			EFbxSceneReimportStatusFlags StatusFlag = EFbxSceneReimportStatusFlags::Removed;
			//Is the asset exist in content browser
			UPackage* PkgExist = OriginalMeshInfo->GetContentPackage();
			if (PkgExist != nullptr)
			{
				PkgExist->FullyLoad();
				//Delete the asset by default
				StatusFlag |= EFbxSceneReimportStatusFlags::FoundContentBrowserAsset | EFbxSceneReimportStatusFlags::ReimportAsset;
				MeshStatusMap->Add(OriginalMeshInfo->OriginalImportPath, StatusFlag);
				ValidFbxMeshesArrayPtr->Add(OriginalMeshInfo);
				AddToFilterFbxMeshesArrayPtr(OriginalMeshInfo);
				//When the asset do not exist in the new fbx we have to add it so we can delete it
				SceneInfo->MeshInfo.Add(OriginalMeshInfo);
			}
			//If the asset is not there anymore we do not care about it
		}
	}
}

TSharedPtr<FFbxNodeInfo> SFbxSSceneBaseMeshListView::FindNodeInfoByUid(uint64 NodeUid, TSharedPtr<FFbxSceneInfo> SceneInfoToSearch)
{
	for (FbxNodeInfoPtr NodeInfo : SceneInfoToSearch->HierarchyInfo)
	{
		if (NodeInfo->UniqueId == NodeUid)
			return NodeInfo;
	}
	return nullptr;
}

void SFbxSSceneBaseMeshListView::AssignToPivot(uint64 NodeUid)
{
	FbxNodeInfoPtr NodeInfo = FindNodeInfoByUid(NodeUid, SceneInfo);
	TArray<FbxMeshInfoPtr> SelectedFbxMeshes;
	int32 SelectCount = GetSelectedItems(SelectedFbxMeshes);
	for (FbxMeshInfoPtr MeshInfo : SelectedFbxMeshes)
	{
		if (MeshInfo->bOriginalTypeChanged)
		{
			continue;
		}
		MeshInfo->PivotNodeUid = NodeUid;
		if (NodeUid == INVALID_UNIQUE_ID)
		{
			MeshInfo->PivotNodeName = TEXT("-");
		}
		else if(NodeInfo.IsValid())
		{
			MeshInfo->PivotNodeName = NodeInfo->NodeName;
		}
	}
}

TSharedPtr<FString> SFbxSSceneBaseMeshListView::FindOptionNameFromName(FString OptionName)
{
	for (TSharedPtr<FString> OptionNamePtr : (*OverrideNameOptions))
	{
		if (OptionNamePtr->Compare(OptionName) == 0)
		{
			return OptionNamePtr;
		}
	}
	return TSharedPtr<FString>();
}

void SFbxSSceneBaseMeshListView::AssignToOptions(FString OptionName)
{
	bool IsDefaultOptions = OptionName.Compare(UFbxSceneImportFactory::DefaultOptionName) == 0;
	if (!OverrideNameOptionsMap->Contains(OptionName) && !IsDefaultOptions)
	{
		return;
	}
	TArray<FbxMeshInfoPtr> SelectedFbxMeshes;
	GetSelectedItems(SelectedFbxMeshes);
	for (FbxMeshInfoPtr ItemPtr : SelectedFbxMeshes)
	{
		if (ItemPtr->bOriginalTypeChanged)
		{
			continue;
		}
		ItemPtr->OptionName = OptionName;
	}
	OptionComboBox->SetSelectedItem(FindOptionNameFromName(OptionName));
}

bool SFbxSSceneBaseMeshListView::CanDeleteOverride()  const
{
	return CurrentMeshImportOptions != GlobalImportSettings;
}

FReply SFbxSSceneBaseMeshListView::OnDeleteOverride()
{
	if (!CanDeleteOverride())
	{
		return FReply::Unhandled();
	}

	FString CurrentOptionName;

	for (auto kvp : *(OverrideNameOptionsMap))
	{
		if (kvp.Value == CurrentMeshImportOptions)
		{
			CurrentOptionName = kvp.Key;
			if (CurrentOptionName.Compare(UFbxSceneImportFactory::DefaultOptionName) == 0)
			{
				//Cannot delete the default options
				return FReply::Handled();
			}
			break;
		}
	}
	if (CurrentOptionName.IsEmpty())
	{
		return FReply::Handled();
	}

	TArray<TSharedPtr<FFbxMeshInfo>> RemoveKeys;
	TArray<TSharedPtr<FFbxMeshInfo>> SceneMeshOverrideKeys;
	for (TSharedPtr<FFbxMeshInfo> MeshInfo : FbxMeshesArray)
	{
		if (CurrentOptionName.Compare(MeshInfo->OptionName) == 0)
		{
			MeshInfo->OptionName = UFbxSceneImportFactory::DefaultOptionName;
		}
	}
	OverrideNameOptions->Remove(FindOptionNameFromName(CurrentOptionName));
	OverrideNameOptionsMap->Remove(CurrentOptionName);
	UnFbx::FBXImportOptions *OldOverrideOption = CurrentMeshImportOptions;
	delete OldOverrideOption;

	CurrentMeshImportOptions = GlobalImportSettings;
	OptionComboBox->SetSelectedItem((*OverrideNameOptions)[0]);

	return FReply::Handled();
}

FReply SFbxSSceneBaseMeshListView::OnSelectAssetUsing()
{
	FString CurrentOptionName;

	for (auto kvp : *(OverrideNameOptionsMap))
	{
		if (kvp.Value == CurrentMeshImportOptions)
		{
			CurrentOptionName = kvp.Key;
			break;
		}
	}
	if (CurrentOptionName.IsEmpty())
	{
		return FReply::Handled();
	}
	ClearSelection();
	for (FbxMeshInfoPtr MeshInfo : FbxMeshesArray)
	{
		if (CurrentOptionName.Compare(MeshInfo->OptionName) == 0)
		{
			SetItemSelection(MeshInfo, true);
		}
	}
	return FReply::Handled();
}

FString SFbxSSceneBaseMeshListView::FindUniqueOptionName(FString OverrideName, bool bForceNumber)
{
	int32 SuffixeIndex = 1;
	bool bFoundSimilarName = false;
	FString UniqueOptionName = bForceNumber ? (OverrideName + TEXT(" ") + FString::FromInt(SuffixeIndex++)) : OverrideName;
	do
	{
		bFoundSimilarName = false;
		for (auto kvp : *(OverrideNameOptionsMap))
		{
			if (kvp.Key.Compare(UniqueOptionName) == 0)
			{
				bFoundSimilarName = true;
				UniqueOptionName = OverrideName + TEXT(" ") + FString::FromInt(SuffixeIndex++);
				break;
			}
		}
	} while (bFoundSimilarName);

	return UniqueOptionName;
}

void SFbxSSceneBaseMeshListView::OnCreateOverrideOptionsWithName(const FText& CommittedText, ETextCommit::Type CommitType)
{
	FString OverrideName = CommittedText.ToString();
	if (CommitType == ETextCommit::OnEnter)
	{
		bool bForceNumber = (OverrideName.Compare(TEXT("Options")) == 0);
		OverrideName = FindUniqueOptionName(OverrideName, bForceNumber);
		UnFbx::FBXImportOptions *OverrideOption = new UnFbx::FBXImportOptions();
		SFbxSceneOptionWindow::CopyFbxOptionsToFbxOptions(GlobalImportSettings, OverrideOption);
		TSharedPtr<FString> OverrideNamePtr = MakeShareable(new FString(OverrideName));
		OverrideNameOptions->Add(OverrideNamePtr);
		OverrideNameOptionsMap->Add(OverrideName, OverrideOption);
		//Update the selection to the new override
		OptionComboBox->SetSelectedItem(OverrideNamePtr);
		FSlateApplication::Get().DismissAllMenus();
	}
	else if (CommitType == ETextCommit::OnCleared)
	{
		//Dont create options set if the user cancel the input
		FSlateApplication::Get().DismissAllMenus();
	}
}

FReply SFbxSSceneBaseMeshListView::OnCreateOverrideOptions()
{
	//pop a dialog to ask the option name, if the user cancel the name will be "Option #"
	TSharedRef<STextEntryPopup> TextEntry =
		SNew(STextEntryPopup)
		.Label(LOCTEXT("FbxOptionWindow_SM_CreateOverrideAskName", "Override Option name"))
		.DefaultText(FText::FromString(TEXT("Options")))
		.OnTextCommitted(this, &SFbxSSceneBaseMeshListView::OnCreateOverrideOptionsWithName);

	FSlateApplication& SlateApp = FSlateApplication::Get();
	SlateApp.PushMenu(
		AsShared(),
		FWidgetPath(),
		TextEntry,
		SlateApp.GetCursorPos(),
		FPopupTransitionEffect::TypeInPopup
		);

	return FReply::Handled();
}

TSharedPtr<STextComboBox> SFbxSSceneBaseMeshListView::CreateOverrideOptionComboBox()
{
	OptionComboBox = SNew(STextComboBox)
		.OptionsSource(OverrideNameOptions)
		.InitiallySelectedItem(DefaultOptionNamePtr)
		.OnSelectionChanged(this, &SFbxSSceneBaseMeshListView::OnChangedOverrideOptions)
		.ToolTipText(LOCTEXT("FbxOptionWindow_SM_CreateOverrideComboboxTooltip", "Select the options set you want to modify.\nTo assign options use context menu on meshes."));
	return OptionComboBox;
}
#undef LOCTEXT_NAMESPACE
