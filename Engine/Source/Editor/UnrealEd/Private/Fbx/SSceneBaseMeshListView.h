// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Input/Reply.h"
#include "Factories/FbxSceneImportFactory.h"
#include "Widgets/Views/SListView.h"

enum class EFbxSceneReimportStatusFlags : uint8;

typedef TSharedPtr<FFbxMeshInfo> FbxMeshInfoPtr;
typedef TMap<FString, EFbxSceneReimportStatusFlags> FbxSceneReimportStatusMap;
typedef FbxSceneReimportStatusMap* FbxSceneReimportStatusMapPtr;
typedef TArray<TSharedPtr<FString>> FbxOverrideNameOptionsArray;
typedef FbxOverrideNameOptionsArray* FbxOverrideNameOptionsArrayPtr;

namespace FbxSceneBaseListViewColumn
{
	const FName PivotColumnId(TEXT("PivotNameHeaderId"));
}

class SFbxSSceneBaseMeshListView : public SListView<FbxMeshInfoPtr>
{
public:
	
	~SFbxSSceneBaseMeshListView();

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime);
	
	FReply OnCreateOverrideOptions();
	TSharedPtr<STextComboBox> CreateOverrideOptionComboBox();
	bool CanDeleteOverride() const;
	FReply OnDeleteOverride();
	void OnCreateOverrideOptionsWithName(const FText& CommittedText, ETextCommit::Type CommitType);
	virtual void OnChangedOverrideOptions(TSharedPtr<FString> ItemSelected, ESelectInfo::Type SelectInfo) = 0;
	
	FReply OnSelectAssetUsing();
	
	void OnSelectionChanged(FbxMeshInfoPtr Item, ESelectInfo::Type SelectionType);
	virtual void OnToggleSelectAll(ECheckBoxState CheckType);

	static void FillMeshStatusMap(FbxSceneReimportStatusMapPtr MeshStatusMap
		, TSharedPtr<FFbxSceneInfo> SceneInfo
		, TSharedPtr<FFbxSceneInfo> SceneInfoOriginal
		, bool bFillSkeletalMeshStatusMap
		, TArray<FbxMeshInfoPtr>* FilterFbxMeshesArrayPtr = nullptr
		, TArray<FbxMeshInfoPtr>* FbxMeshesArrayPtr = nullptr);

	static TSharedPtr<FFbxNodeInfo> FindNodeInfoByUid(uint64 NodeUid, TSharedPtr<FFbxSceneInfo> SceneInfoOriginal);

protected:
	TSharedPtr<FFbxSceneInfo> SceneInfo;
	UnFbx::FBXImportOptions* GlobalImportSettings;
	UnFbx::FBXImportOptions *CurrentMeshImportOptions;
	/** the elements we show in the tree view */
	TArray<FbxMeshInfoPtr> FbxMeshesArray;

	
	void AddSelectionToImport();
	void RemoveSelectionFromImport();
	virtual void SetSelectionImportState(bool MarkForImport);
	

	TSharedPtr<FString> FindOptionNameFromName(FString OptionName);
	void AssignToOptions(FString OptionName);
	FString FindUniqueOptionName(FString OverrideName, bool bForceNumber);
	
	void AddBakePivotMenu(class FMenuBuilder& MenuBuilder);
	void FillPivotContextMenu(class FMenuBuilder& MenuBuilder);
	void AssignToPivot(uint64 NodeUid);
	
	FbxOverrideNameOptionsArrayPtr OverrideNameOptions;
	ImportOptionsNameMapPtr OverrideNameOptionsMap;
	TSharedPtr<STextComboBox> OptionComboBox;
	TSharedPtr<FString> DefaultOptionNamePtr;
};
