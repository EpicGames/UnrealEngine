// Copyright Epic Games, Inc. All Rights Reserved.

#include "Fbx/SSceneImportNodeTreeView.h"

#include "Camera/CameraComponent.h"
#include "Components/DirectionalLightComponent.h"
#include "Components/LightComponent.h"
#include "Components/PointLightComponent.h"
#include "Components/SpotLightComponent.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Factories/FbxSceneImportFactory.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Views/ITypedTableView.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Visibility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateIconFinder.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Views/STableRow.h"

class FUICommandList;
class ITableRow;
class SWidget;
class UClass;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "SFbxSceneTreeView"

SFbxSceneTreeView::~SFbxSceneTreeView()
{
	FbxRootNodeArray.Empty();
	SceneInfo = NULL;
}

void SFbxSceneTreeView::Construct(const SFbxSceneTreeView::FArguments& InArgs)
{
	SceneInfo = InArgs._SceneInfo;
	//Build the FbxNodeInfoPtr tree data
	check(SceneInfo.IsValid());
	for (auto NodeInfoIt = SceneInfo->HierarchyInfo.CreateIterator(); NodeInfoIt; ++NodeInfoIt)
	{
		FbxNodeInfoPtr NodeInfo = (*NodeInfoIt);

		if (!NodeInfo->ParentNodeInfo.IsValid())
		{
			FbxRootNodeArray.Add(NodeInfo);
		}
	}

	STreeView::Construct
	(
		STreeView::FArguments()
		.TreeItemsSource(&FbxRootNodeArray)
		.SelectionMode(ESelectionMode::Multi)
		.OnGenerateRow(this, &SFbxSceneTreeView::OnGenerateRowFbxSceneTreeView)
		.OnGetChildren(this, &SFbxSceneTreeView::OnGetChildrenFbxSceneTreeView)
		.OnContextMenuOpening(this, &SFbxSceneTreeView::OnOpenContextMenu)
		.OnSelectionChanged(this, &SFbxSceneTreeView::OnSelectionChanged)
	);
}

/** The item used for visualizing the class in the tree. */
class SFbxSceneTreeViewItem : public STableRow< FbxNodeInfoPtr >
{
public:

	SLATE_BEGIN_ARGS(SFbxSceneTreeViewItem)
		: _FbxNodeInfo(nullptr)
		, _SceneInfo(nullptr)
	{}

	/** The item content. */
	SLATE_ARGUMENT(FbxNodeInfoPtr, FbxNodeInfo)
	SLATE_ARGUMENT(TSharedPtr<FFbxSceneInfo>, SceneInfo)
	SLATE_END_ARGS()

	/**
	* Construct the widget
	*
	* @param InArgs   A declaration from which to construct the widget
	*/
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		FbxNodeInfo = InArgs._FbxNodeInfo;
		SceneInfo = InArgs._SceneInfo;
		
		//This is suppose to always be valid
		check(FbxNodeInfo.IsValid());
		check(SceneInfo.IsValid());

		UClass *IconClass = AActor::StaticClass();
		if (FbxNodeInfo->AttributeInfo.IsValid())
		{
			IconClass = FbxNodeInfo->AttributeInfo->GetType();
		}
		else if (FbxNodeInfo->AttributeType.Compare(TEXT("eLight")) == 0)
		{
			IconClass = ULightComponent::StaticClass();
			if (SceneInfo->LightInfo.Contains(FbxNodeInfo->AttributeUniqueId))
			{
				TSharedPtr<FFbxLightInfo> LightInfo = *SceneInfo->LightInfo.Find(FbxNodeInfo->AttributeUniqueId);
				if (LightInfo->Type == 0)
				{
					IconClass = UPointLightComponent::StaticClass();
				}
				else if (LightInfo->Type == 1)
				{
					IconClass = UDirectionalLightComponent::StaticClass();
				}
				else if (LightInfo->Type == 2)
				{
					IconClass = USpotLightComponent::StaticClass();
				}
			}
		}
		else if (FbxNodeInfo->AttributeType.Compare(TEXT("eCamera")) == 0)
		{
			IconClass = UCameraComponent::StaticClass();
		}
		const FSlateBrush* ClassIcon = FSlateIconFinder::FindIconBrushForClass(IconClass);

		//Prepare the tooltip
		FString Tooltip = FbxNodeInfo->NodeName;
		if (!FbxNodeInfo->AttributeType.IsEmpty() && FbxNodeInfo->AttributeType.Compare(TEXT("eNull")) != 0)
		{
			Tooltip += TEXT(" [") + FbxNodeInfo->AttributeType + TEXT("]");
		}

		this->ChildSlot
		[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f, 0.0f, 2.0f, 0.0f)
			[
				SNew(SCheckBox)
				.OnCheckStateChanged(this, &SFbxSceneTreeViewItem::OnItemCheckChanged)
				.IsChecked(this, &SFbxSceneTreeViewItem::IsItemChecked)
			]
		+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SExpanderArrow, SharedThis(this))
			]

		+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 2.0f, 6.0f, 2.0f)
			[
				SNew(SImage)
				.Image(ClassIcon)
				.Visibility(ClassIcon != FAppStyle::GetDefaultBrush() ? EVisibility::Visible : EVisibility::Collapsed)
			]

		+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(0.0f, 3.0f, 6.0f, 3.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::FromString(FbxNodeInfo->NodeName))
				.ToolTipText(FText::FromString(Tooltip))
			]

		];

		STableRow< FbxNodeInfoPtr >::ConstructInternal(
			STableRow::FArguments()
			.ShowSelection(true),
			InOwnerTableView
		);
	}

private:

	void RecursivelySetLODMeshImportState(TSharedPtr<FFbxNodeInfo> NodeInfo, bool bState)
	{
		//Set the bImportNode property for all mesh under eLODGroup
		for (TSharedPtr<FFbxNodeInfo> ChildNodeInfo : NodeInfo->Childrens)
		{
			if (!ChildNodeInfo.IsValid())
				continue;
			if (ChildNodeInfo->AttributeType.Compare(TEXT("eMesh")) == 0)
			{
				ChildNodeInfo->bImportNode = bState;
			}
			else
			{
				RecursivelySetLODMeshImportState(ChildNodeInfo, bState);
			}
		}
	}

	void OnItemCheckChanged(ECheckBoxState CheckType)
	{
		if (!FbxNodeInfo.IsValid())
			return;
		FbxNodeInfo->bImportNode = CheckType == ECheckBoxState::Checked;
		if (FbxNodeInfo->AttributeType.Compare(TEXT("eLODGroup")) == 0)
		{
			RecursivelySetLODMeshImportState(FbxNodeInfo, FbxNodeInfo->bImportNode);
		}
		if (FbxNodeInfo->AttributeType.Compare(TEXT("eMesh")) == 0)
		{
			//Verify if parent is a LOD group
			TSharedPtr<FFbxNodeInfo> ParentLODNodeInfo = FFbxSceneInfo::RecursiveFindLODParentNode(FbxNodeInfo);
			if (ParentLODNodeInfo.IsValid())
			{
				ParentLODNodeInfo->bImportNode = FbxNodeInfo->bImportNode;
				RecursivelySetLODMeshImportState(ParentLODNodeInfo, FbxNodeInfo->bImportNode);
			}
		}
	}

	ECheckBoxState IsItemChecked() const
	{
		return FbxNodeInfo->bImportNode ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	/** The node info to build the tree view row from. */
	FbxNodeInfoPtr FbxNodeInfo;
	TSharedPtr<FFbxSceneInfo> SceneInfo;
};

TSharedRef< ITableRow > SFbxSceneTreeView::OnGenerateRowFbxSceneTreeView(FbxNodeInfoPtr Item, const TSharedRef< STableViewBase >& OwnerTable)
{
	TSharedRef< SFbxSceneTreeViewItem > ReturnRow = SNew(SFbxSceneTreeViewItem, OwnerTable)
		.FbxNodeInfo(Item)
		.SceneInfo(SceneInfo);
	return ReturnRow;
}

void SFbxSceneTreeView::OnGetChildrenFbxSceneTreeView(FbxNodeInfoPtr InParent, TArray< FbxNodeInfoPtr >& OutChildren)
{
	if (InParent->AttributeType.Compare(TEXT("eLODGroup")) == 0 && InParent->Childrens.Num() > 0)
	{
		TSharedPtr<FFbxNodeInfo> Child = InParent;
		do
		{
			Child = Child->Childrens[0];
			if (Child.IsValid() && Child->AttributeType.Compare(TEXT("eMesh")) == 0)
			{
				OutChildren.Add(Child);
				return;
			}
		} while (Child->Childrens.Num() > 0);
		//We do not found any LOD mesh don't add any child
	}
	else
	{
		for (TSharedPtr<FFbxNodeInfo> Child : InParent->Childrens)
		{
			//We hide skeletal mesh from the tree, a mesh without a valid attributeinfo is a sub skeletal mesh
			if (Child.IsValid() && (Child->AttributeType.Compare(TEXT("eMesh")) != 0 || Child->AttributeInfo.IsValid()))
			{
				OutChildren.Add(Child);
			}
		}
	}
}

void SFbxSceneTreeView::RecursiveSetImport(FbxNodeInfoPtr NodeInfoPtr, bool ImportStatus)
{
	NodeInfoPtr->bImportNode = ImportStatus;
	for (auto Child : NodeInfoPtr->Childrens)
	{
		RecursiveSetImport(Child, ImportStatus);
	}
}

void SFbxSceneTreeView::OnToggleSelectAll(ECheckBoxState CheckType)
{
	//check all actor for import
	for (auto NodeInfoIt = SceneInfo->HierarchyInfo.CreateIterator(); NodeInfoIt; ++NodeInfoIt)
	{
		FbxNodeInfoPtr NodeInfo = (*NodeInfoIt);

		if (!NodeInfo->ParentNodeInfo.IsValid())
		{
			RecursiveSetImport(NodeInfo, CheckType == ECheckBoxState::Checked);
		}
	}
}

void RecursiveSetExpand(SFbxSceneTreeView *TreeView, FbxNodeInfoPtr NodeInfoPtr, bool ExpandState)
{
	TreeView->SetItemExpansion(NodeInfoPtr, ExpandState);
	for (auto Child : NodeInfoPtr->Childrens)
	{
		RecursiveSetExpand(TreeView, Child, ExpandState);
	}
}

FReply SFbxSceneTreeView::OnExpandAll()
{
	for (auto NodeInfoIt = SceneInfo->HierarchyInfo.CreateIterator(); NodeInfoIt; ++NodeInfoIt)
	{
		FbxNodeInfoPtr NodeInfo = (*NodeInfoIt);

		if (!NodeInfo->ParentNodeInfo.IsValid())
		{
			RecursiveSetExpand(this, NodeInfo, true);
		}
	}
	return FReply::Handled();
}

FReply SFbxSceneTreeView::OnCollapseAll()
{
	for (auto NodeInfoIt = SceneInfo->HierarchyInfo.CreateIterator(); NodeInfoIt; ++NodeInfoIt)
	{
		FbxNodeInfoPtr NodeInfo = (*NodeInfoIt);

		if (!NodeInfo->ParentNodeInfo.IsValid())
		{
			RecursiveSetExpand(this, NodeInfo, false);
		}
	}
	return FReply::Handled();
}

TSharedPtr<SWidget> SFbxSceneTreeView::OnOpenContextMenu()
{
	// Build up the menu for a selection
	const bool bCloseAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, TSharedPtr<FUICommandList>());

	//Get the different type of the multi selection
	TSet<TSharedPtr<FFbxAttributeInfo>> SelectAssets;
	TArray<FbxNodeInfoPtr> SelectedFbxNodeInfos;
	const auto NumSelectedItems = GetSelectedItems(SelectedFbxNodeInfos);
	for (auto Item : SelectedFbxNodeInfos)
	{
		FbxNodeInfoPtr ItemPtr = Item;
		if (ItemPtr->AttributeInfo.IsValid())
		{
			SelectAssets.Add(ItemPtr->AttributeInfo);
		}
	}

	// We always create a section here, even if there is no parent so that clients can still extend the menu
	MenuBuilder.BeginSection("FbxSceneTreeViewContextMenuImportSection");
	{
		const FSlateIcon PlusIcon(FAppStyle::GetAppStyleSetName(), "Icons.Plus");
		MenuBuilder.AddMenuEntry(LOCTEXT("CheckForImport", "Add Selection To Import"), FText(), PlusIcon, FUIAction(FExecuteAction::CreateSP(this, &SFbxSceneTreeView::AddSelectionToImport)));
		const FSlateIcon MinusIcon(FAppStyle::GetAppStyleSetName(), "Icons.Minus");
		MenuBuilder.AddMenuEntry(LOCTEXT("UncheckForImport", "Remove Selection From Import"), FText(), MinusIcon, FUIAction(FExecuteAction::CreateSP(this, &SFbxSceneTreeView::RemoveSelectionFromImport)));
	}
	MenuBuilder.EndSection();
/*
	if (SelectAssets.Num() > 0)
	{
		MenuBuilder.AddMenuSeparator();
		MenuBuilder.BeginSection("FbxSceneTreeViewContextMenuGotoAssetSection", LOCTEXT("Gotoasset","Go to asset:"));
		{
			for (auto AssetItem : SelectAssets)
			{
				//const FSlateBrush* ClassIcon = FClassIconFinder::FindIconForClass(AssetItem->GetType());
				MenuBuilder.AddMenuEntry(FText::FromString(AssetItem->Name), FText(), FSlateIcon(), FUIAction(FExecuteAction::CreateSP(this, &SFbxSceneTreeView::GotoAsset, AssetItem)));
			}
		}
		MenuBuilder.EndSection();
	}
*/
	

	return MenuBuilder.MakeWidget();
}

void SFbxSceneTreeView::AddSelectionToImport()
{
	SetSelectionImportState(true);
}

void SFbxSceneTreeView::RemoveSelectionFromImport()
{
	SetSelectionImportState(false);
}

void SFbxSceneTreeView::SetSelectionImportState(bool MarkForImport)
{
	TArray<FbxNodeInfoPtr> SelectedFbxNodeInfos;
	GetSelectedItems(SelectedFbxNodeInfos);
	for (auto Item : SelectedFbxNodeInfos)
	{
		FbxNodeInfoPtr ItemPtr = Item;
		ItemPtr->bImportNode = MarkForImport;
	}
}

void SFbxSceneTreeView::OnSelectionChanged(FbxNodeInfoPtr Item, ESelectInfo::Type SelectionType)
{
}

void SFbxSceneTreeView::GotoAsset(TSharedPtr<FFbxAttributeInfo> AssetAttribute)
{
	//Switch to the asset tab and select the asset
}

#undef LOCTEXT_NAMESPACE
