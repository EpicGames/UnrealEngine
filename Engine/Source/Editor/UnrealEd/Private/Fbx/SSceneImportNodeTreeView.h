// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Misc/Optional.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

class FFbxAttributeInfo;
class FFbxNodeInfo;
class FFbxSceneInfo;
class ITableRow;
class SWidget;

typedef TSharedPtr<FFbxNodeInfo> FbxNodeInfoPtr;

class SFbxSceneTreeView : public STreeView<FbxNodeInfoPtr>
{
public:
	~SFbxSceneTreeView();
	SLATE_BEGIN_ARGS(SFbxSceneTreeView)
	: _SceneInfo(NULL)
	{}
		SLATE_ARGUMENT(TSharedPtr<FFbxSceneInfo>, SceneInfo)
	SLATE_END_ARGS()
	
	/** Construct this widget */
	void Construct(const FArguments& InArgs);
	TSharedRef< ITableRow > OnGenerateRowFbxSceneTreeView(FbxNodeInfoPtr Item, const TSharedRef< STableViewBase >& OwnerTable);
	void OnGetChildrenFbxSceneTreeView(FbxNodeInfoPtr InParent, TArray< FbxNodeInfoPtr >& OutChildren);

	void OnToggleSelectAll(ECheckBoxState CheckType);
	FReply OnExpandAll();
	FReply OnCollapseAll();
protected:
	TSharedPtr<FFbxSceneInfo> SceneInfo;


	/** the elements we show in the tree view */
	TArray<FbxNodeInfoPtr> FbxRootNodeArray;

	/** Open a context menu for the current selection */
	TSharedPtr<SWidget> OnOpenContextMenu();
	void AddSelectionToImport();
	void RemoveSelectionFromImport();
	void SetSelectionImportState(bool MarkForImport);
	void OnSelectionChanged(FbxNodeInfoPtr Item, ESelectInfo::Type SelectionType);

	void GotoAsset(TSharedPtr<FFbxAttributeInfo> AssetAttribute);
	void RecursiveSetImport(FbxNodeInfoPtr NodeInfoPtr, bool ImportStatus);
};
