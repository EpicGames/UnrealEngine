// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Fbx/SSceneBaseMeshListView.h"

struct FPropertyChangedEvent;

class SFbxSceneStaticMeshListView : public SFbxSSceneBaseMeshListView
{
public:
	
	~SFbxSceneStaticMeshListView();

	SLATE_BEGIN_ARGS(SFbxSceneStaticMeshListView)
	: _SceneInfo(nullptr)
	, _GlobalImportSettings(nullptr)
	, _OverrideNameOptionsMap(nullptr)
	, _SceneImportOptionsStaticMeshDisplay(nullptr)
	{}
		SLATE_ARGUMENT(TSharedPtr<FFbxSceneInfo>, SceneInfo)
		SLATE_ARGUMENT(UnFbx::FBXImportOptions*, GlobalImportSettings)
		SLATE_ARGUMENT(FbxOverrideNameOptionsArrayPtr, OverrideNameOptions)
		SLATE_ARGUMENT(ImportOptionsNameMapPtr, OverrideNameOptionsMap)
		SLATE_ARGUMENT(class UFbxSceneImportOptionsStaticMesh*, SceneImportOptionsStaticMeshDisplay)
	SLATE_END_ARGS()

	/** Construct this widget */
	void Construct(const FArguments& InArgs);
	TSharedRef< ITableRow > OnGenerateRowFbxSceneListView(FbxMeshInfoPtr Item, const TSharedRef< STableViewBase >& OwnerTable);
	
	void OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent);

protected:
	/** the elements we show in the tree view */
	class UFbxSceneImportOptionsStaticMesh *SceneImportOptionsStaticMeshDisplay;
	
	/** Open a context menu for the current selection */
	TSharedPtr<SWidget> OnOpenContextMenu();

	virtual void OnChangedOverrideOptions(TSharedPtr<FString> ItemSelected, ESelectInfo::Type SelectInfo);
};
