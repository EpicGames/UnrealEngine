// Copyright Epic Games, Inc. All Rights Reserved.

#include "Fbx/SSceneMaterialsListView.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SCheckBox.h"
#include "Styling/AppStyle.h"
#include "Materials/MaterialInterface.h"
#include "Materials/Material.h"
#include "AssetRegistry/AssetData.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "Fbx/SSceneImportNodeTreeView.h"
#include "FbxImporter.h"
#include "PackageTools.h"

#define LOCTEXT_NAMESPACE "SFbxSceneMaterialsListView"

const FName MaterialCheckBoxSelectionHeaderIdName(TEXT("CheckBoxSelectionHeaderId"));
const FName MaterialNameHeaderIdName(TEXT("AssetNameHeaderId"));
const FName MaterialContentPathHeaderIdName(TEXT("ContentPathHeaderId"));
const FName MaterialStatusNameHeaderIdName(TEXT("OptionsNameHeaderId"));

/** The item used for visualizing the class in the tree. */
class SFbxMaterialItemTableListViewRow : public SMultiColumnTableRow<FbxMaterialInfoPtr>
{
public:

	SLATE_BEGIN_ARGS(SFbxMaterialItemTableListViewRow)
		: _FbxMaterialInfo(nullptr)
	{}

	/** The item content. */
	SLATE_ARGUMENT(FbxMaterialInfoPtr, FbxMaterialInfo)
	SLATE_END_ARGS()

	/**
	* Construct the widget
	*
	* @param InArgs   A declaration from which to construct the widget
	*/
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		FbxMaterialInfo = InArgs._FbxMaterialInfo;

		//This is suppose to always be valid
		check(FbxMaterialInfo.IsValid());

		SMultiColumnTableRow<FbxMaterialInfoPtr>::Construct(
			FSuperRowType::FArguments()
			.Style(FAppStyle::Get(), "DataTableEditor.CellListViewRow"),
			InOwnerTableView
			);
	}

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the list view. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == MaterialCheckBoxSelectionHeaderIdName)
		{
			return SNew(SBox)
				.HAlign(HAlign_Center)
				[
					SNew(SCheckBox)
					.OnCheckStateChanged(this, &SFbxMaterialItemTableListViewRow::OnItemCheckChanged)
					.IsChecked(this, &SFbxMaterialItemTableListViewRow::IsItemChecked)
				];
		}
		else if (ColumnName == MaterialNameHeaderIdName)
		{
			return SNew(STextBlock)
				.Text(FText::FromString(FbxMaterialInfo->Name))
				.ToolTipText(FText::FromString(FbxMaterialInfo->Name));
		}
		else if (ColumnName == MaterialContentPathHeaderIdName)
		{
			return SNew(STextBlock)
				.Text(this, &SFbxMaterialItemTableListViewRow::GetAssetFullName)
				.ColorAndOpacity(this, &SFbxMaterialItemTableListViewRow::GetContentPathTextColor)
				.ToolTipText(this, &SFbxMaterialItemTableListViewRow::GetAssetFullName);
		}
		else if (ColumnName == MaterialStatusNameHeaderIdName)
		{
			return SNew(STextBlock)
				.Text(this, &SFbxMaterialItemTableListViewRow::GetAssetStatus)
				.ToolTipText(this, &SFbxMaterialItemTableListViewRow::GetAssetStatus);
		}

		return SNullWidget::NullWidget;
	}

private:

	void OnItemCheckChanged(ECheckBoxState CheckType)
	{
		if (!FbxMaterialInfo.IsValid())
			return;
		FbxMaterialInfo->bImportAttribute = CheckType == ECheckBoxState::Checked;
	}

	ECheckBoxState IsItemChecked() const
	{
		return FbxMaterialInfo->bImportAttribute ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	FSlateColor GetContentPathTextColor() const
	{
		return FbxMaterialInfo->bOverridePath ? FLinearColor(0.75f, 0.75f, 0.0f, 1.0f) : FSlateColor::UseForeground();
	}

	FText GetAssetFullName() const
	{
		return FText::FromString(FbxMaterialInfo->GetFullImportName());
	}

	FText GetAssetStatus() const
	{
		//Is the material already exist
		if (FbxMaterialInfo->GetContentObject() != nullptr)
		{
			return FText::FromString(TEXT("Use Existing"));
		}
		else
		{
			return FText::FromString(TEXT("Create"));
		}
	}

	/** The node info to build the tree view row from. */
	FbxMaterialInfoPtr FbxMaterialInfo;
};

//////////////////////////////////////////////////////////////////////////
// Materials List

SFbxSceneMaterialsListView::~SFbxSceneMaterialsListView()
{
	SceneInfo = nullptr;
	SceneInfoOriginal = nullptr;
	GlobalImportSettings = nullptr;
	MaterialsArray.Reset();
	TexturesArray = nullptr;
	FullPath.Empty();
	IsReimport = false;
	CreateContentFolderHierarchy = false;
}

void SFbxSceneMaterialsListView::GetMaterialsFromHierarchy(TArray<FbxMaterialInfoPtr> &OutMaterials, TSharedPtr<FFbxSceneInfo> SceneInfoSource, bool bFillPathInformation)
{
	OutMaterials.Reset();
	for (FbxNodeInfoPtr NodeInfo : SceneInfoSource->HierarchyInfo)
	{
		//Get all the mesh material
		if (NodeInfo->AttributeType.Compare(TEXT("eMesh")) != 0 || NodeInfo->NodeName.Compare("RootNode") == 0)
		{
			continue;
		}
		for (TSharedPtr<FFbxMaterialInfo> MaterialInfo : NodeInfo->Materials)
		{
			bool bAlreadyExist = false;
			for (TSharedPtr<FFbxMaterialInfo> ExistingMaterialInfo : OutMaterials)
			{
				if (ExistingMaterialInfo->UniqueId == MaterialInfo->UniqueId)
				{
					bAlreadyExist = true;
					break;
				}
			}
			if (bAlreadyExist)
			{
				//This Material is already in the list
				continue;
			}
			if (bFillPathInformation)
			{
				FString NodeTreePath;
				if (CreateContentFolderHierarchy)
				{
					FbxNodeInfoPtr CurrentNode = NodeInfo->ParentNodeInfo;
					while (CurrentNode.IsValid())
					{
						if (CurrentNode->NodeName.Compare("RootNode") != 0)
						{
							FString StringToInsert = TEXT("/") + CurrentNode->NodeName;
							NodeTreePath.InsertAt(0, StringToInsert);
						}
						CurrentNode = CurrentNode->ParentNodeInfo;
					}
				}
				FString AssetName = GlobalImportSettings->MaterialBasePath == NAME_None ? FullPath + NodeTreePath + TEXT("/") + MaterialInfo->Name : GlobalImportSettings->MaterialBasePath.ToString() + MaterialInfo->Name;
				MaterialInfo->SetOriginalImportPath(AssetName);
				FString OriginalFullImportName = UPackageTools::SanitizePackageName(AssetName);
				OriginalFullImportName = OriginalFullImportName + TEXT(".") + UPackageTools::SanitizePackageName(MaterialInfo->Name);
				MaterialInfo->SetOriginalFullImportName(OriginalFullImportName);
			}
			OutMaterials.Add(MaterialInfo);
		}
	}
}

void SFbxSceneMaterialsListView::FindMatchAndFillOverrideInformation(TArray<FbxMaterialInfoPtr> &OldMaterials, TArray<FbxMaterialInfoPtr> &NewMaterials)
{
	
	for (FbxMaterialInfoPtr NewMaterial : NewMaterials)
	{
		for (FbxMaterialInfoPtr OldMaterial : OldMaterials)
		{
			if( (OldMaterial->Name.Compare(NewMaterial->Name) == 0) &&
				OldMaterial->HierarchyPath.Compare(NewMaterial->HierarchyPath) == 0)
			{
				//Only copy the override attribute and the import flag
				NewMaterial->bImportAttribute = OldMaterial->bImportAttribute;
				NewMaterial->SetOverridePath(OldMaterial->bOverridePath);
				NewMaterial->OverrideImportPath = OldMaterial->OverrideImportPath;
				NewMaterial->OverrideFullImportName = OldMaterial->OverrideFullImportName;
			}
		}
	}
}

void SFbxSceneMaterialsListView::Construct(const SFbxSceneMaterialsListView::FArguments& InArgs)
{
	SceneInfo = InArgs._SceneInfo;
	SceneInfoOriginal = InArgs._SceneInfoOriginal;
	GlobalImportSettings = InArgs._GlobalImportSettings;
	TexturesArray = InArgs._TexturesArray;
	FullPath = InArgs._FullPath;
	IsReimport = InArgs._IsReimport;
	CreateContentFolderHierarchy = InArgs._CreateContentFolderHierarchy;

	check(SceneInfo.IsValid());
	check(GlobalImportSettings != nullptr);
	check(TexturesArray != nullptr);

	if (IsReimport)
	{
		check(SceneInfoOriginal.IsValid());
		//We reimport this we have probably some override material
		//First match the previous import material with the current import material
		TArray<FbxMaterialInfoPtr> OldMaterials;
		TArray<FbxMaterialInfoPtr> NewMaterials;
		GetMaterialsFromHierarchy(OldMaterials, SceneInfoOriginal, false);
		GetMaterialsFromHierarchy(NewMaterials, SceneInfo, true);
		FindMatchAndFillOverrideInformation(OldMaterials, NewMaterials);
		//Copy the new material array in the one use by the list view widget
		MaterialsArray = NewMaterials;
	}
	else
	{
		//Fill the original information and the Materials array use by the list view widget
		GetMaterialsFromHierarchy(MaterialsArray, SceneInfo, true);
	}

	SListView::Construct
	(
		SListView::FArguments()
		.ListItemsSource(&MaterialsArray)
		.SelectionMode(ESelectionMode::Multi)
		.OnGenerateRow(this, &SFbxSceneMaterialsListView::OnGenerateRowFbxSceneListView)
		.OnContextMenuOpening(this, &SFbxSceneMaterialsListView::OnOpenContextMenu)
		.OnSelectionChanged(this, &SFbxSceneMaterialsListView::OnSelectionChanged)
		.HeaderRow
		(
			SNew(SHeaderRow)
				
			+ SHeaderRow::Column(MaterialCheckBoxSelectionHeaderIdName)
			.FixedWidth(26)
			.DefaultLabel(FText::GetEmpty())
			[
				SNew(SCheckBox)
				.HAlign(HAlign_Center)
				.OnCheckStateChanged(this, &SFbxSceneMaterialsListView::OnToggleSelectAll)
			]
				
			+ SHeaderRow::Column(MaterialNameHeaderIdName)
			.FillWidth(300)
			.HAlignCell(EHorizontalAlignment::HAlign_Left)
			.DefaultLabel(LOCTEXT("AssetNameHeaderName", "Asset Name"))

			+ SHeaderRow::Column(MaterialContentPathHeaderIdName)
			.FillWidth(300)
			.HAlignCell(EHorizontalAlignment::HAlign_Left)
				.DefaultLabel(LOCTEXT("ContentPathHeaderName", "Content Path"))

			+ SHeaderRow::Column(MaterialStatusNameHeaderIdName)
			.FillWidth(60)
			.HAlignCell(EHorizontalAlignment::HAlign_Left)
			.DefaultLabel(LOCTEXT("StatusHeaderName", "Status"))
		)
	);
}

TSharedRef< ITableRow > SFbxSceneMaterialsListView::OnGenerateRowFbxSceneListView(FbxMaterialInfoPtr Item, const TSharedRef< STableViewBase >& OwnerTable)
{
	TSharedRef< SFbxMaterialItemTableListViewRow > ReturnRow = SNew(SFbxMaterialItemTableListViewRow, OwnerTable)
		.FbxMaterialInfo(Item);
	return ReturnRow;
}

void SFbxSceneMaterialsListView::UpdateMaterialBasePath()
{
	TArray<FbxMaterialInfoPtr> MaterialsUpdated;
	GetMaterialsFromHierarchy(MaterialsUpdated, SceneInfo, true);
}

TSharedPtr<SWidget> SFbxSceneMaterialsListView::OnOpenContextMenu()
{
	TArray<FbxMaterialInfoPtr> SelectedFbxMaterialInfos;
	int32 SelectCount = GetSelectedItems(SelectedFbxMaterialInfos);
	// Build up the menu for a selection
	const bool bCloseAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, TSharedPtr<FUICommandList>());

	// We always create a section here, even if there is no parent so that clients can still extend the menu
	MenuBuilder.BeginSection("FbxScene_MAT_ImportSection");
	{
		const FSlateIcon PlusIcon(FAppStyle::GetAppStyleSetName(), "Plus");
		MenuBuilder.AddMenuEntry(LOCTEXT("CheckForImport", "Add Selection To Import"), FText(), PlusIcon, FUIAction(FExecuteAction::CreateSP(this, &SFbxSceneMaterialsListView::AddSelectionToImport)));
		const FSlateIcon MinusIcon(FAppStyle::GetAppStyleSetName(), "Icons.Minus");
		MenuBuilder.AddMenuEntry(LOCTEXT("UncheckForImport", "Remove Selection From Import"), FText(), MinusIcon, FUIAction(FExecuteAction::CreateSP(this, &SFbxSceneMaterialsListView::RemoveSelectionFromImport)));
	}
	MenuBuilder.EndSection();
	MenuBuilder.BeginSection("FbxScene_MAT_AssignSection");
	{
		if (SelectCount == 1)
		{
			MenuBuilder.AddMenuEntry(LOCTEXT("AssignExistingMaterial", "Assign Existing Material..."), FText(), FSlateIcon(), FUIAction(FExecuteAction::CreateSP(this, &SFbxSceneMaterialsListView::AssignMaterialToExisting)));
			MenuBuilder.AddMenuEntry(LOCTEXT("ResetAssignExistingMaterial", "Reset Material to Fbx content"), FText(), FSlateIcon(), FUIAction(FExecuteAction::CreateSP(this, &SFbxSceneMaterialsListView::ResetAssignMaterial)));
			MenuBuilder.AddMenuSeparator();
			
			FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
			// Configure filter for asset picker
			FAssetPickerConfig Config;
			Config.Filter.bRecursiveClasses = true;
			Config.Filter.ClassPaths.Add(UMaterialInterface::StaticClass()->GetClassPathName());
			Config.InitialAssetViewType = EAssetViewType::List;
			Config.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SFbxSceneMaterialsListView::AssignMaterialAssetData);
			Config.bAllowNullSelection = false;
			Config.bFocusSearchBoxWhenOpened = true;
			Config.bAllowDragging = false;
			Config.Filter.bRecursivePaths = true;
			Config.bForceShowEngineContent = false;
			Config.bForceShowPluginContent = false;

			//We have to limit the height in case there is a lot of item, the computed height can overpass the 16 bit integer maximum.
			TSharedRef<SWidget> Widget =
				SNew(SBox)
				.WidthOverride(300.0f)
				.HeightOverride(300.0f)
				[
					ContentBrowserModule.Get().CreateAssetPicker(Config)
				];
			MenuBuilder.AddWidget(Widget, FText::GetEmpty());
		}
	}
	MenuBuilder.EndSection();
	return MenuBuilder.MakeWidget();
}

void SFbxSceneMaterialsListView::AddSelectionToImport()
{
	SetSelectionImportState(true);
}

void SFbxSceneMaterialsListView::RemoveSelectionFromImport()
{
	SetSelectionImportState(false);
}

void SFbxSceneMaterialsListView::SetSelectionImportState(bool MarkForImport)
{
	TArray<FbxMaterialInfoPtr> SelectedFbxMaterialInfos;
	GetSelectedItems(SelectedFbxMaterialInfos);
	for (FbxMaterialInfoPtr Item : SelectedFbxMaterialInfos)
	{
		Item->bImportAttribute = MarkForImport;
	}
}

void SFbxSceneMaterialsListView::OnSelectionChanged(FbxMaterialInfoPtr Item, ESelectInfo::Type SelectionType)
{
	//Change the texture list
	TArray<FbxMaterialInfoPtr> SelectedFbxMaterialInfos;
	GetSelectedItems(SelectedFbxMaterialInfos);
	TexturesArray->Reset();
	for (FbxMaterialInfoPtr SelectItem : SelectedFbxMaterialInfos)
	{
		TexturesArray->Append(SelectItem->Textures);
	}
}

void SFbxSceneMaterialsListView::OnToggleSelectAll(ECheckBoxState CheckType)
{
	for (FbxMaterialInfoPtr MaterialInfo : MaterialsArray)
	{
		MaterialInfo->bImportAttribute = CheckType == ECheckBoxState::Checked;
	}
}

void SFbxSceneMaterialsListView::AssignMaterialAssetData(const FAssetData& AssetData)
{
	TArray<FbxMaterialInfoPtr> SelectedFbxMaterialInfos;
	int32 SelectCount = GetSelectedItems(SelectedFbxMaterialInfos);
	if (SelectCount != 1)
	{
		FSlateApplication::Get().DismissAllMenus();
		return;
	}

	UPackage *ContentPackage = AssetData.GetPackage();
	UObject *ContentObject = AssetData.GetAsset();
	if (ContentObject != nullptr)
	{
		if (!ContentObject->HasAnyFlags(RF_Transient) && IsValid(ContentObject))
		{
			for (FbxMaterialInfoPtr ItemPtr : SelectedFbxMaterialInfos)
			{
				//Override the MeshInfo with the new asset path
				ItemPtr->SetOverridePath(true);
				ItemPtr->OverrideImportPath = AssetData.PackageName.ToString();
				ItemPtr->OverrideFullImportName = AssetData.GetObjectPathString();
			}
		}
	}
	FSlateApplication::Get().DismissAllMenus();
}

void SFbxSceneMaterialsListView::AssignMaterialToExisting()
{
	TArray<FbxMaterialInfoPtr> SelectedFbxMaterialInfos;
	int32 SelectCount = GetSelectedItems(SelectedFbxMaterialInfos);
	if (SelectCount != 1)
		return;
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	for (FbxMaterialInfoPtr ItemPtr : SelectedFbxMaterialInfos)
	{
		FOpenAssetDialogConfig SelectAssetConfig;
		SelectAssetConfig.DialogTitleOverride = LOCTEXT("FbxChooseMaterialAssetContentPath", "Choose a material asset");
		//SelectAssetConfig.DefaultPath = ItemPtr->OriginalImportPath;
		SelectAssetConfig.bAllowMultipleSelection = false;
		SelectAssetConfig.AssetClassNames.Add(UMaterial::StaticClass()->GetClassPathName());
		TArray<FAssetData> AssetData = ContentBrowserModule.Get().CreateModalOpenAssetDialog(SelectAssetConfig);
		if (AssetData.Num() == 1)
		{
			UPackage *ContentPackage = AssetData[0].GetPackage();
			UObject *ContentObject = AssetData[0].GetAsset();
			if (ContentObject != nullptr)
			{
				if (!ContentObject->HasAnyFlags(RF_Transient) && IsValid(ContentObject))
				{
					//Override the MeshInfo with the new asset path
					ItemPtr->SetOverridePath(true);
					ItemPtr->OverrideImportPath = AssetData[0].PackageName.ToString();
					ItemPtr->OverrideFullImportName = AssetData[0].GetObjectPathString();
				}
			}
		}
	}
}

void SFbxSceneMaterialsListView::ResetAssignMaterial()
{
	TArray<FbxMaterialInfoPtr> SelectedFbxMaterialInfos;
	int32 SelectCount = GetSelectedItems(SelectedFbxMaterialInfos);

	for (FbxMaterialInfoPtr ItemPtr : SelectedFbxMaterialInfos)
	{
		if (ItemPtr->bOverridePath)
		{
			ItemPtr->SetOverridePath(false);
			ItemPtr->OverrideImportPath.Empty();
			ItemPtr->OverrideFullImportName.Empty();
		}
	}
}

#undef LOCTEXT_NAMESPACE
