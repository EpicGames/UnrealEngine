// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Factories/FbxSceneImportFactory.h"
#include "Widgets/Views/SListView.h"

struct FAssetData;

typedef TSharedPtr<FFbxMaterialInfo> FbxMaterialInfoPtr;
typedef TSharedPtr<FFbxTextureInfo> FbxTextureInfoPtr;
typedef TArray<FbxTextureInfoPtr> FbxTextureInfoArray;

class SFbxSceneMaterialsListView : public SListView<FbxMaterialInfoPtr>
{
public:
	
	~SFbxSceneMaterialsListView();

	SLATE_BEGIN_ARGS(SFbxSceneMaterialsListView)
		: _SceneInfo(nullptr)
		, _SceneInfoOriginal(nullptr)
		, _GlobalImportSettings(nullptr)
		, _TexturesArray(nullptr)
		, _FullPath(TEXT(""))
		, _IsReimport(false)
		, _CreateContentFolderHierarchy(false)
	{
		_Clipping = EWidgetClipping::ClipToBounds;
	}
		SLATE_ARGUMENT(TSharedPtr<FFbxSceneInfo>, SceneInfo)
		SLATE_ARGUMENT(TSharedPtr<FFbxSceneInfo>, SceneInfoOriginal)
		SLATE_ARGUMENT(UnFbx::FBXImportOptions*, GlobalImportSettings)
		SLATE_ARGUMENT(FbxTextureInfoArray *, TexturesArray)
		SLATE_ARGUMENT(FString, FullPath)
		SLATE_ARGUMENT(bool, IsReimport)
		SLATE_ARGUMENT(bool, CreateContentFolderHierarchy)
	SLATE_END_ARGS()
	
	/** Construct this widget */
	void Construct(const FArguments& InArgs);
	TSharedRef< ITableRow > OnGenerateRowFbxSceneListView(FbxMaterialInfoPtr Item, const TSharedRef< STableViewBase >& OwnerTable);
	void OnToggleSelectAll(ECheckBoxState CheckType);

	void SetCreateContentFolderHierarchy(bool CreateFolder) { CreateContentFolderHierarchy = CreateFolder; }
	void UpdateMaterialBasePath();
protected:
	FString FullPath;
	bool IsReimport;
	bool CreateContentFolderHierarchy;

	TSharedPtr<FFbxSceneInfo> SceneInfo;
	TSharedPtr<FFbxSceneInfo> SceneInfoOriginal;
	UnFbx::FBXImportOptions* GlobalImportSettings;

	/** the elements we show in the tree view */
	TArray<FbxMaterialInfoPtr> MaterialsArray;
	FbxTextureInfoArray *TexturesArray;

	/** Open a context menu for the current selection */
	TSharedPtr<SWidget> OnOpenContextMenu();
	void AddSelectionToImport();
	void RemoveSelectionFromImport();
	void SetSelectionImportState(bool MarkForImport);
	void OnSelectionChanged(FbxMaterialInfoPtr Item, ESelectInfo::Type SelectionType);

	void AssignMaterialToExisting();
	void AssignMaterialAssetData(const FAssetData& AssetData);
	void ResetAssignMaterial();

	void GetMaterialsFromHierarchy(TArray<FbxMaterialInfoPtr> &OutMaterials, TSharedPtr<FFbxSceneInfo> SceneInfoSource, bool bFillPathInformation);
	void FindMatchAndFillOverrideInformation(TArray<FbxMaterialInfoPtr> &OldMaterials, TArray<FbxMaterialInfoPtr> &NewMaterials);
};
