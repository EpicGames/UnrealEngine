// Copyright Epic Games, Inc. All Rights Reserved.

#include "Fbx/SSceneReimportNodeTreeView.h"

#include "Camera/CameraComponent.h"
#include "Components/DirectionalLightComponent.h"
#include "Components/LightComponent.h"
#include "Components/PointLightComponent.h"
#include "Components/SpotLightComponent.h"
#include "Containers/UnrealString.h"
#include "Factories/FbxSceneImportData.h"
#include "Factories/FbxSceneImportFactory.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Views/ITypedTableView.h"
#include "GameFramework/Actor.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Visibility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateIconFinder.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Views/STableRow.h"

class FUICommandList;
class ITableRow;
class SWidget;
class UClass;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "SFbxReimportSceneTreeView"

SFbxReimportSceneTreeView::~SFbxReimportSceneTreeView()
{
	FbxRootNodeArray.Empty();
	SceneInfo = nullptr;
	SceneInfoOriginal = nullptr;
	NodeStatusMap = nullptr;
	
	//Clear the internal data it is all SharedPtr
	NodeTreeData.Empty();
}

void SFbxReimportSceneTreeView::FillNodeStatusMapInternal(FbxSceneReimportStatusMapPtr NodeStatusMap
							, TSharedPtr<FFbxSceneInfo> SceneInfo
							, TSharedPtr<FFbxSceneInfo> SceneInfoOriginal
							, TMap<FbxNodeInfoPtr, TSharedPtr<FTreeNodeValue>>* NodeTreeDataPtr
							, TArray<FbxNodeInfoPtr>* FbxRootNodeArrayPtr)
{
	auto AddToNodeTreeDataPtr = [&NodeTreeDataPtr](FbxNodeInfoPtr NodeInfo, TSharedPtr<FTreeNodeValue> NodeValue)
	{
		if (NodeTreeDataPtr)
		{
			NodeTreeDataPtr->Add(NodeInfo, NodeValue);
		}
	};

	auto AddToFbxRootNodeArrayPtr = [&FbxRootNodeArrayPtr](FbxNodeInfoPtr CurrentNode)
	{
		if (FbxRootNodeArrayPtr)
		{
			FbxRootNodeArrayPtr->Add(CurrentNode);
		}
	};

	TMap<FString, TSharedPtr<FTreeNodeValue>> NodeTreePath;
	//Build hierarchy path
	for (FbxNodeInfoPtr NodeInfo : SceneInfo->HierarchyInfo)
	{
		TSharedPtr<FTreeNodeValue> NodeValue = MakeShareable(new FTreeNodeValue());
		NodeValue->CurrentNode = NodeInfo;
		AddToNodeTreeDataPtr(NodeInfo, NodeValue);
		NodeValue->OriginalNode = nullptr;
		NodeTreePath.Add(NodeInfo->NodeHierarchyPath, NodeValue);
	}
	//Build hierarchy path for original fbx
	for (FbxNodeInfoPtr NodeInfo : SceneInfoOriginal->HierarchyInfo)
	{
		TSharedPtr<FTreeNodeValue> NodeValue = nullptr;
		if (NodeTreePath.Contains(NodeInfo->NodeHierarchyPath))
		{
			NodeValue = *(NodeTreePath.Find(NodeInfo->NodeHierarchyPath));
		}
		else
		{
			NodeValue = MakeShareable(new FTreeNodeValue());
			NodeValue->CurrentNode = nullptr;
			NodeTreePath.Add(NodeInfo->NodeHierarchyPath, NodeValue);
		}
		NodeValue->OriginalNode = NodeInfo;
		AddToNodeTreeDataPtr(NodeInfo, NodeValue);
	}

	//build the status array
	TArray<TSharedPtr<FTreeNodeValue>> NodeTreeValues;
	NodeTreePath.GenerateValueArray(NodeTreeValues);
	for (TSharedPtr<FTreeNodeValue> NodeValue : NodeTreeValues)
	{
		EFbxSceneReimportStatusFlags NodeStatus = EFbxSceneReimportStatusFlags::None;
		if (NodeValue->CurrentNode.IsValid())
		{
			if (!NodeValue->CurrentNode->ParentNodeInfo.IsValid())
			{
				//Add root node to the UI tree view
				AddToFbxRootNodeArrayPtr(NodeValue->CurrentNode);
			}
			if (NodeValue->OriginalNode.IsValid())
			{
				NodeStatus |= EFbxSceneReimportStatusFlags::Same;
				if (NodeValue->OriginalNode->bImportNode)
				{
					NodeStatus |= EFbxSceneReimportStatusFlags::ReimportAsset;
				}
			}
			else
			{
				//by default reimport new node user can uncheck them
				NodeStatus |= EFbxSceneReimportStatusFlags::Added | EFbxSceneReimportStatusFlags::ReimportAsset;
			}
			NodeStatusMap->Add(NodeValue->CurrentNode->NodeHierarchyPath, NodeStatus);
		}
		else if (NodeValue->OriginalNode.IsValid())
		{
			if (!NodeValue->OriginalNode->ParentNodeInfo.IsValid())
			{
				//Add root node to the UI tree view
				AddToFbxRootNodeArrayPtr(NodeValue->OriginalNode);
			}
			NodeStatus |= EFbxSceneReimportStatusFlags::Removed;
			if (NodeValue->OriginalNode->bImportNode)
			{
				NodeStatus |= EFbxSceneReimportStatusFlags::ReimportAsset;
			}
			NodeStatusMap->Add(NodeValue->OriginalNode->NodeHierarchyPath, NodeStatus);
		}

	}
	NodeTreePath.Empty();
}

void SFbxReimportSceneTreeView::FillNodeStatusMap(FbxSceneReimportStatusMapPtr NodeStatusMap, TSharedPtr<FFbxSceneInfo> SceneInfo, TSharedPtr<FFbxSceneInfo> SceneInfoOriginal)
{
	FillNodeStatusMapInternal(NodeStatusMap, SceneInfo, SceneInfoOriginal, nullptr, nullptr);
}

void SFbxReimportSceneTreeView::Construct(const SFbxReimportSceneTreeView::FArguments& InArgs)
{
	SceneInfo = InArgs._SceneInfo;
	SceneInfoOriginal = InArgs._SceneInfoOriginal;
	NodeStatusMap = InArgs._NodeStatusMap;
	//Build the FbxNodeInfoPtr tree data
	check(SceneInfo.IsValid());
	check(SceneInfoOriginal.IsValid());
	check(NodeStatusMap != nullptr);

	FillNodeStatusMapInternal(NodeStatusMap, SceneInfo, SceneInfoOriginal, &NodeTreeData, &FbxRootNodeArray);

	STreeView::Construct
	(
		STreeView::FArguments()
		.TreeItemsSource(&FbxRootNodeArray)
		.SelectionMode(ESelectionMode::Multi)
		.OnGenerateRow(this, &SFbxReimportSceneTreeView::OnGenerateRowFbxSceneTreeView)
		.OnGetChildren(this, &SFbxReimportSceneTreeView::OnGetChildrenFbxSceneTreeView)
		.OnContextMenuOpening(this, &SFbxReimportSceneTreeView::OnOpenContextMenu)
		.OnSelectionChanged(this, &SFbxReimportSceneTreeView::OnSelectionChanged)
	);
}

/** The item used for visualizing the class in the tree. */
class SFbxReimportSceneTreeViewItem : public STableRow< FbxNodeInfoPtr >
{
public:

	SLATE_BEGIN_ARGS(SFbxReimportSceneTreeViewItem)
		: _FbxNodeInfo(nullptr)
		, _NodeStatusMap(nullptr)
		, _SceneInfo(nullptr)
	{}

	/** The item content. */
	SLATE_ARGUMENT(FbxNodeInfoPtr, FbxNodeInfo)
	SLATE_ARGUMENT(FbxSceneReimportStatusMapPtr, NodeStatusMap)
	SLATE_ARGUMENT(TSharedPtr<FFbxSceneInfo>, SceneInfo)
	SLATE_END_ARGS()

	/**
	* Construct the widget
	*
	* @param InArgs   A declaration from which to construct the widget
	*/
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		FbxNodeInfo = InArgs._FbxNodeInfo;
		NodeStatusMap = InArgs._NodeStatusMap;
		SceneInfo = InArgs._SceneInfo;
		
		//This is suppose to always be valid
		check(FbxNodeInfo.IsValid());
		check(NodeStatusMap != nullptr);
		check(SceneInfo.IsValid());

		UClass *IconClass = AActor::StaticClass();
		if (FbxNodeInfo->AttributeInfo.IsValid())
		{
			if(!FbxNodeInfo->AttributeInfo->bOriginalTypeChanged)
				IconClass = FbxNodeInfo->AttributeInfo->GetType();
		}
		else if (FbxNodeInfo->AttributeType.Compare(TEXT("eLight")) == 0)
		{
			IconClass = ULightComponent::StaticClass();
			if (SceneInfo->LightInfo.Contains(FbxNodeInfo->AttributeUniqueId))
			{
				TSharedPtr<FFbxLightInfo> LightInfo = *SceneInfo->LightInfo.Find(FbxNodeInfo->AttributeUniqueId);
				if (LightInfo->Type == 0)
				{
					IconClass = UPointLightComponent::StaticClass();
				}
				else if (LightInfo->Type == 1)
				{
					IconClass = UDirectionalLightComponent::StaticClass();
				}
				else if (LightInfo->Type == 2)
				{
					IconClass = USpotLightComponent::StaticClass();
				}
			}
		}
		else if (FbxNodeInfo->AttributeType.Compare(TEXT("eCamera")) == 0)
		{
			IconClass = UCameraComponent::StaticClass();
		}
		const FSlateBrush* ClassIcon = FSlateIconFinder::FindIconBrushForClass(IconClass);

		this->ChildSlot
		[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
			.Padding(2.0f, 0.0f, 2.0f, 0.0f)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.OnCheckStateChanged(this, &SFbxReimportSceneTreeViewItem::OnItemCheckChanged)
				.IsChecked(this, &SFbxReimportSceneTreeViewItem::IsItemChecked)
			]
		+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SExpanderArrow, SharedThis(this))
			]

		+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 2.0f, 6.0f, 2.0f)
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(ClassIcon)
					.Visibility(ClassIcon != FAppStyle::GetDefaultBrush() ? EVisibility::Visible : EVisibility::Collapsed)
				]
				+ SOverlay::Slot()
				.HAlign(HAlign_Left)
				[
					SNew(SImage)
					.Image(this, &SFbxReimportSceneTreeViewItem::GetIconOverlay)
				]
			]
		+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(0.0f, 3.0f, 6.0f, 3.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::FromString(FbxNodeInfo->NodeName))
				.ToolTipText(this, &SFbxReimportSceneTreeViewItem::GetTooltip)
			]

		];

		STableRow< FbxNodeInfoPtr >::ConstructInternal(
			STableRow::FArguments()
			.ShowSelection(true),
			InOwnerTableView
		);
	}

private:
	FText GetTooltip() const
	{
		FString TooltipText = FbxNodeInfo->NodeName;
		if (NodeStatusMap->Contains(FbxNodeInfo->NodeHierarchyPath))
		{
			EFbxSceneReimportStatusFlags ReimportFlags = *NodeStatusMap->Find(FbxNodeInfo->NodeHierarchyPath);
			//The remove only should not be possible this is why there is no remove only case
			if(FbxNodeInfo->AttributeInfo.IsValid() && FbxNodeInfo->AttributeInfo->bOriginalTypeChanged)
			{
				TooltipText += LOCTEXT("SFbxReimportSceneTreeViewItem_TypeChangedTooltip", " type has changed, only the transform can be reimport.").ToString();
			}
			else if ((ReimportFlags & EFbxSceneReimportStatusFlags::Added) != EFbxSceneReimportStatusFlags::None)
			{
				TooltipText += LOCTEXT("SFbxReimportSceneTreeViewItem_AddedTooltip", " Will be add to the blueprint hierarchy.").ToString();
			}
			else if ((ReimportFlags & EFbxSceneReimportStatusFlags::Same) != EFbxSceneReimportStatusFlags::None)
			{
				//No tooltip because the item will not change
			}
			else if ((ReimportFlags & EFbxSceneReimportStatusFlags::Removed) != EFbxSceneReimportStatusFlags::None)
			{
				TooltipText += LOCTEXT("SFbxReimportSceneTreeViewItem_RemovedTooltip", " Will be remove from the blueprint hierarchy.").ToString();
			}
		}
		return FText::FromString(TooltipText);
	}
	const FSlateBrush *GetIconOverlay() const
	{
		const FSlateBrush *SlateBrush = FAppStyle::GetBrush("FBXIcon.ReimportError");
		if (NodeStatusMap->Contains(FbxNodeInfo->NodeHierarchyPath))
		{
			EFbxSceneReimportStatusFlags ReimportFlags = *NodeStatusMap->Find(FbxNodeInfo->NodeHierarchyPath);
			//The remove only should not be possible this is why there is no remove only case

			if ((ReimportFlags & EFbxSceneReimportStatusFlags::Added) != EFbxSceneReimportStatusFlags::None)
			{
				SlateBrush = FAppStyle::GetBrush("FBXIcon.ReimportAdded");
			}
			else if ((ReimportFlags & EFbxSceneReimportStatusFlags::Same) != EFbxSceneReimportStatusFlags::None)
			{
				SlateBrush = FAppStyle::GetBrush("FBXIcon.ReimportSameContent");
			}
			else if ((ReimportFlags & EFbxSceneReimportStatusFlags::Removed) != EFbxSceneReimportStatusFlags::None)
			{
				SlateBrush = FAppStyle::GetBrush("FBXIcon.ReimportRemovedContent");
			}
		}
		return SlateBrush;
	}

	void RecursivelySetLODMeshImportState(TSharedPtr<FFbxNodeInfo> NodeInfo, bool bState)
	{
		//Set the bImportNode property for all mesh under eLODGroup
		for (TSharedPtr<FFbxNodeInfo> ChildNodeInfo : NodeInfo->Childrens)
		{
			if (!ChildNodeInfo.IsValid())
				continue;
			if (ChildNodeInfo->AttributeType.Compare(TEXT("eMesh")) == 0)
			{
				EFbxSceneReimportStatusFlags *StatusFlag = NodeStatusMap->Find(ChildNodeInfo->NodeHierarchyPath);
				if (bState)
				{
					*StatusFlag = *StatusFlag | EFbxSceneReimportStatusFlags::ReimportAsset;
				}
				else
				{
					*StatusFlag = *StatusFlag & ~EFbxSceneReimportStatusFlags::ReimportAsset;
				}
			}
			else
			{
				RecursivelySetLODMeshImportState(ChildNodeInfo, bState);
			}
		}
	}

	void OnItemCheckChanged(ECheckBoxState CheckType)
	{
		if (!FbxNodeInfo.IsValid() || !NodeStatusMap->Contains(FbxNodeInfo->NodeHierarchyPath))
			return;
		bool bNewState = CheckType == ECheckBoxState::Checked;
		EFbxSceneReimportStatusFlags *StatusFlag = NodeStatusMap->Find(FbxNodeInfo->NodeHierarchyPath);
		if (bNewState)
		{
			*StatusFlag = *StatusFlag | EFbxSceneReimportStatusFlags::ReimportAsset;
		}
		else
		{
			*StatusFlag = *StatusFlag & ~EFbxSceneReimportStatusFlags::ReimportAsset;
		}

		if (FbxNodeInfo->AttributeType.Compare(TEXT("eLODGroup")) == 0)
		{
			RecursivelySetLODMeshImportState(FbxNodeInfo, bNewState);
		}
		if (FbxNodeInfo->AttributeType.Compare(TEXT("eMesh")) == 0)
		{
			//Verify if parent is a LOD group
			TSharedPtr<FFbxNodeInfo> ParentLODNodeInfo = FFbxSceneInfo::RecursiveFindLODParentNode(FbxNodeInfo);
			if (ParentLODNodeInfo.IsValid())
			{
				EFbxSceneReimportStatusFlags *ParentStatusFlag = NodeStatusMap->Find(ParentLODNodeInfo->NodeHierarchyPath);
				if (bNewState)
				{
					*ParentStatusFlag = *ParentStatusFlag | EFbxSceneReimportStatusFlags::ReimportAsset;
				}
				else
				{
					*ParentStatusFlag = *ParentStatusFlag & ~EFbxSceneReimportStatusFlags::ReimportAsset;
				}
				RecursivelySetLODMeshImportState(ParentLODNodeInfo, bNewState);
			}
		}
	}

	ECheckBoxState IsItemChecked() const
	{
		if (NodeStatusMap->Contains(FbxNodeInfo->NodeHierarchyPath))
		{
			return (*(NodeStatusMap->Find(FbxNodeInfo->NodeHierarchyPath)) & EFbxSceneReimportStatusFlags::ReimportAsset) != EFbxSceneReimportStatusFlags::None ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
		return ECheckBoxState::Unchecked;
	}

	/** The node info to build the tree view row from. */
	FbxNodeInfoPtr FbxNodeInfo;

	FbxSceneReimportStatusMapPtr NodeStatusMap;
	TSharedPtr<FFbxSceneInfo> SceneInfo;
};

TSharedRef< ITableRow > SFbxReimportSceneTreeView::OnGenerateRowFbxSceneTreeView(FbxNodeInfoPtr Item, const TSharedRef< STableViewBase >& OwnerTable)
{
	TSharedRef< SFbxReimportSceneTreeViewItem > ReturnRow = SNew(SFbxReimportSceneTreeViewItem, OwnerTable)
		.FbxNodeInfo(Item)
		.NodeStatusMap(NodeStatusMap)
		.SceneInfo(SceneInfo);
	return ReturnRow;
}

void SFbxReimportSceneTreeView::OnGetChildrenFbxSceneTreeView(FbxNodeInfoPtr InParent, TArray< FbxNodeInfoPtr >& OutChildren)
{
	check(NodeTreeData.Contains(InParent));
	TSharedPtr<FTreeNodeValue> NodeValue = *(NodeTreeData.Find(InParent));
	TArray<FString> ChildNames;
	TArray<FString> ChildProcess;
	//Current node contain the add and same node
	if (NodeValue->CurrentNode.IsValid())
	{
		if (NodeValue->CurrentNode->AttributeType.Compare(TEXT("eLODGroup")) == 0 && NodeValue->CurrentNode->Childrens.Num() > 0)
		{
			TSharedPtr<FFbxNodeInfo> Child = NodeValue->CurrentNode;
			do
			{
				Child = Child->Childrens[0];
				if (Child.IsValid() && Child->AttributeType.Compare(TEXT("eMesh")) == 0)
				{
					ChildNames.Add(Child->NodeName);
					ChildProcess.Add(Child->NodeName);
					OutChildren.Add(Child);
					return;
				}
			} while (Child->Childrens.Num() > 0);
			//We do not found any LOD mesh don't add any child
		}
		else
		{
			for (FbxNodeInfoPtr Child : NodeValue->CurrentNode->Childrens)
			{
				if (Child.IsValid() && (Child->AttributeType.Compare(TEXT("eMesh")) != 0 || Child->AttributeInfo.IsValid()))
				{
					ChildNames.Add(Child->NodeName);
					ChildProcess.Add(Child->NodeName);
					OutChildren.Add(Child);
				}
				else if(Child.IsValid() && Child->AttributeType.Compare(TEXT("eMesh")) == 0)
				{
					ChildProcess.Add(Child->NodeName);
				}
			}
		}
	}

	//Original node is use for finding the remove nodes
	if (NodeValue->OriginalNode.IsValid())
	{
		if (NodeValue->OriginalNode->AttributeType.Compare(TEXT("eLODGroup")) == 0 && NodeValue->OriginalNode->Childrens.Num() > 0)
		{
			TSharedPtr<FFbxNodeInfo> Child = NodeValue->OriginalNode;
			do
			{
				Child = Child->Childrens[0];
				if (Child.IsValid() && Child->AttributeType.Compare(TEXT("eMesh")) == 0)
				{
					if (!ChildProcess.Contains(Child->NodeName))
					{
						OutChildren.Add(Child);
					}
					return;
				}
			} while (Child->Childrens.Num() > 0);
			//We do not found any LOD mesh don't add any child
		}
		else
		{
			for (FbxNodeInfoPtr Child : NodeValue->OriginalNode->Childrens)
			{
				//We have a delete node
				if (Child.IsValid() && !ChildProcess.Contains(Child->NodeName))
				{
					OutChildren.Add(Child);
				}
			}
		}
	}
}

void SFbxReimportSceneTreeView::OnToggleSelectAll(ECheckBoxState CheckType)
{
	//check all actor for import
	TArray<TSharedPtr<FTreeNodeValue>> NodeValues;
	NodeTreeData.GenerateValueArray(NodeValues);
	for (TSharedPtr<FTreeNodeValue> NodeValue: NodeValues)
	{
		FbxNodeInfoPtr NodeInfo;
		if (NodeValue->CurrentNode.IsValid())
		{
			NodeInfo = NodeValue->CurrentNode;
		}
		else
		{
			NodeInfo = NodeValue->OriginalNode;
		}
		
		//We should never see no current and no original node
		check(NodeInfo.IsValid());

		EFbxSceneReimportStatusFlags *ItemStatus = NodeStatusMap->Find(NodeInfo->NodeHierarchyPath);
		if (ItemStatus != nullptr)
		{
			*ItemStatus = (CheckType == ECheckBoxState::Checked) ? *ItemStatus | EFbxSceneReimportStatusFlags::ReimportAsset : *ItemStatus & ~EFbxSceneReimportStatusFlags::ReimportAsset;
		}
	}
}

void RecursiveSetExpand(SFbxReimportSceneTreeView *TreeView, FbxNodeInfoPtr NodeInfoPtr, bool ExpandState)
{
	TreeView->SetItemExpansion(NodeInfoPtr, ExpandState);
	for (auto Child : NodeInfoPtr->Childrens)
	{
		RecursiveSetExpand(TreeView, Child, ExpandState);
	}
}

FReply SFbxReimportSceneTreeView::OnExpandAll()
{
	for (auto NodeInfoIt = SceneInfo->HierarchyInfo.CreateIterator(); NodeInfoIt; ++NodeInfoIt)
	{
		FbxNodeInfoPtr NodeInfo = (*NodeInfoIt);

		if (!NodeInfo->ParentNodeInfo.IsValid())
		{
			RecursiveSetExpand(this, NodeInfo, true);
		}
	}
	return FReply::Handled();
}

FReply SFbxReimportSceneTreeView::OnCollapseAll()
{
	for (auto NodeInfoIt = SceneInfo->HierarchyInfo.CreateIterator(); NodeInfoIt; ++NodeInfoIt)
	{
		FbxNodeInfoPtr NodeInfo = (*NodeInfoIt);

		if (!NodeInfo->ParentNodeInfo.IsValid())
		{
			RecursiveSetExpand(this, NodeInfo, false);
		}
	}
	return FReply::Handled();
}

TSharedPtr<SWidget> SFbxReimportSceneTreeView::OnOpenContextMenu()
{
	// Build up the menu for a selection
	const bool bCloseAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, TSharedPtr<FUICommandList>());

	//Get the different type of the multi selection
	TSet<TSharedPtr<FFbxAttributeInfo>> SelectAssets;
	TArray<FbxNodeInfoPtr> SelectedFbxNodeInfos;
	const auto NumSelectedItems = GetSelectedItems(SelectedFbxNodeInfos);
	for (auto Item : SelectedFbxNodeInfos)
	{
		FbxNodeInfoPtr ItemPtr = Item;
		if (ItemPtr->AttributeInfo.IsValid())
		{
			SelectAssets.Add(ItemPtr->AttributeInfo);
		}
	}

	// We always create a section here, even if there is no parent so that clients can still extend the menu
	MenuBuilder.BeginSection("FbxSceneTreeViewContextMenuImportSection");
	{
		const FSlateIcon PlusIcon(FAppStyle::GetAppStyleSetName(), "Plus");
		MenuBuilder.AddMenuEntry(LOCTEXT("CheckForImport", "Add Selection To Import"), FText(), PlusIcon, FUIAction(FExecuteAction::CreateSP(this, &SFbxReimportSceneTreeView::AddSelectionToImport)));
		const FSlateIcon MinusIcon(FAppStyle::GetAppStyleSetName(), "Icons.Minus");
		MenuBuilder.AddMenuEntry(LOCTEXT("UncheckForImport", "Remove Selection From Import"), FText(), MinusIcon, FUIAction(FExecuteAction::CreateSP(this, &SFbxReimportSceneTreeView::RemoveSelectionFromImport)));
	}
	MenuBuilder.EndSection();
/*
	if (SelectAssets.Num() > 0)
	{
		MenuBuilder.AddMenuSeparator();
		MenuBuilder.BeginSection("FbxSceneTreeViewContextMenuGotoAssetSection", LOCTEXT("Gotoasset","Go to asset:"));
		{
			for (auto AssetItem : SelectAssets)
			{
				//const FSlateBrush* ClassIcon = FClassIconFinder::FindIconForClass(AssetItem->GetType());
				MenuBuilder.AddMenuEntry(FText::FromString(AssetItem->Name), FText(), FSlateIcon(), FUIAction(FExecuteAction::CreateSP(this, &SFbxReimportSceneTreeView::GotoAsset, AssetItem)));
			}
		}
		MenuBuilder.EndSection();
	}
*/
	

	return MenuBuilder.MakeWidget();
}

void SFbxReimportSceneTreeView::AddSelectionToImport()
{
	SetSelectionImportState(true);
}

void SFbxReimportSceneTreeView::RemoveSelectionFromImport()
{
	SetSelectionImportState(false);
}

void SFbxReimportSceneTreeView::SetSelectionImportState(bool MarkForImport)
{
	TArray<FbxNodeInfoPtr> SelectedFbxNodeInfos;
	GetSelectedItems(SelectedFbxNodeInfos);
	for (FbxNodeInfoPtr Item : SelectedFbxNodeInfos)
	{
		EFbxSceneReimportStatusFlags *ItemStatus = NodeStatusMap->Find(Item->NodeHierarchyPath);
		if (ItemStatus != nullptr)
		{
			*ItemStatus = MarkForImport ? *ItemStatus | EFbxSceneReimportStatusFlags::ReimportAsset : *ItemStatus & ~EFbxSceneReimportStatusFlags::ReimportAsset;
		}
	}
}

void SFbxReimportSceneTreeView::OnSelectionChanged(FbxNodeInfoPtr Item, ESelectInfo::Type SelectionType)
{
}

void SFbxReimportSceneTreeView::GotoAsset(TSharedPtr<FFbxAttributeInfo> AssetAttribute)
{
	//Switch to the asset tab and select the asset
}

#undef LOCTEXT_NAMESPACE
