// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Delegates/Delegate.h"
#include "Fbx/SSceneBaseMeshListView.h"
#include "Fbx/SSceneImportNodeTreeView.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Misc/Optional.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

class FFbxAttributeInfo;
class FFbxSceneInfo;
class ITableRow;
class SWidget;

struct FTreeNodeValue
{
public:
	FbxNodeInfoPtr CurrentNode;
	FbxNodeInfoPtr OriginalNode;
};

class SFbxReimportSceneTreeView : public STreeView<FbxNodeInfoPtr>
{
public:
	~SFbxReimportSceneTreeView();
	SLATE_BEGIN_ARGS(SFbxReimportSceneTreeView)
	: _SceneInfo(nullptr)
	, _SceneInfoOriginal(nullptr)
	, _NodeStatusMap(nullptr)
	{}
		SLATE_ARGUMENT(TSharedPtr<FFbxSceneInfo>, SceneInfo)
		SLATE_ARGUMENT(TSharedPtr<FFbxSceneInfo>, SceneInfoOriginal)
		SLATE_ARGUMENT(FbxSceneReimportStatusMapPtr, NodeStatusMap)
	SLATE_END_ARGS()
	
	/** Construct this widget */
	void Construct(const FArguments& InArgs);
	TSharedRef< ITableRow > OnGenerateRowFbxSceneTreeView(FbxNodeInfoPtr Item, const TSharedRef< STableViewBase >& OwnerTable);
	void OnGetChildrenFbxSceneTreeView(FbxNodeInfoPtr InParent, TArray< FbxNodeInfoPtr >& OutChildren);

	void OnToggleSelectAll(ECheckBoxState CheckType);
	FReply OnExpandAll();
	FReply OnCollapseAll();

	static void FillNodeStatusMap(FbxSceneReimportStatusMapPtr NodeStatusMap, TSharedPtr<FFbxSceneInfo> SceneInfo, TSharedPtr<FFbxSceneInfo> SceneInfoOriginal);

protected:
	TSharedPtr<FFbxSceneInfo> SceneInfo;
	TSharedPtr<FFbxSceneInfo> SceneInfoOriginal;
	FbxSceneReimportStatusMapPtr NodeStatusMap;


	/** the elements we show in the tree view */
	TArray<FbxNodeInfoPtr> FbxRootNodeArray;

	/** Open a context menu for the current selection */
	TSharedPtr<SWidget> OnOpenContextMenu();
	void AddSelectionToImport();
	void RemoveSelectionFromImport();
	void SetSelectionImportState(bool MarkForImport);
	void OnSelectionChanged(FbxNodeInfoPtr Item, ESelectInfo::Type SelectionType);

	void GotoAsset(TSharedPtr<FFbxAttributeInfo> AssetAttribute);
	void RecursiveSetImport(FbxNodeInfoPtr NodeInfoPtr, bool ImportStatus);

	// Internal structure and function to create the tree view status data
	TMap<FbxNodeInfoPtr, TSharedPtr<FTreeNodeValue>> NodeTreeData;

	static void FillNodeStatusMapInternal(FbxSceneReimportStatusMapPtr NodeStatusMap
		, TSharedPtr<FFbxSceneInfo> SceneInfo
		, TSharedPtr<FFbxSceneInfo> SceneInfoOriginal
		, TMap<FbxNodeInfoPtr, TSharedPtr<FTreeNodeValue>>* NodeTreeDataPtr
		, TArray<FbxNodeInfoPtr>* FbxRootNodeArrayPtr);
};
