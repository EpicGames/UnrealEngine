// Copyright Epic Games, Inc. All Rights Reserved.

#include "Fbx/SSceneReimportSkeletalMeshListView.h"
#include "UObject/Package.h"
#include "Widgets/SOverlay.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Images/SImage.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SCheckBox.h"
#include "Styling/AppStyle.h"
#include "Factories/FbxSceneImportData.h"
#include "Styling/SlateIconFinder.h"
#include "Fbx/SSceneImportNodeTreeView.h"
#include "SFbxSceneOptionWindow.h"
#include "FbxImporter.h"

#define LOCTEXT_NAMESPACE "SFbxReimportSceneSkeletalMeshListView"

namespace FbxSceneReimportSkeletalMesh
{
	const FName CheckBoxSelectionHeaderIdName(TEXT("CheckBoxSelectionHeaderId"));
	const FName ClassIconHeaderIdName(TEXT("ClassIconHeaderId"));
	const FName AssetNameHeaderIdName(TEXT("AssetNameHeaderId"));
	const FName AssetStatusHeaderIdName(TEXT("AssetStatusHeaderId"));
	const FName ContentPathHeaderIdName(TEXT("ContentPathHeaderId"));
	const FName OptionNameHeaderIdName(TEXT("OptionNameHeaderId"));
}


class SFbxSkeletalReimportItemTableListViewRow : public SMultiColumnTableRow<FbxMeshInfoPtr>
{
public:
	SLATE_BEGIN_ARGS(SFbxSkeletalReimportItemTableListViewRow)
	: _FbxMeshInfo(nullptr)
	, _MeshStatusMap(nullptr)
	, _GlobalImportSettings(nullptr)
	{}
		SLATE_ARGUMENT(FbxMeshInfoPtr, FbxMeshInfo)
		SLATE_ARGUMENT(FbxSceneReimportStatusMapPtr, MeshStatusMap)
		SLATE_ARGUMENT(UnFbx::FBXImportOptions*, GlobalImportSettings)
	SLATE_END_ARGS()


	/** Construct function for this widget */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		FbxMeshInfo = InArgs._FbxMeshInfo;
		MeshStatusMap = InArgs._MeshStatusMap;
		GlobalImportSettings = InArgs._GlobalImportSettings;

		//This is suppose to always be valid
		check(FbxMeshInfo.IsValid());
		check(MeshStatusMap != nullptr);
		check(GlobalImportSettings != nullptr);

		//Get item data
		GetItemRowData();

		SMultiColumnTableRow<FbxMeshInfoPtr>::Construct(
			FSuperRowType::FArguments()
			.Style(FAppStyle::Get(), "DataTableEditor.CellListViewRow"),
			InOwnerTableView
			);
	}
	
	//Update the data at every tick
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
	{
		GetItemRowData();
	}

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the list view. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == FbxSceneReimportSkeletalMesh::CheckBoxSelectionHeaderIdName)
		{
			return SNew(SBox)
				.HAlign(HAlign_Center)
				[
					SNew(SCheckBox)
					.OnCheckStateChanged(this, &SFbxSkeletalReimportItemTableListViewRow::OnItemCheckChanged)
					.IsChecked(this, &SFbxSkeletalReimportItemTableListViewRow::IsItemChecked)
					.IsEnabled(!FbxMeshInfo->bOriginalTypeChanged)
				];
		}
		else if (ColumnName == FbxSceneReimportSkeletalMesh::ClassIconHeaderIdName && SlateBrush != nullptr)
		{
			UClass *IconClass = FbxMeshInfo->GetType();
			const FSlateBrush* ClassIcon = FSlateIconFinder::FindIconBrushForClass(IconClass);
			TSharedRef<SOverlay> IconContent = SNew(SOverlay)
				+ SOverlay::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(ClassIcon)
				]
				+ SOverlay::Slot()
				.HAlign(HAlign_Left)
				[
					SNew(SImage)
					.Image(this, &SFbxSkeletalReimportItemTableListViewRow::GetIconOverlay)
				]
				+ SOverlay::Slot()
				.HAlign(HAlign_Left)
				[
					SNew(SImage)
					.Image(this, &SFbxSkeletalReimportItemTableListViewRow::GetBrushForOverrideIcon)
				];
				return IconContent;
		}
		else if (ColumnName == FbxSceneReimportSkeletalMesh::AssetNameHeaderIdName)
		{
			return SNew(STextBlock)
				.Text(FText::FromString(FbxMeshInfo->Name))
				.ToolTipText(FText::FromString(FbxMeshInfo->Name));
		}
		else if (ColumnName == FbxSceneReimportSkeletalMesh::AssetStatusHeaderIdName)
		{
			return SNew(STextBlock)
				.Text(this, &SFbxSkeletalReimportItemTableListViewRow::GetAssetStatus)
				.ToolTipText(this, &SFbxSkeletalReimportItemTableListViewRow::GetAssetStatusTooltip);
				
		}
		else if (ColumnName == FbxSceneReimportSkeletalMesh::ContentPathHeaderIdName)
		{
			return SNew(STextBlock)
				.Text(this, &SFbxSkeletalReimportItemTableListViewRow::GetAssetFullName)
				.ColorAndOpacity(this, &SFbxSkeletalReimportItemTableListViewRow::GetContentPathTextColor)
				.ToolTipText(this, &SFbxSkeletalReimportItemTableListViewRow::GetAssetFullName);
		}
		else if (ColumnName == FbxSceneReimportSkeletalMesh::OptionNameHeaderIdName)
		{
			return SNew(STextBlock)
				.Text(this, &SFbxSkeletalReimportItemTableListViewRow::GetOptionName)
				.ToolTipText(this, &SFbxSkeletalReimportItemTableListViewRow::GetOptionName);
		}
		else if (ColumnName == FbxSceneBaseListViewColumn::PivotColumnId)
		{
			return SNew(STextBlock)
				.Text(this, &SFbxSkeletalReimportItemTableListViewRow::GetAssetPivotNodeName)
				.ToolTipText(this, &SFbxSkeletalReimportItemTableListViewRow::GetAssetPivotNodeName);
		}

		return SNullWidget::NullWidget;
	}

private:
	
	FSlateColor GetContentPathTextColor() const
	{
		return FbxMeshInfo->bOverridePath ? FLinearColor(0.75f, 0.75f, 0.0f, 1.0f) : FSlateColor::UseForeground();
	}

	const FSlateBrush *GetIconOverlay() const
	{
		return SlateBrush;
	}

	const FSlateBrush* GetBrushForOverrideIcon() const
	{
		if(UFbxSceneImportFactory::DefaultOptionName.Compare(FbxMeshInfo->OptionName) != 0)
		{
			return FAppStyle::GetBrush("FBXIcon.ImportOptionsOverride");
		}
		return FAppStyle::GetBrush("FBXIcon.ImportOptionsDefault");
	}
	
	FText GetOptionName() const
	{
		return FText::FromString(FbxMeshInfo->OptionName);
	}

	FText GetAssetFullName() const
	{
		return FText::FromString(FbxMeshInfo->GetFullImportName());
	}

	FText GetAssetStatus() const
	{
		return FText::FromString(AssetStatus);
	}
	
	FText GetAssetStatusTooltip() const
	{
		return FText::FromString(AssetStatusTooltip);
	}

	void GetItemRowData()
	{
		AssetStatus = LOCTEXT("SFbxSkeletalReimportItemTableListViewRow_NoValidStatus", "No valid status").ToString();
		AssetStatusTooltip = LOCTEXT("SFbxSkeletalReimportItemTableListViewRow_CannotBeReimport", "This item cannot be reimport because there is no valid status").ToString();
		SlateBrush = FAppStyle::GetBrush("FBXIcon.ReimportError");
		if (MeshStatusMap->Contains(FbxMeshInfo->OriginalImportPath))
		{
			EFbxSceneReimportStatusFlags ReimportFlags = *MeshStatusMap->Find(FbxMeshInfo->OriginalImportPath);
			//The remove only should not be possible this is why there is no remove only case

			if (FbxMeshInfo->bOriginalTypeChanged)
			{
				AssetStatus = LOCTEXT("SFbxSkeletalReimportItemTableListViewRow_AssetTypeChanged", "Type Changed, no reimport").ToString();
				AssetStatusTooltip = LOCTEXT("SFbxSkeletalReimportItemTableListViewRow_AssetTypeChangedTooltip", "This item type changed, we cannot reimport an asset of a different type").ToString();
			}
			else if ((ReimportFlags & EFbxSceneReimportStatusFlags::FoundContentBrowserAsset) == EFbxSceneReimportStatusFlags::None)
			{
				if ((ReimportFlags & EFbxSceneReimportStatusFlags::Added) != EFbxSceneReimportStatusFlags::None)
				{
					SlateBrush = FAppStyle::GetBrush("FBXIcon.ReimportAdded");
					AssetStatus = LOCTEXT("SFbxSkeletalReimportItemTableListViewRow_AddCreateContent", "Added, create content").ToString();
					AssetStatusTooltip = LOCTEXT("SFbxSkeletalReimportItemTableListViewRow_AddCreateContentTooltip", "This item was added to the fbx scene file, content will be create if this item is select for reimport").ToString();
				}
				else if ((ReimportFlags & EFbxSceneReimportStatusFlags::Same) != EFbxSceneReimportStatusFlags::None)
				{
					SlateBrush = FAppStyle::GetBrush("FBXIcon.ReimportSame");
					AssetStatus = LOCTEXT("SFbxSkeletalReimportItemTableListViewRow_SameCreateContent", "Same, create content").ToString();
					AssetStatusTooltip = LOCTEXT("SFbxSkeletalReimportItemTableListViewRow_SameCreateContentTooltip", "This item match the old fbx but no content was found, content will be create if this item is select for reimport").ToString();
				}
			}
			else
			{
				if ((ReimportFlags & EFbxSceneReimportStatusFlags::Added) != EFbxSceneReimportStatusFlags::None)
				{
					SlateBrush = FAppStyle::GetBrush("FBXIcon.ReimportAddedContent");
					AssetStatus = LOCTEXT("SFbxSkeletalReimportItemTableListViewRow_AddOverrideContent", "Added, override content").ToString();
					AssetStatusTooltip = LOCTEXT("SFbxSkeletalReimportItemTableListViewRow_AddOverrideContentTooltip", "This item was added but a content was found, content will be override if this item is select for reimport").ToString();
				}
				else if ((ReimportFlags & EFbxSceneReimportStatusFlags::Removed) != EFbxSceneReimportStatusFlags::None)
				{
					SlateBrush = FAppStyle::GetBrush("FBXIcon.ReimportRemovedContent");
					AssetStatus = LOCTEXT("SFbxSkeletalReimportItemTableListViewRow_RemoveDeleteContent", "Removed, delete content").ToString();
					AssetStatusTooltip = LOCTEXT("SFbxSkeletalReimportItemTableListViewRow_RemoveDeleteContentTooltip", "This item was deleted but a content was found, content will be delete if this item is select for reimport").ToString();
				}
				else if ((ReimportFlags & EFbxSceneReimportStatusFlags::Same) != EFbxSceneReimportStatusFlags::None)
				{
					SlateBrush = FAppStyle::GetBrush("FBXIcon.ReimportSameContent");
					AssetStatus = LOCTEXT("SFbxSkeletalReimportItemTableListViewRow_SameReplaceContent", "Same, replace content").ToString();
					AssetStatusTooltip = LOCTEXT("SFbxSkeletalReimportItemTableListViewRow_SameReplaceContentTooltip", "This item match the old fbx, content will be replace if this item is select for reimport").ToString();
				}
			}
		}
	}

	void OnItemCheckChanged(ECheckBoxState CheckType)
	{
		if (!FbxMeshInfo.IsValid() || FbxMeshInfo->bOriginalTypeChanged)
			return;
		if (MeshStatusMap->Contains(FbxMeshInfo->OriginalImportPath))
		{
			EFbxSceneReimportStatusFlags *StatusFlag = MeshStatusMap->Find(FbxMeshInfo->OriginalImportPath);
			if (CheckType == ECheckBoxState::Checked)
			{
				*StatusFlag = *StatusFlag | EFbxSceneReimportStatusFlags::ReimportAsset;
			}
			else
			{
				*StatusFlag = *StatusFlag & ~EFbxSceneReimportStatusFlags::ReimportAsset;
			}
		}
	}

	ECheckBoxState IsItemChecked() const
	{
		if (MeshStatusMap->Contains(FbxMeshInfo->OriginalImportPath) && !FbxMeshInfo->bOriginalTypeChanged)
		{
			return (*(MeshStatusMap->Find(FbxMeshInfo->OriginalImportPath)) & EFbxSceneReimportStatusFlags::ReimportAsset) != EFbxSceneReimportStatusFlags::None ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
		return ECheckBoxState::Unchecked;
	}

	FText GetAssetPivotNodeName() const
	{
		//TODO support bBakePivotInVertex
		//return GlobalImportSettings->bBakePivotInVertex ? FText::FromString(FbxMeshInfo->PivotNodeName) : FText::FromString(TEXT("-"));
		return FText::FromString(TEXT("-"));
	}

	/** The node info to build the tree view row from. */
	FbxMeshInfoPtr FbxMeshInfo;
	FbxSceneReimportStatusMapPtr MeshStatusMap;
	UnFbx::FBXImportOptions *GlobalImportSettings;
	
	//Get item data
	FString AssetStatus;
	FString AssetStatusTooltip;
	const FSlateBrush *SlateBrush;
};

SFbxSceneSkeletalMeshReimportListView::~SFbxSceneSkeletalMeshReimportListView()
{
	FbxMeshesArray.Empty();
	SceneInfo = nullptr;
	SceneInfoOriginal = nullptr;
	GlobalImportSettings = nullptr;
	OverrideNameOptions = nullptr;
	OverrideNameOptionsMap = nullptr;
	SceneImportOptionsSkeletalMeshDisplay = nullptr;
	MeshStatusMap = nullptr;
	FilterFbxMeshesArray.Empty();
	FilterAddContent = false;
	FilterDeleteContent = false;
	FilterOverwriteContent = false;
	FilterDiff = false;
}

void SFbxSceneSkeletalMeshReimportListView::Construct(const SFbxSceneSkeletalMeshReimportListView::FArguments& InArgs)
{
	SceneInfo = InArgs._SceneInfo;
	SceneInfoOriginal = InArgs._SceneInfoOriginal;
	MeshStatusMap = InArgs._MeshStatusMap;
	GlobalImportSettings = InArgs._GlobalImportSettings;
	OverrideNameOptions = InArgs._OverrideNameOptions;
	OverrideNameOptionsMap = InArgs._OverrideNameOptionsMap;
	SceneImportOptionsSkeletalMeshDisplay = InArgs._SceneImportOptionsSkeletalMeshDisplay;

	check(SceneInfo.IsValid());
	check(SceneInfoOriginal.IsValid());
	check(MeshStatusMap != nullptr);
	check(GlobalImportSettings != nullptr);
	check(OverrideNameOptions != nullptr);
	check(OverrideNameOptionsMap != nullptr);
	check(SceneImportOptionsSkeletalMeshDisplay != nullptr);
	
	DefaultOptionNamePtr = MakeShareable(new FString(UFbxSceneImportFactory::DefaultOptionName));

	for (auto kvp : *OverrideNameOptionsMap)
	{
		bool bNameExist = false;
		for (TSharedPtr<FString> OverrideName : (*OverrideNameOptions))
		{
			if (OverrideName.Get()->Compare(kvp.Key) == 0)
			{
				bNameExist = true;
				break;
			}
		}
		if (!bNameExist)
		{
			if (kvp.Key.Compare(UFbxSceneImportFactory::DefaultOptionName) == 0)
			{
				OverrideNameOptions->Add(DefaultOptionNamePtr);
				SFbxSceneOptionWindow::CopyFbxOptionsToFbxOptions(kvp.Value, GlobalImportSettings);
			}
			else
			{
				OverrideNameOptions->Add(MakeShareable(new FString(kvp.Key)));
			}
		}
	}
	//Set the default options to the current global import settings
	GlobalImportSettings->bTransformVertexToAbsolute = false;
	GlobalImportSettings->StaticMeshLODGroup = NAME_None;
	CurrentMeshImportOptions = GlobalImportSettings;

	FbxMeshesArray.Empty();
	FilterFbxMeshesArray.Empty();
	FilterAddContent = false;
	FilterDeleteContent = false;
	FilterOverwriteContent = false;
	FilterDiff = false;

	const bool bFillSkeletalMeshStatusMap = true;
	SFbxSSceneBaseMeshListView::FillMeshStatusMap(MeshStatusMap, SceneInfo, SceneInfoOriginal, bFillSkeletalMeshStatusMap, &FilterFbxMeshesArray, &FbxMeshesArray);

	SListView::Construct
		(
			SListView::FArguments()
			.ListItemsSource(&FilterFbxMeshesArray)
			.SelectionMode(ESelectionMode::Multi)
			.OnGenerateRow(this, &SFbxSceneSkeletalMeshReimportListView::OnGenerateRowFbxSceneListView)
			.OnContextMenuOpening(this, &SFbxSceneSkeletalMeshReimportListView::OnOpenContextMenu)
			.HeaderRow
			(
				SNew(SHeaderRow)

				+ SHeaderRow::Column(FbxSceneReimportSkeletalMesh::CheckBoxSelectionHeaderIdName)
				.FixedWidth(26)
				.DefaultLabel(FText::GetEmpty())
				[
					SNew(SCheckBox)
					.HAlign(HAlign_Center)
				.OnCheckStateChanged(this, &SFbxSceneSkeletalMeshReimportListView::OnToggleSelectAll)
				]

				+ SHeaderRow::Column(FbxSceneReimportSkeletalMesh::ClassIconHeaderIdName)
				.FixedWidth(20)
				.DefaultLabel(FText::GetEmpty())

				+ SHeaderRow::Column(FbxSceneReimportSkeletalMesh::AssetNameHeaderIdName)
				.FillWidth(250)
				.HAlignCell(EHorizontalAlignment::HAlign_Left)
				.DefaultLabel(LOCTEXT("AssetNameHeaderName", "Asset Name"))

				+ SHeaderRow::Column(FbxSceneReimportSkeletalMesh::ContentPathHeaderIdName)
				.FillWidth(250)
				.HAlignCell(EHorizontalAlignment::HAlign_Left)
				.DefaultLabel(LOCTEXT("ContentPathHeaderName", "Content Path"))

				+ SHeaderRow::Column(FbxSceneReimportSkeletalMesh::AssetStatusHeaderIdName)
				.FillWidth(160)
				.HAlignCell(EHorizontalAlignment::HAlign_Left)
				.DefaultLabel(LOCTEXT("AssetStatusHeaderName", "Asset Status"))

				+ SHeaderRow::Column(FbxSceneReimportSkeletalMesh::OptionNameHeaderIdName)
				.FillWidth(100)
				.HAlignCell(EHorizontalAlignment::HAlign_Left)
				.DefaultLabel(LOCTEXT("AssetOptionNameHeaderName", "Option Name"))
				)
			);
}

TSharedRef< ITableRow > SFbxSceneSkeletalMeshReimportListView::OnGenerateRowFbxSceneListView(FbxMeshInfoPtr Item, const TSharedRef< STableViewBase >& OwnerTable)
{
	TSharedRef< SFbxSkeletalReimportItemTableListViewRow > ReturnRow = SNew(SFbxSkeletalReimportItemTableListViewRow, OwnerTable)
		.FbxMeshInfo(Item)
		.MeshStatusMap(MeshStatusMap)
		.GlobalImportSettings(GlobalImportSettings);
	return ReturnRow;
}

void SFbxSceneSkeletalMeshReimportListView::OnChangedOverrideOptions(TSharedPtr<FString> ItemSelected, ESelectInfo::Type SelectInfo)
{
	if (ItemSelected.IsValid())
	{
		if (ItemSelected.Get()->Compare(UFbxSceneImportFactory::DefaultOptionName) == 0)
		{
			CurrentMeshImportOptions = GlobalImportSettings;
		}
		else if (OverrideNameOptionsMap->Contains(*(ItemSelected)))
		{
			CurrentMeshImportOptions = *OverrideNameOptionsMap->Find(*(ItemSelected));
		}
		SFbxSceneOptionWindow::CopyFbxOptionsToSkeletalMeshOptions(CurrentMeshImportOptions, SceneImportOptionsSkeletalMeshDisplay);
	}
}

void SFbxSceneSkeletalMeshReimportListView::OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent)
{
	SFbxSceneOptionWindow::CopySkeletalMeshOptionsToFbxOptions(CurrentMeshImportOptions, SceneImportOptionsSkeletalMeshDisplay);
}

TSharedPtr<SWidget> SFbxSceneSkeletalMeshReimportListView::OnOpenContextMenu()
{
	TArray<FbxMeshInfoPtr> SelectedFbxMeshInfos;
	int32 SelectCount = GetSelectedItems(SelectedFbxMeshInfos);
	// Build up the menu for a selection
	const bool bCloseAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, TSharedPtr<FUICommandList>());

	// We always create a section here, even if there is no parent so that clients can still extend the menu
	MenuBuilder.BeginSection("FbxScene_SM_ImportSection");
	{
		const FSlateIcon PlusIcon(FAppStyle::GetAppStyleSetName(), "Plus");
		MenuBuilder.AddMenuEntry(LOCTEXT("CheckForImport", "Add Selection To Import"), FText(), PlusIcon, FUIAction(FExecuteAction::CreateSP(this, &SFbxSceneSkeletalMeshReimportListView::AddSelectionToImport)));
		const FSlateIcon MinusIcon(FAppStyle::GetAppStyleSetName(), "Icons.Minus");
		MenuBuilder.AddMenuEntry(LOCTEXT("UncheckForImport", "Remove Selection From Import"), FText(), MinusIcon, FUIAction(FExecuteAction::CreateSP(this, &SFbxSceneSkeletalMeshReimportListView::RemoveSelectionFromImport)));
	}
	MenuBuilder.EndSection();
	
	//TODO support bBakePivotInVertex 
	//AddBakePivotMenu(MenuBuilder);

	bool bShowOptionMenu = false;
	for (FbxMeshInfoPtr MeshInfo : SelectedFbxMeshInfos)
	{
		EFbxSceneReimportStatusFlags ReimportFlags = *MeshStatusMap->Find(MeshInfo->OriginalImportPath);
		if ((ReimportFlags & EFbxSceneReimportStatusFlags::Removed) == EFbxSceneReimportStatusFlags::None)
		{
			bShowOptionMenu = true;
		}
	}
	if (bShowOptionMenu)
	{
		MenuBuilder.BeginSection("FbxScene_SM_OptionsSection", LOCTEXT("FbxScene_SM_Options", "Options:"));
		{
			for (auto OptionName : (*OverrideNameOptions))
			{
				MenuBuilder.AddMenuEntry(FText::FromString(*OptionName.Get()), FText(), FSlateIcon(), FUIAction(FExecuteAction::CreateSP(this, &SFbxSceneSkeletalMeshReimportListView::AssignToOptions, *OptionName.Get())));
			}
		}
		MenuBuilder.EndSection();
	}
	return MenuBuilder.MakeWidget();
}

void SFbxSceneSkeletalMeshReimportListView::SetSelectionImportState(bool MarkForImport)
{
	TArray<FbxMeshInfoPtr> SelectedFbxMeshInfos;
	GetSelectedItems(SelectedFbxMeshInfos);
	for (FbxMeshInfoPtr ItemPtr : SelectedFbxMeshInfos)
	{
		EFbxSceneReimportStatusFlags *ItemStatus = MeshStatusMap->Find(ItemPtr->OriginalImportPath);
		if (ItemStatus != nullptr)
		{
			*ItemStatus = MarkForImport ? *ItemStatus | EFbxSceneReimportStatusFlags::ReimportAsset : *ItemStatus & ~EFbxSceneReimportStatusFlags::ReimportAsset;
		}
	}
}

void SFbxSceneSkeletalMeshReimportListView::OnToggleSelectAll(ECheckBoxState CheckType)
{
	for (FbxMeshInfoPtr MeshInfo : FilterFbxMeshesArray)
	{
		EFbxSceneReimportStatusFlags *ItemStatus = MeshStatusMap->Find(MeshInfo->OriginalImportPath);
		if (ItemStatus != nullptr)
		{
			*ItemStatus = (CheckType == ECheckBoxState::Checked) ? *ItemStatus | EFbxSceneReimportStatusFlags::ReimportAsset : *ItemStatus & ~EFbxSceneReimportStatusFlags::ReimportAsset;
		}
	}
}

void SFbxSceneSkeletalMeshReimportListView::OnToggleFilterAddContent(ECheckBoxState CheckType)
{
	FilterAddContent = (CheckType == ECheckBoxState::Checked);
	UpdateFilterList();
}

void SFbxSceneSkeletalMeshReimportListView::OnToggleFilterDeleteContent(ECheckBoxState CheckType)
{
	FilterDeleteContent = (CheckType == ECheckBoxState::Checked);
	UpdateFilterList();
}

void SFbxSceneSkeletalMeshReimportListView::OnToggleFilterOverwriteContent(ECheckBoxState CheckType)
{
	FilterOverwriteContent = (CheckType == ECheckBoxState::Checked);
	UpdateFilterList();
}

void SFbxSceneSkeletalMeshReimportListView::OnToggleFilterDiff(ECheckBoxState CheckType)
{
	FilterDiff = (CheckType == ECheckBoxState::Checked);
	UpdateFilterList();
}

void SFbxSceneSkeletalMeshReimportListView::UpdateFilterList()
{
	FilterFbxMeshesArray.Empty();
	if (FilterAddContent || FilterDeleteContent || FilterOverwriteContent || FilterDiff)
	{
		for (FbxMeshInfoPtr MeshInfo : FbxMeshesArray)
		{
			EFbxSceneReimportStatusFlags ItemStatus = *(MeshStatusMap->Find(MeshInfo->OriginalImportPath));
			bool bStatusAdd = (ItemStatus & EFbxSceneReimportStatusFlags::Added) != EFbxSceneReimportStatusFlags::None;
			bool bStatusSame = (ItemStatus & EFbxSceneReimportStatusFlags::Same) != EFbxSceneReimportStatusFlags::None;
			bool bStatusRemove = (ItemStatus & EFbxSceneReimportStatusFlags::Removed) != EFbxSceneReimportStatusFlags::None;
			bool bStatusFoundContent = (ItemStatus & EFbxSceneReimportStatusFlags::FoundContentBrowserAsset) != EFbxSceneReimportStatusFlags::None;

			if (FilterAddContent && (bStatusAdd || bStatusSame) && !bStatusFoundContent)
			{
				FilterFbxMeshesArray.Add(MeshInfo);
			}
			else if (FilterDeleteContent && bStatusRemove && bStatusFoundContent)
			{
				FilterFbxMeshesArray.Add(MeshInfo);
			}
			else if (FilterOverwriteContent && (bStatusAdd || bStatusSame) && bStatusFoundContent)
			{
				FilterFbxMeshesArray.Add(MeshInfo);
			}
			else if (FilterDiff && !bStatusSame)
			{
				FilterFbxMeshesArray.Add(MeshInfo);
			}
		}
	}
	else
	{
		for (FbxMeshInfoPtr MeshInfo : FbxMeshesArray)
		{
			FilterFbxMeshesArray.Add(MeshInfo);
		}
		
	}
	this->RequestListRefresh();
}

#undef LOCTEXT_NAMESPACE
