// Copyright Epic Games, Inc. All Rights Reserved.

#include "Fbx/SSceneReimportStaticMeshListView.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SOverlay.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Images/SImage.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SCheckBox.h"
#include "Styling/AppStyle.h"
#include "Factories/FbxSceneImportData.h"
#include "Engine/StaticMesh.h"
#include "AssetRegistry/AssetData.h"
#include "Styling/SlateIconFinder.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "Fbx/SSceneImportNodeTreeView.h"
#include "SFbxSceneOptionWindow.h"
#include "FbxImporter.h"

#define LOCTEXT_NAMESPACE "SFbxReimportSceneStaticMeshListView"

namespace FbxSceneReimportStaticMesh
{
	const FName CheckBoxSelectionHeaderIdName(TEXT("CheckBoxSelectionHeaderId"));
	const FName ClassIconHeaderIdName(TEXT("ClassIconHeaderId"));
	const FName AssetNameHeaderIdName(TEXT("AssetNameHeaderId"));
	const FName AssetStatusHeaderIdName(TEXT("AssetStatusHeaderId"));
	const FName ContentPathHeaderIdName(TEXT("ContentPathHeaderId"));
	const FName OptionNameHeaderIdName(TEXT("OptionNameHeaderId"));
}

class SFbxMeshReimportItemTableListViewRow : public SMultiColumnTableRow<FbxMeshInfoPtr>
{
public:
	SLATE_BEGIN_ARGS(SFbxMeshReimportItemTableListViewRow)
	: _FbxMeshInfo(nullptr)
	, _MeshStatusMap(nullptr)
	, _GlobalImportSettings(nullptr)
	{}
		SLATE_ARGUMENT(FbxMeshInfoPtr, FbxMeshInfo)
		SLATE_ARGUMENT(FbxSceneReimportStatusMapPtr, MeshStatusMap)
		SLATE_ARGUMENT(UnFbx::FBXImportOptions*, GlobalImportSettings)
	SLATE_END_ARGS()


	/** Construct function for this widget */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		FbxMeshInfo = InArgs._FbxMeshInfo;
		MeshStatusMap = InArgs._MeshStatusMap;
		GlobalImportSettings = InArgs._GlobalImportSettings;

		//This is suppose to always be valid
		check(FbxMeshInfo.IsValid());
		check(MeshStatusMap != nullptr);
		check(GlobalImportSettings != nullptr);

		//Get item data
		GetItemRowData();

		SMultiColumnTableRow<FbxMeshInfoPtr>::Construct(
			FSuperRowType::FArguments()
			.Style(FAppStyle::Get(), "DataTableEditor.CellListViewRow"),
			InOwnerTableView
			);
	}
	
	//Update the data at every tick
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
	{
		GetItemRowData();
	}

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the list view. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == FbxSceneReimportStaticMesh::CheckBoxSelectionHeaderIdName)
		{
			return SNew(SBox)
				.HAlign(HAlign_Center)
				[
					SNew(SCheckBox)
					.OnCheckStateChanged(this, &SFbxMeshReimportItemTableListViewRow::OnItemCheckChanged)
					.IsChecked(this, &SFbxMeshReimportItemTableListViewRow::IsItemChecked)
					.IsEnabled(!FbxMeshInfo->bOriginalTypeChanged)
				];
		}
		else if (ColumnName == FbxSceneReimportStaticMesh::ClassIconHeaderIdName && SlateBrush != nullptr)
		{
			UClass *IconClass = FbxMeshInfo->GetType();
			const FSlateBrush* ClassIcon = FSlateIconFinder::FindIconBrushForClass(IconClass);
			TSharedRef<SOverlay> IconContent = SNew(SOverlay)
				+ SOverlay::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(ClassIcon)
				]
				+ SOverlay::Slot()
				.HAlign(HAlign_Left)
				[
					SNew(SImage)
					.Image(this, &SFbxMeshReimportItemTableListViewRow::GetIconOverlay)
				]
				+ SOverlay::Slot()
				.HAlign(HAlign_Left)
				[
					SNew(SImage)
					.Image(this, &SFbxMeshReimportItemTableListViewRow::GetBrushForOverrideIcon)
				];
				return IconContent;
		}
		else if (ColumnName == FbxSceneReimportStaticMesh::AssetNameHeaderIdName)
		{
			return SNew(STextBlock)
				.Text(FText::FromString(FbxMeshInfo->Name))
				.ToolTipText(FText::FromString(FbxMeshInfo->Name));
		}
		else if (ColumnName == FbxSceneReimportStaticMesh::AssetStatusHeaderIdName)
		{
			return SNew(STextBlock)
				.Text(this, &SFbxMeshReimportItemTableListViewRow::GetAssetStatus)
				.ToolTipText(this, &SFbxMeshReimportItemTableListViewRow::GetAssetStatusTooltip);
				
		}
		else if (ColumnName == FbxSceneReimportStaticMesh::ContentPathHeaderIdName)
		{
			return SNew(STextBlock)
				.Text(this, &SFbxMeshReimportItemTableListViewRow::GetAssetFullName)
				.ColorAndOpacity(this, &SFbxMeshReimportItemTableListViewRow::GetContentPathTextColor)
				.ToolTipText(this, &SFbxMeshReimportItemTableListViewRow::GetAssetFullName);
		}
		else if (ColumnName == FbxSceneReimportStaticMesh::OptionNameHeaderIdName)
		{
			return SNew(STextBlock)
				.Text(this, &SFbxMeshReimportItemTableListViewRow::GetOptionName)
				.ToolTipText(this, &SFbxMeshReimportItemTableListViewRow::GetOptionName);
		}
		else if (ColumnName == FbxSceneBaseListViewColumn::PivotColumnId)
		{
			return SNew(STextBlock)
				.Text(this, &SFbxMeshReimportItemTableListViewRow::GetAssetPivotNodeName)
				.ToolTipText(this, &SFbxMeshReimportItemTableListViewRow::GetAssetPivotNodeName);
		}

		return SNullWidget::NullWidget;
	}

private:
	
	FSlateColor GetContentPathTextColor() const
	{
		return FbxMeshInfo->bOverridePath ? FLinearColor(0.75f, 0.75f, 0.0f, 1.0f) : FSlateColor::UseForeground();
	}

	const FSlateBrush *GetIconOverlay() const
	{
		return SlateBrush;
	}

	const FSlateBrush* GetBrushForOverrideIcon() const
	{
		if(UFbxSceneImportFactory::DefaultOptionName.Compare(FbxMeshInfo->OptionName) != 0)
		{
			return FAppStyle::GetBrush("FBXIcon.ImportOptionsOverride");
		}
		return FAppStyle::GetBrush("FBXIcon.ImportOptionsDefault");
	}
	
	FText GetOptionName() const
	{
		return FText::FromString(FbxMeshInfo->OptionName);
	}

	FText GetAssetFullName() const
	{
		return FText::FromString(FbxMeshInfo->GetFullImportName());
	}

	FText GetAssetStatus() const
	{
		return FText::FromString(AssetStatus);
	}
	
	FText GetAssetStatusTooltip() const
	{
		return FText::FromString(AssetStatusTooltip);
	}

	void GetItemRowData()
	{
		AssetStatus = LOCTEXT("SFbxMeshReimportItemTableListViewRow_NoValidStatus", "No valid status").ToString();
		AssetStatusTooltip = LOCTEXT("SFbxMeshReimportItemTableListViewRow_CannotBeReimport", "This item cannot be reimport because there is no valid status").ToString();
		SlateBrush = FAppStyle::GetBrush("FBXIcon.ReimportError");
		if (MeshStatusMap->Contains(FbxMeshInfo->OriginalImportPath))
		{
			EFbxSceneReimportStatusFlags ReimportFlags = *MeshStatusMap->Find(FbxMeshInfo->OriginalImportPath);
			//The remove only should not be possible this is why there is no remove only case
			if (FbxMeshInfo->bOriginalTypeChanged)
			{
				AssetStatus = LOCTEXT("SFbxMeshReimportItemTableListViewRow_AssetTypeChanged", "Type Changed, no reimport").ToString();
				AssetStatusTooltip = LOCTEXT("SFbxMeshReimportItemTableListViewRow_AssetTypeChangedTooltip", "This item type changed, we cannot reimport an asset of a different type").ToString();
			}
			else if ((ReimportFlags & EFbxSceneReimportStatusFlags::FoundContentBrowserAsset) == EFbxSceneReimportStatusFlags::None)
			{
				if ((ReimportFlags & EFbxSceneReimportStatusFlags::Added) != EFbxSceneReimportStatusFlags::None)
				{
					SlateBrush = FAppStyle::GetBrush("FBXIcon.ReimportAdded");
					AssetStatus = LOCTEXT("SFbxMeshReimportItemTableListViewRow_AddCreateContent", "Added, create content").ToString();
					AssetStatusTooltip = LOCTEXT("SFbxMeshReimportItemTableListViewRow_AddCreateContentTooltip", "This item was added to the fbx scene file, content will be create if this item is select for reimport").ToString();
				}
				else if ((ReimportFlags & EFbxSceneReimportStatusFlags::Same) != EFbxSceneReimportStatusFlags::None)
				{
					SlateBrush = FAppStyle::GetBrush("FBXIcon.ReimportSame");
					AssetStatus = LOCTEXT("SFbxMeshReimportItemTableListViewRow_SameCreateContent", "Same, create content").ToString();
					AssetStatusTooltip = LOCTEXT("SFbxMeshReimportItemTableListViewRow_SameCreateContentTooltip", "This item match the old fbx but no content was found, content will be create if this item is select for reimport").ToString();
				}
			}
			else
			{
				if ((ReimportFlags & EFbxSceneReimportStatusFlags::Added) != EFbxSceneReimportStatusFlags::None)
				{
					SlateBrush = FAppStyle::GetBrush("FBXIcon.ReimportAddedContent");
					AssetStatus = LOCTEXT("SFbxMeshReimportItemTableListViewRow_AddOverrideContent", "Added, override content").ToString();
					AssetStatusTooltip = LOCTEXT("SFbxMeshReimportItemTableListViewRow_AddOverrideContentTooltip", "This item was added but a content was found, content will be override if this item is select for reimport").ToString();
				}
				else if ((ReimportFlags & EFbxSceneReimportStatusFlags::Removed) != EFbxSceneReimportStatusFlags::None)
				{
					SlateBrush = FAppStyle::GetBrush("FBXIcon.ReimportRemovedContent");
					AssetStatus = LOCTEXT("SFbxMeshReimportItemTableListViewRow_RemoveDeleteContent", "Removed, delete content").ToString();
					AssetStatusTooltip = LOCTEXT("SFbxMeshReimportItemTableListViewRow_RemoveDeleteContentTooltip", "This item was deleted but a content was found, content will be delete if this item is select for reimport").ToString();
				}
				else if ((ReimportFlags & EFbxSceneReimportStatusFlags::Same) != EFbxSceneReimportStatusFlags::None)
				{
					SlateBrush = FAppStyle::GetBrush("FBXIcon.ReimportSameContent");
					AssetStatus = LOCTEXT("SFbxMeshReimportItemTableListViewRow_SameReplaceContent", "Same, replace content").ToString();
					AssetStatusTooltip = LOCTEXT("SFbxMeshReimportItemTableListViewRow_SameReplaceContentTooltip", "This item match the old fbx, content will be replace if this item is select for reimport").ToString();
				}
			}
		}
	}

	void OnItemCheckChanged(ECheckBoxState CheckType)
	{
		if (!FbxMeshInfo.IsValid() || FbxMeshInfo->bOriginalTypeChanged)
			return;
		if (MeshStatusMap->Contains(FbxMeshInfo->OriginalImportPath))
		{
			EFbxSceneReimportStatusFlags *StatusFlag = MeshStatusMap->Find(FbxMeshInfo->OriginalImportPath);
			if (CheckType == ECheckBoxState::Checked)
			{
				*StatusFlag = *StatusFlag | EFbxSceneReimportStatusFlags::ReimportAsset;
			}
			else
			{
				*StatusFlag = *StatusFlag & ~EFbxSceneReimportStatusFlags::ReimportAsset;
			}
		}
	}

	ECheckBoxState IsItemChecked() const
	{
		if (MeshStatusMap->Contains(FbxMeshInfo->OriginalImportPath) && !FbxMeshInfo->bOriginalTypeChanged)
		{
			return (*(MeshStatusMap->Find(FbxMeshInfo->OriginalImportPath)) & EFbxSceneReimportStatusFlags::ReimportAsset) != EFbxSceneReimportStatusFlags::None ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
		return ECheckBoxState::Unchecked;
	}

	FText GetAssetPivotNodeName() const
	{
		return GlobalImportSettings->bBakePivotInVertex ? FText::FromString(FbxMeshInfo->PivotNodeName) : FText::FromString(TEXT("-"));
	}

	/** The node info to build the tree view row from. */
	FbxMeshInfoPtr FbxMeshInfo;
	FbxSceneReimportStatusMapPtr MeshStatusMap;
	UnFbx::FBXImportOptions *GlobalImportSettings;
	
	//Get item data
	FString AssetStatus;
	FString AssetStatusTooltip;
	const FSlateBrush *SlateBrush;
};

SFbxSceneStaticMeshReimportListView::~SFbxSceneStaticMeshReimportListView()
{
	FbxMeshesArray.Empty();
	SceneInfo = nullptr;
	SceneInfoOriginal = nullptr;
	GlobalImportSettings = nullptr;
	OverrideNameOptions = nullptr;
	OverrideNameOptionsMap = nullptr;
	SceneImportOptionsStaticMeshDisplay = nullptr;
	MeshStatusMap = nullptr;
	FilterFbxMeshesArray.Empty();
	FilterAddContent = false;
	FilterDeleteContent = false;
	FilterOverwriteContent = false;
	FilterDiff = false;
}

void SFbxSceneStaticMeshReimportListView::Construct(const SFbxSceneStaticMeshReimportListView::FArguments& InArgs)
{
	SceneInfo = InArgs._SceneInfo;
	SceneInfoOriginal = InArgs._SceneInfoOriginal;
	MeshStatusMap = InArgs._MeshStatusMap;
	GlobalImportSettings = InArgs._GlobalImportSettings;
	OverrideNameOptions = InArgs._OverrideNameOptions;
	OverrideNameOptionsMap = InArgs._OverrideNameOptionsMap;
	SceneImportOptionsStaticMeshDisplay = InArgs._SceneImportOptionsStaticMeshDisplay;

	check(SceneInfo.IsValid());
	check(SceneInfoOriginal.IsValid());
	check(MeshStatusMap != nullptr);
	check(GlobalImportSettings != nullptr);
	check(OverrideNameOptions != nullptr);
	check(OverrideNameOptionsMap != nullptr);
	check(SceneImportOptionsStaticMeshDisplay != nullptr);
	
	DefaultOptionNamePtr = MakeShareable(new FString(UFbxSceneImportFactory::DefaultOptionName));

	for (auto kvp : *OverrideNameOptionsMap)
	{
		bool bNameExist = false;
		for (TSharedPtr<FString> OverrideName : (*OverrideNameOptions))
		{
			if (OverrideName.Get()->Compare(kvp.Key) == 0)
			{
				bNameExist = true;
				break;
			}
		}
		if (!bNameExist)
		{
			if (kvp.Key.Compare(UFbxSceneImportFactory::DefaultOptionName) == 0)
			{
				OverrideNameOptions->Add(DefaultOptionNamePtr);
				SFbxSceneOptionWindow::CopyFbxOptionsToFbxOptions(kvp.Value, GlobalImportSettings);
			}
			else
			{
				OverrideNameOptions->Add(MakeShareable(new FString(kvp.Key)));
			}
		}
	}
	//Set the default options to the current global import settings
	GlobalImportSettings->bTransformVertexToAbsolute = false;
	GlobalImportSettings->StaticMeshLODGroup = NAME_None;
	CurrentMeshImportOptions = GlobalImportSettings;

	FbxMeshesArray.Empty();
	FilterFbxMeshesArray.Empty();
	FilterAddContent = false;
	FilterDeleteContent = false;
	FilterOverwriteContent = false;
	FilterDiff = false;

	const bool bFillSkeletalMeshStatusMap = false;
	SFbxSSceneBaseMeshListView::FillMeshStatusMap(MeshStatusMap, SceneInfo, SceneInfoOriginal, bFillSkeletalMeshStatusMap, &FilterFbxMeshesArray, &FbxMeshesArray);

	SListView::Construct
		(
			SListView::FArguments()
			.ListItemsSource(&FilterFbxMeshesArray)
			.SelectionMode(ESelectionMode::Multi)
			.OnGenerateRow(this, &SFbxSceneStaticMeshReimportListView::OnGenerateRowFbxSceneListView)
			.OnContextMenuOpening(this, &SFbxSceneStaticMeshReimportListView::OnOpenContextMenu)
			.HeaderRow
			(
				SNew(SHeaderRow)
				
				+ SHeaderRow::Column(FbxSceneReimportStaticMesh::CheckBoxSelectionHeaderIdName)
				.FixedWidth(26)
				.DefaultLabel(FText::GetEmpty())
				[
					SNew(SCheckBox)
					.HAlign(HAlign_Center)
					.OnCheckStateChanged(this, &SFbxSceneStaticMeshReimportListView::OnToggleSelectAll)
				]
				
				+ SHeaderRow::Column(FbxSceneReimportStaticMesh::ClassIconHeaderIdName)
				.FixedWidth(20)
				.DefaultLabel(FText::GetEmpty())
				
				+ SHeaderRow::Column(FbxSceneReimportStaticMesh::AssetNameHeaderIdName)
				.FillWidth(250)
				.HAlignCell(EHorizontalAlignment::HAlign_Left)
				.DefaultLabel(LOCTEXT("AssetNameHeaderName", "Asset Name"))

				+ SHeaderRow::Column(FbxSceneReimportStaticMesh::ContentPathHeaderIdName)
				.FillWidth(250)
				.HAlignCell(EHorizontalAlignment::HAlign_Left)
				.DefaultLabel(LOCTEXT("ContentPathHeaderName", "Content Path"))

				+ SHeaderRow::Column(FbxSceneReimportStaticMesh::AssetStatusHeaderIdName)
				.FillWidth(160)
				.HAlignCell(EHorizontalAlignment::HAlign_Left)
				.DefaultLabel(LOCTEXT("AssetStatusHeaderName", "Asset Status"))
				
				+ SHeaderRow::Column(FbxSceneReimportStaticMesh::OptionNameHeaderIdName)
				.FillWidth(100)
				.HAlignCell(EHorizontalAlignment::HAlign_Left)
				.DefaultLabel(LOCTEXT("AssetOptionNameHeaderName", "Option Name"))
			)
		);
}

TSharedRef< ITableRow > SFbxSceneStaticMeshReimportListView::OnGenerateRowFbxSceneListView(FbxMeshInfoPtr Item, const TSharedRef< STableViewBase >& OwnerTable)
{
	TSharedRef< SFbxMeshReimportItemTableListViewRow > ReturnRow = SNew(SFbxMeshReimportItemTableListViewRow, OwnerTable)
		.FbxMeshInfo(Item)
		.MeshStatusMap(MeshStatusMap)
		.GlobalImportSettings(GlobalImportSettings);
	return ReturnRow;
}

void SFbxSceneStaticMeshReimportListView::OnChangedOverrideOptions(TSharedPtr<FString> ItemSelected, ESelectInfo::Type SelectInfo)
{
	if (ItemSelected.IsValid())
	{
		if (ItemSelected.Get()->Compare(UFbxSceneImportFactory::DefaultOptionName) == 0)
		{
			CurrentMeshImportOptions = GlobalImportSettings;
		}
		else if (OverrideNameOptionsMap->Contains(*(ItemSelected)))
		{
			CurrentMeshImportOptions = *OverrideNameOptionsMap->Find(*(ItemSelected));
		}
		SFbxSceneOptionWindow::CopyFbxOptionsToStaticMeshOptions(CurrentMeshImportOptions, SceneImportOptionsStaticMeshDisplay);
	}
}

void SFbxSceneStaticMeshReimportListView::OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent)
{
	SFbxSceneOptionWindow::CopyStaticMeshOptionsToFbxOptions(CurrentMeshImportOptions, SceneImportOptionsStaticMeshDisplay);
}

TSharedPtr<SWidget> SFbxSceneStaticMeshReimportListView::OnOpenContextMenu()
{
	TArray<FbxMeshInfoPtr> SelectedFbxMeshInfos;
	int32 SelectCount = GetSelectedItems(SelectedFbxMeshInfos);
	// Build up the menu for a selection
	const bool bCloseAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, TSharedPtr<FUICommandList>());

	// We always create a section here, even if there is no parent so that clients can still extend the menu
	MenuBuilder.BeginSection("FbxScene_SM_ImportSection");
	{
		const FSlateIcon PlusIcon(FAppStyle::GetAppStyleSetName(), "Plus");
		MenuBuilder.AddMenuEntry(LOCTEXT("CheckForImport", "Add Selection To Import"), FText(), PlusIcon, FUIAction(FExecuteAction::CreateSP(this, &SFbxSceneStaticMeshReimportListView::AddSelectionToImport)));
		const FSlateIcon MinusIcon(FAppStyle::GetAppStyleSetName(), "Icons.Minus");
		MenuBuilder.AddMenuEntry(LOCTEXT("UncheckForImport", "Remove Selection From Import"), FText(), MinusIcon, FUIAction(FExecuteAction::CreateSP(this, &SFbxSceneStaticMeshReimportListView::RemoveSelectionFromImport)));
	}
	MenuBuilder.EndSection();
	
	//Build the Assign section
	if (SelectCount == 1 || ShowResetAssignToStaticMesh())
	{
		MenuBuilder.BeginSection("FbxImportScene_SM_Assign", LOCTEXT("FbxScene_SM_Assign", "Assign"));
		{
			if (SelectCount == 1)
			{
				MenuBuilder.AddMenuEntry(LOCTEXT("FbxImportScene_SM_Assign", "Assign existing static mesh..."), FText(), FSlateIcon(), FUIAction(FExecuteAction::CreateSP(this, &SFbxSceneStaticMeshReimportListView::AssignToStaticMesh)));
			}
			if (ShowResetAssignToStaticMesh())
			{
				MenuBuilder.AddMenuEntry(LOCTEXT("FbxImportScene_SM_ResetAssign", "Reset to FBX content"), FText(), FSlateIcon(), FUIAction(FExecuteAction::CreateSP(this, &SFbxSceneStaticMeshReimportListView::ResetAssignToStaticMesh)));
			}
		}
		MenuBuilder.EndSection();
	}

	AddBakePivotMenu(MenuBuilder);

	bool bShowOptionMenu = false;
	for (FbxMeshInfoPtr MeshInfo : SelectedFbxMeshInfos)
	{
		EFbxSceneReimportStatusFlags ReimportFlags = *MeshStatusMap->Find(MeshInfo->OriginalImportPath);
		if ((ReimportFlags & EFbxSceneReimportStatusFlags::Removed) == EFbxSceneReimportStatusFlags::None)
		{
			bShowOptionMenu = true;
		}
	}
	if (bShowOptionMenu)
	{
		MenuBuilder.BeginSection("FbxScene_SM_OptionsSection", LOCTEXT("FbxScene_SM_Options", "Options:"));
		{
			for (auto OptionName : (*OverrideNameOptions))
			{
				MenuBuilder.AddMenuEntry(FText::FromString(*OptionName.Get()), FText(), FSlateIcon(), FUIAction(FExecuteAction::CreateSP(this, &SFbxSceneStaticMeshReimportListView::AssignToOptions, *OptionName.Get())));
			}
		}
		MenuBuilder.EndSection();
	}
	return MenuBuilder.MakeWidget();
}

void SFbxSceneStaticMeshReimportListView::AssignToStaticMesh()
{
	TArray<FbxMeshInfoPtr> SelectedFbxMeshInfos;
	int32 SelectCount = GetSelectedItems(SelectedFbxMeshInfos);
	if (SelectCount != 1)
		return;
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	for (FbxMeshInfoPtr ItemPtr : SelectedFbxMeshInfos)
	{
		FOpenAssetDialogConfig SelectAssetConfig;
		SelectAssetConfig.DialogTitleOverride = LOCTEXT("FbxChooseReimportAssetContentPath", "Choose static mesh asset for reimporting the fbx scene content");
		//SelectAssetConfig.DefaultPath = ItemPtr->OriginalImportPath;
		SelectAssetConfig.bAllowMultipleSelection = false;
		SelectAssetConfig.AssetClassNames.Add(UStaticMesh::StaticClass()->GetClassPathName());
		TArray<FAssetData> AssetData = ContentBrowserModule.Get().CreateModalOpenAssetDialog(SelectAssetConfig);
		if (AssetData.Num() == 1)
		{
			EFbxSceneReimportStatusFlags StatusFlag = *(MeshStatusMap->Find(ItemPtr->OriginalImportPath));
			//Remove the old MeshStatusMap
			MeshStatusMap->Remove(ItemPtr->OriginalImportPath);
			
			//Override the MeshInfo with the new asset path
			ItemPtr->SetOverridePath(true);
			ItemPtr->OverrideImportPath = AssetData[0].PackageName.ToString();
			ItemPtr->OverrideFullImportName = AssetData[0].GetObjectPathString();
			StatusFlag |= EFbxSceneReimportStatusFlags::FoundContentBrowserAsset;
			
			//Add the new reimport status
			MeshStatusMap->Add(ItemPtr->OriginalImportPath, StatusFlag);
		}
	}
}

bool SFbxSceneStaticMeshReimportListView::ShowResetAssignToStaticMesh()
{
	TArray<FbxMeshInfoPtr> SelectedFbxMeshInfos;
	int32 SelectCount = GetSelectedItems(SelectedFbxMeshInfos);
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	for (FbxMeshInfoPtr ItemPtr : SelectedFbxMeshInfos)
	{
		if (ItemPtr->bOverridePath)
		{
			return true;
		}
	}
	return false;
}

void SFbxSceneStaticMeshReimportListView::ResetAssignToStaticMesh()
{
	TArray<FbxMeshInfoPtr> SelectedFbxMeshInfos;
	int32 SelectCount = GetSelectedItems(SelectedFbxMeshInfos);

	for (FbxMeshInfoPtr ItemPtr : SelectedFbxMeshInfos)
	{
		if (ItemPtr->bOverridePath)
		{
			ItemPtr->SetOverridePath(false);
			ItemPtr->OverrideImportPath.Empty();
			ItemPtr->OverrideFullImportName.Empty();
			//Get the original status flag
			EFbxSceneReimportStatusFlags StatusFlag = *(MeshStatusMap->Find(ItemPtr->OriginalImportPath));
			MeshStatusMap->Remove(ItemPtr->OriginalImportPath);
			if (ItemPtr->GetContentObject() != nullptr)
			{
				StatusFlag |= EFbxSceneReimportStatusFlags::FoundContentBrowserAsset;
			}
			else
			{
				StatusFlag &= ~EFbxSceneReimportStatusFlags::FoundContentBrowserAsset;
			}
			//Add the new reimport status
			MeshStatusMap->Add(ItemPtr->OriginalImportPath, StatusFlag);
		}
	}
}

void SFbxSceneStaticMeshReimportListView::SetSelectionImportState(bool MarkForImport)
{
	TArray<FbxMeshInfoPtr> SelectedFbxMeshInfos;
	GetSelectedItems(SelectedFbxMeshInfos);
	for (FbxMeshInfoPtr ItemPtr : SelectedFbxMeshInfos)
	{
		EFbxSceneReimportStatusFlags *ItemStatus = MeshStatusMap->Find(ItemPtr->OriginalImportPath);
		if (ItemStatus != nullptr)
		{
			*ItemStatus = MarkForImport ? *ItemStatus | EFbxSceneReimportStatusFlags::ReimportAsset : *ItemStatus & ~EFbxSceneReimportStatusFlags::ReimportAsset;
		}
	}
}

void SFbxSceneStaticMeshReimportListView::OnToggleSelectAll(ECheckBoxState CheckType)
{
	for (FbxMeshInfoPtr MeshInfo : FilterFbxMeshesArray)
	{
		EFbxSceneReimportStatusFlags *ItemStatus = MeshStatusMap->Find(MeshInfo->OriginalImportPath);
		if (ItemStatus != nullptr)
		{
			*ItemStatus = (CheckType == ECheckBoxState::Checked) ? *ItemStatus | EFbxSceneReimportStatusFlags::ReimportAsset : *ItemStatus & ~EFbxSceneReimportStatusFlags::ReimportAsset;
		}
	}
}

void SFbxSceneStaticMeshReimportListView::OnToggleFilterAddContent(ECheckBoxState CheckType)
{
	FilterAddContent = (CheckType == ECheckBoxState::Checked);
	UpdateFilterList();
}

void SFbxSceneStaticMeshReimportListView::OnToggleFilterDeleteContent(ECheckBoxState CheckType)
{
	FilterDeleteContent = (CheckType == ECheckBoxState::Checked);
	UpdateFilterList();
}

void SFbxSceneStaticMeshReimportListView::OnToggleFilterOverwriteContent(ECheckBoxState CheckType)
{
	FilterOverwriteContent = (CheckType == ECheckBoxState::Checked);
	UpdateFilterList();
}

void SFbxSceneStaticMeshReimportListView::OnToggleFilterDiff(ECheckBoxState CheckType)
{
	FilterDiff = (CheckType == ECheckBoxState::Checked);
	UpdateFilterList();
}

void SFbxSceneStaticMeshReimportListView::UpdateFilterList()
{
	FilterFbxMeshesArray.Empty();
	if (FilterAddContent || FilterDeleteContent || FilterOverwriteContent || FilterDiff)
	{
		for (FbxMeshInfoPtr MeshInfo : FbxMeshesArray)
		{
			EFbxSceneReimportStatusFlags ItemStatus = *(MeshStatusMap->Find(MeshInfo->OriginalImportPath));
			bool bStatusAdd = (ItemStatus & EFbxSceneReimportStatusFlags::Added) != EFbxSceneReimportStatusFlags::None;
			bool bStatusSame = (ItemStatus & EFbxSceneReimportStatusFlags::Same) != EFbxSceneReimportStatusFlags::None;
			bool bStatusRemove = (ItemStatus & EFbxSceneReimportStatusFlags::Removed) != EFbxSceneReimportStatusFlags::None;
			bool bStatusFoundContent = (ItemStatus & EFbxSceneReimportStatusFlags::FoundContentBrowserAsset) != EFbxSceneReimportStatusFlags::None;

			if (FilterAddContent && (bStatusAdd || bStatusSame) && !bStatusFoundContent)
			{
				FilterFbxMeshesArray.Add(MeshInfo);
			}
			else if (FilterDeleteContent && bStatusRemove && bStatusFoundContent)
			{
				FilterFbxMeshesArray.Add(MeshInfo);
			}
			else if (FilterOverwriteContent && (bStatusAdd || bStatusSame) && bStatusFoundContent)
			{
				FilterFbxMeshesArray.Add(MeshInfo);
			}
			else if (FilterDiff && !bStatusSame)
			{
				FilterFbxMeshesArray.Add(MeshInfo);
			}
		}
	}
	else
	{
		for (FbxMeshInfoPtr MeshInfo : FbxMeshesArray)
		{
			FilterFbxMeshesArray.Add(MeshInfo);
		}
		
	}
	this->RequestListRefresh();
}

#undef LOCTEXT_NAMESPACE
