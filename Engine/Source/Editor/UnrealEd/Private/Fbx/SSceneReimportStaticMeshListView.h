// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Fbx/SSceneBaseMeshListView.h"

struct FPropertyChangedEvent;

class SFbxSceneStaticMeshReimportListView : public SFbxSSceneBaseMeshListView
{
public:

	~SFbxSceneStaticMeshReimportListView();

	SLATE_BEGIN_ARGS(SFbxSceneStaticMeshReimportListView)
		: _SceneInfo(nullptr)
		, _SceneInfoOriginal(nullptr)
		, _MeshStatusMap(nullptr)
		, _GlobalImportSettings(nullptr)
		, _OverrideNameOptionsMap(nullptr)
		, _SceneImportOptionsStaticMeshDisplay(nullptr)
	{}
		SLATE_ARGUMENT(TSharedPtr<FFbxSceneInfo>, SceneInfo)
		SLATE_ARGUMENT(TSharedPtr<FFbxSceneInfo>, SceneInfoOriginal)
		SLATE_ARGUMENT(FbxSceneReimportStatusMapPtr, MeshStatusMap)
		SLATE_ARGUMENT( UnFbx::FBXImportOptions*, GlobalImportSettings)
		SLATE_ARGUMENT(FbxOverrideNameOptionsArrayPtr, OverrideNameOptions)
		SLATE_ARGUMENT( ImportOptionsNameMapPtr, OverrideNameOptionsMap)
		SLATE_ARGUMENT( class UFbxSceneImportOptionsStaticMesh*, SceneImportOptionsStaticMeshDisplay)
		SLATE_END_ARGS()

	/** Construct this widget */
	
	void Construct(const FArguments& InArgs);
	TSharedRef< ITableRow > OnGenerateRowFbxSceneListView(FbxMeshInfoPtr Item, const TSharedRef< STableViewBase >& OwnerTable);

	void OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent);

	/** Filter show every reimport that will add content */
	void OnToggleFilterAddContent(ECheckBoxState CheckType);
	ECheckBoxState IsFilterAddContentChecked() const
	{
		return FilterAddContent ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	/** Filter show every reimport that will delete content */
	void OnToggleFilterDeleteContent(ECheckBoxState CheckType);
	ECheckBoxState IsFilterDeleteContentChecked() const
	{
		return FilterDeleteContent ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	/** Filter show every reimport that will override content */
	void OnToggleFilterOverwriteContent(ECheckBoxState CheckType);
	ECheckBoxState IsFilterOverwriteContentChecked() const
	{
		return FilterOverwriteContent ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	/** Filter show every reimport that dont match between the original fbx and the new one */
	void OnToggleFilterDiff(ECheckBoxState CheckType);
	ECheckBoxState IsFilterDiffChecked() const
	{
		return FilterDiff ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

protected:
	TSharedPtr<FFbxSceneInfo> SceneInfoOriginal;
	class UFbxSceneImportOptionsStaticMesh* SceneImportOptionsStaticMeshDisplay;

	/** the elements we show in the list view */
	TArray<FbxMeshInfoPtr> FilterFbxMeshesArray;
	bool FilterAddContent;
	bool FilterDeleteContent;
	bool FilterOverwriteContent;
	bool FilterDiff;

	void UpdateFilterList();

	FbxSceneReimportStatusMapPtr MeshStatusMap;
	
	virtual void SetSelectionImportState(bool MarkForImport);
	virtual void OnToggleSelectAll(ECheckBoxState CheckType);

	/** Open a context menu for the current selection */
	TSharedPtr<SWidget> OnOpenContextMenu();
	void AssignToStaticMesh();
	bool ShowResetAssignToStaticMesh();
	void ResetAssignToStaticMesh();
	virtual void OnChangedOverrideOptions(TSharedPtr<FString> ItemSelected, ESelectInfo::Type SelectInfo);
};
