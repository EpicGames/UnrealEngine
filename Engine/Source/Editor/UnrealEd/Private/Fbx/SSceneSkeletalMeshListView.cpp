// Copyright Epic Games, Inc. All Rights Reserved.

#include "Fbx/SSceneSkeletalMeshListView.h"
#include "Widgets/SOverlay.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Images/SImage.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SCheckBox.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateIconFinder.h"
#include "SFbxSceneOptionWindow.h"
#include "FbxImporter.h"

#define LOCTEXT_NAMESPACE "SFbxSceneSkeletalMeshListView"

namespace FbxSceneImportSkeletalMesh
{
	const FName SceneImportCheckBoxSelectionHeaderIdName(TEXT("CheckBoxSelectionHeaderId"));
	const FName SceneImportClassIconHeaderIdName(TEXT("ClassIconHeaderId"));
	const FName SceneImportAssetNameHeaderIdName(TEXT("AssetNameHeaderId"));
	const FName SceneImportContentPathHeaderIdName(TEXT("ContentPathHeaderId"));
	const FName SceneImportOptionsNameHeaderIdName(TEXT("OptionsNameHeaderId"));
}

/** The item used for visualizing the class in the tree. */
class SFbxSleletalItemTableListViewRow : public SMultiColumnTableRow<FbxMeshInfoPtr>
{
public:

	SLATE_BEGIN_ARGS(SFbxSleletalItemTableListViewRow)
		: _FbxMeshInfo(nullptr)
		, _GlobalImportSettings(nullptr)
	{}

	/** The item content. */
	SLATE_ARGUMENT(FbxMeshInfoPtr, FbxMeshInfo)
	SLATE_ARGUMENT(UnFbx::FBXImportOptions*, GlobalImportSettings)
	SLATE_END_ARGS()

	/**
	* Construct the widget
	*
	* @param InArgs   A declaration from which to construct the widget
	*/
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		FbxMeshInfo = InArgs._FbxMeshInfo;
		GlobalImportSettings = InArgs._GlobalImportSettings;

		//This is suppose to always be valid
		check(FbxMeshInfo.IsValid());
		check(GlobalImportSettings != nullptr);

		SMultiColumnTableRow<FbxMeshInfoPtr>::Construct(
			FSuperRowType::FArguments()
			.Style(FAppStyle::Get(), "DataTableEditor.CellListViewRow"),
			InOwnerTableView
			);
	}

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the list view. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == FbxSceneImportSkeletalMesh::SceneImportCheckBoxSelectionHeaderIdName)
		{
			return SNew(SBox)
				.HAlign(HAlign_Center)
				[
					SNew(SCheckBox)
					.OnCheckStateChanged(this, &SFbxSleletalItemTableListViewRow::OnItemCheckChanged)
					.IsChecked(this, &SFbxSleletalItemTableListViewRow::IsItemChecked)
				];
		}
		else if (ColumnName == FbxSceneImportSkeletalMesh::SceneImportClassIconHeaderIdName)
		{
			UClass *IconClass = FbxMeshInfo->GetType();
			const FSlateBrush* ClassIcon = FSlateIconFinder::FindIconBrushForClass(IconClass);

			TSharedRef<SOverlay> IconContent = SNew(SOverlay)
				+ SOverlay::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(ClassIcon)
				];


			// Add the component mobility icon
			IconContent->AddSlot()
				.HAlign(HAlign_Left)
				[
					SNew(SImage)
					.Image(this, &SFbxSleletalItemTableListViewRow::GetBrushForOverrideIcon)
				];
			return IconContent;
		}
		else if (ColumnName == FbxSceneImportSkeletalMesh::SceneImportAssetNameHeaderIdName)
		{
			return SNew(STextBlock)
				.Text(FText::FromString(FbxMeshInfo->Name))
				.ToolTipText(FText::FromString(FbxMeshInfo->Name));
		}
		else if (ColumnName == FbxSceneImportSkeletalMesh::SceneImportContentPathHeaderIdName)
		{
			return SNew(STextBlock)
				.Text(this, &SFbxSleletalItemTableListViewRow::GetAssetFullName)
				.ColorAndOpacity(this, &SFbxSleletalItemTableListViewRow::GetContentPathTextColor)
				.ToolTipText(this, &SFbxSleletalItemTableListViewRow::GetAssetFullName);
		}
		else if (ColumnName == FbxSceneImportSkeletalMesh::SceneImportOptionsNameHeaderIdName)
		{
			return SNew(STextBlock)
				.Text(this, &SFbxSleletalItemTableListViewRow::GetAssetOptionName)
				.ToolTipText(this, &SFbxSleletalItemTableListViewRow::GetAssetOptionName);
		}
		else if (ColumnName == FbxSceneBaseListViewColumn::PivotColumnId)
		{
			return SNew(STextBlock)
				.Text(this, &SFbxSleletalItemTableListViewRow::GetAssetPivotNodeName)
				.ToolTipText(this, &SFbxSleletalItemTableListViewRow::GetAssetPivotNodeName);
		}

		return SNullWidget::NullWidget;
	}

	const FSlateBrush* GetBrushForOverrideIcon() const
	{
		if (UFbxSceneImportFactory::DefaultOptionName.Compare(FbxMeshInfo->OptionName) != 0)
		{
			return FAppStyle::GetBrush("FBXIcon.ImportOptionsOverride");
		}
		return FAppStyle::GetBrush("FBXIcon.ImportOptionsDefault");
	}

private:

	void OnItemCheckChanged(ECheckBoxState CheckType)
	{
		if (!FbxMeshInfo.IsValid())
			return;
		FbxMeshInfo->bImportAttribute = CheckType == ECheckBoxState::Checked;
	}

	ECheckBoxState IsItemChecked() const
	{
		return FbxMeshInfo->bImportAttribute ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	FSlateColor GetContentPathTextColor() const
	{
		return FbxMeshInfo->bOverridePath ? FLinearColor(0.75f, 0.75f, 0.0f, 1.0f) : FSlateColor::UseForeground();
	}

	FText GetAssetFullName() const
	{
		return FText::FromString(FbxMeshInfo->GetFullImportName());
	}

	FText GetAssetOptionName() const
	{
			return FText::FromString(FbxMeshInfo->OptionName);
	}

	FText GetAssetPivotNodeName() const
	{
		return GlobalImportSettings->bBakePivotInVertex ? FText::FromString(FbxMeshInfo->PivotNodeName) : FText::FromString(TEXT("-"));
	}

	/** The node info to build the tree view row from. */
	FbxMeshInfoPtr FbxMeshInfo;
	UnFbx::FBXImportOptions *GlobalImportSettings;
};

//////////////////////////////////////////////////////////////////////////
// Skeletal Mesh List

SFbxSceneSkeletalMeshListView::~SFbxSceneSkeletalMeshListView()
{
	SceneInfo = nullptr;
	GlobalImportSettings = nullptr;
	SceneImportOptionsSkeletalMeshDisplay = nullptr;
	CurrentMeshImportOptions = nullptr;
	FbxMeshesArray.Empty();
	SceneImportOptionsSkeletalMeshDisplay = nullptr;
	OverrideNameOptions = nullptr;
	OverrideNameOptionsMap = nullptr;
	OptionComboBox = nullptr;
	DefaultOptionNamePtr = nullptr;
}

void SFbxSceneSkeletalMeshListView::Construct(const SFbxSceneSkeletalMeshListView::FArguments& InArgs)
{
	SceneInfo = InArgs._SceneInfo;
	GlobalImportSettings = InArgs._GlobalImportSettings;
	OverrideNameOptions = InArgs._OverrideNameOptions;
	OverrideNameOptionsMap = InArgs._OverrideNameOptionsMap;
	SceneImportOptionsSkeletalMeshDisplay = InArgs._SceneImportOptionsSkeletalMeshDisplay;

	check(SceneInfo.IsValid());
	check(GlobalImportSettings != nullptr);
	check(OverrideNameOptions != nullptr);
	check(OverrideNameOptionsMap != nullptr);
	check(SceneImportOptionsSkeletalMeshDisplay != nullptr);

	SFbxSceneOptionWindow::CopySkeletalMeshOptionsToFbxOptions(GlobalImportSettings, SceneImportOptionsSkeletalMeshDisplay);
	//Set the default options to the current global import settings
	GlobalImportSettings->bTransformVertexToAbsolute = false;
	GlobalImportSettings->StaticMeshLODGroup = NAME_None;
	CurrentMeshImportOptions = GlobalImportSettings;
	
	bool bNameExist = false;
	for (TSharedPtr<FString> OverrideName : (*OverrideNameOptions))
	{
		if (OverrideName.Get()->Compare(UFbxSceneImportFactory::DefaultOptionName) == 0)
		{
			DefaultOptionNamePtr = OverrideName;
			bNameExist = true;
			break;
		}
	}

	if (!bNameExist)
	{
		DefaultOptionNamePtr = MakeShareable(new FString(UFbxSceneImportFactory::DefaultOptionName));
		OverrideNameOptions->Add(DefaultOptionNamePtr);
		OverrideNameOptionsMap->Add(UFbxSceneImportFactory::DefaultOptionName, GlobalImportSettings);
	}
	
	for (auto MeshInfoIt = SceneInfo->MeshInfo.CreateIterator(); MeshInfoIt; ++MeshInfoIt)
	{
		FbxMeshInfoPtr MeshInfo = (*MeshInfoIt);

		if (MeshInfo->bIsSkelMesh && !MeshInfo->IsLod && !MeshInfo->IsCollision)
		{
			MeshInfo->OptionName = UFbxSceneImportFactory::DefaultOptionName;
			FbxMeshesArray.Add(MeshInfo);
		}
	}

	SListView::Construct
	(
		SListView::FArguments()
		.ListItemsSource(&FbxMeshesArray)
		.SelectionMode(ESelectionMode::Multi)
		.OnGenerateRow(this, &SFbxSceneSkeletalMeshListView::OnGenerateRowFbxSceneListView)
		.OnContextMenuOpening(this, &SFbxSceneSkeletalMeshListView::OnOpenContextMenu)
		.OnSelectionChanged(this, &SFbxSSceneBaseMeshListView::OnSelectionChanged)
		.HeaderRow
		(
			SNew(SHeaderRow)
				
			+ SHeaderRow::Column(FbxSceneImportSkeletalMesh::SceneImportCheckBoxSelectionHeaderIdName)
			.FixedWidth(25)
			.DefaultLabel(FText::GetEmpty())
			[
				SNew(SCheckBox)
				.HAlign(HAlign_Center)
				.OnCheckStateChanged(this, &SFbxSceneSkeletalMeshListView::OnToggleSelectAll)
			]
				
			+ SHeaderRow::Column(FbxSceneImportSkeletalMesh::SceneImportClassIconHeaderIdName)
			.FixedWidth(20)
			.DefaultLabel(FText::GetEmpty())
				
			+ SHeaderRow::Column(FbxSceneImportSkeletalMesh::SceneImportAssetNameHeaderIdName)
			.FillWidth(300)
			.HAlignCell(EHorizontalAlignment::HAlign_Left)
			.DefaultLabel(LOCTEXT("AssetNameHeaderName", "Asset Name"))

			+ SHeaderRow::Column(FbxSceneImportSkeletalMesh::SceneImportOptionsNameHeaderIdName)
			.FillWidth(300)
			.HAlignCell(EHorizontalAlignment::HAlign_Left)
			.DefaultLabel(LOCTEXT("OptionsNameHeaderName", "Options Name"))

/*			+ SHeaderRow::Column(SceneImportContentPathHeaderIdName)
			.FillWidth(300)
			.HAlignCell(EHorizontalAlignment::HAlign_Left)
			.DefaultLabel(LOCTEXT("ContentPathHeaderName", "Content Path"))*/
		)
	);
}

TSharedRef< ITableRow > SFbxSceneSkeletalMeshListView::OnGenerateRowFbxSceneListView(FbxMeshInfoPtr Item, const TSharedRef< STableViewBase >& OwnerTable)
{
	TSharedRef< SFbxSleletalItemTableListViewRow > ReturnRow = SNew(SFbxSleletalItemTableListViewRow, OwnerTable)
		.FbxMeshInfo(Item)
		.GlobalImportSettings(GlobalImportSettings);
	return ReturnRow;
}

TSharedPtr<SWidget> SFbxSceneSkeletalMeshListView::OnOpenContextMenu()
{
	TArray<FbxMeshInfoPtr> SelectedFbxMeshInfos;
	int32 SelectCount = GetSelectedItems(SelectedFbxMeshInfos);
	// Build up the menu for a selection
	const bool bCloseAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, TSharedPtr<FUICommandList>());

	const FSlateIcon PlusIcon(FAppStyle::GetAppStyleSetName(), "Plus");
	MenuBuilder.AddMenuEntry(LOCTEXT("CheckForImport", "Add Selection To Import"), FText(), PlusIcon, FUIAction(FExecuteAction::CreateSP(this, &SFbxSceneSkeletalMeshListView::AddSelectionToImport)));
	const FSlateIcon MinusIcon(FAppStyle::GetAppStyleSetName(), "Icons.Minus");
	MenuBuilder.AddMenuEntry(LOCTEXT("UncheckForImport", "Remove Selection From Import"), FText(), MinusIcon, FUIAction(FExecuteAction::CreateSP(this, &SFbxSceneSkeletalMeshListView::RemoveSelectionFromImport)));

	AddBakePivotMenu(MenuBuilder);

	if (OverrideNameOptions->Num() > 0)
	{
		MenuBuilder.BeginSection("FbxScene_SM_OptionsSection", LOCTEXT("FbxScene_SM_Options", "Options:"));
		{
			for (auto OptionName : (*OverrideNameOptions))
			{
				MenuBuilder.AddMenuEntry(FText::FromString(*OptionName.Get()), FText(), FSlateIcon(), FUIAction(FExecuteAction::CreateSP(this, &SFbxSceneSkeletalMeshListView::AssignToOptions, *OptionName.Get())));
			}
		}
		MenuBuilder.EndSection();
	}

	return MenuBuilder.MakeWidget();
}

void SFbxSceneSkeletalMeshListView::OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent)
{
	SFbxSceneOptionWindow::CopySkeletalMeshOptionsToFbxOptions(CurrentMeshImportOptions, SceneImportOptionsSkeletalMeshDisplay);
}

void SFbxSceneSkeletalMeshListView::OnChangedOverrideOptions(TSharedPtr<FString> ItemSelected, ESelectInfo::Type SelectInfo)
{
	if (ItemSelected.IsValid())
	{
		if (ItemSelected.Get()->Compare(UFbxSceneImportFactory::DefaultOptionName) == 0)
		{
			CurrentMeshImportOptions = GlobalImportSettings;
		}
		else if (OverrideNameOptionsMap->Contains(*(ItemSelected.Get())))
		{
			CurrentMeshImportOptions = *OverrideNameOptionsMap->Find(*(ItemSelected.Get()));
		}
		SFbxSceneOptionWindow::CopyFbxOptionsToSkeletalMeshOptions(CurrentMeshImportOptions, SceneImportOptionsSkeletalMeshDisplay);
	}
}

#undef LOCTEXT_NAMESPACE
