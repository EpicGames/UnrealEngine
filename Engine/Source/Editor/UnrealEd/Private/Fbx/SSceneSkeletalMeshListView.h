// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Fbx/SSceneBaseMeshListView.h"

struct FPropertyChangedEvent;

class SFbxSceneSkeletalMeshListView : public SFbxSSceneBaseMeshListView
{
public:
	
	~SFbxSceneSkeletalMeshListView();

	SLATE_BEGIN_ARGS(SFbxSceneSkeletalMeshListView)
	: _SceneInfo(nullptr)
	, _GlobalImportSettings(nullptr)
	, _OverrideNameOptionsMap(nullptr)
	, _SceneImportOptionsSkeletalMeshDisplay(nullptr)
	{}
		SLATE_ARGUMENT(TSharedPtr<FFbxSceneInfo>, SceneInfo)
		SLATE_ARGUMENT(UnFbx::FBXImportOptions*, GlobalImportSettings)
		SLATE_ARGUMENT(FbxOverrideNameOptionsArrayPtr, OverrideNameOptions)
		SLATE_ARGUMENT(ImportOptionsNameMapPtr, OverrideNameOptionsMap)
		SLATE_ARGUMENT(class UFbxSceneImportOptionsSkeletalMesh*, SceneImportOptionsSkeletalMeshDisplay)
	SLATE_END_ARGS()
	
		/** Construct this widget */
	void Construct(const FArguments& InArgs);
	TSharedRef< ITableRow > OnGenerateRowFbxSceneListView(FbxMeshInfoPtr Item, const TSharedRef< STableViewBase >& OwnerTable);
	
	void OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent);
protected:
	
	class UFbxSceneImportOptionsSkeletalMesh *SceneImportOptionsSkeletalMeshDisplay;

	virtual void OnChangedOverrideOptions(TSharedPtr<FString> ItemSelected, ESelectInfo::Type SelectInfo);
	/** Open a context menu for the current selection */
	TSharedPtr<SWidget> OnOpenContextMenu();
};
