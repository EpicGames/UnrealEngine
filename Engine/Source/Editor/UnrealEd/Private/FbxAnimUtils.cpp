// Copyright Epic Games, Inc. All Rights Reserved.

#include "FbxAnimUtils.h"

#include "Animation/AnimationSettings.h"
#include "Animation/AnimTypes.h"
#include "Curves/RichCurve.h"
#include "EditorDirectories.h"
#include "Engine/CurveTable.h"
#include "Exporters/FbxExportOption.h"
#include "FbxExporter.h"
#include "Misc/Paths.h"

namespace FbxAnimUtils
{
	void ExportAnimFbx( const FString& ExportFilename, UAnimSequence* AnimSequence, USkeletalMesh* Mesh, bool BatchMode, bool &OutExportAll, bool &OutCancelExport )
	{
		if( !ExportFilename.IsEmpty() && AnimSequence && Mesh )
		{
			FString FileName = ExportFilename;
			FEditorDirectories::Get().SetLastDirectory(ELastDirectory::FBX_ANIM, FPaths::GetPath(FileName)); // Save path as default for next time.

			UnFbx::FFbxExporter* Exporter = UnFbx::FFbxExporter::GetInstance();
			//Show the fbx export dialog options
			Exporter->FillExportOptions(BatchMode, (!BatchMode || !OutExportAll), ExportFilename, OutCancelExport, OutExportAll);
			if (!OutCancelExport)
			{
				Exporter->CreateDocument();

				Exporter->ExportAnimSequence(AnimSequence, Mesh, Exporter->GetExportOptions()->bExportPreviewMesh);

				// Save to disk
				Exporter->WriteToFile(*ExportFilename);
			}
		}
	}

	static FbxNode* FindCurveNodeRecursive(FbxNode* NodeToQuery, const FString& InCurveNodeName, FbxNodeAttribute::EType InNodeType)
	{
		if (InCurveNodeName == UTF8_TO_TCHAR(NodeToQuery->GetName()) && NodeToQuery->GetNodeAttribute() && NodeToQuery->GetNodeAttribute()->GetAttributeType() == InNodeType)
		{
			return NodeToQuery;
		}

		const int32 NodeCount = NodeToQuery->GetChildCount();
		for (int32 NodeIndex = 0; NodeIndex < NodeCount; ++NodeIndex)
		{
			FbxNode* FoundNode = FindCurveNodeRecursive(NodeToQuery->GetChild(NodeIndex), InCurveNodeName, InNodeType);
			if (FoundNode != nullptr)
			{
				return FoundNode;
			}
		}

		return nullptr;
	}

	static FbxNode* FindCurveNode(UnFbx::FFbxImporter* FbxImporter, const FString& InCurveNodeName, FbxNodeAttribute::EType InNodeType)
	{
		FbxNode *RootNode = FbxImporter->Scene->GetRootNode();
		return FindCurveNodeRecursive(RootNode, InCurveNodeName, InNodeType);
	}

	bool IsSupportedCurveDataType(int32 InDataType)
	{
		switch (InDataType)
		{
		case eFbxShort:		//!< 16 bit signed integer.
		case eFbxUShort:	//!< 16 bit unsigned integer.
		case eFbxUInt:		//!< 32 bit unsigned integer.
		case eFbxHalfFloat:	//!< 16 bit floating point.
		case eFbxInt:		//!< 32 bit signed integer.
		case eFbxFloat:		//!< Floating point value.
		case eFbxDouble:	//!< Double width floating point value.
		case eFbxDouble2:	//!< Vector of two double values.
		case eFbxDouble3:	//!< Vector of three double values.
		case eFbxDouble4:	//!< Vector of four double values.
		case eFbxDouble4x4:	//!< Four vectors of four double values.
			return true;
		}

		return false;
	}

	bool ShouldImportCurve(FbxAnimCurve* Curve, bool bDoNotImportWithZeroValues)
	{
		if (Curve && Curve->KeyGetCount() > 0)
		{
			if (bDoNotImportWithZeroValues)
			{
				for (int32 KeyIndex = 0; KeyIndex < Curve->KeyGetCount(); ++KeyIndex)
				{
					if (!FMath::IsNearlyZero(Curve->KeyGetValue(KeyIndex)))
					{
						return true;
					}
				}
			}
			else
			{
				return true;
			}
		}

		return false;
	}

	bool ImportCurveTableFromNode(const FString& InFbxFilename, const FString& InCurveNodeName, UCurveTable* InOutCurveTable, float& OutPreRoll)
	{
		UnFbx::FFbxImporter* FbxImporter = UnFbx::FFbxImporter::GetInstance();

		const FString FileExtension = FPaths::GetExtension(InFbxFilename);
		if (FbxImporter->ImportFromFile(*InFbxFilename, FileExtension, true))
		{
			if (FbxImporter->Scene != nullptr)
			{
				// merge animation layer at first (@TODO: do I need to do this?)
				FbxAnimStack* AnimStack = FbxImporter->Scene->GetMember<FbxAnimStack>(0);
				if (AnimStack != nullptr)
				{
					FbxTimeSpan AnimTimeSpan = FbxImporter->GetAnimationTimeSpan(FbxImporter->Scene->GetRootNode(), AnimStack);

					// Grab the start time, as we might have a preroll
					OutPreRoll = FMath::Abs((float)AnimTimeSpan.GetStart().GetSecondDouble());

					// @TODO: do I need this check?
					FbxAnimLayer* AnimLayer = AnimStack->GetMember<FbxAnimLayer>(0);
					if (AnimLayer != nullptr)
					{
						// Try the legacy path with curves in blendshapes on a dummy mesh
						if (FbxNode* MeshNode = FindCurveNode(FbxImporter, InCurveNodeName, FbxNodeAttribute::eMesh))
						{
							// We have a node, so clear the curve table
							InOutCurveTable->EmptyTable();

							FbxGeometry* Geometry = (FbxGeometry*)MeshNode->GetNodeAttribute();
							int32 BlendShapeDeformerCount = Geometry->GetDeformerCount(FbxDeformer::eBlendShape);
							for (int32 BlendShapeIndex = 0; BlendShapeIndex < BlendShapeDeformerCount; ++BlendShapeIndex)
							{
								FbxBlendShape* BlendShape = (FbxBlendShape*)Geometry->GetDeformer(BlendShapeIndex, FbxDeformer::eBlendShape);

								const int32 BlendShapeChannelCount = BlendShape->GetBlendShapeChannelCount();

								FString BlendShapeName = FbxImporter->MakeName(BlendShape->GetName());

								for (int32 ChannelIndex = 0; ChannelIndex < BlendShapeChannelCount; ++ChannelIndex)
								{
									FbxBlendShapeChannel* Channel = BlendShape->GetBlendShapeChannel(ChannelIndex);

									if (Channel)
									{
										FString ChannelName = FbxImporter->MakeName(Channel->GetName());

										// Maya adds the name of the blendshape and an underscore to the front of the channel name, so remove it
										if (ChannelName.StartsWith(BlendShapeName))
										{
											ChannelName.RightInline(ChannelName.Len() - (BlendShapeName.Len() + 1), EAllowShrinking::No);
										}

										FbxAnimCurve* Curve = Geometry->GetShapeChannel(BlendShapeIndex, ChannelIndex, (FbxAnimLayer*)AnimStack->GetMember(0));
										if (Curve)
										{
											FRichCurve& RichCurve = InOutCurveTable->AddRichCurve(*ChannelName);
											RichCurve.Reset();
											constexpr bool bNegative = false;
											FbxImporter->ImportCurve(Curve, RichCurve, AnimTimeSpan, bNegative, 0.01f);
										}
									}
								}
							}
							return true;
						}
						// Now look for attribute curves on a skeleton node
						else if (FbxNode* SkeletonNode = FindCurveNode(FbxImporter, InCurveNodeName, FbxNodeAttribute::eSkeleton))
						{
							// We have a node, so clear the curve table
							InOutCurveTable->EmptyTable();

							ExtractAttributeCurves(SkeletonNode, false, [&InOutCurveTable, &AnimTimeSpan, &FbxImporter](FbxAnimCurve* InCurve, const FString& InCurveName)
							{
								FRichCurve& RichCurve = InOutCurveTable->AddRichCurve(*InCurveName);
								RichCurve.Reset();
								constexpr bool bNegative = false;
								FbxImporter->ImportCurve(InCurve, RichCurve, AnimTimeSpan, bNegative, 1.0f);
							});
							return true;
						}
					}
				}
			}
		}

		FbxImporter->ReleaseScene();

		return false;
	}

	bool DefaultAttributeFilter(const FbxProperty& Property, const FbxAnimCurveNode* CurveNode)
	{
		return CurveNode && Property.GetFlag(FbxPropertyFlags::eUserDefined) &&
			CurveNode->IsAnimated() && FbxAnimUtils::IsSupportedCurveDataType(Property.GetPropertyDataType().GetType());
	}

	FString GetCurveName(const FString& InCurveName, const FString& InChannelName, int32 InNumChannels)
	{
		FString FinalCurveName;
		if (InNumChannels == 1)
		{
			FinalCurveName = InCurveName;
		}
		else
		{
			FinalCurveName = InCurveName + "_" + InChannelName;
		}

		return FinalCurveName;
	}

	void ExtractAttributeCurves(FbxNode* InNode, bool bInDoNotImportCurveWithZero, TFunctionRef<void(FbxAnimCurve* /*InCurve*/, const FString& /*InCurveName*/)> ImportFunction)
	{
		FbxProperty Property = InNode->GetFirstProperty();
		UAnimationSettings* AnimationSettings = UAnimationSettings::Get();
		const TArray<FString> CustomAttributeNamesToImport = AnimationSettings->GetBoneCustomAttributeNamesToImport();

		while (Property.IsValid())
		{
			FbxAnimCurveNode* CurveNode = Property.GetCurveNode();
			FString PropertyName = UTF8_TO_TCHAR(Property.GetName());
			// do this if user defined and animated

			if (CurveNode && Property.GetFlag(FbxPropertyFlags::eUserDefined) &&
				CurveNode->IsAnimated() && FbxAnimUtils::IsSupportedCurveDataType(Property.GetPropertyDataType().GetType()) &&
				!CustomAttributeNamesToImport.Contains(PropertyName))
			{
				FString CurveName = UTF8_TO_TCHAR(CurveNode->GetName());
				UE_LOGF(LogFbx, Log, "CurveName : %ls", *CurveName);

				int32 ChannelCount = CurveNode->GetChannelsCount();
				for (int32 ChannelIndex = 0; ChannelIndex < ChannelCount; ++ChannelIndex)
				{
					FbxAnimCurve* AnimCurve = CurveNode->GetCurve(ChannelIndex);
					FString ChannelName = CurveNode->GetChannelName(ChannelIndex).Buffer();

					if (FbxAnimUtils::ShouldImportCurve(AnimCurve, bInDoNotImportCurveWithZero))
					{
						FString FinalCurveName = GetCurveName(CurveName, ChannelName, ChannelCount);

						ImportFunction(AnimCurve, FinalCurveName);
					}
					else
					{
						UE_LOGF(LogFbx, Log, "CurveName(%ls) is skipped because it only contains invalid values.", *CurveName);
					}
				}
			}

			Property = InNode->GetNextProperty(Property);
		}
	}

	void ExtractNodeAttributes(fbxsdk::FbxNode* InNode, bool bInDoNotImportCurveWithZero, bool bImportAllCustomAttributes, TFunctionRef<void(fbxsdk::FbxProperty& /*InProperty*/, fbxsdk::FbxAnimCurve* /*InCurve*/, const FString& /*InCurveName*/)> ImportFunction)
	{
		FbxProperty Property = InNode->GetFirstProperty();
		UAnimationSettings* AnimationSettings = UAnimationSettings::Get();
		const TArray<FString> CustomAttributeNamesToImport = AnimationSettings->GetBoneCustomAttributeNamesToImport();

		while (Property.IsValid())
		{
			FbxAnimCurveNode* CurveNode = Property.GetCurveNode();
			const FString PropertyName = UTF8_TO_TCHAR(Property.GetName());
			const EFbxType PropertyType = Property.GetPropertyDataType().GetType();
			const bool bIsTypeSupported = FbxAnimUtils::IsSupportedCurveDataType(PropertyType) || PropertyType == eFbxString || PropertyType == eFbxEnum;

			if (Property.GetFlag(FbxPropertyFlags::eUserDefined) && bIsTypeSupported 
				&& (bImportAllCustomAttributes || CustomAttributeNamesToImport.Contains(PropertyName)))
			{
				UE_LOGF(LogFbx, Log, "PropertyName : %ls", *PropertyName);

				if (CurveNode && CurveNode->IsAnimated())
				{
					int32 ChannelCount = CurveNode->GetChannelsCount();
					for (int32 ChannelIndex = 0; ChannelIndex < ChannelCount; ++ChannelIndex)
					{
						FbxAnimCurve* AnimCurve = CurveNode->GetCurve(ChannelIndex);
						FString ChannelName = CurveNode->GetChannelName(ChannelIndex).Buffer();

						if (FbxAnimUtils::ShouldImportCurve(AnimCurve, bInDoNotImportCurveWithZero))
						{
							FString FinalCurveName = GetCurveName(PropertyName, ChannelName, ChannelCount);

							ImportFunction(Property, AnimCurve, FinalCurveName);
						}
						else
						{
							UE_LOGF(LogFbx, Log, "CurveName(%ls) is skipped because it only contains invalid values.", *PropertyName);
						}
					}
				}
				else
				{
					// The value of the attribute is constant, we should add the property value directly at Key 0.
					ImportFunction(Property, nullptr, PropertyName);
				}
			}

			Property = InNode->GetNextProperty(Property);
		}
	}
}
