// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/SNullWidget.h"
#include "Styling/AppStyle.h"

enum EFBXCompareSection
{
	EFBXCompareSection_Skeleton = 0,
	EFBXCompareSection_References,
	EFBXCompareSection_Count
};

class FCompMaterial
{
public:
	FCompMaterial(FName InMaterialSlotName, FName InImportedMaterialSlotName)
		: MaterialSlotName(InMaterialSlotName)
		, ImportedMaterialSlotName(InImportedMaterialSlotName)
	{}
	FName MaterialSlotName;
	FName ImportedMaterialSlotName;
};

class FCompJoint
{
public:
	FCompJoint()
		: Name(NAME_None)
		, ParentIndex(INDEX_NONE)
	{ }

	~FCompJoint()
	{
		ChildIndexes.Empty();
	}

	FName Name;
	int32 ParentIndex;
	TArray<int32> ChildIndexes;
};

class FCompSkeleton
{
public:
	FCompSkeleton()
		: bSkeletonFitMesh(true)
	{}

	~FCompSkeleton()
	{
		Joints.Empty();
	}

	TArray<FCompJoint> Joints;
	bool bSkeletonFitMesh;
};

class FCompMesh
{
public:
	~FCompMesh() = default;

	FCompSkeleton CompSkeleton;

	TArray<FString> ErrorMessages;
	TArray<FString> WarningMessages;
};

class FSkeletonCompareData : public TSharedFromThis<FSkeletonCompareData>
{
public:
	FSkeletonCompareData()
		: CurrentJointIndex(INDEX_NONE)
		, FbxJointIndex(INDEX_NONE)
		, JointName(NAME_None)
		, ParentJoint(nullptr)
		, bMatchJoint(false)
		, bChildConflict(false)
		, bInitialAutoExpand(false)
	{}
	int32 CurrentJointIndex;
	int32 FbxJointIndex;
	FName JointName;
	TSharedPtr<FSkeletonCompareData> ParentJoint;
	bool bMatchJoint;
	bool bChildConflict;
	bool bInitialAutoExpand;
	TArray<int32> ChildJointIndexes;
	TArray<TSharedPtr<FSkeletonCompareData>> ChildJoints;
};

class FCompareRowData : public TSharedFromThis<FCompareRowData>
{
public:
	FCompareRowData()
		: RowIndex(INDEX_NONE)
		, CurrentData(nullptr)
		, FbxData(nullptr)
	{}
	virtual ~FCompareRowData() = default;

	int32 RowIndex;
	FCompMesh *CurrentData;
	FCompMesh *FbxData;

	virtual TSharedRef<SWidget> ConstructCellCurrent()
	{
		return SNullWidget::NullWidget;
	};
	virtual TSharedRef<SWidget> ConstructCellFbx()
	{
		return SNullWidget::NullWidget;
	};
};


class SCompareRowDataTableListViewRow : public SMultiColumnTableRow<TSharedPtr<FCompareRowData>>
{
public:

	SLATE_BEGIN_ARGS(SCompareRowDataTableListViewRow)
		: _CompareRowData(nullptr)
	{}

	/** The item content. */
	SLATE_ARGUMENT(TSharedPtr<FCompareRowData>, CompareRowData)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		CompareRowData = InArgs._CompareRowData;

		//This is suppose to always be valid
		check(CompareRowData.IsValid());

		SMultiColumnTableRow<TSharedPtr<FCompareRowData>>::Construct(
			FSuperRowType::FArguments()
			.Style(FAppStyle::Get(), "DataTableEditor.CellListViewRow"),
			InOwnerTableView
			);
	}

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the list view. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == FName(TEXT("RowIndex")))
		{
			return SNew(SBox)
				.Padding(FMargin(5.0f, 2.0f, 0.0f, 2.0f))
				[
					SNew(STextBlock)
					.Text(FText::FromString(FString::FromInt(CompareRowData->RowIndex)))
				];
		}
		else if (ColumnName == FName(TEXT("Current")))
		{
			return CompareRowData->ConstructCellCurrent();
		}
		else if (ColumnName == FName(TEXT("Fbx")))
		{
			return CompareRowData->ConstructCellFbx();
		}
		return SNullWidget::NullWidget;
	}
private:

	/** The node info to build the tree view row from. */
	TSharedPtr<FCompareRowData> CompareRowData;
};


/*
 * This dialog show the conflict between different skeleton
 */
class SFbxSkeltonConflictWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SFbxSkeltonConflictWindow)
		: _WidgetWindow()
		, _AssetReferencingSkeleton(nullptr)
		, _SourceData(nullptr)
		, _ResultData(nullptr)
		, _SourceObject(nullptr)
		, _bIsPreviewConflict(false)
		{}

		SLATE_ARGUMENT( TSharedPtr<SWindow>, WidgetWindow )
		SLATE_ARGUMENT( TArray<TSharedPtr<FString>>*, AssetReferencingSkeleton)
		SLATE_ARGUMENT( FCompMesh*, SourceData)
		SLATE_ARGUMENT( FCompMesh*, ResultData)
		SLATE_ARGUMENT( UObject*, SourceObject )
		SLATE_ARGUMENT( bool, bIsPreviewConflict )
			
	SLATE_END_ARGS()

public:
	bool HasConflict();
	bool bRevertReimport;
	void Construct(const FArguments& InArgs);
	virtual bool SupportsKeyboardFocus() const override { return true; }

	FReply OnDone()
	{
		if ( WidgetWindow.IsValid() )
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		bRevertReimport = false;
		return FReply::Handled();
	}


	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override
	{
		if( InKeyEvent.GetKey() == EKeys::Escape )
		{
			return OnDone();
		}
		return FReply::Unhandled();
	}

	SFbxSkeltonConflictWindow()
	{}
		
private:
	TWeakPtr< SWindow > WidgetWindow;

	//Meshes
	UObject *SourceObject;
	bool bIsPreviewConflict;

	//////////////////////////////////////////////////////////////////////////
	//Collapse generic
	bool bShowSectionFlag[EFBXCompareSection_Count];
	FReply SetSectionVisible(EFBXCompareSection SectionIndex);
	EVisibility IsSectionVisible(EFBXCompareSection SectionIndex);
	const FSlateBrush* GetCollapsableArrow(EFBXCompareSection SectionIndex) const;
	//////////////////////////////////////////////////////////////////////////
	
	//////////////////////////////////////////////////////////////////////////
	// Compare data
	FCompMesh *SourceData;
	FCompMesh *ResultData;
	//////////////////////////////////////////////////////////////////////////

	//////////////////////////////////////////////////////////////////////////
	// Skeleton Data
	TSharedPtr<STreeView<TSharedPtr<FSkeletonCompareData>>> CompareTree;
	TArray<TSharedPtr<FSkeletonCompareData>> DisplaySkeletonTreeItem;

	TArray<TSharedPtr<FSkeletonCompareData>> CurrentSkeletonTreeItem;
	TArray<TSharedPtr<FSkeletonCompareData>> FbxSkeletonTreeItem;

	TArray<TSharedPtr<FString>> AssetReferencingSkeleton;
	
	void FilSkeletonTreeItem();
	void RecursiveMatchJointInfo(TSharedPtr<FSkeletonCompareData> CurrentItem);
	void SetMatchJointInfo();
	//Construct slate
	TSharedPtr<SWidget> ConstructSkeletonComparison();
	TSharedPtr<SWidget> ConstructSkeletonReference();
	//Slate events
	TSharedRef<ITableRow> OnGenerateRowCompareTreeView(TSharedPtr<FSkeletonCompareData> RowData, const TSharedRef<STableViewBase>& Table);
	void OnGetChildrenRowCompareTreeView(TSharedPtr<FSkeletonCompareData> InParent, TArray< TSharedPtr<FSkeletonCompareData> >& OutChildren);
	TSharedRef<ITableRow> OnGenerateRowAssetReferencingSkeleton(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable);
	//////////////////////////////////////////////////////////////////////////
};
