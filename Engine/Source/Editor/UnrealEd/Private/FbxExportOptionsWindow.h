// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Events.h"
#include "Input/Reply.h"
#include "InputCoreTypes.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"

class SButton;
class UFbxExportOption;
struct FGeometry;

class SFbxExportOptionsWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SFbxExportOptionsWindow)
		: _ExportOptions(nullptr)
		, _WidgetWindow()
		, _FullPath()
		, _BatchMode()
		{}

		SLATE_ARGUMENT( UFbxExportOption*, ExportOptions )
		SLATE_ARGUMENT( TSharedPtr<SWindow>, WidgetWindow )
		SLATE_ARGUMENT( FText, FullPath )
		SLATE_ARGUMENT( bool, BatchMode )
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);
	virtual bool SupportsKeyboardFocus() const override { return true; }

	FReply OnExport()
	{
		bShouldExport = true;
		if ( WidgetWindow.IsValid() )
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	FReply OnExportAll()
	{
		bShouldExportAll = true;
		return OnExport();
	}

	FReply OnCancel()
	{
		bShouldExport = false;
		bShouldExportAll = false;
		if ( WidgetWindow.IsValid() )
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override
	{
		if( InKeyEvent.GetKey() == EKeys::Escape )
		{
			return OnCancel();
		}

		return FReply::Unhandled();
	}

	bool ShouldExport() const
	{
		return bShouldExport;
	}

	bool ShouldExportAll() const
	{
		return bShouldExportAll;
	}

	SFbxExportOptionsWindow()
		: ExportOptions(nullptr)
		, bShouldExport(false)
		, bShouldExportAll(false)
	{}
		
private:

	FReply OnResetToDefaultClick() const;

private:
	UFbxExportOption* ExportOptions;
	TSharedPtr<class IDetailsView> DetailsView;
	TWeakPtr< SWindow > WidgetWindow;
	TSharedPtr< SButton > ImportButton;
	bool			bShouldExport;
	bool			bShouldExportAll;
};
