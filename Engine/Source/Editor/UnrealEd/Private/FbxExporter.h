// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "EngineDefines.h"
#include "Engine/StaticMesh.h"
#include "CinematicExporter.h"
#include "MovieSceneSequenceID.h"
#include "Evaluation/MovieSceneSequenceTransform.h"
#include "MovieSceneFwd.h"
#include "FbxImporter.h"
#include "INodeAndChannelMappings.h"
#include "UObject/GCObject.h"
#include "Animation/AnimTypes.h"

#include "SceneTypes.h"
#include "LightMap.h"

class ABrush;
class ACameraActor;
class ALandscapeProxy;
class ALight;
class ASkeletalMeshActor;
class IMovieScenePlayer;
class UAnimSequence;
class UCameraComponent;
class UInstancedStaticMeshComponent;
class ULightComponent;
class UMaterialInterface;
class UModel;
class UMovieScene;
class UMovieSceneSequence;
class UMovieSceneSkeletalAnimationTrack;
class UMovieScene3DTransformTrack;
class UMovieSceneColorTrack;
class UMovieScenePropertyTrack;
class UMovieSceneTrack;
class UMovieSceneDoubleVectorTrack;
class USkeletalMesh;
class USkeletalMeshComponent;
class USplineMeshComponent;
class UStaticMeshComponent;
class FColorVertexBuffer;
class UFbxExportOption;
struct FAnimControlTrackKey;
struct FExpressionInput;
struct FMovieSceneDoubleChannel;
struct FMovieSceneFloatChannel;
struct FMovieSceneIntegerChannel;
struct FMovieSceneSequenceTransform;

namespace UnFbx
{
	struct FFbxMaterialBakingMeshData;

	/** Adapter interface which allows ExportAnimTrack to act on sequencer without a tight coupling. */
	class IAnimTrackAdapter
	{
	public:
		virtual ~IAnimTrackAdapter() = default;
		virtual int32 GetLocalStartFrame() const = 0;
		virtual int32 GetStartFrame() const { return GetLocalStartFrame(); }
		virtual int32 GetLength() const = 0;
		/** Updates the runtime state of the animation track to the specified frame. */
		virtual void UpdateAnimation(int32 LocalFrame) = 0;
		virtual double GetFrameRate() const { return 1.0 / static_cast<double>(DEFAULT_SAMPLERATE); }
		/** The anim sequence that drives this anim track */
		virtual UAnimSequence* GetAnimSequence(int32 LocalFrame) const { return nullptr; }
		/** The time into the anim sequence for the given LocalFrame */
		virtual float GetAnimTime(int32 LocalFrame) const { return 0.f; }
		/** Set optional range in frames(tick resolution)*/
		virtual void SetRange(const FFrameNumber& StartFrame, const FFrameNumber& EndFrame) = 0;
	};

	/** An anim track adapter for a level sequence. */
	class FLevelSequenceAnimTrackAdapter : public IAnimTrackAdapter
	{
	public:
		struct FAnimTrackSettings
		{
			IMovieScenePlayer* MovieScenePlayer = nullptr;
			UMovieSceneSequence* MovieSceneSequence = nullptr;
			UMovieSceneSequence* RootMovieSceneSequence = nullptr;
			FMovieSceneSequenceTransform RootToLocalTransform;
			UMovieSceneSkeletalAnimationTrack* AnimTrack = nullptr;
			bool bForceUseOfMovieScenePlaybackRange = false; //needed for when evaluate in isolation is used by Sequencer, the subsequence range will not be correct
		};
		UNREALED_API FLevelSequenceAnimTrackAdapter(const FAnimTrackSettings& Settings);
		
		UE_DEPRECATED(5.6, "FLevelSequenceAnimTrackAdapter constructor taking multiple arguments is deprecated, use version that takes FAnimTrackSettings instead")
		UNREALED_API FLevelSequenceAnimTrackAdapter(IMovieScenePlayer* InMovieScenePlayer, UMovieSceneSequence* InMovieSceneSequence, UMovieSceneSequence* InRootMovieSceneSequence, const FMovieSceneSequenceTransform& InRootToLocalTransform, UMovieSceneSkeletalAnimationTrack* InAnimTrack = nullptr);
		UE_DEPRECATED(5.5, "FLevelSequenceAnimTrackAdapter constructor taking a movie scene is deprecated, use version that takes movie scene sequences")
		UNREALED_API FLevelSequenceAnimTrackAdapter(IMovieScenePlayer* InMovieScenePlayer, UMovieScene* InMovieScene, const FMovieSceneSequenceTransform& InRootToLocalTransform, UMovieSceneSkeletalAnimationTrack* InAnimTrack = nullptr);
		
		UNREALED_API virtual int32 GetLocalStartFrame() const override;
		UNREALED_API virtual int32 GetStartFrame() const override;
		UNREALED_API virtual int32 GetLength() const override;
		UNREALED_API virtual void UpdateAnimation(int32 LocalFrame) override;
		UNREALED_API virtual double GetFrameRate() const override;
		UNREALED_API virtual UAnimSequence* GetAnimSequence(int32 LocalFrame) const override;
		UNREALED_API virtual float GetAnimTime(int32 LocalFrame) const override;
		UNREALED_API virtual void SetRange(const FFrameNumber& StartFrame, const FFrameNumber& EndFrame) override;

		//set the frame rate to use, either from movie scene(default) or custom one
		UNREALED_API void SetFrameRate(FFrameRate InFrameRate);
	private:
		TRange<FFrameNumber> GetSequenceRange() const;
		//will be either sequencer display rate or custom frame rate
		FFrameRate GetDisplayRate() const; 

	private:
		IMovieScenePlayer* MovieScenePlayer;
		UMovieSceneSequence* MovieSceneSequence;
		UMovieSceneSequence* RootMovieSceneSequence;
		bool bForceUseOfMovieScenePlaybackRange;
		UMovieScene* MovieScene;
		FMovieSceneSequenceTransform RootToLocalTransform;
		UMovieSceneSkeletalAnimationTrack* AnimTrack;
		TOptional < TPair<FFrameNumber, FFrameNumber>> OptionalRange;
		TOptional<FFrameRate> OptionalFrameRate;
	};
/**
 * Main FBX Exporter class.
 */
class FFbxExporter : public FCinematicExporter, public FGCObject
{
public:
	/**
	 * Returns the exporter singleton. It will be created on the first request.
	 */
	static UNREALED_API FFbxExporter* GetInstance();
	static UNREALED_API void DeleteInstance();
	UNREALED_API ~FFbxExporter();
	
	//~ FGCObject
	UNREALED_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FFbxExporter");
	}

	/**
	* Load the export option from the last save state and show the dialog if bShowOptionDialog is true.
	* FullPath is the export file path we display in the dialog.
	* If the user cancels the dialog, the OutOperationCanceled will be true.
	* bOutExportAll will be true if the user wants to use the same option for all other assets they want to export.
	*
	* The function is saving the dialog state in a user ini file and reload it from there. It is not changing the CDO.
	*/
	UNREALED_API void FillExportOptions(bool BatchMode, bool bShowOptionDialog, const FString& FullPath, bool& OutOperationCanceled, bool& bOutExportAll);

	/**
	* Custom set of export options instead of UI dialog. For automation.
	*/
	UNREALED_API void SetExportOptionsOverride(UFbxExportOption* OverrideOptions);

	/**
	 * Creates and readies an empty document for export.
	 */
	UNREALED_API virtual void CreateDocument();
	
	/**
	 * Closes the FBX document, releasing its memory.
	 */
	UNREALED_API virtual void CloseDocument();
	
	/**
	 * Writes the FBX document to disk and releases it by calling the CloseDocument() function.
	 */
	UNREALED_API virtual void WriteToFile(const TCHAR* Filename);
	
	/**
	 * Exports the light-specific information for a light actor.
	 */
	UNREALED_API virtual void ExportLight( ALight* Actor, INodeNameAdapter& NodeNameAdapter );

	/**
	 * Exports the camera-specific information for a camera actor.
	 */
	UNREALED_API virtual void ExportCamera( ACameraActor* Actor, bool bExportComponents, INodeNameAdapter& NodeNameAdapter );

	/**
	 * Exports the mesh and the actor information for a brush actor.
	 */
	UNREALED_API virtual void ExportBrush(ABrush* Actor, UModel* InModel, bool bConvertToStaticMesh, INodeNameAdapter& NodeNameAdapter );

	/**
	 * Exports the basic scene information to the FBX document.
	 */
	UNREALED_API virtual void ExportLevelMesh( ULevel* InLevel, bool bSelectedOnly, INodeNameAdapter& NodeNameAdapter , bool bSaveAnimSeq = true);

	/**
	* Exports the basic scene information to the FBX document, using the passed in Actors
	*/
	UNREALED_API virtual void ExportLevelMesh(ULevel* InLevel, bool bExportLevelGeometry, TArray<AActor*>& ActorToExport, INodeNameAdapter& NodeNameAdapter, bool bSaveAnimSeq = true);


	/**
	 * Exports the given level sequence information into a FBX document.
	 *
	 * @return	true, if successful
	 */
	UNREALED_API bool ExportLevelSequence(UMovieSceneSequence* MovieSceneSequence, UMovieSceneSequence* RootMovieSceneSequence, const TArray<FGuid>& InBindings, IMovieScenePlayer* MovieScenePlayer, INodeNameAdapter& NodeNameAdapter, FMovieSceneSequenceIDRef SequenceID, const FMovieSceneSequenceTransform& RootToLocalTransform);

	UE_DEPRECATED(5.5, "ExportLevelSequence taking a movie scene is deprecated, use version that takes movie scene sequences")
	UNREALED_API bool ExportLevelSequence(UMovieScene* MovieScene, const TArray<FGuid>& InBindings, IMovieScenePlayer* MovieScenePlayer, INodeNameAdapter& NodeNameAdapter, FMovieSceneSequenceIDRef SequenceID, const FMovieSceneSequenceTransform& RootToLocalTransform);

	/** Add timecode attributes to the given fbx node and add a key at the sequence playback start, using the timecode of the source section */
	UNREALED_API void AddTimecodeAttributesAndSetKey(const UMovieSceneSection* InSection, FbxNode* InFbxNode, const FMovieSceneSequenceTransform& RootToLocalTransform);

	/** Export an FBX from the given control rig section. Optionally remapping and filtering controls. */
	UNREALED_API bool ExportControlRigSection(const UMovieSceneSection* Section, const TArray<FControlRigFbxNodeMapping>& ChannelsMapping, const TArray<FName>& FilterControls, const FMovieSceneSequenceTransform& RootToLocalTransform);
	
	/**
	 * Exports the given level sequence track information into a FBX document.
	 *
	 * @return	true, if successful
	 */
	UNREALED_API bool ExportLevelSequenceTracks(UMovieSceneSequence* MovieSceneSequence, UMovieSceneSequence* RootMovieSceneSequence, IMovieScenePlayer* MovieScenePlayer, FMovieSceneSequenceIDRef InSequenceID, FbxNode* FbxActor, UObject* BoundObject, const TArray<UMovieSceneTrack*>& Tracks, const FMovieSceneSequenceTransform& RootToLocalTransform);

	UE_DEPRECATED(5.5, "ExportLevelSequenceTracks taking a movie scene is deprecated, use version that takes movie scene sequences")
	UNREALED_API bool ExportLevelSequenceTracks(UMovieScene* MovieScene, IMovieScenePlayer* MovieScenePlayer, FMovieSceneSequenceIDRef InSequenceID, FbxNode* FbxActor, UObject* BoundObject, const TArray<UMovieSceneTrack*>& Tracks, const FMovieSceneSequenceTransform& RootToLocalTransform);

	/**
	 * Exports the mesh and the actor information for a static mesh actor.
	 */
	UNREALED_API virtual void ExportStaticMesh( AActor* Actor, UStaticMeshComponent* StaticMeshComponent, INodeNameAdapter& NodeNameAdapter );

	/**
	 * Exports a static mesh
	 * @param StaticMesh	The static mesh to export
	 * @param MaterialOrder	Optional ordering of materials to set up correct material ID's across multiple meshes being export such as BSP surfaces which share common materials. Should be used sparingly
	 */
	UNREALED_API virtual void ExportStaticMesh( UStaticMesh* StaticMesh, const TArray<FStaticMaterial>* MaterialOrder = nullptr );

	/**
	 * Exports BSP
	 * @param Model			 The model with BSP to export
	 * @param bSelectedOnly  true to export only selected surfaces (or brushes)
	 */
	UNREALED_API virtual void ExportBSP( UModel* Model, bool bSelectedOnly );

	/**
	 * Exports a static mesh light map
	 */
	UNREALED_API virtual void ExportStaticMeshLightMap( UStaticMesh* StaticMesh, int32 LODIndex, int32 UVChannel );

	/**
	 * Exports a skeletal mesh
	 */
	UNREALED_API virtual void ExportSkeletalMesh( USkeletalMesh* SkeletalMesh );

	/**
	 * Exports the mesh and the actor information for a skeletal mesh actor.
	 */
	UNREALED_API virtual void ExportSkeletalMesh( AActor* Actor, USkeletalMeshComponent* SkeletalMeshComponent, INodeNameAdapter& NodeNameAdapter );
	
	/**
	 * Exports the mesh and the actor information for a landscape actor.
	 */
	UNREALED_API void ExportLandscape(ALandscapeProxy* Landscape, bool bSelectedOnly, INodeNameAdapter& NodeNameAdapter);

	/**
	 * Exports a single UAnimSequence, and optionally a skeletal mesh
	 */
	UNREALED_API FbxNode* ExportAnimSequence( const UAnimSequence* AnimSeq, const USkeletalMesh* SkelMesh, bool bExportSkelMesh, const TCHAR* MeshNames=nullptr, FbxNode* ActorRootNode=nullptr, const TArray<UMaterialInterface*>* OverrideMaterials = nullptr);

	/** A node name adapter for a level sequence. */
	class FLevelSequenceNodeNameAdapter : public INodeNameAdapter
	{
	public:
		UNREALED_API FLevelSequenceNodeNameAdapter( UMovieScene* InMovieScene, IMovieScenePlayer* InMovieScenePlayer, FMovieSceneSequenceIDRef InSequenceID);
		UNREALED_API virtual FString GetActorNodeName(const AActor* InActor) override;
		UNREALED_API virtual void AddFbxNode(UObject* InObject, FbxNode* InFbxNode) override;
		UNREALED_API virtual FbxNode* GetFbxNode(UObject* InObject) override;
	private:
		UMovieScene* MovieScene;
		IMovieScenePlayer* MovieScenePlayer;
		FMovieSceneSequenceID SequenceID;
		TMap<FGuid, FbxNode*> GuidToFbxNodeMap;
	};

	/* Get a valid unique name from a name */
	UNREALED_API FString GetFbxObjectName(const FString &FbxObjectNode, INodeNameAdapter& NodeNameAdapter);

	/**
	 * Exports the basic information about an actor and buffers it.
	 * This function creates one FBX node for the actor with its placement.
	 */
	UNREALED_API FbxNode* ExportActor(AActor* Actor, bool bExportComponents, INodeNameAdapter& NodeNameAdapter, bool bSaveAnimSeq = true);

private:
	UNREALED_API FFbxExporter();

	static UNREALED_API TSharedPtr<FFbxExporter> StaticInstance;

	FbxManager* SdkManager;
	FbxScene* Scene;
	FbxAnimStack* AnimStack;
	FbxAnimLayer* AnimLayer;
	FbxCamera* DefaultCamera;
	
	FFbxDataConverter Converter;
	
	TMap<FString,int32> FbxNodeNameToIndexMap;
	TMap<const AActor*, FbxNode*> FbxActors;
	TMap<const USkeletalMeshComponent*, FbxNode*> FbxSkeletonRoots;
	TMap<const UMaterialInterface*, TMap<int32, FbxSurfaceMaterial*>> FbxMaterials;
	TMap<const UStaticMesh*, FbxMesh*> FbxMeshes;
	TMap<const UStaticMesh*, FbxMesh*> FbxCollisionMeshes;

	/** The frames-per-second (FPS) used when baking transforms */
	static UNREALED_API const float BakeTransformsFPS;
	
	/** Whether or not to export vertices unwelded */
	static UNREALED_API bool bStaticMeshExportUnWeldedVerts;

	TObjectPtr<UFbxExportOption> ExportOptionsUI;
	TObjectPtr<UFbxExportOption> ExportOptionsOverride;

	/**
	* Export Anim Track of the given SkeletalMeshComponent
	*/
	UNREALED_API void ExportAnimTrack( IAnimTrackAdapter& AnimTrackAdapter, AActor* Actor, USkeletalMeshComponent* SkeletalMeshComponent, double SamplingRate );

	UNREALED_API void ExportModel(UModel* Model, FbxNode* Node, const char* Name, const FFbxMaterialBakingMeshData& MaterialBakingMeshData);

	UNREALED_API FbxNode* ExportCollisionMesh(const UStaticMesh* StaticMesh, const TCHAR* MeshName, FbxNode* ParentActor);

	/**
	 * Exports a static mesh
	 * @param StaticMesh	The static mesh to export
	 * @param MeshName		The name of the mesh for the FBX file
	 * @param FbxActor		The fbx node representing the mesh
	 * @param ExportLOD		The LOD of the mesh to export
	 * @param LightmapUVChannel Optional UV channel to export
	 * @param ColorBuffer	Vertex color overrides to export
	 * @param MaterialOrderOverride	Optional ordering of materials to set up correct material ID's across multiple meshes being export such as BSP surfaces which share common materials. Should be used sparingly
	 * @param OverrideMaterials	Optional array of materials to be used instead of the static mesh materials. Used for material overrides in static mesh components.
	 */
	UNREALED_API FbxNode* ExportStaticMeshToFbx(const UStaticMesh* StaticMesh, int32 ExportLOD, const TCHAR* MeshName, FbxNode* FbxActor, const FFbxMaterialBakingMeshData& MaterialBakingMeshData, int32 LightmapUVChannel = -1, const FColorVertexBuffer* ColorBuffer = nullptr, const TArray<FStaticMaterial>* MaterialOrderOverride = nullptr, const TArray<UMaterialInterface*>* OverrideMaterials = nullptr);

	UNREALED_API bool ExportStaticMeshFromMeshDescription(FbxMesh* Mesh
		, const UStaticMesh* StaticMesh
		, const FMeshDescription* MeshDescription
		, FbxNode* FbxActor
		, int32 LightmapUVChannel
		, const TArray<FStaticMaterial>* MaterialOrderOverride
		, const TArray<UMaterialInterface*>* OverrideMaterials
		, const FFbxMaterialBakingMeshData& MaterialBakingMeshData);

	UNREALED_API bool ExportStaticMeshFromRenderData(FbxMesh* Mesh
		, const UStaticMesh* StaticMesh
		, const FStaticMeshLODResources& RenderMesh
		, FbxNode* FbxActor
		, int32 LightmapUVChannel
		, const FColorVertexBuffer* ColorBuffer
		, const TArray<FStaticMaterial>* MaterialOrderOverride
		, const TArray<UMaterialInterface*>* OverrideMaterials
		, const FFbxMaterialBakingMeshData& MaterialBakingMeshData);

	/**
	 * Exports a spline mesh
	 * @param SplineMeshComp	The spline mesh component to export
	 * @param MeshName		The name of the mesh for the FBX file
	 * @param FbxActor		The fbx node representing the mesh
	 */
	UNREALED_API void ExportSplineMeshToFbx(const USplineMeshComponent* SplineMeshComp, const TCHAR* MeshName, FbxNode* FbxActor, const FFbxMaterialBakingMeshData& MaterialBakingMeshData);

	/**
	 * Exports an instanced mesh
	 * @param InstancedMeshComp	The instanced mesh component to export
	 * @param MeshName		The name of the mesh for the FBX file
	 * @param FbxActor		The fbx node representing the mesh
	 */
	UNREALED_API void ExportInstancedMeshToFbx(const UInstancedStaticMeshComponent* InstancedMeshComp, const TCHAR* MeshName, FbxNode* FbxActor, const FFbxMaterialBakingMeshData& MaterialBakingMeshData);

	/**
	* Exports a landscape
	* @param Landscape		The landscape to export
	* @param MeshName		The name of the mesh for the FBX file
	* @param FbxActor		The fbx node representing the mesh
	*/
	UNREALED_API void ExportLandscapeToFbx(ALandscapeProxy* Landscape, const TCHAR* MeshName, FbxNode* FbxActor, bool bSelectedOnly);

	/**
	* Fill an fbx light with from a unreal light component
	*@param ParentNode			The parent FbxNode the one over the light node
	* @param Camera				Fbx light object
	* @param CameraComponent	Unreal light component
	*/
	UNREALED_API void FillFbxLightAttribute(FbxLight* Light, FbxNode* FbxParentNode, ULightComponent* BaseLight);

	/**
	* Fill an fbx camera with from a unreal camera component
	* @param ParentNode			The parent FbxNode the one over the camera node
	* @param Camera				Fbx camera object
	* @param CameraComponent	Unreal camera component
	*/
	UNREALED_API void FillFbxCameraAttribute(FbxNode* ParentNode, FbxCamera* Camera, UCameraComponent *CameraComponent);

	/**
	 * Adds FBX skeleton nodes to the FbxScene based on the skeleton in the given USkeletalMesh, and fills
	 * the given array with the nodes created
	 */
	UNREALED_API FbxNode* CreateSkeleton(const USkeletalMesh* SkelMesh, TArray<FbxNode*>& BoneNodes);

	/**
	 * Adds an Fbx Mesh to the FBX scene based on the data in the given FSkeletalMeshLODModel
	 * @param SkelMesh			The SkeletalMesh we are exporting
	 * @param MeshName			The SkeletalMesh name
	 * @param LODIndex			The mesh LOD index we are exporting
	 * @param AnimSeq			If an AnimSeq is provided and are exporting MorphTarget, the MorphTarget animation will be exported as well.
	 * @param OverrideMaterials Optional array of materials to be used instead of the skeletal mesh materials. Used for material overrides in skeletal mesh components.
	 */
	UNREALED_API FbxNode* CreateMesh(const USkeletalMesh* SkelMesh, const TCHAR* MeshName, int32 LODIndex, const UAnimSequence* AnimSeq = nullptr, const TArray<UMaterialInterface*>* OverrideMaterials = nullptr);

	/**
	 * Adds Fbx Clusters necessary to skin a skeletal mesh to the bones in the BoneNodes list
	 */
	UNREALED_API void BindMeshToSkeleton(const USkeletalMesh* SkelMesh, FbxNode* MeshRootNode, TArray<FbxNode*>& BoneNodes, int32 LODIndex);

	/**
	 * Add a bind pose to the scene based on the FbxMesh and skinning settings of the given node
	 */
	UNREALED_API void CreateBindPose(FbxNode* MeshRootNode);

	/**
	 * Add the given skeletal mesh to the Fbx scene in preparation for exporting.  Makes all new nodes a child of the given node
	 */
	UNREALED_API FbxNode* ExportSkeletalMeshToFbx(const USkeletalMesh* SkelMesh, const UAnimSequence* AnimSeq, const TCHAR* MeshName, FbxNode* ActorRootNode, const TArray<UMaterialInterface*>* OverrideMaterials = nullptr);

	/** Export SkeletalMeshComponent */
	UNREALED_API void ExportSkeletalMeshComponent(USkeletalMeshComponent* SkelMeshComp, const TCHAR* MeshName, FbxNode* ActorRootNode, INodeNameAdapter& NodeNameAdapter, bool bSaveAnimSeq = true);

	/** Initializing the AnimStack playrate from the AnimSequence */
	UNREALED_API bool SetupAnimStack(const UAnimSequence* AnimSeq);

	/**
	 * Add the given animation sequence as rotation and translation tracks to the given list of bone nodes
	 */
	UNREALED_API void ExportAnimSequenceToFbx(const UAnimSequence* AnimSeq, const USkeletalMesh* SkelMesh, TArray<FbxNode*>& BoneNodes, FbxAnimLayer* AnimLayer, FFrameTime StartFrameTime, FFrameTime EndFrameTime, float FrameRateScale, float StartTime);

	/**
	 * Add the custom Curve data to the FbxAnimCurves passed in parameter by matching their name to the skeletal mesh custom curves.
	 */
	UNREALED_API void ExportCustomAnimCurvesToFbx(const TMap<FName, FbxAnimCurve*>& CustomCurves, const UAnimSequence* AnimSeq,
		FFrameTime AnimStartOffset, FFrameTime AnimEndOffset, float FrameRateScale, float StartTime, float ValueScale = 1.f);
	
	UE_DEPRECATED(5.1, "ExportCustomAnimCurvesToFbx is deprecated, use different signature")
	UNREALED_API void ExportCustomAnimCurvesToFbx(const TMap<FName, FbxAnimCurve*>& CustomCurves, const UAnimSequence* AnimSeq,
		float AnimStartOffset, float AnimEndOffset, float AnimPlayRate, float StartTime, float ValueScale = 1.f);

	/**
	 * Used internally to reuse the AnimSequence iteration code when exporting various kind of curves.
	 */
	UNREALED_API void IterateInsideAnimSequence(const UAnimSequence* AnimSeq, FFrameTime StartFrameTime, FFrameTime EndFrameTime, float FrameRateScale, float StartTime, TFunctionRef<void(double, FbxTime, bool)> IterationLambda);

	UE_DEPRECATED(5.1, "IterateInsideAnimSequence is deprecated, use different signature")
	UNREALED_API void IterateInsideAnimSequence(const UAnimSequence* AnimSeq, float AnimStartOffset, float AnimEndOffset, float AnimPlayRate, float StartTime, TFunctionRef<void(float, FbxTime, bool)> IterationLambda);

	/** 
	 * The curve code doesn't differentiate between angles and other data, so an interpolation from 179 to -179
	 * will cause the bone to rotate all the way around through 0 degrees.  So here we make a second pass over the 
	 * rotation tracks to convert the angles into a more interpolation-friendly format.  
	 */
	UNREALED_API void CorrectAnimTrackInterpolation( TArray<FbxNode*>& BoneNodes, FbxAnimLayer* AnimLayer );

	/**
	 * Exports a level sequence 3D transform track into the FBX animation stack.
	 */
	UNREALED_API void ExportLevelSequence3DTransformTrack(FbxNode* FbxActor, IMovieScenePlayer* MovieScenePlayer, FMovieSceneSequenceIDRef InSequenceID, UMovieScene3DTransformTrack& TransformTrack, UObject* BoundObject, const TRange<FFrameNumber>& InPlaybackRange, const FMovieSceneSequenceTransform& RootToLocalTransform);

	/**
	 * Exports a level sequence 3D transform track that's getting baked (sample all sections) onto the FBX animation stack.
	 */
	UNREALED_API void ExportLevelSequenceBaked3DTransformTrack(IAnimTrackAdapter& AnimTrackAdapter, FbxNode* FbxActor, IMovieScenePlayer* MovieScenePlayer, FMovieSceneSequenceIDRef InSequenceID, TArray<TWeakObjectPtr<UMovieScene3DTransformTrack> > TransformTracks, UObject* BoundObject, const TRange<FFrameNumber>& InPlaybackRange, const FMovieSceneSequenceTransform& RootToLocalTransform);

	/**
	 * Exports a level sequence color track into the FBX animation stack.
	 */
	UNREALED_API void ExportLevelSequenceColorTrack(FbxNode* FbxActor, UMovieSceneColorTrack& ColorTrack, UObject* BoundObject, const TRange<FFrameNumber>& InPlaybackRange, const FMovieSceneSequenceTransform& RootToLocalTransform);
	
	/**
	 * Exports a level sequence vector track into the FBX animation stack.
	 */
	UNREALED_API void ExportLevelSequenceVectorTrack(FbxNode* FbxActor, UMovieSceneDoubleVectorTrack& VectorTrack, UObject* BoundObject, const TRange<FFrameNumber>& InPlaybackRange, const FMovieSceneSequenceTransform& RootToLocalTransform);

	/**
	 * Exports a level sequence property track into the FBX animation stack. 
	 */
	UNREALED_API void ExportLevelSequenceTrackChannels( FbxNode* FbxActor, UMovieSceneTrack& Track, const TRange<FFrameNumber>& InPlaybackRange, const FMovieSceneSequenceTransform& RootToLocalTransform, bool bBakeBezierCurves);

	UE_DEPRECATED(5.4, "Use ExportLevelSequenceTrackChannels that indicates whether bezier channels should be baked")
	void ExportLevelSequenceTrackChannels(FbxNode* FbxActor, UMovieSceneTrack& Track, const TRange<FFrameNumber>& InPlaybackRange, const FMovieSceneSequenceTransform& RootToLocalTransform) { ExportLevelSequenceTrackChannels(FbxActor, Track, InPlaybackRange, RootToLocalTransform, false); }

	/** Defines value export modes for the EportRichCurveToFbxCurve method. */
	enum class ERichCurveValueMode
	{
		/** Export values directly */
		Default,
		/** Export fov values which get processed to focal length. */
		Fov
	};

	/** Generic implementation of exporting a movie scene bezier curve channel to an fbx animation curve, baked per frame */
	template<typename ChannelType>
	void ExportBezierChannelToFbxCurveBaked(FbxAnimCurve& InFbxCurve, const ChannelType& InChannel, FFrameRate TickResolution, const UMovieSceneTrack* Track, ERichCurveValueMode ValueMode, bool bNegative, const FMovieSceneSequenceTransform& RootToLocalTransform);

	/** Generic implementation of exporting a movie scene bezier curve channel to an fbx animation curve */
	template<typename ChannelType>
	void ExportBezierChannelToFbxCurve(FbxAnimCurve& InFbxCurve, const ChannelType& InChannel, FFrameRate TickResolution, ERichCurveValueMode ValueMode, bool bNegative, const FMovieSceneSequenceTransform& RootToLocalTransform);

	/** Exports a movie scene float channel to an fbx animation curve. */
	UNREALED_API void ExportChannelToFbxCurve(FbxAnimCurve& InFbxCurve, const FMovieSceneFloatChannel& InChannel, FFrameRate TickResolution, ERichCurveValueMode ValueMode = ERichCurveValueMode::Default, bool bNegative = false, const FMovieSceneSequenceTransform& RootToLocalTransform = FMovieSceneSequenceTransform());

	/** Exports a movie scene double channel to an fbx animation curve. */
	UNREALED_API void ExportChannelToFbxCurve(FbxAnimCurve& InFbxCurve, const FMovieSceneDoubleChannel& InChannel, FFrameRate TickResolution, ERichCurveValueMode ValueMode = ERichCurveValueMode::Default, bool bNegative = false, const FMovieSceneSequenceTransform& RootToLocalTransform = FMovieSceneSequenceTransform());

	/** Exports a movie scene integer channel to an fbx animation curve. */
	UNREALED_API void ExportChannelToFbxCurve(FbxAnimCurve& InFbxCurve, const FMovieSceneIntegerChannel& InChannel, FFrameRate TickResolution, const FMovieSceneSequenceTransform& RootToLocalTransform = FMovieSceneSequenceTransform());

	UNREALED_API void ExportChannelToFbxCurve(FbxAnimCurve& InFbxCurve, const FMovieSceneBoolChannel& InChannel, FFrameRate TickResolution, const FMovieSceneSequenceTransform& RootToLocalTransform);
	
	UNREALED_API void ExportChannelToFbxCurve(FbxAnimCurve& InFbxCurve, const FMovieSceneByteChannel& InChannel, FFrameRate TickResolution, const FMovieSceneSequenceTransform& RootToLocalTransform);

	template <class ChannelType, typename T>
	void ExportConstantChannelToFbxCurve(FbxAnimCurve& InFbxCurve, const ChannelType& InChannel, FFrameRate TickResolution, const FMovieSceneSequenceTransform& RootToLocalTransform);

	UNREALED_API void ExportTransformChannelsToFbxCurve(FbxNode* InFbxNode, TPair<FMovieSceneFloatChannel*, bool> ChannelX, TPair<FMovieSceneFloatChannel*, bool> ChannelY, TPair<FMovieSceneFloatChannel*, bool> ChannelZ, int TmPropertyIndex, const UMovieSceneTrack* Track, const FMovieSceneSequenceTransform& RootToLocalTransform);

	/**
	 * Finds the given actor in the already-exported list of structures
	 * @return FbxNode* the FBX node created from the UE actor
	 */
	UNREALED_API FbxNode* FindActor(AActor* Actor, INodeNameAdapter* NodeNameAdapter = nullptr);

	/** Create fbx node with the given name */
	UNREALED_API FbxNode* CreateNode(const FString& NodeName);

	/**
	 * Find bone array of FbxNOdes of the given skeletalmeshcomponent  
	 */
	UNREALED_API bool FindSkeleton(USkeletalMeshComponent* SkelComp, TArray<FbxNode*>& BoneNodes, INodeNameAdapter* NodeNameAdapter = nullptr);

	/** recursively get skeleton */
	UNREALED_API void GetSkeleton(FbxNode* RootNode, TArray<FbxNode*>& BoneNodes);

	UNREALED_API bool FillFbxTextureProperty(const char *PropertyName, const FExpressionInput& MaterialInput, FbxSurfaceMaterial* FbxMaterial);
	/**
	 * Exports the profile_COMMON information for a material.
	 */
	UNREALED_API FbxSurfaceMaterial* ExportMaterial(UMaterialInterface* Material, const int32& MaterialIndex, const FFbxMaterialBakingMeshData& BakingMeshData);
	
	UNREALED_API FbxSurfaceMaterial* CreateDefaultMaterial();
	
	/**
	 * Create user property in Fbx Node.
	 * Some Unreal animatable property can't be animated in FBX property. So create user property to record the animation of property.
	 *
	 * @param Node  FBX Node the property append to.
	 * @param Value Property value.
	 * @param Name  Property name.
	 * @param Label Property label.
	 */
	template<typename T>
	void CreateAnimatableUserProperty(FbxNode* Node, T Value, const char* Name, const char* Label, FbxDataType DataType = FbxFloatDT);

	/** Exports all the object's FBX metadata to the FBX node */
	UNREALED_API void ExportObjectMetadata(const UObject* ObjectToExport, FbxNode* Node);

public:
	/** Returns currently active FBX export options. Automation or UI dialog based options. */
	UNREALED_API UFbxExportOption* GetExportOptions();

	bool bSceneGlobalTimeLineSet = false;
};



} // namespace UnFbx
