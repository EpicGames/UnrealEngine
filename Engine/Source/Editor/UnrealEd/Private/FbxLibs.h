// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Load the FBx dll's if needed 
 */
void LoadFBxLibraries();


/**
 * Unload the FBx dll's if needed 
 */
void UnloadFBxLibraries();
