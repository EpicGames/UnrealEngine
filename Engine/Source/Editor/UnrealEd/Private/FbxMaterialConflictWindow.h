// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/SNullWidget.h"
#include "Styling/AppStyle.h"
#include "FbxCompareWindow.h"
#include "FbxImporter.h"

class FMaterialConflictData : public FCompareRowData
{
public:

	FMaterialConflictData(TArray<FCompMaterial>& InSourceMaterials, TArray<FCompMaterial>& InResultMaterials, TArray<int32>& InRemapMaterials, TArray<bool>& InAutoRemapMaterials, TArray<bool>& InCustomRemapMaterials, bool InbIsPreviewConflict)
		: SourceMaterialIndex(INDEX_NONE)
		, ResultMaterialIndex(INDEX_NONE)
		, SourceMaterials(InSourceMaterials)
		, ResultMaterials(InResultMaterials)
		, RemapMaterials(InRemapMaterials)
		, AutoRemapMaterials(InAutoRemapMaterials)
		, CustomRemapMaterials(InCustomRemapMaterials)
		, bIsPreviewConflict(InbIsPreviewConflict)
	{}

	virtual ~FMaterialConflictData() = default;

	int32 SourceMaterialIndex;
	int32 ResultMaterialIndex;

	virtual TSharedRef<SWidget> ConstructCellCurrent() override;
	virtual TSharedRef<SWidget> ConstructCellFbx() override;

	FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool IsResultData);
	void AssignMaterialMatch(int32 InMatchMaterialIndex);
	FText GetCellString(bool IsResultData) const;
	FText GetCellTooltipString(bool IsResultData) const;
	FSlateColor GetCellColor(bool IsResultData) const;

	TSharedPtr<SWidget> ParentContextMenu;

	TArray<FCompMaterial>& SourceMaterials;
	TArray<FCompMaterial>& ResultMaterials;
	TArray<int32>& RemapMaterials;
	TArray<bool>& AutoRemapMaterials;
	TArray<bool>& CustomRemapMaterials;
	bool bIsPreviewConflict;
};

class SFbxMaterialConflictWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SFbxMaterialConflictWindow)
		: _WidgetWindow()
		, _SourceMaterials(nullptr)
		, _ResultMaterials(nullptr)
		, _RemapMaterials(nullptr)
		, _AutoRemapMaterials(nullptr)
		, _bIsPreviewConflict(false)
		{}

		SLATE_ARGUMENT( TSharedPtr<SWindow>, WidgetWindow )
		SLATE_ARGUMENT( TArray<FCompMaterial>*, SourceMaterials)
		SLATE_ARGUMENT( TArray<FCompMaterial>*, ResultMaterials)
		SLATE_ARGUMENT( TArray<int32>*, RemapMaterials)
		SLATE_ARGUMENT( TArray<bool>*, AutoRemapMaterials )
		SLATE_ARGUMENT( bool, bIsPreviewConflict )
			
	SLATE_END_ARGS()

public:
	UnFbx::EFBXReimportDialogReturnOption GetReturnOption() { return ReturnOption; }

	void Construct(const FArguments& InArgs);
	virtual bool SupportsKeyboardFocus() const override { return true; }

	

	FReply OnDone();
	FReply OnReset();
	FReply OnCancel();

	EVisibility CollapsePreviewVisibility()
	{
		return bIsPreviewConflict ? EVisibility::Collapsed : EVisibility::Visible;
	}

	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override
	{
		if( InKeyEvent.GetKey() == EKeys::Escape )
		{
			return OnReset();
		}

		return FReply::Unhandled();
	}

	SFbxMaterialConflictWindow()
	{
		ReturnOption = UnFbx::EFBXReimportDialogReturnOption::FBXRDRO_Cancel;
	}
		
private:
	TWeakPtr< SWindow > WidgetWindow;

	UnFbx::EFBXReimportDialogReturnOption ReturnOption;

	TArray<FCompMaterial> *SourceMaterials;
	TArray<FCompMaterial> *ResultMaterials;
	TArray<int32>* RemapMaterials;
	TArray<bool>* AutoRemapMaterials;
	TArray<bool> CustomRemapMaterials;
	bool bIsPreviewConflict;

	TArray<TSharedPtr<FMaterialConflictData>> ConflictMaterialListItem;
	
	TSharedPtr<SWidget> ConstructMaterialComparison();
	void FillMaterialListItem();
	//Slate events
	TSharedRef<ITableRow> OnGenerateRowForCompareMaterialList(TSharedPtr<FMaterialConflictData> RowData, const TSharedRef<STableViewBase>& Table);
};
