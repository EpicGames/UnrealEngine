// Copyright Epic Games, Inc. All Rights Reserved.

#include "FbxMeshUtils.h"
#include "EngineDefines.h"
#include "Misc/Paths.h"
#include "UObject/Package.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Factories/FbxAssetImportData.h"
#include "Factories/FbxSkeletalMeshImportData.h"
#include "Factories/FbxStaticMeshImportData.h"
#include "Factories/FbxImportUI.h"
#include "Engine/StaticMesh.h"
#include "EditorDirectories.h"
#include "Framework/Application/SlateApplication.h"
#include "MeshDescription.h"
#include "Misc/MessageDialog.h"
#include "ComponentReregisterContext.h"
#include "Logging/TokenizedMessage.h"
#include "FbxImporter.h"
#include "StaticMeshResources.h"
#include "Components/SkeletalMeshComponent.h"
#include "Rendering/SkeletalMeshModel.h"
#include "EditorFramework/AssetImportData.h"
#include "DesktopPlatformModule.h"
#include "Editor.h"
#include "ImportUtils/SkeletalMeshImportUtils.h"
#include "ImportUtils/StaticMeshImportUtils.h"

#include "Async/Async.h"
#include "InterchangeAssetImportData.h"
#include "InterchangeManager.h"
#include "InterchangeMeshUtilities.h"
#include "InterchangeProjectSettings.h"

#include "Misc/FbxErrors.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#include "ClothingAsset.h"
#include "SkinWeightsUtilities.h"
#include "LODUtilities.h"
#include "StaticMeshOperations.h"

DEFINE_LOG_CATEGORY_STATIC(LogExportMeshUtils, Log, All);

#define LOCTEXT_NAMESPACE "FbxMeshUtil"


namespace FbxMeshUtils
{
	namespace Private
	{
		void ShowFailedToImportLodDialog(int32 LodIndex)
		{
			FMessageDialog::Open(EAppMsgType::Ok, FText::Format(LOCTEXT("LODImport_Failure", "Failed to import LOD{0}"), FText::AsNumber(LodIndex)));
		}

		UFbxStaticMeshImportData* SetupFbxImportOptions(UStaticMesh* BaseStaticMesh, UnFbx::FBXImportOptions* ImportOptions)
		{
			check(BaseStaticMesh);

			UFbxStaticMeshImportData* ImportData = Cast<UFbxStaticMeshImportData>(BaseStaticMesh->GetAssetImportData());
			if (ImportData == nullptr)
			{
				//Convert interchange asset import data to legacy fbx static mesh import data.
				if (UInterchangeAssetImportData* InterchangeAssetImportData = Cast<UInterchangeAssetImportData>(BaseStaticMesh->GetAssetImportData()))
				{
					UInterchangeManager::GetInterchangeManager().ConvertImportData(InterchangeAssetImportData, UFbxStaticMeshImportData::StaticClass(), reinterpret_cast<UObject**>(&ImportData));
				}
			}

			if (ImportData != nullptr)
			{
				UFbxImportUI* ReimportUI = NewObject<UFbxImportUI>();
				ReimportUI->MeshTypeToImport = FBXIT_StaticMesh;
				UnFbx::FBXImportOptions::ResetOptions(ImportOptions);
				// Import data already exists, apply it to the fbx import options
				ReimportUI->StaticMeshImportData = ImportData;
				ApplyImportUIToImportOptions(ReimportUI, *ImportOptions);
			}
			else
			{
				// Use the lod 0 to set the import settings
				const FStaticMeshSourceModel& SourceModel = BaseStaticMesh->GetSourceModel(0);
				ImportOptions->NormalGenerationMethod = SourceModel.BuildSettings.bUseMikkTSpace ? EFBXNormalGenerationMethod::MikkTSpace : EFBXNormalGenerationMethod::BuiltIn;
				ImportOptions->bComputeWeightedNormals = SourceModel.BuildSettings.bComputeWeightedNormals;
				ImportOptions->DistanceFieldResolutionScale = SourceModel.BuildSettings.DistanceFieldResolutionScale;
				ImportOptions->bRemoveDegenerates = SourceModel.BuildSettings.bRemoveDegenerates;
				ImportOptions->bBuildReversedIndexBuffer = SourceModel.BuildSettings.bBuildReversedIndexBuffer;
				ImportOptions->bGenerateLightmapUVs = SourceModel.BuildSettings.bGenerateLightmapUVs;
			}

			// Set a couple of settings that shouldn't change while importing a lod
			ImportOptions->bBuildNanite = BaseStaticMesh->IsNaniteEnabled();
			ImportOptions->StaticMeshLODGroup = BaseStaticMesh->GetLODGroup();
			ImportOptions->bIsImportCancelable = false;
			ImportOptions->bImportMaterials = false;
			ImportOptions->bImportTextures = false;

			ImportOptions->bAutoComputeLodDistances = true; //Setting auto compute distance to true will avoid changing the staticmesh flag

			return ImportData;
		}

		bool CopyHighResMeshDescription(UStaticMesh* SrcStaticMesh, UStaticMesh* BaseStaticMesh)
		{
			if (!SrcStaticMesh || !SrcStaticMesh->IsSourceModelValid(0))
			{
				return false;
			}

			{
				BaseStaticMesh->Modify();

				FMeshDescription* HiResMeshDescription = BaseStaticMesh->GetHiResMeshDescription();
				if (HiResMeshDescription == nullptr)
				{
					HiResMeshDescription = BaseStaticMesh->CreateHiResMeshDescription();
				}

				check(HiResMeshDescription);

				BaseStaticMesh->ModifyHiResMeshDescription();

				FMeshDescription* TempLOD0MeshDescription = SrcStaticMesh->GetMeshDescription(0);
				check(TempLOD0MeshDescription);

				if (FMeshDescription* BaseMeshDescription = BaseStaticMesh->GetMeshDescription(0))
				{
					FString MaterialNameConflictMsg = TEXT("[Asset ") + BaseStaticMesh->GetPathName() + TEXT("] Nanite hi - res import have some material name that differ from the LOD 0 material name.Your nanite hi - res should use the same material names the LOD 0 use to ensure we can remap the section in the same order.");
					FString MaterialCountConflictMsg = TEXT("[Asset ") + BaseStaticMesh->GetPathName() + TEXT("] Nanite hi-res import dont have the same material count then LOD 0. Your nanite hi-res should have equal number of material.");
					FStaticMeshOperations::ReorderMeshDescriptionPolygonGroups(*BaseMeshDescription, *TempLOD0MeshDescription, MaterialNameConflictMsg, MaterialCountConflictMsg);
				}

				*HiResMeshDescription = MoveTemp(*TempLOD0MeshDescription);

				BaseStaticMesh->CommitHiResMeshDescription();

				BaseStaticMesh->PostEditChange();
				BaseStaticMesh->MarkPackageDirty();

				return true;
			}
		}
	}

	/** Helper function used for retrieving data required for importing static mesh LODs */
	void PopulateFBXStaticMeshLODList(UnFbx::FFbxImporter* FFbxImporter, FbxNode* Node, TArray< TUniquePtr<TArray<FbxNode*>> >& LODNodeList, int32& MaxLODCount, bool bUseLODs)
	{
		// Check for LOD nodes, if one is found, add it to the list
		if (bUseLODs && Node->GetNodeAttribute() && Node->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup)
		{
			for (int32 ChildIdx = 0; ChildIdx < Node->GetChildCount(); ++ChildIdx)
			{
				if ((LODNodeList.Num() - 1) < ChildIdx)
				{
					LODNodeList.Add(MakeUnique<TArray<FbxNode*>>());
				}
				FFbxImporter->FindAllLODGroupNode(*(LODNodeList[ChildIdx]), Node, ChildIdx);
			}

			if (MaxLODCount < (Node->GetChildCount() - 1))
			{
				MaxLODCount = Node->GetChildCount() - 1;
			}
		}
		else
		{
			// If we're just looking for meshes instead of LOD nodes, add those to the list
			if (!bUseLODs && Node->GetMesh())
			{
				if (LODNodeList.Num() == 0)
				{
					LODNodeList.Add(MakeUnique<TArray<FbxNode*>>());
				}

				LODNodeList[0]->Add(Node);
			}

			// Recursively examine child nodes
			for (int32 ChildIndex = 0; ChildIndex < Node->GetChildCount(); ++ChildIndex)
			{
				PopulateFBXStaticMeshLODList(FFbxImporter, Node->GetChild(ChildIndex), LODNodeList, MaxLODCount, bUseLODs);
			}
		}
	}

	bool ImportStaticMeshLOD( UStaticMesh* BaseStaticMesh, const FString& Filename, int32 LODLevel, bool bAsync)
	{
		if (!BaseStaticMesh)
		{
			UE_LOGF(LogExportMeshUtils, Log, "Cannot import custom LOD because the staticmesh is NULL.");
			return false;
		}

		UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
		const UInterchangeSourceData* SourceData = InterchangeManager.CreateSourceData(Filename);
		const bool bInterchangeCanImportSourceData = InterchangeManager.CanTranslateSourceData(SourceData);
		if (bInterchangeCanImportSourceData)
		{
			//Call interchange mesh utilities to import custom LOD
			UInterchangeMeshUtilities::ImportCustomLod(BaseStaticMesh, LODLevel, SourceData, bAsync).Then([BaseStaticMesh, LODLevel, bAsync](TFuture<bool> Result)
				{
					bool bResult = Result.Get();
					auto OnImportCustomLodDone = [BaseStaticMesh, LODLevel, bResult]()
						{
							if (bResult)
							{
								// Notification of success
								FNotificationInfo NotificationInfo(FText::GetEmpty());
								NotificationInfo.Text = FText::Format(NSLOCTEXT("UnrealEd", "StaticMeshLODImportSuccessful", "Static mesh LOD {0} imported successfully!"), FText::AsNumber(LODLevel));
								NotificationInfo.ExpireDuration = 5.0f;
								FSlateNotificationManager::Get().AddNotification(NotificationInfo);
							}
							else
							{
								// Notification of failure
								FNotificationInfo NotificationInfo(FText::GetEmpty());
								NotificationInfo.Text = FText::Format(NSLOCTEXT("UnrealEd", "StaticMeshLODImportFail", "Failed to import static mesh LOD {0}!"), FText::AsNumber(LODLevel));
								NotificationInfo.ExpireDuration = 5.0f;
								FSlateNotificationManager::Get().AddNotification(NotificationInfo);
							}
						};

					if (IsInGameThread())
					{
						OnImportCustomLodDone();
					}
					else
					{
						ensure(bAsync);
						Async(EAsyncExecution::TaskGraphMainThread, OnImportCustomLodDone);
					}
				});

			return true;
		}

		bool bSuccess = false;

		UE_LOGF(LogExportMeshUtils, Log, "Fbx LOD loading");

		// logger for all error/warnings
		// this one prints all messages that are stored in FFbxImporter
		// this function seems to get called outside of FBX factory
		UnFbx::FFbxImporter* FFbxImporter = UnFbx::FFbxImporter::GetInstance();
		UnFbx::FFbxLoggerSetter Logger(FFbxImporter);

		UnFbx::FBXImportOptions* ImportOptions = FFbxImporter->GetImportOptions();
		UFbxStaticMeshImportData* ImportData = Private::SetupFbxImportOptions(BaseStaticMesh, ImportOptions);

		const bool bIsReimport = BaseStaticMesh->GetRenderData()->LODResources.Num() > LODLevel;

		if ( !FFbxImporter->ImportFromFile( *Filename, FPaths::GetExtension( Filename ), true ) )
		{
			// Log the error message and fail the import.
			// @todo verify if the message works
			FFbxImporter->FlushToTokenizedErrorMessage(EMessageSeverity::Error);
		}
		else
		{
			FFbxImporter->FlushToTokenizedErrorMessage(EMessageSeverity::Warning);
			if (ImportData)
			{
				FFbxImporter->ApplyTransformSettingsToFbxNode(FFbxImporter->Scene->GetRootNode(), ImportData);
			}

			bool bUseLODs = true;
			int32 MaxLODLevel = 0;
			TArray< TUniquePtr<TArray<FbxNode*>> > LODNodeList;

			// Create a list of LOD nodes
			PopulateFBXStaticMeshLODList(FFbxImporter, FFbxImporter->Scene->GetRootNode(), LODNodeList, MaxLODLevel, bUseLODs);

			// No LODs, so just grab all of the meshes in the file
			if (MaxLODLevel == 0)
			{
				bUseLODs = false;
				MaxLODLevel = BaseStaticMesh->GetNumLODs();

				// Create a list of meshes
				PopulateFBXStaticMeshLODList(FFbxImporter, FFbxImporter->Scene->GetRootNode(), LODNodeList, MaxLODLevel, bUseLODs);

				// Nothing found, error out
				if (LODNodeList.Num() == 0)
				{
					FFbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText(LOCTEXT("Prompt_NoMeshFound", "No meshes were found in file."))), FFbxErrors::Generic_Mesh_MeshNotFound);

					FFbxImporter->ReleaseScene();
					return bSuccess;
				}
			}

			TSharedPtr<FExistingStaticMeshData> ExistMeshDataPtr;
			if (bIsReimport)
			{
				ExistMeshDataPtr = StaticMeshImportUtils::SaveExistingStaticMeshData(BaseStaticMesh, FFbxImporter->ImportOptions, LODLevel);
			}

			// Display the LOD selection dialog
			if (LODLevel > BaseStaticMesh->GetNumLODs())
			{
				// Make sure they don't manage to select a bad LOD index
				FFbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("Prompt_InvalidLODIndex", "Invalid mesh LOD index {0}, as no prior LOD index exists!"), FText::AsNumber(LODLevel))), FFbxErrors::Generic_Mesh_LOD_InvalidIndex);
			}
			else
			{
				UStaticMesh* TempStaticMesh = NULL;
				if (!LODNodeList.IsValidIndex(bUseLODs ? LODLevel : 0))
				{
					if (bUseLODs)
					{
						//Use the first LOD when user try to add or re-import a LOD from a file(different from the LOD 0 file) containing multiple LODs
						bUseLODs = false;
					}
				}
				
				if (LODNodeList.IsValidIndex(bUseLODs ? LODLevel : 0))
				{
					TempStaticMesh = (UStaticMesh*)FFbxImporter->ImportStaticMeshAsSingle(BaseStaticMesh->GetOutermost(), *(LODNodeList[bUseLODs ? LODLevel : 0]), NAME_None, RF_NoFlags, ImportData, BaseStaticMesh, LODLevel, ExistMeshDataPtr.Get());
				}
				
				// Add imported mesh to existing model
				if( TempStaticMesh )
				{
					//Build the staticmesh
					FFbxImporter->PostImportStaticMesh(TempStaticMesh, *(LODNodeList[bUseLODs ? LODLevel : 0]), LODLevel);
					TArray<int32> ReimportLodList;
					ReimportLodList.Add(LODLevel);
					StaticMeshImportUtils::UpdateSomeLodsImportMeshData(BaseStaticMesh, &ReimportLodList);
					if(bIsReimport)
					{
						StaticMeshImportUtils::RestoreExistingMeshData(ExistMeshDataPtr, BaseStaticMesh, LODLevel, false, ImportOptions->bResetToFbxOnMaterialConflict);
					}

					// Update mesh component
					BaseStaticMesh->PostEditChange();
					BaseStaticMesh->MarkPackageDirty();

					// Import worked
					FNotificationInfo NotificationInfo(FText::GetEmpty());
					NotificationInfo.Text = FText::Format(LOCTEXT("LODImportSuccessful", "Mesh for LOD {0} imported successfully!"), FText::AsNumber(LODLevel));
					NotificationInfo.ExpireDuration = 5.0f;
					FSlateNotificationManager::Get().AddNotification(NotificationInfo);
					if (BaseStaticMesh->IsSourceModelValid(LODLevel))
					{
						FStaticMeshSourceModel& SourceModel = BaseStaticMesh->GetSourceModel(LODLevel);
						SourceModel.SourceImportFilename = UAssetImportData::SanitizeImportFilename(Filename, nullptr);
						SourceModel.bImportWithBaseMesh = false;
					}
					bSuccess = true;
				}
				else
				{
					// Import failed
					FNotificationInfo NotificationInfo(FText::GetEmpty());
					NotificationInfo.Text = FText::Format(LOCTEXT("LODImportFail", "Failed to import mesh for LOD {0}!"), FText::AsNumber( LODLevel ));
					NotificationInfo.ExpireDuration = 5.0f;
					FSlateNotificationManager::Get().AddNotification(NotificationInfo);

					bSuccess = false;
				}
			}
		}
		FFbxImporter->ReleaseScene();

		return bSuccess;
	}

	bool ImportStaticMeshHiResSourceModel(UStaticMesh* BaseStaticMesh, const FString& Filename, bool bAsync)
	{
		if (!BaseStaticMesh)
		{
			UE_LOGF(LogExportMeshUtils, Log, "Cannot import custom high res mesh because the staticmesh is NULL.");
			return false;
		}

		UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
		const UInterchangeSourceData* SourceData = InterchangeManager.CreateSourceData(Filename);
		const bool bInterchangeCanImportSourceData = InterchangeManager.CanTranslateSourceData(SourceData);
		if (bInterchangeCanImportSourceData)
		{
			UStaticMesh* TempStaticMesh = NewObject<UStaticMesh>(GetTransientPackage(), NAME_None, RF_Transient | RF_Public | RF_Standalone);
			TempStaticMesh->AddSourceModel();
			//Set the asset import data to pass the correct import options
			TempStaticMesh->SetAssetImportData(BaseStaticMesh->GetAssetImportData());
			//Call interchange mesh utilities to import custom LOD
			UInterchangeMeshUtilities::ImportCustomLod(TempStaticMesh, 0, SourceData, bAsync).Then([BaseStaticMesh,TempStaticMesh, Filename, bAsync](TFuture<bool> Result)
			{
				bool bResult = Result.Get();
				auto OnImportCustomLodDone = [BaseStaticMesh, TempStaticMesh, Filename, bResult]()
					{
						// Copy high res mesh from temporary static mesh to targeted one
						if (bResult && Private::CopyHighResMeshDescription(TempStaticMesh, BaseStaticMesh))
						{
							FStaticMeshSourceModel& SourceModel = BaseStaticMesh->GetHiResSourceModel();
							SourceModel.SourceImportFilename = UAssetImportData::SanitizeImportFilename(Filename, nullptr);
							SourceModel.bImportWithBaseMesh = false;

							// Notification of success
							FNotificationInfo NotificationInfo(FText::GetEmpty());
							NotificationInfo.Text = NSLOCTEXT("UnrealEd", "ImportStaticMeshHiResSourceModelSuccessful", "High res mesh imported successfully!");
							NotificationInfo.ExpireDuration = 5.0f;
							FSlateNotificationManager::Get().AddNotification(NotificationInfo);
						}
						else
						{
							// Notification of failure
							FNotificationInfo NotificationInfo(FText::GetEmpty());
							NotificationInfo.Text = NSLOCTEXT("UnrealEd", "ImportStaticMeshHiResSourceModelFail", "Failed to import high res mesh!");
							NotificationInfo.ExpireDuration = 5.0f;
							FSlateNotificationManager::Get().AddNotification(NotificationInfo);
						}

						if (TempStaticMesh)
						{
							TempStaticMesh->ClearFlags(RF_Public | RF_Standalone);
							TempStaticMesh->MarkAsGarbage();
						}
					};
				if (IsInGameThread())
				{
					OnImportCustomLodDone();
				}
				else
				{
					ensure(bAsync);
					Async(EAsyncExecution::TaskGraphMainThread, OnImportCustomLodDone);
				}
			});

			return true;
		}

		bool bSuccess = false;

		UE_LOGF(LogExportMeshUtils, Log, "Fbx Mesh loading");

		UnFbx::FFbxImporter* FFbxImporter = UnFbx::FFbxImporter::GetInstance();
		UnFbx::FFbxLoggerSetter Logger(FFbxImporter);

		UnFbx::FBXImportOptions* ImportOptions = FFbxImporter->GetImportOptions();
		UFbxStaticMeshImportData* ImportData = Private::SetupFbxImportOptions(BaseStaticMesh, ImportOptions);
		ImportOptions->StaticMeshLODGroup = NAME_None;
		ImportOptions->bImportLOD = false;

		const bool bPreventMaterialNameClash = true;
		if (!FFbxImporter->ImportFromFile(*Filename, FPaths::GetExtension(Filename), bPreventMaterialNameClash))
		{
			FFbxImporter->FlushToTokenizedErrorMessage(EMessageSeverity::Error);
		}
		else
		{
			FFbxImporter->FlushToTokenizedErrorMessage(EMessageSeverity::Warning);
			if (ImportData)
			{
				FFbxImporter->ApplyTransformSettingsToFbxNode(FFbxImporter->Scene->GetRootNode(), ImportData);
			}

			constexpr int32 TempLODLevel = 0; // We are importing as LOD0 in a temporary mesh then transferring the geometry to the high res source model
			int32 MaxLODLevel = 0;
			TArray< TUniquePtr<TArray<FbxNode*>> > MeshNodeList;

			const bool bUseLODs = false;
			PopulateFBXStaticMeshLODList(FFbxImporter, FFbxImporter->Scene->GetRootNode(), MeshNodeList, MaxLODLevel, bUseLODs);

			// Nothing found, error out
			if (MeshNodeList.Num() == 0)
			{
				FFbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText(LOCTEXT("HiResImport_NoMeshFound", "No meshes were found in file."))), FFbxErrors::Generic_Mesh_MeshNotFound);

				FFbxImporter->ReleaseScene();
				return bSuccess;
			}

			{
				UStaticMesh* TempStaticMesh = NULL;

				if (MeshNodeList.IsValidIndex(0))
				{
					TempStaticMesh = (UStaticMesh*)FFbxImporter->ImportStaticMeshAsSingle(GetTransientPackage(), *(MeshNodeList[0]), NAME_None, RF_Transient,
						ImportData, BaseStaticMesh, TempLODLevel);
				}

				// Add the imported mesh to the existing model
				if (Private::CopyHighResMeshDescription(TempStaticMesh, BaseStaticMesh))
				{
					FNotificationInfo NotificationInfo(FText::GetEmpty());
					NotificationInfo.Text = FText::Format(LOCTEXT("HiResMeshImportSuccessful", "High res mesh imported successfully!"), FText::AsNumber(0));
					NotificationInfo.ExpireDuration = 5.0f;
					FSlateNotificationManager::Get().AddNotification(NotificationInfo);

					FStaticMeshSourceModel& SourceModel = BaseStaticMesh->GetHiResSourceModel();
					SourceModel.SourceImportFilename = UAssetImportData::SanitizeImportFilename(Filename, nullptr);
					SourceModel.bImportWithBaseMesh = false;

					bSuccess = true;
				}

				if (!bSuccess) // Import failed
				{
					FNotificationInfo NotificationInfo(FText::GetEmpty());
					NotificationInfo.Text = FText::Format(LOCTEXT("HiResMeshImportFail", "Failed to import high res mesh!"), FText::AsNumber(0));
					NotificationInfo.ExpireDuration = 5.0f;
					FSlateNotificationManager::Get().AddNotification(NotificationInfo);
				}
			}
		}

		FFbxImporter->ReleaseScene();

		return bSuccess;
	}

	bool ImportSkeletalMeshLOD( class USkeletalMesh* SelectedSkelMesh, const FString& Filename, int32 LODLevel, bool bAsync)
	{
		//Make sure skeletal mesh is valid
		if (!SelectedSkelMesh)
		{
			UE_LOGF(LogExportMeshUtils, Error, "Cannot import a LOD if there is not a valid selected skeletal mesh.");
			return false;
		}

		UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
		const UInterchangeSourceData* SourceData = InterchangeManager.CreateSourceData(Filename);
		const bool bInterchangeCanImportSourceData = InterchangeManager.CanTranslateSourceData(SourceData);
		if (bInterchangeCanImportSourceData)
		{
			//Call interchange mesh utilities to import custom LOD
			UInterchangeMeshUtilities::ImportCustomLod(SelectedSkelMesh, LODLevel, SourceData, bAsync).Then([SelectedSkelMesh, LODLevel, bAsync](TFuture<bool> Result)
				{
					bool bResult = Result.Get();
					auto OnImportCustomLodDone = [SelectedSkelMesh, LODLevel, bResult]()
						{
							if (bResult)
							{
								//If we use alternate skinweight, we must re-import all profile for this LOD
								if (SelectedSkelMesh && !SelectedSkelMesh->GetSkinWeightProfiles().IsEmpty())
								{
									//Enqueue the re-import alternate skinning
									TSharedPtr<FInterchangeSkeletalMeshAlternateSkinWeightPostImportTask> SkeletalMeshPostImportTask = MakeShared<FInterchangeSkeletalMeshAlternateSkinWeightPostImportTask>(SelectedSkelMesh);
									SkeletalMeshPostImportTask->ReimportAlternateSkinWeightDelegate.BindLambda([](USkeletalMesh* SkeletalMesh, int32 LodIndex)
										{
											return FSkinWeightsUtilities::ReimportAlternateSkinWeight(SkeletalMesh, LodIndex);
										});
									SkeletalMeshPostImportTask->AddLodToReimportAlternate(LODLevel);
									UInterchangeManager::GetInterchangeManager().EnqueuePostImportTask(SkeletalMeshPostImportTask);
								}


								// Notification of success
								FNotificationInfo NotificationInfo(FText::GetEmpty());
								NotificationInfo.Text = FText::Format(NSLOCTEXT("UnrealEd", "LODImportSuccessful", "Mesh for LOD {0} imported successfully!"), FText::AsNumber(LODLevel));
								NotificationInfo.ExpireDuration = 5.0f;
								FSlateNotificationManager::Get().AddNotification(NotificationInfo);
							}
							else
							{
								// Notification of failure
								FNotificationInfo NotificationInfo(FText::GetEmpty());
								NotificationInfo.Text = FText::Format(NSLOCTEXT("UnrealEd", "MeshLODImportFail", "Failed to import mesh for LOD {0}!"), FText::AsNumber(LODLevel));
								NotificationInfo.ExpireDuration = 5.0f;
								FSlateNotificationManager::Get().AddNotification(NotificationInfo);
							}
						};

					if (IsInGameThread())
					{
						OnImportCustomLodDone();
					}
					else
					{
						ensure(bAsync);
						Async(EAsyncExecution::TaskGraphMainThread, OnImportCustomLodDone);
					}
				});
			return true;
		}

		UnFbx::FFbxImporter* FFbxImporter = UnFbx::FFbxImporter::GetInstance();

		bool bSuccess = false;

		// Check the file extension for FBX. Anything that isn't .FBX is rejected
		const FString FileExtension = FPaths::GetExtension(Filename);
		const bool bIsFBX = FCString::Stricmp(*FileExtension, TEXT("FBX")) == 0;
		bool bSceneIsCleanUp = false;
		TArray< TArray<FbxNode*>* > MeshArray;
		auto CleanUpScene = [&bSceneIsCleanUp, &MeshArray, &FFbxImporter]()
		{
			if (bSceneIsCleanUp)
			{
				return;
			}
			bSceneIsCleanUp = true;
			// Cleanup
			for (int32 i = 0; i < MeshArray.Num(); i++)
			{
				delete MeshArray[i];
			}
			FFbxImporter->ReleaseScene();
			FFbxImporter = nullptr;
		};

		//Skip none fbx file
		if (!bIsFBX)
		{
			return false;
		}

		//Import LOD using fbx importer
		{
			FScopedSkeletalMeshPostEditChange ScopePostEditChange(SelectedSkelMesh);
			UnFbx::FFbxScopedOperation FbxScopedOperation(FFbxImporter);

			//If the imported LOD already exist, we will need to reimport all the skin weight profiles
			bool bMustReimportAlternateSkinWeightProfile = false;

			// Get a list of all the clothing assets affecting this LOD so we can re-apply later
			TArray<ClothingAssetUtils::FClothingAssetMeshBinding> ClothingBindings;
			TArray<UClothingAssetBase*> ClothingAssetsInUse;
			TArray<int32> ClothingAssetSectionIndices;
			TArray<int32> ClothingAssetInternalLodIndices;

			FSkeletalMeshModel* ImportedResource = SelectedSkelMesh->GetImportedModel();
			if (ImportedResource && ImportedResource->LODModels.IsValidIndex(LODLevel))
			{
				bMustReimportAlternateSkinWeightProfile = true;
				FLODUtilities::UnbindClothingAndBackup(SelectedSkelMesh, ClothingBindings, LODLevel);
			}

			//Lambda to call to re-apply the clothing
			auto ReapplyClothing = [&SelectedSkelMesh, &ClothingBindings, &ImportedResource, &LODLevel]()
				{
					if (ImportedResource && ImportedResource->LODModels.IsValidIndex(LODLevel))
					{
						// Re-apply our clothing assets
						FLODUtilities::RestoreClothingFromBackup(SelectedSkelMesh, ClothingBindings, LODLevel);
					}
				};

			// don't import material and animation
			UnFbx::FBXImportOptions* ImportOptions = FFbxImporter->GetImportOptions();

			//Set the skeletal mesh import data from the base mesh, this make sure the import rotation transform is use when importing a LOD
			UFbxSkeletalMeshImportData* FbxImportData = Cast<UFbxSkeletalMeshImportData>(SelectedSkelMesh->GetAssetImportData());
			if (!FbxImportData)
			{
				//Convert the data if its Interchange import data
				if (UInterchangeAssetImportData* InterchangeAssetImportData = Cast<UInterchangeAssetImportData>(SelectedSkelMesh->GetAssetImportData()))
				{
					InterchangeManager.ConvertImportData(InterchangeAssetImportData, UFbxSkeletalMeshImportData::StaticClass(), reinterpret_cast<UObject**>(&FbxImportData));
				}
			}

			if (FbxImportData)
			{
				UnFbx::FBXImportOptions::ResetOptions(ImportOptions);
				// Prepare the import options
				UFbxImportUI* ReimportUI = NewObject<UFbxImportUI>();
				ReimportUI->MeshTypeToImport = FBXIT_SkeletalMesh;
				ReimportUI->Skeleton = SelectedSkelMesh->GetSkeleton();
				ReimportUI->PhysicsAsset = SelectedSkelMesh->GetPhysicsAsset();
				// Import data already exists, apply it to the fbx import options
				ReimportUI->SkeletalMeshImportData = FbxImportData;
				//Some options not supported with skeletal mesh
				ReimportUI->SkeletalMeshImportData->bBakePivotInVertex = false;
				ReimportUI->SkeletalMeshImportData->bTransformVertexToAbsolute = true;
				ApplyImportUIToImportOptions(ReimportUI, *ImportOptions);
				ImportOptions->bImportMaterials = false;
				ImportOptions->bImportTextures = false;
			}
			ImportOptions->bImportAnimations = false;
			//Adjust the option in case we import only the skinning or the geometry
			if (ImportOptions->bImportAsSkeletalSkinning)
			{
				ImportOptions->bImportMaterials = false;
				ImportOptions->bImportTextures = false;
				ImportOptions->bImportLOD = false;
				ImportOptions->bImportSkeletalMeshLODs = false;
				ImportOptions->bImportAnimations = false;
				ImportOptions->bImportMorph = false;
			}
			else if (ImportOptions->bImportAsSkeletalGeometry)
			{
				ImportOptions->bImportAnimations = false;
				ImportOptions->bUpdateSkeletonReferencePose = false;
			}

			if (!FFbxImporter->ImportFromFile(*Filename, FPaths::GetExtension(Filename), true))
			{
				ReapplyClothing();
				// Log the error message and fail the import.
				FFbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("FBXImport_ParseFailed", "FBX file parsing failed.")), FFbxErrors::Generic_FBXFileParseFailed);
			}
			else
			{
				bool bUseLODs = true;
				int32 MaxLODLevel = 0;
				TArray<FbxNode*>* MeshObject = NULL;

				//Set the build options if the BuildDat is not available so it is the same option we use to import the LOD
				if (ImportedResource && ImportedResource->LODModels.IsValidIndex(LODLevel) && !SelectedSkelMesh->HasMeshDescription(LODLevel))
				{
					FSkeletalMeshLODInfo* LODInfo = SelectedSkelMesh->GetLODInfo(LODLevel);
					if (LODInfo)
					{
						LODInfo->BuildSettings.bRecomputeNormals = !ImportOptions->ShouldImportNormals();
						LODInfo->BuildSettings.bRecomputeTangents = !ImportOptions->ShouldImportTangents();
						LODInfo->BuildSettings.bUseMikkTSpace = (ImportOptions->NormalGenerationMethod == EFBXNormalGenerationMethod::MikkTSpace) && (!ImportOptions->ShouldImportNormals() || !ImportOptions->ShouldImportTangents());
						LODInfo->BuildSettings.bComputeWeightedNormals = ImportOptions->bComputeWeightedNormals;
						LODInfo->BuildSettings.bRemoveDegenerates = ImportOptions->bRemoveDegenerates;
						LODInfo->BuildSettings.ThresholdPosition = ImportOptions->OverlappingThresholds.ThresholdPosition;
						LODInfo->BuildSettings.ThresholdTangentNormal = ImportOptions->OverlappingThresholds.ThresholdTangentNormal;
						LODInfo->BuildSettings.ThresholdUV = ImportOptions->OverlappingThresholds.ThresholdUV;
						LODInfo->BuildSettings.MorphThresholdPosition = ImportOptions->OverlappingThresholds.MorphThresholdPosition;
					}
				}

				// Populate the mesh array
				FFbxImporter->FillFbxSkelMeshArrayInScene(FFbxImporter->Scene->GetRootNode(), MeshArray, false, ImportOptions->bImportAsSkeletalGeometry || ImportOptions->bImportAsSkeletalSkinning, ImportOptions->bImportScene);

				// Nothing found, error out
				if (MeshArray.Num() == 0)
				{
					ReapplyClothing();
					FFbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("FBXImport_NoMesh", "No meshes were found in file.")), FFbxErrors::Generic_MeshNotFound);
					CleanUpScene();
					return false;
				}

				MeshObject = MeshArray[0];

				// check if there is LODGroup for this skeletal mesh
				for (int32 j = 0; j < MeshObject->Num(); j++)
				{
					FbxNode* Node = (*MeshObject)[j];
					if (Node->GetNodeAttribute() && Node->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup)
					{
						// get max LODgroup level
						if (MaxLODLevel < (Node->GetChildCount() - 1))
						{
							MaxLODLevel = Node->GetChildCount() - 1;
						}
					}
				}

				// No LODs found, switch to supporting a mesh array containing meshes instead of LODs
				if (MaxLODLevel == 0)
				{
					bUseLODs = false;
					MaxLODLevel = SelectedSkelMesh->GetLODNum();
				}

				int32 SelectedLOD = LODLevel;
				if (SelectedLOD > SelectedSkelMesh->GetLODNum())
				{
					ReapplyClothing();
					// Make sure they don't manage to select a bad LOD index
					FFbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("FBXImport_InvalidLODIdx", "Invalid mesh LOD index {0}, no prior LOD index exists"), FText::AsNumber(SelectedLOD))), FFbxErrors::Generic_Mesh_LOD_InvalidIndex);
				}
				else
				{
					TArray<FbxNode*> SkelMeshNodeArray;

					if (bUseLODs || ImportOptions->bImportMorph)
					{
						for (int32 j = 0; j < MeshObject->Num(); j++)
						{
							FbxNode* Node = (*MeshObject)[j];
							if (Node->GetNodeAttribute() && Node->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup)
							{
								TArray<FbxNode*> NodeInLod;
								if (Node->GetChildCount() > SelectedLOD)
								{
									FFbxImporter->FindAllLODGroupNode(NodeInLod, Node, SelectedLOD);
								}
								else // in less some LODGroups have less level, use the last level
								{
									FFbxImporter->FindAllLODGroupNode(NodeInLod, Node, Node->GetChildCount() - 1);
								}

								for (FbxNode* MeshNode : NodeInLod)
								{
									SkelMeshNodeArray.Add(MeshNode);
								}
							}
							else
							{
								SkelMeshNodeArray.Add(Node);
							}
						}
					}

					// Import mesh
					USkeletalMesh* TempSkelMesh = NULL;
					TArray<FName> OrderedMaterialNames;
					{
						int32 NoneNameCount = 0;
						for (const FSkeletalMaterial& Material : SelectedSkelMesh->GetMaterials())
						{
							if (Material.ImportedMaterialSlotName == NAME_None)
								NoneNameCount++;

							OrderedMaterialNames.Add(Material.ImportedMaterialSlotName);
						}
						if (NoneNameCount >= OrderedMaterialNames.Num())
						{
							OrderedMaterialNames.Empty();
						}
					}

					TSharedPtr<FExistingSkelMeshData> SkelMeshDataPtr;
					if (SelectedSkelMesh->GetLODNum() > SelectedLOD)
					{
						SelectedSkelMesh->PreEditChange(NULL);
						SkelMeshDataPtr = SkeletalMeshImportUtils::SaveExistingSkelMeshData(SelectedSkelMesh, true, SelectedLOD);
					}

					//Original fbx data storage
					TArray<FName> ImportMaterialOriginalNameData;
					TArray<FImportMeshLodSectionsData> ImportMeshLodData;
					ImportMeshLodData.AddZeroed();
					FSkeletalMeshImportData OutData;

					UnFbx::FFbxImporter::FImportSkeletalMeshArgs ImportSkeletalMeshArgs;
					ImportSkeletalMeshArgs.InParent = SelectedSkelMesh->GetOutermost();
					ImportSkeletalMeshArgs.NodeArray = bUseLODs ? SkelMeshNodeArray : *MeshObject;
					ImportSkeletalMeshArgs.Name = NAME_None;
					ImportSkeletalMeshArgs.Flags = RF_Transient;
					ImportSkeletalMeshArgs.TemplateImportData = FbxImportData;
					ImportSkeletalMeshArgs.LodIndex = SelectedLOD;
					ImportSkeletalMeshArgs.OrderedMaterialNames = OrderedMaterialNames.Num() > 0 ? &OrderedMaterialNames : nullptr;
					ImportSkeletalMeshArgs.ImportMaterialOriginalNameData = &ImportMaterialOriginalNameData;
					ImportSkeletalMeshArgs.ImportMeshSectionsData = &ImportMeshLodData[0];
					ImportSkeletalMeshArgs.OutData = &OutData;

					TempSkelMesh = (USkeletalMesh*)FFbxImporter->ImportSkeletalMesh(ImportSkeletalMeshArgs);
					// Add the new imported LOD to the existing model (check skeleton compatibility)
					if (TempSkelMesh && FFbxImporter->ImportSkeletalMeshLOD(TempSkelMesh, SelectedSkelMesh, SelectedLOD, FbxImportData))
					{
						//Update the import data for this lod
						UnFbx::FFbxImporter::UpdateSkeletalMeshImportData(SelectedSkelMesh, nullptr, SelectedLOD, &ImportMaterialOriginalNameData, &ImportMeshLodData);

						const FString SourceImportFilename = UAssetImportData::SanitizeImportFilename(Filename, nullptr);
						if (SkelMeshDataPtr)
						{
							//Setting the source filename allow to not restore the reduction settings in case we import a custom LOD over a generated LOD.
							//This value will be wipe during the restore but we put it back just after.
							SelectedSkelMesh->GetLODInfo(SelectedLOD)->SourceImportFilename = SourceImportFilename;
							SkeletalMeshImportUtils::RestoreExistingSkelMeshData(SkelMeshDataPtr, SelectedSkelMesh, SelectedLOD, false, ImportOptions->bImportAsSkeletalSkinning, ImportOptions->bResetToFbxOnMaterialConflict);
						}

						if (ImportOptions->bImportMorph)
						{
							FFbxImporter->ImportFbxMorphTarget(SkelMeshNodeArray, SelectedSkelMesh, SelectedLOD, OutData, ImportSkeletalMeshArgs.bMapMorphTargetToTimeZero);
						}

						bSuccess = true;

						// Set LOD source filename
						SelectedSkelMesh->GetLODInfo(SelectedLOD)->SourceImportFilename = SourceImportFilename;
						SelectedSkelMesh->GetLODInfo(SelectedLOD)->bImportWithBaseMesh = false;

						ReapplyClothing();

						//Must be the last step because it cleanup the fbx importer to import the alternate skinning FBX
						if (bMustReimportAlternateSkinWeightProfile)
						{
							//We cannot use anymore the FFbxImporter after the cleanup
							CleanUpScene();
							FSkinWeightsUtilities::ReimportAlternateSkinWeight(SelectedSkelMesh, SelectedLOD);
						}

						// Notification of success
						FNotificationInfo NotificationInfo(FText::GetEmpty());
						NotificationInfo.Text = FText::Format(NSLOCTEXT("UnrealEd", "LODImportSuccessful", "Mesh for LOD {0} imported successfully!"), FText::AsNumber(SelectedLOD));
						NotificationInfo.ExpireDuration = 5.0f;
						FSlateNotificationManager::Get().AddNotification(NotificationInfo);
					}
					else
					{
						ReapplyClothing();
						// Notification of failure
						FNotificationInfo NotificationInfo(FText::GetEmpty());
						NotificationInfo.Text = FText::Format(NSLOCTEXT("UnrealEd", "MeshLODImportFail2", "Failed to import mesh for LOD {0}!"), FText::AsNumber(SelectedLOD));
						NotificationInfo.ExpireDuration = 5.0f;
						FSlateNotificationManager::Get().AddNotification(NotificationInfo);
					}
				}
			}
			CleanUpScene();
		}
		return bSuccess;
	}

	FString PromptForLODImportFile(const FText& PromptTitle)
	{
		FString ChosenFilname("");

		FString ExtensionStr;
		ExtensionStr += TEXT("All model files|*.fbx;*.obj|");
		ExtensionStr += TEXT("FBX files|*.fbx|");
		ExtensionStr += TEXT("Object files|*.obj|");
		ExtensionStr += TEXT("All files|*.*");

		// First, display the file open dialog for selecting the file.
		TArray<FString> OpenFilenames;
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
		bool bOpen = false;
		if(DesktopPlatform)
		{
			bOpen = DesktopPlatform->OpenFileDialog(
				FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
				PromptTitle.ToString(),
				*FEditorDirectories::Get().GetLastDirectory(ELastDirectory::FBX),
				TEXT(""),
				*ExtensionStr,
				EFileDialogFlags::None,
				OpenFilenames
				);
		}

		// Only continue if we pressed OK and have only one file selected.
		if(bOpen)
		{
			if(OpenFilenames.Num() == 0)
			{
				UnFbx::FFbxImporter* FFbxImporter = UnFbx::FFbxImporter::GetInstance();
				FFbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("NoFileSelectedForLOD", "No file was selected for the LOD.")), FFbxErrors::Generic_Mesh_LOD_NoFileSelected);
			}
			else if(OpenFilenames.Num() > 1)
			{
				UnFbx::FFbxImporter* FFbxImporter = UnFbx::FFbxImporter::GetInstance();
				FFbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("MultipleFilesSelectedForLOD", "You may only select one file for the LOD.")), FFbxErrors::Generic_Mesh_LOD_MultipleFilesSelected);
			}
			else
			{
				ChosenFilname = OpenFilenames[0];
				FEditorDirectories::Get().SetLastDirectory(ELastDirectory::FBX, FPaths::GetPath(ChosenFilname)); // Save path as default for next time.
			}
		}
		
		return ChosenFilname;
	}

	TFuture<bool> ImportMeshLODDialog(class UObject* SelectedMesh, int32 LODLevel, bool bNotifyCB /*= true*/, bool bReimportWithNewFile /*= false*/)
	{
		TSharedPtr<TPromise<bool>> Promise = MakeShared<TPromise<bool>>();
		if(!SelectedMesh)
		{
			Promise->SetValue(false);
			return Promise->GetFuture();
		}

		USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(SelectedMesh);
		UStaticMesh* StaticMesh = Cast<UStaticMesh>(SelectedMesh);

		FString FilenameToImport("");
		//Make sure the LODLevel is valid, it should not be more then one over the existing lod count
		bool bInvalidLodIndex = false;
		if (SkeletalMesh)
		{
			if (LODLevel > SkeletalMesh->GetLODNum())
			{
				bInvalidLodIndex = true;
			}
			else
			{
				if (!bReimportWithNewFile && SkeletalMesh->IsValidLODIndex(LODLevel))
				{
					FilenameToImport = SkeletalMesh->GetLODInfo(LODLevel)->SourceImportFilename.IsEmpty() ?
						SkeletalMesh->GetLODInfo(LODLevel)->SourceImportFilename :
						UAssetImportData::ResolveImportFilename(SkeletalMesh->GetLODInfo(LODLevel)->SourceImportFilename, nullptr);
				}
			}
		}
		else if (StaticMesh)
		{
			if (LODLevel > StaticMesh->GetNumSourceModels())
			{
				bInvalidLodIndex = true;
			}
			else
			{
				if (!bReimportWithNewFile && StaticMesh->IsSourceModelValid(LODLevel))
				{
					const FStaticMeshSourceModel& SourceModel = StaticMesh->GetSourceModel(LODLevel);
					FilenameToImport = SourceModel.SourceImportFilename.IsEmpty() ?
						SourceModel.SourceImportFilename :
						UAssetImportData::ResolveImportFilename(SourceModel.SourceImportFilename, nullptr);
				}
			}
		}
		else
		{
			//We support only staticmesh and skeletalmesh asset for LOD import
			Promise->SetValue(false);
			return Promise->GetFuture();
		}

		if (bInvalidLodIndex)
		{
			UE_LOGF(LogExportMeshUtils, Warning, "ImportMeshLODDialog: Invalid mesh LOD index %d, no prior LOD index exists.", LODLevel);
			FbxMeshUtils::Private::ShowFailedToImportLodDialog(LODLevel);
			Promise->SetValue(false);
			return Promise->GetFuture();
		}

		// Check the file exists first
		bool bSourceFileExists = FPaths::FileExists(FilenameToImport);

		if (!bSourceFileExists)
		{
			//Pop a file picker that join both interchange and other format
			//Ask the user for a file path
			const UInterchangeProjectSettings* InterchangeProjectSettings = GetDefault<UInterchangeProjectSettings>();
			UInterchangeFilePickerBase* FilePicker = nullptr;

			//In runtime we do not have any pipeline configurator
#if WITH_EDITORONLY_DATA
			TSoftClassPtr <UInterchangeFilePickerBase> FilePickerClass = InterchangeProjectSettings->FilePickerClass;
			if (FilePickerClass.IsValid())
			{
				UClass* FilePickerClassLoaded = FilePickerClass.LoadSynchronous();
				if (FilePickerClassLoaded)
				{
					FilePicker = NewObject<UInterchangeFilePickerBase>(GetTransientPackage(), FilePickerClassLoaded, NAME_None, RF_NoFlags);
				}
			}
#endif
			if (FilePicker)
			{
				FInterchangeFilePickerParameters Parameters;
				Parameters.bAllowMultipleFiles = false;
				Parameters.Title = FText::Format(NSLOCTEXT("Interchange", "ImportCustomLodAsync_FilePickerTitle", "Choose a file to import a custom LOD for LOD{0}"), FText::AsNumber(LODLevel));
				Parameters.bShowAllFactoriesExtension = false;
				//Lod import support interchange format and we force fbx in case it is disabled
				Parameters.ExtraFormats = { TEXT("fbx;Filmbox") };
				TArray<FString> Filenames;
				bool bFilePickerResult = FilePicker->ScriptedFilePickerForTranslatorAssetType(EInterchangeTranslatorAssetType::Meshes, Parameters, Filenames);
				if (bFilePickerResult)
				{
					if (Filenames.Num() > 0)
					{
						FilenameToImport = Filenames[0];
						bSourceFileExists = FPaths::FileExists(FilenameToImport);
					}
					else
					{
						UE_LOGF(LogExportMeshUtils, Display, "ImportMeshLODDialog: Error when picking a file to import LOD index %d.", LODLevel);
						Promise->SetValue(false);
						return Promise->GetFuture();
					}
				}
				else
				{
					UE_LOGF(LogExportMeshUtils, Display, "ImportMeshLODDialog: User cancel import LOD index %d.", LODLevel);
					Promise->SetValue(false);
					return Promise->GetFuture();
				}
			}
		}

		if (!bSourceFileExists)
		{
			UE_LOGF(LogExportMeshUtils, Display, "ImportMeshLODDialog: Cannot import LOD index %d. The filename do not exist.", LODLevel);
			Promise->SetValue(false);
			return Promise->GetFuture();
		}

		//Convert the import data if necessary
		UInterchangeAssetImportData* SelectedInterchangeAssetImportData = nullptr;
		if (SkeletalMesh)
		{
			if (SkeletalMesh->IsValidLODIndex(LODLevel))
			{
				SelectedInterchangeAssetImportData = Cast<UInterchangeAssetImportData>(SkeletalMesh->GetAssetImportData());
			}
		}
		else if (StaticMesh)
		{
			if (LODLevel >= 0 && LODLevel <= StaticMesh->GetNumSourceModels())
			{
				SelectedInterchangeAssetImportData = Cast<UInterchangeAssetImportData>(StaticMesh->GetAssetImportData());
			}
		}

		UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
		const UInterchangeSourceData* SourceData = InterchangeManager.CreateSourceData(FilenameToImport);
		const bool bInterchangeCanImportSourceData = InterchangeManager.CanTranslateSourceData(SourceData);

		if (bInterchangeCanImportSourceData)
		{
			constexpr bool bAsyncTrue = true;
			TFuture<bool> Result = UInterchangeMeshUtilities::ImportCustomLod(SelectedMesh, LODLevel, SourceData, bAsyncTrue);

			Result.Then([Promise, bNotifyCB, SkeletalMesh, StaticMesh, LODLevel](TFuture<bool> FutureResult)
				{
					check(IsInGameThread());
					bool bResult = FutureResult.Get();
					if (bResult)
					{
						//If we use alternate skinweight, we must re-import all profile for this LOD
						if (SkeletalMesh && !SkeletalMesh->GetSkinWeightProfiles().IsEmpty())
						{
							//Enqueue the re-import alternate skinning
							TSharedPtr<FInterchangeSkeletalMeshAlternateSkinWeightPostImportTask> SkeletalMeshPostImportTask = MakeShared<FInterchangeSkeletalMeshAlternateSkinWeightPostImportTask>(SkeletalMesh);
							SkeletalMeshPostImportTask->ReimportAlternateSkinWeightDelegate.BindLambda([](USkeletalMesh* SkeletalMesh, int32 LodIndex)
								{
									return FSkinWeightsUtilities::ReimportAlternateSkinWeight(SkeletalMesh, LodIndex);
								});
							SkeletalMeshPostImportTask->AddLodToReimportAlternate(LODLevel);
							UInterchangeManager::GetInterchangeManager().EnqueuePostImportTask(SkeletalMeshPostImportTask);
						}
						if (bNotifyCB)
						{
							if (SkeletalMesh)
							{
								GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostLODImport(SkeletalMesh, LODLevel);
							}
							else if (StaticMesh)
							{
								GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostLODImport(StaticMesh, LODLevel);
							}
						}
					}
					else
					{
						FbxMeshUtils::Private::ShowFailedToImportLodDialog(LODLevel);
					}
					Promise->SetValue(bResult);
				});

			//Since we start an asynchronous lod import we can return the future of the promise.
			return Promise->GetFuture();
		}
		
		bool bImportSuccess = false;
		if(!FilenameToImport.IsEmpty())
		{
			constexpr bool bAsyncFalse = false;
			if(SkeletalMesh)
			{
				bImportSuccess = ImportSkeletalMeshLOD(SkeletalMesh, FilenameToImport, LODLevel, bAsyncFalse);
			}
			else if(StaticMesh)
			{
				bImportSuccess = ImportStaticMeshLOD(StaticMesh, FilenameToImport, LODLevel, bAsyncFalse);
			}
		}

		//If the filename is empty it mean the user cancel the file selection
		if(!bImportSuccess && !FilenameToImport.IsEmpty())
		{
			// Failed to import a LOD, even after retries (if applicable)
			FbxMeshUtils::Private::ShowFailedToImportLodDialog(LODLevel);
		}

		if (bImportSuccess && bNotifyCB)
		{
			if (SkeletalMesh)
			{
				GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostLODImport(SkeletalMesh, LODLevel);
			}				
			else if(StaticMesh)
			{
				GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostLODImport(StaticMesh, LODLevel);
			}
		}

		Promise->SetValue(bImportSuccess);
		return Promise->GetFuture();
	}

	bool ImportStaticMeshHiResSourceModelDialog( UStaticMesh* StaticMesh )
	{
		if (!StaticMesh)
		{
			return false;
		}

		// TODO: Interchange support

		FString FilenameToImport("");

		const FStaticMeshSourceModel& SourceModel = StaticMesh->GetHiResSourceModel();
		FilenameToImport = SourceModel.SourceImportFilename.IsEmpty() ?
			SourceModel.SourceImportFilename :
			UAssetImportData::ResolveImportFilename(SourceModel.SourceImportFilename, nullptr);

		// Check if the file exists first
		const bool bSourceFileExists = FPaths::FileExists(FilenameToImport);

		// We'll give the user a chance to choose a new file if a previously set file fails to import
		const bool bPromptOnFail = bSourceFileExists;

		if (!bSourceFileExists || FilenameToImport.IsEmpty())
		{
			FText PromptTitle;

			if (FilenameToImport.IsEmpty())
			{
				PromptTitle = LOCTEXT("HiResImportPrompt_NoSource", "Choose a file to import for the High Resolution Mesh");
			}
			else if (!bSourceFileExists)
			{
				PromptTitle = LOCTEXT("HiResImportPrompt_SourceNotFound", "High Resolution Mesh Source file not found. Choose a new file.");
			}

			FilenameToImport = PromptForLODImportFile(PromptTitle);
		}

		bool bImportSuccess = false;
		constexpr bool bAsyncFalse = false;

		if (!FilenameToImport.IsEmpty())
		{
			bImportSuccess = ImportStaticMeshHiResSourceModel(StaticMesh, FilenameToImport, bAsyncFalse);
		}

		if (!bImportSuccess && bPromptOnFail)
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("HiResImport_SourceMissingDialog", "Failed to import the High Resolution Mesh as the source file failed to import, please select a new source file."));

			FText PromptTitle = LOCTEXT("HiResImportPrompt_SourceFailed", "Failed to import source file for the High Resolution Mesh, choose a new file");
			FilenameToImport = PromptForLODImportFile(PromptTitle);

			if (FilenameToImport.Len() > 0 && FPaths::FileExists(FilenameToImport))
			{
				bImportSuccess = ImportStaticMeshHiResSourceModel(StaticMesh, FilenameToImport, bAsyncFalse);
			}
		}

		if (!bImportSuccess && !FilenameToImport.IsEmpty())
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("HiResImport_Failure", "Failed to import the High Resolution Mesh"));
		}

		return bImportSuccess;
	}

	bool RemoveStaticMeshHiRes(UStaticMesh* StaticMesh)
	{
		if (!StaticMesh || !StaticMesh->IsHiResMeshDescriptionValid())
		{
			return false;
		}

		StaticMesh->Modify();

		StaticMesh->ModifyHiResMeshDescription();
		StaticMesh->ClearHiResMeshDescription();
		StaticMesh->CommitHiResMeshDescription();

		StaticMesh->GetHiResSourceModel().SourceImportFilename.Empty();

		StaticMesh->PostEditChange();
		return true;
	}

	void SetImportOption(UFbxImportUI* ImportUI)
	{
		UnFbx::FFbxImporter* FFbxImporter = UnFbx::FFbxImporter::GetInstance();
		UnFbx::FBXImportOptions* ImportOptions = FFbxImporter->GetImportOptions();
		ApplyImportUIToImportOptions(ImportUI, *ImportOptions);
	}
}  //end namespace MeshUtils

#undef LOCTEXT_NAMESPACE