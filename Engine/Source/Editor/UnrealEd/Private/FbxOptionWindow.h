// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/FbxImportUI.h"
#include "HAL/PlatformCrt.h"
#include "Input/Events.h"
#include "Input/Reply.h"
#include "InputCoreTypes.h"
#include "Internationalization/Text.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"

class SButton;
struct FGeometry;

class SFbxOptionWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SFbxOptionWindow )
		: _ImportUI(NULL)
		, _WidgetWindow()
		, _FullPath()
		, _ForcedImportType()
		, _IsObjFormat(false)
		, _MaxWindowHeight(0.0f)
		, _MaxWindowWidth(0.0f)
		{}

		SLATE_ARGUMENT( UFbxImportUI*, ImportUI )
		SLATE_ARGUMENT( TSharedPtr<SWindow>, WidgetWindow )
		SLATE_ARGUMENT( FText, FullPath )
		SLATE_ARGUMENT( TOptional<EFBXImportType>, ForcedImportType )
		SLATE_ARGUMENT( bool, IsObjFormat )
		SLATE_ARGUMENT( float, MaxWindowHeight)
		SLATE_ARGUMENT(float, MaxWindowWidth)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);
	virtual bool SupportsKeyboardFocus() const override { return true; }

	FReply OnImport()
	{
		bShouldImport = true;
		if ( WidgetWindow.IsValid() )
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	FReply OnImportAll()
	{
		bShouldImportAll = true;
		return OnImport();
	}

	FReply OnCancel()
	{
		bShouldImport = false;
		bShouldImportAll = false;
		if ( WidgetWindow.IsValid() )
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override
	{
		if( InKeyEvent.GetKey() == EKeys::Escape )
		{
			return OnCancel();
		}

		return FReply::Unhandled();
	}

	bool ShouldImport() const
	{
		return bShouldImport;
	}

	bool ShouldImportAll() const
	{
		return bShouldImportAll;
	}

	SFbxOptionWindow() 
		: ImportUI(NULL)
		, bShouldImport(false)
		, bShouldImportAll(false)
	{}
		
private:
	EActiveTimerReturnType SetFocusPostConstruct(double InCurrentTime, float InDeltaTime);
	bool CanImport() const;
	FReply OnResetToDefaultClick() const;
	FText GetImportTypeDisplayText() const;

private:
	UFbxImportUI*	ImportUI;
	TSharedPtr<class IDetailsView> DetailsView;
	TWeakPtr< SWindow > WidgetWindow;
	TSharedPtr<SButton> ImportAllButton;
	bool			bShouldImport;
	bool			bShouldImportAll;
	bool			bIsObjFormat;
};
