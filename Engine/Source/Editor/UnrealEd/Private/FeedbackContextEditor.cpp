// Copyright Epic Games, Inc. All Rights Reserved.


#include "FeedbackContextEditor.h"
#include "HAL/PlatformSplash.h"
#include "Modules/ModuleManager.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "RenderingThread.h"
#include "RenderDeferredCleanup.h"
#include "RHI.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Styling/CoreStyle.h"
#include "Styling/AppStyle.h"
#include "Editor.h"
#include "Dialogs/SBuildProgress.h"
#include "Interfaces/IMainFrameModule.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Engine/Engine.h"
#include "AnalyticsEventAttribute.h"
#include "Styling/StarshipCoreStyle.h"

/** Called to cancel the slow task activity */
DECLARE_DELEGATE( FOnCancelClickedDelegate );

/** Called when checking if the cancel button should be enabled */
DECLARE_DELEGATE_RetVal(bool, FReceiveUserCancelDelegate);

/**
 * Simple "slow task" widget
 */
class SSlowTaskWidget : public SBorder
{
	/** The maximum number of secondary bars to show on the widget */
	static const int32 MaxNumSecondaryBars = 1;

	/** The width of the dialog, and horizontal padding */
	static const int32 FixedWidth = 600, FixedPaddingH = 24;

	/** The heights of the progress bars on this widget */
	static const int32 MainBarHeight = 12, SecondaryBarHeight = 3;
public:
	SLATE_BEGIN_ARGS( SSlowTaskWidget )	{ }

		/** Called to when an asset is clicked */
		SLATE_EVENT( FOnCancelClickedDelegate, OnCancelClickedDelegate )

		/** Called when checking if the cancel button should be enabled */
		SLATE_EVENT( FReceiveUserCancelDelegate, ReceiveUserCancelDelegate)

		/** The feedback scope stack that we are presenting to the user */
		SLATE_ARGUMENT(TWeakPtr<FSlowTaskStack>, ScopeStack)

	SLATE_END_ARGS()

	/** Construct this widget */
	void Construct( const FArguments& InArgs )
	{
		OnCancelClickedDelegate = InArgs._OnCancelClickedDelegate;
		ReceiveUserCancelDelegate = InArgs._ReceiveUserCancelDelegate;
		WeakStack = InArgs._ScopeStack;

		// This is a temporary widget that needs to be updated over its entire lifespan => has an active timer registered for its entire lifespan
		RegisterActiveTimer( 0.f, FWidgetActiveTimerDelegate::CreateSP( this, &SSlowTaskWidget::UpdateProgress ) );

		TSharedRef<SVerticalBox> VerticalBox = SNew(SVerticalBox)

			// Construct the main progress bar and text
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(FMargin(0, 0, 0, 5.f))
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.HeightOverride(24.f)
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						[
							SNew(STextBlock)
							.AutoWrapText(true)
							.Text(this, &SSlowTaskWidget::GetProgressText, 0)
							// The main font size dynamically changes depending on the content
							.Font(this, &SSlowTaskWidget::GetMainTextFont)
						]

						+ SHorizontalBox::Slot()
						.Padding(FMargin(5.f, 0, 0, 0))
						.AutoWidth()
						[
							SNew(STextBlock )
							.Text(this, &SSlowTaskWidget::GetPercentageText)
							// The main font size dynamically changes depending on the content
							.Font(FStarshipCoreStyle::GetDefaultFontStyle("NormalFont", 14))
						]
					]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SBox)
					.HeightOverride(static_cast<float>(MainBarHeight))
					[
						SNew(SProgressBar)
						.BorderPadding(FVector2D::ZeroVector)
						.Percent(this, &SSlowTaskWidget::GetProgressFraction, 0)
					]
				]
			]
			
			// Secondary progress bars
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(0.f, 8.f, 0.f, 0.f))
			[
				SAssignNew(SecondaryBars, SVerticalBox)
			];
		

		if ( OnCancelClickedDelegate.IsBound() )
		{
			VerticalBox->AddSlot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				.Padding(10.0f, 7.0f)
				[
					SNew(SButton)
					.Text( NSLOCTEXT("FeedbackContextProgress", "Cancel", "Cancel") )
					.HAlign(EHorizontalAlignment::HAlign_Center)
					.OnClicked(this, &SSlowTaskWidget::OnCancel)
					.IsEnabled(this, &SSlowTaskWidget::GetCancelEnabledState)
				];
		}

		SBorder::Construct( SBorder::FArguments()
			.BorderImage(FAppStyle::GetBrush("Brushes.Header"))
			.VAlign(VAlign_Center)
			.Padding(FMargin(FixedPaddingH))
			[
				SNew(SBox).
				WidthOverride(static_cast<float>(FixedWidth))
				[
					VerticalBox 
				]
			]
		);

		for (int SecondaryBarIndex = 0; SecondaryBarIndex < MaxNumSecondaryBars; ++SecondaryBarIndex)
		{
			CreateSecondaryBar(SecondaryBarIndex + 1); // Index 0 is the main bar
		}

		// Make sure all our bars are set up
		UpdateDynamicProgressBars();
	}

private:

	/** Active timer to update the progress bars */
	EActiveTimerReturnType UpdateProgress(double InCurrentTime, float InDeltaTime)
	{
		UpdateDynamicProgressBars();

		return EActiveTimerReturnType::Continue;
	}

	/** Updates the dynamic progress bars for this widget */
	void UpdateDynamicProgressBars()
	{
		TSharedPtr<FSlowTaskStack> ScopeStackPtr = WeakStack.Pin();
		if (!ScopeStackPtr.IsValid())
		{
			return;
		}
		const FSlowTaskStack &ScopeStack = *(ScopeStackPtr.Get());

		static const double VisibleScopeThreshold = 0.5;

		DynamicProgressIndices.Reset(MaxNumSecondaryBars + 1); // Add one for the main bar
		
		// Always show the first one
		DynamicProgressIndices.Emplace(0);

		const int32 ScopeCount = ScopeStack.Num();

		// Start from the top of the stack to find the newest Important tasks, if there are any
		for (int32 ReverseScopeIndex = ScopeCount - 1; ReverseScopeIndex > 0 && DynamicProgressIndices.Num() <= MaxNumSecondaryBars; --ReverseScopeIndex)
		{
			const FSlowTask *const Scope = ScopeStack[ReverseScopeIndex];
			if (Scope->Visibility == ESlowTaskVisibility::Important)
			{
				if (Scope->DefaultMessage.IsEmpty())
				{
					UE_LOGF(LogSlate, Warning, "An important slow task had no default message, if this is intended this warning can be ignored.");
				}

				DynamicProgressIndices.Emplace(ReverseScopeIndex);
			}
		}

		// The Importants were added in reverse order, make sure to skip the first element: Data() + 1
		Algo::Reverse(DynamicProgressIndices.GetData() + 1, DynamicProgressIndices.Num() - 1);
				
		for (int32 ScopeIndex = 1, InsertIndex = 1; ScopeIndex < ScopeCount && DynamicProgressIndices.Num() <= MaxNumSecondaryBars; ++ScopeIndex)
		{
			const FSlowTask *const Scope = ScopeStack[ScopeIndex];
			switch (Scope->Visibility)
			{
				case ESlowTaskVisibility::Default:
				{
					if (!Scope->DefaultMessage.IsEmpty())
					{
						const double TimeOpen = FPlatformTime::Seconds() - Scope->StartTime;

						// We only show default scopes if they have been opened a while
						if (TimeOpen > VisibleScopeThreshold)
						{
							DynamicProgressIndices.EmplaceAt(InsertIndex, ScopeIndex);
							++InsertIndex;
						}
					}

					break;
				}

				case ESlowTaskVisibility::ForceVisible:
				{
					if (Scope->DefaultMessage.IsEmpty())
					{
						UE_LOGF(LogSlate, Warning, "A forced visible slow task had no default message, if this is intended this warning can be ignored.");
					}

					DynamicProgressIndices.EmplaceAt(InsertIndex, ScopeIndex);
					++InsertIndex;

					break;
				}

				case ESlowTaskVisibility::Important: // These have already been added
				{
					// Increase the insert index to start inserting after the already added Important task
					++InsertIndex;
					break;
				}

				// ESlowTaskVisibility::Invisible:
				default:
				{
					break;
				}
			}
		}
	}

	/** Create a progress bar for the specified index */
	void CreateSecondaryBar(int32 Index) 
	{
		SecondaryBars->AddSlot()
		.Padding( 0.f, 16.f, 0.f, 0.f )
		[
			SNew(SVerticalBox)
			.Visibility( this, &SSlowTaskWidget::GetSecondaryBarVisibility, Index )
			+ SVerticalBox::Slot()
			.Padding( 0.f, 0.f, 0.f, 4.f )
			.AutoHeight()
			[
				SNew( STextBlock )
				.Text( this, &SSlowTaskWidget::GetProgressText, Index )
				.ColorAndOpacity( FSlateColor::UseSubduedForeground() )
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBox)
				.HeightOverride(static_cast<float>(SecondaryBarHeight))
				[
					SNew(SProgressBar)
					.BorderPadding(FVector2D::ZeroVector)
					.Percent( this, &SSlowTaskWidget::GetProgressFraction, Index )
				]
			]
		];
	}

private:

	/** The main text that we will display in the window */
	FText GetPercentageText() const
	{
		auto ScopeStack = WeakStack.Pin();
		if (ScopeStack.IsValid())
		{
			const float ProgressInterp = ScopeStack->GetProgressFraction(0);
			return FText::AsPercent(ProgressInterp);
		}
		return FText();
	}

	/** Calculate the best font to display the main text with */
	FSlateFontInfo GetMainTextFont() const
	{
		TSharedRef<FSlateFontMeasure> MeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

		const int32 MaxFontSize = 14;
		FSlateFontInfo FontInfo = FStarshipCoreStyle::GetDefaultFontStyle("NormalFont", MaxFontSize);

		const FText MainText = GetProgressText(0);
		const int32 MaxTextWidth = FixedWidth - FixedPaddingH*2;
		while( FontInfo.Size > 9 && MeasureService->Measure(MainText, FontInfo).X > MaxTextWidth )
		{
			FontInfo.Size -= 2;
		}

		return FontInfo;
	}

	/** Get the tint for a secondary progress bar */
	FLinearColor GetSecondaryProgressBarTint(int32 Index) const
	{
		auto ScopeStack = WeakStack.Pin();
		if (ScopeStack.IsValid())
		{
			if (!DynamicProgressIndices.IsValidIndex(Index) || !ScopeStack->IsValidIndex(DynamicProgressIndices[Index]))
			{
				return FLinearColor::White.CopyWithNewOpacity(0.25f);
			}
		}
		return FLinearColor::White;
	}

	/** Get the fractional percentage of completion for a progress bar */
	TOptional<float> GetProgressFraction(int32 Index) const
	{
		auto ScopeStack = WeakStack.Pin();
		if (ScopeStack.IsValid())
		{
			if (DynamicProgressIndices.IsValidIndex(Index) && ScopeStack->IsValidIndex(DynamicProgressIndices[Index]))
			{
				return ScopeStack->GetProgressFraction(DynamicProgressIndices[Index]);
			}
		}
		return TOptional<float>();
	}

	/** Get the text to display for a progress bar */
	FText GetProgressText(int32 Index) const
	{
		auto ScopeStack = WeakStack.Pin();
		if (ScopeStack.IsValid())
		{
			if (DynamicProgressIndices.IsValidIndex(Index) && ScopeStack->IsValidIndex(DynamicProgressIndices[Index]))
			{
				return (*ScopeStack)[DynamicProgressIndices[Index]]->GetCurrentMessage();
			}
		}

		return FText();
	}

	EVisibility GetSecondaryBarVisibility(int32 Index) const
	{
		return DynamicProgressIndices.IsValidIndex(Index) ? EVisibility::HitTestInvisible : EVisibility::Hidden;
	}

	/** Called when the cancel button is clicked */
	FReply OnCancel()
	{
		OnCancelClickedDelegate.ExecuteIfBound();
		return FReply::Handled();
	}

	bool GetCancelEnabledState() const
	{
		if (ReceiveUserCancelDelegate.IsBound())
		{
			return !ReceiveUserCancelDelegate.Execute();
		}

		return true;
	}

private:

	/** Delegate to invoke if the user clicks cancel */
	FOnCancelClickedDelegate OnCancelClickedDelegate;

	/** Delegate that returns if the cancel state is already active */
	FReceiveUserCancelDelegate ReceiveUserCancelDelegate;

	/** The scope stack that we are reflecting */
	TWeakPtr<FSlowTaskStack> WeakStack;

	/** The vertical box containing the secondary progress bars */
	TSharedPtr<SVerticalBox> SecondaryBars;

	/** Array mapping progress bar index -> scope stack index. Updated every tick. */
	TArray<int32> DynamicProgressIndices;
};

/** Static integer definitions required on some builds where the linker needs access to these */
const int32 SSlowTaskWidget::MaxNumSecondaryBars;
const int32 SSlowTaskWidget::FixedWidth;
const int32 SSlowTaskWidget::FixedPaddingH;;
const int32 SSlowTaskWidget::MainBarHeight;
const int32 SSlowTaskWidget::SecondaryBarHeight;

static void TickSlate(TSharedPtr<SWindow> SlowTaskWindow)
{
	// Avoid re-entrancy by ticking the active modal window again. This can happen if the slow task window is open and a sibling modal window is open as well.  We only tick slate if we are the active modal window or a child of the active modal window
	if( SlowTaskWindow.IsValid() && ( FSlateApplication::Get().GetActiveModalWindow() == SlowTaskWindow || SlowTaskWindow->IsDescendantOf( FSlateApplication::Get().GetActiveModalWindow() ) ) )
	{
		// Testing if we are already ticking the rendering. That is to prevent a double "BeginFrame" in case the user wrongly uses the FSlateApplication::OnPreTick to start a slow task.
		bool bIsTicking = FSlateApplication::Get().IsTicking();

		// Tick Slate application
		FSlateApplication::Get().Tick();

		// End frame so frame fence number gets incremented
		if (!bIsTicking && GIsRHIInitialized)
		{
			ENQUEUE_RENDER_COMMAND(EndFrameCmd)([](FRHICommandListImmediate& RHICmdList) { RHICmdList.EndFrame(); });
		}

		// Sync the game thread and the render thread. This is needed if many StatusUpdate are called
		FSlateApplication::Get().GetRenderer()->Sync();
	}
}

FFeedbackContextEditor::FFeedbackContextEditor()
	: HasTaskBeenCancelled(false)
{
	
}

static bool IsFallbackSplashScreenAllowed()
{
#if WITH_EDITOR
	return !FApp::IsUnattended() && !IsRunningDedicatedServer() && FPlatformProcess::SupportsMultithreading() && !FParse::Param(FCommandLine::Get(), TEXT("RenderOffScreen"));
#else
	return false;
#endif
}

void FFeedbackContextEditor::StartSlowTask( const FText& Task, bool bShowCancelButton )
{
	FFeedbackContext::StartSlowTask( Task, bShowCancelButton );

	if (GEditor)
	{
		// Don't show a dialog in the middle of a drag/drop operation, as it will cause reentrancy into user
		// drag operations:
		if(FSlateApplication::Get().IsDragDropping())
		{
			UE_LOGF(LogSlate, Log, "Slow task dialog suppressed because a drag operation is in progress");
			return;
		}

		// reset the cancellation flag
		HasTaskBeenCancelled = false;

		// If there is a pie window and it is active attempt to parent any slow task dialogs to it to prevent the game window from falling behind due to a slowtask window opening.
		TSharedPtr<SWindow> ParentWindow;
		if (FWorldContext* PieWorldContext = GEditor->GetPIEWorldContext())
		{
			FSlatePlayInEditorInfo* SlatePlayInEditorSession = GEditor->SlatePlayInEditorMap.Find(PieWorldContext->ContextHandle);
	
			if (SlatePlayInEditorSession && SlatePlayInEditorSession->SlatePlayInEditorWindow.IsValid())
			{
				if (FSlateApplication::Get().GetActiveTopLevelWindow() == SlatePlayInEditorSession->SlatePlayInEditorWindow)
				{
					ParentWindow = SlatePlayInEditorSession->SlatePlayInEditorWindow.Pin();
				}
			}
		}

		// Attempt to parent the slow task window to the current active window
		// This is more likely to show the slow task window where it's relevant and the user is already looking, then return focus appropriately when destroyed
		if (!ParentWindow.IsValid())
		{
			ParentWindow = FSlateApplicationBase::Get().GetActiveTopLevelRegularWindow();
		}

		// If we still don't have a valid window, attempt to parent the slow task window to the slate main frame
		if (!ParentWindow.IsValid() && FModuleManager::Get().IsModuleLoaded("MainFrame"))
		{
			IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
			ParentWindow = MainFrame.GetParentWindow();
		}

		if (ParentWindow.IsValid())
		{
			GSlowTaskOccurred = GIsSlowTask;

			// Don't show the progress dialog if the Build Progress dialog is already visible
			bool bProgressWindowShown = BuildProgressWidget.IsValid();

			// Don't show the progress dialog if a Slate menu is currently open
			const bool bHaveOpenMenu = FSlateApplication::Get().AnyMenusVisible();

			if (bHaveOpenMenu)
			{
				UE_LOGF(LogSlate, Log, "Prevented a slow task dialog from being summoned while a context menu was open");
			}

			if (!bProgressWindowShown && !bHaveOpenMenu)
			{
				if (FSlateApplication::Get().CanDisplayWindows())
				{
					FOnCancelClickedDelegate OnCancelClicked;
					if (bShowCancelButton)
					{
						// The cancel button is only displayed if a delegate is bound to it.
						OnCancelClicked = FOnCancelClickedDelegate::CreateRaw(this, &FFeedbackContextEditor::OnUserCancel);
					}

					const bool bFocus = FApp::HasFocus();
					FReceiveUserCancelDelegate ReceiveUserCancelDelegate = FReceiveUserCancelDelegate::CreateRaw(this, &FFeedbackContextEditor::ReceivedUserCancel);
					TSharedRef<SWindow> SlowTaskWindowRef = SNew(SWindow)
						.SizingRule(ESizingRule::Autosized)
						.AutoCenter(EAutoCenter::PreferredWorkArea)
						.IsPopupWindow(true)
						.CreateTitleBar(true)
						.ActivationPolicy(bFocus ? EWindowActivationPolicy::Always : EWindowActivationPolicy::Never)
						.FocusWhenFirstShown(bFocus);

					SlowTaskWindowRef->SetContent(
						SNew(SSlowTaskWidget)
						.ScopeStack(GetScopeStackSharedPtr())
						.OnCancelClickedDelegate(OnCancelClicked)
						.ReceiveUserCancelDelegate(ReceiveUserCancelDelegate)
					);

					SlowTaskWindow = SlowTaskWindowRef;

					const bool bSlowTask = true;
					FSlateApplication::Get().AddModalWindow(SlowTaskWindowRef, ParentWindow, bSlowTask);

					SlowTaskWindowRef->ShowWindow();

					TickSlate(SlowTaskWindow.Pin());
				}
				else if(IsFallbackSplashScreenAllowed() && !FPlatformSplash::IsShown())
				{
					bFallbackSplashScreenVisible = true;
					FPlatformSplash::Show();
				}
			}

			FPlatformSplash::SetSplashText(SplashTextType::StartupProgress, Task);
		}
	}
}

void FFeedbackContextEditor::FinalizeSlowTask()
{
	auto Window = SlowTaskWindow.Pin();
	if (Window.IsValid())
	{	
		Window->SetContent(SNullWidget::NullWidget);
		Window->RequestDestroyWindow();
		SlowTaskWindow.Reset();
	}
	if (bFallbackSplashScreenVisible)
	{
		FPlatformSplash::Hide();
	}

	FFeedbackContext::FinalizeSlowTask( );
}

void FFeedbackContextEditor::ProgressReported( const float TotalProgressInterp, FText DisplayMessage )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FFeedbackContextEditor::ProgressReported);

	if (!(FPlatformSplash::IsShown() || BuildProgressWidget.IsValid() || SlowTaskWindow.IsValid()))
	{
		return;
	}

	// Clean up deferred cleanup objects from rendering thread every once in a while.
	static double LastTimePendingCleanupObjectsWhereDeleted;
	if( FPlatformTime::Seconds() - LastTimePendingCleanupObjectsWhereDeleted > 1 )
	{
		// Get list of objects that are pending cleanup.
		FPendingCleanupObjects* PendingCleanupObjects = GetPendingCleanupObjects();
		if (!PendingCleanupObjects->IsEmpty())
		{
			// Flush rendering commands in the queue.
			FlushRenderingCommands();
		}
		// It is now safe to delete the pending clean objects.
		delete PendingCleanupObjects;
		// Keep track of time this operation was performed so we don't do it too often.
		LastTimePendingCleanupObjectsWhereDeleted = FPlatformTime::Seconds();
	}

	if (BuildProgressWidget.IsValid() || SlowTaskWindow.IsValid())
	{
		// CanDisplayWindows can be slow when called repeatedly, so we only call it if a window is open
		if (!FSlateApplication::Get().CanDisplayWindows())
		{
			return;
		}

		if (BuildProgressWidget.IsValid())
		{
			if (!DisplayMessage.IsEmpty())
			{
				BuildProgressWidget->SetBuildStatusText(DisplayMessage);
			}

			BuildProgressWidget->SetBuildProgressPercent(static_cast<int32>(TotalProgressInterp * 100), 100);
			TickSlate(BuildProgressWindow.Pin());
		}
		else if (SlowTaskWindow.IsValid())
		{
			TickSlate(SlowTaskWindow.Pin());
		}
	}
	else if (FPlatformSplash::IsShown())
	{
		// look for important messages:
		bool bFoundImportantMessage = false;
		for (int32 i = ScopeStack.Num() - 1; i > -1; --i)
		{
			if (ScopeStack[i]->Visibility == ESlowTaskVisibility::Important)
			{
				const FText ThisMessage = ScopeStack[i]->GetCurrentMessage();
				if (!ThisMessage.IsEmpty())
				{
					bFoundImportantMessage = true;
					DisplayMessage = ThisMessage;
					break;
				}
			}
		}

		// If nothing important, always show the top-most message
		if (!bFoundImportantMessage)
		{
			for (int32 i = ScopeStack.Num() - 1; i > -1; --i)
			{
				const FText ThisMessage = ScopeStack[i]->GetCurrentMessage();
				if (!ThisMessage.IsEmpty())
				{
					DisplayMessage = ThisMessage;
					break;
				}
			}
		}

		int DisplayProgress = 0;

		if (!DisplayMessage.IsEmpty())
		{
			const int32 DotCount = 4;
			const float MinTimeBetweenUpdates = 0.2f;
			static double LastUpdateTime = -100000.0;
			static int32 DotProgress = 0;
			const double CurrentTime = FPlatformTime::Seconds();
			if( CurrentTime - LastUpdateTime >= MinTimeBetweenUpdates )
			{
				LastUpdateTime = CurrentTime;
				DotProgress = ( DotProgress + 1 ) % DotCount;
			}

			FString NewDisplayMessage = DisplayMessage.ToString();
			NewDisplayMessage.RemoveFromEnd( TEXT( "..." ) );
			for( int32 DotIndex = 0; DotIndex <= DotCount; ++DotIndex )
			{
				if( DotIndex <= DotProgress )
				{
					NewDisplayMessage.AppendChar( TCHAR( '.' ) );
				}
				else
				{
					NewDisplayMessage.AppendChar( TCHAR( ' ' ) );
				}
			}

			DisplayProgress = int(TotalProgressInterp * 100.f);
			DisplayMessage = FText::Format(NSLOCTEXT("FeedbackContextProgress", "StartupProgress", "{0} - {1}"), FText::AsPercent(TotalProgressInterp), FText::AsCultureInvariant(NewDisplayMessage));
		}

		FPlatformSplash::SetProgress(DisplayProgress);
		FPlatformSplash::SetSplashText(SplashTextType::StartupProgress, DisplayMessage);
	}
}

bool FFeedbackContextEditor::ReceivedUserCancel( void )
{
	return HasTaskBeenCancelled;
}

void FFeedbackContextEditor::OnUserCancel()
{
	HasTaskBeenCancelled = true;
}

/** 
 * Show the Build Progress Window 
 * @return Handle to the Build Progress Widget created
 */
TWeakPtr<class SBuildProgressWidget> FFeedbackContextEditor::ShowBuildProgressWindow()
{
	TSharedRef<SWindow> BuildProgressWindowRef = SNew(SWindow)
		.ClientSize(FVector2D(600,200))
		.IsPopupWindow(true);

	BuildProgressWidget = SNew(SBuildProgressWidget);
		
	BuildProgressWindowRef->SetContent( BuildProgressWidget.ToSharedRef() );

	BuildProgressWindow = BuildProgressWindowRef;
				
	// Attempt to parent the slow task window to the slate main frame
	TSharedPtr<SWindow> ParentWindow;

	if( FModuleManager::Get().IsModuleLoaded( "MainFrame" ) )
	{
		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>( "MainFrame" );
		ParentWindow = MainFrame.GetParentWindow();
	}

	FSlateApplication::Get().AddModalWindow( BuildProgressWindowRef, ParentWindow, true );
	BuildProgressWindowRef->ShowWindow();	

	BuildProgressWidget->MarkBuildStartTime();
	
	if (FSlateApplication::Get().CanDisplayWindows())
	{
		TickSlate(BuildProgressWindow.Pin());
	}

	return BuildProgressWidget;
}

/** Close the Build Progress Window */
void FFeedbackContextEditor::CloseBuildProgressWindow()
{
	if( BuildProgressWindow.IsValid() && BuildProgressWidget.IsValid())
	{
		BuildProgressWindow.Pin()->RequestDestroyWindow();
		BuildProgressWindow.Reset();
		BuildProgressWidget.Reset();	
	}
}

bool FFeedbackContextEditor::IsPlayingInEditor() const
{
	return (GIsPlayInEditorWorld || (GEditor && GEditor->PlayWorld != nullptr));
}
