// Copyright Epic Games, Inc. All Rights Reserved.

#include "FileHelpersInternal.h"

namespace UE::FileHelpers::Internal
{

FGetAdditionalInteractiveSavePackageCandidates GetAdditionalInteractiveSavePackageCandidates;
FOnPreInteractiveCheckoutPackages OnPreInteractiveCheckoutPackages;
FOnPostInteractiveCheckoutPackages OnPostInteractiveCheckoutPackages;
FOnPackagesInteractivelyCheckedOut OnPackagesInteractivelyCheckedOut;
FOnPackagesInteractivelyMadeWritable OnPackagesInteractivelyMadeWritable;
FOnPackagesInteractivelyDiscarded OnPackagesInteractivelyDiscarded;

}
