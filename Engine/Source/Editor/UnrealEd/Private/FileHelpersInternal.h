// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

namespace UE::FileHelpers::Internal
{

/**
 * Low-level delegate called by FEditorFileUtils::GetDirtyXPackages to collect additional candidates to save if dirty.
 * @note This doesn't forcibly add any candidates to the dirty/save lists, but may allow them to bypass certain filtering that would have otherwise removed them (eg, to allow an empty actor package).
 * @param OutPackages The set of additional candidates, to be filled by the callback.
 */
using FGetAdditionalInteractiveSavePackageCandidates = TMulticastDelegate<void(TSet<UPackage*>& OutPackages)>;
extern FGetAdditionalInteractiveSavePackageCandidates GetAdditionalInteractiveSavePackageCandidates;

/**
 * Low-level delegate called by FEditorFileUtils::PromptToCheckoutPackagesInternal prior to deciding which packages need check out.
 * @param Packages The list of packages that will be considered for checkout.
 * @param OutReadOnlyPackages The set of packages to be considered read-only by FEditorFileUtils::AddCheckoutPackageItems, even if they're not actually read-only on disk.
 */
using FOnPreInteractiveCheckoutPackages = TMulticastDelegate<void(const TArray<UPackage*>& Packages, TSet<FName>& OutReadOnlyPackages)>;
extern FOnPreInteractiveCheckoutPackages OnPreInteractiveCheckoutPackages;

/**
 * Low-level delegate called by FEditorFileUtils::PromptToCheckoutPackagesInternal after the dialog has been closed, and after packages may have been checked out or made writable.
 * @param Packages The list of packages that were considered for checkout.
 * @param bUserResponse False if the checkout was canceled.
 */
using FOnPostInteractiveCheckoutPackages = TMulticastDelegate<void(const TArray<UPackage*>& Packages, bool bUserResponse)>;
extern FOnPostInteractiveCheckoutPackages OnPostInteractiveCheckoutPackages;

/**
 * Low-level delegate called by FEditorFileUtils::PromptToCheckoutPackagesInternal after packages have been checked out.
 * @param Packages The list of packages that were checked out.
 */
using FOnPackagesInteractivelyCheckedOut = TMulticastDelegate<void(const TArray<UPackage*>& Packages)>;
extern FOnPackagesInteractivelyCheckedOut OnPackagesInteractivelyCheckedOut;

/**
 * Low-level delegate called by FEditorFileUtils::PromptToCheckoutPackagesInternal after packages have been made writable.
 * @param Packages The list of packages that were made writable.
 */
using FOnPackagesInteractivelyMadeWritable = TMulticastDelegate<void(const TArray<UPackage*>& Packages)>;
extern FOnPackagesInteractivelyMadeWritable OnPackagesInteractivelyMadeWritable;

/**
 * Low-level delegate called by FEditorFileUtils::PromptForCheckoutAndSave after packages have been discarded for save.
 * @param Packages The list of packages that were discarded.
 */
using FOnPackagesInteractivelyDiscarded = TMulticastDelegate<void(const TArray<UPackage*>& Packages)>;
extern FOnPackagesInteractivelyDiscarded OnPackagesInteractivelyDiscarded;

}
