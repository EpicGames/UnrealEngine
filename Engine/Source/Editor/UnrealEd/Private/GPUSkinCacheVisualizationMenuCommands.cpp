// Copyright Epic Games, Inc. All Rights Reserved.

#include "GPUSkinCacheVisualizationMenuCommands.h"
#include "GPUSkinCacheVisualizationData.h"
#include "Containers/UnrealString.h"
#include "Framework/Commands/InputChord.h"
#include "Internationalization/Text.h"
#include "Templates/Function.h"
#include "Styling/AppStyle.h"
#include "EditorViewportClient.h"

#define LOCTEXT_NAMESPACE "GPUSkinCacheVisualizationMenuCommands"

FGPUSkinCacheVisualizationMenuCommands::FGPUSkinCacheVisualizationMenuCommands()
	: TCommands<FGPUSkinCacheVisualizationMenuCommands>
	(
		TEXT("GPUSkinCacheVisualizationMenu"), // Context name for fast lookup
		NSLOCTEXT("Contexts", "GPUSkinCacheMenu", "GPU Skin Cache Visualization"), // Localized context name for displaying
		NAME_None, // Parent context name.  
		FAppStyle::GetAppStyleSetName() // Icon Style Set
	)
{
}

void FGPUSkinCacheVisualizationMenuCommands::BuildCommandMap()
{
	const FGPUSkinCacheVisualizationData& VisualizationData = GetGPUSkinCacheVisualizationData();
	const FGPUSkinCacheVisualizationData::TModeMap& ModeMap = VisualizationData.GetModeMap();

	CommandMap.Empty();
	for (FGPUSkinCacheVisualizationData::TModeMap::TConstIterator It = ModeMap.CreateConstIterator(); It; ++It)
	{
		const FGPUSkinCacheVisualizationData::FModeRecord& Entry = It.Value();
		FGPUSkinCacheVisualizationRecord& Record = CommandMap.Add(Entry.ModeName, FGPUSkinCacheVisualizationRecord());
		Record.Name = Entry.ModeName;
		Record.Command = FUICommandInfoDecl(
			this->AsShared(),
			Entry.ModeName,
			Entry.ModeText,
			Entry.ModeDesc)
			.UserInterfaceType(EUserInterfaceActionType::RadioButton)
			.DefaultChord(FInputChord());

		switch (Entry.ModeType)
		{
		case FGPUSkinCacheVisualizationData::FModeType::Overview:
			Record.Type = FGPUSkinCacheVisualizationType::Overview;
			break;
		case FGPUSkinCacheVisualizationData::FModeType::Memory:
			Record.Type = FGPUSkinCacheVisualizationType::Memory;
			break;
		case FGPUSkinCacheVisualizationData::FModeType::RayTracingLODOffset:
			Record.Type = FGPUSkinCacheVisualizationType::RayTracingLODOffset;
			break;
		}
	}
}

void FGPUSkinCacheVisualizationMenuCommands::BuildVisualisationSubMenu(FMenuBuilder& Menu)
{
	const FGPUSkinCacheVisualizationMenuCommands& Commands = FGPUSkinCacheVisualizationMenuCommands::Get();
	if (Commands.IsPopulated())
	{
		Menu.BeginSection("GPUSkinCacheVisualizationMode", LOCTEXT("GPUSkinCacheVisualizationHeader", "Visualization Mode"));

		Commands.AddCommandTypeToMenu(Menu, FGPUSkinCacheVisualizationType::Overview, false);
		Commands.AddCommandTypeToMenu(Menu, FGPUSkinCacheVisualizationType::Memory, false);
		Commands.AddCommandTypeToMenu(Menu, FGPUSkinCacheVisualizationType::RayTracingLODOffset, false);

		Menu.EndSection();
	}
}

bool FGPUSkinCacheVisualizationMenuCommands::AddCommandTypeToMenu(FMenuBuilder& Menu, const FGPUSkinCacheVisualizationType Type, bool bSeparatorBefore) const
{
	bool bAddedCommands = false;

	const TGPUSkinCacheVisualizationModeCommandMap& Commands = CommandMap;
	for (TCommandConstIterator It = CreateCommandConstIterator(); It; ++It)
	{
		const FGPUSkinCacheVisualizationRecord& Record = It.Value();
		if (Record.Type == Type)
		{
			if (!bAddedCommands && bSeparatorBefore)
			{
				Menu.AddMenuSeparator();
			}
			Menu.AddMenuEntry(Record.Command, NAME_None, Record.Command->GetLabel());
			bAddedCommands = true;
		}
	}

	return bAddedCommands;
}

FGPUSkinCacheVisualizationMenuCommands::TCommandConstIterator FGPUSkinCacheVisualizationMenuCommands::CreateCommandConstIterator() const
{
	return CommandMap.CreateConstIterator();
}

void FGPUSkinCacheVisualizationMenuCommands::RegisterCommands()
{
	BuildCommandMap();
}

void FGPUSkinCacheVisualizationMenuCommands::BindCommands(FUICommandList& CommandList, const TSharedPtr<FEditorViewportClient>& Client) const
{
	// Map Virtual shadow map visualization mode actions
	for (FGPUSkinCacheVisualizationMenuCommands::TCommandConstIterator It = FGPUSkinCacheVisualizationMenuCommands::Get().CreateCommandConstIterator(); It; ++It)
	{
		const FGPUSkinCacheVisualizationMenuCommands::FGPUSkinCacheVisualizationRecord& Record = It.Value();
		CommandList.MapAction(
			Record.Command,
			FExecuteAction::CreateStatic(&FGPUSkinCacheVisualizationMenuCommands::ChangeGPUSkinCacheVisualizationMode, Client.ToWeakPtr(), Record.Name),
			FCanExecuteAction(),
			FIsActionChecked::CreateStatic(&FGPUSkinCacheVisualizationMenuCommands::IsGPUSkinCacheVisualizationModeSelected, Client.ToWeakPtr(), Record.Name)
		);
	}
}

void FGPUSkinCacheVisualizationMenuCommands::ChangeGPUSkinCacheVisualizationMode(TWeakPtr<FEditorViewportClient> WeakClient, FName InName)
{
	if (TSharedPtr<FEditorViewportClient> Client = WeakClient.Pin())
	{
		Client->ChangeGPUSkinCacheVisualizationMode(InName);
	}
}

bool FGPUSkinCacheVisualizationMenuCommands::IsGPUSkinCacheVisualizationModeSelected(TWeakPtr<FEditorViewportClient> WeakClient, FName InName)
{
	if (TSharedPtr<FEditorViewportClient> Client = WeakClient.Pin())
	{
		return Client->IsGPUSkinCacheVisualizationModeSelected(InName);
	}

	return false;
}

#undef LOCTEXT_NAMESPACE

