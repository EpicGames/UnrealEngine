// Copyright Epic Games, Inc. All Rights Reserved.

#include "GatherTextMetaDataHelper.h"

#include "Internationalization/InternationalizationManifest.h"
#include "Internationalization/Text.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Logging/LogVerbosity.h"
#include "Logging/StructuredLog.h"
#include "Misc/PackageName.h"
#include "UObject/Package.h"
#include "UObject/UObjectHash.h"

DEFINE_LOG_CATEGORY_STATIC(LogGatherTextMetaDataHelper, Log, All);

FGatherTextMetaDataHelper::FGatherTextMetaDataHelper(TSharedPtr<FLocTextHelper> InGatherManifestHelper)
	: GatherManifestHelper(MoveTemp(InGatherManifestHelper))
{
}

void FGatherTextMetaDataHelper::SetShouldGatherFromEditorOnlyData(const bool bValue)
{
	bShouldGatherFromEditorOnlyData = bValue;
}

void FGatherTextMetaDataHelper::SetGatherParameters(TArray<FString> InInputKeys, TArray<FString> InOutputNamespaces, TArray<FString> InOutputKeys)
{
	InputKeys = MoveTemp(InInputKeys);
	OutputNamespaces = MoveTemp(InOutputNamespaces);
	OutputKeys = MoveTemp(InOutputKeys);
	checkf(InputKeys.Num() == OutputNamespaces.Num(), TEXT("InputKeys must have the same number of entries as OutputNamespaces"));
	checkf(InputKeys.Num() == OutputKeys.Num(), TEXT("InputKeys must have the same number of entries as OutputKeys"));
}

bool FGatherTextMetaDataHelper::IsConfigured() const
{
	return InputKeys.Num() > 0;
}

void FGatherTextMetaDataHelper::SetFieldTypesFromStrings(const TArray<FString>& TypeNames, const bool bInclude, const TCHAR* LogContext)
{
	if (TypeNames.Num() == 0)
	{
		return;
	}

	TArray<FFieldClassFilter>& OutFieldTypes = bInclude ? FieldTypesToInclude : FieldTypesToExclude;

	TArray<FFieldClassFilter> AllFieldTypes;
	// FField types
	{
		for (const FFieldClass* FieldClass : FFieldClass::GetAllFieldClasses())
		{
			AllFieldTypes.Emplace(FieldClass);
		}
	}
	// UField types
	{
		TArray<UClass*> AllUFieldClasses;
		AllUFieldClasses.Add(UField::StaticClass());
		GetDerivedClasses(UField::StaticClass(), AllUFieldClasses);
		for (const UClass* FieldClass : AllUFieldClasses)
		{
			AllFieldTypes.Emplace(FieldClass);
		}
	}

	for (const FString& FieldTypeStr : TypeNames)
	{
		const bool bIsWildcard = FieldTypeStr.GetCharArray().Contains(TEXT('*')) || FieldTypeStr.GetCharArray().Contains(TEXT('?'));
		if (bIsWildcard)
		{
			for (const FFieldClassFilter& FieldTypeFilter : AllFieldTypes)
			{
				if (FieldTypeFilter.GetName().MatchesWildcard(FieldTypeStr))
				{
					OutFieldTypes.Emplace(FieldTypeFilter);
				}
			}
		}
		else
		{
			const FFieldClass* FieldClass = FFieldClass::GetNameToFieldClassMap().FindRef(*FieldTypeStr);
			const UClass* UFieldClass = FindFirstObject<UClass>(*FieldTypeStr, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("GatherTextMetaDataHelper: Looking for field types to include or exclude"));
			if (!FieldClass && !UFieldClass)
			{
				UE_LOGFMT(LogGatherTextMetaDataHelper, Warning, "Field Type {fieldType} was not found{context}. Did you forget a ModulesToPreload entry?",
					("fieldType", *FieldTypeStr),
					("context", LogContext ? FString::Printf(TEXT(" (from '%s')"), LogContext) : FString())
				);
				continue;
			}

			if (FieldClass)
			{
				OutFieldTypes.Emplace(FieldClass);
			}

			if (UFieldClass)
			{
				check(UFieldClass->IsChildOf<UField>());
				OutFieldTypes.Emplace(UFieldClass);
			}
		}
	}
}

void FGatherTextMetaDataHelper::SetFieldOwnerTypesFromStrings(const TArray<FString>& TypeNames, const bool bInclude, const TCHAR* LogContext)
{
	if (TypeNames.Num() == 0)
	{
		return;
	}
	
	TArray<const UStruct*>& OutFieldOwnerTypes = bInclude ? FieldOwnerTypesToInclude : FieldOwnerTypesToExclude;

	TArray<const UStruct*> AllFieldOwnerClassTypes;
	TArray<const UStruct*> AllFieldOwnerScriptStructTypes;
	GetObjectsOfClass(UClass::StaticClass(), (TArray<UObject*>&)AllFieldOwnerClassTypes, false);
	GetObjectsOfClass(UScriptStruct::StaticClass(), (TArray<UObject*>&)AllFieldOwnerScriptStructTypes, false);

	for (const FString& FieldOwnerTypeStr : TypeNames)
	{
		const bool bIsWildcard = FieldOwnerTypeStr.GetCharArray().Contains(TEXT('*')) || FieldOwnerTypeStr.GetCharArray().Contains(TEXT('?'));
		if (bIsWildcard)
		{
			auto AddFieldOwnersMatchingWildcard = [&FieldOwnerTypeStr, &OutFieldOwnerTypes](const TArray<const UStruct*>& AllFieldOwnerTypes)
			{
				for (const UStruct* FieldOwnerType : AllFieldOwnerTypes)
				{
					if (FieldOwnerType->GetName().MatchesWildcard(FieldOwnerTypeStr))
					{
						OutFieldOwnerTypes.Add(FieldOwnerType);
					}
				}
			};
			AddFieldOwnersMatchingWildcard(AllFieldOwnerClassTypes);
			AddFieldOwnersMatchingWildcard(AllFieldOwnerScriptStructTypes);
		}
		else
		{
			const UStruct* FieldOwnerType = FindFirstObject<UStruct>(*FieldOwnerTypeStr, EFindFirstObjectOptions::EnsureIfAmbiguous);
			if (!FieldOwnerType)
			{
				UE_LOGFMT(LogGatherTextMetaDataHelper, Warning, "Field Owner Type {fieldOwner} was not found{context}. Did you forget a ModulesToPreload entry?",
					("fieldOwner", *FieldOwnerTypeStr),
					("context", LogContext ? FString::Printf(TEXT(" (from '%s')"), LogContext) : FString())
				);
				continue;
			}

			OutFieldOwnerTypes.Add(FieldOwnerType);
			if (const UClass* FieldOwnerClass = Cast<UClass>(FieldOwnerType))
			{
				GetDerivedClasses(FieldOwnerClass, (TArray<UClass*>&)OutFieldOwnerTypes);
			}
			if (FieldOwnerType == UScriptStruct::StaticClass())
			{
				// Structs don't have a catch-all base, so we allow ScriptStruct to mean "all struct types"
				OutFieldOwnerTypes.Append(AllFieldOwnerScriptStructTypes);
			}
		}
	}
}

void FGatherTextMetaDataHelper::SetFieldOuterTypesFromStrings(const TArray<FString>& TypeNames, const bool bInclude, const TCHAR* LogContext)
{
	if (TypeNames.Num() == 0)
	{
		return;
	}

	TArray<const UStruct*>& OutFieldOuterTypes = bInclude ? FieldOuterTypesToInclude : FieldOuterTypesToExclude;

	TArray<const UStruct*> AllFieldOuterClassTypes;
	TArray<const UStruct*> AllFieldOuterScriptStructTypes;
	GetObjectsOfClass(UClass::StaticClass(), (TArray<UObject*>&)AllFieldOuterClassTypes, false);
	GetObjectsOfClass(UScriptStruct::StaticClass(), (TArray<UObject*>&)AllFieldOuterScriptStructTypes, false);

	for (const FString& FieldOuterTypeStr : TypeNames)
	{
		const bool bIsWildcard = FieldOuterTypeStr.GetCharArray().Contains(TEXT('*')) || FieldOuterTypeStr.GetCharArray().Contains(TEXT('?'));
		if (bIsWildcard)
		{
			auto AddFieldOuterTypesMatchingWildcard = [&FieldOuterTypeStr, &OutFieldOuterTypes](const TArray<const UStruct*>& AllFieldOuterTypes)
			{
				for (const UStruct* Candidate : AllFieldOuterTypes)
				{
					if (Candidate->GetName().MatchesWildcard(FieldOuterTypeStr))
					{
						OutFieldOuterTypes.Add(Candidate);
					}
				}
			};
			AddFieldOuterTypesMatchingWildcard(AllFieldOuterClassTypes);
			AddFieldOuterTypesMatchingWildcard(AllFieldOuterScriptStructTypes);
		}
		else
		{
			const UStruct* OuterType = FindFirstObject<UStruct>(*FieldOuterTypeStr, EFindFirstObjectOptions::EnsureIfAmbiguous);
			if (!OuterType)
			{
				UE_LOGFMT(LogGatherTextMetaDataHelper, Warning, "Field Outer Type {fieldOuter} not found{context}",
					("fieldOuter", *FieldOuterTypeStr),
					("context", LogContext ? FString::Printf(TEXT(" (from '%s')"), LogContext) : FString())
				);
				continue;
			}

			OutFieldOuterTypes.Add(OuterType);

			if (const UClass* OuterClass = Cast<UClass>(OuterType))
			{
				GetDerivedClasses(OuterClass, (TArray<UClass*>&)OutFieldOuterTypes);
			}
		}
	}
}

void FGatherTextMetaDataHelper::GatherTextFromField(UField* Field, const FName InPlatformName) const
{
	// For structs, also gather the new non-object field values.
	if (UStruct* Struct = Cast<UStruct>(Field))
	{
		for (TFieldIterator<FField> FieldIt(Struct, EFieldIterationFlags::None); FieldIt; ++FieldIt)
		{
			GatherTextFromField(*FieldIt, InPlatformName);
		}
	}

	if (ShouldGatherFromField(Field, false))
	{
		// Gather for object.
		{
			EnsureFieldDisplayNameImpl(Field, false);
			GatherTextFromFieldImpl(Field, InPlatformName);
		}

		// For enums, also gather for enum values.
		if (UEnum* Enum = Cast<UEnum>(Field))
		{
			const int32 ValueCount = Enum->NumEnums();
			for (int32 i = 0; i < ValueCount; ++i)
			{
				if (!Enum->HasMetaData(TEXT("DisplayName"), i))
				{
					Enum->SetMetaData(TEXT("DisplayName"), *FName::NameToDisplayString(Enum->GetAuthoredNameStringByIndex(i), false), i);
				}

				for (int32 j = 0; j < InputKeys.Num(); ++j)
				{
					FStringFormatNamedArguments PatternArguments;
					PatternArguments.Add(TEXT("FieldPath"), Enum->GetFullGroupName(false) + TEXT(".") + Enum->GetNameStringByIndex(i));

					if (Enum->HasMetaData(*InputKeys[j], i))
					{
						const FString& MetaDataValue = Enum->GetMetaData(*InputKeys[j], i);
						if (FText::ShouldGatherForLocalization(MetaDataValue))
						{
							PatternArguments.Add(TEXT("MetaDataValue"), MetaDataValue);

							const FString Namespace = OutputNamespaces[j];
							FLocItem LocItem(MetaDataValue);
							FManifestContext Context;
							Context.Key = FString::Format(*OutputKeys[j], PatternArguments);
							Context.SourceLocation = FString::Printf(TEXT("%s:%s [EnumEntry %s meta-data]"), *Enum->GetPathName(), *Enum->GetNameStringByIndex(i), *InputKeys[j]);
							Context.PlatformName = InPlatformName;
							GatherManifestHelper->AddSourceText(Namespace, LocItem, Context);
						}
					}
				}
			}
		}
	}
}

void FGatherTextMetaDataHelper::GatherTextFromField(FField* Field, const FName InPlatformName) const
{
	FProperty* Property = CastField<FProperty>(Field);
	if (ShouldGatherFromField(Field, Property && Property->HasAnyPropertyFlags(CPF_EditorOnly)))
	{
		EnsureFieldDisplayNameImpl(Field, Field->IsA(FBoolProperty::StaticClass()));
		GatherTextFromFieldImpl(Field, InPlatformName);
	}
}

template <typename FieldType>
bool FGatherTextMetaDataHelper::ShouldGatherFromField(const FieldType* Field, const bool bIsEditorOnly) const
{
	auto ShouldGatherFieldByType = [this, Field]()
	{
		if (FieldTypesToInclude.Num() == 0 && FieldTypesToExclude.Num() == 0)
		{
			return true;
		}

		const auto* FieldClass = Field->GetClass();
		auto TestClassFilter = [FieldClass](const TArray<FFieldClassFilter>& InFieldTypeFilters)
		{
			for (const FFieldClassFilter& FieldTypeFilter : InFieldTypeFilters)
			{
				if (FieldTypeFilter.TestClass(FieldClass))
				{
					return true;
				}
			}
			return false;
		};

		return (FieldTypesToInclude.Num() == 0 || TestClassFilter(FieldTypesToInclude))
			&& (FieldTypesToExclude.Num() == 0 || !TestClassFilter(FieldTypesToExclude));
	};

	auto ShouldGatherFieldByOwnerType = [this, Field]()
	{
		if (FieldOwnerTypesToInclude.Num() == 0 && FieldOwnerTypesToExclude.Num() == 0)
		{
			return true;
		}

		const UStruct* FieldOwnerType = Field->GetOwnerStruct();
		// The filter only contains native types, so this if this a non-native struct, walk up its super-chain to find the first native struct instead
		while (FieldOwnerType && !FPackageName::IsScriptPackage(FNameBuilder(FieldOwnerType->GetPackage()->GetFName())))
		{
			FieldOwnerType = FieldOwnerType->GetSuperStruct();
		}
		if (FieldOwnerType)
		{
			// Only properties and functions will have an owner struct type
			return (FieldOwnerTypesToInclude.Num() == 0 || FieldOwnerTypesToInclude.Contains(FieldOwnerType))
				&& (FieldOwnerTypesToExclude.Num() == 0 || !FieldOwnerTypesToExclude.Contains(FieldOwnerType));
		}

		return true;
	};

	auto ShouldGatherFieldByOuterType = [this, Field]()
	{
		if (FieldOuterTypesToInclude.Num() == 0 && FieldOuterTypesToExclude.Num() == 0)
		{
			return true;
		}

		UObject* FieldOuter = nullptr;
		if constexpr (TIsDerivedFrom<FieldType, FField>::IsDerived)
		{
			FieldOuter = Field->GetOwnerUObject();
		}
		else
		{
			FieldOuter = Field->GetOuter();
		}

		if (FieldOuter)
		{
			if (const UStruct* FieldOuterType = Cast<UStruct>(FieldOuter))
			{
				return (FieldOuterTypesToInclude.Num() == 0 || FieldOuterTypesToInclude.Contains(FieldOuterType))
					&& (FieldOuterTypesToExclude.Num() == 0 || !FieldOuterTypesToExclude.Contains(FieldOuterType));
			}
		}

		return true;
	};

	return (!bIsEditorOnly || bShouldGatherFromEditorOnlyData) && ShouldGatherFieldByType() && ShouldGatherFieldByOwnerType() && ShouldGatherFieldByOuterType();
}

template <typename FieldType>
void FGatherTextMetaDataHelper::GatherTextFromFieldImpl(FieldType* Field, const FName InPlatformName) const
{
	for (int32 i = 0; i < InputKeys.Num(); ++i)
	{
		FStringFormatNamedArguments PatternArguments;
		PatternArguments.Add(TEXT("FieldPath"), Field->GetFullGroupName(false));

		if (Field->HasMetaData(*InputKeys[i]))
		{
			const FString& MetaDataValue = Field->GetMetaData(*InputKeys[i]);
			if (FText::ShouldGatherForLocalization(MetaDataValue))
			{
				PatternArguments.Add(TEXT("MetaDataValue"), MetaDataValue);

				const FString Namespace = OutputNamespaces[i];
				FLocItem LocItem(MetaDataValue);
				FManifestContext Context;
				Context.Key = FString::Format(*OutputKeys[i], PatternArguments);
				Context.SourceLocation = FString::Printf(TEXT("%s [%s %s meta-data]"), *Field->GetPathName(), *Field->GetClass()->GetName(), *InputKeys[i]);
				Context.PlatformName = InPlatformName;
				GatherManifestHelper->AddSourceText(Namespace, LocItem, Context);
			}
		}
	}
}

template <typename FieldType>
void FGatherTextMetaDataHelper::EnsureFieldDisplayNameImpl(FieldType* Field, const bool bIsBool) const
{
	if (!Field->HasMetaData(TEXT("DisplayName")))
	{
		Field->SetMetaData(TEXT("DisplayName"), *FName::NameToDisplayString(Field->GetAuthoredName(), bIsBool));
	}
}
