// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "LocTextHelper.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealType.h"

/**
 * Helper for gathering localization text from field metadata.
 * Shared between UGatherTextFromMetaDataCommandlet and UGatherTextFromAssetsCommandlet.
 *
 * Usage:
 *  1. Construct with a FLocTextHelper to receive gathered text.
 *  2. Call Set* methods to configure filters and gather parameters.
 *  3. Call GatherTextFromField() for each field to process.
 */
class FGatherTextMetaDataHelper
{
public:
	explicit FGatherTextMetaDataHelper(TSharedPtr<FLocTextHelper> InGatherManifestHelper);

	/** Whether to process fields marked as editor-only */
	void SetShouldGatherFromEditorOnlyData(bool bValue);

	/** Set the parallel arrays defining what metadata keys to look for and how to key the output */
	void SetGatherParameters(TArray<FString> InInputKeys, TArray<FString> InOutputNamespaces, TArray<FString> InOutputKeys);

	/**
	 * Resolve type name strings to field class filters.
	 * Supports wildcards (* and ?). Warns if a type is not found.
	 * @param TypeNames     Raw type name strings from config.
	 * @param bInclude      True to add to include list, false to add to exclude list.
	 * @param LogContext    Optional context string for warning messages (e.g. config key name).
	 */
	void SetFieldTypesFromStrings(const TArray<FString>& TypeNames, bool bInclude, const TCHAR* LogContext = nullptr);

	/**
	 * Resolve owner type name strings to UStruct* filters (includes derived classes for exact matches).
	 * Supports wildcards (* and ?).
	 */
	void SetFieldOwnerTypesFromStrings(const TArray<FString>& TypeNames, bool bInclude, const TCHAR* LogContext = nullptr);

	/**
	 * Resolve outer type name strings to UStruct* filters (includes derived classes for exact matches).
	 * Supports wildcards (* and ?).
	 */
	void SetFieldOuterTypesFromStrings(const TArray<FString>& TypeNames, bool bInclude, const TCHAR* LogContext = nullptr);

	/** Returns true if InputKeys is non-empty — i.e., metadata gathering is configured */
	bool IsConfigured() const;

	/** Gather metadata text from a UField (iterates FField children for structs, gathers enum value metadata) */
	void GatherTextFromField(UField* Field, FName InPlatformName = NAME_None) const;

	/** Gather metadata text from an FField property within a struct */
	void GatherTextFromField(FField* Field, FName InPlatformName = NAME_None) const;

private:
	/**
	 * Filter that tests a field's class against an FFieldClass or UClass.
	 * Matches if the field's class is a child of the stored FieldClass or ObjectClass.
	 */
	struct FFieldClassFilter
	{
		explicit FFieldClassFilter(const FFieldClass* InFieldClass)
			: FieldClass(InFieldClass)
			, ObjectClass(nullptr)
		{
		}

		explicit FFieldClassFilter(const UClass* InObjectClass)
			: FieldClass(nullptr)
			, ObjectClass(InObjectClass)
		{
		}

		FString GetName() const
		{
			if (FieldClass)
			{
				return FieldClass->GetName();
			}
			if (ObjectClass)
			{
				return ObjectClass->GetName();
			}
			return FString();
		}

		bool TestClass(const FFieldClass* InFieldClass) const
		{
			return FieldClass && InFieldClass->IsChildOf(FieldClass);
		}

		bool TestClass(const UClass* InObjectClass) const
		{
			return ObjectClass && InObjectClass->IsChildOf(ObjectClass);
		}

	private:
		const FFieldClass* FieldClass;
		const UClass* ObjectClass;
	};

	template <typename FieldType>
	bool ShouldGatherFromField(const FieldType* Field, bool bIsEditorOnly) const;

	template <typename FieldType>
	void GatherTextFromFieldImpl(FieldType* Field, FName InPlatformName) const;

	template <typename FieldType>
	void EnsureFieldDisplayNameImpl(FieldType* Field, bool bIsBool) const;

	TSharedPtr<FLocTextHelper> GatherManifestHelper;
	bool bShouldGatherFromEditorOnlyData = false;
	TArray<FString> InputKeys;
	TArray<FString> OutputNamespaces;
	TArray<FString> OutputKeys;
	TArray<FFieldClassFilter> FieldTypesToInclude;
	TArray<FFieldClassFilter> FieldTypesToExclude;
	TArray<const UStruct*> FieldOwnerTypesToInclude;
	TArray<const UStruct*> FieldOwnerTypesToExclude;
	TArray<const UStruct*> FieldOuterTypesToInclude;
	TArray<const UStruct*> FieldOuterTypesToExclude;
};
