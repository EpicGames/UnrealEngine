// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GeomFitUtils.cpp: Utilities for fitting collision models to static meshes.
=============================================================================*/

#include "GeomFitUtils.h"
#include "EngineDefines.h"
#include "Misc/MessageDialog.h"
#include "UObject/UObjectIterator.h"
#include "Components/StaticMeshComponent.h"
#include "Model.h"
#include "Engine/Polys.h"
#include "StaticMeshResources.h"
#include "EditorSupportDelegates.h"
#include "BSPOps.h"
#include "RawMesh.h"
#include "PhysicsEngine/BoxElem.h"
#include "PhysicsEngine/SphereElem.h"
#include "PhysicsEngine/SphylElem.h"
#include "PhysicsEngine/BodySetup.h"
#include "Engine/StaticMesh.h"
#include "MeshDescription.h"
#include "StaticMeshAttributes.h"
#include "Settings/EditorExperimentalSettings.h"
#include "CompGeom/FitKDOP3.h"

static bool PromptToRemoveExistingCollision(UStaticMesh* StaticMesh)
{
	check(StaticMesh);
	UBodySetup* bs = StaticMesh->GetBodySetup();
	if (bs && (bs->AggGeom.GetElementCount() > 0))
	{
		// If we already have some simplified collision for this mesh - check before we clobber it.
		/*const EAppReturnType::Type ret = FMessageDialog::Open(EAppMsgType::YesNoCancel, NSLOCTEXT("UnrealEd", "StaticMeshAlreadyHasGeom", "Static Mesh already has simple collision.\nDo you want to replace it?"));
		if (ret == EAppReturnType::Yes)
		{
			bs->RemoveSimpleCollision();
		}
		else if (ret == EAppReturnType::Cancel)
		{
			return false;
		}*/
	}
	else
	{
		// Otherwise, create one here.
		StaticMesh->CreateBodySetup();
		bs = StaticMesh->GetBodySetup();
	}
	return true;
}

/* ******************************** KDOP ******************************** */

// This function takes the current collision model, and fits a k-DOP around it.
// It uses the array of k unit-length direction vectors to define the k bounding planes.

int32 GenerateKDopAsSimpleCollision(UStaticMesh* StaticMesh, const TArray<FVector> &Dirs, bool bUpdate)
{
	// Make sure rendering is done - so we are not changing data being used by collision drawing.
	if(bUpdate)
	{
		FlushRenderingCommands();
	}

	if (!PromptToRemoveExistingCollision(StaticMesh))
	{
		return INDEX_NONE;
	}

	UBodySetup* bs = StaticMesh->GetBodySetup();

	int32 NumVertices = 0;
	TFunction<UE::Math::TVector<double>(int32)> GetPointFunc;
	if(FStaticMeshRenderData* RenderData = StaticMesh->GetRenderData())
	{
		NumVertices = RenderData->LODResources[0].GetNumVertices();
		GetPointFunc = [RenderData](int32 VertIdx)
		{
			return FVector{ RenderData->LODResources[0].VertexBuffers.PositionVertexBuffer.VertexPosition(VertIdx) };
		};
	}
	else if(const FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription(0))
	{		
		NumVertices = MeshDescription->Vertices().Num();
		GetPointFunc = [MeshDescription](int32 VertIdx)
		{
			return FVector(MeshDescription->GetVertexPosition(VertIdx));
		};
	}

	if(!GetPointFunc)
	{
		UE_LOGF(LogStaticMesh, Error, "Couldn't find a valid RenderData or MeshDescription for the StaticMesh");
		return INDEX_NONE;
	}

	TArray<FVector> HullVertices;
	UE::Geometry::FitKDOPVertices3<double>(Dirs, NumVertices, GetPointFunc, HullVertices);

	if(bUpdate)
	{
		bs->Modify();
	}

	// Create new GUID
	bs->InvalidatePhysicsData();

	FKConvexElem ConvexElem;
	ConvexElem.VertexData = HullVertices;
	// Note: UpdateElemBox also computes the convex hull indices
	ConvexElem.UpdateElemBox();
	ConvexElem.bIsGenerated = true;
	bs->AggGeom.ConvexElems.Add(ConvexElem);
	
	// create all body instances
	RefreshCollisionChange(*StaticMesh);

	// Mark staticmesh as dirty, to help make sure it gets saved.
	if(bUpdate)
	{
		StaticMesh->MarkPackageDirty();
	}

	StaticMesh->SetCustomizedCollision(true);	//mark the static mesh for collision customization

	return bs->AggGeom.ConvexElems.Num() - 1;
}

/* ******************************** BOX ******************************** */

void ComputeBoundingBox(UStaticMesh* StaticMesh, FVector& Center, FVector& Extents)
{
	// Calculate bounding Box.
	FBox BoundingBox = StaticMesh->GetMeshDescription(0)->ComputeBoundingBox();
	BoundingBox.GetCenterAndExtents(Center, Extents);
}

int32 GenerateBoxAsSimpleCollision(UStaticMesh* StaticMesh, bool bUpdate)
{
	if (!PromptToRemoveExistingCollision(StaticMesh))
	{
		return INDEX_NONE;
	}

	UBodySetup* bs = StaticMesh->GetBodySetup();

	// Calculate bounding Box.
	FVector Center, Extents;
	StaticMesh->GetMeshDescription(0)->ComputeBoundingBox().GetCenterAndExtents(Center, Extents);
	Extents *= (FVector)bs->BuildScale3D;

	if(bUpdate)
	{
		bs->Modify();
	}

	// Create new GUID
	bs->InvalidatePhysicsData();

	FKBoxElem BoxElem;
	BoxElem.Center = Center;
	BoxElem.X = Extents.X * 2.0f;
	BoxElem.Y = Extents.Y * 2.0f;
	BoxElem.Z = Extents.Z * 2.0f;
	BoxElem.bIsGenerated = true;
	bs->AggGeom.BoxElems.Add(BoxElem);

	// refresh collision change back to staticmesh components
	RefreshCollisionChange(*StaticMesh);

	// Mark staticmesh as dirty, to help make sure it gets saved.
	if(bUpdate)
	{
		StaticMesh->MarkPackageDirty();
	}

	StaticMesh->SetCustomizedCollision(true);	//mark the static mesh for collision customization

	return bs->AggGeom.BoxElems.Num() - 1;
}

/* ******************************** SPHERE ******************************** */

// Can do bounding circles as well... Set elements of limitVect to 1.f for directions to consider, and 0.f to not consider.
// Have 2 algorithms, seem better in different cirumstances

// This algorithm taken from Ritter, 1990
// This one seems to do well with asymmetric input.
static void CalcBoundingSphere(const FMeshDescription* MeshDescription, FSphere& sphere, FVector& LimitVec)
{
	if (MeshDescription->Vertices().Num() == 0)
		return;

	FBox Box;
	FVector MinIx[3] = { FVector::ZeroVector, FVector::ZeroVector, FVector::ZeroVector };
	FVector MaxIx[3] = { FVector::ZeroVector, FVector::ZeroVector, FVector::ZeroVector };

	FStaticMeshConstAttributes Attributes(*MeshDescription);

	TVertexAttributesConstRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();

	bool bFirstVertex = true;
	for (const FVertexID VertexID : MeshDescription->Vertices().GetElementIDs())
	{
		FVector p = (FVector)VertexPositions[VertexID] * LimitVec;
		if (bFirstVertex)
		{
			// First, find AABB, remembering furthest points in each dir.
			Box.Min = p;
			Box.Max = Box.Min;

			MinIx[0] = (FVector)VertexPositions[VertexID];
			MinIx[1] = (FVector)VertexPositions[VertexID];
			MinIx[2] = (FVector)VertexPositions[VertexID];

			MaxIx[0] = (FVector)VertexPositions[VertexID];
			MaxIx[1] = (FVector)VertexPositions[VertexID];
			MaxIx[2] = (FVector)VertexPositions[VertexID];
			bFirstVertex = false;
			continue;
		}

		// X //
		if (p.X < Box.Min.X)
		{
			Box.Min.X = p.X;
			MinIx[0] = (FVector)VertexPositions[VertexID];
		}
		else if (p.X > Box.Max.X)
		{
			Box.Max.X = p.X;
			MaxIx[0] = (FVector)VertexPositions[VertexID];
		}

		// Y //
		if (p.Y < Box.Min.Y)
		{
			Box.Min.Y = p.Y;
			MinIx[1] = (FVector)VertexPositions[VertexID];
		}
		else if (p.Y > Box.Max.Y)
		{
			Box.Max.Y = p.Y;
			MaxIx[1] = (FVector)VertexPositions[VertexID];
		}

		// Z //
		if (p.Z < Box.Min.Z)
		{
			Box.Min.Z = p.Z;
			MinIx[2] = (FVector)VertexPositions[VertexID];
		}
		else if (p.Z > Box.Max.Z)
		{
			Box.Max.Z = p.Z;
			MaxIx[2] = (FVector)VertexPositions[VertexID];
		}
	}

	const FVector Extremes[3] = { (MaxIx[0] - MinIx[0]) * LimitVec,
		(MaxIx[1] - MinIx[1]) * LimitVec,
		(MaxIx[2] - MinIx[2]) * LimitVec };

	// Now find extreme points furthest apart, and initial center and radius of sphere.
	float d2 = 0.f;
	for (int32 i = 0; i < 3; i++)
	{
		const float tmpd2 = Extremes[i].SizeSquared();
		if (tmpd2 > d2)
		{
			d2 = tmpd2;
			sphere.Center = (MinIx[i] + (0.5f * Extremes[i])) * LimitVec;
			sphere.W = 0.f;
		}
	}

	const FVector Extents = FVector(Extremes[0].X, Extremes[1].Y, Extremes[2].Z);

	// radius and radius squared
	float r = 0.5f * Extents.GetMax();
	float r2 = FMath::Square(r);

	// Now check each point lies within this sphere. If not - expand it a bit.
	for (const FVertexID VertexID : MeshDescription->Vertices().GetElementIDs())
	{
		const FVector cToP = ((FVector)VertexPositions[VertexID] * LimitVec) - sphere.Center;

		const float pr2 = cToP.SizeSquared();

		// If this point is outside our current bounding sphere's radius
		if (pr2 > r2)
		{
			// ..expand radius just enough to include this point.
			const float pr = FMath::Sqrt(pr2);
			r = 0.5f * (r + pr);
			r2 = FMath::Square(r);

			sphere.Center += ((pr - r) / pr * cToP);
		}
	}

	sphere.W = r;
}

// This is the one thats already used by unreal.
// Seems to do better with more symmetric input...
static void CalcBoundingSphere2(const FMeshDescription* MeshDescription, FSphere& sphere, FVector& LimitVec)
{
	FVector Center = MeshDescription->ComputeBoundingBox().GetCenter();

	sphere.Center = Center;
	sphere.W = 0.0f;

	FStaticMeshConstAttributes Attributes(*MeshDescription);
	TVertexAttributesConstRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();

	for (const FVertexID VertexID : MeshDescription->Vertices().GetElementIDs())
	{
		float Dist = FVector::DistSquared((FVector)VertexPositions[VertexID] * LimitVec, sphere.Center);
		if (Dist > sphere.W)
			sphere.W = Dist;
	}
	sphere.W = FMath::Sqrt(sphere.W);
}

// // //

int32 GenerateSphereAsSimpleCollision(UStaticMesh* StaticMesh, bool bUpdate)
{
	if (!PromptToRemoveExistingCollision(StaticMesh))
	{
		return INDEX_NONE;
	}

	UBodySetup* bs = StaticMesh->GetBodySetup();
	FSphere bSphere, bSphere2, bestSphere;
	FVector unitVec = (FVector)bs->BuildScale3D;

	// Calculate bounding sphere.
	FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription(0);
	check(MeshDescription);
	CalcBoundingSphere(MeshDescription, bSphere, unitVec);
	CalcBoundingSphere2(MeshDescription, bSphere2, unitVec);

	if(bSphere.W < bSphere2.W)
		bestSphere = bSphere;
	else
		bestSphere = bSphere2;

	// Dont use if radius is zero.
	if(bestSphere.W <= 0.f)
	{
		FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Prompt_10", "Could not create geometry.") );
		return INDEX_NONE;
	}

	if(bUpdate)
	{
		bs->Modify();
	}

	// Create new GUID
	bs->InvalidatePhysicsData();

	FKSphereElem SphereElem;
	SphereElem.Center = bestSphere.Center;
	SphereElem.Radius = bestSphere.W;
	SphereElem.bIsGenerated = true;
	bs->AggGeom.SphereElems.Add(SphereElem);

	// refresh collision change back to staticmesh components
	RefreshCollisionChange(*StaticMesh);

	// Mark staticmesh as dirty, to help make sure it gets saved.
	if(bUpdate)
	{
		StaticMesh->MarkPackageDirty();
	}

	StaticMesh->SetCustomizedCollision(true);	//mark the static mesh for collision customization
	return bs->AggGeom.SphereElems.Num() - 1;
}

/* ******************************** SPHYL ******************************** */

static void CalcBoundingSphyl(const FMeshDescription* MeshDescription, FSphere& sphere, float& length, FRotator& rotation, FVector& LimitVec)
{
	if (MeshDescription->Vertices().Num() == 0)
		return;

	FVector Center, Extents;
	MeshDescription->ComputeBoundingBox().GetCenterAndExtents(Center, Extents);
	Extents *= LimitVec;

	// @todo sphere.Center could perhaps be adjusted to best fit if model is non-symmetric on it's longest axis
	sphere.Center = Center;

	// Work out best axis aligned orientation (longest side)
	double Extent = Extents.GetMax();
	if (Extent == Extents.X)
	{
		rotation = FRotator(90.f, 0.f, 0.f);
		Extents.X = 0.0f;
	}
	else if (Extent == Extents.Y)
	{
		rotation = FRotator(0.f, 0.f, 90.f);
		Extents.Y = 0.0f;
	}
	else
	{
		rotation = FRotator(0.f, 0.f, 0.f);
		Extents.Z = 0.0f;
	}

	// Cleared the largest axis above, remaining determines the radius
	float r = Extents.GetMax();
	float r2 = FMath::Square(r);
	
	FStaticMeshConstAttributes Attributes(*MeshDescription);
	TVertexAttributesConstRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();

	// Now check each point lies within this the radius. If not - expand it a bit.
	for (const FVertexID VertexID : MeshDescription->Vertices().GetElementIDs())
	{
		FVector cToP = ((FVector)VertexPositions[VertexID] * LimitVec) - sphere.Center;
		cToP = rotation.UnrotateVector(cToP);

		const float pr2 = cToP.SizeSquared2D();	// Ignore Z here...

		// If this point is outside our current bounding sphere's radius
		if (pr2 > r2)
		{
			// ..expand radius just enough to include this point.
			const float pr = FMath::Sqrt(pr2);
			r = 0.5f * (r + pr);
			r2 = FMath::Square(r);
		}
	}
	
	// The length is the longest side minus the radius.
	float hl = FMath::Max(0.0f, Extent - r);

	// Now check each point lies within the length. If not - expand it a bit.
	for (const FVertexID VertexID : MeshDescription->Vertices().GetElementIDs())
	{
		FVector cToP = ((FVector)VertexPositions[VertexID] * LimitVec) - sphere.Center;
		cToP = rotation.UnrotateVector(cToP);

		// If this point is outside our current bounding sphyl's length
		if (FMath::Abs(cToP.Z) > hl)
		{
			const bool bFlip = (cToP.Z < 0.f ? true : false);
			const FVector cOrigin(0.f, 0.f, (bFlip ? -hl : hl));

			const float pr2 = (cOrigin - cToP).SizeSquared();

			// If this point is outside our current bounding sphyl's radius
			if (pr2 > r2)
			{
				FVector cPoint;
				FMath::SphereDistToLine(cOrigin, r, cToP, (bFlip ? FVector(0.f, 0.f, 1.f) : FVector(0.f, 0.f, -1.f)), cPoint);

				// Don't accept zero as a valid diff when we know it's outside the sphere (saves needless retest on further iterations of like points)
				hl += FMath::Max<float>(FMath::Abs(cToP.Z - cPoint.Z), 1.e-6f);
			}
		}
	}

	sphere.W = r;
	length = hl * 2.0f;
}

// // //

int32 GenerateSphylAsSimpleCollision(UStaticMesh* StaticMesh, bool bUpdate)
{
	if (!PromptToRemoveExistingCollision(StaticMesh))
	{
		return INDEX_NONE;
	}

	UBodySetup* bs = StaticMesh->GetBodySetup();

	FSphere sphere;
	float length;
	FRotator rotation;
	FVector unitVec = (FVector)bs->BuildScale3D;

	// Calculate bounding box.
	FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription(0);
	check(MeshDescription);
	CalcBoundingSphyl(MeshDescription, sphere, length, rotation, unitVec);

	// Dont use if radius is zero.
	if (sphere.W <= 0.f)
	{
		FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Prompt_10", "Could not create geometry."));
		return INDEX_NONE;
	}

	// If height is zero, then a sphere would be better (should we just create one instead?)
	if (length <= 0.f)
	{
		length = SMALL_NUMBER;
	}

	if(bUpdate)
	{
		bs->Modify();
	}

	// Create new GUID
	bs->InvalidatePhysicsData();

	FKSphylElem SphylElem;
	SphylElem.Center = sphere.Center;
	SphylElem.Rotation = rotation;
	SphylElem.Radius = sphere.W;
	SphylElem.Length = length;
	SphylElem.bIsGenerated = true;
	bs->AggGeom.SphylElems.Add(SphylElem);

	// refresh collision change back to staticmesh components
	RefreshCollisionChange(*StaticMesh);

	// Mark staticmesh as dirty, to help make sure it gets saved.
	if(bUpdate)
	{
		StaticMesh->MarkPackageDirty();
	}

	StaticMesh->SetCustomizedCollision(true);	//mark the static mesh for collision customization

	return bs->AggGeom.SphylElems.Num() - 1;
}

void RefreshCollisionChange(UStaticMesh& StaticMesh)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(RefreshCollisionChange)

	StaticMesh.CreateNavCollision(/*bIsUpdate=*/true);

	for (FThreadSafeObjectIterator Iter(UStaticMeshComponent::StaticClass()); Iter; ++Iter)
	{
		UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(*Iter);
		if (StaticMeshComponent->GetStaticMesh() == &StaticMesh)
		{
			// it needs to recreate IF it already has been created
			if (StaticMeshComponent->IsPhysicsStateCreated())
			{
				StaticMeshComponent->RecreatePhysicsState();
			}
		}
	}

	FEditorSupportDelegates::RedrawAllViewports.Broadcast();
}

void RefreshCollisionChanges(const TArray<UStaticMesh*>& StaticMeshes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(RefreshCollisionChanges)

	for (UStaticMesh* StaticMesh : StaticMeshes)
	{
		StaticMesh->CreateNavCollision(/*bIsUpdate=*/true);
	}

	for (FThreadSafeObjectIterator Iter(UStaticMeshComponent::StaticClass()); Iter; ++Iter)
	{
		UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(*Iter);
		if (StaticMeshes.Contains(StaticMeshComponent->GetStaticMesh()))
		{
			// it needs to recreate IF it already has been created
			if (StaticMeshComponent->IsPhysicsStateCreated())
			{
				StaticMeshComponent->RecreatePhysicsState();
			}
		}
	}

	FEditorSupportDelegates::RedrawAllViewports.Broadcast();
}

void RefreshCollisionChangeComponentsOnly(UStaticMesh& StaticMesh)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(RefreshCollisionChangeComponentsOnly)

	for (FThreadSafeObjectIterator Iter(UStaticMeshComponent::StaticClass()); Iter; ++Iter)
	{
		UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(*Iter);
		if (StaticMeshComponent->GetStaticMesh() == &StaticMesh)
		{
			// it needs to recreate IF it already has been created
			if (StaticMeshComponent->IsPhysicsStateCreated())
			{
				StaticMeshComponent->RecreatePhysicsState();
			}
		}
	}

	FEditorSupportDelegates::RedrawAllViewports.Broadcast();
}
