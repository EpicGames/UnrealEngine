// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
/*=============================================================================
	GeomFitUtils.h: Utilities for fitting collision models to static meshes.
=============================================================================*/

// k-DOP (k-Discrete Oriented Polytopes) Direction Vectors
#define RCP_SQRT2 (0.70710678118654752440084436210485f)
#define RCP_SQRT3 (0.57735026918962576450914878050196f)

class UStaticMesh;

const FVector KDopDir10X[10] = 
{
	FVector( 1.f, 0.f, 0.f),
	FVector(-1.f, 0.f, 0.f),
	FVector( 0.f, 1.f, 0.f),
	FVector( 0.f,-1.f, 0.f),
	FVector( 0.f, 0.f, 1.f),
	FVector( 0.f, 0.f,-1.f),
	FVector( 0.f, RCP_SQRT2,  RCP_SQRT2),
	FVector( 0.f,-RCP_SQRT2, -RCP_SQRT2),
	FVector( 0.f, RCP_SQRT2, -RCP_SQRT2),
	FVector( 0.f,-RCP_SQRT2,  RCP_SQRT2)
};

const FVector KDopDir10Y[10] = 
{
	FVector( 1.f, 0.f, 0.f),
	FVector(-1.f, 0.f, 0.f),
	FVector( 0.f, 1.f, 0.f),
	FVector( 0.f,-1.f, 0.f),
	FVector( 0.f, 0.f, 1.f),
	FVector( 0.f, 0.f,-1.f),
	FVector( RCP_SQRT2, 0.f,  RCP_SQRT2),
	FVector(-RCP_SQRT2, 0.f, -RCP_SQRT2),
	FVector( RCP_SQRT2, 0.f, -RCP_SQRT2),
	FVector(-RCP_SQRT2, 0.f,  RCP_SQRT2)
};

const FVector KDopDir10Z[10] = 
{
	FVector( 1.f, 0.f, 0.f),
	FVector(-1.f, 0.f, 0.f),
	FVector( 0.f, 1.f, 0.f),
	FVector( 0.f,-1.f, 0.f),
	FVector( 0.f, 0.f, 1.f),
	FVector( 0.f, 0.f,-1.f),
	FVector( RCP_SQRT2,  RCP_SQRT2, 0.f),
	FVector(-RCP_SQRT2, -RCP_SQRT2, 0.f),
	FVector( RCP_SQRT2, -RCP_SQRT2, 0.f),
	FVector(-RCP_SQRT2,  RCP_SQRT2, 0.f)
};

const FVector KDopDir18[18] = 
{
	FVector( 1.f, 0.f, 0.f),
	FVector(-1.f, 0.f, 0.f),
	FVector( 0.f, 1.f, 0.f),
	FVector( 0.f,-1.f, 0.f),
	FVector( 0.f, 0.f, 1.f),
	FVector( 0.f, 0.f,-1.f),
	FVector( 0.f, RCP_SQRT2,  RCP_SQRT2),
	FVector( 0.f,-RCP_SQRT2, -RCP_SQRT2),
	FVector( 0.f, RCP_SQRT2, -RCP_SQRT2),
	FVector( 0.f,-RCP_SQRT2,  RCP_SQRT2),
	FVector( RCP_SQRT2, 0.f,  RCP_SQRT2),
	FVector(-RCP_SQRT2, 0.f, -RCP_SQRT2),
	FVector( RCP_SQRT2, 0.f, -RCP_SQRT2),
	FVector(-RCP_SQRT2, 0.f,  RCP_SQRT2),
	FVector( RCP_SQRT2,  RCP_SQRT2, 0.f),
	FVector(-RCP_SQRT2, -RCP_SQRT2, 0.f),
	FVector( RCP_SQRT2, -RCP_SQRT2, 0.f),
	FVector(-RCP_SQRT2,  RCP_SQRT2, 0.f)
};

const FVector KDopDir26[26] = 
{
	FVector( 1.f, 0.f, 0.f),
	FVector(-1.f, 0.f, 0.f),
	FVector( 0.f, 1.f, 0.f),
	FVector( 0.f,-1.f, 0.f),
	FVector( 0.f, 0.f, 1.f),
	FVector( 0.f, 0.f,-1.f),
	FVector( 0.f, RCP_SQRT2,  RCP_SQRT2),
	FVector( 0.f,-RCP_SQRT2, -RCP_SQRT2),
	FVector( 0.f, RCP_SQRT2, -RCP_SQRT2),
	FVector( 0.f,-RCP_SQRT2,  RCP_SQRT2),
	FVector( RCP_SQRT2, 0.f,  RCP_SQRT2),
	FVector(-RCP_SQRT2, 0.f, -RCP_SQRT2),
	FVector( RCP_SQRT2, 0.f, -RCP_SQRT2),
	FVector(-RCP_SQRT2, 0.f,  RCP_SQRT2),
	FVector( RCP_SQRT2,  RCP_SQRT2, 0.f),
	FVector(-RCP_SQRT2, -RCP_SQRT2, 0.f),
	FVector( RCP_SQRT2, -RCP_SQRT2, 0.f),
	FVector(-RCP_SQRT2,  RCP_SQRT2, 0.f),
	FVector( RCP_SQRT3,  RCP_SQRT3,  RCP_SQRT3),
	FVector( RCP_SQRT3,  RCP_SQRT3, -RCP_SQRT3),
	FVector( RCP_SQRT3, -RCP_SQRT3,  RCP_SQRT3),
	FVector( RCP_SQRT3, -RCP_SQRT3, -RCP_SQRT3),
	FVector(-RCP_SQRT3,  RCP_SQRT3,  RCP_SQRT3),
	FVector(-RCP_SQRT3,  RCP_SQRT3, -RCP_SQRT3),
	FVector(-RCP_SQRT3, -RCP_SQRT3,  RCP_SQRT3),
	FVector(-RCP_SQRT3, -RCP_SQRT3, -RCP_SQRT3),
};

// Utilities
UNREALED_API int32 GenerateKDopAsSimpleCollision(UStaticMesh* StaticMesh, const TArray<FVector>& dirs, bool bUpdate = true);
UNREALED_API int32 GenerateBoxAsSimpleCollision(UStaticMesh* StaticMesh, bool bUpdate = true);
UNREALED_API int32 GenerateSphereAsSimpleCollision(UStaticMesh* StaticMesh, bool bUpdate = true);
UNREALED_API int32 GenerateSphylAsSimpleCollision(UStaticMesh* StaticMesh, bool bUpdate = true);
UNREALED_API void ComputeBoundingBox(UStaticMesh* StaticMesh, FVector& Center, FVector& Extents);

/**
 * Refresh Collision Change
 * 
 * Collision has been changed, so it will need to recreate physics state to reflect it
 * Utilities functions to propagate BodySetup change for StaticMesh
 *
 * @param	StaticMesh	StaticMesh that collision has been changed for
 */	
UNREALED_API void RefreshCollisionChange(UStaticMesh& StaticMesh);

/**
 * Refresh Collision Changes for multiple static meshes
 * 
 * Collision has been changed, so it will need to recreate physics state to reflect it
 * Utilities functions to propagate BodySetup change for StaticMesh
 *
 * @param	StaticMeshes  Set of StaticMesh that collision has been changed for
 */
UNREALED_API void RefreshCollisionChanges(const TArray<UStaticMesh*>& StaticMeshes);

/**
 * Refresh Collision Change Components Only
 *
 * Collision has been changed, so it will need to recreate physics state to reflect it
 * Utilities functions to propagate BodySetup change for StaticMesh to its component
 * Use this method if static mesh nav collision has already been updated elsewhere.
 *
 * @param	StaticMesh	StaticMesh that collision has been changed for
 */
UNREALED_API void RefreshCollisionChangeComponentsOnly(UStaticMesh& StaticMesh);
