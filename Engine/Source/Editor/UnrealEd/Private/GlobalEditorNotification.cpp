// Copyright Epic Games, Inc. All Rights Reserved.

#include "GlobalEditorNotification.h"
#include "UnrealClient.h"
#include "Engine/EngineTypes.h"
#include "Editor.h"

void FGlobalEditorNotification::Tick(float DeltaTime)
{
	TickNotification(DeltaTime);
}

TStatId FGlobalEditorNotification::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FGlobalEditorNotification, STATGROUP_Tickables);
}

void FGlobalEditorProgressNotification::UpdateProgressMessage(const FText NewMessage)
{
	ProgressMessage = NewMessage;
}

void FGlobalEditorProgressNotification::CancelNotification()
{
	FSlateNotificationManager::Get().CancelProgressNotification(NotificationHandle);
	NotificationHandle.Reset();
	TotalWorkNeeded = 0;
	CurrentWorkCompleted = 0;
}

void FGlobalEditorProgressNotification::StartNotification(int32 InTotalWorkNeeded, FText ProgressText)
{
	CurrentWorkCompleted = 0;
	TotalWorkNeeded = InTotalWorkNeeded;
	NotificationHandle = FSlateNotificationManager::Get().StartProgressNotification(ProgressText, TotalWorkNeeded);
}

void FGlobalEditorProgressNotification::UpdateNotification(int32 InTotalWorkComplete, FText ProgressText)
{
	FSlateNotificationManager::Get().UpdateProgressNotification(NotificationHandle, InTotalWorkComplete, TotalWorkNeeded, ProgressText);
}

void FGlobalEditorProgressNotification::Tick(float DeltaTime)
{
	if(AllowedToStartNotification() || NotificationHandle.IsValid())
	{
		const int32 RemainingJobs = UpdateProgress();

		if (RemainingJobs != 0 && !NotificationHandle.IsValid())
		{
			StartNotification(RemainingJobs, ProgressMessage);
		}
		else if (NotificationHandle.IsValid())
		{
			const int32 DeltaWorkDone = (TotalWorkNeeded - RemainingJobs) - CurrentWorkCompleted;
			// More tasks were added, update the total work
			if (DeltaWorkDone < 0)
			{
				TotalWorkNeeded += -DeltaWorkDone;
			}
			else if (DeltaWorkDone > 0)
			{
				CurrentWorkCompleted += DeltaWorkDone;
			}

			UpdateNotification(CurrentWorkCompleted, ProgressMessage);

			
		}

		if (RemainingJobs == 0 && NotificationHandle.IsValid())
		{
			NotificationHandle.Reset();
			TotalWorkNeeded = 0;
			CurrentWorkCompleted = 0;
		}
	}
}

TStatId FGlobalEditorProgressNotification::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FGlobalEditorProgressNotification, STATGROUP_Tickables);
}



/** Notification class for asynchronous card representation building. */
class FTestProgressNotification : public FGlobalEditorProgressNotification
{

public:
	FTestProgressNotification()
		: FGlobalEditorProgressNotification(FText::FromString(TEXT("Test Async Progres")))
	{
		GEditor->GetTimerManager()->SetTimer(
			Handle,
			FTimerDelegate::CreateLambda
			(
				[this]()
				{
					--RemainingJobs;
				}
			),
			1.0f,
			true
		);
	}

	~FTestProgressNotification()
	{
		GEditor->GetTimerManager()->ClearTimer(Handle);
	}
private:
	virtual int32 UpdateProgress()
	{
		if (RemainingJobs <= 0)
		{
			GEditor->GetTimerManager()->ClearTimer(Handle);
			Handle.Invalidate();
		}
		return RemainingJobs;
	}
public:
	FTimerHandle Handle;
	int32 RemainingJobs = 0;
};

static TArray<TUniquePtr<FTestProgressNotification>> TestProgressNotificationTasks;


static void StartWorkTest()
{
	TUniquePtr<FTestProgressNotification> TestProgressNotificationTask = MakeUnique<FTestProgressNotification>();
	TestProgressNotificationTask->RemainingJobs = 100;

	TestProgressNotificationTasks.Add(MoveTemp(TestProgressNotificationTask));
}

static void StopWorkTest()
{
	FTimerHandle Handle;
	GEditor->GetTimerManager()->SetTimer(
		Handle,
		FTimerDelegate::CreateLambda
		(
			[]()
			{
				int32 TaskIndex = FMath::Rand() % TestProgressNotificationTasks.Num();
				TUniquePtr<FTestProgressNotification>& TestProgressNotificationTask = TestProgressNotificationTasks[TaskIndex];
				TestProgressNotificationTask->CancelNotification();
				TestProgressNotificationTask->RemainingJobs = 0;
			}
		),
		5.0f,
		false);
}

static void AddWork()
{
	int32 TaskIndex = FMath::Rand() % TestProgressNotificationTasks.Num();
	TUniquePtr<FTestProgressNotification>& TestProgressNotificationTask = TestProgressNotificationTasks[TaskIndex];
	//if (TestProgressNotificationTask)
	{
		TestProgressNotificationTask->RemainingJobs += 100;
	}
}
FAutoConsoleCommand StartWorkTestCommand(TEXT("StartWorkTest"), TEXT(""), FConsoleCommandDelegate::CreateStatic(&StartWorkTest));
FAutoConsoleCommand AddWorkTestCommand(TEXT("AddWork"), TEXT(""), FConsoleCommandDelegate::CreateStatic(&AddWork));
FAutoConsoleCommand StopWorkTestCommand(TEXT("StopWorkTest"), TEXT(""), FConsoleCommandDelegate::CreateStatic(&StopWorkTest));
