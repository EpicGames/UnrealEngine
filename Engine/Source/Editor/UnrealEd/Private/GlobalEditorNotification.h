// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Notifications/GlobalNotification.h"
#include "Stats/Stats.h"
#include "TickableEditorObject.h"
#include "Framework/Notifications/NotificationManager.h"

/**
 * Class used to provide simple global editor notifications (for things like shader compilation and texture streaming) 
 */
class FGlobalEditorNotification : public FGlobalNotification, public FTickableEditorObject
{

public:
	FGlobalEditorNotification(const double InEnableDelayInSeconds = 1.0)
		: FGlobalNotification(InEnableDelayInSeconds)
	{
	}

	virtual ~FGlobalEditorNotification() = default;

private:
	/** FTickableEditorObject interface */
	virtual void Tick(float DeltaTime) override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
	virtual TStatId GetStatId() const override;

};

/**
 * A special kind of editor notification for showing async or background tasks with measurable progress
 */
class FGlobalEditorProgressNotification : public FTickableEditorObject
{
public:
	FGlobalEditorProgressNotification(const FText InProgressMessage)
		: ProgressMessage(InProgressMessage)
	{}

public:
	/** Whether or not the notification is allowed to start. */
	virtual bool AllowedToStartNotification() const { return true; }

	/**
	 * Called every frame to update progress. The first time a non-zero value is returned, the notification starts and will end and reach 100% completion when
	 * the number of jobs reaches zero again
	 * 
	 * @return Return the number of remaining work for this notification. (E.g number of threaded jobs left)
 	 */
	virtual int32 UpdateProgress() = 0;

	/** Updates the display text of this notification */
	void UpdateProgressMessage(const FText NewMessage);

	/** Cancels this notification */
	void CancelNotification();
private:
	void StartNotification(int32 InStartWorkNum, FText ProgressText);
	void UpdateNotification(int32 InTotalWorkComplete, FText ProgressText);

	/** FTickableEditorObject interface */
	virtual void Tick(float DeltaTime) override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
	virtual TStatId GetStatId() const override;
private:
	FProgressNotificationHandle NotificationHandle;
	FText ProgressMessage;
	int32 TotalWorkNeeded = 0;
	int32 CurrentWorkCompleted = 0;
};
