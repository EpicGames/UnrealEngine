// Copyright Epic Games, Inc. All Rights Reserved.


#include "GraphEditor.h"

#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "GraphEditorModule.h"
#include "HAL/PlatformCrt.h"
#include "Layout/Children.h"
#include "Layout/SlateRect.h"
#include "Misc/AssertionMacros.h"
#include "Modules/ModuleManager.h"
#include "Types/ISlateMetaData.h"
#include "UObject/ObjectPtr.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SMissingWidget.h"

struct FPropertyChangedEvent;

// List of all active GraphEditor wrappers
TArray< TWeakPtr<SGraphEditor> > SGraphEditor::AllInstances;



void SGraphEditor::ConstructImplementation( const FArguments& InArgs )
{
	FGraphEditorModule& GraphEdModule = FModuleManager::LoadModuleChecked<FGraphEditorModule>(TEXT("GraphEditor"));

	// Upgrade any deprecated delegates from the events argument
	UpgradeDeprecatedDelegates(const_cast<FArguments&>(InArgs)._GraphEvents);

	// Construct the implementation and make it the contents of this widget.
	Implementation = GraphEdModule.PRIVATE_MakeGraphEditor( InArgs._AdditionalCommands, 
		InArgs._IsEditable, 
		InArgs._DisplayAsReadOnly, 
		InArgs._IsEmpty,
		InArgs._Appearance,
		InArgs._TitleBar,
		InArgs._GraphToEdit,
		InArgs._GraphEvents,
		InArgs._AutoExpandActionMenu,
		InArgs._DiffResults,
		InArgs._FocusedDiffResult,
		InArgs._OnNavigateHistoryBack,
		InArgs._OnNavigateHistoryForward,
		InArgs._ShowGraphStateOverlay,
		InArgs._AllowConnectionSlicing,
		InArgs._OnTick,
		InArgs._GraphSupportsInvalidation
		);

	Implementation->AssetEditorToolkit = InArgs._AssetEditorToolkit;

	this->ChildSlot
	[
		SNew( SBox )
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("GraphEditorPanel")))
		[
			Implementation.ToSharedRef()
		]
	];
}

void SGraphEditor::UpgradeDeprecatedDelegates(FGraphEditorEvents& EventsToUpdate)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (EventsToUpdate.OnDropActor.IsBound() && !EventsToUpdate.OnDropActors.IsBound())
	{
		EventsToUpdate.OnDropActors = FOnDropActors::CreateLambda([OnDropActorCopy = EventsToUpdate.OnDropActor](const TArray< TWeakObjectPtr<AActor> >& Actors, UEdGraph* InGraph, const FVector2f& InDropLocation)
		{
			OnDropActorCopy.Execute(Actors, InGraph, FVector2D(InDropLocation));
		});
	}
	if (EventsToUpdate.OnDropStreamingLevel.IsBound() && !EventsToUpdate.OnDropStreamingLevels.IsBound())
	{
		EventsToUpdate.OnDropStreamingLevels = FOnDropStreamingLevels::CreateLambda([OnDropStreamingLevelCopy = EventsToUpdate.OnDropStreamingLevel](const TArray< TWeakObjectPtr<class ULevelStreaming> >& Levels, UEdGraph* InGraph, const FVector2f& InDropLocation)
		{
			OnDropStreamingLevelCopy.Execute(Levels, InGraph, FVector2D(InDropLocation));
		});
	}
	if (EventsToUpdate.OnCreateActionMenu.IsBound() && !EventsToUpdate.OnCreateActionMenuAtLocation.IsBound())
	{
		EventsToUpdate.OnCreateActionMenuAtLocation = FOnCreateActionMenuAtLocation::CreateLambda([OnCreateActionMenuCopy = EventsToUpdate.OnCreateActionMenu](UEdGraph* InGraph, const FVector2f& InLocation, const TArray<UEdGraphPin*>& InPins, bool bAutoExpand, FActionMenuClosed InMenuClosed)
		{
			return OnCreateActionMenuCopy.Execute(InGraph, FVector2D(InLocation), InPins, bAutoExpand, InMenuClosed);
		});
	}
	if (EventsToUpdate.OnSpawnNodeByShortcut.IsBound() && !EventsToUpdate.OnSpawnNodeByShortcutAtLocation.IsBound())
	{
		EventsToUpdate.OnSpawnNodeByShortcutAtLocation = FOnSpawnNodeByShortcutAtLocation::CreateLambda([OnSpawnNodeByShortcutCopy = EventsToUpdate.OnSpawnNodeByShortcut](FInputChord InInputChord, const FVector2f& InLocation)
		{
			return OnSpawnNodeByShortcutCopy.Execute(InInputChord, FVector2D(InLocation));
		});
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}


/**
 * Loads the GraphEditorModule and constructs a GraphEditor as a child of this widget.
 *
 * @param InArgs   Declaration params from which to construct the widget.
 */
void SGraphEditor::Construct( const FArguments& InArgs )
{
	EdGraphObj = InArgs._GraphToEdit;
	OnGraphModuleReloadedCallback = InArgs._OnGraphModuleReloaded;
	AssetEditorToolkit = InArgs._AssetEditorToolkit;

	// Register this widget with the module so that we can gracefully handle the module being unloaded.
	// See OnModuleUnloading()
	RegisterGraphEditor( SharedThis(this) );

	// Register a graph modified handler
	if (EdGraphObj != NULL)
	{
		EdGraphObj->AddOnGraphChangedHandler( FOnGraphChanged::FDelegate::CreateSP( this, &SGraphEditor::OnGraphChanged ) );
	}

	// Make the actual GraphEditor instance
	ConstructImplementation(InArgs);
}

// Invoked to let this widget know that the GraphEditor module has been reloaded
void SGraphEditor::OnModuleReloaded()
{
	OnGraphModuleReloadedCallback.ExecuteIfBound(EdGraphObj);
}

// Invoked to let this widget know that the GraphEditor module is being unloaded.
void SGraphEditor::OnModuleUnloading()
{
	this->ChildSlot
	[
		SMissingWidget::MakeMissingWidget()
	];

	check( Implementation.IsUnique() ); 
	Implementation.Reset();
}

void SGraphEditor::RegisterGraphEditor( const TSharedRef<SGraphEditor>& InGraphEditor )
{
	// Compact the list of GraphEditor instances
	for (int32 WidgetIndex = 0; WidgetIndex < AllInstances.Num(); ++WidgetIndex)
	{
		if (!AllInstances[WidgetIndex].IsValid())
		{
			AllInstances.RemoveAt(WidgetIndex);
			--WidgetIndex;
		}
	}

	AllInstances.Add(InGraphEditor);
}

void SGraphEditor::NotifyPrePropertyChange( const FString& PropertyName )
{
	if (EdGraphObj)
	{
		EdGraphObj->NotifyPreChange(PropertyName);
	}
}

void SGraphEditor::NotifyPostPropertyChange( const FPropertyChangedEvent& PropertyChangedEvent, const FString& PropertyName )
{
	if (EdGraphObj)
	{
		EdGraphObj->NotifyPostChange(PropertyChangedEvent, PropertyName);
	}
}

void SGraphEditor::ResetAllNodesUnrelatedStates()
{
	TArray<class UEdGraphNode*> AllNodes = GetCurrentGraph()->Nodes;

	for (auto& GraphNode : AllNodes)
	{
		if (GraphNode->IsNodeUnrelated())
		{
			GraphNode->SetNodeUnrelated(false);
		}
	}
}

void SGraphEditor::FocusCommentNodes(TArray<UEdGraphNode*> &CommentNodes, TArray<UEdGraphNode*> &RelatedNodes)
{
	for (auto& CommentNode : CommentNodes)
	{
		CommentNode->SetNodeUnrelated(true);

		const FSlateRect CommentRect(
			static_cast<float>(CommentNode->NodePosX),
			static_cast<float>(CommentNode->NodePosY),
			static_cast<float>(CommentNode->NodePosX + CommentNode->NodeWidth),
			static_cast<float>(CommentNode->NodePosY + CommentNode->NodeHeight)
		);

		for (auto& RelatedNode : RelatedNodes)
		{
			const FVector2D NodePos(RelatedNode->NodePosX, RelatedNode->NodePosY);

			if (CommentRect.ContainsPoint(NodePos))
			{
				CommentNode->SetNodeUnrelated(false);
				break;
			}
		}
	}
}

TSharedPtr<SGraphEditor> SGraphEditor::FindGraphEditorForGraph(const UEdGraph* Graph)
{
	for (TWeakPtr<SGraphEditor>& WeakWidget : AllInstances)
	{
		TSharedPtr<SGraphEditor> WidgetPtr = WeakWidget.Pin();
		if (WidgetPtr.IsValid() && (WidgetPtr->GetCurrentGraph() == Graph))
		{
			return WidgetPtr;
		}
	}

	return nullptr;
}
