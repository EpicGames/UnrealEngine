// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/GroupActor.h"
#include "Misc/MessageDialog.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/Selection.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "UnrealEdGlobals.h"
#include "SceneInterface.h"
#include "SceneView.h"
#include "ScopedTransaction.h"
#include "LevelEditorViewport.h"
#include "Layers/LayersSubsystem.h"
#include "ActorGroupingUtils.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementSelectionSet.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroupActor)

const FLinearColor BOXCOLOR_LOCKEDGROUPS( 0.0f, 1.0f, 0.0f );
const FLinearColor BOXCOLOR_UNLOCKEDGROUPS( 1.0f, 0.0f, 0.0f );


AGroupActor::AGroupActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bLocked = true;

	USceneComponent* GroupComponent = CreateDefaultSubobject<USceneComponent>(TEXT("GroupComponent"));
	RootComponent = GroupComponent;
}

void AGroupActor::PostActorCreated()
{
	// Cache our newly created group
	if( !GetWorld()->IsPlayInEditor() && !IsRunningCommandlet() && GIsEditor )
	{
		GetWorld()->ActiveGroupActors.AddUnique(this);
	}
	Super::PostActorCreated();
}

void AGroupActor::PostLoad()
{
	GetLevel()->ConditionalPostLoad();

	if( !GetWorld()->IsPlayInEditor() && !IsRunningCommandlet() && GIsEditor )
	{
		// Cache group on de-serialization
		GetWorld()->ActiveGroupActors.AddUnique(this);

		// Fix up references for GetParentForActor()
		for(int32 i=0; i<GroupActors.Num(); ++i)
		{
			if( GroupActors[i] != NULL )
			{
				GroupActors[i]->GroupActor = this;
			}
		}
	}
	Super::PostLoad();
}

void AGroupActor::PostEditChangeProperty( FPropertyChangedEvent& PropertyChangedEvent )
{
	// Re-instate group as active if it had children after undo/redo
	if(GroupActors.Num() || SubGroups.Num())
	{
		GetWorld()->ActiveGroupActors.AddUnique(this);
	}
	else // Otherwise, attempt to remove them
	{
		GetWorld()->ActiveGroupActors.Remove(this);
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void AGroupActor::PostEditUndo()
{
	Super::PostEditUndo();

	if (!IsValidChecked(this))
	{
		GetWorld()->ActiveGroupActors.RemoveSwap(this);
	}
	else
	{
		// Cache group on de-serialization
		GetWorld()->ActiveGroupActors.AddUnique(this);

		// Fix up references for GetParentForActor()
		for (int32 i = 0; i < GroupActors.Num(); ++i)
		{
			if (GroupActors[i] != NULL)
			{
				GroupActors[i]->GroupActor = this;
			}
		}
	}
}

bool AGroupActor::IsSelected() const
{
	// Group actors can only count as 'selected' if they are locked 
	return (IsLocked() && HasSelectedActors()) || Super::IsSelected();
}

void AGroupActor::ForEachActorInGroup(TFunctionRef<void(AActor*, AGroupActor*)> InCallback)
{
	for (AActor* Actor : GroupActors)
	{
		if (Actor)
		{
			InCallback(Actor, this);
		}
	}
	for (AGroupActor* SubGroup : SubGroups)
	{
		if (SubGroup)
		{
			SubGroup->ForEachActorInGroup(InCallback);
		}
	}
	InCallback(this, this);
}

namespace GroupActorHelpers
{

bool ActorHasParentInGroup(const AActor* Actor, const AGroupActor* GroupActor)
{
	check(Actor && GroupActor);
	// Check that we've not got a parent attachment within the group.
	if (USceneComponent* RootComponent = Actor->GetRootComponent())
	{
		for (const AActor* OtherActor : GroupActor->GroupActors)
		{
			if (OtherActor && OtherActor != Actor)
			{
				USceneComponent* OtherRootComponent = OtherActor->GetRootComponent();
				if (OtherRootComponent && RootComponent->IsAttachedTo(OtherRootComponent))
				{
					// We do have parent so don't apply the delta - our parent object will apply it instead.
					return true;
				}
			}
		}
	}
	return false;
}

bool ActorHasParentInSelection(const AActor* Actor, FTypedElementListConstRef SelectionSet)
{
	check(Actor);
	for (const AActor* ParentActor = Actor->GetAttachParentActor(); ParentActor; ParentActor = ParentActor->GetAttachParentActor())
	{
		FTypedElementHandle ParentActorElementHandle = UEngineElementsLibrary::AcquireEditorActorElementHandle(ParentActor, /*bAllowCreate*/false);
		if (ParentActorElementHandle && SelectionSet->Contains(ParentActorElementHandle))
		{
			return true;
		}
	}
	return false;
}

} // namespace GroupActorHelpers

void AGroupActor::ForEachMovableActorInGroup(const UTypedElementSelectionSet* InSelectionSet, TFunctionRef<void(AActor*, AGroupActor*)> InCallback)
{
	const UTypedElementSelectionSet* SelectionSet = InSelectionSet ? InSelectionSet : GEditor->GetSelectedActors()->GetElementSelectionSet();
	for (AActor* Actor : GroupActors)
	{
		if (Actor)
		{
			// Check that we've not got a parent attachment within the group/selection
			const bool bCanApplyDelta = !GroupActorHelpers::ActorHasParentInGroup(Actor, this) && !GroupActorHelpers::ActorHasParentInSelection(Actor, SelectionSet->GetElementList());
			if (bCanApplyDelta)
			{
				InCallback(Actor, this);
			}
		}
	}
	for (AGroupActor* SubGroup : SubGroups)
	{
		if (SubGroup)
		{
			SubGroup->ForEachMovableActorInGroup(SelectionSet, InCallback);
		}
	}
	InCallback(this, this);
}

void AGroupActor::GroupApplyDelta(const FVector& InDrag, const FRotator& InRot, const FVector& InScale )
{
	ForEachMovableActorInGroup(nullptr, [&InDrag, &InRot, &InScale](AActor* InGroupedActor, AGroupActor* InGroupActor)
	{
		GEditor->ApplyDeltaToActor(InGroupActor, true, &InDrag, &InRot, &InScale);
	});
}

void AGroupActor::SetIsTemporarilyHiddenInEditor( bool bIsHidden )
{
	Super::SetIsTemporarilyHiddenInEditor( bIsHidden );

	for(int32 ActorIndex=0; ActorIndex<GroupActors.Num(); ++ActorIndex)
	{
		if( GroupActors[ActorIndex] != NULL )
		{
			GroupActors[ActorIndex]->SetIsTemporarilyHiddenInEditor(bIsHidden);
		}
	}

	for(int32 SubGroupIndex=0; SubGroupIndex<SubGroups.Num(); ++SubGroupIndex)
	{
		if( SubGroups[SubGroupIndex] != NULL )
		{
			SubGroups[SubGroupIndex]->SetIsTemporarilyHiddenInEditor(bIsHidden);
		}
	}
}

void AGroupActor::GetActorBounds(bool bOnlyCollidingComponents, FVector& Origin, FVector& BoxExtent, bool bIncludeFromChildActors) const
{
	FBox Bounds = GetComponentsBoundingBox(!bOnlyCollidingComponents);;

	for(int32 ActorIndex=0; ActorIndex<GroupActors.Num(); ++ActorIndex)
	{
		if( GroupActors[ActorIndex] != NULL )
		{
			FVector ActorOrigin;
			FVector ActorBoxExtent;
			GroupActors[ActorIndex]->GetActorBounds(bOnlyCollidingComponents, ActorOrigin, ActorBoxExtent, bIncludeFromChildActors);

			Bounds += FBox(ActorOrigin - ActorBoxExtent, ActorOrigin + ActorBoxExtent);
		}
	}

	for(int32 SubGroupIndex=0; SubGroupIndex<SubGroups.Num(); ++SubGroupIndex)
	{
		if( SubGroups[SubGroupIndex] != NULL )
		{
			FVector SubGroupOrigin;
			FVector SubGroupBoxExtent;
			SubGroups[SubGroupIndex]->GetActorBounds(bOnlyCollidingComponents, SubGroupOrigin, SubGroupBoxExtent, bIncludeFromChildActors);

			Bounds += FBox(SubGroupOrigin - SubGroupBoxExtent, SubGroupOrigin + SubGroupBoxExtent);
		}
	}

	// To keep consistency with the other GetBounds functions, transform our result into an origin / extent formatting
	Bounds.GetCenterAndExtents(Origin, BoxExtent);
}

#if WITH_EDITOR
void AGroupActor::GetStreamingBounds(FBox& OutRuntimeBounds, FBox& OutEditorBounds) const
{
	Super::GetStreamingBounds(OutRuntimeBounds, OutEditorBounds);

	for (AActor* Actor : GroupActors)
	{
		if (Actor)
		{
			FBox RuntimeBounds;
			FBox EditorBounds;
			Actor->GetStreamingBounds(RuntimeBounds, EditorBounds);
			OutRuntimeBounds += RuntimeBounds;
			OutEditorBounds += EditorBounds;
		}
	}

	for (AGroupActor* SubGroupActor : SubGroups)
	{
		if (SubGroupActor)
		{
			FBox RuntimeBounds;
			FBox EditorBounds;
			SubGroupActor->GetStreamingBounds(RuntimeBounds, EditorBounds);
			OutRuntimeBounds += RuntimeBounds;
			OutEditorBounds += EditorBounds;

		}
	}
}
#endif

void GetBoundingVectorsForGroup(AGroupActor* GroupActor, FViewport* Viewport, FVector& OutVectorMin, FVector& OutVectorMax)
{
	// Draw a bounding box for grouped actors using the vector range we can gather from any child actors (including subgroups)
	OutVectorMin = FVector(BIG_NUMBER);
	OutVectorMax = FVector(-BIG_NUMBER);

	// Grab all actors for this group, including those within subgroups
	TArray<AActor*> ActorsInGroup;
	GroupActor->GetGroupActors(ActorsInGroup, true);

	// Loop through and collect each actor, using their bounding box to create the bounds for this group
	for(int32 ActorIndex = 0; ActorIndex < ActorsInGroup.Num(); ++ActorIndex)
	{
		AActor* Actor = ActorsInGroup[ActorIndex];
		uint64 HiddenClients = Actor->HiddenEditorViews;
		bool bActorHiddenForViewport = false;
		if(!Actor->IsHiddenEd())
		{
			if(Viewport)
			{
				for(int32 ViewIndex=0; ViewIndex<GEditor->GetLevelViewportClients().Num(); ++ViewIndex)
				{
					// If the current viewport is hiding this actor, don't draw brackets around it
					if(Viewport->GetClient() == GEditor->GetLevelViewportClients()[ViewIndex] && HiddenClients & ((uint64)1 << ViewIndex))
					{
						bActorHiddenForViewport = true;
						break;
					}
				}
			}

			if(!bActorHiddenForViewport)
			{
				FBox ActorBox = Actor->GetComponentsBoundingBox( true );

				// MinVector
				OutVectorMin.X = FMath::Min<FVector::FReal>( ActorBox.Min.X, OutVectorMin.X );
				OutVectorMin.Y = FMath::Min<FVector::FReal>( ActorBox.Min.Y, OutVectorMin.Y );
				OutVectorMin.Z = FMath::Min<FVector::FReal>( ActorBox.Min.Z, OutVectorMin.Z );
				// MaxVector
				OutVectorMax.X = FMath::Max<FVector::FReal>( ActorBox.Max.X, OutVectorMax.X );
				OutVectorMax.Y = FMath::Max<FVector::FReal>( ActorBox.Max.Y, OutVectorMax.Y );
				OutVectorMax.Z = FMath::Max<FVector::FReal>( ActorBox.Max.Z, OutVectorMax.Z );
			}
		}
	}	
}

/**
 * Draw brackets around all given groups
 * @param	PDI			FPrimitiveDrawInterface used to draw lines in active viewports
 * @param	Viewport	Current viewport being rendered
 * @param	InGroupList	Array of groups to draw brackets for
 */
void PrivateDrawBracketsForGroups( FPrimitiveDrawInterface* PDI, FViewport* Viewport, const TArray<AGroupActor*>& InGroupList )
{
	// Loop through each given group and draw all subgroups and actors
	for(int32 GroupIndex=0; GroupIndex<InGroupList.Num(); ++GroupIndex)
	{
		AGroupActor* GroupActor = InGroupList[GroupIndex];
		if( GroupActor->GetWorld() == PDI->View->Family->Scene->GetWorld() )
		{
			const FLinearColor GROUP_COLOR = GroupActor->IsLocked() ? BOXCOLOR_LOCKEDGROUPS : BOXCOLOR_UNLOCKEDGROUPS;

			FVector MinVector;
			FVector MaxVector;
			GetBoundingVectorsForGroup( GroupActor, Viewport, MinVector, MaxVector );

			// Create a bracket offset to pad the space between brackets and actor(s) and determine the length of our corner axises
			float BracketOffset = FVector::Dist(MinVector, MaxVector) * 0.1f;
			MinVector = MinVector - BracketOffset;
			MaxVector = MaxVector + BracketOffset;

			// Calculate bracket corners based on min/max vectors
			TArray<FVector> BracketCorners;

			// Bottom Corners
			BracketCorners.Add(FVector(MinVector.X, MinVector.Y, MinVector.Z));
			BracketCorners.Add(FVector(MinVector.X, MaxVector.Y, MinVector.Z));
			BracketCorners.Add(FVector(MaxVector.X, MaxVector.Y, MinVector.Z));
			BracketCorners.Add(FVector(MaxVector.X, MinVector.Y, MinVector.Z));

			// Top Corners
			BracketCorners.Add(FVector(MinVector.X, MinVector.Y, MaxVector.Z));
			BracketCorners.Add(FVector(MinVector.X, MaxVector.Y, MaxVector.Z));
			BracketCorners.Add(FVector(MaxVector.X, MaxVector.Y, MaxVector.Z));
			BracketCorners.Add(FVector(MaxVector.X, MinVector.Y, MaxVector.Z));

			for(int32 BracketCornerIndex=0; BracketCornerIndex<BracketCorners.Num(); ++BracketCornerIndex)
			{
				// Direction corner axis should be pointing based on min/max
				const FVector CORNER = BracketCorners[BracketCornerIndex];
				const int32 DIR_X = CORNER.X == MaxVector.X ? -1 : 1;
				const int32 DIR_Y = CORNER.Y == MaxVector.Y ? -1 : 1;
				const int32 DIR_Z = CORNER.Z == MaxVector.Z ? -1 : 1;

				PDI->DrawLine( CORNER, FVector(CORNER.X + (BracketOffset * DIR_X), CORNER.Y, CORNER.Z), GROUP_COLOR, SDPG_Foreground );
				PDI->DrawLine( CORNER, FVector(CORNER.X, CORNER.Y + (BracketOffset * DIR_Y), CORNER.Z), GROUP_COLOR, SDPG_Foreground );
				PDI->DrawLine( CORNER, FVector(CORNER.X, CORNER.Y, CORNER.Z + (BracketOffset * DIR_Z)), GROUP_COLOR, SDPG_Foreground );
			}

			// Recurse through to any subgroups
			TArray<AGroupActor*> SubGroupsInGroup;
			GroupActor->GetSubGroups(SubGroupsInGroup);
			PrivateDrawBracketsForGroups(PDI, Viewport, SubGroupsInGroup);
		}
	}
}


void AGroupActor::DrawBracketsForGroups( FPrimitiveDrawInterface* PDI, FViewport* Viewport, bool bMustBeSelected/*=true*/ )
{
	// Don't draw group actor brackets in game view
	if (Viewport->GetClient()->IsInGameView())
	{
		return;
	}

	if( UActorGroupingUtils::IsGroupingActive() )
	{
		check(PDI);
		
		UWorld* EditorWorld = GEditor->GetEditorWorldContext().World();
		if (EditorWorld)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AGroupActor::DrawBracketsForGroup);
			TArray<AGroupActor*> GroupsToDraw;
			
			for(int32 GroupIndex=0; GroupIndex < EditorWorld->ActiveGroupActors.Num(); ++GroupIndex)
			{
				AGroupActor* GroupActor = Cast<AGroupActor>(EditorWorld->ActiveGroupActors[GroupIndex]);
				if (GroupActor != NULL)
				{
					if (bMustBeSelected)
					{
						// If we're only drawing for selected group, grab only those that have currently selected actors
						if (GroupActor->HasSelectedActors())
						{
							// We want to start drawing groups from the highest root level.
							// Subgroups will be propagated through during the draw code.
							GroupActor = AGroupActor::GetRootForActor(GroupActor);
							GroupsToDraw.Add(GroupActor);
						}
					}
					else
					{
						// Otherwise, just add all group actors
						GroupsToDraw.Add(GroupActor);
					}
				}
			}

			PrivateDrawBracketsForGroups(PDI, Viewport, GroupsToDraw);
		}
	}
}

/**
 * Checks to see if the given GroupActor has any parents in the given Array.
 * @param	InGroupActor	Group to check lineage
 * @param	InGroupArray	Array to search for the given group's parent
 * @return	True if a parent was found.
 */
bool GroupHasParentInArray(AGroupActor* InGroupActor, TArray<AGroupActor*>& InGroupArray)
{
	check(InGroupActor);
	AGroupActor* CurrentParentNode = AGroupActor::GetParentForActor(InGroupActor);

	// Use a cursor pointer to continually move up from our starting pointer (InGroupActor) through the hierarchy until
	// we find a valid parent in the given array, or run out of nodes.
	while( CurrentParentNode )
	{
		if(InGroupArray.Contains(CurrentParentNode))
		{
			return true;
		}
		CurrentParentNode = AGroupActor::GetParentForActor(CurrentParentNode);
	}
	return false;
}


void AGroupActor::RemoveSubGroupsFromArray(TArray<AGroupActor*>& GroupArray)
{
	for(int32 GroupIndex=0; GroupIndex<GroupArray.Num(); ++GroupIndex)
	{
		AGroupActor* GroupToCheck = GroupArray[GroupIndex];
		if(GroupHasParentInArray(GroupToCheck, GroupArray))
		{
			GroupArray.Remove(GroupToCheck);
			--GroupIndex;
		}
	}
}

AGroupActor* AGroupActor::GetRootForActor(AActor* InActor, bool bMustBeLocked/*=false*/, bool bMustBeSelected/*=false*/, bool bMustBeUnlocked/*=false*/, bool bMustBeUnselected/*=false*/)
{
	AGroupActor* RootNode = NULL;
	// If InActor is a group, use that as the beginning iteration node, else try to find the parent
	AGroupActor* InGroupActor = Cast<AGroupActor>(InActor);
	AGroupActor* IteratingNode = InGroupActor == NULL ? AGroupActor::GetParentForActor(InActor) : InGroupActor;
	while( IteratingNode )
	{
		if ( (!bMustBeLocked   ||  IteratingNode->IsLocked()) && (!bMustBeSelected   ||  IteratingNode->HasSelectedActors())
		&&   (!bMustBeUnlocked || !IteratingNode->IsLocked()) && (!bMustBeUnselected || !IteratingNode->HasSelectedActors()) )
		{
			RootNode = IteratingNode;
		}
		IteratingNode = AGroupActor::GetParentForActor(IteratingNode);
	}
	return RootNode;
}


AGroupActor* AGroupActor::GetParentForActor(AActor* InActor)
{
	return Cast<AGroupActor>(InActor->GroupActor);
}


const int32 AGroupActor::NumActiveGroups( bool bSelected/*=false*/, bool bDeepSearch/*=true*/ )
{
	UWorld* EditorWorld = GEditor->GetEditorWorldContext().World();
	if (EditorWorld)
	{
		if(!bSelected)
		{
			return EditorWorld->ActiveGroupActors.Num();
		}

		int32 ActiveSelectedGroups = 0;
		for(int32 GroupIdx=0; GroupIdx < EditorWorld->ActiveGroupActors.Num(); ++GroupIdx )
		{
			AGroupActor* GroupActor = Cast<AGroupActor>(EditorWorld->ActiveGroupActors[GroupIdx]);
			if( GroupActor != NULL )
			{
				if(GroupActor->HasSelectedActors(bDeepSearch))
				{
					++ActiveSelectedGroups;
				}
			}
		}

		return ActiveSelectedGroups;
	}
	
	return 0;
}


void AGroupActor::AddSelectedActorsToSelectedGroup()
{
	UWorld* EditorWorld = GEditor->GetEditorWorldContext().World();
	if (EditorWorld)
	{
		int32 SelectedGroupIndex = -1;
		for(int32 GroupIdx=0; GroupIdx < EditorWorld->ActiveGroupActors.Num(); ++GroupIdx )
		{
			AGroupActor* GroupActor = Cast<AGroupActor>(EditorWorld->ActiveGroupActors[GroupIdx]);
			if( GroupActor != NULL )
			{
				if(GroupActor->HasSelectedActors(false))
				{
					// Assign the index of the selected group.
					// If this is the second group we find, too many groups are selected, return.
					if( SelectedGroupIndex == -1 )
					{
						SelectedGroupIndex = GroupIdx;
					}
					else
					{
						return;
					}
				}
			}
		}

		AGroupActor* SelectedGroup = SelectedGroupIndex != -1 ? Cast<AGroupActor>(EditorWorld->ActiveGroupActors[SelectedGroupIndex]) : nullptr;
		if( SelectedGroup != nullptr )
		{
			ULevel* GroupLevel = SelectedGroup->GetLevel();

			// We've established that only one group is selected, so we can just call Add on all these actors.
			// Any actors already in the group will be ignored.
		
			TArray<AActor*> ActorsToAdd;

			bool bActorsInSameLevel = true;
			for ( FSelectionIterator It( GEditor->GetSelectedActorIterator() ) ; It ; ++It )
			{
				AActor* Actor = CastChecked<AActor>( *It );
		
				if( Actor->GetLevel() == GroupLevel )
				{
					ActorsToAdd.Add( Actor );
				}
				else
				{
					bActorsInSameLevel = false;
					break;
				}
			}

			if( bActorsInSameLevel )
			{
				if( ActorsToAdd.Num() > 0 )
				{
					const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "Group_Add", "Add Actors to Group") );
					for( int32 ActorIndex = 0; ActorIndex < ActorsToAdd.Num(); ++ActorIndex )
					{
						if ( ActorsToAdd[ActorIndex] != SelectedGroup )
						{
							SelectedGroup->Add( *ActorsToAdd[ActorIndex] );
						}
					}

					SelectedGroup->CenterGroupLocation();
					SelectedGroup->SetActorRotation(ActorsToAdd.Last()->GetActorRotation());
				}
			}
			else
			{
				FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Group_CantCreateGroupMultipleLevels", "Can't group the selected actors because they are in different levels.") );
			}
		}
	}
}


void AGroupActor::LockSelectedGroups()
{
	UWorld* EditorWorld = GEditor->GetEditorWorldContext().World();
	if (EditorWorld)
	{
		TArray<AGroupActor*> GroupsToLock;
		for ( int32 GroupIndex=0; GroupIndex<EditorWorld->ActiveGroupActors.Num(); ++GroupIndex )
		{
			AGroupActor* GroupToLock = Cast<AGroupActor>(EditorWorld->ActiveGroupActors[GroupIndex]);
			if( GroupToLock != NULL )
			{
				if( GroupToLock->HasSelectedActors(false) )
				{
					// If our selected group is already locked, move up a level to add it's potential parent for locking
					if( GroupToLock->IsLocked() )
					{
						AGroupActor* GroupParent = AGroupActor::GetParentForActor(GroupToLock);
						if(GroupParent && !GroupParent->IsLocked())
						{
							GroupsToLock.AddUnique(GroupParent);
						}
					}
					else // if it's not locked, add it instead!
					{
						GroupsToLock.AddUnique(GroupToLock);
					}
				}
			}
		}

		if( GroupsToLock.Num() > 0 )
		{
			const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "Group_Lock", "Lock Groups") );
			for ( int32 GroupIndex=0; GroupIndex<GroupsToLock.Num(); ++GroupIndex )
			{
				AGroupActor* GroupToLock = GroupsToLock[GroupIndex];
				GroupToLock->Modify();
				GroupToLock->Lock();
				GEditor->SelectGroup(GroupToLock, false );
			}
			GEditor->NoteSelectionChange();
		}
	}
}


void AGroupActor::UnlockSelectedGroups()
{
	UWorld* EditorWorld = GEditor->GetEditorWorldContext().World();
	if (EditorWorld)
	{
		TArray<AGroupActor*> GroupsToUnlock;
		for ( int32 GroupIndex=0; GroupIndex<EditorWorld->ActiveGroupActors.Num(); ++GroupIndex )
		{
			AGroupActor* GroupToUnlock = Cast<AGroupActor>(EditorWorld->ActiveGroupActors[GroupIndex]);
			if( GroupToUnlock != NULL )
			{
				if( GroupToUnlock->IsSelected() )
				{
					GroupsToUnlock.Add(GroupToUnlock);
				}
			}
		}

		// Only unlock topmost selected group(s)
		AGroupActor::RemoveSubGroupsFromArray(GroupsToUnlock);
		if( GroupsToUnlock.Num() > 0 )
		{
			const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "Group_Unlock", "Unlock Groups") );
			for ( int32 GroupIndex=0; GroupIndex<GroupsToUnlock.Num(); ++GroupIndex)
			{
				AGroupActor* GroupToUnlock = GroupsToUnlock[GroupIndex];
				GroupToUnlock->Modify();
				GroupToUnlock->Unlock();
			}
			GEditor->NoteSelectionChange();
		}
	}
}


void AGroupActor::ToggleGroupMode()
{
	UActorGroupingUtils::SetGroupingActive(!UActorGroupingUtils::IsGroupingActive());

	// Update group selection in the editor to reflect the toggle
	SelectGroupsInSelection();
	GEditor->RedrawAllViewports();

	GEditor->SaveConfig();
}


void AGroupActor::SelectGroupsInSelection()
{
	if( UActorGroupingUtils::IsGroupingActive() )
	{
		TArray<AGroupActor*> GroupsToSelect;
		for ( FSelectionIterator It( GEditor->GetSelectedActorIterator() ) ; It ; ++It )
		{
			AActor* Actor = static_cast<AActor*>( *It );
			checkSlow( Actor->IsA(AActor::StaticClass()) );
			AGroupActor* GroupActor = AGroupActor::GetRootForActor(Actor, true);

			if(GroupActor)
			{
				GroupsToSelect.AddUnique(GroupActor);
			}
		}

		// Select any groups from the currently selected actors
		for ( int32 GroupIndex=0; GroupIndex<GroupsToSelect.Num(); ++GroupIndex)
		{
			AGroupActor* GroupToSelect = GroupsToSelect[GroupIndex];
			GEditor->SelectGroup(GroupToSelect);
		}
		GEditor->NoteSelectionChange();
	}
}


void AGroupActor::Lock()
{
	bLocked = true;
	for(int32 SubGroupIdx=0; SubGroupIdx < SubGroups.Num(); ++SubGroupIdx )
	{
		if( SubGroups[SubGroupIdx] != NULL )
		{
			SubGroups[SubGroupIdx]->Lock();
		}
	}
}


void AGroupActor::Add(AActor& InActor)
{	
	// See if the incoming actor already belongs to a group
	AGroupActor* InActorParent = AGroupActor::GetParentForActor(&InActor);
	if(InActorParent) // If so, detach it first
	{
		if(InActorParent == this)
		{
			return;
		}
		InActorParent->Modify();
		InActorParent->Remove(InActor);
	}
	
	Modify();
	AGroupActor* InGroupPtr = Cast<AGroupActor>(&InActor);
	if(InGroupPtr)
	{
		check(InGroupPtr != this);
		SubGroups.AddUnique(InGroupPtr);
	}
	else
	{
		GroupActors.AddUnique(&InActor);
		InActor.Modify();
		InActor.GroupActor = this;
	}
}


void AGroupActor::Remove(AActor& InActor)
{
	AGroupActor* InGroupPtr = Cast<AGroupActor>(&InActor);
	if(InGroupPtr && SubGroups.Contains(InGroupPtr))
	{
		Modify();
		SubGroups.Remove(InGroupPtr);
	}
	else if(GroupActors.Contains(&InActor))
	{
		Modify();
		GroupActors.Remove(&InActor);
		InActor.Modify();
		InActor.GroupActor = NULL;
	}
	
	PostRemove();
}

void AGroupActor::PostRemove()
{
	// If all children have been removed (or only one subgroup remains), this group is no longer active.
	if( GroupActors.Num() == 0 && SubGroups.Num() <= 1 )
	{
		// Remove any potentially remaining subgroups
		SubGroups.Empty();

		// Destroy the actor and remove it from active groups
		AGroupActor* ParentGroup = AGroupActor::GetParentForActor(this);
		if(ParentGroup)
		{
			ParentGroup->Modify();
			ParentGroup->Remove(*this);
		}

		UWorld* MyWorld = GetWorld();
		if( MyWorld )
		{
			// Group is no longer active
			MyWorld->ActiveGroupActors.Remove(this);

			MyWorld->ModifyLevel(GetLevel());
			
			// Mark the group actor for removal
			MarkPackageDirty();

			// If not currently garbage collecting (changing maps, saving, etc), remove the group immediately
			if(!IsGarbageCollecting())
			{
				// Refresh all editor browsers after removal
				FScopedRefreshAllBrowsers LevelRefreshAllBrowsers;

				// Destroy group and clear references.
				GEditor->SelectActor( this, /*bSelected=*/ false, /*bNotify=*/ false );
				ULayersSubsystem* LayersSubsystem = GEditor->GetEditorSubsystem<ULayersSubsystem>();
				LayersSubsystem->DisassociateActorFromLayers( this );
				MyWorld->EditorDestroyActor( this, false );			
				
				LevelRefreshAllBrowsers.Request();
			}
		}
	}
}


bool AGroupActor::Contains(AActor& InActor) const
{
	AActor* InActorPtr = &InActor;
	AGroupActor* InGroupPtr = Cast<AGroupActor>(InActorPtr);
	if(InGroupPtr)
	{
		return SubGroups.Contains(InGroupPtr);
	}
	return GroupActors.Contains(InActorPtr);
}


bool AGroupActor::HasSelectedActors(bool bDeepSearch/*=true*/) const
{
	for(int32 ActorIndex=0; ActorIndex<GroupActors.Num(); ++ActorIndex)
	{
		if( GroupActors[ActorIndex] != NULL )
		{
			if( GroupActors[ActorIndex]->IsSelected() )
			{
				return true; 
			}
		}
	}
	if(bDeepSearch)
	{
		for(int32 GroupIndex=0; GroupIndex<SubGroups.Num(); ++GroupIndex)
		{
			if( SubGroups[ GroupIndex ] != NULL )
			{
				if( SubGroups[GroupIndex]->HasSelectedActors(bDeepSearch) )
				{
					return true; 
				}
			}
		}
	}
	return false;
}


void AGroupActor::ClearAndRemove()
{
	// Actors can potentially be NULL here. Some older maps can serialize invalid Actors 
	// into GroupActors or SubGroups.
	for(int32 ActorIndex=0; ActorIndex<GroupActors.Num(); ++ActorIndex)
	{
		if( GroupActors[ActorIndex] )
		{
			Remove(*GroupActors[ActorIndex]);
		}
		else
		{
			GroupActors.RemoveAt(ActorIndex);
			PostRemove();
		}
		--ActorIndex;
	}
	for(int32 SubGroupIndex=0; SubGroupIndex<SubGroups.Num(); ++SubGroupIndex)
	{
		if( SubGroups[SubGroupIndex] )
		{
			Remove(*SubGroups[SubGroupIndex]);
		}
		else
		{
			SubGroups.RemoveAt(SubGroupIndex);
			PostRemove();
		}
		--SubGroupIndex;
	}
}


void AGroupActor::CenterGroupLocation()
{
	FVector MinVector;
	FVector MaxVector;
	GetBoundingVectorsForGroup( this, NULL, MinVector, MaxVector );

	SetActorLocation((MinVector + MaxVector) * 0.5f, false);
	GEditor->NoteSelectionChange();
}


void AGroupActor::GetGroupActors(TArray<AActor*>& OutGroupActors, bool bRecurse/*=false*/) const
{
	if( bRecurse )
	{
		for(int32 i=0; i<SubGroups.Num(); ++i)
		{
			if( SubGroups[i] != NULL )
			{
				SubGroups[i]->GetGroupActors(OutGroupActors, bRecurse);
			}
		}
	}
	else
	{
		OutGroupActors.Empty();
	}
	for(int32 i=0; i<GroupActors.Num(); ++i)
	{
		if( GroupActors[i] != NULL )
		{
			OutGroupActors.Add(GroupActors[i]);
		}
	}
}


void AGroupActor::GetSubGroups(TArray<AGroupActor*>& OutSubGroups, bool bRecurse/*=false*/) const
{
	if( bRecurse )
	{
		for(int32 i=0; i<SubGroups.Num(); ++i)
		{
			if( SubGroups[i] != NULL )
			{
				SubGroups[i]->GetSubGroups(OutSubGroups, bRecurse);
			}
		}
	}
	else
	{
		OutSubGroups.Empty();
	}
	for(int32 i=0; i<SubGroups.Num(); ++i)
	{
		if( SubGroups[i] != NULL )
		{
			OutSubGroups.Add(SubGroups[i]);
		}
	}
}


void AGroupActor::GetAllChildren(TArray<AActor*>& OutChildren, bool bRecurse/*=false*/) const
{
	GetGroupActors(OutChildren, bRecurse);
	TArray<AGroupActor*> OutSubGroups;
	GetSubGroups(OutSubGroups, bRecurse);
	for(int32 SubGroupIdx=0; SubGroupIdx<OutSubGroups.Num(); ++SubGroupIdx)
	{
		OutChildren.Add(OutSubGroups[SubGroupIdx]);
	}
}


int32 AGroupActor::GetActorNum() const
{
	return GroupActors.Num();
}

namespace UE::Editor::GroupActorUtil
{
	TArray<AGroupActor*> GetSelectedGroupActors(TArray<TObjectPtr<AActor>> ActiveGroupActors)
	{
		TArray<AGroupActor*> Result;
		for (const TObjectPtr<AActor>& Actor : ActiveGroupActors)
		{
			if (AGroupActor* SelectedGroup = Cast<AGroupActor>(Actor))
			{
				if (SelectedGroup->IsSelected())
				{
					Result.Add(SelectedGroup);
				}
			}
		}
		return Result;
	}
}

bool AGroupActor::SelectedGroupNeedsFixup()
{
	if (UWorld* EditorWorld = GEditor->GetEditorWorldContext().World())
	{
		TArray<AGroupActor*> SelectedGroupActors = UE::Editor::GroupActorUtil::GetSelectedGroupActors(EditorWorld->ActiveGroupActors);
		if (!SelectedGroupActors.IsEmpty())
		{
			//check if there's at least 1 GroupActor contains a nullptr within it's list of Actors
			return SelectedGroupActors.ContainsByPredicate([](const AGroupActor* SelectedGroupActor)
				{
					if (SelectedGroupActor)
					{
						return SelectedGroupActor->GroupActors.ContainsByPredicate([](const TObjectPtr<class AActor> Actor) { return Actor == nullptr; });
					}
					return false;
				});
		}
	}
	return false;
} // namespace UE::Editor::GroupActorUtil

void AGroupActor::FixupGroupActor()
{
	if (UWorld* EditorWorld = GEditor->GetEditorWorldContext().World())
	{
		TArray<AGroupActor*> SelectedGroupActors = UE::Editor::GroupActorUtil::GetSelectedGroupActors(EditorWorld->ActiveGroupActors);
		if (!SelectedGroupActors.IsEmpty())
		{
			for (AGroupActor* SelectedGroupActor : SelectedGroupActors)
			{
				if (SelectedGroupActor && SelectedGroupActor->GroupActors.ContainsByPredicate([](const TObjectPtr<class AActor> Actor) { return Actor == nullptr; }))
				{
					//remove all nullptr entries in the GroupActors array.
					SelectedGroupActor->Modify();
					SelectedGroupActor->GroupActors.RemoveAll([](const TObjectPtr<class AActor> Actor) { return Actor == nullptr; });
					SelectedGroupActor->GroupActors.Shrink();

					if (SelectedGroupActor->GroupActors.IsEmpty())
					{
						SelectedGroupActor->PostRemove();
					}
				}
			}
		}
	}
}
