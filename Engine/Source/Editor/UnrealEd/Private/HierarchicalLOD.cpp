// Copyright Epic Games, Inc. All Rights Reserved.

#include "HierarchicalLOD.h"
#include "Engine/World.h"
#include "Model.h"
#include "Stats/StatsMisc.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/PackageName.h"
#include "Components/StaticMeshComponent.h"
#include "Components/InstancedStaticMeshComponent.h"

#include "Logging/TokenizedMessage.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "Misc/MapErrors.h"
#include "HLOD/HLODSetup.h"

#include "Algo/Transform.h"
#include "EditorLevelUtils.h"
#include "Modules/ModuleManager.h"
#include "HAL/ThreadManager.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/LODActor.h"
#include "HLOD/HLODEngineSubsystem.h"
#include "HierarchicalLODProxyProcessor.h"
#include "HierarchicalLODUtilitiesModule.h"
#include "IHierarchicalLODUtilities.h"
#include "LevelUtils.h"
#include "MaterialUtilities.h"
#include "MeshDescription.h"
#include "MeshMergeModule.h"
#include "ObjectTools.h"
#include "StaticMeshOperations.h"
#include "UnrealEdGlobals.h"

#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"

#endif // WITH_EDITOR


#include "HierarchicalLODVolume.h"
#include "EngineUtils.h"
#include "FileHelpers.h"
#include "Serialization/ArchiveObjectCrc32.h"
#include "IMeshReductionManagerModule.h"
#include "Engine/HLODProxy.h"
#include "Engine/LevelStreaming.h"
#include "AssetCompilingManager.h"

#include "Materials/Material.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HierarchicalLOD)

DEFINE_LOG_CATEGORY_STATIC(LogLODGenerator, Log, All);

#define LOCTEXT_NAMESPACE "HierarchicalLOD"
#define CM_TO_METER		0.01f
#define METER_TO_CM		100.0f

UHierarchicalLODSettings::UHierarchicalLODSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer), bForceSettingsInAllMaps(false), BaseMaterial(nullptr)
{	
	if (!IsTemplate())
	{
		BaseMaterial = GEngine->DefaultHLODFlattenMaterial;
	}
}

bool UHierarchicalLODSettings::IsValidFlattenMaterial(const UMaterialInterface* InBaseMaterial, bool bShowToaster)
{
	bool bIsValid = FMaterialUtilities::IsValidFlattenMaterial(InBaseMaterial);

#if WITH_EDITOR
	if (!bIsValid && bShowToaster)
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("MaterialName"), FText::FromString(InBaseMaterial->GetName()));
		FText ErrorMessage = FText::Format(LOCTEXT("UHierarchicalLODSettings_PostEditChangeProperty", "Material {MaterialName} is missing required Material Parameters (check log for details)"), Arguments);
		FNotificationInfo Info(ErrorMessage);
		Info.ExpireDuration = 5.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
	}
#endif

	return bIsValid;
}

void UHierarchicalLODSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UHierarchicalLODSettings, BaseMaterial))
	{
		if (!BaseMaterial.IsNull())
		{
			if (!IsValidFlattenMaterial(BaseMaterial.LoadSynchronous(), true))
			{
				BaseMaterial = GEngine->DefaultFlattenMaterial;
			}
		}
	}
	else if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UHierarchicalLODSettings, bSaveLODActorsToHLODPackages))
	{
		GEngine->GetEngineSubsystem<UHLODEngineSubsystem>()->OnSaveLODActorsToHLODPackagesChanged();
	}
}

FHierarchicalLODBuilder::FHierarchicalLODBuilder(UWorld* InWorld, bool bInPersistentLevelOnly /*= false*/)
	: World(InWorld)
	, bPersistentLevelOnly(bInPersistentLevelOnly)
{
	checkf(InWorld != nullptr, TEXT("Invalid nullptr world provided"));
	HLODSettings = GetDefault<UHierarchicalLODSettings>();
}

FHierarchicalLODBuilder::FHierarchicalLODBuilder()
	: World(nullptr)
	, bPersistentLevelOnly(false)
	, HLODSettings(nullptr)
{
	EnsureRetrievingVTablePtrDuringCtor(TEXT("FHierarchicalLODBuilder()"));
}

void FHierarchicalLODBuilder::Build()
{
	PreviewBuild();
	BuildMeshesForLODActors(true);
}

void FHierarchicalLODBuilder::PreviewBuild()
{
	if (ensure(World))
	{
		for (ULevel* Level : World->GetLevels())
		{
			if (ShouldBuildHLODForLevel(World, Level))
			{
				BuildClusters(Level);
			}
		}
	}
}

void FHierarchicalLODBuilder::BuildClusters(ULevel* InLevel)
{	
	TRACE_CPUPROFILER_EVENT_SCOPE(FHierarchicalLODBuilder::BuildClusters);

	SCOPE_LOG_TIME(TEXT("STAT_HLOD_BuildClusters"), nullptr);

	// This may execute pending construction scripts.
	FAssetCompilingManager::Get().ProcessAsyncTasks();

	const TArray<FHierarchicalSimplification>& BuildLODLevelSettings = InLevel->GetWorldSettings()->GetHierarchicalLODSetup();
	
	LODLevelLODActors.Empty();
	ValidStaticMeshActorsInLevel.Empty();
	HLODVolumeActors.Empty();
	RejectedActorsInLevel.Empty();

	// I'm using stack mem within this scope of the function
	// so we need this
	FMemMark Mark(FMemStack::Get());

	for (AActor* Actor : InLevel->Actors)
	{
		if (ALODActor* LODActor = Cast<ALODActor>(Actor))
		{
			OldLODActors.Add(LODActor);
		}
	}

	const int32 NumHLODLevels = BuildLODLevelSettings.Num();
	LODLevelLODActors.AddDefaulted(NumHLODLevels);

	// only build if it's enabled
	if (BuildLODLevelSettings.Num() > 0)
	{
		CreateTempLODActorLevel(InLevel);

		if (InLevel->GetWorldSettings()->bGenerateSingleClusterForLevel)
		{
			GenerateAsSingleCluster(NumHLODLevels, InLevel);
		}
		else
		{
			for (int32 LODId = 0; LODId < NumHLODLevels; ++LODId)
			{
				// Handle HierachicalLOD volumes first
				HandleHLODVolumes(InLevel, LODId);

				// Reuse clusters from previous HLOD level (only works for HLOD level 1 and beyond)
				if (BuildLODLevelSettings[LODId].bReusePreviousLevelClusters && LODId > 0)
				{
					for (ALODActor* PreviousLODActor : LODLevelLODActors[LODId - 1])
					{
						FLODCluster PreviousActorCluster(PreviousLODActor);

						// Reassess whether or not actors that were excluded from the previous HLOD level should be included in this one
						auto EvaluateRejectedActors = [this, &PreviousActorCluster, LODId](TFunctionRef<bool(const AActor*)> InPredicate)
						{
							for (auto It = RejectedActorsInLevel.CreateIterator(); It; ++It)
							{
								AActor* Actor = *It;
								if (!ShouldGenerateCluster(Actor, LODId - 1) && ShouldGenerateCluster(Actor, LODId))
								{
									if (InPredicate(Actor))
									{
										PreviousActorCluster += Actor;
										It.RemoveCurrent(); // Don't use it again later once it's in a cluster
									}
								}
							}
						};

						auto EvaluateValidActors = [this, &PreviousActorCluster, LODId](TFunctionRef<bool(const AActor*)> InPredicate)
						{
							for (auto It = ValidStaticMeshActorsInLevel.CreateIterator(); It; ++It)
							{
								AActor* Actor = *It;
								if (ShouldGenerateCluster(Actor, LODId))
								{
									if (InPredicate(Actor))
									{
										PreviousActorCluster += Actor;
										It.RemoveCurrent(); // Don't use it again later once it's in a cluster
									}
								}
							}
						};

						if (BuildLODLevelSettings[LODId].bOnlyGenerateClustersForVolumes)
						{
							AHierarchicalLODVolume** VolumePtr = HLODVolumeActors.Find(PreviousLODActor);
							if (VolumePtr && (*VolumePtr)->AppliesToHLODLevel(LODId))
							{
								AHierarchicalLODVolume* Volume = *VolumePtr;

								auto IsInVolume = [Volume](const AActor* Actor)
								{
									return Volume->IsActorIncluded(Actor);
								};

								EvaluateValidActors(IsInVolume);
								EvaluateRejectedActors(IsInVolume);
							}
						}
						else
						{
							const FBoxSphereBounds ClusterBounds(PreviousLODActor->GetComponentsBoundingBox(true));

							auto IsInCluster = [&ClusterBounds](const AActor* Actor)
							{
								return FBoxSphereBounds::SpheresIntersect(ClusterBounds, FSphere(Actor->GetActorLocation(), Actor->GetComponentsBoundingBox().GetSize().Size()));
							};

							EvaluateRejectedActors(IsInCluster);
						}

						ALODActor* LODActor = CreateLODActor(PreviousActorCluster, InLevel, LODId);
						LODActor->SetLODActorTag(PreviousLODActor->GetLODActorTag());
						LODLevelLODActors[LODId].Add(LODActor);
					}
				}
				else
				{
					// we use meter for bound. Otherwise it's very easy to get to overflow and have problem with filling ratio because
					// bound is too huge
					const float DesiredBoundRadius = BuildLODLevelSettings[LODId].DesiredBoundRadius * CM_TO_METER;
					const float DesiredFillingRatio = BuildLODLevelSettings[LODId].DesiredFillingPercentage * CM_TO_METER;
					ensure(DesiredFillingRatio != 0.f);
					const float HighestCost = FMath::Pow(DesiredBoundRadius, 3) / (DesiredFillingRatio);
					const int32 MinNumActors = BuildLODLevelSettings[LODId].MinNumberOfActorsToBuild;
					check(MinNumActors > 0);

					// since to show progress of initialization, I'm scoping it
					{
						FString LevelName = FPackageName::GetShortName(InLevel->GetOutermost()->GetName());
						FFormatNamedArguments Arguments;
						Arguments.Add(TEXT("LODIndex"), FText::AsNumber(LODId + 1));
						Arguments.Add(TEXT("LevelName"), FText::FromString(LevelName));

						FScopedSlowTask SlowTask(100, FText::Format(LOCTEXT("HierarchicalLOD_InitializeCluster", "Initializing Clusters for LOD {LODIndex} of {LevelName}..."), Arguments));
						SlowTask.MakeDialog();

						// initialize Clusters
						InitializeClusters(InLevel, LODId, HighestCost, BuildLODLevelSettings[LODId].bOnlyGenerateClustersForVolumes);

						// move a half way - I know we can do this better but as of now this is small progress
						SlowTask.EnterProgressFrame(50);

						// now we have all pair of nodes
						FindMST();
					}

					// now we have to calculate merge clusters and build actors
					MergeClustersAndBuildActors(InLevel, LODId, HighestCost, MinNumActors);
				}
			}
		}

		ApplyClusteringChanges(InLevel);
	}

	// Clear Clusters. It is using stack mem, so it won't be good after this
	Clusters.Empty();
	Clusters.Shrink();
}

void FHierarchicalLODBuilder::CreateTempLODActorLevel(ULevel* InLevel)
{
	TempLevel = NewObject<ULevel>(GetTransientPackage(), TEXT("TempLODActorLevel"));
	TempLevel->Initialize(FURL(nullptr));
	check(TempLevel);
	TempLevel->AddToRoot();
	TempLevel->OwningWorld = InLevel->GetWorld();
	TempLevel->Model = NewObject<UModel>(TempLevel);
	TempLevel->Model->Initialize(nullptr, true);
	TempLevel->bIsVisible = true;

	TempLevel->SetFlags(RF_Transactional);
	TempLevel->Model->SetFlags(RF_Transactional);
}

// Hash the clusters using the LODLevel & subactors pointers
// Take into account child ALODActors too.
int32 HashLODActorForClusterComparison(ALODActor* LODActor)
{
	uint32 HashValue = 0;

	HashValue = HashCombine(HashValue, LODActor->LODLevel);
	HashValue = HashCombine(HashValue, LODActor->SubActors.Num());
	HashValue = HashCombine(HashValue, GetTypeHash(LODActor->GetLODActorTag()));

	TArray<AActor*> Actors = LODActor->SubActors;
	Actors.Sort();

	TArray<ALODActor*> ChildLODActors;

	for (AActor* Actor : Actors)
	{
		if (ALODActor* ChildLODActor = Cast<ALODActor>(Actor))
		{
			ChildLODActors.Add(ChildLODActor);
		}
		else
		{
			HashValue = HashCombine(HashValue, GetTypeHash(Actor));
		}
	}

	TArray<int32> ChildLODActorsHashes;
	Algo::Transform(ChildLODActors, ChildLODActorsHashes, [](ALODActor* ChildLODActor) { return HashLODActorForClusterComparison(ChildLODActor); });
	ChildLODActorsHashes.Sort();

	for (int32 ChildLODActorsHash : ChildLODActorsHashes)
	{
		HashValue = HashCombine(HashValue, ChildLODActorsHash);
	}

	return HashValue;
}

void FHierarchicalLODBuilder::ApplyClusteringChanges(ULevel* InLevel)
{
	bool bSaveLODActorsToHLODPackages = GetDefault<UHierarchicalLODSettings>()->bSaveLODActorsToHLODPackages;

	// Check if the level must be resaved
	bool bLevelChanged = false;
	for (ALODActor* LODActor : OldLODActors)
	{
		// If the config was changed, we must resave the level
		if (LODActor->WasBuiltFromHLODDesc() != bSaveLODActorsToHLODPackages)
		{
			bLevelChanged = true;
			break;
		}

		// If actors spawned from the HLODPackage aren't transients, we must resave the level
		if (LODActor->WasBuiltFromHLODDesc() && !LODActor->HasAllFlags(EObjectFlags::RF_Transient | EObjectFlags::RF_DuplicateTransient))
		{
			bLevelChanged = true;
			break;
		}
	}

	// Compare the LOD actors we spawned against those in the level
	bool bActorChanged = bLevelChanged || OldLODActors.Num() != NewLODActors.Num();
	if (!bActorChanged)
	{
		TSet<int32> HashedLODActors;
		Algo::Transform(OldLODActors, HashedLODActors, [](ALODActor* LODActor) { return HashLODActorForClusterComparison(LODActor); });

		for (ALODActor* LODActor : NewLODActors)
		{
			int32 Hash = HashLODActorForClusterComparison(LODActor);
			if (!HashedLODActors.Contains(Hash))
			{
				bActorChanged = true;
				break;
			}
		}
	}

	// If clusters changed, delete old LOD actors and move the new ones in the proper level
	if (bActorChanged)
	{
		// Delete all 
		DeleteLODActors(InLevel);

		FHierarchicalLODUtilitiesModule& Module = FModuleManager::LoadModuleChecked<FHierarchicalLODUtilitiesModule>("HierarchicalLODUtilities");
		IHierarchicalLODUtilities* Utilities = Module.GetUtilities();

		for (ALODActor* LODActor : NewLODActors)
		{
			// Move the LOD actor from the temp level to the proper level
			LODActor->Rename(nullptr, InLevel, REN_DoNotDirty);

			// Ensure the new LODActor use it's own package if the level is setup to use external actors.
			if (InLevel->IsUsingExternalActors())
			{
				LODActor->SetPackageExternal(true, false);
			}

			LODActor->MarkPackageDirty();

			// Reinsert actors properly in the LODActor subactors array
			// Will also setup LODParentPrimitive for each actors primitive components.
			TArray<AActor*> SubActors = LODActor->SubActors;
			LODActor->SubActors.Empty();
			for (AActor* Actor : SubActors)
			{
				LODActor->AddSubActor(Actor);
			}

			if (bSaveLODActorsToHLODPackages)
			{
				UHLODProxy* Proxy = Utilities->CreateOrRetrieveLevelHLODProxy(InLevel, LODActor->LODLevel - 1);
				Proxy->AddLODActor(LODActor);
			}
		}

		// If the level must be resaved, mark its package dirty
		if (bLevelChanged)
		{
			InLevel->MarkPackageDirty();
		}
	}

	// Delete the temporary level
	TempLevel->ClearLevelComponents();
	InLevel->GetWorld()->RemoveLevel(TempLevel);
	TempLevel->OwningWorld = NULL;
	TempLevel->RemoveFromRoot();
	TempLevel = nullptr;

	OldLODActors.Empty();
	NewLODActors.Empty();
}

void FHierarchicalLODBuilder::GenerateAsSingleCluster(const int32 NumHLODLevels, ULevel* InLevel)
{
	Clusters.Empty();

	ALODActor* PreviousLevelActor = nullptr;
	TArray<AActor*> GenerationActors;
	for (int32 LODId = 0; LODId < NumHLODLevels; ++LODId)
	{
		FLODCluster LevelCluster;

		if (PreviousLevelActor == nullptr)
		{
			for (int32 ActorId = 0; ActorId < InLevel->Actors.Num(); ++ActorId)
			{
				AActor* Actor = InLevel->Actors[ActorId];
				if (ShouldGenerateCluster(Actor, LODId))
				{
					FLODCluster ActorCluster(Actor);
					ValidStaticMeshActorsInLevel.Add(Actor);

					LevelCluster += ActorCluster;
				}
				else
				{
					GenerationActors.Add(Actor);
				}
			}
		}
		else
		{
			LevelCluster += PreviousLevelActor;
			// Make sure we take into account previously excluded actors, could be caused by specifically disabled inclusion in previos HLOD level(s)
			for (int32 ActorIndex = 0; ActorIndex < GenerationActors.Num(); ++ActorIndex)
			{
				AActor* Actor = GenerationActors[ActorIndex];
				if (ShouldGenerateCluster(Actor, LODId))
				{
					FLODCluster ActorCluster(Actor);
					ValidStaticMeshActorsInLevel.Add(Actor);
					LevelCluster += ActorCluster;
					GenerationActors.Remove(Actor);
					--ActorIndex;
				}
			}
		}

		if (LevelCluster.IsValid())
		{
			ALODActor* LODActor = CreateLODActor(LevelCluster, InLevel, LODId);
			LODActor->SetLODActorTag("SingleCluster");
			PreviousLevelActor = LODActor;
		}
	}
}

void FHierarchicalLODBuilder::InitializeClusters(ULevel* InLevel, const int32 LODIdx, float CullCost, bool const bVolumesOnly)
{
	SCOPE_LOG_TIME(TEXT("STAT_HLOD_InitializeClusters"), nullptr);

	// Check whether or not this actor falls within a HierarchicalLODVolume, if so add to the Volume's cluster and exclude from normal process
	auto ProcessVolumeClusters = [this, LODIdx](AActor* InActor) -> bool
	{
		FBox ActorBox = InActor->GetComponentsBoundingBox(true);
		for (TPair<AHierarchicalLODVolume*, FLODCluster>& Cluster : HLODVolumeClusters)
		{
			if (Cluster.Key->IsActorIncluded(InActor))
			{
				Cluster.Value += InActor;
				return true;
			}
		}

		return false;
	};

	// Actors are either handled by a volume, valid or rejected
	auto FilterActors = [this, LODIdx, bVolumesOnly, ProcessVolumeClusters](const TArray<AActor*>& InActors)
	{
		ValidStaticMeshActorsInLevel.Reset();
		RejectedActorsInLevel.Reset();

		for (int32 ActorId = 0; ActorId < InActors.Num(); ++ActorId)
		{
			AActor* Actor = InActors[ActorId];
			const bool bShouldGenerate = ShouldGenerateCluster(Actor, LODIdx);
			if (bShouldGenerate)
			{
				if (!ProcessVolumeClusters(Actor))
				{
					if (bVolumesOnly)
					{
						// Add them to the RejectedActorsInLevel to be re-considered at the next LOD in case that one isn't using bVolumesOnly
						RejectedActorsInLevel.Add(Actor);
					}
					else
					{
						ValidStaticMeshActorsInLevel.Add(Actor);
					}
				}
			}
			else if(Actor)
			{
				RejectedActorsInLevel.Add(Actor);
			}
		}
	};

	// Create clusters from actor pairs
	auto CreateClusters = [this, CullCost, bVolumesOnly](const TArray<AActor*>& InActors)
	{
		Clusters.Reset();

		if (!bVolumesOnly)
		{
			const int32 NumActors = InActors.Num();
			if (NumActors == 1)
			{
				// Only one actor means a simple one-to-one relationship
				Clusters.Add(FLODCluster(InActors[0]));
			}
			else
			{
				// Create clusters using actor pairs
				for (int32 ActorId = 0; ActorId < InActors.Num(); ++ActorId)
				{
					AActor* Actor1 = InActors[ActorId];

					for (int32 SubActorId = ActorId + 1; SubActorId < InActors.Num(); ++SubActorId)
					{
						AActor* Actor2 = InActors[SubActorId];

						FLODCluster NewClusterCandidate = FLODCluster(Actor1, Actor2);
						double NewClusterCost = NewClusterCandidate.GetCost();

						if (NewClusterCost <= CullCost)
						{
							Clusters.Add(MoveTemp(NewClusterCandidate));
						}
					}
				}
			}
		}
	};

	if (LODIdx == 0)
	{
		FilterActors(ObjectPtrDecay(InLevel->Actors));
		CreateClusters(ValidStaticMeshActorsInLevel);
	}
	else
	{
		// Filter actors
		TArray<AActor*> Actors;
		Actors.Reset(ValidStaticMeshActorsInLevel.Num() + RejectedActorsInLevel.Num());
		Actors.Append(ValidStaticMeshActorsInLevel);
		Actors.Append(RejectedActorsInLevel);
		FilterActors(Actors);

		// Create clusters, taking previous level LODActor into account.
		Actors.Reset();
		Actors.Append(LODLevelLODActors[LODIdx - 1]);
		Actors.RemoveAll(ProcessVolumeClusters);
		Actors.Append(ValidStaticMeshActorsInLevel);
		CreateClusters(Actors);
	}
}

void FHierarchicalLODBuilder::FindMST() 
{
	SCOPE_LOG_TIME(TEXT("STAT_HLOD_FindMST"), nullptr);
	if (Clusters.Num() > 0)
	{
		// now sort edge in the order of weight
		struct FCompareCluster
		{
			FORCEINLINE bool operator()(const FLODCluster& A, const FLODCluster& B) const
			{
				return (A.GetCost() < B.GetCost());
			}
		};

		Clusters.HeapSort(FCompareCluster());
	}
}

void FHierarchicalLODBuilder::HandleHLODVolumes(ULevel* InLevel, int32 LODIdx)
{	
	HLODVolumeClusters.Reset();

	for (AActor* Actor : InLevel->Actors)
	{
		if (AHierarchicalLODVolume* VolumeActor = Cast<AHierarchicalLODVolume>(Actor))
		{
			if (VolumeActor->AppliesToHLODLevel(LODIdx))
			{
				// Came across a HLOD volume
				FLODCluster& NewCluster = HLODVolumeClusters.Add(VolumeActor);

				FVector Origin, Extent;
				VolumeActor->GetActorBounds(false, Origin, Extent);
				NewCluster.Bound = FSphere(Origin * CM_TO_METER, Extent.Size() * CM_TO_METER);

				// calculate new filling factor
				NewCluster.FillingFactor = 1.f;
				NewCluster.ClusterCost = FMath::Pow(NewCluster.Bound.W, 3) / NewCluster.FillingFactor;
			}
		}
	}
}

bool FHierarchicalLODBuilder::ShouldBuildHLODForLevel(const UWorld* InWorld, const ULevel* InLevel) const
{
	check(InWorld);
	if (!InLevel)
	{
		return false;
	}

	// If we only want to build HLODs for the persistent level
	if (bPersistentLevelOnly && InLevel != InWorld->PersistentLevel)
	{
		return false;
	}

	ULevelStreaming* const* SourceLevelStreamingPtr = InWorld->GetStreamingLevels().FindByPredicate([InLevel](ULevelStreaming* LS) { return LS && LS->GetLoadedLevel() == InLevel; });
	ULevelStreaming* SourceLevelStreaming = SourceLevelStreamingPtr ? *SourceLevelStreamingPtr : nullptr;
	if (SourceLevelStreaming && SourceLevelStreaming->HasAnyFlags(RF_Transient))
	{
		// Skip over levels from transient ULevelStreamings. These are levels that are not saved in the map and should not contribute to the HLOD
		return false;
	}

	return true;
}

bool FHierarchicalLODBuilder::ShouldGenerateCluster(AActor* Actor, const int32 HLODLevelIndex)
{
	if (!IsValid(Actor))
	{
		return false;
	}

	if (!Actor->IsHLODRelevant())
	{
		return false;
	}

	if (ALODActor* LODActor = Cast<ALODActor>(Actor))
	{
		// Ignore previous LOD actors
		if (OldLODActors.Contains(LODActor))
		{
			return false;
		}

		// Should never happen... newly created LOD actors haven't been assigned a static mesh yet
		if (LODActor->GetStaticMeshComponent()->GetStaticMesh())
		{
			return false;
		}
	}

	TArray<UStaticMeshComponent*> Components;
	Actor->GetComponents(Components);

	bool bHasValidComponent = false;
	for (UStaticMeshComponent* Component : Components)
	{
		// see if we should generate it
		if (Component->ShouldGenerateAutoLOD(HLODLevelIndex))
		{
			bHasValidComponent = true;
			break;
		}
	}

	return bHasValidComponent;
}

void FHierarchicalLODBuilder::ClearHLODs()
{
	if (ensure(World))
	{
		for (ULevel* Level : World->GetLevels())
		{
			if (ShouldBuildHLODForLevel(World, Level))
			{
				DeleteLODActors(Level);
			}
		}
	}
}

void FHierarchicalLODBuilder::ClearPreviewBuild()
{
	ClearHLODs();
}

void FHierarchicalLODBuilder::BuildMeshesForLODActors(bool bForceAll)
{	
	// Finalize asset compilation before we potentially uses them during the HLOD generation.
	FAssetCompilingManager::Get().FinishAllCompilation();

	// This may execute pending construction scripts.
	FAssetCompilingManager::Get().ProcessAsyncTasks();

	bool bVisibleLevelsWarning = false;

	const TArray<ULevel*>& Levels = World->GetLevels();
	for (ULevel* LevelIter : Levels)
	{
		if (!ShouldBuildHLODForLevel(World, LevelIter))
		{
			continue;
		}

		FScopedSlowTask SlowTask(105, (LOCTEXT("HierarchicalLOD_BuildLODActorMeshes", "Building LODActor meshes")));
		const bool bShowCancelButton = true;
		SlowTask.MakeDialog(bShowCancelButton);

		const TArray<FHierarchicalSimplification>& BuildLODLevelSettings = LevelIter->GetWorldSettings()->GetHierarchicalLODSetup();
		UMaterialInterface* BaseMaterial = LevelIter->GetWorldSettings()->GetHierarchicalLODBaseMaterial();
		TArray<TArray<ALODActor*>> LODLevelActors;
		LODLevelActors.AddDefaulted(BuildLODLevelSettings.Num());

		if (LevelIter->Actors.Num() > 0)
		{
			FHierarchicalLODUtilitiesModule& Module = FModuleManager::LoadModuleChecked<FHierarchicalLODUtilitiesModule>("HierarchicalLODUtilities");
			IHierarchicalLODUtilities* Utilities = Module.GetUtilities();

			// Retrieve LOD actors from the level
			uint32 NumLODActors = 0;
			for (AActor* Actor : LevelIter->Actors)
			{
				if (ALODActor* LODActor = Cast<ALODActor>(Actor))
				{
					// Ensure the LODActor we found is valid for our current HLOD build settings.
					if (LODActor->LODLevel - 1 >= LODLevelActors.Num())
					{
						FMessageLog("HLODResults").Error()
							->AddToken(FTextToken::Create(LOCTEXT("HLODError_ClusterRebuildNeeded", "Invalid LODActor found (invalid LOD level with regards to the current HLOD build settings). You must rebuild the HLOD clusters!")))
							->AddToken(FUObjectToken::Create(LODActor));
						continue;
					}

					if (bForceAll || (!LODActor->IsBuilt(true) && LODActor->HasValidSubActors()))
					{
						// Dirty actors that werent already if we are forcing
						if(bForceAll && LODActor->IsBuilt())
						{
							LODActor->ForceUnbuilt();
						}
						LODLevelActors[LODActor->LODLevel - 1].Add(LODActor);
						NumLODActors++;
					}
				}
			}

			// If there are any available process them
			if (NumLODActors)
			{
				const FTransform& HLODBakingTransform = LevelIter->GetWorldSettings()->HLODBakingTransform;
				bool bUseCustomTransformForHLODBaking = !HLODBakingTransform.Equals(FTransform::Identity);

				// Apply the HLOD transform prior to baking
				if (bUseCustomTransformForHLODBaking)
				{
					FLevelUtils::FApplyLevelTransformParams TransformParams(LevelIter, HLODBakingTransform);
					TransformParams.bDoPostEditMove = false;
					FLevelUtils::ApplyLevelTransform(TransformParams);
				}

				// Only create the outer package if we are going to save something to it (otherwise we end up with an empty HLOD folder)
				const int32 NumLODLevels = LODLevelActors.Num();

				if(NumLODLevels > 0)
				{
					UE_LOGF(LogLODGenerator, Log, "Building HLOD meshes for %ls", *LevelIter->GetOutermost()->GetName());
				}

				for (int32 LODIndex = 0; LODIndex < NumLODLevels; ++LODIndex)
				{
					TArray<ALODActor*>& LODLevel = LODLevelActors[LODIndex];

					if (LODLevel.Num() > 0)
					{
						UHLODProxy* Proxy = Utilities->CreateOrRetrieveLevelHLODProxy(LevelIter, LODIndex);

						UPackage* AssetsOuter = Proxy->GetOutermost();
						checkf(AssetsOuter != nullptr, TEXT("Failed to created outer for generated HLOD assets"));
						if (AssetsOuter)
						{
							int32 LODActorIndex = 0;
							for (ALODActor* Actor : LODLevel)
							{
								SlowTask.EnterProgressFrame(100.0f / (float)NumLODActors, FText::Format(LOCTEXT("HierarchicalLOD_BuildLODActorMeshesProgress", "Building LODActor Mesh {0} of {1} (LOD Level {2})"), FText::AsNumber(LODActorIndex), FText::AsNumber(LODLevelActors[LODIndex].Num()), FText::AsNumber(LODIndex + 1)));

								bool bBuildSuccessful = Utilities->BuildStaticMeshForLODActor(Actor, Proxy, BuildLODLevelSettings[LODIndex], BaseMaterial);

								// Report an error if the build failed
								if (!bBuildSuccessful)
								{
									FMessageLog("HLODResults").Error()
										->AddToken(FTextToken::Create(LOCTEXT("HLODError_MeshNotBuildOne", "Cannot create proxy mesh for ")))
										->AddToken(FUObjectToken::Create(Actor))
										->AddToken(FTextToken::Create(LOCTEXT("HLODError_MeshNotBuildTwo", " this could be caused by incorrect mesh components in the sub actors")));
								}
								else
								{
									AssetsOuter->Modify();
								}

								++LODActorIndex;

								if (SlowTask.ShouldCancel())
									break;
							}
						}
					}
					else
					{
						// No HLODs were generated for this HLOD level, ensure the proxy is cleaned and that the associated package is deleted
						UHLODProxy* Proxy = Utilities->RetrieveLevelHLODProxy(LevelIter, LODIndex);
						if (Proxy)
						{
							Proxy->Clean();
						}
					}

					if (SlowTask.ShouldCancel())
						break;
				}

				// Ensure HLOD proxy generation has completed
				FHierarchicalLODProxyProcessor* Processor = Module.GetProxyProcessor();
				while (Processor->IsProxyGenerationRunning())
				{
					FTSTicker::GetCoreTicker().Tick(static_cast<float>(FApp::GetDeltaTime()));
					FThreadManager::Get().Tick();
					FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
					FPlatformProcess::Sleep(0.1f);
				}

				if (bUseCustomTransformForHLODBaking)
				{
					const FTransform TransformInv = HLODBakingTransform.Inverse();

					FLevelUtils::FApplyLevelTransformParams TransformParams(LevelIter, TransformInv);
					TransformParams.bDoPostEditMove = false;

					// Undo HLOD transform that was performed prior to baking
					FLevelUtils::ApplyLevelTransform(TransformParams);

					for (int32 LODIndex = 0; LODIndex < NumLODLevels; ++LODIndex)
					{
						const FHierarchicalSimplification& LODLevelSettings = BuildLODLevelSettings[LODIndex];

						for (ALODActor* LODActor : LODLevelActors[LODIndex])
						{
							UStaticMesh* StaticMesh = LODActor->GetStaticMeshComponent() ? ToRawPtr(LODActor->GetStaticMeshComponent()->GetStaticMesh()) : nullptr;
							if (StaticMesh == nullptr)
							{
								continue;
							}

							FMeshDescription* SMDesc = StaticMesh->GetMeshDescription(0);

							if (LODLevelSettings.SimplificationMethod != EHierarchicalSimplificationMethod::Merge)
							{
								LODActor->SetActorTransform(FTransform::Identity);
								FStaticMeshOperations::ApplyTransform(*SMDesc, TransformInv);
							}
							else
							{
								switch (LODLevelSettings.MergeSetting.PivotType)
								{
								case EMeshMergePivotType::WorldOrigin:
									LODActor->SetActorTransform(FTransform::Identity);
									FStaticMeshOperations::ApplyTransform(*SMDesc, TransformInv);
									break;

								case EMeshMergePivotType::Custom:
									LODActor->SetActorTransform(FTransform(LODLevelSettings.MergeSetting.CustomPivotLocation));
									FStaticMeshOperations::ApplyTransform(*SMDesc, TransformInv);
									break;

								case EMeshMergePivotType::Automatic:
									FStaticMeshOperations::ApplyTransform(*SMDesc, FTransform(TransformInv.GetRotation()));
									break;

								default:
									checkNoEntry();
								}
							}

							StaticMesh->CommitMeshDescription(0);
							StaticMesh->PostEditChange();

							// Update key since positions have changed
							LODActor->GetProxy()->AddMesh(LODActor, StaticMesh, UHLODProxy::GenerateKeyForActor(LODActor));
						}
					}
				}
			}
		}
	}
}

void FHierarchicalLODBuilder::DeleteEmptyHLODPackages(ULevel* InLevel)
{
	// Do not process HLOD packages when dealing with streamed levels.
	if (!ShouldBuildHLODForLevel(InLevel->GetWorld(), InLevel))
	{
		return;
	}

	FHierarchicalLODUtilitiesModule& Module = FModuleManager::LoadModuleChecked<FHierarchicalLODUtilitiesModule>("HierarchicalLODUtilities");
	IHierarchicalLODUtilities* Utilities = Module.GetUtilities();

	// Look for HLODProxy packages associated with this level
	int32 NumLODLevels = InLevel->GetWorldSettings()->GetHierarchicalLODSetup().Num();
	for (int32 LODIndex = 0; LODIndex < NumLODLevels; ++LODIndex)
	{
		// Obtain HLOD package for the current HLOD level
		UHLODProxy* HLODProxy = Utilities->RetrieveLevelHLODProxy(InLevel, LODIndex);
		if (HLODProxy)
		{
			HLODProxy->Clean();

			// If this proxy is empty, we can delete the package
			if (HLODProxy->IsEmpty())
			{
				HLODProxy->DeletePackage();
			}
		}
	}
}

void FHierarchicalLODBuilder::GetMeshesPackagesToSave(ULevel* InLevel, TSet<UPackage*>& InHLODPackagesToSave, const FString& PreviousLevelName /*= ""*/)
{
	// Do not process HLOD packages when dealing with streamed levels.
	if (!ShouldBuildHLODForLevel(InLevel->GetWorld(), InLevel))
	{
		return;
	}

	const TArray<FHierarchicalSimplification>& BuildLODLevelSettings = InLevel->GetWorldSettings()->GetHierarchicalLODSetup();
	UMaterialInterface* BaseMaterial = InLevel->GetWorldSettings()->GetHierarchicalLODBaseMaterial();
	TArray<TArray<ALODActor*>> LODLevelActors;
	LODLevelActors.AddDefaulted(BuildLODLevelSettings.Num());

	if (InLevel->Actors.Num() > 0)
	{
		FHierarchicalLODUtilitiesModule& Module = FModuleManager::LoadModuleChecked<FHierarchicalLODUtilitiesModule>("HierarchicalLODUtilities");
		IHierarchicalLODUtilities* Utilities = Module.GetUtilities();

		// Retrieve LOD actors from the level
		for (int32 ActorId = 0; ActorId < InLevel->Actors.Num(); ++ActorId)
		{
			AActor* Actor = InLevel->Actors[ActorId];
			if (Actor && Actor->IsA<ALODActor>())
			{
				ALODActor* LODActor = CastChecked<ALODActor>(Actor);

				LODLevelActors[LODActor->LODLevel - 1].Add(LODActor);
			}
		}

		const int32 NumLODLevels = LODLevelActors.Num();
		for (int32 LODIndex = 0; LODIndex < NumLODLevels; ++LODIndex)
		{
			UHLODProxy* HLODProxy = Utilities->RetrieveLevelHLODProxy(InLevel, LODIndex);
			if (HLODProxy)
			{
				// Ensure the HLOD descs are up to date.
				HLODProxy->Clean();

				// Add the HLODProxy package to the list of packages to save
				InHLODPackagesToSave.Add(HLODProxy->GetOutermost());
			}
			// If we couldn't find the HLOD package, the level may have been renamed, 
			// so we need to relocate our old HLOD package before saving it.
			else if (PreviousLevelName.Len() != 0)
			{
				FString NewLevelName = InLevel->GetOutermost()->GetName();

				FString OldHLODProxyName = Utilities->GetLevelHLODProxyName(PreviousLevelName, LODIndex);
				UHLODProxy* OldHLODPProxy = FindObject<UHLODProxy>(nullptr, *OldHLODProxyName);
				if (OldHLODPProxy)
				{
					FString NewHLODProxyName = Utilities->GetLevelHLODProxyName(NewLevelName, LODIndex);
					FString OldHLODPackageName = FPackageName::ObjectPathToPackageName(OldHLODProxyName);
					FString NewHLODPackageName = FPackageName::ObjectPathToPackageName(NewHLODProxyName);
					UPackage* OldHLODPackage = FindObject<UPackage>(nullptr, *OldHLODPackageName);
					if (OldHLODPackage)
					{
						OldHLODPProxy->Rename(*FPackageName::ObjectPathToObjectName(NewHLODProxyName), OldHLODPackage, REN_NonTransactional | REN_DontCreateRedirectors);
						OldHLODPackage->Rename(*NewHLODPackageName, nullptr, REN_NonTransactional | REN_DontCreateRedirectors);

						InHLODPackagesToSave.Add(OldHLODPackage);

						// Mark the level package as dirty as we have changed export locations, and without a resave we will not pick up
						// HLOD packages when reloaded.
						InLevel->GetOutermost()->MarkPackageDirty();
					}
				}
			}

			// We might have created imposters static mesh packages during the HLOD generation, we must save them too.
			for (ALODActor* LODActor : LODLevelActors[LODIndex])
			{
				for (UInstancedStaticMeshComponent* Component : LODActor->GetInstancedStaticMeshComponents())
				{
					if (Component->GetStaticMesh())
					{
						InHLODPackagesToSave.Add(Component->GetStaticMesh()->GetOutermost());
					}
				}
			}
		}
	}
}

void FHierarchicalLODBuilder::SaveMeshesForActors()
{
	TArray<UPackage*> LevelPackagesToSave;
	TArray<FString> OldLevelPackageNames;

	bool bUnsavedLevel = false;
	const TArray<ULevel*>& Levels = World->GetLevels();
	for (const ULevel* Level : Levels)
	{
		// Levels might also need a resave, or levels might not have been saved yet
		LevelPackagesToSave.Add(Level->GetOutermost());
		OldLevelPackageNames.Add(Level->GetOutermost()->GetName());
		bUnsavedLevel |= Level->GetOutermost()->GetName().StartsWith("/Temp/");
	}

	bool bSuccess = true;

	// Save levels first if they are in the /Temp/ mount point
	if(bUnsavedLevel)
	{
		bSuccess = UEditorLoadingAndSavingUtils::SavePackagesWithDialog(LevelPackagesToSave, true);
	}
	
	if(bSuccess)
	{
		check(LevelPackagesToSave.Num() == OldLevelPackageNames.Num() && LevelPackagesToSave.Num() == Levels.Num());

		TSet<UPackage*> HLODPackagesToSave;
		for(int32 PackageIndex = 0; PackageIndex < LevelPackagesToSave.Num(); ++PackageIndex)
		{
			FString PreviousLevelName; 
			bool bLevelRenamed = bUnsavedLevel && LevelPackagesToSave[PackageIndex]->GetName() != OldLevelPackageNames[PackageIndex];
			if (bLevelRenamed)
			{
				PreviousLevelName = OldLevelPackageNames[PackageIndex];
			}

			ULevel* Level = Levels[PackageIndex];
			HLODPackagesToSave.Add(Level->GetOutermost());
			GetMeshesPackagesToSave(Level, HLODPackagesToSave, PreviousLevelName);
		}

		TArray<UPackage*> PackagesToSave = HLODPackagesToSave.Array();
		UEditorLoadingAndSavingUtils::SavePackagesWithDialog(PackagesToSave, true);
	}
}

bool FHierarchicalLODBuilder::NeedsBuild(bool bInForce) const
{
	if(World)
	{
		for (TActorIterator<ALODActor> HLODIt(World); HLODIt; ++HLODIt)
		{
			if (!HLODIt->IsBuilt(bInForce))
			{
				return true;
			}
		}
	}

	return false;
}

void FHierarchicalLODBuilder::DeleteLODActors(ULevel* InLevel)
{
	FHierarchicalLODUtilitiesModule& Module = FModuleManager::LoadModuleChecked<FHierarchicalLODUtilitiesModule>("HierarchicalLODUtilities");
	IHierarchicalLODUtilities* Utilities = Module.GetUtilities();

	// you still have to delete all objects just in case they had it and didn't want it anymore
	for (int32 ActorId = InLevel->Actors.Num() - 1; ActorId >= 0; --ActorId)
	{
		ALODActor* LodActor = Cast<ALODActor>(InLevel->Actors[ActorId]);
		if (LodActor)
		{
			Utilities->DestroyLODActor(LodActor);
		}
	}
}

void FHierarchicalLODBuilder::BuildMeshForLODActor(ALODActor* LODActor, const uint32 LODLevel)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FHierarchicalLODBuilder::BuildMeshForLODActor);

	const TArray<FHierarchicalSimplification>& BuildLODLevelSettings = LODActor->GetLevel()->GetWorldSettings()->GetHierarchicalLODSetup();
	UMaterialInterface* BaseMaterial = LODActor->GetLevel()->GetWorldSettings()->GetHierarchicalLODBaseMaterial();
	
	FHierarchicalLODUtilitiesModule& Module = FModuleManager::LoadModuleChecked<FHierarchicalLODUtilitiesModule>("HierarchicalLODUtilities");
	IHierarchicalLODUtilities* Utilities = Module.GetUtilities();

	UHLODProxy* Proxy = Utilities->CreateOrRetrieveLevelHLODProxy(LODActor->GetLevel(), LODLevel);
	const bool bResult = Utilities->BuildStaticMeshForLODActor(LODActor, Proxy, BuildLODLevelSettings[LODLevel], BaseMaterial);

	if (bResult == false)
	{
		FMessageLog("HLODResults").Error()
			->AddToken(FTextToken::Create(LOCTEXT("HLODError_MeshNotBuildOne", "Cannot create proxy mesh for ")))
			->AddToken(FUObjectToken::Create(LODActor))
			->AddToken(FTextToken::Create(LOCTEXT("HLODError_MeshNotBuildTwo", " this could be caused by incorrect mesh components in the sub actors")));
	}
}

void FHierarchicalLODBuilder::MergeClustersAndBuildActors(ULevel* InLevel, const int32 LODIdx, float HighestCost, int32 MinNumActors)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FHierarchicalLODBuilder::MergeClustersAndBuildActors);

	if (Clusters.Num() > 0 || HLODVolumeClusters.Num() > 0)
	{
		FString LevelName = FPackageName::GetShortName(InLevel->GetOutermost()->GetName());
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("LODIndex"), FText::AsNumber(LODIdx + 1));
		Arguments.Add(TEXT("LevelName"), FText::FromString(LevelName));
		// merge clusters first
		{
			SCOPE_LOG_TIME(TEXT("HLOD_MergeClusters"), nullptr);
			bool bStable = false;

			while (!bStable)
			{
				const int32 NumClusters = Clusters.Num();

				FScopedSlowTask SlowTask(static_cast<float>(NumClusters), FText::Format(LOCTEXT("HierarchicalLOD_BuildClusters", "Building Clusters for LOD {LODIndex} of {LevelName}..."), Arguments));
				SlowTask.MakeDialog();

				TArray<FLODCluster> ValidMergedClusters;
				ValidMergedClusters.Reserve(NumClusters);

				bStable = true;

				// now we have minimum Clusters
				for (int32 ClusterId = 0; ClusterId < NumClusters; ++ClusterId)
				{
					SlowTask.EnterProgressFrame(1.0f);

					FLODCluster& Cluster = Clusters[ClusterId];
					UE_LOGF(LogLODGenerator, Verbose, "%d. %0.2f {%ls}", ClusterId + 1, Cluster.GetCost(), *Cluster.ToString());

					if (Cluster.IsValid())
					{
						// compare with previous valid clusters
						for (FLODCluster& MergedCluster : ValidMergedClusters)
						{
							check(MergedCluster.IsValid())

							// if valid, see if it contains any of this actors
							if (MergedCluster.Contains(Cluster))
							{
								double MergeCost = MergedCluster.GetMergedCost(Cluster);

								// merge two clusters
								if (MergeCost <= HighestCost)
								{
									MergedCluster += Cluster;
									// now this cluster is invalid
									Cluster.Invalidate();

									bStable = false;
									break;
								}
								else
								{
									Cluster -= MergedCluster;
									bStable = false;

									if (!Cluster.IsValid())
									{
										// If the cluster becomes invalid, MergedCluster.Contains() will always return false, so exit immediately
										break;
									}
								}
							}
						}

						if (Cluster.IsValid())
						{
							ValidMergedClusters.Add(Cluster);
						}

						UE_LOGF(LogLODGenerator, Verbose, "Processed(%ls): %0.2f {%ls}", Cluster.IsValid() ? TEXT("Valid") : TEXT("Invalid"), Cluster.GetCost(), *Cluster.ToString());
					}
				}

				Clusters = ValidMergedClusters;
			}
		}

		for (TPair<AHierarchicalLODVolume*, FLODCluster>& Cluster : HLODVolumeClusters)
		{
			Clusters.Add(MoveTemp(Cluster.Value));
		}


		{
			SCOPE_LOG_TIME(TEXT("HLOD_BuildActors"), nullptr);
			// print data
			int32 TotalValidCluster = 0;
			for (FLODCluster& Cluster : Clusters)
			{
				if (Cluster.IsValid())
				{
					++TotalValidCluster;
				}
			}

			FScopedSlowTask SlowTask(static_cast<float>(TotalValidCluster), FText::Format(LOCTEXT("HierarchicalLOD_MergeActors", "Merging Actors for LOD {LODIndex} of {LevelName}..."), Arguments));
			SlowTask.MakeDialog();

			for (FLODCluster& Cluster : Clusters)
			{
				if (Cluster.IsValid())
				{
					SlowTask.EnterProgressFrame();

					if (Cluster.Actors.Num() >= MinNumActors)
					{
						ALODActor* LODActor = CreateLODActor(Cluster, InLevel, LODIdx);
						if (LODActor)
						{
							LODLevelLODActors[LODIdx].Add(LODActor);

							if (AHierarchicalLODVolume* const* Volume = HLODVolumeClusters.FindKey(Cluster))
							{
								HLODVolumeActors.Add(LODActor, *Volume);
								LODActor->SetLODActorTag((*Volume)->GetName());
							}
						}

						for (AActor* RemoveActor : Cluster.Actors)
						{
							ValidStaticMeshActorsInLevel.RemoveSingleSwap(RemoveActor, EAllowShrinking::No);
							RejectedActorsInLevel.RemoveSingleSwap(RemoveActor, EAllowShrinking::No);
						}
					}
					else
					{
						// Add them to the RejectedActorsInLevel to be re-considered at the next LOD
						ValidStaticMeshActorsInLevel.RemoveAllSwap([&Cluster](const AActor* Actor) { return Cluster.Actors.Contains(Actor); });
						RejectedActorsInLevel.Append(Cluster.Actors.Array());
					}
				}
			}
		}
	}
}

ALODActor* FHierarchicalLODBuilder::CreateLODActor(const FLODCluster& InCluster, ULevel* InLevel, const int32 LODIdx)
{
	ALODActor* NewActor = nullptr;

	if (InLevel && InLevel->GetWorld())
	{
		// create asset using Actors
		const FHierarchicalSimplification& LODSetup = InLevel->GetWorldSettings()->GetHierarchicalLODSetup()[LODIdx];

		// Retrieve draw distance for current and next LOD level
		const int32 LODCount = InLevel->GetWorldSettings()->GetNumHierarchicalLODLevels();

		// Where generated assets will be stored
		FHierarchicalLODUtilitiesModule& Module = FModuleManager::LoadModuleChecked<FHierarchicalLODUtilitiesModule>("HierarchicalLODUtilities");
		IHierarchicalLODUtilities* Utilities = Module.GetUtilities();

		TArray<UStaticMeshComponent*> AllComponents;
		for (auto& Actor : InCluster.Actors)
		{
			TArray<UStaticMeshComponent*> Components;

			if (ALODActor* LODActor = Cast<ALODActor>(Actor))
			{
				UHLODProxy::ExtractStaticMeshComponentsFromLODActor(LODActor, Components);
			}
			else
			{
				Actor->GetComponents(Components);
			}

			AllComponents.Append(Components);
		}

		if (AllComponents.Num())
		{
			// Create LOD Actor
			UWorld* LevelWorld = Cast<UWorld>(InLevel->GetOuter());
			check(LevelWorld);

			FTransform Transform;

			FActorSpawnParameters ActorSpawnParams;
			ActorSpawnParams.OverrideLevel = TempLevel;

			// LODActors that are saved to HLOD packages must be transient
			ActorSpawnParams.ObjectFlags = GetDefault<UHierarchicalLODSettings>()->bSaveLODActorsToHLODPackages ? EObjectFlags::RF_Transient | EObjectFlags::RF_DuplicateTransient : EObjectFlags::RF_NoFlags;

			NewActor = LevelWorld->SpawnActor<ALODActor>(ALODActor::StaticClass(), Transform, ActorSpawnParams);
			NewLODActors.Add(NewActor);
			NewActor->LODLevel = LODIdx + 1;
			NewActor->CachedNumHLODLevels = IntCastChecked<uint8>(InLevel->GetWorldSettings()->GetNumHierarchicalLODLevels());
			NewActor->SetDrawDistance(0.0f);

			// now set as parent
			for (auto& Actor : InCluster.Actors)
			{
				NewActor->SubActors.Add(Actor);
			}
		}
	}

	return NewActor;
}
#undef LOCTEXT_NAMESPACE 
