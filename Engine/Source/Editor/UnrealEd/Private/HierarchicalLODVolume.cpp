// Copyright Epic Games, Inc. All Rights Reserved.

#include "HierarchicalLODVolume.h"
#include "Engine/CollisionProfile.h"
#include "Components/BrushComponent.h"
#include "Algo/AllOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HierarchicalLODVolume)

AHierarchicalLODVolume::AHierarchicalLODVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bIncludeOverlappingActors(false)
{
	if (UBrushComponent* MyBrushComponent = GetBrushComponent())
	{
		MyBrushComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
		MyBrushComponent->SetCanEverAffectNavigation(false);
		MyBrushComponent->SetGenerateOverlapEvents(false);
	}

	bNotForClientOrServer = true;
	bIsEditorOnlyActor = true;

	bColored = true;
	BrushColor.R = 255;
	BrushColor.G = 100;
	BrushColor.B = 255;
	BrushColor.A = 255;
}

bool AHierarchicalLODVolume::AppliesToHLODLevel(int32 LODIdx) const
{
	return ApplyOnlyToSpecificHLODLevels.Num() == 0 ||
		   ApplyOnlyToSpecificHLODLevels.Contains(LODIdx);
}

bool AHierarchicalLODVolume::IsActorIncluded(const AActor* InActor) const
{
	FBox ActorBoundingBox = InActor->GetComponentsBoundingBox(true);

	if (bIncludeOverlappingActors)
	{
		return EncompassesPoint(ActorBoundingBox.GetCenter(), static_cast<float>(ActorBoundingBox.GetExtent().Size()));
	}
	else
	{
		FVector Vertices[8];
		ActorBoundingBox.GetVertices(Vertices);
		return Algo::AllOf(Vertices, [this](const FVector& Vertex) { return EncompassesPoint(Vertex); });
	}		
}
