// Copyright Epic Games, Inc. All Rights Reserved.

#include "IPIEAuthorizer.h"

#include "Features/IModularFeatures.h"
#include "Internationalization/Text.h"

TValueOrError<bool, FText> IPIEAuthorizer::IsPIEAuthorized(const bool bIsSimulateInEditor)
{
	TArray<IPIEAuthorizer*> PlayAuthorizers = IModularFeatures::Get().GetModularFeatureImplementations<IPIEAuthorizer>(GetModularFeatureName());
	for (const IPIEAuthorizer* Authority : PlayAuthorizers)
	{
		// break on first denial
		TValueOrError<bool, FText> Result = Authority->IsPIEAuthorizedInternal(bIsSimulateInEditor);
		if (Result.HasError()
			|| !Result.GetValue())
		{
			return MoveTemp(Result);
		}
	}
	return MakeValue(true);
}

TValueOrError<bool, FText> IPIEAuthorizer::RequestPIEPermission(const bool bIsSimulateInEditor) const
{
	const TValueOrError<bool, FText> IsAuthorized = IsPIEAuthorizedInternal(bIsSimulateInEditor);
	if (IsAuthorized.HasValue()
		&& IsAuthorized.GetValue())
	{
		return RequestPIEPermissionInternal(bIsSimulateInEditor);
	}

	return IsAuthorized;
}

TValueOrError<bool, FText> IPIEAuthorizer::RequestPIEPermissionInternal(bool bIsSimulateInEditor) const
{
	// Allowed by default
	return MakeValue(true);
}

// deprecated
bool IPIEAuthorizer::RequestPIEPermission(const bool bIsSimulateInEditor, FString& OutReason) const
{
	TValueOrError<bool, FText> ValueOrError = RequestPIEPermission(bIsSimulateInEditor);
	if (ValueOrError.HasError())
	{
		OutReason = ValueOrError.GetError().ToString();
	}
	return ValueOrError.HasValue() && ValueOrError.GetValue();
}
