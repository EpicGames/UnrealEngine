// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	InternalImportUtils.cpp: Internal import utils functions.
=============================================================================*/

#include "ImportUtils/InternalImportUtils.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "EditorFramework/AssetImportData.h"
#include "Factories/FbxMeshImportData.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "UObject/MetaData.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "UObject/UnrealNames.h"


bool InternalImportUtils::IsUsingMaterialSlotNameWorkflow(UAssetImportData* AssetImportData)
{
	UFbxMeshImportData* ImportData = Cast<UFbxMeshImportData>(AssetImportData);
	if (ImportData == nullptr || ImportData->ImportMaterialOriginalNameData.Num() <= 0)
	{
		return false;
	}
	bool AllNameAreNone = true;
	for (FName ImportMaterialName : ImportData->ImportMaterialOriginalNameData)
	{
		if (ImportMaterialName != NAME_None)
		{
			AllNameAreNone = false;
			break;
		}
	}
	return !AllNameAreNone;
}


void InternalImportUtils::RestoreMetaData(UObject* TargetObject, const TMap<FName, FString>& ExistingUMetaDataTagValues)
{
	if (ExistingUMetaDataTagValues.Num() > 0)
	{
		FMetaData& PackageMetaData = TargetObject->GetPackage()->GetMetaData();

		TMap<FName, FString> MetaDataToApply(ExistingUMetaDataTagValues);
		// Keep all existing metadata, but override/add those who were imported.
		if (TMap<FName, FString>* ImportedMetaData = FMetaData::GetMapForObject(TargetObject))
		{
			for (const TPair<FName, FString>& KeyValue : *ImportedMetaData)
			{
				MetaDataToApply.Add(KeyValue);
			}
		}
		PackageMetaData.SetObjectValues(TargetObject, MetaDataToApply);
	}
}
