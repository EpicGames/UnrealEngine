// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	InternalImportUtils.h: Internal import utils functions.
=============================================================================*/

#pragma once

#include "Containers/Map.h"

class FName;
class FString;
class UAssetImportData;
class UObject;

namespace InternalImportUtils
{
	/**
	 * Returns true if the asset was imported with FBX importer and is using the "Material Slot Name Workflow".
	 */
	bool IsUsingMaterialSlotNameWorkflow(UAssetImportData* AssetImportData);

	/**
	 * Add and override the values of ExistingUMetaDataTagValues metadata map in the TargetObject package. 
	 * The existing metadata in the package are not removed.
	 */
	void RestoreMetaData(UObject* TargetObject, const TMap<FName, FString>& ExistingUMetaDataTagValues);
}
