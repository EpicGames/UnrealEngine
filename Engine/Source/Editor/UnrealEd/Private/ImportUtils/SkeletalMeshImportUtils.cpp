// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SkeletalMeshImportUtils.cpp: Skeletal mesh import code.
=============================================================================*/

#include "ImportUtils/SkeletalMeshImportUtils.h"

#include "ClothingAssetBase.h"
#include "CoreMinimal.h"
#include "EditorFramework/ThumbnailInfo.h"
#include "Engine/AssetUserData.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkeletalMeshLODSettings.h"
#include "Engine/SkeletalMeshSocket.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Factories/FbxSkeletalMeshImportData.h"
#include "FbxImporter.h"
#include "ImportUtils/InternalImportUtils.h"
#include "ImportUtils/SkelImport.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "LODUtilities.h"
#include "Materials/MaterialInterface.h"
#include "Misc/CoreMisc.h"
#include "Misc/FbxErrors.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "ReferenceSkeleton.h"
#include "Rendering/SkeletalMeshLODImporterData.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/SkeletalMeshModel.h"
#include "UObject/MetaData.h"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"

DEFINE_LOG_CATEGORY_STATIC(LogSkeletalMeshImport, Log, All);

#define LOCTEXT_NAMESPACE "SkeletalMeshImport"

namespace SkeletalMesUtilsImpl
{
	/** Check that root bone is the same, and that any bones that are common have the correct parent. */
	bool SkeletonsAreCompatible(const FReferenceSkeleton& NewSkel, const FReferenceSkeleton& ExistSkel, bool bFailNoError);

	void SaveSkeletalMeshLODModelSections(USkeletalMesh* SourceSkeletalMesh, TSharedPtr<FExistingSkelMeshData>& ExistingMeshDataPtr, int32 LodIndex, bool bSaveNonReducedMeshData);

	void SaveSkeletalMeshMaterialNameWorkflowData(TSharedPtr<FExistingSkelMeshData>& ExistingMeshDataPtr, const USkeletalMesh* SourceSkeletalMesh);

	void SaveSkeletalMeshAssetUserData(TSharedPtr<FExistingSkelMeshData>& ExistingMeshDataPtr, const TArray<UAssetUserData*>* UserData);

	void RestoreDependentLODs(const TSharedPtr<const FExistingSkelMeshData>& MeshData, USkeletalMesh* SkeletalMesh);

	void RestoreLODInfo(const TSharedPtr<const FExistingSkelMeshData>& MeshData, USkeletalMesh* SkeletalMesh, int32 LodIndex);

	void RestoreMaterialNameWorkflowSection(const TSharedPtr<const FExistingSkelMeshData>& MeshData, USkeletalMesh* SkeletalMesh, int32 LodIndex, TArray<int32>& RemapMaterial, bool bMaterialReset);
}

bool SkeletalMesUtilsImpl::SkeletonsAreCompatible(const FReferenceSkeleton& NewSkel, const FReferenceSkeleton& ExistSkel, bool bFailNoError)
{
	if (NewSkel.GetBoneName(0) != ExistSkel.GetBoneName(0))
	{
		if (!bFailNoError)
		{
			UnFbx::FFbxImporter* FFbxImporter = UnFbx::FFbxImporter::GetInstance();
			FFbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("MeshHasDifferentRoot", "Root Bone is '{0}' instead of '{1}'.\nDiscarding existing LODs."),
				FText::FromName(NewSkel.GetBoneName(0)), FText::FromName(ExistSkel.GetBoneName(0)))), FFbxErrors::SkeletalMesh_DifferentRoots);
		}
		return false;
	}

	for (int32 i = 1; i < NewSkel.GetRawBoneNum(); i++)
	{
		// See if bone is in both skeletons.
		int32 NewBoneIndex = i;
		FName NewBoneName = NewSkel.GetBoneName(NewBoneIndex);
		int32 BBoneIndex = ExistSkel.FindBoneIndex(NewBoneName);

		// If it is, check parents are the same.
		if (BBoneIndex != INDEX_NONE)
		{
			FName NewParentName = NewSkel.GetBoneName(NewSkel.GetParentIndex(NewBoneIndex));
			FName ExistParentName = ExistSkel.GetBoneName(ExistSkel.GetParentIndex(BBoneIndex));

			if (NewParentName != ExistParentName)
			{
				if (!bFailNoError)
				{
					UnFbx::FFbxImporter* FFbxImporter = UnFbx::FFbxImporter::GetInstance();
					FFbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("MeshHasDifferentParent", "Parent Bone of '{0}' is '{1}' instead of '{2}'. Discarding existing LODs."),
						FText::FromName(NewBoneName), FText::FromName(NewParentName), FText::FromName(ExistParentName))), FFbxErrors::SkeletalMesh_DifferentRoots);
				}
				return false;
			}
		}
	}

	return true;
}

/**
* Process and fill in the mesh Materials using the raw binary import data
*
* @param Materials - [out] array of materials to update
* @param ImportData - raw binary import data to process
*/
void SkeletalMeshImportUtils::ProcessImportMeshMaterials(TArray<FSkeletalMaterial>& Materials, FSkeletalMeshImportData& ImportData)
{
	TArray <SkeletalMeshImportData::FMaterial>&	ImportedMaterials = ImportData.Materials;

	// If direct linkup of materials is requested, try to find them here - to get a texture name from a 
	// material name, cut off anything in front of the dot (beyond are special flags).
	Materials.Empty();
	int32 SkinOffset = INDEX_NONE;
	for (int32 MatIndex = 0; MatIndex < ImportedMaterials.Num(); ++MatIndex)
	{
		const SkeletalMeshImportData::FMaterial& ImportedMaterial = ImportedMaterials[MatIndex];

		UMaterialInterface* Material = NULL;
		FString MaterialNameNoSkin = ImportedMaterial.MaterialImportName;
		if (ImportedMaterial.Material.IsValid())
		{
			Material = ImportedMaterial.Material.Get();
		}
		else
		{
			const FString& MaterialName = ImportedMaterial.MaterialImportName;
			MaterialNameNoSkin = MaterialName;
			Material = FindFirstObject<UMaterialInterface>(*MaterialName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("ProcessImportMeshMaterials"));
			if (Material == nullptr)
			{
				SkinOffset = MaterialName.Find(TEXT("_skin"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
				if (SkinOffset != INDEX_NONE)
				{
					FString SkinXXNumber = MaterialName.Right(MaterialName.Len() - (SkinOffset + 1)).RightChop(4);
					if (SkinXXNumber.IsNumeric())
					{
						MaterialNameNoSkin = MaterialName.LeftChop(MaterialName.Len() - SkinOffset);
						Material = FindFirstObject<UMaterialInterface>(*MaterialNameNoSkin, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("ProcessImportMeshMaterials"));
					}
				}
			}
		}

		const bool bEnableShadowCasting = true;
		Materials.Add(FSkeletalMaterial(Material, bEnableShadowCasting, false, Material != nullptr ? Material->GetFName() : FName(*MaterialNameNoSkin), FName(*(ImportedMaterial.MaterialImportName))));
	}

	int32 NumMaterialsToAdd = FMath::Max<int32>(ImportedMaterials.Num(), ImportData.MaxMaterialIndex + 1);

	// Pad the material pointers
	while (NumMaterialsToAdd > Materials.Num())
	{
		Materials.Add(FSkeletalMaterial(NULL, true, false, NAME_None, NAME_None));
	}
}

/**
* Process and fill in the mesh ref skeleton bone hierarchy using the raw binary import data
*
* @param RefSkeleton - [out] reference skeleton hierarchy to update
* @param SkeletalDepth - [out] depth of the reference skeleton hierarchy
* @param ImportData - raw binary import data to process
* @return true if the operation completed successfully
*/
bool SkeletalMeshImportUtils::ProcessImportMeshSkeleton(const USkeleton* SkeletonAsset, FReferenceSkeleton& RefSkeleton, int32& SkeletalDepth, FSkeletalMeshImportData& ImportData)
{
	TArray <SkeletalMeshImportData::FBone>&	RefBonesBinary = ImportData.RefBonesBinary;

	// Setup skeletal hierarchy + names structure.
	RefSkeleton.Empty();

	FReferenceSkeletonModifier RefSkelModifier(RefSkeleton, SkeletonAsset);

	// Digest bones to the serializable format.
	for (int32 b = 0; b < RefBonesBinary.Num(); b++)
	{
		const SkeletalMeshImportData::FBone & BinaryBone = RefBonesBinary[b];
		const FString BoneName = FSkeletalMeshImportData::FixupBoneName(BinaryBone.Name);
		const FMeshBoneInfo BoneInfo(FName(*BoneName, FNAME_Add), BinaryBone.Name, BinaryBone.ParentIndex);
		const FTransform BoneTransform(BinaryBone.BonePos.Transform);

		if (RefSkeleton.FindRawBoneIndex(BoneInfo.Name) != INDEX_NONE)
		{
			UnFbx::FFbxImporter* FFbxImporter = UnFbx::FFbxImporter::GetInstance();
			FFbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("SkeletonHasDuplicateBones", "Skeleton has non-unique bone names.\nBone named '{0}' encountered more than once."), FText::FromName(BoneInfo.Name))), FFbxErrors::SkeletalMesh_DuplicateBones);
			return false;
		}

		RefSkelModifier.Add(BoneInfo, BoneTransform);
	}

	// Add hierarchy index to each bone and detect max depth.
	SkeletalDepth = 0;

	TArray<int32> SkeletalDepths;
	SkeletalDepths.Empty(RefBonesBinary.Num());
	SkeletalDepths.AddZeroed(RefBonesBinary.Num());
	for (int32 b = 0; b < RefSkeleton.GetRawBoneNum(); b++)
	{
		int32 Parent = RefSkeleton.GetRawParentIndex(b);
		int32 Depth = 1;

		SkeletalDepths[b] = 1;
		if (Parent != INDEX_NONE)
		{
			Depth += SkeletalDepths[Parent];
		}
		if (SkeletalDepth < Depth)
		{
			SkeletalDepth = Depth;
		}
		SkeletalDepths[b] = Depth;
	}

	return true;
}

/**
* Process and update the vertex Influences using the raw binary import data
*
* @param ImportData - raw binary import data to process
*/
void SkeletalMeshImportUtils::ProcessImportMeshInfluences(FSkeletalMeshImportData& ImportData, const FString& SkeletalMeshName)
{
	FLODUtilities::ProcessImportMeshInfluences(ImportData.Points.Num(), ImportData.Influences, SkeletalMeshName);
}

void SkeletalMesUtilsImpl::SaveSkeletalMeshLODModelSections(USkeletalMesh* SourceSkeletalMesh, TSharedPtr<FExistingSkelMeshData>& ExistingMeshDataPtr, int32 LodIndex, bool bSaveNonReducedMeshData)
{
	const FSkeletalMeshModel* SourceMeshModel = SourceSkeletalMesh->GetImportedModel();
	const FSkeletalMeshLODModel* SourceLODModel = &SourceMeshModel->LODModels[LodIndex];
	FSkeletalMeshLODModel OriginalLODModel;

	ExistingMeshDataPtr->ExistingImportMeshLodSectionMaterialData.AddZeroed();
	check(ExistingMeshDataPtr->ExistingImportMeshLodSectionMaterialData.IsValidIndex(LodIndex));

	for (const FSkelMeshSection& CurrentSection : SourceLODModel->Sections)
	{
		int32 SectionMaterialIndex = CurrentSection.MaterialIndex;
		bool SectionCastShadow = CurrentSection.bCastShadow;
		bool SectionVisibleInRayTracing = CurrentSection.bVisibleInRayTracing;
		bool SectionRecomputeTangents = CurrentSection.bRecomputeTangent;
		ESkinVertexColorChannel RecomputeTangentsVertexMaskChannel = CurrentSection.RecomputeTangentsVertexMaskChannel;
		int32 GenerateUpTo = CurrentSection.GenerateUpToLodIndex;
		bool bDisabled = CurrentSection.bDisabled;
		bool bBoneChunkedSection = CurrentSection.ChunkedParentSectionIndex != INDEX_NONE;
		//Save all the sections, even the chunked sections
		if (ExistingMeshDataPtr->ExistingImportMaterialOriginalNameData.IsValidIndex(SectionMaterialIndex))
		{
			ExistingMeshDataPtr->ExistingImportMeshLodSectionMaterialData[LodIndex].Emplace(ExistingMeshDataPtr->ExistingImportMaterialOriginalNameData[SectionMaterialIndex], SectionCastShadow, SectionVisibleInRayTracing, SectionRecomputeTangents, RecomputeTangentsVertexMaskChannel, GenerateUpTo, bDisabled);
		}
	}
}

void SkeletalMesUtilsImpl::SaveSkeletalMeshMaterialNameWorkflowData(TSharedPtr<FExistingSkelMeshData>& ExistingMeshDataPtr, const USkeletalMesh* SourceSkeletalMesh)
{
	const UFbxSkeletalMeshImportData* ImportData = Cast<UFbxSkeletalMeshImportData>(SourceSkeletalMesh->GetAssetImportData());
	if (!ImportData)
	{
		return;
	}

	for (int32 ImportMaterialOriginalNameDataIndex = 0; ImportMaterialOriginalNameDataIndex < ImportData->ImportMaterialOriginalNameData.Num(); ++ImportMaterialOriginalNameDataIndex)
	{
		FName MaterialName = ImportData->ImportMaterialOriginalNameData[ImportMaterialOriginalNameDataIndex];
		ExistingMeshDataPtr->LastImportMaterialOriginalNameData.Add(MaterialName);
	}

	for (int32 LodIndex = 0; LodIndex < ImportData->ImportMeshLodData.Num(); ++LodIndex)
	{
		ExistingMeshDataPtr->LastImportMeshLodSectionMaterialData.AddZeroed();
		const FImportMeshLodSectionsData &ImportMeshLodSectionsData = ImportData->ImportMeshLodData[LodIndex];
		for (int32 SectionIndex = 0; SectionIndex < ImportMeshLodSectionsData.SectionOriginalMaterialName.Num(); ++SectionIndex)
		{
			FName MaterialName = ImportMeshLodSectionsData.SectionOriginalMaterialName[SectionIndex];
			ExistingMeshDataPtr->LastImportMeshLodSectionMaterialData[LodIndex].Add(MaterialName);
		}
	}
}

void SkeletalMesUtilsImpl::SaveSkeletalMeshAssetUserData(TSharedPtr<FExistingSkelMeshData>& ExistingMeshDataPtr, const TArray<UAssetUserData*>* UserData)
{
	if (!UserData)
	{
		return;
	}

	for (int32 Idx = 0; Idx < UserData->Num(); Idx++)
	{
		if ((*UserData)[Idx] != nullptr)
		{
			UAssetUserData* DupObject = (UAssetUserData*)StaticDuplicateObject((*UserData)[Idx], GetTransientPackage());
			bool bAddDupToRoot = !(DupObject->IsRooted());
			if (bAddDupToRoot)
			{
				DupObject->AddToRoot();
			}
			ExistingMeshDataPtr->ExistingAssetUserData.Add(DupObject, bAddDupToRoot);
		}
	}
}

TSharedPtr<FExistingSkelMeshData> SkeletalMeshImportUtils::SaveExistingSkelMeshData(USkeletalMesh* SourceSkeletalMesh, bool bSaveMaterials, int32 ReimportLODIndex)
{
	using namespace SkeletalMesUtilsImpl;

	if (!SourceSkeletalMesh)
	{
		return TSharedPtr<FExistingSkelMeshData>();
	}

	const int32 SafeReimportLODIndex = ReimportLODIndex < 0 ? 0 : ReimportLODIndex;
	TSharedPtr<FExistingSkelMeshData> ExistingMeshDataPtr(MakeShared<FExistingSkelMeshData>());

	//Save the package FMetaData
	TMap<FName, FString>* MetaDataTagValues = FMetaData::GetMapForObject(SourceSkeletalMesh);
	if (MetaDataTagValues && MetaDataTagValues->Num() > 0)
	{
		ExistingMeshDataPtr->ExistingUMetaDataTagValues = *MetaDataTagValues;
	}
	
	ExistingMeshDataPtr->UseMaterialNameSlotWorkflow = InternalImportUtils::IsUsingMaterialSlotNameWorkflow(SourceSkeletalMesh->GetAssetImportData());
	ExistingMeshDataPtr->MinLOD = SourceSkeletalMesh->GetMinLod();
	ExistingMeshDataPtr->QualityLevelMinLOD = SourceSkeletalMesh->GetQualityLevelMinLod();
	ExistingMeshDataPtr->DisableBelowMinLodStripping = SourceSkeletalMesh->GetDisableBelowMinLodStripping();
	ExistingMeshDataPtr->bOverrideLODStreamingSettings = SourceSkeletalMesh->GetOverrideLODStreamingSettings();
	ExistingMeshDataPtr->bSupportLODStreaming = SourceSkeletalMesh->GetSupportLODStreaming();
	ExistingMeshDataPtr->MaxNumStreamedLODs = SourceSkeletalMesh->GetMaxNumStreamedLODs();
	ExistingMeshDataPtr->MaxNumOptionalLODs = SourceSkeletalMesh->GetMaxNumOptionalLODs();

	ExistingMeshDataPtr->ExistingStreamableRenderAssetData.Save(SourceSkeletalMesh);

	const TArray<FSkeletalMaterial>& SourceMaterials = SourceSkeletalMesh->GetMaterials();
	//Add the existing Material slot name data
	for (int32 MaterialIndex = 0; MaterialIndex < SourceMaterials.Num(); ++MaterialIndex)
	{
		ExistingMeshDataPtr->ExistingImportMaterialOriginalNameData.Add(SourceMaterials[MaterialIndex].ImportedMaterialSlotName);
	}

	FSkeletalMeshModel* SourceMeshModel = SourceSkeletalMesh->GetImportedModel();
	for (int32 LodIndex = 0; LodIndex < SourceMeshModel->LODModels.Num(); ++LodIndex)
	{
		const bool bImportNonReducedData = LodIndex == SafeReimportLODIndex;
		SaveSkeletalMeshLODModelSections(SourceSkeletalMesh, ExistingMeshDataPtr, LodIndex, bImportNonReducedData);
	}

	ExistingMeshDataPtr->ExistingSockets = SourceSkeletalMesh->GetMeshOnlySocketList();
	ExistingMeshDataPtr->bSaveRestoreMaterials = bSaveMaterials;
	if (ExistingMeshDataPtr->bSaveRestoreMaterials)
	{
		ExistingMeshDataPtr->ExistingMaterials = SourceMaterials;
	}

	ExistingMeshDataPtr->ExistingInlineReductionCacheData = SourceMeshModel->InlineReductionCacheDatas;

	const int32 NumLODModels = SourceMeshModel->LODModels.Num();
	if (NumLODModels > 0 && SourceSkeletalMesh->GetLODNum() == NumLODModels)
	{
		// Copy LOD models and LOD Infos.
		ExistingMeshDataPtr->ExistingLODModels.Empty(NumLODModels);
		ExistingMeshDataPtr->ExistingLODMeshDescriptions.Reserve(NumLODModels);
		ExistingMeshDataPtr->ExistingLODInfo.Empty(NumLODModels);
		
		for ( int32 LODIndex = 0; LODIndex < NumLODModels; ++LODIndex)
		{
			//Add a new LOD Model to the existing LODModels data
			const FSkeletalMeshLODModel& LODModel = SourceMeshModel->LODModels[LODIndex];
			ExistingMeshDataPtr->ExistingLODModels.Add(FSkeletalMeshLODModel::CreateCopy(&LODModel));
			
			//Store the import data for every LOD
			FMeshDescription& MeshDescription = ExistingMeshDataPtr->ExistingLODMeshDescriptions.AddDefaulted_GetRef();
			SourceSkeletalMesh->CloneMeshDescription(LODIndex, MeshDescription);
			
			ExistingMeshDataPtr->ExistingLODInfo.Add(*SourceSkeletalMesh->GetLODInfo(LODIndex));	
		}
		
		ExistingMeshDataPtr->ExistingRefSkeleton = SourceSkeletalMesh->GetRefSkeleton();
	}

	// First asset should be the one that the skeletal mesh should point too
	ExistingMeshDataPtr->ExistingPhysicsAssets.Empty();
	ExistingMeshDataPtr->ExistingPhysicsAssets.Add(SourceSkeletalMesh->GetPhysicsAsset());
	for (TObjectIterator<UPhysicsAsset> It; It; ++It)
	{
		UPhysicsAsset* PhysicsAsset = *It;
		if (PhysicsAsset->PreviewSkeletalMesh == SourceSkeletalMesh && SourceSkeletalMesh->GetPhysicsAsset() != PhysicsAsset)
		{
			ExistingMeshDataPtr->ExistingPhysicsAssets.Add(PhysicsAsset);
		}
	}

	ExistingMeshDataPtr->ExistingShadowPhysicsAsset = SourceSkeletalMesh->GetShadowPhysicsAsset();
	ExistingMeshDataPtr->ExistingSkeleton = SourceSkeletalMesh->GetSkeleton();
	// since copying back original skeleton, this should be safe to do
	ExistingMeshDataPtr->ExistingPostProcessAnimBlueprint = SourceSkeletalMesh->GetPostProcessAnimBlueprint();
	ExistingMeshDataPtr->ExistingLODSettings = SourceSkeletalMesh->GetLODSettings();

	ExistingMeshDataPtr->ExistingMorphTargets = SourceSkeletalMesh->GetMorphTargets();
	ExistingMeshDataPtr->ExistingAssetImportData = SourceSkeletalMesh->GetAssetImportData();
	ExistingMeshDataPtr->ExistingThumbnailInfo = SourceSkeletalMesh->GetThumbnailInfo();
	ExistingMeshDataPtr->ExistingClothingAssets = SourceSkeletalMesh->GetMeshClothingAssets();
	ExistingMeshDataPtr->ExistingSamplingInfo = SourceSkeletalMesh->GetSamplingInfo();
	ExistingMeshDataPtr->ExistingDefaultAnimatingRig = SourceSkeletalMesh->GetDefaultAnimatingRig();
	ExistingMeshDataPtr->ExistingDefaultMeshDeformer = SourceSkeletalMesh->GetDefaultMeshDeformer();
	ExistingMeshDataPtr->ExistingTargetMeshDeformers = SourceSkeletalMesh->GetTargetMeshDeformers();

	if (ExistingMeshDataPtr->UseMaterialNameSlotWorkflow)
	{
		//Add the last fbx import data
		SaveSkeletalMeshMaterialNameWorkflowData(ExistingMeshDataPtr, SourceSkeletalMesh);
	}

	//Store the user asset data
	SaveSkeletalMeshAssetUserData(ExistingMeshDataPtr, SourceSkeletalMesh->GetAssetUserDataArray());
	
	//Store mesh changed delegate data
	ExistingMeshDataPtr->ExistingOnMeshChanged = SourceSkeletalMesh->GetOnMeshChanged();

	//Store mesh bounds extensions
	ExistingMeshDataPtr->PositiveBoundsExtension = SourceSkeletalMesh->GetPositiveBoundsExtension();
	ExistingMeshDataPtr->NegativeBoundsExtension = SourceSkeletalMesh->GetNegativeBoundsExtension();

	ExistingMeshDataPtr->bExistingSupportRayTracing = SourceSkeletalMesh->GetSupportRayTracing();
	ExistingMeshDataPtr->ExistingRayTracingMinLOD = SourceSkeletalMesh->GetRayTracingMinLOD();
	ExistingMeshDataPtr->ExistingClothLODBiasMode = SourceSkeletalMesh->GetClothLODBiasMode();

	return ExistingMeshDataPtr;
}

void SkeletalMesUtilsImpl::RestoreDependentLODs(const TSharedPtr<const FExistingSkelMeshData>& MeshData, USkeletalMesh* SkeletalMesh)
{
	check(SkeletalMesh != nullptr);
	const int32 TotalLOD = MeshData->ExistingLODModels.Num();
	FSkeletalMeshModel* SkeletalMeshImportedModel = SkeletalMesh->GetImportedModel();

	for (int32 LODIndex = 1; LODIndex < TotalLOD; ++LODIndex)
	{
		if (LODIndex >= SkeletalMesh->GetLODNum())
		{
			// Create a copy of LODInfo and reset material maps, it won't work anyway. 
			FSkeletalMeshLODInfo ExistLODInfo = MeshData->ExistingLODInfo[LODIndex];
			ExistLODInfo.LODMaterialMap.Empty();
			// add LOD info back
			SkeletalMesh->AddLODInfo(MoveTemp(ExistLODInfo));
			check(LODIndex < SkeletalMesh->GetLODNum());

			const FSkeletalMeshLODModel& ExistLODModel = MeshData->ExistingLODModels[LODIndex];
			SkeletalMeshImportedModel->LODModels.Add(FSkeletalMeshLODModel::CreateCopy(&ExistLODModel));
		}
	}
}

void SkeletalMesUtilsImpl::RestoreLODInfo(const TSharedPtr<const FExistingSkelMeshData>& MeshData, USkeletalMesh* SkeletalMesh, int32 LodIndex)
{
	FSkeletalMeshLODInfo& ImportedLODInfo = *SkeletalMesh->GetLODInfo(LodIndex);
	if (!MeshData->ExistingLODInfo.IsValidIndex(LodIndex))
	{
		return;
	}

	const FSkeletalMeshLODInfo& ExistingLODInfo = MeshData->ExistingLODInfo[LodIndex];

	ImportedLODInfo.ScreenSize = ExistingLODInfo.ScreenSize;
	ImportedLODInfo.LODHysteresis = ExistingLODInfo.LODHysteresis;
	ImportedLODInfo.BuildSettings = ExistingLODInfo.BuildSettings;
	//Old assets may have non-applied reduction settings, so only restore the reduction settings if the LOD was effectively reduced and we did not import a custom LOD over this generated LOD.
	if (ExistingLODInfo.bHasBeenSimplified && (!ExistingLODInfo.SourceImportFilename.IsEmpty() || ImportedLODInfo.SourceImportFilename.IsEmpty()))
	{
		ImportedLODInfo.ReductionSettings = ExistingLODInfo.ReductionSettings;
	}
	ImportedLODInfo.BonesToRemove = ExistingLODInfo.BonesToRemove;
	ImportedLODInfo.BonesToPrioritize = ExistingLODInfo.BonesToPrioritize;
	ImportedLODInfo.SectionsToPrioritize = ExistingLODInfo.SectionsToPrioritize;
	ImportedLODInfo.WeightOfPrioritization = ExistingLODInfo.WeightOfPrioritization;
	ImportedLODInfo.BakePose = ExistingLODInfo.BakePose;
	ImportedLODInfo.BakePoseOverride = ExistingLODInfo.BakePoseOverride;
	ImportedLODInfo.SourceImportFilename = ExistingLODInfo.SourceImportFilename;
	ImportedLODInfo.SkinCacheUsage = ExistingLODInfo.SkinCacheUsage;
	ImportedLODInfo.MorphTargetPositionErrorTolerance = ExistingLODInfo.MorphTargetPositionErrorTolerance;
	ImportedLODInfo.bAllowCPUAccess = ExistingLODInfo.bAllowCPUAccess;
	ImportedLODInfo.bBuildHalfEdgeBuffers = ExistingLODInfo.bBuildHalfEdgeBuffers;
	ImportedLODInfo.bSupportUniformlyDistributedSampling = ExistingLODInfo.bSupportUniformlyDistributedSampling;
	ImportedLODInfo.bAllowMeshDeformer = ExistingLODInfo.bAllowMeshDeformer;
}

void SkeletalMeshImportUtils::ApplySkinning(USkeletalMesh* SkeletalMesh, FSkeletalMeshLODModel& SrcLODModel, FSkeletalMeshLODModel& DestLODModel)
{
	TArray<FSoftSkinVertex> SrcVertices;
	SrcLODModel.GetVertices(SrcVertices);

	FBox OldBounds(EForceInit::ForceInit);
	for (int32 SrcIndex = 0; SrcIndex < SrcVertices.Num(); ++SrcIndex)
	{
		const FSoftSkinVertex& SrcVertex = SrcVertices[SrcIndex];
		OldBounds += (FVector)SrcVertex.Position;
	}

	TWedgeInfoPosOctree SrcWedgePosOctree(OldBounds.GetCenter(), OldBounds.GetExtent().GetMax());
	// Add each old vertex to the octree
	for (int32 SrcIndex = 0; SrcIndex < SrcVertices.Num(); ++SrcIndex)
	{
		FWedgeInfo WedgeInfo;
		WedgeInfo.WedgeIndex = SrcIndex;
		WedgeInfo.Position = (FVector)SrcVertices[SrcIndex].Position;
		SrcWedgePosOctree.AddElement(WedgeInfo);
	}

	FOctreeQueryHelper OctreeQueryHelper(&SrcWedgePosOctree);

	TArray<FBoneIndexType> RequiredActiveBones;

	bool bUseBone = false;
	for (int32 SectionIndex = 0; SectionIndex < DestLODModel.Sections.Num(); SectionIndex++)
	{
		FSkelMeshSection& Section = DestLODModel.Sections[SectionIndex];
		Section.BoneMap.Reset();
		for (FSoftSkinVertex& DestVertex : Section.SoftVertices)
		{
			//Find the nearest wedges in the src model
			TArray<FWedgeInfo> NearestSrcWedges;
			OctreeQueryHelper.FindNearestWedgeIndexes(DestVertex.Position, NearestSrcWedges);
			if (NearestSrcWedges.Num() < 1)
			{
				//Should we check???
				continue;
			}
			//Find the matching wedges in the src model
			int32 MatchingSrcWedge = INDEX_NONE;
			for (FWedgeInfo& SrcWedgeInfo : NearestSrcWedges)
			{
				int32 SrcIndex = SrcWedgeInfo.WedgeIndex;
				const FSoftSkinVertex& SrcVertex = SrcVertices[SrcIndex];
				if (SrcVertex.Position.Equals(DestVertex.Position, THRESH_POINTS_ARE_SAME) &&
					SrcVertex.UVs[0].Equals(DestVertex.UVs[0], THRESH_UVS_ARE_SAME) &&
					(SrcVertex.TangentX == DestVertex.TangentX) &&
					(SrcVertex.TangentY == DestVertex.TangentY) &&
					(SrcVertex.TangentZ == DestVertex.TangentZ))
				{
					MatchingSrcWedge = SrcIndex;
					break;
				}
			}
			if (MatchingSrcWedge == INDEX_NONE)
			{
				//We have to find the nearest wedges, then find the most similar normal
				float MinDistance = MAX_FLT;
				float MinNormalAngle = MAX_FLT;
				for (FWedgeInfo& SrcWedgeInfo : NearestSrcWedges)
				{
					int32 SrcIndex = SrcWedgeInfo.WedgeIndex;
					const FSoftSkinVertex& SrcVertex = SrcVertices[SrcIndex];
					float VectorDelta = FVector3f::DistSquared(SrcVertex.Position, DestVertex.Position);
					if (VectorDelta <= (MinDistance + KINDA_SMALL_NUMBER))
					{
						if (VectorDelta < MinDistance - KINDA_SMALL_NUMBER)
						{
							MinDistance = VectorDelta;
							MinNormalAngle = MAX_FLT;
						}
						FVector DestTangentZ = FVector4(DestVertex.TangentZ);
						DestTangentZ.Normalize();
						FVector SrcTangentZ = FVector4(SrcVertex.TangentZ);
						SrcTangentZ.Normalize();
						double AngleDiff = FMath::Abs(FMath::Acos(FVector::DotProduct(DestTangentZ, SrcTangentZ)));
						if (AngleDiff < MinNormalAngle)
						{
							MinNormalAngle = AngleDiff;
							MatchingSrcWedge = SrcIndex;
						}
					}
				}
			}
			check(SrcVertices.IsValidIndex(MatchingSrcWedge));
			const FSoftSkinVertex& SrcVertex = SrcVertices[MatchingSrcWedge];

			//Find the src section to assign the correct remapped bone
			int32 SrcSectionIndex = INDEX_NONE;
			int32 SrcSectionWedgeIndex = INDEX_NONE;
			SrcLODModel.GetSectionFromVertexIndex(MatchingSrcWedge, SrcSectionIndex, SrcSectionWedgeIndex);
			check(SrcSectionIndex != INDEX_NONE);

			for (int32 InfluenceIndex = 0; InfluenceIndex < MAX_TOTAL_INFLUENCES; ++InfluenceIndex)
			{
				if (SrcVertex.InfluenceWeights[InfluenceIndex] > 0.0f)
				{
					Section.MaxBoneInfluences = FMath::Max(Section.MaxBoneInfluences, InfluenceIndex + 1);
					//Copy the weight
					DestVertex.InfluenceWeights[InfluenceIndex] = SrcVertex.InfluenceWeights[InfluenceIndex];
					//Copy the bone ID
					FBoneIndexType OriginalBoneIndex = SrcLODModel.Sections[SrcSectionIndex].BoneMap[SrcVertex.InfluenceBones[InfluenceIndex]];
					int32 OverrideIndex;
					if (Section.BoneMap.Find(OriginalBoneIndex, OverrideIndex))
					{
						DestVertex.InfluenceBones[InfluenceIndex] = IntCastChecked<FBoneIndexType>(OverrideIndex);
					}
					else
					{
						DestVertex.InfluenceBones[InfluenceIndex] = IntCastChecked<FBoneIndexType>(Section.BoneMap.Add(OriginalBoneIndex));
						DestLODModel.ActiveBoneIndices.AddUnique(OriginalBoneIndex);
					}
					bUseBone = true;
				}
			}
		}
	}

	if (bUseBone)
	{
		//Set the required/active bones
		DestLODModel.RequiredBones = SrcLODModel.RequiredBones;
		DestLODModel.RequiredBones.Sort();
		SkeletalMesh->GetRefSkeleton().EnsureParentsExistAndSort(DestLODModel.ActiveBoneIndices);
	}
}


void SkeletalMeshImportUtils::RestoreExistingSkelMeshData(const TSharedPtr<const FExistingSkelMeshData>& MeshData, USkeletalMesh* SkeletalMesh, int32 ReimportLODIndex, bool bCanShowDialog, bool bImportSkinningOnly, bool bForceMaterialReset)
{
	using namespace SkeletalMesUtilsImpl;
	if (!MeshData || !SkeletalMesh)
	{
		return;
	}

	//Restore the package metadata
	InternalImportUtils::RestoreMetaData(SkeletalMesh, MeshData->ExistingUMetaDataTagValues);

	int32 SafeReimportLODIndex = ReimportLODIndex < 0 ? 0 : ReimportLODIndex;
	SkeletalMesh->SetMinLod(MeshData->MinLOD);
	SkeletalMesh->SetQualityLevelMinLod(MeshData->QualityLevelMinLOD);
	SkeletalMesh->SetDisableBelowMinLodStripping(MeshData->DisableBelowMinLodStripping);
	SkeletalMesh->SetOverrideLODStreamingSettings(MeshData->bOverrideLODStreamingSettings);
	SkeletalMesh->SetSupportLODStreaming(MeshData->bSupportLODStreaming);
	SkeletalMesh->SetMaxNumStreamedLODs(MeshData->MaxNumStreamedLODs);
	SkeletalMesh->SetMaxNumOptionalLODs(MeshData->MaxNumOptionalLODs);

	MeshData->ExistingStreamableRenderAssetData.Restore(SkeletalMesh);

	FSkeletalMeshModel* SkeletalMeshImportedModel = SkeletalMesh->GetImportedModel();

	//Create a remap material Index use to find the matching section later
	TArray<int32> RemapMaterial;
	RemapMaterial.AddZeroed(SkeletalMesh->GetMaterials().Num());
	TArray<FName> RemapMaterialName;
	RemapMaterialName.AddZeroed(SkeletalMesh->GetMaterials().Num());

	bool bMaterialReset = false;
	if (MeshData->bSaveRestoreMaterials)
	{
		UnFbx::EFBXReimportDialogReturnOption ReturnOption;
		//Ask the user to match the materials conflict
		UnFbx::FFbxImporter::PrepareAndShowMaterialConflictDialog<FSkeletalMaterial>(MeshData->ExistingMaterials, SkeletalMesh->GetMaterials(), RemapMaterial, RemapMaterialName, bCanShowDialog, false, bForceMaterialReset, ReturnOption);

		if (ReturnOption != UnFbx::EFBXReimportDialogReturnOption::FBXRDRO_ResetToFbx)
		{
			//Build a ordered material list that try to keep intact the existing material list
			TArray<FSkeletalMaterial> MaterialOrdered;
			TArray<bool> MatchedNewMaterial;
			MatchedNewMaterial.AddZeroed(SkeletalMesh->GetMaterials().Num());
			for (int32 ExistMaterialIndex = 0; ExistMaterialIndex < MeshData->ExistingMaterials.Num(); ++ExistMaterialIndex)
			{
				int32 MaterialIndexOrdered = MaterialOrdered.Add(MeshData->ExistingMaterials[ExistMaterialIndex]);
				FSkeletalMaterial& OrderedMaterial = MaterialOrdered[MaterialIndexOrdered];
				int32 NewMaterialIndex = INDEX_NONE;
				if (RemapMaterial.Find(ExistMaterialIndex, NewMaterialIndex))
				{
					MatchedNewMaterial[NewMaterialIndex] = true;
					RemapMaterial[NewMaterialIndex] = MaterialIndexOrdered;
					OrderedMaterial.ImportedMaterialSlotName = SkeletalMesh->GetMaterials()[NewMaterialIndex].ImportedMaterialSlotName;
				}
				else
				{
					//Unmatched material must be conserve
				}
			}

			//Add the new material entries (the one that do not match with any existing material)
			for (int32 NewMaterialIndex = 0; NewMaterialIndex < MatchedNewMaterial.Num(); ++NewMaterialIndex)
			{
				if (MatchedNewMaterial[NewMaterialIndex] == false)
				{
					int32 NewMeshIndex = MaterialOrdered.Add(SkeletalMesh->GetMaterials()[NewMaterialIndex]);
					RemapMaterial[NewMaterialIndex] = NewMeshIndex;
				}
			}

			//Set the RemapMaterialName array helper
			for (int32 MaterialIndex = 0; MaterialIndex < RemapMaterial.Num(); ++MaterialIndex)
			{
				int32 SourceMaterialMatch = RemapMaterial[MaterialIndex];
				if (MeshData->ExistingMaterials.IsValidIndex(SourceMaterialMatch))
				{
					RemapMaterialName[MaterialIndex] = MeshData->ExistingMaterials[SourceMaterialMatch].ImportedMaterialSlotName;
				}
			}

			//Copy the re ordered materials (this ensure the material array do not change when we re-import)
			SkeletalMesh->SetMaterials(MaterialOrdered);
		}
		else
		{
			bMaterialReset = true;
		}
	}
	SkeletalMesh->SetLODSettings(MeshData->ExistingLODSettings);
	// ensure LOD 0 contains correct setting 
	if (SkeletalMesh->GetLODSettings() && SkeletalMesh->GetLODNum() > 0)
	{
		SkeletalMesh->GetLODSettings()->SetLODSettingsToMesh(SkeletalMesh, 0);
	}

	//Do everything we need for base LOD re-import
	if (SafeReimportLODIndex == 0)
	{
		// Assign sockets from old version of this SkeletalMesh.
		// Only copy ones for bones that exist in the new mesh.
		for (int32 i = 0; i < MeshData->ExistingSockets.Num(); i++)
		{
			const int32 BoneIndex = SkeletalMesh->GetRefSkeleton().FindBoneIndex(MeshData->ExistingSockets[i]->BoneName);
			if (BoneIndex != INDEX_NONE)
			{
				SkeletalMesh->GetMeshOnlySocketList().Add(MeshData->ExistingSockets[i]);
			}
		}
		if (!ensure(SkeletalMeshImportedModel->InlineReductionCacheDatas.IsValidIndex(0)))
		{
			UE_LOGF(LogSkeletalMeshImport, Display, "Reimport SkeletalMesh did not have a valid inline reduction cache data for LOD 0");
		}

		// We copy back and fix-up the LODs that still work with this skeleton.
		if (MeshData->ExistingLODModels.Num() > 1)
		{
			if (SkeletonsAreCompatible(SkeletalMesh->GetRefSkeleton(), MeshData->ExistingRefSkeleton, bImportSkinningOnly))
			{
				// First create mapping table from old skeleton to new skeleton.
				TArray<int32> OldToNewMap;
				OldToNewMap.AddUninitialized(MeshData->ExistingRefSkeleton.GetRawBoneNum());
				for (int32 i = 0; i < MeshData->ExistingRefSkeleton.GetRawBoneNum(); i++)
				{
					OldToNewMap[i] = SkeletalMesh->GetRefSkeleton().FindBoneIndex(MeshData->ExistingRefSkeleton.GetBoneName(i));
				}

				//Starting at index 1 because we only need to add LOD models of LOD 1 and higher.
				for (int32 LODIndex = 1; LODIndex < MeshData->ExistingLODModels.Num(); ++LODIndex)
				{
					FSkeletalMeshLODModel* LODModelCopy = FSkeletalMeshLODModel::CreateCopy(&MeshData->ExistingLODModels[LODIndex]);
					const FSkeletalMeshLODInfo& LODInfo = MeshData->ExistingLODInfo[LODIndex];

					// Fix ActiveBoneIndices array.
					bool bMissingBone = false;
					FName MissingBoneName = NAME_None;
					for (int32 j = 0; j < LODModelCopy->ActiveBoneIndices.Num() && !bMissingBone; j++)
					{
						int32 OldActiveBoneIndex = LODModelCopy->ActiveBoneIndices[j];
						if (OldToNewMap.IsValidIndex(OldActiveBoneIndex))
						{
							int32 NewBoneIndex = OldToNewMap[OldActiveBoneIndex];
							if (NewBoneIndex == INDEX_NONE)
							{
								bMissingBone = true;
								MissingBoneName = MeshData->ExistingRefSkeleton.GetBoneName(LODModelCopy->ActiveBoneIndices[j]);
							}
							else
							{
								LODModelCopy->ActiveBoneIndices[j] = IntCastChecked<FBoneIndexType>(NewBoneIndex);
							}
						}
						else
						{
							LODModelCopy->ActiveBoneIndices.RemoveAt(j, EAllowShrinking::No);
							--j;
						}
					}

					// Fix RequiredBones array.
					for (int32 j = 0; j < LODModelCopy->RequiredBones.Num() && !bMissingBone; j++)
					{
						const int32 OldBoneIndex = LODModelCopy->RequiredBones[j];

						if (OldToNewMap.IsValidIndex(OldBoneIndex))	//Previously virtual bones could end up in this array
																	// Must validate against this
						{
							const int32 NewBoneIndex = OldToNewMap[OldBoneIndex];
							if (NewBoneIndex == INDEX_NONE)
							{
								bMissingBone = true;
								MissingBoneName = MeshData->ExistingRefSkeleton.GetBoneName(OldBoneIndex);
							}
							else
							{
								LODModelCopy->RequiredBones[j] = IntCastChecked<FBoneIndexType>(NewBoneIndex);
							}
						}
						else
						{
							//Bone didn't exist in our required bones, clean up. 
							LODModelCopy->RequiredBones.RemoveAt(j, EAllowShrinking::No);
							--j;
						}
					}

					// Sort ascending for parent child relationship
					LODModelCopy->RequiredBones.Sort();
					SkeletalMesh->GetRefSkeleton().EnsureParentsExistAndSort(LODModelCopy->ActiveBoneIndices);

					// Fix the sections' BoneMaps.
					for (int32 SectionIndex = 0; SectionIndex < LODModelCopy->Sections.Num(); SectionIndex++)
					{
						FSkelMeshSection& Section = LODModelCopy->Sections[SectionIndex];
						for (int32 BoneIndex = 0; BoneIndex < Section.BoneMap.Num(); BoneIndex++)
						{
							int32 NewBoneIndex = OldToNewMap[Section.BoneMap[BoneIndex]];
							if (NewBoneIndex == INDEX_NONE)
							{
								bMissingBone = true;
								MissingBoneName = MeshData->ExistingRefSkeleton.GetBoneName(Section.BoneMap[BoneIndex]);
								break;
							}
							else
							{
								Section.BoneMap[BoneIndex] = IntCastChecked<FBoneIndexType>(NewBoneIndex);
							}
						}
						if (bMissingBone)
						{
							break;
						}
					}

					if (bMissingBone)
					{
						UnFbx::FFbxImporter* FFbxImporter = UnFbx::FFbxImporter::GetInstance();
						FFbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("NewMeshMissingBoneFromLOD", "New mesh is missing bone '{0}' required by an LOD."), FText::FromName(MissingBoneName))), FFbxErrors::SkeletalMesh_LOD_MissingBone);
						break;
					}
					else
					{
						//We need to add LODInfo
						SkeletalMeshImportedModel->LODModels.Add(LODModelCopy);
						SkeletalMesh->AddLODInfo(LODInfo);
						
						//Restore custom LOD import data
						FMeshDescription MeshDescription(MeshData->ExistingLODMeshDescriptions[LODIndex]);
						SkeletalMesh->CreateMeshDescription(LODIndex, MoveTemp(MeshDescription));

						auto FillInlineReductionData = [&MeshData, &SkeletalMeshImportedModel, LODIndex]()
						{
							if (MeshData->ExistingInlineReductionCacheData.IsValidIndex(LODIndex))
							{
								SkeletalMeshImportedModel->InlineReductionCacheDatas[LODIndex] = MeshData->ExistingInlineReductionCacheData[LODIndex];
							}
							else
							{
								SkeletalMeshImportedModel->InlineReductionCacheDatas[LODIndex].SetCacheGeometryInfo(SkeletalMeshImportedModel->LODModels[LODIndex]);
							}
						};

						if (!SkeletalMeshImportedModel->InlineReductionCacheDatas.IsValidIndex(LODIndex))
						{
							SkeletalMeshImportedModel->InlineReductionCacheDatas.AddDefaulted(LODIndex + 1 - SkeletalMeshImportedModel->InlineReductionCacheDatas.Num());
							check(SkeletalMeshImportedModel->InlineReductionCacheDatas.IsValidIndex(LODIndex));
							FillInlineReductionData();
						}
						else
						{
							uint32 LODVertexCount = MAX_uint32;
							uint32 LODTriangleCount = MAX_uint32;
							SkeletalMeshImportedModel->InlineReductionCacheDatas[LODIndex].GetCacheGeometryInfo(LODVertexCount, LODTriangleCount);
							if (LODVertexCount == MAX_uint32 || LODTriangleCount == MAX_uint32)
							{
								FillInlineReductionData();
							}
						}
					}
				}
			}
			//We just need to restore the LOD model and LOD info the build should regenerate the LODs
			RestoreDependentLODs(MeshData, SkeletalMesh);
			
			
			//Old asset cannot use the new build system, we need to regenerate dependent LODs
			if (!SkeletalMesh->HasMeshDescription(SafeReimportLODIndex))
			{
				FLODUtilities::RegenerateDependentLODs(SkeletalMesh, SafeReimportLODIndex, GetTargetPlatformManagerRef().GetRunningTargetPlatform());
			}
		}

		for (int32 AssetIndex = 0; AssetIndex < MeshData->ExistingPhysicsAssets.Num(); ++AssetIndex)
		{
			UPhysicsAsset* PhysicsAsset = MeshData->ExistingPhysicsAssets[AssetIndex];
			if (AssetIndex == 0)
			{
				// First asset is the one that the skeletal mesh should point too
				SkeletalMesh->SetPhysicsAsset(PhysicsAsset);
			}
			// No need to mark as modified here, because the asset hasn't actually changed
			if (PhysicsAsset)
			{
				PhysicsAsset->PreviewSkeletalMesh = SkeletalMesh;
			}
		}

		SkeletalMesh->SetShadowPhysicsAsset(MeshData->ExistingShadowPhysicsAsset);
		SkeletalMesh->SetSkeleton(MeshData->ExistingSkeleton);
		SkeletalMesh->SetPostProcessAnimBlueprint(MeshData->ExistingPostProcessAnimBlueprint);
		SkeletalMesh->SetDefaultAnimatingRig(MeshData->ExistingDefaultAnimatingRig);
		SkeletalMesh->SetDefaultMeshDeformer(MeshData->ExistingDefaultMeshDeformer);
		SkeletalMesh->SetTargetMeshDeformers(MeshData->ExistingTargetMeshDeformers);

		SkeletalMesh->GetMorphTargets().Empty(MeshData->ExistingMorphTargets.Num());
		SkeletalMesh->GetMorphTargets().Append(MeshData->ExistingMorphTargets);
		SkeletalMesh->InitMorphTargets();
		SkeletalMesh->SetAssetImportData(MeshData->ExistingAssetImportData.Get());
		SkeletalMesh->SetThumbnailInfo(MeshData->ExistingThumbnailInfo.Get());
		SkeletalMesh->SetMeshClothingAssets(MeshData->ExistingClothingAssets);

		for (UClothingAssetBase* ClothingAsset : SkeletalMesh->GetMeshClothingAssets())
		{
			if (ClothingAsset)
			{
				ClothingAsset->RefreshBoneMapping(SkeletalMesh);
			}
		}

		SkeletalMesh->SetSamplingInfo(MeshData->ExistingSamplingInfo);
	}

	//Restore the section change only for the reimport LOD, other LOD are not affected since the material array can only grow.
	if (MeshData->UseMaterialNameSlotWorkflow)
	{
		RestoreMaterialNameWorkflowSection(MeshData, SkeletalMesh, SafeReimportLODIndex, RemapMaterial, bMaterialReset);
	}

	//Copy back the reimported LOD's specific data
	if (SafeReimportLODIndex >= 0 && SafeReimportLODIndex < SkeletalMesh->GetLODNum())
	{
		RestoreLODInfo(MeshData, SkeletalMesh, SafeReimportLODIndex);
	}

	// Copy user data to newly created mesh
	for (auto Kvp : MeshData->ExistingAssetUserData)
	{
		UAssetUserData* UserDataObject = Kvp.Key;
		if (Kvp.Value)
		{
			//if the duplicated temporary UObject was add to root, we must remove it from the root
			UserDataObject->RemoveFromRoot();
		}
		UserDataObject->Rename(nullptr, SkeletalMesh, REN_DontCreateRedirectors | REN_DoNotDirty);
		SkeletalMesh->AddAssetUserData(UserDataObject);
	}

	//Copy mesh changed delegate data
	SkeletalMesh->GetOnMeshChanged() = MeshData->ExistingOnMeshChanged;

	//Copy mesh bounds extensions
	SkeletalMesh->SetPositiveBoundsExtension(MeshData->PositiveBoundsExtension);
	SkeletalMesh->SetNegativeBoundsExtension(MeshData->NegativeBoundsExtension);

	SkeletalMesh->SetSupportRayTracing(MeshData->bExistingSupportRayTracing);
	SkeletalMesh->SetRayTracingMinLOD(MeshData->ExistingRayTracingMinLOD);
	SkeletalMesh->SetClothLODBiasMode(MeshData->ExistingClothLODBiasMode);
}

void SkeletalMesUtilsImpl::RestoreMaterialNameWorkflowSection(const TSharedPtr<const FExistingSkelMeshData>& MeshData, USkeletalMesh* SkeletalMesh, int32 LodIndex, TArray<int32>& RemapMaterial, bool bMaterialReset)
{
	FSkeletalMeshModel* SkeletalMeshImportedModel = SkeletalMesh->GetImportedModel();
	FSkeletalMeshLODModel &SkeletalMeshLodModel = SkeletalMeshImportedModel->LODModels[LodIndex];

	//Restore the base LOD materialMap the LODs LODMaterialMap are restore differently
	if (LodIndex == 0 && SkeletalMesh->GetLODNum() > 0)
	{
		FSkeletalMeshLODInfo* BaseLODInfo = SkeletalMesh->GetLODInfo(LodIndex);
		if (bMaterialReset)
		{
			//If we reset the material array there is no point keeping the user changes
			BaseLODInfo->LODMaterialMap.Empty();
		}
		else if (SkeletalMeshImportedModel->LODModels.IsValidIndex(LodIndex))
		{
			//Restore the Base MaterialMap
			for (int32 SectionIndex = 0; SectionIndex < SkeletalMeshLodModel.Sections.Num(); ++SectionIndex)
			{
				int32 MaterialIndex = SkeletalMeshLodModel.Sections[SectionIndex].MaterialIndex;
				if (MeshData->ExistingLODInfo[LodIndex].LODMaterialMap.IsValidIndex(SectionIndex))
				{
					int32 ExistingLODMaterialIndex = MeshData->ExistingLODInfo[LodIndex].LODMaterialMap[SectionIndex];
					while (BaseLODInfo->LODMaterialMap.Num() <= SectionIndex)
					{
						BaseLODInfo->LODMaterialMap.Add(INDEX_NONE);
					}
					BaseLODInfo->LODMaterialMap[SectionIndex] = ExistingLODMaterialIndex;
				}
			}
		}
	}

	const bool bIsValidSavedSectionMaterialData = MeshData->ExistingImportMeshLodSectionMaterialData.IsValidIndex(LodIndex) && MeshData->LastImportMeshLodSectionMaterialData.IsValidIndex(LodIndex);
	const int32 MaxExistSectionNumber = bIsValidSavedSectionMaterialData ? FMath::Max(MeshData->ExistingImportMeshLodSectionMaterialData[LodIndex].Num(), MeshData->LastImportMeshLodSectionMaterialData[LodIndex].Num()) : 0;
	TBitArray<> MatchedExistSectionIndex;
	MatchedExistSectionIndex.Init(false, MaxExistSectionNumber);

	const TArray<FSkeletalMaterial>& SkeletalMeshMaterials = SkeletalMesh->GetMaterials();
	//Restore the section changes from the old import data
	for (int32 SectionIndex = 0; SectionIndex < SkeletalMeshLodModel.Sections.Num(); SectionIndex++)
	{
		//Find the import section material index by using the RemapMaterial array. Fallback on the imported index if the remap entry is not valid
		FSkelMeshSection& NewSection = SkeletalMeshLodModel.Sections[SectionIndex];
		int32 RemapMaterialIndex = RemapMaterial.IsValidIndex(NewSection.MaterialIndex) ? RemapMaterial[NewSection.MaterialIndex] : NewSection.MaterialIndex;
		if (!SkeletalMeshMaterials.IsValidIndex(RemapMaterialIndex))
		{
			//We have an invalid material section, in this case we set the material index to 0
			NewSection.MaterialIndex = 0;
			UE_LOGF(LogSkeletalMeshImport, Display, "Reimport material match issue: Invalid RemapMaterialIndex [%d], will make it point to material index [0]", RemapMaterialIndex);
			continue;
		}
		NewSection.MaterialIndex = RemapMaterialIndex;

		//skip the rest of the loop if we do not have valid saved data
		if (!bIsValidSavedSectionMaterialData)
		{
			continue;
		}
		//Get the RemapMaterial section Imported material slot name. We need it to match the saved existing section, so we can put back the saved existing section data
		FName CurrentSectionImportedMaterialName = SkeletalMeshMaterials[RemapMaterialIndex].ImportedMaterialSlotName;
		for (int32 ExistSectionIndex = 0; ExistSectionIndex < MaxExistSectionNumber; ++ExistSectionIndex)
		{
			//Skip already matched exist section
			if (MatchedExistSectionIndex[ExistSectionIndex])
			{
				continue;
			}
			//Verify we have valid existing section data, if not break from the loop higher index wont be valid
			if (!MeshData->LastImportMeshLodSectionMaterialData[LodIndex].IsValidIndex(ExistSectionIndex) || !MeshData->ExistingImportMeshLodSectionMaterialData[LodIndex].IsValidIndex(ExistSectionIndex))
			{
				break;
			}

			//Get the Last imported skelmesh section slot import name
			FName OriginalImportMeshSectionSlotName = MeshData->LastImportMeshLodSectionMaterialData[LodIndex][ExistSectionIndex];
			if (OriginalImportMeshSectionSlotName != CurrentSectionImportedMaterialName)
			{
				//Skip until we found a match between the last import
				continue;
			}

			//We have a match put back the data
			NewSection.bCastShadow = MeshData->ExistingImportMeshLodSectionMaterialData[LodIndex][ExistSectionIndex].bCastShadow;
			NewSection.bVisibleInRayTracing = MeshData->ExistingImportMeshLodSectionMaterialData[LodIndex][ExistSectionIndex].bVisibleInRayTracing;
			NewSection.bRecomputeTangent = MeshData->ExistingImportMeshLodSectionMaterialData[LodIndex][ExistSectionIndex].bRecomputeTangents;
			NewSection.RecomputeTangentsVertexMaskChannel = MeshData->ExistingImportMeshLodSectionMaterialData[LodIndex][ExistSectionIndex].RecomputeTangentsVertexMaskChannel;
			NewSection.GenerateUpToLodIndex = MeshData->ExistingImportMeshLodSectionMaterialData[LodIndex][ExistSectionIndex].GenerateUpTo;
			NewSection.bDisabled = MeshData->ExistingImportMeshLodSectionMaterialData[LodIndex][ExistSectionIndex].bDisabled;
			bool bBoneChunkedSection = NewSection.ChunkedParentSectionIndex >= 0;
			int32 ParentOriginalSectionIndex = NewSection.OriginalDataSectionIndex;
			if (!bBoneChunkedSection)
			{
				//Set the new Parent Index
				FSkelMeshSourceSectionUserData& UserSectionData = SkeletalMeshLodModel.UserSectionsData.FindOrAdd(ParentOriginalSectionIndex);
				UserSectionData.bDisabled = NewSection.bDisabled;
				UserSectionData.bCastShadow = NewSection.bCastShadow;
				UserSectionData.bVisibleInRayTracing = NewSection.bVisibleInRayTracing;
				UserSectionData.bRecomputeTangent = NewSection.bRecomputeTangent;
				UserSectionData.RecomputeTangentsVertexMaskChannel = NewSection.RecomputeTangentsVertexMaskChannel;
				UserSectionData.GenerateUpToLodIndex = NewSection.GenerateUpToLodIndex;
				//The cloth will be rebind later after the reimport is done
			}
			//Set the matched section to true to avoid using it again
			MatchedExistSectionIndex[ExistSectionIndex] = true;

			//find the corresponding current slot name in the skeletal mesh materials list to remap properly the material index, in case the user have change it before re-importing
			FName ExistMeshSectionSlotName = MeshData->ExistingImportMeshLodSectionMaterialData[LodIndex][ExistSectionIndex].ImportedMaterialSlotName;
			{
				for (int32 SkelMeshMaterialIndex = 0; SkelMeshMaterialIndex < SkeletalMeshMaterials.Num(); ++SkelMeshMaterialIndex)
				{
					const FSkeletalMaterial &NewSectionMaterial = SkeletalMeshMaterials[SkelMeshMaterialIndex];
					if (NewSectionMaterial.ImportedMaterialSlotName == ExistMeshSectionSlotName)
					{
						if (ExistMeshSectionSlotName != OriginalImportMeshSectionSlotName)
						{
							NewSection.MaterialIndex = SkelMeshMaterialIndex;
						}
						break;
					}
				}
			}
			//Break because we found a match and have restore the data for this SectionIndex
			break;
		}
	}
	//Make sure we reset the User section array to only what we have in the fbx
	SkeletalMeshLodModel.SyncronizeUserSectionsDataArray(true);
}

#undef LOCTEXT_NAMESPACE
