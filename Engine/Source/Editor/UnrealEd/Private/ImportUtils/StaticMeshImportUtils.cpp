// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	StaticMeshImportUtils.cpp: Static mesh edit functions.
=============================================================================*/

#include "ImportUtils/StaticMeshImportUtils.h"

#include "BSPOps.h"
#include "EditorFramework/ThumbnailInfo.h"
#include "Engine/StaticMeshSocket.h"
#include "Engine/StaticMesh.h"
#include "Factories/FbxMeshImportData.h"
#include "Factories/FbxStaticMeshImportData.h"
#include "FbxImporter.h"
#include "IMeshReductionInterfaces.h"
#include "IMeshReductionManagerModule.h"
#include "ImportUtils/InternalImportUtils.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Interfaces/ITargetPlatform.h"
#include "MeshDescription.h"
#include "Model.h"
#include "Modules/ModuleManager.h"
#include "PhysicsEngine/AggregateGeom.h"
#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/BoxElem.h"
#include "PhysicsEngine/ConvexElem.h"
#include "PhysicsEngine/SphereElem.h"
#include "PhysicsEngine/SphylElem.h"
#include "UObject/MetaData.h"

DEFINE_LOG_CATEGORY_STATIC(LogStaticMeshImportUtils, Log, All);

#define LOCTEXT_NAMESPACE "StaticMeshImportUtils"

static float MeshToPrimTolerance = 0.001f;

/** Floating point comparitor */
static FORCEINLINE bool AreEqual(float a, float b)
{
	return FMath::Abs(a - b) < MeshToPrimTolerance;
}

/** Returns 1 if vectors are parallel OR anti-parallel */
static FORCEINLINE bool AreParallel(const FVector3f& a, const FVector3f& b)
{
	float Dot = a | b;

	if( AreEqual(FMath::Abs(Dot), 1.f) )
	{
		return true;
	}
	else
	{
		return false;
	}
}

/** Utility struct used in AddBoxGeomFromTris. */
struct FPlaneInfo
{
	FVector3f Normal;
	int32 DistCount;
	float PlaneDist[2];

	FPlaneInfo()
	{
		Normal = FVector3f::ZeroVector;
		DistCount = 0;
		PlaneDist[0] = 0.f;
		PlaneDist[1] = 0.f;
	}
};

struct FMeshConnectivityVertex
{
	FVector3f				Position;
	TArray<int32>		Triangles;

	/** Constructor */
	FMeshConnectivityVertex( const FVector3f &v )
		: Position( v )
	{
	}

	/** Check if this vertex is in the same place as given point */
	FORCEINLINE bool IsSame( const FVector3f &v )
	{
		const float eps = 0.01f;
		return v.Equals( Position, eps );
	}

	/** Add link to triangle */
	FORCEINLINE void AddTriangleLink( int32 Triangle )
	{
		Triangles.Add( Triangle );
	}
};

struct FMeshConnectivityTriangle
{
	int32				Vertices[3];
	int32				Group;

	/** Constructor */
	FMeshConnectivityTriangle( int32 a, int32 b, int32 c )
		: Group( INDEX_NONE )
	{
		Vertices[0] = a;
		Vertices[1] = b;
		Vertices[2] = c;
	}
};

struct FMeshConnectivityGroup
{
	TArray<int32>		Triangles;
};

class FMeshConnectivityBuilder
{
public:
	TArray<FMeshConnectivityVertex>		Vertices;
	TArray<FMeshConnectivityTriangle>	Triangles;
	TArray<FMeshConnectivityGroup>		Groups;

public:
	/** Add vertex to connectivity information */
	int32 AddVertex( const FVector3f &v )
	{
		// Try to find existing vertex
		// TODO: should use hash map
		for ( int32 i=0; i<Vertices.Num(); ++i )
		{
			if ( Vertices[i].IsSame( v ) )
			{
				return i;
			}
		}

		// Add new vertex
		new ( Vertices ) FMeshConnectivityVertex( v );
		return Vertices.Num() - 1;
	}

	/** Add triangle to connectivity information */
	int32 AddTriangle( const FVector3f &a, const FVector3f &b, const FVector3f &c )
	{
		// Map vertices
		int32 VertexA = AddVertex( a );
		int32 VertexB = AddVertex( b );
		int32 VertexC = AddVertex( c );

		// Make sure triangle is not degenerated
		if ( VertexA!=VertexB && VertexB!=VertexC && VertexC!=VertexA )
		{
			// Setup connectivity info
			int32 TriangleIndex = Triangles.Num();
			Vertices[ VertexA ].AddTriangleLink( TriangleIndex );
			Vertices[ VertexB ].AddTriangleLink( TriangleIndex );
			Vertices[ VertexC ].AddTriangleLink( TriangleIndex );

			// Create triangle
			new ( Triangles ) FMeshConnectivityTriangle( VertexA, VertexB, VertexC );
			return TriangleIndex;
		}
		else
		{
			// Degenerated triangle
			return INDEX_NONE;
		}
	}

	/** Create connectivity groups */
	void CreateConnectivityGroups()
	{
		// Delete group list
		Groups.Empty();

		// Reset group assignments
		for ( int32 i=0; i<Triangles.Num(); i++ )
		{
			Triangles[i].Group = INDEX_NONE;
		}

		// Flood fill using connectivity info
		for ( ;; )
		{
			// Find first triangle without group assignment
			int32 InitialTriangle = INDEX_NONE;
			for ( int32 i=0; i<Triangles.Num(); i++ )
			{
				if ( Triangles[i].Group == INDEX_NONE )
				{
					InitialTriangle = i;
					break;
				}
			}

			// No more unassigned triangles, flood fill is done
			if ( InitialTriangle == INDEX_NONE )
			{
				break;
			}

			// Create group
			int32 GroupIndex = Groups.AddZeroed( 1 );

			// Start flood fill using connectivity information
			FloodFillTriangleGroups( InitialTriangle, GroupIndex );
		}
	}

private:
	/** FloodFill core */
	void FloodFillTriangleGroups( int32 InitialTriangleIndex, int32 GroupIndex )
	{
		TArray<int32> TriangleStack;

		// Start with given triangle
		TriangleStack.Add( InitialTriangleIndex );

		// Set the group for our first triangle
		Triangles[ InitialTriangleIndex ].Group = GroupIndex;

		// Process until we have triangles in stack
		while ( TriangleStack.Num() )
		{
			// Pop triangle index from stack
			int32 TriangleIndex = TriangleStack.Pop();

			FMeshConnectivityTriangle &Triangle = Triangles[ TriangleIndex ];

			// All triangles should already have a group before we start processing neighbors
			checkSlow( Triangle.Group == GroupIndex );

			// Add to list of triangles in group
			Groups[ GroupIndex ].Triangles.Add( TriangleIndex );

			// Recurse to all other triangles connected with this one
			for ( int32 i=0; i<3; i++ )
			{
				int32 VertexIndex = Triangle.Vertices[ i ];
				const FMeshConnectivityVertex &Vertex = Vertices[ VertexIndex ];

				for ( int32 j=0; j<Vertex.Triangles.Num(); j++ )
				{
					int32 OtherTriangleIndex = Vertex.Triangles[ j ];
					FMeshConnectivityTriangle &OtherTriangle = Triangles[ OtherTriangleIndex ];

					// Only recurse if triangle was not already assigned to a group
					if ( OtherTriangle.Group == INDEX_NONE )
					{
						// OK, the other triangle now belongs to our group!
						OtherTriangle.Group = GroupIndex;

						// Add the other triangle to the stack to be processed
						TriangleStack.Add( OtherTriangleIndex );
					}
				}
			}
		}
	}
};

bool StaticMeshImportUtils::DecomposeUCXMesh( const TArray<FVector3f>& CollisionVertices, const TArray<int32>& CollisionFaceIdx, UBodySetup* BodySetup )
{
	// We keep no ref to this Model, so it will be GC'd at some point after the import.
	auto TempModel = NewObject<UModel>();
	TempModel->Initialize(nullptr, 1);

	FMeshConnectivityBuilder ConnectivityBuilder;

	// Send triangles to connectivity builder
	for(int32 x = 0;x < CollisionFaceIdx.Num();x += 3)
	{
		const FVector3f &VertexA = CollisionVertices[ CollisionFaceIdx[x + 2] ];
		const FVector3f &VertexB = CollisionVertices[ CollisionFaceIdx[x + 1] ];
		const FVector3f &VertexC = CollisionVertices[ CollisionFaceIdx[x + 0] ];
		ConnectivityBuilder.AddTriangle( VertexA, VertexB, VertexC );
	}

	ConnectivityBuilder.CreateConnectivityGroups();

	// For each valid group build BSP and extract convex hulls
	bool bSuccess = true;
	for ( int32 i=0; i<ConnectivityBuilder.Groups.Num(); i++ )
	{
		const FMeshConnectivityGroup &Group = ConnectivityBuilder.Groups[ i ];

		// TODO: add some BSP friendly checks here
		// e.g. if group triangles form a closed mesh

		// Generate polygons from group triangles
		TempModel->Polys->Element.Empty();

		for ( int32 j=0; j<Group.Triangles.Num(); j++ )
		{
			const FMeshConnectivityTriangle &Triangle = ConnectivityBuilder.Triangles[ Group.Triangles[j] ];

			FPoly*	Poly = new( TempModel->Polys->Element ) FPoly();
			Poly->Init();
			Poly->iLink = j / 3;

			// Add vertices
			new( Poly->Vertices ) FVector3f( ConnectivityBuilder.Vertices[ Triangle.Vertices[0] ].Position );
			new( Poly->Vertices ) FVector3f( ConnectivityBuilder.Vertices[ Triangle.Vertices[1] ].Position );
			new( Poly->Vertices ) FVector3f( ConnectivityBuilder.Vertices[ Triangle.Vertices[2] ].Position );

			// Update polygon normal
			Poly->CalcNormal(1);
		}

		// Build bounding box.
		TempModel->BuildBound();

		// Build BSP for the brush.
		FBSPOps::bspBuild( TempModel,FBSPOps::BSP_Good,15,70,1,0 );
		FBSPOps::bspRefresh( TempModel, 1 );
		FBSPOps::bspBuildBounds( TempModel );

		// Convert collision model into a collection of convex hulls.
		// Generated convex hulls will be added to existing ones
		bSuccess = BodySetup->CreateFromModel( TempModel, false ) && bSuccess;
	}

	// Could all meshes be properly decomposed?
	return bSuccess;
}

/** 
 *	Function for adding a box collision primitive to the supplied collision geometry based on the mesh of the box.
 * 
 *	We keep a list of triangle normals found so far. For each normal direction,
 *	we should have 2 distances from the origin (2 parallel box faces). If the 
 *	mesh is a box, we should have 3 distinct normal directions, and 2 distances
 *	found for each. The difference between these distances should be the box
 *	dimensions. The 3 directions give us the key axes, and therefore the
 *	box transformation matrix. This shouldn't rely on any vertex-ordering on 
 *	the triangles (normals are compared +ve & -ve). It also shouldn't matter 
 *	about how many triangles make up each side (but it will take longer). 
 *	We get the centre of the box from the centre of its AABB.
 */
bool StaticMeshImportUtils::AddBoxGeomFromTris( const TArray<FPoly>& Tris, FKAggregateGeom* AggGeom, const TCHAR* ObjName )
{
	TArray<FPlaneInfo> Planes;

	for(int32 i=0; i<Tris.Num(); i++)
	{
		bool bFoundPlane = false;
		for(int32 j=0; j<Planes.Num() && !bFoundPlane; j++)
		{
			// if this triangle plane is already known...
			if( AreParallel( Tris[i].Normal, Planes[j].Normal ) )
			{
				// Always use the same normal when comparing distances, to ensure consistent sign.
				float Dist = Tris[i].Vertices[0] | Planes[j].Normal;

				// we only have one distance, and its not that one, add it.
				if( Planes[j].DistCount == 1 && !AreEqual(Dist, Planes[j].PlaneDist[0]) )
				{
					Planes[j].PlaneDist[1] = Dist;
					Planes[j].DistCount = 2;
				}
				// if we have a second distance, and its not that either, something is wrong.
				else if( Planes[j].DistCount == 2 && !AreEqual(Dist, Planes[j].PlaneDist[1]) )
				{
					UE_LOGF(LogStaticMeshImportUtils, Log, "AddBoxGeomFromTris (%ls): Found more than 2 planes with different distances.", ObjName);
					return false;
				}

				bFoundPlane = true;
			}
		}

		// If this triangle does not match an existing plane, add to list.
		if(!bFoundPlane)
		{
			check( Planes.Num() < Tris.Num() );

			FPlaneInfo NewPlane;
			NewPlane.Normal = Tris[i].Normal;
			NewPlane.DistCount = 1;
			NewPlane.PlaneDist[0] = Tris[i].Vertices[0] | NewPlane.Normal;
			
			Planes.Add(NewPlane);
		}
	}

	// Now we have our candidate planes, see if there are any problems

	// Wrong number of planes.
	if(Planes.Num() != 3)
	{
		UE_LOGF(LogStaticMeshImportUtils, Log, "AddBoxGeomFromTris (%ls): Not very box-like (need 3 sets of planes).", ObjName);
		return false;
	}

	// If we don't have 3 pairs, we can't carry on.
	if((Planes[0].DistCount != 2) || (Planes[1].DistCount != 2) || (Planes[2].DistCount != 2))
	{
		UE_LOGF(LogStaticMeshImportUtils, Log, "AddBoxGeomFromTris (%ls): Incomplete set of planes (need 2 per axis).", ObjName);
		return false;
	}

	FMatrix BoxTM = FMatrix::Identity;

	FVector3f Axis[3] = {{1.0f, 0.0f, 0.0f}, {0.0f, 1.0f, 0.0f}, {0.0f, 0.0f, 1.0f}};
	int32 Reorder[3] = {INDEX_NONE, INDEX_NONE, INDEX_NONE };
	for (int32 PlaneIndex = 0; PlaneIndex < 3; ++PlaneIndex)
	{
		for (int32 AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
		{
			if(AreParallel(Planes[PlaneIndex].Normal, Axis[AxisIndex]))
			{
				Reorder[PlaneIndex] = AxisIndex;
				break;
			}
		}
	}
	
	if (Reorder[0] == INDEX_NONE || Reorder[1] == INDEX_NONE || Reorder[2] == INDEX_NONE)
	{
		Reorder[0] = 0;
		Reorder[1] = 1;
		Reorder[2] = 2;
	}

	BoxTM.SetAxis(0, (FVector)Planes[Reorder[0]].Normal);
	BoxTM.SetAxis(1, (FVector)Planes[Reorder[1]].Normal);

	// ensure valid TM by cross-product
	FVector3f ZAxis = Planes[Reorder[0]].Normal ^ Planes[Reorder[1]].Normal;

	if (!AreParallel(ZAxis, Planes[Reorder[2]].Normal))
	{
		UE_LOGF(LogStaticMeshImportUtils, Log, "AddBoxGeomFromTris (%ls): Box axes are not perpendicular.", ObjName);
		return false;
	}

	BoxTM.SetAxis(2, (FVector)ZAxis);

	// OBB centre == AABB centre.
	FBox Box(ForceInit);
	for(int32 i=0; i<Tris.Num(); i++)
	{
		Box += (FVector)Tris[i].Vertices[0];
		Box += (FVector)Tris[i].Vertices[1];
		Box += (FVector)Tris[i].Vertices[2];
	}

	BoxTM.SetOrigin( Box.GetCenter() );

	// Allocate box in array
	FKBoxElem BoxElem;
	BoxElem.SetTransform( FTransform( BoxTM ) );	
	// distance between parallel planes is box edge lengths.
	BoxElem.X = FMath::Abs(Planes[Reorder[0]].PlaneDist[0] - Planes[Reorder[0]].PlaneDist[1]);
	BoxElem.Y = FMath::Abs(Planes[Reorder[1]].PlaneDist[0] - Planes[Reorder[1]].PlaneDist[1]);
	BoxElem.Z = FMath::Abs(Planes[Reorder[2]].PlaneDist[0] - Planes[Reorder[2]].PlaneDist[1]);
	AggGeom->BoxElems.Add(BoxElem);

	return true;
}

/**
 *	Function for adding a sphere collision primitive to the supplied collision geometry based on a set of Verts.
 *	
 *	Simply put an AABB around mesh and use that to generate centre and radius.
 *	It checks that the AABB is square, and that all vertices are either at the
 *	centre, or within 5% of the radius distance away.
 */
bool StaticMeshImportUtils::AddSphereGeomFromVerts( const TArray<FVector3f>& Verts, FKAggregateGeom* AggGeom, const TCHAR* ObjName )
{
	if(Verts.Num() == 0)
	{
		return false;
	}

	FBox Box(ForceInit);

	for(int32 i=0; i<Verts.Num(); i++)
	{
		Box += (FVector)Verts[i];
	}

	FVector Center, Extents;
	Box.GetCenterAndExtents(Center, Extents);
	double Longest = 2.f * Extents.GetMax();
	double Shortest = 2.f * Extents.GetMin();

	// check that the AABB is roughly a square (5% tolerance)
	if((Longest - Shortest) / Longest > 0.05f)
	{
		UE_LOGF(LogStaticMeshImportUtils, Log, "AddSphereGeomFromVerts (%ls): Sphere bounding box not square.", ObjName);
		return false;
	}

	float Radius = 0.5f * Longest;

	// Test that all vertices are a similar radius (5%) from the sphere centre.
	float MaxR = 0;
	float MinR = BIG_NUMBER;
	for(int32 i=0; i<Verts.Num(); i++)
	{
		FVector3f CToV = Verts[i] - (FVector3f)Center;
		float RSqr = CToV.SizeSquared();

		MaxR = FMath::Max(RSqr, MaxR);

		// Sometimes vertex at centre, so reject it.
		if(RSqr > KINDA_SMALL_NUMBER)
		{
			MinR = FMath::Min(RSqr, MinR);
		}
	}

	MaxR = FMath::Sqrt(MaxR);
	MinR = FMath::Sqrt(MinR);

	if((MaxR-MinR)/Radius > 0.05f)
	{
		UE_LOGF(LogStaticMeshImportUtils, Log, "AddSphereGeomFromVerts (%ls): Vertices not at constant radius.", ObjName );
		return false;
	}

	// Allocate sphere in array
	FKSphereElem SphereElem;
	SphereElem.Center = Center;
	SphereElem.Radius = Radius;
	AggGeom->SphereElems.Add(SphereElem);
	
	return true;
}

bool StaticMeshImportUtils::AddCapsuleGeomFromVerts(const TArray<FVector3f>& Verts, FKAggregateGeom* AggGeom, const TCHAR* ObjName)
{
	if (Verts.Num() < 3)
	{
		return false;
	}

	FVector3f AxisStart = FVector3f::ZeroVector;
	FVector3f AxisEnd = FVector3f::ZeroVector;
	float MaxDistSqr = 0.f;

	for (int32 IndexA = 0; IndexA < Verts.Num() - 1; IndexA++)
	{
		for (int32 IndexB = IndexA + 1; IndexB < Verts.Num(); IndexB++)
		{
			float DistSqr = (Verts[IndexA] - Verts[IndexB]).SizeSquared();
			if (DistSqr > MaxDistSqr)
			{
				AxisStart = Verts[IndexA];
				AxisEnd = Verts[IndexB];
				MaxDistSqr = DistSqr;
			}
		}
	}

	// If we got a valid axis, find vertex furthest from it
	if (MaxDistSqr > SMALL_NUMBER)
	{
		float MaxRadius = 0.f;

		const FVector3f LineOrigin = AxisStart;
		const FVector3f LineDir = (AxisEnd - AxisStart).GetSafeNormal();

		for (int32 IndexA = 0; IndexA < Verts.Num() - 1; IndexA++)
		{
			float DistToAxis = FMath::PointDistToLine((FVector)Verts[IndexA], (FVector)LineDir, (FVector)LineOrigin);
			if (DistToAxis > MaxRadius)
			{
				MaxRadius = DistToAxis;
			}
		}

		if (MaxRadius > SMALL_NUMBER)
		{
			// Allocate capsule in array
			FKSphylElem SphylElem;
			SphylElem.Center = 0.5f * (FVector)(AxisStart + AxisEnd);
			SphylElem.Rotation = FQuat::FindBetweenVectors(FVector(0,0,1), (FVector)LineDir).Rotator(); // Get quat that takes you from z axis to desired axis
			SphylElem.Radius = MaxRadius;
			SphylElem.Length = FMath::Max(FMath::Sqrt(MaxDistSqr) - (2.f * MaxRadius), 0.f); // subtract two radii from total length to get segment length (ensure > 0)
			AggGeom->SphylElems.Add(SphylElem);
			return true;
		}
	}

	return false;
}


/** Utility for adding one convex hull from the given verts */
bool StaticMeshImportUtils::AddConvexGeomFromVertices( const TArray<FVector3f>& Verts, FKAggregateGeom* AggGeom, const TCHAR* ObjName )
{
	if(Verts.Num() == 0)
	{
		return false;
	}

	FKConvexElem* ConvexElem = new(AggGeom->ConvexElems) FKConvexElem();
	ConvexElem->VertexData = UE::LWC::ConvertArrayType<FVector>(Verts);	// LWC_TODO: Perf pessimization
	ConvexElem->UpdateElemBox();

	return true;
}

TSharedPtr<FExistingStaticMeshData> StaticMeshImportUtils::SaveExistingStaticMeshData(UStaticMesh* ExistingMesh, bool bImportMaterials, int32 LodIndex)
{
	UnFbx::FBXImportOptions ImportOptions;
	ImportOptions.bImportMaterials = bImportMaterials;
	return SaveExistingStaticMeshData(ExistingMesh, &ImportOptions, LodIndex);
}

TSharedPtr<FExistingStaticMeshData> StaticMeshImportUtils::SaveExistingStaticMeshData(UStaticMesh* ExistingMesh, UnFbx::FBXImportOptions* ImportOptions, int32 LodIndex)
{
	if (!ExistingMesh)
	{
		return TSharedPtr<FExistingStaticMeshData>();
	}

	bool bSaveMaterials = !ImportOptions->bImportMaterials;
	TSharedPtr<FExistingStaticMeshData> ExistingMeshDataPtr = MakeShared<FExistingStaticMeshData>();

	//Save the package FMetaData
	TMap<FName, FString>* ExistingUMetaDataTagValues = FMetaData::GetMapForObject(ExistingMesh);
	if(ExistingUMetaDataTagValues && ExistingUMetaDataTagValues->Num() > 0)
	{
		ExistingMeshDataPtr->ExistingUMetaDataTagValues = *ExistingUMetaDataTagValues;
	}

	ExistingMeshDataPtr->ImportVersion = ExistingMesh->GetImportVersion();
	ExistingMeshDataPtr->UseMaterialNameSlotWorkflow = InternalImportUtils::IsUsingMaterialSlotNameWorkflow(ExistingMesh->GetAssetImportData());

	FMeshSectionInfoMap OldSectionInfoMap = ExistingMesh->GetSectionInfoMap();

	bool bIsReimportCustomLODOverGeneratedLOD = ExistingMesh->IsSourceModelValid(LodIndex) &&
		(ExistingMesh->GetSourceModel(LodIndex).IsRawMeshEmpty() || (ExistingMesh->IsReductionActive(LodIndex) && ExistingMesh->GetSourceModel(LodIndex).ReductionSettings.BaseLODModel != LodIndex));

	//We need to reset some data in case we import a custom LOD over a generated LOD
	if (bIsReimportCustomLODOverGeneratedLOD)
	{
		//Reset the section info map for this LOD
		for (int32 SectionIndex = 0; SectionIndex < ExistingMesh->GetNumSections(LodIndex); ++SectionIndex)
		{
			OldSectionInfoMap.Remove(LodIndex, SectionIndex);
		}
	}

	ExistingMeshDataPtr->ExistingMaterials.Empty();
	if (bSaveMaterials)
	{
		for (const FStaticMaterial &StaticMaterial : ExistingMesh->GetStaticMaterials())
		{
			ExistingMeshDataPtr->ExistingMaterials.Add(StaticMaterial);
		}
	}

	ExistingMeshDataPtr->ExistingLODData.AddZeroed(ExistingMesh->GetNumSourceModels());

	// refresh material and section info map here
	// we have to make sure it only contains valid item
	// we go through section info and only add it back if used, otherwise we don't want to use
	if (LodIndex == INDEX_NONE)
	{
		ExistingMesh->GetSectionInfoMap().Clear();
	}
	else
	{
		//Remove only the target section InfoMap, if we destroy more we will not restore the correct material assignment for other Lods contain in the same file.
		int32 ReimportSectionNumber = ExistingMesh->GetSectionInfoMap().GetSectionNumber(LodIndex);
		for (int32 SectionIndex = 0; SectionIndex < ReimportSectionNumber; ++SectionIndex)
		{
			ExistingMesh->GetSectionInfoMap().Remove(LodIndex, SectionIndex);
		}
	}

	
	/******************************************
	 * Nanite Begin
	 */

	//Nanite Save the settings
	ExistingMeshDataPtr->ExistingNaniteSettings = ExistingMesh->GetNaniteSettings();

	//Nanite Save the source model
	const FStaticMeshSourceModel& HiResSourceModel = ExistingMesh->GetHiResSourceModel();
	ExistingMeshDataPtr->HiResSourceData.ExistingBuildSettings = HiResSourceModel.BuildSettings;
	ExistingMeshDataPtr->HiResSourceData.ExistingReductionSettings = HiResSourceModel.ReductionSettings;
	ExistingMeshDataPtr->HiResSourceData.ExistingScreenSize = HiResSourceModel.ScreenSize;
	ExistingMeshDataPtr->HiResSourceData.ExistingSourceImportFilename = HiResSourceModel.SourceImportFilename;
		
	//Nanite Save the hi res mesh description
	if(const FMeshDescription* HiResMeshDescription = ExistingMesh->GetHiResMeshDescription())
	{
		ExistingMeshDataPtr->HiResSourceData.ExistingMeshDescription = MakeUnique<FMeshDescription>(*HiResMeshDescription);
	}

	/*
	 * Nanite End
	 ******************************************/


	int32 TotalMaterialIndex = ExistingMeshDataPtr->ExistingMaterials.Num();
	for (int32 SourceModelIndex = 0; SourceModelIndex < ExistingMesh->GetNumSourceModels(); SourceModelIndex++)
	{
		//If the last import was exceeding the maximum number of LOD the source model will contain more LOD so just break the loop
		if (SourceModelIndex >= ExistingMesh->GetRenderData()->LODResources.Num())
			break;
		FStaticMeshLODResources& LOD = ExistingMesh->GetRenderData()->LODResources[SourceModelIndex];
		int32 NumSections = LOD.Sections.Num();
		for(int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
		{
			FMeshSectionInfo Info = OldSectionInfoMap.Get(SourceModelIndex, SectionIndex);
			if(bSaveMaterials && ExistingMesh->GetStaticMaterials().IsValidIndex(Info.MaterialIndex))
			{
				if (ExistingMeshDataPtr->UseMaterialNameSlotWorkflow)
				{
					int32 ExistMaterialIndex = ExistingMeshDataPtr->ExistingMaterials.Find(ExistingMesh->GetStaticMaterials()[Info.MaterialIndex]);
					if (ExistMaterialIndex == INDEX_NONE)
					{
						ExistMaterialIndex = ExistingMeshDataPtr->ExistingMaterials.Add(ExistingMesh->GetStaticMaterials()[Info.MaterialIndex]);
					}
					Info.MaterialIndex = ExistMaterialIndex;
				}
				else
				{
					// we only save per LOD separeate IF the material index isn't added yet. 
					// if it's already added, we don't have to add another one. 
					if (Info.MaterialIndex >= TotalMaterialIndex)
					{
						ExistingMeshDataPtr->ExistingMaterials.Add(ExistingMesh->GetStaticMaterials()[Info.MaterialIndex]);

						// @Todo @fixme
						// have to refresh material index since it might be pointing at wrong one
						// this will break IF the base material number grows or shoterns and index will be off
						// I think we have to save material index per section, so that we don't have to worry about global index
						Info.MaterialIndex = TotalMaterialIndex++;
					}
				}
				ExistingMeshDataPtr->ExistingSectionInfoMap.Set(SourceModelIndex, SectionIndex, Info);
			}
		}

		const FStaticMeshSourceModel& SourceModel = ExistingMesh->GetSourceModel(SourceModelIndex);
		FExistingLODMeshData& ExistingLODData = ExistingMeshDataPtr->ExistingLODData[SourceModelIndex];
		ExistingLODData.ExistingBuildSettings = SourceModel.BuildSettings;
		ExistingLODData.ExistingReductionSettings = SourceModel.ReductionSettings;
		ExistingLODData.ExisitingMeshTrianglesCount = SourceModel.CacheMeshDescriptionTrianglesCount;
		ExistingLODData.ExisitingMeshVerticesCount = SourceModel.CacheMeshDescriptionVerticesCount;
		if (bIsReimportCustomLODOverGeneratedLOD && (SourceModelIndex == LodIndex))
		{
			//Reset the reduction
			ExistingLODData.ExistingReductionSettings.PercentTriangles = 1.0f;
			ExistingLODData.ExistingReductionSettings.MaxNumOfTriangles = MAX_uint32;
			ExistingLODData.ExistingReductionSettings.PercentVertices = 1.0f;
			ExistingLODData.ExistingReductionSettings.MaxNumOfVerts = MAX_uint32;
			ExistingLODData.ExistingReductionSettings.MaxDeviation = 0.0f;
		}
		ExistingLODData.ExistingScreenSize = SourceModel.ScreenSize;
		ExistingLODData.ExistingSourceImportFilename = SourceModel.SourceImportFilename;

		const FMeshDescription* MeshDescription = ExistingMesh->GetMeshDescription(SourceModelIndex);
		if (MeshDescription)
		{
			ExistingLODData.ExistingMeshDescription = MakeUnique<FMeshDescription>(*MeshDescription);
		}
	}

	ExistingMeshDataPtr->ExistingSockets = ExistingMesh->Sockets;

	ExistingMeshDataPtr->ExistingCustomizedCollision = ExistingMesh->GetCustomizedCollision();
	ExistingMeshDataPtr->bAutoComputeLODScreenSize = ExistingMesh->GetAutoComputeLODScreenSize();

	ExistingMeshDataPtr->ExistingLightMapResolution = ExistingMesh->GetLightMapResolution();
	ExistingMeshDataPtr->ExistingLightMapCoordinateIndex = ExistingMesh->GetLightMapCoordinateIndex();

	ExistingMeshDataPtr->ExistingImportData = ExistingMesh->GetAssetImportData();
	ExistingMeshDataPtr->ExistingThumbnailInfo = ExistingMesh->GetThumbnailInfo();

	ExistingMeshDataPtr->ExistingBodySetup = ExistingMesh->GetBodySetup();

	ExistingMeshDataPtr->bHasNavigationData = ExistingMesh->bHasNavigationData;
	ExistingMeshDataPtr->LODGroup = ExistingMesh->GetLODGroup();
	ExistingMeshDataPtr->MinLOD = ExistingMesh->GetMinLOD();
	ExistingMeshDataPtr->QualityLevelMinLOD = ExistingMesh->GetQualityLevelMinLOD();

	ExistingMeshDataPtr->ExistingGenerateMeshDistanceField = ExistingMesh->bGenerateMeshDistanceField;
	ExistingMeshDataPtr->ExistingLODForCollision = ExistingMesh->LODForCollision;
	ExistingMeshDataPtr->ExistingDistanceFieldSelfShadowBias = ExistingMesh->DistanceFieldSelfShadowBias;
	ExistingMeshDataPtr->ExistingSupportUniformlyDistributedSampling = ExistingMesh->bSupportUniformlyDistributedSampling;
	ExistingMeshDataPtr->ExistingAllowCpuAccess = ExistingMesh->bAllowCPUAccess;
	ExistingMeshDataPtr->ExistingPositiveBoundsExtension = (FVector3f)ExistingMesh->GetPositiveBoundsExtension();
	ExistingMeshDataPtr->ExistingNegativeBoundsExtension = (FVector3f)ExistingMesh->GetNegativeBoundsExtension();

	ExistingMeshDataPtr->ExistingSupportPhysicalMaterialMasks = ExistingMesh->bSupportPhysicalMaterialMasks;
	ExistingMeshDataPtr->ExistingSupportGpuUniformlyDistributedSampling = ExistingMesh->bSupportGpuUniformlyDistributedSampling;
	ExistingMeshDataPtr->ExistingSupportRayTracing = ExistingMesh->bSupportRayTracing;
	ExistingMeshDataPtr->ExistingForceMiplevelsToBeResident = ExistingMesh->bGlobalForceMipLevelsToBeResident;
	ExistingMeshDataPtr->ExistingNeverStream = ExistingMesh->NeverStream;
	ExistingMeshDataPtr->ExistingNumCinematicMipLevels = ExistingMesh->NumCinematicMipLevels;

	UFbxStaticMeshImportData* ImportData = Cast<UFbxStaticMeshImportData>(ExistingMesh->GetAssetImportData());
	if (ImportData && ExistingMeshDataPtr->UseMaterialNameSlotWorkflow)
	{
		for (int32 ImportMaterialOriginalNameDataIndex = 0; ImportMaterialOriginalNameDataIndex < ImportData->ImportMaterialOriginalNameData.Num(); ++ImportMaterialOriginalNameDataIndex)
		{
			FName MaterialName = ImportData->ImportMaterialOriginalNameData[ImportMaterialOriginalNameDataIndex];
			ExistingMeshDataPtr->LastImportMaterialOriginalNameData.Add(MaterialName);
		}
		for (int32 InternalLodIndex = 0; InternalLodIndex < ImportData->ImportMeshLodData.Num(); ++InternalLodIndex)
		{
			ExistingMeshDataPtr->LastImportMeshLodSectionMaterialData.AddZeroed();
			const FImportMeshLodSectionsData &ImportMeshLodSectionsData = ImportData->ImportMeshLodData[InternalLodIndex];
			for (int32 SectionIndex = 0; SectionIndex < ImportMeshLodSectionsData.SectionOriginalMaterialName.Num(); ++SectionIndex)
			{
				FName MaterialName = ImportMeshLodSectionsData.SectionOriginalMaterialName[SectionIndex];
				ExistingMeshDataPtr->LastImportMeshLodSectionMaterialData[InternalLodIndex].Add(MaterialName);
			}
		}
	}
	ExistingMeshDataPtr->ExistingOnMeshChanged = ExistingMesh->OnMeshChanged;
	ExistingMeshDataPtr->ExistingComplexCollisionMesh = ExistingMesh->ComplexCollisionMesh;

	return ExistingMeshDataPtr;
}

// Helper to find if some reduction settings are active
bool IsReductionActive(const FExistingLODMeshData& ExisitingLodMeshData)
{
	const FMeshReductionSettings& ReductionSettings = ExisitingLodMeshData.ExistingReductionSettings;
	bool bUseQuadricSimplier = true;
	{
		// Are we using our tool, or simplygon?  The tool is only changed during editor restarts
		IMeshReduction* ReductionModule = FModuleManager::Get().LoadModuleChecked<IMeshReductionManagerModule>("MeshReductionInterface").GetStaticMeshReductionInterface();
		FString VersionString = ReductionModule->GetVersionString();
		TArray<FString> SplitVersionString;
		VersionString.ParseIntoArray(SplitVersionString, TEXT("_"), true);
		bUseQuadricSimplier = SplitVersionString[0].Equals("QuadricMeshReduction");
	}
	if (!bUseQuadricSimplier)
	{
		return (ReductionSettings.MaxDeviation > 0.0f)
			|| (ReductionSettings.TerminationCriterion == EStaticMeshReductionTerimationCriterion::Triangles && (ReductionSettings.PercentTriangles < 1.0f));
	}
	
	switch (ReductionSettings.TerminationCriterion)
	{
	case EStaticMeshReductionTerimationCriterion::Triangles:
		return (ReductionSettings.PercentTriangles < 1.0f) || ReductionSettings.MaxNumOfTriangles < ExisitingLodMeshData.ExisitingMeshTrianglesCount;
	case EStaticMeshReductionTerimationCriterion::Vertices:
		return (ReductionSettings.PercentVertices < 1.0f) || ReductionSettings.MaxNumOfVerts < ExisitingLodMeshData.ExisitingMeshVerticesCount;
	case EStaticMeshReductionTerimationCriterion::Any:
		return (ReductionSettings.PercentTriangles < 1.0f)
			|| ReductionSettings.MaxNumOfTriangles < ExisitingLodMeshData.ExisitingMeshTrianglesCount
			|| (ReductionSettings.PercentVertices < 1.0f)
			|| ReductionSettings.MaxNumOfVerts < ExisitingLodMeshData.ExisitingMeshVerticesCount;
	}
	return false;
}

/* This function is call before building the mesh when we do a re-import*/
void StaticMeshImportUtils::RestoreExistingMeshSettings(const FExistingStaticMeshData* ExistingMesh, UStaticMesh* NewMesh, int32 LODIndex)
{
	if (!ExistingMesh)
	{
		return;
	}
	NewMesh->SetLODGroup(ExistingMesh->LODGroup);
	NewMesh->SetMinLOD(ExistingMesh->MinLOD);
	NewMesh->SetQualityLevelMinLOD(ExistingMesh->QualityLevelMinLOD);
	int32 ExistingNumLods = ExistingMesh->ExistingLODData.Num();
	int32 CurrentNumLods = NewMesh->GetNumSourceModels();
	if (LODIndex == INDEX_NONE)
	{
		if (CurrentNumLods > ExistingNumLods)
		{
			NewMesh->SetNumSourceModels(ExistingNumLods);
		}
		//Create only the LOD Group we need, extra LOD will be put back when calling RestoreExistingMeshData later in the re import process
		if (NewMesh->GetLODGroup() != NAME_None)
		{
			ITargetPlatform* CurrentPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
			check(CurrentPlatform);
			const FStaticMeshLODGroup& LODGroup = CurrentPlatform->GetStaticMeshLODSettings().GetLODGroup(NewMesh->GetLODGroup());
			ExistingNumLods = FMath::Min(ExistingNumLods, LODGroup.GetDefaultNumLODs());
		}

		for (int32 i = 0; i < ExistingNumLods; i++)
		{
			if (NewMesh->GetNumSourceModels() <= i)
			{
				NewMesh->AddSourceModel();
			}
			FMeshDescription* LODMeshDescription = NewMesh->GetMeshDescription(i);
			bool bSwapFromGeneratedToImported = !ExistingMesh->ExistingLODData[i].ExistingMeshDescription.IsValid() && (LODMeshDescription && LODMeshDescription->Polygons().Num() > 0);
			bool bWasReduced = IsReductionActive(ExistingMesh->ExistingLODData[i]);

			FStaticMeshSourceModel& SourceModel = NewMesh->GetSourceModel(i);
			if (!bSwapFromGeneratedToImported && bWasReduced)
			{
				SourceModel.ReductionSettings = ExistingMesh->ExistingLODData[i].ExistingReductionSettings;
			}
			SourceModel.BuildSettings = ExistingMesh->ExistingLODData[i].ExistingBuildSettings;
			SourceModel.ScreenSize = ExistingMesh->ExistingLODData[i].ExistingScreenSize;
			SourceModel.SourceImportFilename = ExistingMesh->ExistingLODData[i].ExistingSourceImportFilename;
		}
	}
	else
	{
		//Just set the old configuration for the desired LODIndex
		if(LODIndex >= 0 && LODIndex < CurrentNumLods && LODIndex < ExistingNumLods)
		{
			FMeshDescription* LODMeshDescription = NewMesh->GetMeshDescription(LODIndex);
			bool bSwapFromGeneratedToImported = !ExistingMesh->ExistingLODData[LODIndex].ExistingMeshDescription.IsValid() && (LODMeshDescription && LODMeshDescription->Polygons().Num() > 0);
			bool bWasReduced = IsReductionActive(ExistingMesh->ExistingLODData[LODIndex]);

			FStaticMeshSourceModel& SourceModel = NewMesh->GetSourceModel(LODIndex);
			if (!bSwapFromGeneratedToImported && bWasReduced)
			{
				SourceModel.ReductionSettings = ExistingMesh->ExistingLODData[LODIndex].ExistingReductionSettings;
			}
			SourceModel.BuildSettings = ExistingMesh->ExistingLODData[LODIndex].ExistingBuildSettings;
			SourceModel.ScreenSize = ExistingMesh->ExistingLODData[LODIndex].ExistingScreenSize;
			SourceModel.SourceImportFilename = ExistingMesh->ExistingLODData[LODIndex].ExistingSourceImportFilename;
		}
	}

	//We need to fill the import version remap before building the mesh since the
	//static mesh component will be register at the end of the build.
	//We do the remap of the material override in the static mesh component in OnRegister()
	if(ExistingMesh->ImportVersion != EImportStaticMeshVersion::LastVersion)
	{
		uint32 MaterialMapKey = 0;
		TArray<int32> ImportRemapMaterial;
		MaterialMapKey = ((uint32)((ExistingMesh->ImportVersion & 0xffff) << 16) | (uint32)(EImportStaticMeshVersion::LastVersion & 0xffff));
		//Avoid matching a material more then once
		TArray<int32> MatchIndex;
		ImportRemapMaterial.AddZeroed(ExistingMesh->ExistingMaterials.Num());
		for (int32 ExistMaterialIndex = 0; ExistMaterialIndex < ExistingMesh->ExistingMaterials.Num(); ++ExistMaterialIndex)
		{
			ImportRemapMaterial[ExistMaterialIndex] = ExistMaterialIndex; //Set default value
			const FStaticMaterial &ExistMaterial = ExistingMesh->ExistingMaterials[ExistMaterialIndex];
			bool bFoundMatchingMaterial = false;
			for (int32 MaterialIndex = 0; MaterialIndex < NewMesh->GetStaticMaterials().Num(); ++MaterialIndex)
			{
				if (MatchIndex.Contains(MaterialIndex))
				{
					continue;
				}
				FStaticMaterial &Material = NewMesh->GetStaticMaterials()[MaterialIndex];
				if (Material.ImportedMaterialSlotName == ExistMaterial.ImportedMaterialSlotName)
				{
					MatchIndex.Add(MaterialIndex);
					ImportRemapMaterial[ExistMaterialIndex] = MaterialIndex;
					bFoundMatchingMaterial = true;
					break;
				}
			}
			if (!bFoundMatchingMaterial)
			{
				for (int32 MaterialIndex = 0; MaterialIndex < NewMesh->GetStaticMaterials().Num(); ++MaterialIndex)
				{
					if (MatchIndex.Contains(MaterialIndex))
					{
						continue;
					}

					FStaticMaterial &Material = NewMesh->GetStaticMaterials()[MaterialIndex];
					if (ExistMaterial.ImportedMaterialSlotName == NAME_None && Material.MaterialInterface == ExistMaterial.MaterialInterface)
					{
						MatchIndex.Add(MaterialIndex);
						ImportRemapMaterial[ExistMaterialIndex] = MaterialIndex;
						bFoundMatchingMaterial = true;
						break;
					}
				}
			}
			if (!bFoundMatchingMaterial)
			{
				ImportRemapMaterial[ExistMaterialIndex] = ExistMaterialIndex;
			}
		}
		NewMesh->AddMaterialRemapIndex(FMaterialRemapIndex(MaterialMapKey, ImportRemapMaterial));
	}
	// Copy mesh changed delegate data
	NewMesh->OnMeshChanged = ExistingMesh->ExistingOnMeshChanged;
}

void StaticMeshImportUtils::UpdateSomeLodsImportMeshData(UStaticMesh* NewMesh, TArray<int32> *ReimportLodList)
{
	if (NewMesh == nullptr)
	{
		return;
	}
	UFbxStaticMeshImportData* ImportData = Cast<UFbxStaticMeshImportData>(NewMesh->GetAssetImportData());
	//Update the LOD import data before restoring the data
	if (ReimportLodList != nullptr && ImportData != nullptr)
	{
		for (int32 LodLevelImport : (*ReimportLodList))
		{
			if (!ImportData->ImportMeshLodData.IsValidIndex(LodLevelImport))
			{
				ImportData->ImportMeshLodData.AddZeroed(LodLevelImport - ImportData->ImportMeshLodData.Num() + 1);
			}
			ImportData->ImportMeshLodData[LodLevelImport].SectionOriginalMaterialName.Empty();
			if (NewMesh->GetRenderData()->LODResources.IsValidIndex(LodLevelImport))
			{
				FStaticMeshLODResources& LOD = NewMesh->GetRenderData()->LODResources[LodLevelImport];
				int32 NumSections = LOD.Sections.Num();
				for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
				{
					int32 MaterialLodSectionIndex = LOD.Sections[SectionIndex].MaterialIndex;
					if (NewMesh->GetSectionInfoMap().IsValidSection(LodLevelImport, SectionIndex))
					{
						MaterialLodSectionIndex = NewMesh->GetSectionInfoMap().Get(LodLevelImport, SectionIndex).MaterialIndex;
					}

					if (NewMesh->GetStaticMaterials().IsValidIndex(MaterialLodSectionIndex))
					{
						bool bFoundMatch = false;
						FName OriginalImportName = NewMesh->GetStaticMaterials()[MaterialLodSectionIndex].ImportedMaterialSlotName;
						//Find the material in the original import data
						int32 ImportMaterialIndex = 0;
						for (ImportMaterialIndex = 0; ImportMaterialIndex < ImportData->ImportMaterialOriginalNameData.Num(); ++ImportMaterialIndex)
						{
							if (ImportData->ImportMaterialOriginalNameData[ImportMaterialIndex] == OriginalImportName)
							{
								bFoundMatch = true;
								break;
							}
						}
						if (!bFoundMatch)
						{
							ImportMaterialIndex = ImportData->ImportMaterialOriginalNameData.Add(OriginalImportName);
						}
						ImportData->ImportMeshLodData[LodLevelImport].SectionOriginalMaterialName.Add(ImportData->ImportMaterialOriginalNameData[ImportMaterialIndex]);
					}
					else
					{
						ImportData->ImportMeshLodData[LodLevelImport].SectionOriginalMaterialName.Add(TEXT("InvalidMaterialIndex"));
					}
				}
			}
		}
	}
}

template<typename T>
void AddGeneratedGeom(const TArray<T>& SrcData, TArray<T>& DstData) {
	for (const T& CurrentElement : SrcData)
	{
		if (CurrentElement.bIsGenerated)
		{
			DstData.Add(CurrentElement);
		}
	}
};

void StaticMeshImportUtils::RestoreExistingMeshData(const TSharedPtr<const FExistingStaticMeshData>& ExistingMeshDataPtr, UStaticMesh* NewMesh, int32 LodLevel, bool bCanShowDialog, bool bForceConflictingMaterialReset)
{
	if (!ExistingMeshDataPtr || !NewMesh)
	{
		return;
	}

	//Restore the package metadata
	InternalImportUtils::RestoreMetaData(NewMesh, ExistingMeshDataPtr->ExistingUMetaDataTagValues);

	//Create a remap material Index use to find the matching section later
	TArray<int32> RemapMaterial;
	RemapMaterial.AddZeroed(NewMesh->GetStaticMaterials().Num());
	TArray<FName> RemapMaterialName;
	RemapMaterialName.AddZeroed(NewMesh->GetStaticMaterials().Num());

	//If user is attended, ask them to verify the match is good
	UnFbx::EFBXReimportDialogReturnOption ReturnOption;
	//Ask the user to match the materials conflict
	UnFbx::FFbxImporter::PrepareAndShowMaterialConflictDialog<FStaticMaterial>(ExistingMeshDataPtr->ExistingMaterials, NewMesh->GetStaticMaterials(), RemapMaterial, RemapMaterialName, bCanShowDialog, false, bForceConflictingMaterialReset, ReturnOption);
	
	if (ReturnOption != UnFbx::EFBXReimportDialogReturnOption::FBXRDRO_ResetToFbx)
	{
		//Build a ordered material list that try to keep intact the existing material list
		TArray<FStaticMaterial> MaterialOrdered;
		TArray<bool> MatchedNewMaterial;
		MatchedNewMaterial.AddZeroed(NewMesh->GetStaticMaterials().Num());
		for (int32 ExistMaterialIndex = 0; ExistMaterialIndex < ExistingMeshDataPtr->ExistingMaterials.Num(); ++ExistMaterialIndex)
		{
			int32 MaterialIndexOrdered = MaterialOrdered.Add(ExistingMeshDataPtr->ExistingMaterials[ExistMaterialIndex]);
			FStaticMaterial& OrderedMaterial = MaterialOrdered[MaterialIndexOrdered];
			int32 NewMaterialIndex = INDEX_NONE;
			if (RemapMaterial.Find(ExistMaterialIndex, NewMaterialIndex))
			{
				MatchedNewMaterial[NewMaterialIndex] = true;
				RemapMaterial[NewMaterialIndex] = MaterialIndexOrdered;
				OrderedMaterial.ImportedMaterialSlotName = NewMesh->GetStaticMaterials()[NewMaterialIndex].ImportedMaterialSlotName;
			}
			else
			{
				//Unmatched material must be conserve
			}
		}

		//Add the new material entries (the one that do not match with any existing material)
		for (int32 NewMaterialIndex = 0; NewMaterialIndex < MatchedNewMaterial.Num(); ++NewMaterialIndex)
		{
			if (MatchedNewMaterial[NewMaterialIndex] == false)
			{
				int32 NewMeshIndex = MaterialOrdered.Add(NewMesh->GetStaticMaterials()[NewMaterialIndex]);
				RemapMaterial[NewMaterialIndex] = NewMeshIndex;
			}
		}

		//Set the RemapMaterialName array helper
		for (int32 MaterialIndex = 0; MaterialIndex < RemapMaterial.Num(); ++MaterialIndex)
		{
			int32 SourceMaterialMatch = RemapMaterial[MaterialIndex];
			if (ExistingMeshDataPtr->ExistingMaterials.IsValidIndex(SourceMaterialMatch))
			{
				RemapMaterialName[MaterialIndex] = ExistingMeshDataPtr->ExistingMaterials[SourceMaterialMatch].ImportedMaterialSlotName;
			}
		}

		//Copy the re ordered materials (this ensure the material array do not change when we re-import)
		NewMesh->SetStaticMaterials(MaterialOrdered);
	}
	int32 NumCommonLODs = FMath::Min<int32>(ExistingMeshDataPtr->ExistingLODData.Num(), NewMesh->GetNumSourceModels());
	for(int32 i=0; i<NumCommonLODs; i++)
	{
		FStaticMeshSourceModel& SourceModel = NewMesh->GetSourceModel(i);
		SourceModel.BuildSettings = ExistingMeshDataPtr->ExistingLODData[i].ExistingBuildSettings;
		FMeshDescription* LODMeshDescription = NewMesh->GetMeshDescription(i);
		//Restore the reduction settings only if the existing data was a using reduction. Because we can set some value if we reimport from existing rawmesh to auto generated.
		bool bSwapFromGeneratedToImported = !ExistingMeshDataPtr->ExistingLODData[i].ExistingMeshDescription.IsValid() && (LODMeshDescription && LODMeshDescription->Polygons().Num() > 0);
		bool bWasReduced = IsReductionActive(ExistingMeshDataPtr->ExistingLODData[i]);
		if ( !bSwapFromGeneratedToImported && bWasReduced)
		{
			SourceModel.ReductionSettings = ExistingMeshDataPtr->ExistingLODData[i].ExistingReductionSettings;
		}
		SourceModel.ScreenSize = ExistingMeshDataPtr->ExistingLODData[i].ExistingScreenSize;
		SourceModel.SourceImportFilename = ExistingMeshDataPtr->ExistingLODData[i].ExistingSourceImportFilename;
	}

	for(int32 i=NumCommonLODs; i < ExistingMeshDataPtr->ExistingLODData.Num(); ++i)
	{
		FStaticMeshSourceModel& SrcModel = NewMesh->AddSourceModel();
		if (ExistingMeshDataPtr->ExistingLODData[i].ExistingMeshDescription.IsValid())
		{
			FMeshDescription* MeshDescription = NewMesh->CreateMeshDescription(i, MoveTemp(*ExistingMeshDataPtr->ExistingLODData[i].ExistingMeshDescription));
			NewMesh->CommitMeshDescription(i);
		}
		SrcModel.BuildSettings = ExistingMeshDataPtr->ExistingLODData[i].ExistingBuildSettings;
		SrcModel.ReductionSettings = ExistingMeshDataPtr->ExistingLODData[i].ExistingReductionSettings;
		SrcModel.ScreenSize = ExistingMeshDataPtr->ExistingLODData[i].ExistingScreenSize;
		SrcModel.SourceImportFilename = ExistingMeshDataPtr->ExistingLODData[i].ExistingSourceImportFilename;
	}

	// Restore the section info of the just imported LOD so is section info map is remap to fit the mesh material array
	if (ExistingMeshDataPtr->ExistingSectionInfoMap.Map.Num() > 0)
	{
		//Build the mesh we need the render data and the existing section info map build before restoring the data
		if (NewMesh->GetRenderData()->LODResources.Num() < NewMesh->GetNumSourceModels())
		{
			NewMesh->Build();
		}
		for (int32 i = 0; i < NewMesh->GetRenderData()->LODResources.Num(); i++)
		{
			//If a LOD was specified, only touch the specified LOD
			if (LodLevel != INDEX_NONE && LodLevel != 0 && LodLevel != i)
			{
				continue;
			}

			
			//When re-importing the asset, do not touch the LOD that was imported from file, the material array is keep intact so the section should still be valid.
			bool NoRemapForThisLOD = LodLevel == INDEX_NONE && i != 0 && !NewMesh->GetSourceModel(i).bImportWithBaseMesh && !NewMesh->IsReductionActive(i);

			FStaticMeshLODResources& LOD = NewMesh->GetRenderData()->LODResources[i];
			
			int32 NumSections = LOD.Sections.Num();
			int32 OldSectionNumber = ExistingMeshDataPtr->ExistingSectionInfoMap.GetSectionNumber(i);
			for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
			{
				//If the SectionInfoMap is not set yet. Because we re-import LOD 0 but we have other LODs
				//Just put back the old section Info Map
				if (NewMesh->GetSectionInfoMap().GetSectionNumber(i) <= SectionIndex)
				{
					NewMesh->GetSectionInfoMap().Set(i, SectionIndex, ExistingMeshDataPtr->ExistingSectionInfoMap.Get(i, SectionIndex));
				}
				//We recreate the SectionInfoMap from the existing data and we do not remap it if LOD is not auto generate and was not imported
				if (NoRemapForThisLOD)
				{
					continue;
				}

				FMeshSectionInfo NewSectionInfo =  NewMesh->GetSectionInfoMap().Get(i, SectionIndex);
				bool bFoundOldMatch = false;
				bool bKeepOldSectionMaterialIndex = false;
				int32 OriginalSectionMaterialIndex = INDEX_NONE;
				if (RemapMaterial.IsValidIndex(NewSectionInfo.MaterialIndex) && NewMesh->GetStaticMaterials().IsValidIndex(NewSectionInfo.MaterialIndex))
				{
					//Find the matching old index
					for (int32 ExistSectionIndex = 0; ExistSectionIndex < OldSectionNumber; ++ExistSectionIndex)
					{
						bKeepOldSectionMaterialIndex = false;
						OriginalSectionMaterialIndex = INDEX_NONE;
						FMeshSectionInfo OldSectionInfo = ExistingMeshDataPtr->ExistingSectionInfoMap.Get(i, ExistSectionIndex);
						if (ExistingMeshDataPtr->UseMaterialNameSlotWorkflow)
						{
							if (ExistingMeshDataPtr->ExistingImportData.IsValid() &&
								ExistingMeshDataPtr->LastImportMeshLodSectionMaterialData.IsValidIndex(i) &&
								ExistingMeshDataPtr->LastImportMeshLodSectionMaterialData[i].IsValidIndex(ExistSectionIndex))
							{
								//Keep the old section material index only if the user has change the section mapping
								bKeepOldSectionMaterialIndex = ExistingMeshDataPtr->LastImportMeshLodSectionMaterialData[i][ExistSectionIndex] != ExistingMeshDataPtr->ExistingMaterials[OldSectionInfo.MaterialIndex].ImportedMaterialSlotName;
								for (int32 ExistMaterialIndex = 0; ExistMaterialIndex < ExistingMeshDataPtr->ExistingMaterials.Num(); ++ExistMaterialIndex)
								{
									if (ExistingMeshDataPtr->LastImportMeshLodSectionMaterialData[i][ExistSectionIndex] == ExistingMeshDataPtr->ExistingMaterials[ExistMaterialIndex].ImportedMaterialSlotName)
									{
										OriginalSectionMaterialIndex = ExistMaterialIndex;
										break;
									}
								}
							}
						}
						int32 OldSectionMatchIndex = OriginalSectionMaterialIndex != INDEX_NONE ? OriginalSectionMaterialIndex : OldSectionInfo.MaterialIndex;
						if (RemapMaterial[NewSectionInfo.MaterialIndex] == OldSectionMatchIndex)
						{
							NewMesh->GetSectionInfoMap().Set(i, SectionIndex, OldSectionInfo);
							bFoundOldMatch = true;
							break;
						}
					}
				}

				if (!bFoundOldMatch)
				{
					if (RemapMaterial.IsValidIndex(NewSectionInfo.MaterialIndex))
					{
						//Find the old section that was using the NewSectionInfo.MaterialIndex
						//This will allow copying the section information: Cast Shadow, enable collision
						for (int32 ExistSectionIndex = 0; ExistSectionIndex < OldSectionNumber; ++ExistSectionIndex)
						{
							FMeshSectionInfo OldSectionInfo = ExistingMeshDataPtr->ExistingSectionInfoMap.Get(i, ExistSectionIndex);
							if (NewSectionInfo.MaterialIndex == OldSectionInfo.MaterialIndex)
							{
								NewSectionInfo.bCastShadow = OldSectionInfo.bCastShadow;
								NewSectionInfo.bEnableCollision = OldSectionInfo.bEnableCollision;
								break;
							}
						}
						//If user has change the section info map, we want to keep the change
						NewSectionInfo.MaterialIndex = RemapMaterial[NewSectionInfo.MaterialIndex];
						NewMesh->GetSectionInfoMap().Set(i, SectionIndex, NewSectionInfo);
					}
				}
			}
		}
		//Store the just imported section info map
		NewMesh->GetOriginalSectionInfoMap().CopyFrom(NewMesh->GetSectionInfoMap());
	}

	// Assign sockets from old version of this StaticMesh.
	for(int32 i=0; i<ExistingMeshDataPtr->ExistingSockets.Num(); i++)
	{
		UStaticMeshSocket* ExistingSocket = ExistingMeshDataPtr->ExistingSockets[i];
		UStaticMeshSocket* Socket = NewMesh->FindSocket(ExistingSocket->SocketName);
		if (!Socket && !ExistingSocket->bSocketCreatedAtImport)
		{
			NewMesh->AddSocket(ExistingSocket);
		}
	}

	NewMesh->SetCustomizedCollision(ExistingMeshDataPtr->ExistingCustomizedCollision);
	NewMesh->SetAutoComputeLODScreenSize(ExistingMeshDataPtr->bAutoComputeLODScreenSize);

	NewMesh->SetLightMapResolution(ExistingMeshDataPtr->ExistingLightMapResolution);
	NewMesh->SetLightMapCoordinateIndex(ExistingMeshDataPtr->ExistingLightMapCoordinateIndex);

	if (ExistingMeshDataPtr->ExistingImportData.IsValid())
	{
		//RestoredLods
		UFbxStaticMeshImportData* ImportData = Cast<UFbxStaticMeshImportData>(NewMesh->GetAssetImportData());
		TArray<FName> ImportMaterialOriginalNameData;
		TArray<FImportMeshLodSectionsData> ImportMeshLodData;
		if (ImportData != nullptr && ImportData->ImportMaterialOriginalNameData.Num() > 0 && ImportData->ImportMeshLodData.Num() > 0)
		{
			ImportMaterialOriginalNameData = ImportData->ImportMaterialOriginalNameData;
			ImportMeshLodData = ImportData->ImportMeshLodData;
		}

		NewMesh->SetAssetImportData(ExistingMeshDataPtr->ExistingImportData.Get());

		ImportData = Cast<UFbxStaticMeshImportData>(NewMesh->GetAssetImportData());
		if (ImportData != nullptr && ImportMaterialOriginalNameData.Num() > 0 && ImportMeshLodData.Num() > 0)
		{
			ImportData->ImportMaterialOriginalNameData = ImportMaterialOriginalNameData;
			ImportData->ImportMeshLodData = ImportMeshLodData;
		}
	}
		

	NewMesh->SetThumbnailInfo(ExistingMeshDataPtr->ExistingThumbnailInfo.Get());

	// If we already had some collision info...
	if(ExistingMeshDataPtr->ExistingBodySetup)
	{
		// If we didn't import anything, keep all collisions otherwise only keep generated collisions.
		bool bKeepAllCollisions;
		if(!NewMesh->GetBodySetup() || NewMesh->GetBodySetup()->AggGeom.GetElementCount() == 0)
		{
			bKeepAllCollisions = true;
		}
		else
		{
			bKeepAllCollisions = false;
		}

		if(bKeepAllCollisions)
		{
			NewMesh->SetBodySetup(ExistingMeshDataPtr->ExistingBodySetup);
		}
		else if (NewMesh->GetBodySetup() != ExistingMeshDataPtr->ExistingBodySetup)
		{
			// New collision geometry, but we still want the original settings and the generated collisions
			NewMesh->GetBodySetup()->CopyBodySetupProperty(ExistingMeshDataPtr->ExistingBodySetup);

			const FKAggregateGeom& ExistingAggGeom = ExistingMeshDataPtr->ExistingBodySetup->AggGeom;
			FKAggregateGeom& NewAggGeom = NewMesh->GetBodySetup()->AggGeom;
			AddGeneratedGeom(ExistingAggGeom.BoxElems, NewAggGeom.BoxElems);
			AddGeneratedGeom(ExistingAggGeom.ConvexElems, NewAggGeom.ConvexElems);
			AddGeneratedGeom(ExistingAggGeom.SphereElems, NewAggGeom.SphereElems);
			AddGeneratedGeom(ExistingAggGeom.SphylElems, NewAggGeom.SphylElems);
		}
	}

	NewMesh->bHasNavigationData = ExistingMeshDataPtr->bHasNavigationData;
	NewMesh->SetLODGroup_Unsafe(ExistingMeshDataPtr->LODGroup);

	NewMesh->bGenerateMeshDistanceField = ExistingMeshDataPtr->ExistingGenerateMeshDistanceField;
	NewMesh->LODForCollision = ExistingMeshDataPtr->ExistingLODForCollision;
	NewMesh->DistanceFieldSelfShadowBias = ExistingMeshDataPtr->ExistingDistanceFieldSelfShadowBias;
	NewMesh->bSupportUniformlyDistributedSampling = ExistingMeshDataPtr->ExistingSupportUniformlyDistributedSampling;
	NewMesh->bAllowCPUAccess = ExistingMeshDataPtr->ExistingAllowCpuAccess;
	NewMesh->SetPositiveBoundsExtension((FVector)ExistingMeshDataPtr->ExistingPositiveBoundsExtension);
	NewMesh->SetNegativeBoundsExtension((FVector)ExistingMeshDataPtr->ExistingNegativeBoundsExtension);

	NewMesh->ComplexCollisionMesh = ExistingMeshDataPtr->ExistingComplexCollisionMesh;

	NewMesh->bSupportPhysicalMaterialMasks = ExistingMeshDataPtr->ExistingSupportPhysicalMaterialMasks;
	NewMesh->bSupportGpuUniformlyDistributedSampling = ExistingMeshDataPtr->ExistingSupportGpuUniformlyDistributedSampling;
	NewMesh->bSupportRayTracing = ExistingMeshDataPtr->ExistingSupportRayTracing;
	NewMesh->bGlobalForceMipLevelsToBeResident = ExistingMeshDataPtr->ExistingForceMiplevelsToBeResident;
	NewMesh->NeverStream = ExistingMeshDataPtr->ExistingNeverStream;
	NewMesh->NumCinematicMipLevels = ExistingMeshDataPtr->ExistingNumCinematicMipLevels;

	/******************************************
	 * Nanite Begin
	 */

	 //Nanite Restore the settings
	NewMesh->SetNaniteSettings(ExistingMeshDataPtr->ExistingNaniteSettings);

	//Nanite Save the source model
	FStaticMeshSourceModel& HiResSourceModel = NewMesh->GetHiResSourceModel();
	HiResSourceModel.BuildSettings = ExistingMeshDataPtr->HiResSourceData.ExistingBuildSettings;
	HiResSourceModel.ReductionSettings = ExistingMeshDataPtr->HiResSourceData.ExistingReductionSettings;
	HiResSourceModel.ScreenSize = ExistingMeshDataPtr->HiResSourceData.ExistingScreenSize;
	HiResSourceModel.SourceImportFilename = ExistingMeshDataPtr->HiResSourceData.ExistingSourceImportFilename;

	//Nanite Restore the hires mesh description
	if (ExistingMeshDataPtr->HiResSourceData.ExistingMeshDescription.IsValid())
	{
		FMeshDescription* HiResMeshDescription = NewMesh->GetHiResMeshDescription();
		if (HiResMeshDescription == nullptr)
		{
			HiResMeshDescription = NewMesh->CreateHiResMeshDescription();
		}
		check(HiResMeshDescription);
		NewMesh->ModifyHiResMeshDescription();
		*HiResMeshDescription = MoveTemp(*ExistingMeshDataPtr->HiResSourceData.ExistingMeshDescription);
		NewMesh->CommitHiResMeshDescription();
	}

	/*
	 * Nanite End
	 ******************************************/	
}

#undef LOCTEXT_NAMESPACE
