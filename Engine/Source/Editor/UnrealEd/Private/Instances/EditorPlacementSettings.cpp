// Copyright Epic Games, Inc. All Rights Reserved.

#include "Instances/EditorPlacementSettings.h"

#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Interfaces/TypedElementAssetDataInterface.h"
#include "Elements/Interfaces/TypedElementObjectInterface.h"
#include "Elements/Interfaces/TypedElementWorldInterface.h"
#include "Instances/InstancedPlacementPartitionActor.h"

#include "Components/InstancedStaticMeshComponent.h"

#include "Editor.h"
#include "Subsystems/PlacementSubsystem.h"
#include "LevelEditorSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorPlacementSettings)

void UEditorInstancedPlacementSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UEditorInstancedPlacementSettings::PostLoad()
{
	Super::PostLoad();
	
	// Register Delegates if necessary
}

void UEditorInstancedPlacementSettings::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	if (Ar.IsLoading())
	{
		// Register Delegates if necessary
	}
}

void UEditorInstancedPlacementSettings::RegisterISMDescriptors(AInstancedPlacementPartitionActor* ParentPartitionActor, TSortedMap<int32, TArray<FTransform>>& ISMDefinition) const
{
	ISMDefinition.Empty();
#if 0
	if (!ParentPartitionActor)
	{
		return;
	}

	UPlacementSubsystem* PlacementSubsystem = GEditor->GetEditorSubsystem<UPlacementSubsystem>();
	if (!PlacementSubsystem)
	{
		return;
	}

	TArray<FTypedElementHandle> SpawnedTemporaryElements;// = PlacementSubsystem->SpawnPreviewElementsFromAssetData(ObjectPath.TryLoad());
	for (FTypedElementHandle& PreviewElement : SpawnedTemporaryElements)
	{
		AActor* PreviewActor = nullptr;
		if (TTypedElement<ITypedElementObjectInterface> ObjectInterface = UTypedElementRegistry::GetInstance()->GetElement<ITypedElementObjectInterface>(PreviewElement))
		{
			PreviewActor = ObjectInterface.GetObjectAs<AActor>();
		}

		if (!PreviewActor)
		{
			return;
		}

		FTransform ActorTransform = PreviewActor->GetActorTransform();
		TArray<UStaticMeshComponent*> StaticMeshComponents;
		PreviewActor->GetComponents(StaticMeshComponents);

		for (UStaticMeshComponent* StaticMeshComponent : StaticMeshComponents)
		{
			if (UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh())
			{
				FISMComponentDescriptor Descriptor = InstancedComponentSettings;
				Descriptor.StaticMesh = StaticMesh;
				Descriptor.ComputeHash();
				int32 DescriptorIndex = ParentPartitionActor->RegisterISMComponentDescriptor(Descriptor);

				TArray<FTransform>& Transforms = ISMDefinition.FindOrAdd(DescriptorIndex);
				if (UInstancedStaticMeshComponent* ISMComponent = Cast<UInstancedStaticMeshComponent>(StaticMeshComponent))
				{
					for (int32 InstanceIndex = 0; InstanceIndex < ISMComponent->GetInstanceCount(); ++InstanceIndex)
					{
						FTransform InstanceTransform;
						constexpr bool bUseWorldSpace = true;
						if (ensure(ISMComponent->GetInstanceTransform(InstanceIndex, InstanceTransform, bUseWorldSpace)))
						{
							FTransform LocalTransform = InstanceTransform.GetRelativeTransform(ActorTransform);
							Transforms.Add(LocalTransform);
						}
					}
				}
				else
				{
					FTransform LocalTransform = StaticMeshComponent->GetComponentTransform().GetRelativeTransform(ActorTransform);
					Transforms.Add(LocalTransform);
				}
			}
		}

		if (TTypedElement<ITypedElementWorldInterface> WorldInterfaceElement = UTypedElementRegistry::GetInstance()->GetElement<ITypedElementWorldInterface>(PreviewElement))
		{
			if (ULevelEditorSubsystem* LevelEditorSubsystem = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>())
			{
				WorldInterfaceElement.DeleteElement(WorldInterfaceElement.GetOwnerWorld(), LevelEditorSubsystem->GetSelectionSet(), FTypedElementDeletionOptions());
			}
		}
	}
#endif
}
