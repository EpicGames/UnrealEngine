// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetReinstanceUtilities.h"
#include "Algo/AnyOf.h"
#include "Algo/Copy.h"
#include "Algo/RemoveIf.h"
#include "Algo/Transform.h"
#include "BlueprintCompilationManager.h"
#include "UObject/Interface.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Components/ActorComponent.h"
#include "Engine/Level.h"
#include "GameFramework/Actor.h"
#include "Engine/SimpleConstructionScript.h"
#include "Engine/SCS_Node.h"
#include "Stats/StatsMisc.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectIterator.h"
#include "UObject/StructOnScope.h"
#include "UObject/MetaData.h"
#include "UObject/TextProperty.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/CoreStyle.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Styling/AppStyle.h"
#include "Exporters/Exporter.h"
#include "Animation/AnimInstance.h"
#include "Editor/EditorEngine.h"
#include "Editor/UnrealEdEngine.h"
#include "Animation/AnimBlueprint.h"
#include "Engine/MemberReference.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "ThumbnailRendering/BlueprintThumbnailRenderer.h"
#include "Engine/LevelScriptActor.h"
#include "Components/TimelineComponent.h"
#include "Engine/TimelineTemplate.h"
#include "StructUtils/UserDefinedStruct.h"
#include "UObject/PropertyPortFlags.h"
#include "Serialization/ArchiveReplaceObjectRef.h"
#include "EngineUtils.h"
#include "EdMode.h"
#include "Dialogs/Dialogs.h"
#include "UnrealEdGlobals.h"
#include "Engine/LevelScriptBlueprint.h"
#include "UObject/BlueprintsObjectVersion.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Algo/Transform.h"
#include "Algo/Count.h"

#include "KismetCompilerModule.h"
#include "EdGraphSchema_K2_Actions.h"
#include "K2Node_Event.h"
#include "K2Node_ActorBoundEvent.h"
#include "K2Node_CallFunction.h"
#include "K2Node_AddComponent.h"
#include "K2Node_BaseMCDelegate.h"
#include "K2Node_AddDelegate.h"
#include "K2Node_BaseAsyncTask.h"
#include "K2Node_Variable.h"
#include "K2Node_CallParentFunction.h"
#include "K2Node_ComponentBoundEvent.h"
#include "K2Node_Tunnel.h"
#include "K2Node_Composite.h"
#include "K2Node_CreateDelegate.h"
#include "K2Node_CustomEvent.h"
#include "K2Node_FunctionTerminator.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_FunctionResult.h"
#include "K2Node_GetClassDefaults.h"
#include "K2Node_Literal.h"
#include "K2Node_MacroInstance.h"
#include "K2Node_MathExpression.h"
#include "K2Node_SpawnActorFromClass.h"
#include "K2Node_StructOperation.h"
#include "K2Node_TemporaryVariable.h"
#include "K2Node_Timeline.h"
#include "K2Node_Knot.h"
#include "MaterialGraph/MaterialGraphNode_Composite.h"
#include "AnimGraphNode_StateMachineBase.h"
#include "AnimStateNodeBase.h"
#include "AnimStateNode.h"
#include "AnimStateTransitionNode.h"
#include "AnimationTransitionSchema.h"
#include "AnimationGraph.h"
#include "AnimationGraphSchema.h"
#include "AnimationStateMachineGraph.h"
#include "AnimationTransitionGraph.h"
#include "AnimStateConduitNode.h"
#include "AnimGraphNode_StateMachine.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/KismetDebugUtilities.h"
#include "Kismet2/StructureEditorUtils.h"
#include "ScopedTransaction.h"
#include "ClassViewerFilter.h"
#include "NodeDependingOnEnumInterface.h"

#include "BlueprintEditorModule.h"
#include "BlueprintEditor.h"
#include "Kismet2/Kismet2NameValidators.h"

#include "Misc/DefaultValueHelper.h"
#include "ObjectEditorUtils.h"
#include "Toolkits/ToolkitManager.h"
#include "UnrealExporter.h"
#include "BlueprintActionDatabase.h"
#include "BlueprintEditorSettings.h"

#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "IBlutilityModule.h"

#include "Engine/InheritableComponentHandler.h"
#include "LevelEditor.h"

#include "EditorCategoryUtils.h"
#include "Styling/SlateIconFinder.h"
#include "BaseWidgetBlueprint.h"
#include "Components/Widget.h"
#include "UObject/UObjectThreadContext.h"
#include "AnimGraphNode_LinkedInputPose.h"
#include "AnimGraphNode_Root.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/FeedbackContext.h"

#include "Containers/ArrayView.h"
#include "UObject/FastReferenceCollector.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Interfaces/TypedElementHierarchyInterface.h"
#include "Elements/Interfaces/TypedElementObjectInterface.h"
#include "Misc/MessageDialog.h"
#include "UObject/FastReferenceCollector.h"
#include "INotifyFieldValueChanged.h"

#include "Components/SkeletalMeshComponent.h"

#define LOCTEXT_NAMESPACE "Blueprint"

DEFINE_LOG_CATEGORY(LogBlueprintDebug);

DEFINE_STAT(EKismetCompilerStats_NotifyBlueprintChanged);
DECLARE_CYCLE_STAT(TEXT("Mark Blueprint as Structurally Modified"), EKismetCompilerStats_MarkBlueprintasStructurallyModified, STATGROUP_KismetCompiler);
DECLARE_CYCLE_STAT(TEXT("Refresh External DependencyNodes"), EKismetCompilerStats_RefreshExternalDependencyNodes, STATGROUP_KismetCompiler);

static void SortNodes(TArray<UK2Node*>& AllNodes, bool bSortByPriorityOnly = false)
{
	auto SortNodesInternalByPriorityOnly = [](const UK2Node& A, const UK2Node& B)
	{
		return A.GetNodeRefreshPriority() > B.GetNodeRefreshPriority();
	};

	auto SortNodesInternal = [SortNodesInternalByPriorityOnly](const UK2Node& A, const UK2Node& B)
	{
		const bool NodeAChangesStructure = A.NodeCausesStructuralBlueprintChange();
		const bool NodeBChangesStructure = B.NodeCausesStructuralBlueprintChange();

		if (NodeAChangesStructure != NodeBChangesStructure)
		{
			return NodeAChangesStructure;
		}

		return SortNodesInternalByPriorityOnly(A, B);
	};

	if (AllNodes.Num() > 1)
	{
		if (bSortByPriorityOnly)
		{
			AllNodes.Sort(SortNodesInternalByPriorityOnly);
		}
		else
		{
			AllNodes.Sort(SortNodesInternal);
		}
	}
}

/**
 * This helper does a depth first search, looking for the highest parent class that
 * implements the specified interface.
 * 
 * @param  Class		The class whose inheritance tree you want to check (NOTE: this class is checked itself as well).
 * @param  Interface	The interface that you're looking for.
 * @return NULL if the interface wasn't found, otherwise the highest parent class that implements the interface.
 */
static UClass* FindInheritedInterface(UClass* const Class, FBPInterfaceDescription const& Interface)
{
	UClass* ClassWithInterface = nullptr;

	if (Class != nullptr)
	{
		UClass* const ParentClass = Class->GetSuperClass();
		// search depth first so that we may find the highest parent in the chain that implements this interface
		ClassWithInterface = FindInheritedInterface(ParentClass, Interface);

		if (ClassWithInterface == nullptr)
		{
			for (const FImplementedInterface& ImplementedInterface : Class->Interfaces)
		{
				if (ImplementedInterface.Class == Interface.Interface)
			{
				ClassWithInterface = Class;
				break;
			}
		}
	}
	}

	return ClassWithInterface;
}

/**
 * This helper can be used to find a duplicate interface that is implemented higher
 * up the inheritance hierarchy (which can happen when you change parents or add an 
 * interface to a parent that's already implemented by a child).
 * 
 * @param  Interface	The interface you wish to find a duplicate of.
 * @param  Blueprint	The blueprint you wish to search.
 * @return True if one of the blueprint's super classes implements the specified interface, false if the child is free to implement it.
 */
static bool IsInterfaceImplementedByParent(FBPInterfaceDescription const& Interface, UBlueprint const* const Blueprint)
{
	check(Blueprint != nullptr);
	return (FindInheritedInterface(Blueprint->ParentClass, Interface) != nullptr);
}

/**
 * A helper function that takes two nodes belonging to the same graph and deletes 
 * one, replacing it with the other (moving over pin connections, etc.).
 * 
 * @param  OldNode	The node you want deleted.
 * @param  NewNode	The new replacement node that should take OldNode's place.
 */
static void ReplaceNode(UK2Node* OldNode, UK2Node* NewNode)
{
	check(OldNode->GetClass() == NewNode->GetClass());
	check(OldNode->GetOuter() == NewNode->GetOuter());

	UEdGraphSchema_K2 const* K2Schema = GetDefault<UEdGraphSchema_K2>();
	K2Schema->BreakNodeLinks(*NewNode);

	for (UEdGraphPin* OldPin : OldNode->Pins)
	{
		UEdGraphPin* NewPin = NewNode->FindPinChecked(OldPin->PinName);
		NewPin->MovePersistentDataFromOldPin(*OldPin);
	}

	NewNode->NodePosX = OldNode->NodePosX;
	NewNode->NodePosY = OldNode->NodePosY;

	FBlueprintEditorUtils::RemoveNode(OldNode->GetBlueprint(), OldNode, /* bDontRecompile =*/ true);
}

/**
 * Promotes any graphs that belong to the specified interface, and repurposes them
 * as parent overrides (function graphs that implement a parent's interface).
 * 
 * @param  Interface	The interface you're looking to promote.
 * @param  BlueprintObj	The blueprint that you want this applied to.
 */
static void PromoteInterfaceImplementationToOverride(FBPInterfaceDescription const& Interface, UBlueprint* const BlueprintObj)
{
	check(BlueprintObj != nullptr);
	// find the parent whose interface we're overriding 
	UClass* ParentClass = FindInheritedInterface(BlueprintObj->ParentClass, Interface);

	if (ParentClass != nullptr)
	{
		for (UEdGraph* InterfaceGraph : Interface.Graphs)
		{
			check(InterfaceGraph != nullptr);

			// The graph can be deleted now that it is a simple function override
			InterfaceGraph->bAllowDeletion = true;

			// Interface functions are ready to be a function graph outside the box, 
			// there will be no auto-call to parent though to maintain current functionality
			// in the graph
			BlueprintObj->FunctionGraphs.Add(InterfaceGraph);

			// No validation should be necessary here. Child blueprints will have interfaces conformed during their own compilation. 
		}

		// if any graphs were moved
		if (Interface.Graphs.Num() > 0)
		{
			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(BlueprintObj);
		}
	}
}

/**
 * Looks through the specified graph for any references to the specified 
 * variable, and renames them accordingly.
 * 
 * @param  InBlueprint		The blueprint that you want to search through.
 * @param  InVariableClass	The class that owns the variable that we're renaming
 * @param  InGraph			Graph to scope the rename to
 * @param  InOldVarName		The current name of the variable we want to replace
 * @param  InNewVarName		The name that we wish to change all references to
 */
static bool RenameVariableReferencesInGraph(UBlueprint* InBlueprint, UClass* InVariableClass, UEdGraph* InGraph, const FName& InOldVarName, const FName& InNewVarName)
{
	bool bFoundReference = false;

	for(UEdGraphNode* GraphNode : InGraph->Nodes)
	{
		// Allow node to handle variable renaming
		if (UK2Node* const K2Node = Cast<UK2Node>(GraphNode))
		{
			bFoundReference |= K2Node->ReferencesVariable(InOldVarName, nullptr);
			K2Node->HandleVariableRenamed(InBlueprint, InVariableClass, InGraph, InOldVarName, InNewVarName);
		}
	}

	return bFoundReference;
}


/**
 * Looks through the specified graph for any references to the specified 
 * function, and renames them accordingly.
 * 
 * @param  InBlueprint		The blueprint that you want to search through.
 * @param  InFunctionClass	The class that owns the function that we're renaming
 * @param  InGraph			Graph to scope the rename to
 * @param  InOldFuncName	The current name of the function we want to replace
 * @param  InNewFuncName	The name that we wish to change all references to
 */
static bool RenameFunctionReferencesInGraph(UBlueprint* InBlueprint, UClass* InFunctionClass, UEdGraph* InGraph, const FName& InOldFuncName, const FName& InNewFuncName)
{
	bool bFoundReference = false;

	for(UEdGraphNode* GraphNode : InGraph->Nodes)
	{
		// Allow node to handle function renaming
		if (UK2Node* const K2Node = Cast<UK2Node>(GraphNode))
		{
			bFoundReference |= K2Node->ReferencesFunction(InOldFuncName, nullptr);
			K2Node->HandleFunctionRenamed(InBlueprint, InFunctionClass, InGraph, InOldFuncName, InNewFuncName);
		}
	}

	return bFoundReference;
}

/**
 * Gathers all variable nodes from all graph's subgraph nodes
 *
 * @param Graph			        The Graph to search
 * @param OutVariableNodes		variable node array to write to
 */
static void GetAllChildGraphVariables(UEdGraph* Graph, TArray<UK2Node_Variable*>& OutVariableNodes)
{
	for (UEdGraph* SubGraph : Graph->SubGraphs)
	{
		check(SubGraph != nullptr);
		SubGraph->GetNodesOfClass<UK2Node_Variable>(OutVariableNodes);
		if (!SubGraph->SubGraphs.IsEmpty())
		{
			GetAllChildGraphVariables(SubGraph, OutVariableNodes);
		}
	}
}

namespace UE::Private
{
	static void ForEachObjectReference(const UObject* RootObject, TFunctionRef<void(const UObject*)> Function);
}

void UE::Private::ForEachObjectReference(const UObject* RootObject, TFunctionRef<void(const UObject*)> Function)
{
	TSet<const UObject*> VisitedObjects;
	TArray<const UObject*> Queue;
	Queue.Add(RootObject);
	VisitedObjects.Add(RootObject);
	while(Queue.Num() != 0)
	{
		const UObject* CurrentObject = Queue.Pop();

		TArray<UObject*> References;
		FReferenceFinder Finder( References );
		Finder.FindReferences( const_cast<UObject*>(CurrentObject) );

		for(UObject* Obj: References) //-V1078
		{
			if(VisitedObjects.Contains(Obj))
			{
				continue;
			}
			if(Obj->IsIn(RootObject))
			{
				Queue.Add(Obj);
			}
			VisitedObjects.Add(Obj);
			Function(Obj);
		}
	}
}

FBlueprintEditorUtils::FOnRenameVariableReferences FBlueprintEditorUtils::OnRenameVariableReferencesEvent;

void FBlueprintEditorUtils::RenameVariableReferences(UBlueprint* Blueprint, UClass* VariableClass, const FName& OldVarName, const FName& NewVarName)
{
	TArray<UEdGraph*> AllGraphs;
	Blueprint->GetAllGraphs(AllGraphs);

	// Update any graph nodes that reference the old variable name to instead reference the new name
	for(UEdGraph* CurrentGraph : AllGraphs)
	{
		if (RenameVariableReferencesInGraph(Blueprint, VariableClass, CurrentGraph, OldVarName, NewVarName))
		{
			MarkBlueprintAsModified(Blueprint);
		}
	}

	OnRenameVariableReferencesEvent.Broadcast(Blueprint, VariableClass, OldVarName, NewVarName);
}

FBlueprintEditorUtils::FOnRenameFunctionReferences FBlueprintEditorUtils::OnRenameFunctionReferencesEvent;

void FBlueprintEditorUtils::RenameFunctionReferences(UBlueprint* Blueprint, UClass* FunctionClass, const FName& OldFuncName, const FName& NewFuncName)
{
	TArray<UEdGraph*> AllGraphs;
	Blueprint->GetAllGraphs(AllGraphs);

	// Update any graph nodes that reference the old function name to instead reference the new name
	for(UEdGraph* CurrentGraph : AllGraphs)
	{
		if (RenameFunctionReferencesInGraph(Blueprint, FunctionClass, CurrentGraph, OldFuncName, NewFuncName))
		{
			MarkBlueprintAsModified(Blueprint);
		}
	}

	OnRenameFunctionReferencesEvent.Broadcast(Blueprint, FunctionClass, OldFuncName, NewFuncName);
}

//////////////////////////////////////
// FBasePinChangeHelper

void FBasePinChangeHelper::Broadcast(UBlueprint* InBlueprint, UK2Node_EditablePinBase* InTargetNode, UEdGraph* Graph)
{
	UK2Node_Tunnel* TunnelNode = Cast<UK2Node_Tunnel>(InTargetNode);
	const UK2Node_FunctionTerminator* FunctionDefNode = Cast<const UK2Node_FunctionTerminator>(InTargetNode);
	const UK2Node_Event* EventNode = Cast<const UK2Node_Event>(InTargetNode);
	if (TunnelNode)
	{
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(Graph);

		const bool bIsTopLevelFunctionGraph = Blueprint->MacroGraphs.Contains(Graph);

		if (bIsTopLevelFunctionGraph)
		{
			// Editing a macro, hit all loaded instances (in open blueprints)
			for (TObjectIterator<UK2Node_MacroInstance> It(RF_Transient); It; ++It)
			{
				UK2Node_MacroInstance* MacroInstance = *It;
				if (NodeIsNotTransient(MacroInstance) && (MacroInstance->GetMacroGraph() == Graph))
				{
					EditMacroInstance(MacroInstance, FBlueprintEditorUtils::FindBlueprintForNode(MacroInstance));
				}
			}
		}
		else if(NodeIsNotTransient(TunnelNode))
		{
			// Editing a composite node, hit the single instance in the parent graph		
			EditCompositeTunnelNode(TunnelNode);
		}
	}
	else if (FunctionDefNode || EventNode)
	{
		UFunction* Func = FFunctionFromNodeHelper::FunctionFromNode(FunctionDefNode ? static_cast<const UK2Node*>(FunctionDefNode) : static_cast<const UK2Node*>(EventNode));
		const FName FuncName = Func 
			? Func->GetFName() 
			: (FunctionDefNode ? FunctionDefNode->FunctionReference.GetMemberName() : EventNode->GetFunctionName());
		const UClass* SignatureClass = Func
			? Func->GetOwnerClass()
			: (UClass*)(FunctionDefNode ? FunctionDefNode->FunctionReference.GetMemberParentClass() : nullptr);

		const bool bIsInterface = FBlueprintEditorUtils::IsInterfaceBlueprint(InBlueprint);

		// Reconstruct all function call sites that call this function (in open blueprints)
		for (TObjectIterator<UK2Node_CallFunction> It(RF_Transient); It; ++It)
		{
			UK2Node_CallFunction* CallSite = *It;
			if (NodeIsNotTransient(CallSite))
			{
				UBlueprint* CallSiteBlueprint = FBlueprintEditorUtils::FindBlueprintForNode(CallSite);
				if (!CallSiteBlueprint)
				{
					// the node doesn't have a Blueprint in its outer chain, 
					// probably signifying that it is part of a graph that has 
					// been removed by the user (and moved off the Blueprint)
					continue;
				}
				
				const bool bValidSchema = CallSite->GetSchema() != nullptr;
				const bool bNameMatches = (CallSite->FunctionReference.GetMemberName() == FuncName);
				if (bNameMatches && bValidSchema)
				{
					if (bIsInterface)
					{
						if (FBlueprintEditorUtils::FindFunctionInImplementedInterfaces(CallSiteBlueprint, FuncName))
						{
							EditCallSite(CallSite, CallSiteBlueprint);
						}
					}
					else
					{
						const UClass* MemberParentClass = CallSite->FunctionReference.GetMemberParentClass(CallSite->GetBlueprintClassFromNode());
						const bool bClassMatchesEasy = (MemberParentClass != nullptr)
							&& ((SignatureClass != nullptr && MemberParentClass->IsChildOf(SignatureClass)) || MemberParentClass->IsChildOf(InBlueprint->GeneratedClass));
						const bool bClassMatchesHard = !bClassMatchesEasy && CallSite->FunctionReference.IsSelfContext() && (SignatureClass == nullptr)
							&& (CallSiteBlueprint == InBlueprint || (CallSiteBlueprint->SkeletonGeneratedClass && CallSiteBlueprint->SkeletonGeneratedClass->IsChildOf(InBlueprint->SkeletonGeneratedClass)));

						if (bClassMatchesEasy || bClassMatchesHard)
						{
							EditCallSite(CallSite, CallSiteBlueprint);
						}
					}
				}
			}
		}

		if(FBlueprintEditorUtils::IsDelegateSignatureGraph(Graph))
		{
			FName GraphName = Graph->GetFName();
			for (TObjectIterator<UK2Node_BaseMCDelegate> It(RF_Transient); It; ++It)
			{
				if(NodeIsNotTransient(*It) && (GraphName == It->GetPropertyName()))
				{
					UBlueprint* CallSiteBlueprint = FBlueprintEditorUtils::FindBlueprintForNode(*It);
					EditDelegates(*It, CallSiteBlueprint);
				}
			}
		}

		for (TObjectIterator<UK2Node_CreateDelegate> It(RF_Transient); It; ++It)
		{
			if(NodeIsNotTransient(*It))
			{
				EditCreateDelegates(*It);
			}
		}
	}
}

//////////////////////////////////////
// FParamsChangedHelper

void FParamsChangedHelper::EditCompositeTunnelNode(UK2Node_Tunnel* TunnelNode)
{
	if (TunnelNode->InputSinkNode != nullptr)
	{
		TunnelNode->InputSinkNode->ReconstructNode();
	}

	if (TunnelNode->OutputSourceNode != nullptr)
	{
		TunnelNode->OutputSourceNode->ReconstructNode();
	}
}

void FParamsChangedHelper::EditMacroInstance(UK2Node_MacroInstance* MacroInstance, UBlueprint* Blueprint)
{
	MacroInstance->ReconstructNode();
	if (Blueprint)
	{
		ModifiedBlueprints.Add(Blueprint);
	}
}

void FParamsChangedHelper::EditCallSite(UK2Node_CallFunction* CallSite, UBlueprint* Blueprint)
{
	CallSite->Modify();
	CallSite->ReconstructNode();
	if (Blueprint != nullptr)
	{
		ModifiedBlueprints.Add(Blueprint);
	}
}

void FParamsChangedHelper::EditDelegates(UK2Node_BaseMCDelegate* CallSite, UBlueprint* Blueprint)
{
	CallSite->Modify();
	CallSite->ReconstructNode();
	if (UK2Node_AddDelegate* AssignNode = Cast<UK2Node_AddDelegate>(CallSite))
	{
		if (UEdGraphPin* DelegateInPin = AssignNode->GetDelegatePin())
		{
			for (UEdGraphPin* DelegateOutPin : DelegateInPin->LinkedTo)
			{
				if (UK2Node_CustomEvent* CustomEventNode = (DelegateOutPin ? Cast<UK2Node_CustomEvent>(DelegateOutPin->GetOwningNode()) : nullptr))
				{
					CustomEventNode->ReconstructNode();
				}
			}
		}
	}
	if (Blueprint != nullptr)
	{
		ModifiedBlueprints.Add(Blueprint);
	}
}

void FParamsChangedHelper::EditCreateDelegates(UK2Node_CreateDelegate* CallSite)
{
	UBlueprint* Blueprint = nullptr;
	UEdGraph* Graph = nullptr;
	CallSite->HandleAnyChange(Graph, Blueprint);
	if(Blueprint)
	{
		ModifiedBlueprints.Add(Blueprint);
	}
	if(Graph)
	{
		ModifiedGraphs.Add(Graph);
	}
}

//////////////////////////////////////
// FUCSComponentId

FUCSComponentId::FUCSComponentId(const UK2Node_AddComponent* UCSNode)
	: GraphNodeGuid(UCSNode->NodeGuid)
{
}

//////////////////////////////////////
// FBlueprintEditorUtils

FBlueprintEditorUtils::FOnRefreshAllNodes FBlueprintEditorUtils::OnRefreshAllNodesEvent;

namespace UE::Private
{
bool UNREALED_API GraphContainsSetSkeletalMeshAssetNode(UEdGraph* Graph)
{
	struct FNodeGraphTraverser
	{
		TSet<UEdGraphNode*> VisitedNodes;
		
		bool TraverseGraph(UEdGraph* Graph)
		{
			bool bFoundSetSkeletalMeshCall = false;
			for (int32 Index = 0; Index < Graph->Nodes.Num(); Index++)
			{
				UK2Node_CallFunction* FunctionNode = Cast<UK2Node_CallFunction>(Graph->Nodes[Index]);
				if (FunctionNode && !VisitedNodes.Contains(FunctionNode))
				{
					VisitedNodes.Add(FunctionNode);

					const UEdGraphNode* ResultEventNode = nullptr;
					UEdGraph* FunctionGraph = FunctionNode->GetFunctionGraph(ResultEventNode);
					if (FunctionGraph)
					{
						bFoundSetSkeletalMeshCall = TraverseGraph(FunctionGraph);
						if (bFoundSetSkeletalMeshCall)
						{
							break;
						}
					}
					else if (FunctionNode->FunctionReference.GetMemberName() == GET_FUNCTION_NAME_CHECKED(USkeletalMeshComponent, SetSkeletalMeshAsset))
					{
						bFoundSetSkeletalMeshCall = true;
						break;
					}
				}
				else
				{
					UK2Node_MacroInstance* MacroInstanceNode = Cast<UK2Node_MacroInstance>(Graph->Nodes[Index]);
					if (MacroInstanceNode && !VisitedNodes.Contains(MacroInstanceNode))
					{
						VisitedNodes.Add(MacroInstanceNode);

						UEdGraph* MacroGraph = MacroInstanceNode->GetMacroGraph();
						if (MacroGraph)
						{
							bFoundSetSkeletalMeshCall = TraverseGraph(MacroGraph);
							if (bFoundSetSkeletalMeshCall)
							{
								break;
							}
						}
					}
				}
				
			}

			return bFoundSetSkeletalMeshCall;
		}
	};

	FNodeGraphTraverser GraphTraverser;
	return GraphTraverser.TraverseGraph(Graph);	
}
}

void FBlueprintEditorUtils::RefreshAllNodes(UBlueprint* Blueprint)
{
	if (!Blueprint || (Blueprint->HasAllFlags(RF_WasLoaded) && !Blueprint->HasAllFlags(RF_LoadCompleted)))
	{
		UE_LOGF(LogBlueprint, Warning, 
			"RefreshAllNodes was called on an invalid or incompletely loaded blueprint '%ls'", 
			Blueprint ? *Blueprint->GetFullName() : TEXT("NULL"));
		return;
	}

	TArray<UK2Node*> AllNodes;
	FBlueprintEditorUtils::GetAllNodesOfClass(Blueprint, AllNodes);

	const bool bIsMacro = (Blueprint->BlueprintType == BPTYPE_MacroLibrary);
	SortNodes(AllNodes);

	bool bLastChangesStructure = (AllNodes.Num() > 0) ? AllNodes[0]->NodeCausesStructuralBlueprintChange() : true;
	for( TArray<UK2Node*>::TIterator NodeIt(AllNodes); NodeIt; ++NodeIt )
	{
		UK2Node* CurrentNode = *NodeIt;

		// See if we've finished the batch of nodes that affect structure, and recompile the skeleton if needed
		const bool bCurrentChangesStructure = CurrentNode->NodeCausesStructuralBlueprintChange();
		if( bLastChangesStructure != bCurrentChangesStructure )
		{
			// Make sure sorting was valid!
			check(bLastChangesStructure && !bCurrentChangesStructure);

			// Recompile the skeleton class, now that all changes to entry point structure has taken place
			// Ignore this for macros
			if (!bIsMacro)
			{
				FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
			}
			bLastChangesStructure = bCurrentChangesStructure;
		}

		//@todo:  Do we really need per-schema refreshing?
		const UEdGraphSchema* Schema = CurrentNode->GetGraph()->GetSchema();
		Schema->ReconstructNode(*CurrentNode, true);
	}

	// If all nodes change structure, catch that case and recompile now
	if( bLastChangesStructure )
	{
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
	}

	OnRefreshAllNodesEvent.Broadcast(Blueprint);
}

FBlueprintEditorUtils::FOnReconstructAllNodes FBlueprintEditorUtils::OnReconstructAllNodesEvent;

void FBlueprintEditorUtils::ReconstructAllNodes(UBlueprint* Blueprint)
{
	if (!Blueprint || (Blueprint->HasAllFlags(RF_WasLoaded) && !Blueprint->HasAllFlags(RF_LoadCompleted)))
	{
		UE_LOGF(LogBlueprint, Warning,
			"ReconstructAllNodes was called on an invalid or incompletely loaded blueprint '%ls'",
			Blueprint ? *Blueprint->GetFullName() : TEXT("NULL"));
		return;
	}

	TArray<UK2Node*> AllNodes;
	FBlueprintEditorUtils::GetAllNodesOfClass(Blueprint, AllNodes);

	SortNodes(AllNodes, true);

	for (TArray<UK2Node*>::TIterator NodeIt(AllNodes); NodeIt; ++NodeIt)
	{
		UK2Node* CurrentNode = *NodeIt;
		//@todo:  Do we really need per-schema refreshing?
		const UEdGraphSchema* Schema = CurrentNode->GetGraph()->GetSchema();
		Schema->ReconstructNode(*CurrentNode, true);
	}

	OnReconstructAllNodesEvent.Broadcast(Blueprint);
}

void FBlueprintEditorUtils::ReplaceDeprecatedNodes(UBlueprint* Blueprint)
{
	Blueprint->ReplaceDeprecatedNodes();
}

void FBlueprintEditorUtils::RefreshExternalBlueprintDependencyNodes(UBlueprint* Blueprint, UStruct* RefreshOnlyChild)
{
	BP_SCOPED_COMPILER_EVENT_STAT(EKismetCompilerStats_RefreshExternalDependencyNodes);

	if (!Blueprint || (Blueprint->HasAllFlags(RF_WasLoaded) && !Blueprint->HasAllFlags(RF_LoadCompleted)))
	{
		UE_LOGF(LogBlueprint, Warning,
			"RefreshExternalBlueprintDependencyNodes was called on an invalid or incompletely loaded blueprint '%ls'",
			Blueprint ? *Blueprint->GetFullName() : TEXT("NULL"));
		return;
	}

	TArray<UK2Node*> AllNodes;
	FBlueprintEditorUtils::GetAllNodesOfClass(Blueprint, AllNodes);

	if (!RefreshOnlyChild)
	{
		for (UK2Node* Node : AllNodes)
		{
			if (Node->HasExternalDependencies())
			{
				//@todo:  Do we really need per-schema refreshing?
				const UEdGraphSchema* Schema = Node->GetGraph()->GetSchema();
				Schema->ReconstructNode(*Node, true);
			}
		}
	}
	else
	{
		for (UK2Node* Node : AllNodes)
		{
			TArray<UStruct*> Dependencies;
			if (Node->HasExternalDependencies(&Dependencies))
			{
				for (UStruct* Struct : Dependencies)
				{
					bool bShouldRefresh = Struct->IsChildOf(RefreshOnlyChild);
					if (!bShouldRefresh)
					{
						UClass* OwnerClass = Struct->GetOwnerClass();
						if (ensureMsgf(!OwnerClass || !OwnerClass->GetClass()->IsChildOf<UBlueprintGeneratedClass>() || OwnerClass->ClassGeneratedBy
							, TEXT("Malformed Blueprint class (%s) - bad node dependency, unable to determine if the %s node (%s) should be refreshed or not. Currently compiling: %s")
							, *OwnerClass->GetName()
							, *Node->GetClass()->GetName()
							, *Node->GetPathName()
							, *Blueprint->GetName()) )
						{
							bShouldRefresh |= OwnerClass &&
								(OwnerClass->IsChildOf(RefreshOnlyChild) || OwnerClass->GetAuthoritativeClass()->IsChildOf(RefreshOnlyChild));
							if (!bShouldRefresh && OwnerClass && Struct->IsA<UFunction>() && OwnerClass->HasAnyClassFlags(CLASS_Interface))
							{
								if (UClass* RefreshClass = Cast<UClass>(RefreshOnlyChild))
								{
									bShouldRefresh = RefreshClass->ImplementsInterface(OwnerClass);
								}
							}
						}						
					}
					if (bShouldRefresh)
					{
						//@todo:  Do we really need per-schema refreshing?
						const UEdGraphSchema* Schema = Node->GetGraph()->GetSchema();
						Schema->ReconstructNode(*Node, true);

						break;
					}
				}
			}
		}
	}
}

void FBlueprintEditorUtils::RefreshGraphNodes(const UEdGraph* Graph)
{
	TArray<UK2Node*> AllNodes;
	Graph->GetNodesOfClass(AllNodes);

	for (UK2Node* Node : AllNodes)
	{
		const UEdGraphSchema* Schema = Node->GetGraph()->GetSchema();
		Schema->ReconstructNode(*Node, true);
	}
}

void FBlueprintEditorUtils::PreloadMembers(UObject* InObject)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FBlueprintEditorUtils::PreloadMembers);

	// Collect a list of all things this element owns
	TArray<UObject*> BPMemberReferences;
	FReferenceFinder ComponentCollector(BPMemberReferences, InObject, false, true, true, true);
	ComponentCollector.FindReferences(InObject);

	// Iterate over the list, and preload everything so it is valid for refreshing
	for( TArray<UObject*>::TIterator it(BPMemberReferences); it; ++it )
	{
		UObject* CurrentObject = *it;
		CurrentObject->ConditionalPreload();
		PreloadMembers(CurrentObject);
	}
}

void FBlueprintEditorUtils::PreloadConstructionScript(UBlueprint* Blueprint)
{
	if (Blueprint)
	{
		PreloadConstructionScript(Blueprint->SimpleConstructionScript);
	}
}

void FBlueprintEditorUtils::PreloadConstructionScript(USimpleConstructionScript* SimpleConstructionScript)
{
	if (!SimpleConstructionScript)
	{
		return;
	}

	if (FLinkerLoad* TargetLinker = SimpleConstructionScript->GetLinker())
	{
		TargetLinker->Preload(SimpleConstructionScript);

		if (USCS_Node* DefaultSceneRootNode = SimpleConstructionScript->GetDefaultSceneRootNode())
		{
			DefaultSceneRootNode->PreloadChain();
		}

		const TArray<USCS_Node*>& RootNodes = SimpleConstructionScript->GetRootNodes();
		for (int32 NodeIndex = 0; NodeIndex < RootNodes.Num(); ++NodeIndex)
		{
			RootNodes[NodeIndex]->PreloadChain();
		}
	}

	for (USCS_Node* SCSNode : SimpleConstructionScript->GetAllNodes())
	{
		if (SCSNode)
		{
			SCSNode->ValidateGuid();
		}
	}
}

void FBlueprintEditorUtils::PatchNewCDOIntoLinker(UObject* CDO, FLinkerLoad* Linker, int32 ExportIndex, FUObjectSerializeContext* InLoadContext)
{
	if (CDO != nullptr && Linker != nullptr)
	{
		Linker->PRIVATE_PatchNewObjectIntoExport(ExportIndex, CDO, InLoadContext);
		CDO->CheckDefaultSubobjects();
	}
}

UClass* FBlueprintEditorUtils::FindFirstNativeClass(UClass* Class)
{
	for(; Class; Class = Class->GetSuperClass() )
	{
		if( 0 != (Class->ClassFlags & CLASS_Native))
		{
			break;
		}
	}
	return Class;
}

const UClass* FBlueprintEditorUtils::FindFirstNativeClass(const UClass* Class)
{
	return FindFirstNativeClass(const_cast<UClass*>(Class));
}

bool FBlueprintEditorUtils::IsNativeSignature(const UFunction* Fn)
{
	if(UClass* OwningClass = Fn->GetOwnerClass())
	{
		if(UClass* PotentialNativeOwner = FindFirstNativeClass(OwningClass))
		{
			return PotentialNativeOwner->FindFunctionByName(Fn->GetFName()) != nullptr;
		}
	}

	// All UFunctions are owned by UClasses, but if we encounter one that
	// is not lets just fall back to the flag:
	return Fn->HasAnyFunctionFlags(FUNC_Native);
}

void FBlueprintEditorUtils::GetAllGraphNames(const UBlueprint* Blueprint, TSet<FName>& GraphNames)
{
	TArray< UEdGraph* > GraphList;
	Blueprint->GetAllGraphs(GraphList);

	for(int32 GraphIdx = 0; GraphIdx < GraphList.Num(); ++GraphIdx)
	{
		GraphNames.Add(GraphList[GraphIdx]->GetFName());
	}

	// Include all functions from parents because they should never conflict
	TArray<UBlueprint*> ParentBPStack;
	UBlueprint::GetBlueprintHierarchyFromClass(Blueprint->SkeletonGeneratedClass, ParentBPStack);
	for (int32 StackIndex = ParentBPStack.Num() - 1; StackIndex >= 0; --StackIndex)
	{
		UBlueprint* ParentBP = ParentBPStack[StackIndex];
		check(ParentBP != nullptr);

		for(int32 FunctionIndex = 0; FunctionIndex < ParentBP->FunctionGraphs.Num(); ++FunctionIndex)
		{
			GraphNames.Add(ParentBP->FunctionGraphs[FunctionIndex]->GetFName());
		}
	}
}

void FBlueprintEditorUtils::GetCompilerRelevantNodeLinks(UEdGraphPin* FromPin, FCompilerRelevantNodeLinkArray& OutNodeLinks)
{
	if(FromPin)
	{
		// Start with the given pin's owning node
		UK2Node* OwningNode = Cast<UK2Node>(FromPin->GetOwningNode());
		if(OwningNode)
		{
			// If this node is not compiler relevant
			if(!OwningNode->IsCompilerRelevant())
			{
				// And if this node has a matching "pass-through" pin
				FromPin = OwningNode->GetPassThroughPin(FromPin);
				if(FromPin)
				{
					// Recursively check each link for a compiler-relevant node that will "pass through" this node at compile time
					for (UEdGraphPin* LinkedPin : FromPin->LinkedTo)
					{
						GetCompilerRelevantNodeLinks(LinkedPin, OutNodeLinks);
					}
				}
			}
			else
			{
				OutNodeLinks.Add(FCompilerRelevantNodeLink(OwningNode, FromPin));
			}
		}		
	}
}

UK2Node* FBlueprintEditorUtils::FindFirstCompilerRelevantNode(UEdGraphPin* FromPin)
{
	FCompilerRelevantNodeLinkArray RelevantNodeLinks;
	GetCompilerRelevantNodeLinks(FromPin, RelevantNodeLinks);
	
	return RelevantNodeLinks.Num() > 0 ? RelevantNodeLinks[0].Node : nullptr;
}

UEdGraphPin* FBlueprintEditorUtils::FindFirstCompilerRelevantLinkedPin(UEdGraphPin* FromPin)
{
	FCompilerRelevantNodeLinkArray RelevantNodeLinks;
	GetCompilerRelevantNodeLinks(FromPin, RelevantNodeLinks);

	return RelevantNodeLinks.Num() > 0 ? RelevantNodeLinks[0].LinkedPin : nullptr;
}

void FBlueprintEditorUtils::RemoveAllLocalBookmarks(const UBlueprint* ForBlueprint)
{
	if (ForBlueprint)
	{
		bool bSaveConfig = false;
		const FString BPPackageName = ForBlueprint->GetOutermost()->GetName();
		UBlueprintEditorSettings* LocalSettings = GetMutableDefault<UBlueprintEditorSettings>();
		for (int32 i = 0; i < LocalSettings->BookmarkNodes.Num(); ++i)
		{
			const FBPEditorBookmarkNode& BookmarkNode = LocalSettings->BookmarkNodes[i];
			const FEditedDocumentInfo* BookmarkInfo = LocalSettings->Bookmarks.Find(BookmarkNode.NodeGuid);
			if (BookmarkInfo != nullptr && BookmarkInfo->EditedObjectPath.GetLongPackageName() == BPPackageName)
			{
				bSaveConfig = true;

				LocalSettings->BookmarkNodes.RemoveAt(i--);
				LocalSettings->Bookmarks.Remove(BookmarkNode.NodeGuid);
			}
		}

		if (bSaveConfig)
		{
			LocalSettings->SaveConfig();
		}
	}
}

//////////////////////////////////////////////////////////////////////////

struct FRegenerationHelper
{
	static void PreloadAndLinkIfNecessary(UStruct* Struct)
	{
		bool bChanged = false;
		if (Struct->HasAnyFlags(RF_NeedLoad) || Struct->HasAnyInternalFlags(EInternalObjectFlags::AsyncLoadingPhase1))
		{
			if (FLinkerLoad* Linker = Struct->GetLinker())
			{
				UE::TUniqueLock LinkerLock(Linker->Mutex);
				bChanged = Struct->HasAnyFlags(RF_NeedLoad);
				Linker->Preload(Struct);
			}
		}

		const int32 OldPropertiesSize = Struct->GetPropertiesSize();
		for (UField* Field = Struct->Children; Field; Field = Field->Next)
		{
			bChanged |= UBlueprint::ForceLoad(Field);
		}

		if (bChanged)
		{
			Struct->StaticLink(true);
			ensure(Struct->IsA<UFunction>() || (OldPropertiesSize == Struct->GetPropertiesSize()) || !Struct->HasAnyFlags(RF_LoadCompleted));
			
			// UStruct::Link is going to attempt to set the StructFlags, but it has no knowledge of UserDefinedStruct::DefaultStructInstance.
			// We don't want to set CPF_ZeroConstructor if UserDefinedStruct::DefaultStructInstance has non zero data, so this call
			// gives UUserDefinedStruct a chance to enforce its invariants. The cooked/EDL path relies on setting struct flags in 
			// serialize, after link has been called.
			if(UUserDefinedStruct* AsUDS = Cast<UUserDefinedStruct>(Struct))
			{
				AsUDS->UpdateStructFlags();
			}
		}
	}

	static UBlueprint* GetGeneratingBlueprint(const UObject* Obj)
	{
		const UBlueprintGeneratedClass* BPGC = nullptr;
		while (!BPGC && Obj)
		{
			BPGC = Cast<const UBlueprintGeneratedClass>(Obj);
			Obj = Obj->GetOuter();
		}

		return UBlueprint::GetBlueprintFromClass(BPGC);
	}

	static void ProcessHierarchy(UStruct* Struct, TSet<UStruct*>& Dependencies)
	{
		if (Struct)
		{
			bool bAlreadyProcessed = false;
			Dependencies.Add(Struct, &bAlreadyProcessed);
			if (!bAlreadyProcessed)
			{
				ProcessHierarchy(Struct->GetSuperStruct(), Dependencies);

				const UBlueprint* BP = GetGeneratingBlueprint(Struct);
				const bool bProcessBPGClass = BP && !BP->bHasBeenRegenerated;
				const bool bProcessUserDefinedStruct = Struct->IsA<UUserDefinedStruct>();
				if (bProcessBPGClass || bProcessUserDefinedStruct)
				{
					PreloadAndLinkIfNecessary(Struct);
				}
			}
		}
	}

	static void PreloadMacroSources(TSet<UBlueprint*>& MacroSources)
	{
		for (UBlueprint* BP : MacroSources)
		{
			if (!BP->bHasBeenRegenerated)
			{
				BP->ConditionalPreload();

				// at the point of blueprint regeneration (on load), we are guaranteed that blueprint dependencies (like this macro) have
				// fully formed classes (meaning the blueprint class and all its direct dependencies have been loaded)... however, we do not 
				// get the guarantee that all of that blueprint's graph dependencies are loaded (hence, why we have to force load 
				// everything here); in the case of cyclic dependencies, macro dependencies could already be loaded, but in the midst of 
				// resolving thier own dependency placeholders (why a ForceLoad() call is not enough); this ensures that 
				// placeholder objects are properly resolved on nodes that will be injected by macro expansion
				FLinkerLoad::PRIVATE_ForceLoadAllDependencies(BP->GetOutermost());
				
				UBlueprint::ForceLoadMembers(BP);
			}
		}
	}

	/**
	 * A helper function that loads (and regenerates) interface dependencies.
	 * Accounts for circular dependencies by following how we handle parent 
	 * classes in FLinkerLoad::RegenerateBlueprintClass() (that is, to complete 
	 * the interface's compilation/regeneration before we utilize it for the
	 * specified blueprint).
	 * 
	 * @param  Blueprint	The blueprint whose implemented interfaces you want loaded.
	 */
	static void PreloadInterfaces(UBlueprint* Blueprint)
	{
#if WITH_EDITORONLY_DATA // ImplementedInterfaces is wrapped WITH_EDITORONLY_DATA 
		for (FBPInterfaceDescription const& InterfaceDesc : Blueprint->ImplementedInterfaces)
		{
			UClass* InterfaceClass = InterfaceDesc.Interface;
			UBlueprint* InterfaceBlueprint = InterfaceClass ? Cast<UBlueprint>(InterfaceClass->ClassGeneratedBy) : nullptr;
			if (InterfaceBlueprint)
			{
				UBlueprint::ForceLoadMembers(InterfaceBlueprint);
				if (InterfaceBlueprint->HasAnyFlags(RF_BeingRegenerated))
				{
					InterfaceBlueprint->RegenerateClass(InterfaceClass, InterfaceClass->GetDefaultObject(false));
				}
			}
		}
#endif // #if WITH_EDITORONLY_DATA
	}

	static void LinkExternalDependencies(UBlueprint* Blueprint)
	{
		check(Blueprint);
		const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
		TSet<UStruct*> Dependencies;
		ProcessHierarchy(Blueprint->ParentClass, Dependencies);
		
		for (const FBPVariableDescription& NewVar : Blueprint->NewVariables)
		{
			if (UObject* TypeObject = NewVar.VarType.PinSubCategoryObject.Get())
			{
				TypeObject->ConditionalPreload();
			}

			if (UClass* TypeClass = NewVar.VarType.PinSubCategoryMemberReference.GetMemberParentClass())
			{
				TypeClass->ConditionalPreload();
			}
		}

		TSet<UBlueprint*> MacroSources;
		TArray<UEdGraph*> Graphs;
		Blueprint->GetAllGraphs(Graphs);
		for (UEdGraph* Graph : Graphs)
		{
			if (Graph && !FBlueprintEditorUtils::IsGraphIntermediate(Graph))
			{
				const bool bIsDelegateSignatureGraph = FBlueprintEditorUtils::IsDelegateSignatureGraph(Graph);

				TArray<UK2Node*> Nodes;
				Graph->GetNodesOfClass(Nodes);
				for (UK2Node* Node : Nodes)
				{
					if (Node)
					{
						TArray<UStruct*> LocalDependentStructures;
						if (Node->HasExternalDependencies(&LocalDependentStructures))
						{
							for (UStruct* Struct : LocalDependentStructures)
							{
								ProcessHierarchy(Struct, Dependencies);
							}

							if (UK2Node_MacroInstance* MacroNode = Cast<UK2Node_MacroInstance>(Node))
							{
								if (UBlueprint* MacroSource = MacroNode->GetSourceBlueprint())
								{
									MacroSources.Add(MacroSource);
								}
							}
						}

						UK2Node_FunctionEntry* FunctionEntry = Cast<UK2Node_FunctionEntry>(Node);
						if (FunctionEntry && !bIsDelegateSignatureGraph)
						{
							const FName FunctionName = (FunctionEntry->CustomGeneratedFunctionName != NAME_None) 
								? FunctionEntry->CustomGeneratedFunctionName 
								: FunctionEntry->FunctionReference.GetMemberName();
							UFunction* ParentFunction = Blueprint->ParentClass ? Blueprint->ParentClass->FindFunctionByName(FunctionName) : nullptr;
							if (ParentFunction && (UEdGraphSchema_K2::FN_UserConstructionScript != FunctionName))
							{
								ProcessHierarchy(ParentFunction, Dependencies);
							}
						}

						// load Enums
						for (UEdGraphPin* Pin : Node->Pins)
						{
							UObject* SubCategoryObject = Pin ? Pin->PinType.PinSubCategoryObject.Get() : nullptr;
							if (SubCategoryObject && SubCategoryObject->IsA<UEnum>())
							{
								UBlueprint::ForceLoad(SubCategoryObject);
							}
						}
					}
				}
			}
		}
		PreloadMacroSources(MacroSources);

		PreloadInterfaces(Blueprint);
	}
};

/**
	Procedure used to remove old function implementations and child properties from data only blueprints.
	These blueprints have a 'fast path' compilation path but we need to make sure that any data regenerated 
	by normal blueprint compilation is cleared here. If we don't then these functions and properties will
	hang around when a class is converted from a real blueprint to a data only blueprint.
*/
void FBlueprintEditorUtils::RemoveStaleFunctions(UBlueprintGeneratedClass* Class, UBlueprint* Blueprint)
{
	if (Class == nullptr)
	{
		return;
	}

	// Removes all existing functions from the class, currently used 
	TFieldIterator<UFunction> Fn(Class, EFieldIteratorFlags::ExcludeSuper);
	if (Fn)
	{
		FString OrphanedClassString = FString::Printf(TEXT("ORPHANED_DATA_ONLY_%s"), *Class->GetName());
		FName OrphanedClassName = MakeUniqueObjectName(GetTransientPackage(), UBlueprintGeneratedClass::StaticClass(), FName(*OrphanedClassString));
		UClass* OrphanedClass = NewObject<UBlueprintGeneratedClass>(GetTransientPackage(), OrphanedClassName, RF_Public | RF_Transient);
		OrphanedClass->CppClassStaticFunctions = Class->CppClassStaticFunctions;
		OrphanedClass->ClassFlags |= CLASS_CompiledFromBlueprint;
		OrphanedClass->ClassGeneratedBy = Class->ClassGeneratedBy;

		const ERenameFlags RenFlags = REN_DontCreateRedirectors | REN_NonTransactional | REN_DoNotDirty | (Blueprint->bIsRegeneratingOnLoad ? REN_AllowPackageLinkerMismatch : 0);

		while (Fn)
		{
			UFunction* Function = *Fn;
			Class->RemoveFunctionFromFunctionMap(Function);

			// invalidate this package's reference to this function, so 
			// subsequent packages that import it will treat it as if it didn't 
			// exist (because data-only blueprints shouldn't have functions)
            // Note, Rename will remove the renamed object's linker when moving 
            // to a new package so invalidate the export beforehand
			FLinkerLoad::InvalidateExport(Function); 
			Function->Rename(nullptr, OrphanedClass, RenFlags);
			++Fn;
		}
	}

	// Clear function map caches which will be rebuilt the next time functions are searched by name
	Class->ClearFunctionMapsCaches();

	Blueprint->GeneratedClass->Children = nullptr;
	Blueprint->GeneratedClass->Bind();
	Blueprint->GeneratedClass->StaticLink(true);
}

void FBlueprintEditorUtils::RefreshVariables(UBlueprint* Blueprint)
{
	// module punchthrough:
	IKismetCompilerInterface& Compiler = FModuleManager::LoadModuleChecked<IKismetCompilerInterface>(KISMET_COMPILER_MODULENAME);
	Compiler.RefreshVariables(Blueprint);
}

void FBlueprintEditorUtils::PreloadBlueprintSpecificData(UBlueprint* Blueprint)
{
	TArray<UK2Node*> AllNodes;
	FBlueprintEditorUtils::GetAllNodesOfClass(Blueprint, AllNodes);

	for( UK2Node* K2Node : AllNodes )
	{
		K2Node->PreloadRequiredAssets();
	}
}

void FBlueprintEditorUtils::LinkExternalDependencies(UBlueprint* Blueprint)
{
	FRegenerationHelper::LinkExternalDependencies(Blueprint);
}

void FBlueprintEditorUtils::RecreateClassMetaData(UBlueprint* Blueprint, UClass* Class, bool bRemoveExistingMetaData)
{
	if (!ensure(Blueprint && Class))
	{
		return;
	}

	UClass* ParentClass = Class->GetSuperClass();
	TArray<FString> AllHideCategories;

	if (bRemoveExistingMetaData)
	{
		Class->RemoveMetaData("HideCategories");
		Class->RemoveMetaData("ShowCategories");
		Class->RemoveMetaData("HideFunctions");
		Class->RemoveMetaData("AutoExpandCategories");
		Class->RemoveMetaData("AutoCollapseCategories");
		Class->RemoveMetaData("PrioritizeCategories");
		Class->RemoveMetaData("SparseClassDataTypes");
		Class->RemoveMetaData("ClassGroupNames");
		Class->RemoveMetaData("Category");
		Class->RemoveMetaData(FBlueprintMetadata::MD_AllowableBlueprintVariableType);
	}

	if (ensure(ParentClass != nullptr))
	{
		if (!ParentClass->HasMetaData(FBlueprintMetadata::MD_IgnoreCategoryKeywordsInSubclasses))
		{
			// we want the categories just as they appear in the parent class 
			// (set bHomogenize to false) - especially since homogenization 
			// could inject spaces
			FEditorCategoryUtils::GetClassHideCategories(ParentClass, AllHideCategories, /*bHomogenize =*/false);
			if (ParentClass->HasMetaData(TEXT("ShowCategories")))
			{
				Class->SetMetaData(TEXT("ShowCategories"), *ParentClass->GetMetaData("ShowCategories"));
			}
			if (ParentClass->HasMetaData(TEXT("AutoExpandCategories")))
			{
				Class->SetMetaData(TEXT("AutoExpandCategories"), *ParentClass->GetMetaData("AutoExpandCategories"));
			}
			if (ParentClass->HasMetaData(TEXT("AutoCollapseCategories")))
			{
				Class->SetMetaData(TEXT("AutoCollapseCategories"), *ParentClass->GetMetaData("AutoCollapseCategories"));
			}
			if (ParentClass->HasMetaData(TEXT("PrioritizeCategories")))
			{
				Class->SetMetaData(TEXT("PrioritizeCategories"), *ParentClass->GetMetaData("PrioritizeCategories"));
			}
		}

		if (ParentClass->HasMetaData(TEXT("HideFunctions")))
		{
			Class->SetMetaData(TEXT("HideFunctions"), *ParentClass->GetMetaData("HideFunctions"));
		}

		if (ParentClass->IsChildOf(UActorComponent::StaticClass()))
		{
			static const FName NAME_ClassGroupNames(TEXT("ClassGroupNames"));
			Class->SetMetaData(FBlueprintMetadata::MD_BlueprintSpawnableComponent, TEXT("true"));

			FString ClassGroupCategory = NSLOCTEXT("BlueprintableComponents", "CategoryName", "Custom").ToString();
			if (!Blueprint->BlueprintCategory.IsEmpty())
			{
				ClassGroupCategory = Blueprint->BlueprintCategory;
			}

			Class->SetMetaData(NAME_ClassGroupNames, *ClassGroupCategory);
		}

		if (ParentClass->HasMetaData(TEXT("SparseClassDataTypes")))
		{
			Class->SetMetaData(TEXT("SparseClassDataTypes"), *ParentClass->GetMetaData("SparseClassDataTypes"));
		}
	}

	// Add a category if one has been specified
	if (Blueprint->BlueprintCategory.Len() > 0)
	{
		Class->SetMetaData(TEXT("Category"), *Blueprint->BlueprintCategory);
	}
	else
	{
		Class->RemoveMetaData(TEXT("Category"));
	}

	if ((Blueprint->BlueprintType == BPTYPE_Normal) ||
		(Blueprint->BlueprintType == BPTYPE_Const) ||
		(Blueprint->BlueprintType == BPTYPE_Interface))
	{
		Class->SetMetaData(FBlueprintMetadata::MD_AllowableBlueprintVariableType, TEXT("true"));
	}

	for (FString HideCategory : Blueprint->HideCategories)
	{
		TArray<TCHAR, FString::AllocatorType>& CharArray = HideCategory.GetCharArray();

		int32 SpaceIndex = CharArray.Find(TEXT(' '));
		while (SpaceIndex != INDEX_NONE)
		{
			CharArray.RemoveAt(SpaceIndex);
			if (SpaceIndex >= CharArray.Num())
			{
				break;
			}

			TCHAR& WordStartingChar = CharArray[SpaceIndex];
			WordStartingChar = FChar::ToUpper(WordStartingChar);

			CharArray.Find(TEXT(' '), SpaceIndex);
		}
		AllHideCategories.Add(HideCategory);
	}

	if (AllHideCategories.Num() > 0)
	{
		Class->SetMetaData(TEXT("HideCategories"), *FString::Join(AllHideCategories, TEXT(" ")));
	}
	else
	{
		Class->RemoveMetaData(TEXT("HideCategories"));
	}
}


void FBlueprintEditorUtils::PatchCDOSubobjectsIntoExport(UObject* PreviousCDO, UObject* NewCDO)
{
	struct PatchCDOSubobjectsIntoExport_Impl
	{
		static void PatchSubObjects(UObject* OldObj, UObject* NewObj)
		{
			// Find all subobjects outered to the old object.
			TArray<UObject*> OldSubObjects;
			GetObjectsWithOuter(OldObj, OldSubObjects, EGetObjectsFlags::None);

			// Exit now if we don't have any subobjects to process.
			if (OldSubObjects.Num() == 0)
			{
				return;
			}

			// Build a lookup table for subobjects outered to the new object.
			TMap<FName, UObject*> NewSubObjLookupTable;
			NewSubObjLookupTable.Reserve(OldSubObjects.Num());
			ForEachObjectWithOuter(NewObj, [&NewSubObjLookupTable](UObject* NewSubObj)
			{
				NewSubObjLookupTable.Add(NewSubObj->GetFName(), NewSubObj);
			},
				EGetObjectsFlags::None);

			// For each subobject that's outered to the old object, find the corresponding new subobject that matches its name and patch it into the export table.
			// This includes any editinline subobjects that are instanced at edit time; the outer object (e.g. CDO) may not have a matching archetype in that case.
			for (UObject* OldSubObj : OldSubObjects)
			{
				if (UObject** NewSubObjPtr = NewSubObjLookupTable.Find(OldSubObj->GetFName()))
				{
					UObject* NewSubObj = *NewSubObjPtr;
					FLinkerLoad::PRIVATE_PatchNewObjectIntoExport(OldSubObj, NewSubObj);
				}
				else
				{
					// Invalidate any existing entry in the export table to avoid generation of implicit dependencies.
					FLinkerLoad::InvalidateExport(OldSubObj);
				}
			}
		}
	};

	if (PreviousCDO && NewCDO)
	{
		PatchCDOSubobjectsIntoExport_Impl::PatchSubObjects(PreviousCDO, NewCDO);
		NewCDO->CheckDefaultSubobjects();
	}
}

void FBlueprintEditorUtils::PropagateParentBlueprintDefaults(UClass* ClassToPropagate)
{
	check(ClassToPropagate);

	UObject* NewCDO = ClassToPropagate->GetDefaultObject();
	
	check(NewCDO);

	// Get the blueprint's BP derived lineage
	TArray<UBlueprintGeneratedClass*> ParentBPStack;
	UBlueprint::GetBlueprintHierarchyFromClass(ClassToPropagate, ParentBPStack);

	// Starting from the least derived BP class, copy the properties into the new CDO
	for(int32 Index = ParentBPStack.Num() - 1; Index > 0; --Index)
	{
		checkf(ParentBPStack[Index], TEXT("Parent classes for class %s have not yet been generated.  Compile-on-load must be processed for the parent class first."), *ClassToPropagate->GetName());
		UObject* LayerCDO = ParentBPStack[Index]->GetDefaultObject();

		UEditorEngine::FCopyPropertiesForUnrelatedObjectsParams CopyDetails;
		CopyDetails.bReplaceObjectClassReferences = false;
		UEditorEngine::CopyPropertiesForUnrelatedObjects(LayerCDO, NewCDO, CopyDetails);
	}
}

uint32 FBlueprintDuplicationScopeFlags::bStaticFlags = FBlueprintDuplicationScopeFlags::NoFlags;

void FBlueprintEditorUtils::PostDuplicateBlueprint(UBlueprint* Blueprint, bool bDuplicateForPIE)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PostDuplicateBlueprint);
	
	// Only recompile after duplication if this isn't PIE
	if (!bDuplicateForPIE)
	{
		if(Blueprint->GeneratedClass != nullptr)
		{
			// Grab the old CDO, which contains the class defaults
			UClass* OldBPGCAsClass = Blueprint->GeneratedClass;
			UBlueprint* OldBlueprint = Cast<UBlueprint>(OldBPGCAsClass->ClassGeneratedBy);
			UBlueprintGeneratedClass* OldBPGC = (UBlueprintGeneratedClass*)(OldBPGCAsClass);
			UObject* OldCDO = OldBPGC->GetDefaultObject();
			check(OldCDO != nullptr);

			// Make sure that OldBPGC isn't garbage collected within this scope
			struct FAddToRootHelper
			{
				FAddToRootHelper(UBlueprintGeneratedClass* InBPGC)
				{
					BPGC = InBPGC;
					bWasRoot = BPGC->IsRooted();
					if(!bWasRoot)
					{
						BPGC->AddToRoot();
					}
				}

				~FAddToRootHelper()
				{
					if (!bWasRoot)
					{
						BPGC->RemoveFromRoot();
					}
				}

				UBlueprintGeneratedClass* BPGC;
				bool bWasRoot;
			} KeepBPGCAlive(OldBPGC);

			if (FBlueprintDuplicationScopeFlags::HasAnyFlag(FBlueprintDuplicationScopeFlags::ValidatePinsUsingSourceClass))
			{
				Blueprint->OriginalClass = OldBPGC;
			}

			// Grab the old class templates, which needs to be moved to the new class
			USimpleConstructionScript* SCSRootNode = Blueprint->SimpleConstructionScript;
			Blueprint->SimpleConstructionScript = nullptr;

			UInheritableComponentHandler* InheritableComponentHandler = Blueprint->InheritableComponentHandler;
			Blueprint->InheritableComponentHandler = nullptr;

			TArray<UActorComponent*> Templates = Blueprint->ComponentTemplates;
			Blueprint->ComponentTemplates.Empty();

			TArray<UTimelineTemplate*> Timelines = Blueprint->Timelines;
			Blueprint->Timelines.Empty();

			Blueprint->GeneratedClass = nullptr;
			Blueprint->SkeletonGeneratedClass = nullptr;

			// Make sure the new blueprint has a shiny new class
			FCompilerResultsLog Results;

			FName NewSkelClassName, NewGenClassName;
			Blueprint->GetBlueprintClassNames(NewGenClassName, NewSkelClassName);

			UClass* NewClass = NewObject<UClass>(
				Blueprint->GetOutermost(), Blueprint->GetBlueprintClass(), NewGenClassName, RF_Public | RF_Transactional);

			Blueprint->GeneratedClass = NewClass;
			NewClass->ClassGeneratedBy = Blueprint;
			NewClass->SetSuperStruct(Blueprint->ParentClass);
			Blueprint->bHasBeenRegenerated = true;		// Set to true, similar to CreateBlueprint, since we've regerated the class by duplicating it

			TMap<UObject*, UObject*> OldToNewMap;

			UClass* NewBPGCAsClass = Blueprint->GeneratedClass;
			UBlueprintGeneratedClass* NewBPGC = (UBlueprintGeneratedClass*)(NewBPGCAsClass);
			if( SCSRootNode )
			{
				NewBPGC->SimpleConstructionScript = Cast<USimpleConstructionScript>(StaticDuplicateObject(SCSRootNode, NewBPGC, SCSRootNode->GetFName()));
				Blueprint->SimpleConstructionScript = NewBPGC->SimpleConstructionScript;				
				const TArray<USCS_Node*>& AllNodes = NewBPGC->SimpleConstructionScript->GetAllNodes();

				// Duplicate all component templates
				for (USCS_Node* CurrentNode : AllNodes)
				{
					if (CurrentNode && CurrentNode->ComponentTemplate)
					{
						UActorComponent* DuplicatedComponent = CastChecked<UActorComponent>(StaticDuplicateObject(CurrentNode->ComponentTemplate, NewBPGC, CurrentNode->ComponentTemplate->GetFName()));
						OldToNewMap.Add(CurrentNode->ComponentTemplate, DuplicatedComponent);
						CurrentNode->ComponentTemplate = DuplicatedComponent;
					}
				}

				if (USCS_Node* DefaultSceneRootNode = NewBPGC->SimpleConstructionScript->GetDefaultSceneRootNode())
				{
					if (!AllNodes.Contains(DefaultSceneRootNode) && DefaultSceneRootNode->ComponentTemplate)
					{
						UActorComponent* DuplicatedComponent =  Cast<UActorComponent>(OldToNewMap.FindRef(DefaultSceneRootNode->ComponentTemplate));
						if (!DuplicatedComponent)
						{
							DuplicatedComponent = CastChecked<UActorComponent>(StaticDuplicateObject(DefaultSceneRootNode->ComponentTemplate, NewBPGC, DefaultSceneRootNode->ComponentTemplate->GetFName()));
							OldToNewMap.Add(DefaultSceneRootNode->ComponentTemplate, DuplicatedComponent);
						}
						DefaultSceneRootNode->ComponentTemplate = DuplicatedComponent;
					}
				}
			}

			for (UActorComponent* OldComponent : Templates)
			{
				UActorComponent* NewComponent = CastChecked<UActorComponent>(StaticDuplicateObject(OldComponent, NewBPGC, OldComponent->GetFName()));

				NewBPGC->ComponentTemplates.Add(NewComponent);
				OldToNewMap.Add(OldComponent, NewComponent);
			}

			for (UTimelineTemplate* OldTimeline : Timelines)
			{
				UTimelineTemplate* NewTimeline = CastChecked<UTimelineTemplate>(StaticDuplicateObject(OldTimeline, NewBPGC, OldTimeline->GetFName()));

				if (FBlueprintDuplicationScopeFlags::HasAnyFlag(FBlueprintDuplicationScopeFlags::TheSameTimelineGuid))
				{
					NewTimeline->TimelineGuid = OldTimeline->TimelineGuid;

					// Ensure that cached names sync back up with the original GUID.
					FUpdateTimelineCachedNames::Execute(NewTimeline);
				}

				NewBPGC->Timelines.Add(NewTimeline);
				OldToNewMap.Add(OldTimeline, NewTimeline);
			}

			if (InheritableComponentHandler)
			{
				NewBPGC->InheritableComponentHandler = Cast<UInheritableComponentHandler>(StaticDuplicateObject(InheritableComponentHandler, NewBPGC, InheritableComponentHandler->GetFName()));
				if (NewBPGC->InheritableComponentHandler)
				{
					NewBPGC->InheritableComponentHandler->UpdateOwnerClass(NewBPGC);
				}
			}

			Blueprint->ComponentTemplates = NewBPGC->ComponentTemplates;
			Blueprint->Timelines = NewBPGC->Timelines;
			Blueprint->InheritableComponentHandler = NewBPGC->InheritableComponentHandler;

			FBlueprintCompilationManager::CompileSynchronously(
				FBPCompileRequest(Blueprint, EBlueprintCompileOptions::RegenerateSkeletonOnly, nullptr)
			);

			// Create a new blueprint guid
			Blueprint->GenerateNewGuid();

			// Give all member variables a new guid
			TMap<FGuid, FGuid> NewVarGuids;
			for (FBPVariableDescription& Var : Blueprint->NewVariables)
			{
				Var.VarGuid = NewVarGuids.Emplace(Var.VarGuid, FGuid::NewGuid());
			}

			TArray< UEdGraphNode* > AllGraphNodes;
			GetAllNodesOfClass(Blueprint, AllGraphNodes);

			// Before we update Guids, we can use them to dupe breakpoints, watchpins to the new BP
			FKismetDebugUtilities::PostDuplicateBlueprint(OldBlueprint, Blueprint, AllGraphNodes);

			// Give all nodes a new Guid
			for(UEdGraphNode* Node : AllGraphNodes)
			{
				if (!FBlueprintDuplicationScopeFlags::HasAnyFlag(FBlueprintDuplicationScopeFlags::TheSameNodeGuid))
				{
					Node->CreateNewGuid();
				}

				// Some variable & delegate nodes must be fixed up on duplicate, this cannot wait for individual 
				// node calls to PostDuplicate because it happens after compilation and will still result in errors
				UK2Node_Variable* VariableNode = Cast<UK2Node_Variable>(Node);
				UK2Node_BaseMCDelegate* DelegateNode = Cast<UK2Node_BaseMCDelegate>(Node);

				if(VariableNode || DelegateNode)
				{
					FMemberReference& OutdatedReference = VariableNode ? VariableNode->VariableReference : DelegateNode->DelegateReference;
					UK2Node* OutdatedNode = Cast<UK2Node>(Node);

					// Self context variable nodes need to be updated with the new Blueprint class
					if(OutdatedReference.IsSelfContext() || OutdatedReference.GetMemberParentClass() != OutdatedNode->GetBlueprintClassFromNode())
					{
						// update variable references with new Guids if necessary
						if (FGuid* NewGuid = NewVarGuids.Find(OutdatedReference.GetMemberGuid()))
						{
							if (OutdatedReference.IsSelfContext())
							{
								OutdatedReference.SetSelfMember(OutdatedReference.GetMemberName(), *NewGuid);
							}
							else
							{
								OutdatedReference.SetExternalMember(OutdatedReference.GetMemberName(), OutdatedNode->GetBlueprintClassFromNode());
							}
						}

						const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
						if(UEdGraphPin* SelfPin = K2Schema->FindSelfPin(*OutdatedNode, EGPD_Input))
						{
							UClass* TargetClass = nullptr;

							if(FProperty* Property = OutdatedReference.ResolveMember<FProperty>(OutdatedNode->GetBlueprintClassFromNode()))
							{
								// Properties that are owned by a Struct - e.g. sparse class data - will not have a class:
								if (UClass* OwningClass = Property->GetOwnerClass())
								{
									TargetClass = OwningClass->GetAuthoritativeClass();
								}
							}
							else
							{
								TargetClass = NewClass;
							}

							SelfPin->PinType.PinSubCategoryObject = TargetClass;
						}
					}
				}
			}

			// Skip CDO validation in this case as we will not have yet propagated values to the new CDO. Also skip
			// Blueprint search data updates, as that will be handled by an OnAssetAdded() delegate in the FiB manager.
			const EBlueprintCompileOptions BPCompileOptions =
				EBlueprintCompileOptions::SkipDefaultObjectValidation |
				EBlueprintCompileOptions::SkipFiBSearchMetaUpdate;

			FBlueprintCompilationManager::CompileSynchronously(
				FBPCompileRequest(Blueprint, BPCompileOptions, nullptr)
			);

			FArchiveReplaceObjectRef<UObject> ReplaceTemplateRefs(NewBPGC, OldToNewMap);

			// Now propagate the values from the old CDO to the new one
			check(Blueprint->SkeletonGeneratedClass != nullptr);

			UObject* NewCDO = Blueprint->GeneratedClass->GetDefaultObject();
			check(NewCDO != nullptr);

			TMap<UClass*, UClass*> InOutOldToNewClassMap;
			TMap<UObject*, UObject*> CreatedInstanceMap;
			TArray< TTuple<UObject*, UObject*>> OrderedListOfObjectToCopy;
			FBlueprintCompileReinstancer::PreCreateSubObjectsForReinstantiation(InOutOldToNewClassMap, OldCDO, NewCDO, CreatedInstanceMap, nullptr, &OrderedListOfObjectToCopy);

			// We only need to copy properties of the pre-created instances, the rest of the default sub object is done inside the UEditorEngine::CopyPropertiesForUnrelatedObjects
			TMap<UObject*, UObject*> OldToNewInstanceMap(CreatedInstanceMap);
			UEngine::FCopyPropertiesForUnrelatedObjectsParams Params;
			Params.OptionalReplacementMappings = &OldToNewInstanceMap;
			Params.bOnlyHandleDirectSubObjects = true;
			Params.bReplaceInternalReferenceUponRead = true;
			for (const auto& Pair : OrderedListOfObjectToCopy)
			{
				UEditorEngine::CopyPropertiesForUnrelatedObjects(Pair.Key, Pair.Value, Params);
			}

			// copy sparse data over to the new class sparse data, if any:
			const TObjectPtr<UScriptStruct> SparseData = Blueprint->GeneratedClass->GetSparseClassDataStruct();
			if (SparseData && OldBPGC->bIsSparseClassDataSerializable)
			{
				void* SparseDataInstance = Blueprint->GeneratedClass->GetOrCreateSparseClassData();

				// Compile may have generated a new sparse class data, in which case we're just going to 
				// use whatever the compiler generated - if we're reusing the source class's sparse
				// class data then we can copy over the values immediately.. We could CPFUO here
				// as well, but if the compiler generated the sparse data that could be undesirable - e.g.
				// because the sparse data is caching information about the CDO or Class
				const TObjectPtr<UScriptStruct> OldSparseData = OldBPGC->GetSparseClassDataStruct();
				if (OldSparseData == SparseData)
				{
					const void* OldSparseDataInstance = OldBPGC->GetSparseClassData(EGetSparseClassDataMethod::ReturnIfNull);
					if (OldSparseDataInstance && ensure(OldSparseDataInstance != SparseDataInstance))
					{
						SparseData->CopyScriptStruct(SparseDataInstance, OldSparseDataInstance);
					}
				}
				NewBPGC->bIsSparseClassDataSerializable = true; // match the object we're being duplicated from
			}

			FBlueprintEditorUtils::ReconstructAllNodes(Blueprint);

			if (!FBlueprintDuplicationScopeFlags::HasAnyFlag(FBlueprintDuplicationScopeFlags::NoExtraCompilation))
			{
				// And compile again to make sure they go into the generated class, get cleaned up, etc...
				FKismetEditorUtilities::CompileBlueprint(Blueprint, EBlueprintCompileOptions::SkipGarbageCollection);
			}
		}

		// it can still keeps references to some external objects
		Blueprint->LastEditedDocuments.Empty();
	}

	// Should be no instances of this new blueprint, so no need to replace any
}

void FBlueprintEditorUtils::RemoveGeneratedClasses(UBlueprint* Blueprint)
{
	IKismetCompilerInterface& Compiler = FModuleManager::LoadModuleChecked<IKismetCompilerInterface>(KISMET_COMPILER_MODULENAME);
	Compiler.RemoveBlueprintGeneratedClasses(Blueprint);
}

void FBlueprintEditorUtils::UpdateDelegatesInBlueprint(UBlueprint* Blueprint)
{
	check(Blueprint);
	TArray<UEdGraph*> Graphs;
	Blueprint->GetAllGraphs(Graphs);
	for (UEdGraph* Graph : Graphs)
	{
		if(!IsGraphIntermediate(Graph))
		{
			TArray<UK2Node_CreateDelegate*> CreateDelegateNodes;
			Graph->GetNodesOfClass(CreateDelegateNodes);
			for (UK2Node_CreateDelegate* DelegateNode: CreateDelegateNodes)
			{
				DelegateNode->HandleAnyChangeWithoutNotifying();
			}

			TArray<UK2Node_Event*> EventNodes;
			Graph->GetNodesOfClass(EventNodes);
			for (UK2Node_Event* EventNode : EventNodes)
			{
				EventNode->UpdateDelegatePin();
			}

			TArray<UK2Node_Knot*> Knots;
			Graph->GetNodesOfClass(Knots);
			for (UK2Node_Knot* Knot : Knots)
			{
				// Indiscriminate reuse of UK2Node_Knot::PostReconstructNode() is the convention established
				// by UEdGraphSchema_K2::OnPinConnectionDoubleCicked. This forces the pin type data to be
				// refreshed (e.g. due to changes in UpdateDelegatePin())
				Knot->PostReconstructNode();
			}
		}
	}
}

// Blueprint has materially changed.  Recompile the skeleton, notify observers, and mark the package as dirty.
void FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(UBlueprint* Blueprint)
{
	if (Blueprint->bSuppressStructurallyModified || Blueprint->bQueuedForCompilation)
	{
		return;
	}

	// The Blueprint has been structurally modified and this means that some node titles will need to be refreshed
	GetDefault<UEdGraphSchema_K2>()->ForceVisualizationCacheClear();

	Blueprint->bCachedDependenciesUpToDate = false;
	if (Blueprint->Status != BS_BeingCreated && !Blueprint->bBeingCompiled)
	{
		BP_SCOPED_COMPILER_EVENT_STAT(EKismetCompilerStats_MarkBlueprintasStructurallyModified);

		FBlueprintCompilationManager::CompileSynchronously(
			FBPCompileRequest(Blueprint, EBlueprintCompileOptions::RegenerateSkeletonOnly, nullptr)
		);

		// Call general modification callback as well
		MarkBlueprintAsModified(Blueprint);

		{
			BP_SCOPED_COMPILER_EVENT_STAT(EKismetCompilerStats_NotifyBlueprintChanged);

			// Notify any interested parties that the blueprint has changed
			Blueprint->BroadcastChanged();
		}
	}
}

// Blueprint has changed in some manner that invalidates the compiled data (link made/broken, default value changed, etc...)
void FBlueprintEditorUtils::MarkBlueprintAsModified(UBlueprint* Blueprint, FPropertyChangedEvent PropertyChangedEvent)
{
	if(Blueprint->bBeingCompiled)
	{
		return;
	}

	Blueprint->bCachedDependenciesUpToDate = false;
	if (Blueprint->Status != BS_BeingCreated)
	{
		// This clears any cached data, which includes the macro tunnel node data
		TArray<UEdGraph*> AllGraphs;
		Blueprint->GetAllGraphs(AllGraphs);
		for (UEdGraph* GraphToClear : AllGraphs)
		{
			for (UEdGraphNode* Node : GraphToClear->Nodes)
			{
				if (UK2Node* BPNode = Cast<UK2Node>(Node))
				{
					BPNode->ClearCachedBlueprintData(Blueprint);
				}
			}
		}

		// If this was called the CDO was probably modified. Regenerate the post construct property list
		UBlueprintGeneratedClass* BPGC = Cast<UBlueprintGeneratedClass>(Blueprint->GeneratedClass);
		if (!Blueprint->bBeingCompiled && BPGC)
		{
			BPGC->UpdateCustomPropertyListForPostConstruction();

			TArray<UClass*> ChildClasses;
			GetDerivedClasses(BPGC, ChildClasses);

			for (UClass* ChildClass : ChildClasses)
			{
				CastChecked<UBlueprintGeneratedClass>(ChildClass)->UpdateCustomPropertyListForPostConstruction();
			}
		}

		Blueprint->Status = BS_Dirty;
		Blueprint->MarkPackageDirty();
		// Previously, PostEditChange() was called on the Blueprint which creates an empty FPropertyChangedEvent. In
		// certain cases, we needed to be able to pass along specific FPropertyChangedEvent that initially triggered
		// this call so that we could keep the Blueprint from refreshing under certain conditions.
		Blueprint->PostEditChangeProperty(PropertyChangedEvent);

		// Clear out the cache as the user may have added or removed a latent action to a macro graph
		FBlueprintEditorUtils::ClearMacroCosmeticInfoCache(Blueprint);
	}
	
	if (GEditor)
	{
		IAssetEditorInstance* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(Blueprint, false);
		if (AssetEditor)
		{
			// Prevent crash with the custom editor operating on the project-specific UBlueprint class
			// Such custom editor might not inherit after FBlueprintEditor, but could still utilize FBlueprintEditorUtils
			FAssetEditorToolkit* AssetEditorToolkit = static_cast<FAssetEditorToolkit*>(AssetEditor);
			if (AssetEditorToolkit->IsBlueprintEditor())
			{
				FBlueprintEditor* BlueprintEditor = static_cast<FBlueprintEditor*>(AssetEditor);
				BlueprintEditor->UpdateNodesUnrelatedStatesAfterGraphChange();
			}
		}
	}
}

void FBlueprintEditorUtils::RecompileBeforeSaveIfNeeded(UBlueprint* Blueprint)
{
	if (Blueprint->GeneratedClass == nullptr)
	{
		return; // no class, nothing could be out of date.. it could be that we need to regenerate it here
	}

	if (Blueprint->Status == EBlueprintStatus::BS_UpToDate)
	{
		return; // the UDS editor marks blueprint as out of date, we can rely on that flag
	}

	if (Blueprint->CachedUDSDependencies.Num() == 0)
	{
		return;
	}

	// Detect references to out of date UUserDefinedStructs (or other UScriptStructs):
	TSet<const UScriptStruct*> OutOfDateStructs;
	UE::Private::ForEachObjectReference(Blueprint->GeneratedClass, [&OutOfDateStructs](const UObject*Obj){
		if (const UScriptStruct* Struct = Cast<UScriptStruct>(Obj))
		{
			const bool bStructOutOfDate = 
				(Struct->StructFlags & STRUCT_NewerVersionExists) != 0
				|| (!Struct->IsNative() && Struct->HasAnyFlags(RF_Transient));
			if (bStructOutOfDate)
			{
				OutOfDateStructs.Add(Struct);
			}
		}
	});
	const bool bHasOutOfDateStructs = OutOfDateStructs.Num() != 0;
	if (bHasOutOfDateStructs)
	{
		// full compile, really we just need to regenerate the class *and reinstance* if necessary,
		// compile is just a convenient way to do that:
		FKismetEditorUtilities::CompileBlueprint(
			Blueprint, EBlueprintCompileOptions::SkipGarbageCollection);
	}
}

bool FBlueprintEditorUtils::ShouldRegenerateBlueprint(UBlueprint* Blueprint)
{
	return !IsCompileOnLoadDisabled(Blueprint)
		&& Blueprint->bRecompileOnLoad
		&& !Blueprint->bIsRegeneratingOnLoad;
}

bool FBlueprintEditorUtils::IsCompileOnLoadDisabled(UBlueprint* Blueprint)
{
	bool bCompilationDisabled = !Blueprint->bRecompileOnLoad;
	if(bCompilationDisabled)
	{
		// bRecompileOnLoad has been unused for awhile, warn once about its use,
		// in case it is unintentional or the user has forgotten they've disabled
		// it:
		static bool bHasWarned = false;
		if(!bHasWarned)
		{
			bHasWarned = true;
			UE_LOGF(LogBlueprint, Warning, 
				"Blueprints are loading but RecompileOnLoad is disabled - no bytecode will be available unless SkipByteCodeSerialization is disabled");
		}
	}
	if(Blueprint->GetLinker())
	{
		bCompilationDisabled = (Blueprint->GetLinker()->LoadFlags & LOAD_DisableCompileOnLoad) != LOAD_None;
	}
	// if the blueprint's package was cooked for editor builds we cannot recompile it as duplication will crash and since
	// it's already cooked, if will already be up to date (likely they shouldn't exist, but in case they do we need to make them work)
	if (Blueprint->GetOutermost()->bIsCookedForEditor)
	{
		bCompilationDisabled = true;
	}
	return bCompilationDisabled;
}

// Helper function to get the blueprint that ultimately owns a node.
UBlueprint* FBlueprintEditorUtils::FindBlueprintForNode(const UEdGraphNode* Node)
{
	UEdGraph* Graph = Node ? Cast<UEdGraph>(Node->GetOuter()) : nullptr;
	return FindBlueprintForGraph(Graph);
}

// Helper function to get the blueprint that ultimately owns a node.  Cannot fail.
TNotNull<UBlueprint*> FBlueprintEditorUtils::FindBlueprintForNodeChecked(const UEdGraphNode* Node)
{
	UBlueprint* Result = FindBlueprintForNode(Node);
	checkf(Result, TEXT("FBlueprintEditorUtils::FindBlueprintForNodeChecked(%s) failed to find a Blueprint."), *GetPathNameSafe(Node));
	return Result;
}


// Helper function to get the blueprint that ultimately owns a graph.
UBlueprint* FBlueprintEditorUtils::FindBlueprintForGraph(const UEdGraph* Graph)
{
	for (UObject* TestOuter = Graph ? Graph->GetOuter() : nullptr; TestOuter; TestOuter = TestOuter->GetOuter())
	{
		if (UBlueprint* Result = Cast<UBlueprint>(TestOuter))
		{
			return Result;
		}
	}

	return nullptr;
}

// Helper function to get the blueprint that ultimately owns a graph.  Cannot fail.
TNotNull<UBlueprint*> FBlueprintEditorUtils::FindBlueprintForGraphChecked(const UEdGraph* Graph)
{
	UBlueprint* Result = FBlueprintEditorUtils::FindBlueprintForGraph(Graph);
	checkf(Result, TEXT("FBlueprintEditorUtils::FindBlueprintForGraphChecked(%s) failed to find a Blueprint."), *GetPathNameSafe(Graph));
	return Result;
}

UClass* FBlueprintEditorUtils::GetSkeletonClass(UClass* FromClass)
{
	if (FromClass)
	{
		if (UBlueprint* Generator = Cast<UBlueprint>(FromClass->ClassGeneratedBy))
		{
			return Generator->SkeletonGeneratedClass;
		}
	}
	return nullptr;
}

const UClass* FBlueprintEditorUtils::GetSkeletonClass(const UClass* FromClass)
{
	return GetSkeletonClass(const_cast<UClass*>(FromClass));
}

UClass* FBlueprintEditorUtils::GetMostUpToDateClass(UClass* FromClass)
{
	if (!FromClass || FromClass->HasAnyClassFlags(CLASS_Native) || FromClass->bCooked)
	{
		return FromClass;
	}

	// It's really not safe/coherent to try and dig out the 'right' class. Things that need the 'most up to date'
	// version of a class should always be looking at the skeleton:
	UClass* SkeletonClass = GetSkeletonClass(FromClass);

	return SkeletonClass ? SkeletonClass : FromClass;
}

const UClass* FBlueprintEditorUtils::GetMostUpToDateClass(const UClass* FromClass)
{
	return GetMostUpToDateClass(const_cast<UClass*>(FromClass));
}

bool FBlueprintEditorUtils::PropertyStillExists(FProperty* Property)
{
	return GetMostUpToDateProperty(Property) != nullptr;
}

FProperty* FBlueprintEditorUtils::GetMostUpToDateProperty(FProperty* Property)
{
	if(const UClass* OwningClass = Property->GetTypedOwner<UClass>())
	{
		const UClass* UpToDateClass = GetMostUpToDateClass(OwningClass);
		if (UpToDateClass && UpToDateClass != OwningClass)
		{
			Property = UpToDateClass->FindPropertyByName(Property->GetFName());
		}
	}

	return Property;
}

const FProperty* FBlueprintEditorUtils::GetMostUpToDateProperty(const FProperty* Property)
{
	return GetMostUpToDateProperty(const_cast<FProperty*>(Property));
}

UFunction* FBlueprintEditorUtils::GetMostUpToDateFunction(UFunction* Function)
{
	if(const UClass* OwningClass = Function->GetTypedOuter<UClass>())
	{
		const UClass* UpToDateClass = GetMostUpToDateClass(OwningClass);
		if (UpToDateClass && UpToDateClass != OwningClass)
		{
			Function = UpToDateClass->FindFunctionByName(Function->GetFName());
		}
	}
	return Function;
}

const UFunction* FBlueprintEditorUtils::GetMostUpToDateFunction(const UFunction* Function)
{
	return GetMostUpToDateFunction(const_cast<UFunction*>(Function));
}

bool FBlueprintEditorUtils::IsGraphNameUnique(UObject* InOuter, const FName& InName)
{
	// Check for any object directly created in the blueprint
	if( !FindObject<UObject>(InOuter, *InName.ToString()) )
	{
		if(UBlueprint* Blueprint = Cast<UBlueprint>(InOuter))
		{
			// Next, check for functions with that name in the blueprint's class scope
			FFieldVariant ExistingField = FindUFieldOrFProperty(Blueprint->SkeletonGeneratedClass, InName);
			if( !ExistingField )
			{
				// Finally, check function entry points
				TArray<UK2Node_Event*> AllEvents;
				FBlueprintEditorUtils::GetAllNodesOfClass<UK2Node_Event>(Blueprint, AllEvents);

				for(int32 i=0; i < AllEvents.Num(); i++)
				{
					UK2Node_Event* EventNode = AllEvents[i];
					check(EventNode);

					if( EventNode->CustomFunctionName == InName
						|| EventNode->EventReference.GetMemberName() == InName )
					{
						return false;
					}
				}

				// All good!
				return true;
			}
		}

		// All good!
		return true;
	}

	return false;
}

UEdGraph* FBlueprintEditorUtils::CreateNewGraph(UObject* ParentScope, const FName& GraphName, TSubclassOf<class UEdGraph> GraphClass, TSubclassOf<class UEdGraphSchema> SchemaClass)
{
	UEdGraph* NewGraph = nullptr;
	bool bRename = false;

	// Ensure this name isn't already being used for a graph
	if (GraphName != NAME_None)
	{
		if (UObject* ExistingObject = FindObject<UObject>(ParentScope, *(GraphName.ToString())))
		{
			if (ExistingObject->IsA<UEdGraph>())
			{
				// Rename the old graph out of the way - this may confuse the user somewhat - and even
				// break their logic. But name collisions are not avoidable e.g. someone can add
				// a function to an interface that conflicts with something in a class hierarchy
				ExistingObject->Rename(nullptr, ExistingObject->GetOuter(), REN_DoNotDirty | REN_AllowPackageLinkerMismatch);
			}
			else if (ExistingObject->IsA<UObjectRedirector>())
			{
				const UBlueprint* Blueprint = Cast<UBlueprint>(ParentScope);
				if (Blueprint && Blueprint->BlueprintType == BPTYPE_MacroLibrary)
				{
					// When renaming a graph inside a macro library, we may have dropped a redirector after a previous
					// rename (see RenameGraph). If we're now reusing it, move the redirector aside to free up the name.
					ExistingObject->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
				}
			}
		}

		// Construct new graph with the supplied name
		NewGraph = NewObject<UEdGraph>(ParentScope, GraphClass, NAME_None, RF_Transactional);
		bRename = true;
	}
	else
	{
		// Construct a new graph with a default name
		NewGraph = NewObject<UEdGraph>(ParentScope, GraphClass, NAME_None, RF_Transactional);
	}

	NewGraph->Schema = SchemaClass;

	// Now move to where we want it to. Workaround to ensure transaction buffer is correctly utilized
	if (bRename)
	{
		NewGraph->Rename(*(GraphName.ToString()), ParentScope, REN_DoNotDirty | REN_AllowPackageLinkerMismatch);
	}
	return NewGraph;
}

void FBlueprintEditorUtils::CreateMatchingFunction(UK2Node_CallFunction* InNode, TSubclassOf<class UEdGraphSchema> InSchemaClass)
{
	if (UBlueprint* Blueprint = InNode->GetBlueprint())
	{
		FScopedTransaction Transaction(LOCTEXT("CreateMatchingFunction", "Create Matching Function"));
		Blueprint->Modify();

		UEdGraph* Graph = FBlueprintEditorUtils::CreateNewGraph(Blueprint, InNode->GetFunctionName(), UEdGraph::StaticClass(), InSchemaClass);
		FBlueprintEditorUtils::AddFunctionGraph<UClass>(Blueprint, Graph, true, nullptr);

		TArray<UK2Node_FunctionEntry*> Entry;
		Graph->GetNodesOfClass<UK2Node_FunctionEntry>(Entry);
		if (ensure(Entry.Num() == 1))
		{
			UK2Node_FunctionResult* Result = nullptr;
			for (UEdGraphPin* Pin : InNode->Pins)
			{
				// if this wasn't a split pin
				if (!Pin->ParentPin)
				{
					FName PinName = Pin->GetFName();
					// If this isn't a default pin, add it to the function entry
					if (PinName != UEdGraphSchema_K2::PN_Self && PinName != UEdGraphSchema_K2::PN_Execute && PinName != UEdGraphSchema_K2::PN_Then)
					{
						if (Pin->Direction == EEdGraphPinDirection::EGPD_Input)
						{
							// add as an input param to function
							Entry[0]->CreateUserDefinedPin(PinName, Pin->PinType, EEdGraphPinDirection::EGPD_Output);
						}
						else if (Pin->Direction == EEdGraphPinDirection::EGPD_Output)
						{
							// only create a result node if there are out parameters
							if (!Result)
							{
								Result = FBlueprintEditorUtils::FindOrCreateFunctionResultNode(Entry[0]);
							}

							// add as an output param to function
							Result->CreateUserDefinedPin(PinName, Pin->PinType, EEdGraphPinDirection::EGPD_Input);
						}
					}
				}
			}
		}

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
		InNode->ReconstructNode();
	}
}

bool FBlueprintEditorUtils::IsFunctionConvertableToEvent(UBlueprint* const BlueprintObj, UFunction* const Function)
{
	return BlueprintObj && BlueprintObj->BlueprintType != BPTYPE_FunctionLibrary && BlueprintObj->BlueprintType != BPTYPE_Interface && Function && !HasFunctionBlueprintThreadSafeMetaData(Function);
}

UFunction* FBlueprintEditorUtils::FindFunctionInImplementedInterfaces(const UBlueprint* Blueprint, const FName& FunctionName, bool * bOutInvalidInterface, bool bGetAllInterfaces)
{
	if(Blueprint)
	{
		TArray<UClass*> InterfaceClasses;
		FindImplementedInterfaces(Blueprint, bGetAllInterfaces, InterfaceClasses);

		if( bOutInvalidInterface )
		{
			*bOutInvalidInterface = false;
		}

		// Now loop through the interface classes and try and find the function
		for (UClass* SearchClass : InterfaceClasses)
		{
			if( SearchClass )
			{
				// Use the skeleton class if possible, as the generated class may not always be up-to-date (e.g. if the compile state is dirty).
				UBlueprint* InterfaceBlueprint = Cast<UBlueprint>(SearchClass->ClassGeneratedBy);
				if (InterfaceBlueprint && InterfaceBlueprint->SkeletonGeneratedClass)
				{
					SearchClass = InterfaceBlueprint->SkeletonGeneratedClass;
				}

				do
				{
					if( UFunction* OverriddenFunction = SearchClass->FindFunctionByName(FunctionName, EIncludeSuperFlag::ExcludeSuper) )
					{
						return OverriddenFunction;
					}
					SearchClass = SearchClass->GetSuperClass();
				} while (SearchClass);
			}
			else if( bOutInvalidInterface )
			{
				*bOutInvalidInterface = true;
			}
		}
	}

	return nullptr;
}

void FBlueprintEditorUtils::FindImplementedInterfaces(const UBlueprint* Blueprint, bool bGetAllInterfaces, TArray<UClass*>& ImplementedInterfaces)
{
	// First get the ones this blueprint implemented
	for (const FBPInterfaceDescription& ImplementedInterface : Blueprint->ImplementedInterfaces)
	{
		ImplementedInterfaces.AddUnique(ImplementedInterface.Interface);
	}

	if (bGetAllInterfaces)
	{
		// Now get all the ones the blueprint's parents implemented
		UClass* BlueprintParent =  Blueprint->ParentClass;
		while (BlueprintParent)
		{
			for (const FImplementedInterface& ImplementedInterface : BlueprintParent->Interfaces)
			{
				ImplementedInterfaces.AddUnique(ImplementedInterface.Class);
			}
			BlueprintParent = BlueprintParent->GetSuperClass();
		}
	}
}

bool FBlueprintEditorUtils::ImplementsInterface(const UBlueprint* Blueprint, bool bIncludeInherited, UClass* InterfaceClass)
{
	// First test this blueprint
	if (Blueprint->ImplementedInterfaces.ContainsByPredicate([InterfaceClass](const FBPInterfaceDescription& ImplementedInterface){ return ImplementedInterface.Interface->IsChildOf(InterfaceClass); }))
	{
		return true;
	}

	if (bIncludeInherited)
	{
		if (UClass* BlueprintParent = Blueprint->ParentClass)
		{
			return BlueprintParent->ImplementsInterface(InterfaceClass);
		}
	}
	return false;
}

UClass* const FBlueprintEditorUtils::GetOverrideFunctionClass(UBlueprint* Blueprint, const FName FuncName, UFunction** OutFunction)
{
	if (!Blueprint->SkeletonGeneratedClass)
	{
		return nullptr;
	}

	UFunction* OverrideFunc = FBlueprintEditorUtils::GetInterfaceFunction(Blueprint, FuncName);

	if (OverrideFunc == nullptr)
	{
		OverrideFunc = FindUField<UFunction>(Blueprint->SkeletonGeneratedClass, FuncName);
		// search up the class hierarchy, we want to find the original declaration of the function to match FBlueprintEventNodeSpawner.
		// Doing so ensures that we can find the existing node if there is one:
		const UClass* Iter = Blueprint->SkeletonGeneratedClass->GetSuperClass();
		while (Iter != nullptr && OverrideFunc == nullptr)
		{
			if (UFunction * F = Iter->FindFunctionByName(FuncName))
			{
				OverrideFunc = F;
			}
			else
			{
				break;
			}
			Iter = Iter->GetSuperClass();
		}
	}
	if (OutFunction != nullptr)
	{
		*OutFunction = OverrideFunc;
	}

	return (OverrideFunc ? CastChecked<UClass>(OverrideFunc->GetOuter())->GetAuthoritativeClass() : nullptr);
}

void FBlueprintEditorUtils::AddMacroGraph( UBlueprint* Blueprint, class UEdGraph* Graph, bool bIsUserCreated, UClass* SignatureFromClass )
{
	// Give the schema a chance to fill out any required nodes (like the entry node or results node)
	const UEdGraphSchema* Schema = Graph->GetSchema();
	const UEdGraphSchema_K2* K2Schema = Cast<const UEdGraphSchema_K2>(Graph->GetSchema());

	Schema->CreateDefaultNodesForGraph(*Graph);

	if (K2Schema != nullptr)
	{
		K2Schema->CreateMacroGraphTerminators(*Graph, SignatureFromClass);

		if (bIsUserCreated)
		{
			// We need to flag the entry node to make sure that the compiled function is callable from Kismet2
			K2Schema->AddExtraFunctionFlags(Graph, (FUNC_BlueprintCallable|FUNC_BlueprintEvent));
			K2Schema->MarkFunctionEntryAsEditable(Graph, true);
		}
	}

	// Mark the graph as public if it's going to be referenced directly from other blueprints
	if (Blueprint->BlueprintType == BPTYPE_MacroLibrary)
	{
		Graph->SetFlags(RF_Public);
	}

	Blueprint->MacroGraphs.Add(Graph);
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
}

void FBlueprintEditorUtils::AddInterfaceGraph(UBlueprint* Blueprint, class UEdGraph* Graph, UClass* InterfaceClass)
{
	const UEdGraphSchema_K2* K2Schema = Cast<const UEdGraphSchema_K2>(Graph->GetSchema());
	if (K2Schema != nullptr)
	{
		K2Schema->CreateFunctionGraphTerminators(*Graph, InterfaceClass);
	}
}

void FBlueprintEditorUtils::AddUbergraphPage(UBlueprint* Blueprint, class UEdGraph* Graph)
{
#if WITH_EDITORONLY_DATA
	Blueprint->UbergraphPages.Add(Graph);
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
#endif	//#if WITH_EDITORONLY_DATA
}

FName FBlueprintEditorUtils::GetUbergraphFunctionName(const UBlueprint* ForBlueprint)
{
	const FString UbergraphCallString = UEdGraphSchema_K2::FN_ExecuteUbergraphBase.ToString() + TEXT("_") + ForBlueprint->GetName();
	return FName(*UbergraphCallString);
}

void FBlueprintEditorUtils::AddDomainSpecificGraph(UBlueprint* Blueprint, class UEdGraph* Graph)
{
	// Give the schema a chance to fill out any required nodes (like the entry node or results node)
	const UEdGraphSchema* Schema = Graph->GetSchema();
	Schema->CreateDefaultNodesForGraph(*Graph);

	check(Blueprint->BlueprintType != BPTYPE_MacroLibrary);

#if WITH_EDITORONLY_DATA
	Blueprint->FunctionGraphs.Add(Graph);
#endif	//#if WITH_EDITORONLY_DATA
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
}

// Remove the supplied set of graphs from the Blueprint.
void FBlueprintEditorUtils::RemoveGraphs( UBlueprint* Blueprint, const TArray<class UEdGraph*>& GraphsToRemove )
{
	for (int32 ItemIndex=0; ItemIndex < GraphsToRemove.Num(); ++ItemIndex)
	{
		UEdGraph* Graph = GraphsToRemove[ItemIndex];
		FBlueprintEditorUtils::RemoveGraph(Blueprint, Graph, EGraphRemoveFlags::MarkTransient);
	}

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
}

namespace UE::Private
{
	// Given a function graph, check if it's field notify and remove its name from the meta data of all field notify variables.
	static void RemoveFieldNotifyFunctionFromAllMetaData(UBlueprint* Blueprint, class UEdGraph* FunctionGraphToRemove)
	{
		if (Blueprint->FunctionGraphs.Contains(FunctionGraphToRemove) && FBlueprintEditorUtils::ImplementsInterface(Blueprint, true, UNotifyFieldValueChanged::StaticClass()))
		{
			for (UEdGraphNode* Node : FunctionGraphToRemove->Nodes)
			{
				if (UK2Node_FunctionEntry* NodeFunctionEntry = Cast<UK2Node_FunctionEntry>(Node))
				{
					if (NodeFunctionEntry->MetaData.HasMetaData(FBlueprintMetadata::MD_FieldNotify))
					{
						FBlueprintEditorUtils::RemoveFieldNotifyFromAllMetadata(Blueprint, FunctionGraphToRemove->GetFName());
					}
					break;
				}
			}
		}
	}
}

// Removes the supplied graph from the Blueprint.
void FBlueprintEditorUtils::RemoveGraph(UBlueprint* Blueprint, class UEdGraph* GraphToRemove, EGraphRemoveFlags::Type Flags /*= Transient | Recompile */)
{
	GraphToRemove->Modify();

	for (UObject* TestOuter = GraphToRemove->GetOuter(); TestOuter; TestOuter = TestOuter->GetOuter())
	{
		if (TestOuter == Blueprint)
		{
			UE::Private::RemoveFieldNotifyFunctionFromAllMetaData(Blueprint, GraphToRemove);

			Blueprint->DelegateSignatureGraphs.Remove( GraphToRemove );
			Blueprint->FunctionGraphs.Remove( GraphToRemove );
			Blueprint->UbergraphPages.Remove( GraphToRemove );

			// Can't just call Remove, the object is wrapped in a struct
			for(int EditedDocIdx = 0; EditedDocIdx < Blueprint->LastEditedDocuments.Num(); ++EditedDocIdx)
			{
				if(Blueprint->LastEditedDocuments[EditedDocIdx].EditedObjectPath.ResolveObject() == GraphToRemove)
				{
					Blueprint->LastEditedDocuments.RemoveAt(EditedDocIdx);
					break;
				}
			}

			if(Blueprint->MacroGraphs.Remove( GraphToRemove ) > 0 ) 
			{
				//removes all macro nodes using this macro graph
				TArray<UK2Node_MacroInstance*> MacroNodes;
				FBlueprintEditorUtils::GetAllNodesOfClass(Blueprint, MacroNodes);
				for (UK2Node_MacroInstance* Node : MacroNodes)
				{
					if(Node->GetMacroGraph() == GraphToRemove)
					{
						FBlueprintEditorUtils::RemoveNode(Blueprint, Node);
					}
				}

				// Clear the cache since it's indexed by graph and one of the graphs is going away
				FBlueprintEditorUtils::ClearMacroCosmeticInfoCache(Blueprint);

				// Clear redirectors to the graph - we may have created these for macro graphs.
				// See comment related to macro graphs in FBlueprintEditorUtils::RenameGraph
				// involving conditional assignment of REN_DontCreateRedirectors
				TArray<UObject*> Inners;
				GetObjectsWithOuter(Blueprint, Inners, EGetObjectsFlags::None);
				for(UObject* Object : Inners)
				{
					if(UObjectRedirector* Redirector = Cast<UObjectRedirector>(Object))
					{
						if(Redirector->DestinationObject == GraphToRemove)
						{
							Redirector->DestinationObject = nullptr;
						}
					}
				}
			}

			for (FBPInterfaceDescription& CurrInterface : Blueprint->ImplementedInterfaces)
			{
				CurrInterface.Graphs.Remove( GraphToRemove );
			}
		}
		else if (UEdGraph* OuterGraph = Cast<UEdGraph>(TestOuter))
		{
			// remove ourselves
			OuterGraph->Modify();
			OuterGraph->SubGraphs.Remove(GraphToRemove);
		}
		else if (! (Cast<UEdGraphNode>(TestOuter) && Cast<UEdGraphNode>(TestOuter)->GetSubGraphs().Num() > 0) )
		{
			break;
		}
	}

	// Remove timelines held in the graph
	TArray<UK2Node_Timeline*> AllTimelineNodes;
	GraphToRemove->GetNodesOfClass<UK2Node_Timeline>(AllTimelineNodes);
	for (UK2Node_Timeline* TimelineNode : AllTimelineNodes)
	{
		TimelineNode->DestroyNode();
	}

	// Handle subgraphs held in graph
	TArray<UEdGraphNode*> AllNodes;
	GraphToRemove->GetNodesOfClass<UEdGraphNode>(AllNodes);

	for (UEdGraphNode* GraphNode : AllNodes)
	{
		for(UEdGraph* SubGraph : GraphNode->GetSubGraphs())
		{
			if (SubGraph && SubGraph->GetOuter()->IsA(UEdGraphNode::StaticClass()))
			{
				FBlueprintEditorUtils::RemoveGraph(Blueprint, SubGraph, EGraphRemoveFlags::None);
			}
		}
	}

	GraphToRemove->GetSchema()->HandleGraphBeingDeleted(*GraphToRemove);

	GraphToRemove->Rename(nullptr, Blueprint ? Blueprint->GetOuter() : nullptr, REN_DoNotDirty | REN_DontCreateRedirectors);
	GraphToRemove->ClearFlags(RF_Standalone | RF_Public);
	GraphToRemove->RemoveFromRoot();

	if (Flags & EGraphRemoveFlags::MarkTransient)
	{
		GraphToRemove->SetFlags(RF_Transient);
	}

	if (Flags & EGraphRemoveFlags::Recompile)
	{
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
	}
}

/** Rename a graph and mark objects for modified */
void FBlueprintEditorUtils::RenameGraph(UEdGraph* Graph, const FString& NewNameStr)
{
	if (Graph)
	{
		// Cache old name
		const FName OldGraphName = Graph->GetFName();
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(Graph);

		// When renaming a graph inside a macro library, we may have dropped a redirector after a previous
		// rename (see below). If we're now trying to reuse it, move the redirector aside to free up the name.
		if (Blueprint->BlueprintType == BPTYPE_MacroLibrary)
		{
			if (UObjectRedirector* Redirector = FindObjectFast<UObjectRedirector>(Graph->GetOuter(), *NewNameStr))
			{
				Redirector->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
			}
		}

		// Ensure that there are no collisions; leave the name as-is if this fails for some reason.
		if (!Graph->Rename(*NewNameStr, Graph->GetOuter(), REN_Test))
		{
			return;
		}

		auto RenameGraphLambda = [](UEdGraph* GraphToRename, const FName LocalOldGraphName, const FName LocalNewGraphName, ERenameFlags RenameFlags)
		{
			// Ensure we have undo records
			GraphToRename->Modify();
			GraphToRename->Rename(*LocalNewGraphName.ToString(), GraphToRename->GetOuter(), RenameFlags);

			// Clean function entry & result nodes if they exist
			for (UEdGraphNode* Node : GraphToRename->Nodes)
			{
				if (UK2Node_FunctionEntry* EntryNode = Cast<UK2Node_FunctionEntry>(Node))
				{
					if (EntryNode->FunctionReference.GetMemberName() == LocalOldGraphName)
					{
						EntryNode->Modify();
						EntryNode->FunctionReference.SetMemberName(LocalNewGraphName);
					}
					else if (EntryNode->CustomGeneratedFunctionName == LocalOldGraphName)
					{
						EntryNode->Modify();
						EntryNode->CustomGeneratedFunctionName = LocalNewGraphName;
					}
				}
				else if (UK2Node_FunctionResult* ResultNode = Cast<UK2Node_FunctionResult>(Node))
				{
					if (ResultNode->FunctionReference.GetMemberName() == LocalOldGraphName)
					{
						ResultNode->Modify();
						ResultNode->FunctionReference.SetMemberName(LocalNewGraphName);
					}
				}
			}
		};

		ERenameFlags RenameFlagsToApply = REN_None;
		if (Blueprint->bIsRegeneratingOnLoad)
		{
			RenameFlagsToApply |= REN_AllowPackageLinkerMismatch;
		}

		// Macro library graphs are referenced indirectly and resolved at edit/compile time via GUID (see FGraphReference).
		// However, they will be exported by name at save time, so renaming a macro library graph implies we should also
		// export a redirector with the old name so that the linker will still be able to resolve existing imports on load.
		if (Blueprint->BlueprintType != BPTYPE_MacroLibrary)
		{
			RenameFlagsToApply |= REN_DontCreateRedirectors;
		}

		// Apply new name
		const FName NewGraphName(*NewNameStr);
		RenameGraphLambda(Graph, OldGraphName, NewGraphName, RenameFlagsToApply);

		TArray<UBlueprint*> ModifiedBlueprints;
		ModifiedBlueprints.Add(Blueprint);

		auto PostValidateChildBlueprintLambda = [Blueprint, &OldGraphName, &NewGraphName, &RenameGraphLambda, &ModifiedBlueprints](UBlueprint* InChildBP, const FName /* InVariableName */, bool /* bValidatedVariable */)
		{
			check(InChildBP);

			// Rename child blueprint override graphs.
			for (UEdGraph* FunctionGraph : InChildBP->FunctionGraphs)
			{
				if (FunctionGraph->GetFName() == OldGraphName)
				{
					RenameGraphLambda(FunctionGraph, OldGraphName, NewGraphName, (InChildBP->bIsRegeneratingOnLoad ? REN_AllowPackageLinkerMismatch : 0) | REN_DontCreateRedirectors);
				}
			}

			if (InChildBP->GetOutermost()->IsDirty())
			{
				ModifiedBlueprints.Add(InChildBP);
			}
		};

		// Potentially adjust variable names for any child blueprints.
		// Note: This will find ALL children (including nested children) so there's no need to do this recursively.
		ValidateBlueprintChildVariables(Blueprint, Graph->GetFName(), PostValidateChildBlueprintLambda);

		// if it's index is >= 0 we know it was found in the array of functiongraphs
		const bool bGraphIsFunction = (Blueprint->FunctionGraphs.IndexOfByKey(Graph) > -1);
		// For any nodes that reference a local variable, update the variable's scope to be the graph's new name (which will mirror the UFunction).
		if (bGraphIsFunction)
		{
			// Find all variable nodes in this graph.
			TArray<UK2Node_Variable*> VariableNodes;
			Graph->GetNodesOfClass<UK2Node_Variable>(VariableNodes);
			GetAllChildGraphVariables(Graph, VariableNodes);

			for (UK2Node_Variable* const VariableNode : VariableNodes)
			{
				if (VariableNode->VariableReference.IsLocalScope())
				{
					// if the rename is the function set the local variable scope to the new name otherwise we leave it with the same scope (Ex: subgraphs in a function)
					VariableNode->VariableReference.SetLocalMember(VariableNode->VariableReference.GetMemberName(), NewNameStr, VariableNode->VariableReference.GetMemberGuid());
				}
			}
		}

		// Rename any function call points (including any calls w/ a child Blueprint as the target).
		for (TObjectIterator<UK2Node_CallFunction> It(RF_ClassDefaultObject | RF_Transient); It; ++It)
		{
			UK2Node_CallFunction* FunctionNode = *It;
			if (FunctionNode && Cast<UEdGraph>(FunctionNode->GetOuter()))
			{
				if (const UBlueprint* FunctionNodeBlueprint = FBlueprintEditorUtils::FindBlueprintForNode(FunctionNode))
				{
					if (FunctionNode->FunctionReference.GetMemberName() == OldGraphName)
					{
						if (FunctionNode->FunctionReference.IsSelfContext() && ModifiedBlueprints.Contains(FunctionNodeBlueprint))
						{
							FunctionNode->Modify();
							FunctionNode->FunctionReference.SetSelfMember(NewGraphName);
						}
						else if (const UBlueprint* TargetBlueprint = UBlueprint::GetBlueprintFromClass(FunctionNode->FunctionReference.GetMemberParentClass()))
						{
							if (ModifiedBlueprints.Contains(TargetBlueprint))
							{
								FunctionNode->Modify();
								FunctionNode->FunctionReference.SetExternalMember(NewGraphName, TargetBlueprint->GeneratedClass);
							}
						}
					}
				}
			}
		}

		// Replace any other nodes that reference this function
		ReplaceFunctionReferences(Blueprint, OldGraphName, NewGraphName);

		// We should let the blueprint know we renamed a graph, some stuff may need to be fixed up.
		Blueprint->NotifyGraphRenamed(Graph, OldGraphName, NewGraphName);

		if (!Blueprint->bIsRegeneratingOnLoad && !Blueprint->bBeingCompiled)
		{
			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
		}
	}
}

void FBlueprintEditorUtils::RenameGraphWithSuggestion(class UEdGraph* Graph, TSharedPtr<class INameValidatorInterface> NameValidator, const FString& DesiredName )
{
	FString NewName = DesiredName;
	NameValidator->FindValidString(NewName);
	UBlueprint* BP = FBlueprintEditorUtils::FindBlueprintForGraphChecked(Graph);
	Graph->Rename(*NewName, Graph->GetOuter(), (BP->bIsRegeneratingOnLoad ? REN_AllowPackageLinkerMismatch : 0) | REN_DontCreateRedirectors);
}

/** 
 * Cleans up a Node in the blueprint
 */
void FBlueprintEditorUtils::RemoveNode(UBlueprint* Blueprint, UEdGraphNode* Node, bool bDontRecompile)
{
	check(Node);

	const UEdGraphSchema* Schema = nullptr;

	// Ensure we mark parent graph modified
	if (UEdGraph* GraphObj = Node->GetGraph())
	{
		GraphObj->Modify();
		Schema = GraphObj->GetSchema();
	}

	if (Blueprint != nullptr)
	{
		// Remove any breakpoints set on the node
		FKismetDebugUtilities::RemoveBreakpointFromNode(Node, Blueprint);

		// Remove any watches set on the node's pins
		for (int32 PinIndex = 0; PinIndex < Node->Pins.Num(); ++PinIndex)
		{
			FKismetDebugUtilities::RemovePinWatch(Blueprint, Node->Pins[PinIndex]);
		}
	}

	Node->Modify();

	// Timelines will be removed from the blueprint if the node is a UK2Node_Timeline
	if (Schema)
	{
		Schema->BreakNodeLinks(*Node);
	}

	Node->DestroyNode(); 

	if (!bDontRecompile && (Blueprint != nullptr))
	{
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
	}
}

UEdGraph* FBlueprintEditorUtils::GetTopLevelGraph(const UEdGraph* InGraph)
{
	UEdGraph* GraphToTest = const_cast<UEdGraph*>(InGraph);

	for (UObject* TestOuter = GraphToTest; TestOuter; TestOuter = TestOuter->GetOuter())
	{
		// reached up to the blueprint for the graph
		if (UBlueprint* Blueprint = Cast<UBlueprint>(TestOuter))
		{
			break;
		}
		else
		{
			GraphToTest = Cast<UEdGraph>(TestOuter);
		}
	}
	return GraphToTest;
}

bool FBlueprintEditorUtils::IsGraphReadOnly(UEdGraph* InGraph)
{
	bool bGraphReadOnly = true;
	if (InGraph)
	{
		bGraphReadOnly = !InGraph->bEditable;

		if (!bGraphReadOnly)
		{
			const UBlueprint* BlueprintForGraph = FBlueprintEditorUtils::FindBlueprintForGraph(InGraph);
			bool const bIsInterface = ((BlueprintForGraph != nullptr) && (BlueprintForGraph->BlueprintType == BPTYPE_Interface));
			bool const bIsDelegate  = FBlueprintEditorUtils::IsDelegateSignatureGraph(InGraph);
			bool const bIsMathExpression = FBlueprintEditorUtils::IsMathExpressionGraph(InGraph);

			bGraphReadOnly = bIsInterface || bIsDelegate || bIsMathExpression;
		}
	}
	return bGraphReadOnly;
}

UK2Node_Event* FBlueprintEditorUtils::FindOverrideForFunction(const UBlueprint* Blueprint, const UClass* SignatureClass, FName SignatureName)
{
	TArray<UK2Node_Event*> AllEvents;
	FBlueprintEditorUtils::GetAllNodesOfClass<UK2Node_Event>(Blueprint, AllEvents);

	for(int32 i=0; i<AllEvents.Num(); i++)
	{
		UK2Node_Event* EventNode = AllEvents[i];
		check(EventNode);
		if(	EventNode->bOverrideFunction == true &&
			EventNode->EventReference.GetMemberName() == SignatureName )
		{
			const UClass* MemberParentClass = EventNode->EventReference.GetMemberParentClass(EventNode->GetBlueprintClassFromNode());
			if(MemberParentClass && MemberParentClass->IsChildOf(SignatureClass))
			{
				return EventNode;
			}
		}
	}

	return nullptr;
}

UK2Node_Event* FBlueprintEditorUtils::FindCustomEventNode(const UBlueprint* Blueprint, FName const CustomName)
{
	UK2Node_Event* FoundNode = nullptr;

	if (CustomName != NAME_None)
	{
		TArray<UK2Node_Event*> AllEvents;
		FBlueprintEditorUtils::GetAllNodesOfClass<UK2Node_Event>(Blueprint, AllEvents);

		for (UK2Node_Event* EventNode : AllEvents)
		{
			if (EventNode->CustomFunctionName == CustomName)
			{
				FoundNode = EventNode;
				break;
			}
		}
	}
	return FoundNode;
}

void FBlueprintEditorUtils::GatherDependencies(const UBlueprint* InBlueprint, TSet<TWeakObjectPtr<UBlueprint>>& Dependencies, TSet<TWeakObjectPtr<UStruct>>& OutUDSDependencies)
{
	struct FGatherDependenciesHelper
	{
		static UBlueprint* GetGeneratingBlueprint(const UObject* Obj)
		{
			const UBlueprintGeneratedClass* BPGC = nullptr;
			while (!BPGC && Obj)
			{
				BPGC = Cast<const UBlueprintGeneratedClass>(Obj);
				Obj = Obj->GetOuter();
			}

			return UBlueprint::GetBlueprintFromClass(BPGC);
		}

		static void ProcessHierarchy(const UStruct* Struct, TSet<TWeakObjectPtr<UBlueprint>>& InDependencies)
		{
			for (UBlueprint* Blueprint = GetGeneratingBlueprint(Struct);
				Blueprint;
				Blueprint = UBlueprint::GetBlueprintFromClass(Cast<UBlueprintGeneratedClass>(Blueprint->ParentClass)))
			{
				bool bAlreadyProcessed = false;
				InDependencies.Add(Blueprint, &bAlreadyProcessed);
				if (bAlreadyProcessed)
				{
					return;
				}

				Blueprint->GatherDependencies(InDependencies);
			}
		}
	};

	check(InBlueprint);
	Dependencies.Empty();
	OutUDSDependencies.Empty();

	// If the Blueprint's GeneratedClass was not generated by the Blueprint, it's either corrupt or a PIE version of the BP
	if (InBlueprint->GeneratedClass && InBlueprint->GeneratedClass->ClassGeneratedBy.Get() != InBlueprint)
	{
		// Dependencies do not matter for PIE duplicated Blueprints
		return;
	}

	InBlueprint->GatherDependencies(Dependencies);

	FGatherDependenciesHelper::ProcessHierarchy(InBlueprint->ParentClass, Dependencies);

	for (const FBPInterfaceDescription& InterfaceDesc : InBlueprint->ImplementedInterfaces)
	{
		UBlueprint* InterfaceBP = InterfaceDesc.Interface ? Cast<UBlueprint>(InterfaceDesc.Interface->ClassGeneratedBy) : nullptr;
		if (InterfaceBP)
		{
			Dependencies.Add(InterfaceBP);
		}
	}

	TArray<UEdGraph*> Graphs;
	InBlueprint->GetAllGraphs(Graphs);
	for (UEdGraph* Graph : Graphs)
	{
		if (Graph && !FBlueprintEditorUtils::IsGraphIntermediate(Graph))
		{
			TArray<UK2Node*> Nodes;
			Graph->GetNodesOfClass(Nodes);
			for (UK2Node* Node : Nodes)
			{
				TArray<UStruct*> LocalDependentStructures;
				if (Node && Node->HasExternalDependencies(&LocalDependentStructures))
				{
					for (UStruct* Struct : LocalDependentStructures)
					{
						if (UUserDefinedStruct* UDS = Cast<UUserDefinedStruct>(Struct))
						{
							OutUDSDependencies.Add(UDS);
						}
						else
						{
							FGatherDependenciesHelper::ProcessHierarchy(Struct, Dependencies);
						}
					}
				}
			}
		}
	}

	Dependencies.Remove(MakeWeakObjectPtr(const_cast<UBlueprint*>(InBlueprint)));
}

void FBlueprintEditorUtils::EnsureCachedDependenciesUpToDate(UBlueprint* Blueprint)
{
	if (Blueprint && !Blueprint->bCachedDependenciesUpToDate)
	{
		GatherDependencies(Blueprint, Blueprint->CachedDependencies, Blueprint->CachedUDSDependencies);
		Blueprint->bCachedDependenciesUpToDate = true;

		// A macro dependency will result in an expansion from an external graph rather than a local one, so we must also include its dependencies.
		TSet<TWeakObjectPtr<UBlueprint>> LocalCopyOfCachedDependencies = Blueprint->CachedDependencies;
		for (const TWeakObjectPtr<UBlueprint>& Dependency : LocalCopyOfCachedDependencies)
		{
			UBlueprint* ResolvedDependency = Dependency.Get();
			if (ResolvedDependency && ResolvedDependency->BlueprintType == BPTYPE_MacroLibrary)
			{
				EnsureCachedDependenciesUpToDate(ResolvedDependency);
				Blueprint->CachedDependencies.Append(ResolvedDependency->CachedDependencies);
				Blueprint->CachedUDSDependencies.Append(ResolvedDependency->CachedUDSDependencies);
			}
		}
	}
}

void FBlueprintEditorUtils::GetDependentBlueprints(UBlueprint* Blueprint, TArray<UBlueprint*>& DependentBlueprints)
{
	for( TWeakObjectPtr<UBlueprint> DependentBPWeak : Blueprint->CachedDependents )
	{
		if(UBlueprint* DependentBP = DependentBPWeak.Get())
		{
			DependentBlueprints.Add(DependentBP);
		}
	}
}

void FBlueprintEditorUtils::FindDependentBlueprints(UBlueprint* Blueprint, TArray<UBlueprint*>& DependentBlueprints)
{	
	if(Blueprint == nullptr)
	{
		return;
	}

	TArray<UObject*> AllBlueprints;
	bool const bIncludeDerivedClasses = true;
	GetObjectsOfClass(UBlueprint::StaticClass(), AllBlueprints, bIncludeDerivedClasses );

	// Sanitize, add correct type.. can't find a UObject* helper to do this, and 
	// the previous version of htis code checked IsPendingKill():
	TArray<UBlueprint*> AllBlueprintSafe;
	Algo::TransformIf(AllBlueprints, AllBlueprintSafe, 
		[](UObject* Obj)->bool { return IsValid(Obj); }, 
		[](UObject* Obj)->UBlueprint* { return static_cast<UBlueprint*>(Obj); } 
	);

	// Update *all* dependencies:
	for (UBlueprint* TestBP : AllBlueprintSafe)
	{
		EnsureCachedDependenciesUpToDate(TestBP);
	}

	// Gather macro blueprints that we're dependent on:
	TSet<UBlueprint*> DepSet;
	Algo::CopyIf(AllBlueprintSafe, DepSet,
		[Blueprint](UBlueprint* TestBP) -> bool 
		{ 
			if(TestBP->CachedDependencies.Contains(Blueprint))
			{
				if (TestBP->BlueprintType == BPTYPE_MacroLibrary)
				{
					return true;
				}
			}
			return false;
		}
	);
	DepSet.Add(Blueprint);
	
	// Find all blueprints that have a cached dep on Blueprint *or* one of the 
	// macros that is dependent on Blueprint:
	Algo::CopyIf(AllBlueprintSafe, DependentBlueprints,
		[DepSet](UBlueprint* TestBP) -> bool
		{
			// check for interesection between CachedDependencies and DepSet:
			for(TWeakObjectPtr<UBlueprint> ObjWeak : TestBP->CachedDependencies)
			{
				if(DepSet.Contains(ObjWeak.Get()))
				{
					return true;
				}
			}
			return false;
		}
	);

	// Remove ourself from the list of dependents:
	DependentBlueprints.RemoveSwap(Blueprint);
}

bool FBlueprintEditorUtils::IsGraphIntermediate(const UEdGraph* Graph)
{
	if (Graph)
	{
		return Graph->HasAllFlags(RF_Transient);
	}
	return false;
}

bool FBlueprintEditorUtils::IsDataOnlyBlueprint(const UBlueprint* Blueprint)
{
	// Blueprint interfaces are always compiled
	if (Blueprint->BlueprintType == BPTYPE_Interface)
	{
		return false;
	}

	if (Blueprint->AlwaysCompileOnLoad())
	{
		return false;
	}

	// No new variables defined
	if (Blueprint->NewVariables.Num() > 0)
	{
		return false;
	}
	
	// No extra functions, other than the user construction script(only AActor and subclasses of AActor have)
	const int32 DefaultFunctionNum = (Blueprint->ParentClass && Blueprint->ParentClass->IsChildOf(AActor::StaticClass())) ? 1 : 0;
	if ((Blueprint->FunctionGraphs.Num() > DefaultFunctionNum) || (Blueprint->MacroGraphs.Num() > 0))
	{
		return false;
	}

	if (Blueprint->DelegateSignatureGraphs.Num())
	{
		return false;
	}

	if (Blueprint->ComponentTemplates.Num() > 0 || Blueprint->Timelines.Num() > 0 || (Blueprint->ComponentClassOverrides.Num() > 0 && GetAllowNativeComponentClassOverrides()))
	{
		return false;
	}

	if (USimpleConstructionScript* SimpleConstructionScript = Blueprint->SimpleConstructionScript)
	{
		const TArray<USCS_Node*>& Nodes = SimpleConstructionScript->GetAllNodes();
		if (Nodes.Num() > 1)
		{
			return false;
		}
		if ((1 == Nodes.Num()) && (Nodes[0] != SimpleConstructionScript->GetDefaultSceneRootNode()))
		{
			return false;
		}
	}

	// Make sure there's nothing in the user construction script, other than an entry node
	UEdGraph* UserConstructionScript = (Blueprint->FunctionGraphs.Num() == 1) ? ToRawPtr(Blueprint->FunctionGraphs[0]) : nullptr;
	if (UserConstructionScript && Blueprint->ParentClass)
	{
		//Call parent construction script may be added automatically
		UBlueprint* BlueprintParent = Cast<UBlueprint>(Blueprint->ParentClass->ClassGeneratedBy);
		// just 1 entry node or just one entry node and a call to our super, which is DataOnly:
		if ( !BlueprintParent && UserConstructionScript->Nodes.Num() > 1 )
		{
			return false;
		}
		else if (BlueprintParent)
		{
			// More than two nodes.. one of them must do something (same logic as above, but we have a call to super as well)
			if (UserConstructionScript->Nodes.Num() > 2)
			{
				return false;
			}
			else
			{
				// Just make sure the nodes are trivial, if they aren't then we're not data only:
				for (UEdGraphNode* Node : UserConstructionScript->Nodes)
				{
					if (!Cast<UK2Node_FunctionEntry>(Node) &&
						!Cast<UK2Node_CallParentFunction>(Node))
					{
						return false;
					}
				}
			}
		}
	}

	// All EventGraphs are empty (at least of non-ghost, non-disabled nodes)
	for (UEdGraph* EventGraph : Blueprint->UbergraphPages)
	{
		for (UEdGraphNode* GraphNode : EventGraph->Nodes)
		{
			// If there is an enabled node in the event graph, the Blueprint is not data only
			if (GraphNode && (GraphNode->GetDesiredEnabledState() != ENodeEnabledState::Disabled))
			{
				return false;
			}
		}
	}

	// No implemented interfaces
	if (Blueprint->ImplementedInterfaces.Num() > 0)
	{
		return false;
	}

	return true;
}

bool FBlueprintEditorUtils::IsBlueprintConst(const UBlueprint* Blueprint)
{
	// Macros aren't marked as const because they can modify variables when instanced into a non const class
	// and will be caught at compile time if they're modifying variables on a const class.
	return Blueprint && Blueprint->BlueprintType == BPTYPE_Const;
}

bool FBlueprintEditorUtils::IsEditorUtilityBlueprint(const UBlueprint* Blueprint)
{
	IBlutilityModule* BlutilityModule = FModuleManager::GetModulePtr<IBlutilityModule>("Blutility");

	if (BlutilityModule)
	{
		return BlutilityModule->IsEditorUtilityBlueprint( Blueprint );
	}
	return false;
}

bool FBlueprintEditorUtils::IsActorBased(const UBlueprint* Blueprint)
{
	return Blueprint && Blueprint->ParentClass && Blueprint->ParentClass->IsChildOf(AActor::StaticClass());
}

bool FBlueprintEditorUtils::IsComponentBased(const UBlueprint* Blueprint)
{
	return Blueprint && Blueprint->ParentClass && Blueprint->ParentClass->IsChildOf(UActorComponent::StaticClass());
}

bool FBlueprintEditorUtils::IsDelegateSignatureGraph(const UEdGraph* Graph)
{
	if(Graph)
	{
		if(const UBlueprint* Blueprint = FindBlueprintForGraph(Graph))
		{
			return (nullptr != Blueprint->DelegateSignatureGraphs.FindByKey(Graph));
		}
	}
	return false;
}

bool FBlueprintEditorUtils::IsMathExpressionGraph(const UEdGraph* InGraph)
{
	if(InGraph)
	{
		return InGraph->GetOuter()->GetClass() == UK2Node_MathExpression::StaticClass();
	}
	return false;
}

bool FBlueprintEditorUtils::IsInterfaceBlueprint(const UBlueprint* Blueprint)
{
	return (Blueprint && Blueprint->BlueprintType == BPTYPE_Interface);
}

bool FBlueprintEditorUtils::IsInterfaceGraph(const UEdGraph* Graph)
{
	return IsInterfaceBlueprint(FindBlueprintForGraph(Graph));
}

bool FBlueprintEditorUtils::IsLevelScriptBlueprint(const UBlueprint* Blueprint)
{
	return (Blueprint && Blueprint->BlueprintType == BPTYPE_LevelScript);
}

bool FBlueprintEditorUtils::IsParentClassABlueprint(const UBlueprint* Blueprint)
{
	if (Blueprint)
	{
		UObject* ParentClass = Blueprint->ParentClass;
		if (ParentClass)
		{
			if (ParentClass->IsA(UBlueprintGeneratedClass::StaticClass()))
			{
				return true;
			}
		}
	}

	return false;
}

bool FBlueprintEditorUtils::IsParentClassAnEditableBlueprint(const UBlueprint* Blueprint)
{
	if (Blueprint)
	{
		UObject* ParentClass = Blueprint->ParentClass;
		if (ParentClass)
		{
			UBlueprintGeneratedClass* ParentBPGC = Cast<UBlueprintGeneratedClass>(ParentClass);
			if (ParentBPGC && ParentBPGC->ClassGeneratedBy)
			{
				return true;
			}
		}
	}

	return false;
}

bool FBlueprintEditorUtils::IsAnonymousBlueprintClass(const UClass* Class)
{
	return (Class && Class->GetOutermost()->ContainsMap());
}

ULevel* FBlueprintEditorUtils::GetLevelFromBlueprint(const UBlueprint* Blueprint)
{
	return (Blueprint ? Cast<ULevel>(Blueprint->GetOuter()) : nullptr);
}

bool FBlueprintEditorUtils::SupportsConstructionScript(const UBlueprint* Blueprint)
{
	return(	!FBlueprintEditorUtils::IsInterfaceBlueprint(Blueprint) && 
			!FBlueprintEditorUtils::IsBlueprintConst(Blueprint) && 
			!FBlueprintEditorUtils::IsLevelScriptBlueprint(Blueprint) && 
			FBlueprintEditorUtils::IsActorBased(Blueprint)) &&
			!(Blueprint->BlueprintType == BPTYPE_MacroLibrary) &&
			!(Blueprint->BlueprintType == BPTYPE_FunctionLibrary);
}

bool FBlueprintEditorUtils::CanClassGenerateEvents(const UClass* InClass)
{
	if( InClass )
	{
		for( TFieldIterator<FMulticastDelegateProperty> PropertyIt( InClass, EFieldIteratorFlags::IncludeSuper ); PropertyIt; ++PropertyIt )
		{
			FProperty* Property = *PropertyIt;
			if( !Property->HasAnyPropertyFlags( CPF_Parm ) && Property->HasAllPropertyFlags( CPF_BlueprintAssignable ))
			{
				return true;
			}
		}
	}
	return false;
}

bool FBlueprintEditorUtils::CanCreateChildBlueprint(const UBlueprint* BP)
{
	if (!BP)
	{
		return false;
	}
	
	// BP function libraries cannot have child BP's created of them. You will only ever get compilation
	// errors if you made one.
	if (BP->BlueprintType == EBlueprintType::BPTYPE_FunctionLibrary)
	{
		return false;
	}

	// Do not allow child classes to be created from deprecated BPs
	return BP->GeneratedClass && !BP->GeneratedClass->HasAnyClassFlags(CLASS_Deprecated);
}

UEdGraph* FBlueprintEditorUtils::FindUserConstructionScript(const UBlueprint* Blueprint)
{
	for (UEdGraph* CurrentGraph : Blueprint->FunctionGraphs)
	{
		if( CurrentGraph->GetFName() == UEdGraphSchema_K2::FN_UserConstructionScript )
		{
			return CurrentGraph;
		}
	}

	return nullptr;
}

UEdGraph* FBlueprintEditorUtils::FindEventGraph(const UBlueprint* Blueprint)
{
	for (UEdGraph* CurrentGraph : Blueprint->UbergraphPages)
	{
		if( CurrentGraph->GetFName() == UEdGraphSchema_K2::GN_EventGraph )
		{
			return CurrentGraph;
		}
	}

	return nullptr;
}

bool FBlueprintEditorUtils::IsEventGraph(const UEdGraph* InGraph)
{
	if (InGraph)
	{
		if (const UBlueprint* Blueprint = FindBlueprintForGraph(InGraph))
		{
			return (nullptr != Blueprint->UbergraphPages.FindByKey(InGraph));
		}
	}
	return false;
}

bool FBlueprintEditorUtils::IsTunnelInstanceNode(const UEdGraphNode* InGraphNode)
{
	if (InGraphNode)
	{
		return InGraphNode->IsA<UK2Node_MacroInstance>() || InGraphNode->IsA<UK2Node_Composite>();
	}
	return false;
}

bool FBlueprintEditorUtils::DoesBlueprintDeriveFrom(const UBlueprint* Blueprint, UClass* TestClass)
{
	check(Blueprint->SkeletonGeneratedClass != nullptr);
	return	TestClass != nullptr && 
		Blueprint->SkeletonGeneratedClass->IsChildOf(TestClass);
}

bool FBlueprintEditorUtils::DoesBlueprintContainField(const UBlueprint* Blueprint, UField* TestField)
{
	// Get the class of the field
	if(TestField)
	{
		// Local properties do not have a UClass outer but are also not a part of the Blueprint
		UClass* TestClass = Cast<UClass>(TestField->GetOuter());
		if(TestClass)
		{
			return FBlueprintEditorUtils::DoesBlueprintDeriveFrom(Blueprint, TestClass);
		}
	}
	return false;
}

bool FBlueprintEditorUtils::DoesSupportOverridingFunctions(const UBlueprint* Blueprint)
{
	return Blueprint->BlueprintType != BPTYPE_MacroLibrary 
		&& Blueprint->BlueprintType != BPTYPE_Interface
		&& Blueprint->BlueprintType != BPTYPE_FunctionLibrary;
}

bool FBlueprintEditorUtils::DoesSupportTimelines(const UBlueprint* Blueprint)
{
	// Right now, just assume actor based blueprints support timelines
	return FBlueprintEditorUtils::IsActorBased(Blueprint) && FBlueprintEditorUtils::DoesSupportEventGraphs(Blueprint);
}

bool FBlueprintEditorUtils::DoesSupportEventGraphs(const UBlueprint* Blueprint)
{
	return Blueprint->BlueprintType == BPTYPE_Normal
			|| Blueprint->BlueprintType == BPTYPE_LevelScript;
}

/** Returns whether or not the blueprint supports implementing interfaces */
bool FBlueprintEditorUtils::DoesSupportImplementingInterfaces(const UBlueprint* Blueprint)
{
	return Blueprint->BlueprintType != BPTYPE_MacroLibrary
		&& Blueprint->BlueprintType != BPTYPE_Interface
		&& Blueprint->BlueprintType != BPTYPE_LevelScript
		&& Blueprint->BlueprintType != BPTYPE_FunctionLibrary;
}

bool FBlueprintEditorUtils::DoesSupportComponents(UBlueprint const* Blueprint)
{
	return (Blueprint->SimpleConstructionScript != nullptr)      // An SCS must be present (otherwise there is nothing valid to edit)
		&& FBlueprintEditorUtils::IsActorBased(Blueprint)        // Must be parented to an AActor-derived class (some older BPs may have an SCS but may not be Actor-based)
		&& (Blueprint->BlueprintType != BPTYPE_MacroLibrary)     // Must not be a macro-type Blueprint
		&& (Blueprint->BlueprintType != BPTYPE_FunctionLibrary); // Must not be a function library
}

bool FBlueprintEditorUtils::DoesSupportDefaults(UBlueprint const* Blueprint)
{
	return Blueprint->BlueprintType != BPTYPE_MacroLibrary
		&& Blueprint->BlueprintType != BPTYPE_FunctionLibrary;
}

bool FBlueprintEditorUtils::DoesSupportLocalVariables(UEdGraph const* InGraph)
{
	if(InGraph)
	{
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(InGraph);
		return Blueprint
			&& Blueprint->BlueprintType != BPTYPE_Interface
			&& InGraph->GetSchema()->GetGraphType(InGraph) == EGraphType::GT_Function
			&& !InGraph->IsA(UAnimationTransitionGraph::StaticClass());
	}
	return false;
}

// Returns a descriptive name of the type of blueprint passed in
FString FBlueprintEditorUtils::GetBlueprintTypeDescription(const UBlueprint* Blueprint)
{
	FString BlueprintTypeString;
	switch (Blueprint->BlueprintType)
	{
	case BPTYPE_LevelScript:
		BlueprintTypeString = LOCTEXT("BlueprintType_LevelScript", "Level Blueprint").ToString();
		break;
	case BPTYPE_MacroLibrary:
		BlueprintTypeString = LOCTEXT("BlueprintType_MacroLibrary", "Macro Library").ToString();
		break;
	case BPTYPE_Interface:
		BlueprintTypeString = LOCTEXT("BlueprintType_Interface", "Interface").ToString();
		break;
	case BPTYPE_FunctionLibrary:
	case BPTYPE_Normal:
	case BPTYPE_Const:
		BlueprintTypeString = Blueprint->GetClass()->GetName();
		break;
	default:
		BlueprintTypeString = TEXT("Unknown blueprint type");
	}

	return BlueprintTypeString;
}

//////////////////////////////////////////////////////////////////////////
// Variables

bool FBlueprintEditorUtils::IsVariableCreatedByBlueprint(UBlueprint* InBlueprint, FProperty* InVariableProperty)
{
	bool bIsVariableCreatedByBlueprint = false;

	if (InBlueprint != nullptr && InVariableProperty != nullptr)
	{
		if (UBlueprintGeneratedClass* GeneratedClass = Cast<UBlueprintGeneratedClass>(InVariableProperty->GetOwnerClass()))
		{
			UBlueprint* OwnerBlueprint = Cast<UBlueprint>(GeneratedClass->ClassGeneratedBy);
			bIsVariableCreatedByBlueprint = (OwnerBlueprint == InBlueprint && FBlueprintEditorUtils::FindNewVariableIndex(InBlueprint, InVariableProperty->GetFName()) != INDEX_NONE);
		}
	}

	return bIsVariableCreatedByBlueprint;
}

// Find the index of a variable first declared in this blueprint. Returns INDEX_NONE if not found.
int32 FBlueprintEditorUtils::FindNewVariableIndex(const UBlueprint* Blueprint, const FName& InName) 
{
	if(InName != NAME_None && Blueprint != nullptr)
	{
		for(int32 i=0; i<Blueprint->NewVariables.Num(); i++)
		{
			if(Blueprint->NewVariables[i].VarName == InName)
			{
				return i;
			}
		}
	}

	return INDEX_NONE;
}

int32 FBlueprintEditorUtils::FindNewVariableIndexAndBlueprint(UBlueprint* InBlueprint, FName InName, UBlueprint*& OutFoundBlueprint)
{
	OutFoundBlueprint = InBlueprint;

	while (OutFoundBlueprint)
	{
		int32 FoundIndex = FindNewVariableIndex(OutFoundBlueprint, InName);
		if (FoundIndex != INDEX_NONE)
		{
			return FoundIndex;
		}
		else
		{
			OutFoundBlueprint = UBlueprint::GetBlueprintFromClass(OutFoundBlueprint->ParentClass);
		}
	}

	return INDEX_NONE;
}

int32 FBlueprintEditorUtils::FindLocalVariableIndex(const UBlueprint* Blueprint, UStruct* VariableScope, const FName& InVariableName)
{
	UK2Node_FunctionEntry* FunctionEntryNode = nullptr;
	FindLocalVariable(Blueprint, VariableScope, InVariableName, &FunctionEntryNode);

	if (FunctionEntryNode != nullptr)
	{
		for (int32 i = 0; i < FunctionEntryNode->LocalVariables.Num(); i++)
		{
			if (FunctionEntryNode->LocalVariables[i].VarName == InVariableName)
			{
				return i;
			}
		}
	}
	return INDEX_NONE;
}

/** Helper function for moving variables around in the list relative to a target variable */
static bool MoveVariableRelativeToVariable(UBlueprint* Blueprint, UStruct* VariableScope, FName VarNameToMove, FName TargetVarName, bool bDontRecompile, bool bInsertBeforeTargetVariable)
{
	check(Blueprint && VariableScope);

	bool bMoved = false;
	int32 VarIndexToMove = INDEX_NONE;
	int32 TargetVarIndex = INDEX_NONE;

	//Get the indices of the variables to be re-ordered
	if (VariableScope->IsA(UFunction::StaticClass()))
	{
		VarIndexToMove = FBlueprintEditorUtils::FindLocalVariableIndex(Blueprint, VariableScope, VarNameToMove);
		TargetVarIndex = FBlueprintEditorUtils::FindLocalVariableIndex(Blueprint, VariableScope, TargetVarName);
	}
	else
	{
		VarIndexToMove = FBlueprintEditorUtils::FindNewVariableIndex(Blueprint, VarNameToMove);
		TargetVarIndex = FBlueprintEditorUtils::FindNewVariableIndex(Blueprint, TargetVarName);
	}

	if (VarIndexToMove != INDEX_NONE && TargetVarIndex != INDEX_NONE)
	{
		// When we remove item, will back all items after it. If your target is after it, need to adjust
		if (TargetVarIndex > VarIndexToMove)
		{
			TargetVarIndex--;
		}

		//Handle Local vs Class scope 
		if (VariableScope->IsA(UFunction::StaticClass()))
		{
			UK2Node_FunctionEntry* FunctionEntryNode = nullptr;
			FBlueprintEditorUtils::FindLocalVariable(Blueprint, VariableScope, VarNameToMove, &FunctionEntryNode);

			if (FunctionEntryNode != nullptr)
			{
				// Copy var we want to move
				FBPVariableDescription MoveVar = FunctionEntryNode->LocalVariables[VarIndexToMove];

				// Remove var we are moving
				FunctionEntryNode->LocalVariables.RemoveAt(VarIndexToMove);
				// Add in before target variable
				FunctionEntryNode->LocalVariables.Insert(MoveVar, bInsertBeforeTargetVariable ? TargetVarIndex : TargetVarIndex + 1);
			}
		}
		else
		{			
			// Copy var we want to move
			FBPVariableDescription MoveVar = Blueprint->NewVariables[VarIndexToMove];
			
			// Remove var we are moving
			Blueprint->NewVariables.RemoveAt(VarIndexToMove);
			// Add in before target variable
			Blueprint->NewVariables.Insert(MoveVar, bInsertBeforeTargetVariable ? TargetVarIndex : TargetVarIndex + 1);
		}

		if (!bDontRecompile)
		{
			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
		}
		bMoved = true;
	}
	return bMoved;
}

bool FBlueprintEditorUtils::MoveVariableBeforeVariable(UBlueprint* Blueprint, UStruct* VariableScope, FName VarNameToMove, FName TargetVarName, bool bDontRecompile)
{
	return MoveVariableRelativeToVariable(Blueprint, VariableScope, VarNameToMove, TargetVarName, bDontRecompile, true);
}

bool FBlueprintEditorUtils::MoveVariableAfterVariable(UBlueprint* Blueprint, UStruct* VariableScope, FName VarNameToMove, FName TargetVarName, bool bDontRecompile)
{
	return MoveVariableRelativeToVariable(Blueprint, VariableScope, VarNameToMove, TargetVarName, bDontRecompile, false);
}

int32 FBlueprintEditorUtils::FindTimelineIndex(const UBlueprint* Blueprint, const FName& InName) 
{
	const FName TimelineTemplateName = *UTimelineTemplate::TimelineVariableNameToTemplateName(InName);
	for(int32 i=0; i<Blueprint->Timelines.Num(); i++)
	{
		if(Blueprint->Timelines[i] && Blueprint->Timelines[i]->GetFName() == TimelineTemplateName)
		{
			return i;
		}
	}

	return INDEX_NONE;
}

void FBlueprintEditorUtils::GetSCSVariableNameList(const UBlueprint* Blueprint, TSet<FName>& VariableNames)
{
	if (Blueprint)
	{
		GetSCSVariableNameList(Blueprint->SimpleConstructionScript, VariableNames);
	}
}

void FBlueprintEditorUtils::GetSCSVariableNameList(const UBlueprintGeneratedClass* BPGC, TSet<FName>& VariableNames)
{
	if (BPGC)
	{
		GetSCSVariableNameList(BPGC->SimpleConstructionScript, VariableNames);
	}
}

void FBlueprintEditorUtils::GetSCSVariableNameList(const USimpleConstructionScript* SCS, TSet<FName>& VariableNames)
{
	if (SCS)
	{
		for (USCS_Node* SCS_Node : SCS->GetAllNodes())
		{
			if (SCS_Node)
			{
				const FName VariableName = SCS_Node->GetVariableName();
				if (VariableName != NAME_None)
				{
					VariableNames.Add(VariableName);
				}
			}
		}
	}
}

void FBlueprintEditorUtils::GetImplementingBlueprintsFunctionNameList(const UBlueprint* Blueprint, TSet<FName>& FunctionNames)
{
	if(Blueprint != nullptr && FBlueprintEditorUtils::IsInterfaceBlueprint(Blueprint))
	{
		for(TObjectIterator<UBlueprint> BlueprintIt; BlueprintIt; ++BlueprintIt)
		{
			const UBlueprint* ChildBlueprint = *BlueprintIt;
			if(ChildBlueprint != nullptr)
			{
				for (int32 InterfaceIndex = 0; InterfaceIndex < ChildBlueprint->ImplementedInterfaces.Num(); InterfaceIndex++)
				{
					const FBPInterfaceDescription& CurrentInterface = ChildBlueprint->ImplementedInterfaces[InterfaceIndex];
					const UBlueprint* BlueprintInterfaceClass = UBlueprint::GetBlueprintFromClass(CurrentInterface.Interface);
					if(BlueprintInterfaceClass != nullptr && BlueprintInterfaceClass == Blueprint)
					{
						FBlueprintEditorUtils::GetAllGraphNames(ChildBlueprint, FunctionNames);
					}
				}
			}
		}
	}
}

USCS_Node* FBlueprintEditorUtils::FindSCS_Node(const UBlueprint* Blueprint, const FName InName) 
{
	if (Blueprint->SimpleConstructionScript)
	{
		const TArray<USCS_Node*>& AllSCS_Nodes = Blueprint->SimpleConstructionScript->GetAllNodes();
	
		for (USCS_Node* SCSNode : AllSCS_Nodes)
		{
			if (SCSNode->GetVariableName() == InName)
			{
				return SCSNode;
			}
		}
	}

	return nullptr;
}

void FBlueprintEditorUtils::SetBlueprintOnlyEditableFlag(UBlueprint* Blueprint, const FName& VarName, const bool bNewBlueprintOnly)
{
	const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(Blueprint, VarName);

	if (bNewBlueprintOnly)
	{
		FBlueprintEditorUtils::RemoveBlueprintVariableMetaData(Blueprint, VarName, nullptr, FEdMode::MD_MakeEditWidget);
	}

	if (VarIndex != INDEX_NONE)
	{
		if( bNewBlueprintOnly )
		{
			Blueprint->NewVariables[VarIndex].PropertyFlags |= CPF_DisableEditOnInstance;
		}
		else
		{
			Blueprint->NewVariables[VarIndex].PropertyFlags &= ~CPF_DisableEditOnInstance;
		}
	}

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
}

void FBlueprintEditorUtils::SetBlueprintPropertyReadOnlyFlag(UBlueprint* Blueprint, const FName& VarName, const bool bVariableReadOnly)
{
	const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(Blueprint, VarName);

	if (bVariableReadOnly)
	{
		FBlueprintEditorUtils::RemoveBlueprintVariableMetaData(Blueprint, VarName, nullptr, FEdMode::MD_MakeEditWidget);
	}

	if (VarIndex != INDEX_NONE)
	{
		if (bVariableReadOnly)
		{
			Blueprint->NewVariables[VarIndex].PropertyFlags |= CPF_BlueprintReadOnly;
		}
		else
		{
			Blueprint->NewVariables[VarIndex].PropertyFlags &= ~CPF_BlueprintReadOnly;
		}
	}

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
}

void FBlueprintEditorUtils::SetInterpFlag(UBlueprint* Blueprint, const FName& VarName, const bool bInterp)
{
	const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(Blueprint, VarName);
	if (VarIndex != INDEX_NONE)
	{
		if( bInterp )
		{
			Blueprint->NewVariables[VarIndex].PropertyFlags |= (CPF_Interp);
		}
		else
		{
			Blueprint->NewVariables[VarIndex].PropertyFlags &= ~(CPF_Interp);
		}
	}

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
}

void FBlueprintEditorUtils::SetVariableTransientFlag(UBlueprint* InBlueprint, const FName& InVarName, const bool bInIsTransient)
{
	const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(InBlueprint, InVarName);

	if (VarIndex != INDEX_NONE)
	{
		if( bInIsTransient )
		{
			InBlueprint->NewVariables[VarIndex].PropertyFlags |= CPF_Transient;
		}
		else
		{
			InBlueprint->NewVariables[VarIndex].PropertyFlags &= ~CPF_Transient;
		}
	}

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(InBlueprint);
}

void FBlueprintEditorUtils::SetVariableSaveGameFlag(UBlueprint* InBlueprint, const FName& InVarName, const bool bInIsSaveGame)
{
	const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(InBlueprint, InVarName);

	if (VarIndex != INDEX_NONE)
	{
		if( bInIsSaveGame )
		{
			InBlueprint->NewVariables[VarIndex].PropertyFlags |= CPF_SaveGame;
		}
		else
		{
			InBlueprint->NewVariables[VarIndex].PropertyFlags &= ~CPF_SaveGame;
		}
	}

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(InBlueprint);
}

void FBlueprintEditorUtils::SetVariableAdvancedDisplayFlag(UBlueprint* InBlueprint, const FName& InVarName, const bool bInIsAdvancedDisplay)
{
	const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(InBlueprint, InVarName);

	if (VarIndex != INDEX_NONE)
	{
		if( bInIsAdvancedDisplay )
		{
			InBlueprint->NewVariables[VarIndex].PropertyFlags |= CPF_AdvancedDisplay;
		}
		else
		{
			InBlueprint->NewVariables[VarIndex].PropertyFlags &= ~CPF_AdvancedDisplay;
		}
	}

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(InBlueprint);
}

void FBlueprintEditorUtils::SetVariableDeprecatedFlag(UBlueprint* InBlueprint, const FName& InVarName, const bool bInIsDeprecated)
{
	const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(InBlueprint, InVarName);

	if (VarIndex != INDEX_NONE)
	{
		if (bInIsDeprecated)
		{
			InBlueprint->NewVariables[VarIndex].PropertyFlags |= CPF_Deprecated;
		}
		else
		{
			InBlueprint->NewVariables[VarIndex].PropertyFlags &= ~CPF_Deprecated;
		}
	}

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(InBlueprint);
}

struct FMetaDataDependencyHelper
{
	static void OnChange(UBlueprint* Blueprint, FName MetaDataKey)
	{
		if (Blueprint && (FBlueprintMetadata::MD_ExposeOnSpawn == MetaDataKey))
		{
			TArray<UEdGraph*> AllGraphs;
			Blueprint->GetAllGraphs(AllGraphs);
			for (UEdGraph* Graph : AllGraphs)
			{
				if (Graph)
				{
					const UEdGraphSchema* Schema = Graph->GetSchema();
					TArray<UK2Node_SpawnActorFromClass*> LocalSpawnNodes;
					Graph->GetNodesOfClass(LocalSpawnNodes);
					for (UK2Node_SpawnActorFromClass* Node : LocalSpawnNodes)
					{
						UClass* ClassToSpawn = Node ? Node->GetClassToSpawn() : nullptr;
						if (ClassToSpawn && ClassToSpawn->IsChildOf(Blueprint->GeneratedClass))
						{
							Schema->ReconstructNode(*Node, true);
						}
					}
				}
			}
		}
	}
};

void FBlueprintEditorUtils::SetBlueprintVariableMetaData(UBlueprint* Blueprint, const FName& VarName, const UStruct* InLocalVarScope, const FName& MetaDataKey, const FString& MetaDataValue)
{
	// If there is a local var scope, we know we are looking at a local variable
	if(InLocalVarScope)
	{
		if(FBPVariableDescription* LocalVariable = FindLocalVariable(Blueprint, InLocalVarScope, VarName))
		{
			LocalVariable->SetMetaData(MetaDataKey, *MetaDataValue);
		}
	}
	else
	{
		const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(Blueprint, VarName);
		if (VarIndex == INDEX_NONE)
		{
			//Not a NewVariable is the VarName from a Timeline?
			const int32 TimelineIndex = FBlueprintEditorUtils::FindTimelineIndex(Blueprint,VarName);

			if (TimelineIndex == INDEX_NONE)
			{
				//Not a Timeline is this a SCS Node?
				if (USCS_Node* SCSNode = FBlueprintEditorUtils::FindSCS_Node(Blueprint,VarName))
				{
					SCSNode->SetMetaData(MetaDataKey, MetaDataValue);
				}
			}
			else
			{
				Blueprint->Timelines[TimelineIndex]->SetMetaData(MetaDataKey, MetaDataValue);
			}
		}
		else
		{
			Blueprint->NewVariables[VarIndex].SetMetaData(MetaDataKey, MetaDataValue);
			FProperty* Property = FindFProperty<FProperty>(Blueprint->SkeletonGeneratedClass, VarName);
			if (Property)
			{
				Property->SetMetaData(MetaDataKey, *MetaDataValue);
			}
			Property = FindFProperty<FProperty>(Blueprint->GeneratedClass, VarName);
			if (Property)
			{
				Property->SetMetaData(MetaDataKey, *MetaDataValue);
			}
		}
	}

	FMetaDataDependencyHelper::OnChange(Blueprint, MetaDataKey);

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
}

bool FBlueprintEditorUtils::GetBlueprintVariableMetaData(const UBlueprint* Blueprint, const FName& VarName, const UStruct* InLocalVarScope, const FName& MetaDataKey, FString& OutMetaDataValue)
{
	// If there is a local var scope, we know we are looking at a local variable
	if(InLocalVarScope)
	{
		if(FBPVariableDescription* LocalVariable = FindLocalVariable(Blueprint, InLocalVarScope, VarName))
		{
			int32 EntryIndex = LocalVariable->FindMetaDataEntryIndexForKey(MetaDataKey);
			if (EntryIndex != INDEX_NONE)
			{
				OutMetaDataValue = LocalVariable->GetMetaData(MetaDataKey);
				return true;
			}
		}
	}
	else
	{
		const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(Blueprint, VarName);
		if (VarIndex == INDEX_NONE)
		{
			//Not a NewVariable is the VarName from a Timeline?
			const int32 TimelineIndex = FBlueprintEditorUtils::FindTimelineIndex(Blueprint,VarName);

			if (TimelineIndex == INDEX_NONE)
			{
				//Not a Timeline is this a SCS Node?
				if (USCS_Node* Desc = FBlueprintEditorUtils::FindSCS_Node(Blueprint,VarName))
				{
					const int32 EntryIndex = Desc->FindMetaDataEntryIndexForKey(MetaDataKey);
					if (EntryIndex != INDEX_NONE)
					{
						OutMetaDataValue = Desc->GetMetaData(MetaDataKey);
						return true;
					}
				}
			}
			else
			{
				UTimelineTemplate& Desc = *Blueprint->Timelines[TimelineIndex];

				int32 EntryIndex = Desc.FindMetaDataEntryIndexForKey(MetaDataKey);
				if (EntryIndex != INDEX_NONE)
				{
					OutMetaDataValue = Desc.GetMetaData(MetaDataKey);
					return true;
				}
			}
		}
		else
		{
			const FBPVariableDescription& Desc = Blueprint->NewVariables[VarIndex];

			int32 EntryIndex = Desc.FindMetaDataEntryIndexForKey(MetaDataKey);
			if (EntryIndex != INDEX_NONE)
			{
				OutMetaDataValue = Desc.GetMetaData(MetaDataKey);
				return true;
			}
		}
	}

	OutMetaDataValue.Empty();
	return false;
}

void FBlueprintEditorUtils::RemoveBlueprintVariableMetaData(UBlueprint* Blueprint, const FName& VarName, const UStruct* InLocalVarScope, const FName& MetaDataKey)
{
	// If there is a local var scope, we know we are looking at a local variable
	if(InLocalVarScope)
	{
		if(FBPVariableDescription* LocalVariable = FindLocalVariable(Blueprint, InLocalVarScope, VarName))
		{
			LocalVariable->RemoveMetaData(MetaDataKey);
		}
	}
	else
	{
		const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(Blueprint, VarName);
		if (VarIndex == INDEX_NONE)
		{
			//Not a NewVariable is the VarName from a Timeline?
			const int32 TimelineIndex = FBlueprintEditorUtils::FindTimelineIndex(Blueprint,VarName);

			if (TimelineIndex == INDEX_NONE)
			{
				//Not a Timeline is this a SCS Node?
				if (USCS_Node* SCSNode = FBlueprintEditorUtils::FindSCS_Node(Blueprint, VarName))
				{
					SCSNode->RemoveMetaData(MetaDataKey);
				}

			}
			else
			{
				Blueprint->Timelines[TimelineIndex]->RemoveMetaData(MetaDataKey);
			}
		}
		else
		{
			Blueprint->NewVariables[VarIndex].RemoveMetaData(MetaDataKey);
			FProperty* Property = FindFProperty<FProperty>(Blueprint->SkeletonGeneratedClass, VarName);
			if (Property)
			{
				Property->RemoveMetaData(MetaDataKey);
			}
			Property = FindFProperty<FProperty>(Blueprint->GeneratedClass, VarName);
			if (Property)
			{
				Property->RemoveMetaData(MetaDataKey);
			}
		}
	}

	FMetaDataDependencyHelper::OnChange(Blueprint, MetaDataKey);

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
}

void FBlueprintEditorUtils::SetBlueprintVariableCategory(UBlueprint* Blueprint, const FName& VarName, const UStruct* InLocalVarScope, const FText& NewCategory, bool bDontRecompile)
{
	if (Blueprint == nullptr)
	{
		return;
	}

	// Ensure we always set a category
	FText SetCategory = NewCategory;
	if (SetCategory.IsEmpty())
	{
		SetCategory = UEdGraphSchema_K2::VR_DefaultCategory;
	}
	
	const FText OldCategory = GetBlueprintVariableCategory(Blueprint, VarName, InLocalVarScope);
	if (OldCategory.EqualTo(SetCategory))
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("ChangeVariableCategory", "Change Variable Category"));
	Blueprint->Modify();

	UClass* SkeletonGeneratedClass = Blueprint->SkeletonGeneratedClass;
	if (FProperty* TargetProperty = FindFProperty<FProperty>(SkeletonGeneratedClass, VarName))
	{
		UClass* OuterClass = TargetProperty->GetOwnerChecked<UClass>();
		const bool bIsNativeVar = (OuterClass->ClassGeneratedBy == nullptr);

		if (!bIsNativeVar)
		{
			if (UTimelineTemplate* Timeline = Blueprint->FindTimelineTemplateByVariableName(TargetProperty->GetFName()))
			{
				Timeline->SetMetaData(TEXT("Category"), SetCategory.ToString());
			}
			else if (UBaseWidgetBlueprint* WidgetBP = Cast<UBaseWidgetBlueprint>(Blueprint))
			{
				WidgetBP->ForEachSourceWidget([&](UWidget* InWidget) {
					if (InWidget->GetFName() == VarName)
					{
						InWidget->SetCategoryName(SetCategory.ToString());
					}
				});
			}
			else
			{
				TargetProperty->SetMetaData(TEXT("Category"), *SetCategory.ToString());
			}
			const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(Blueprint, VarName);
			if (VarIndex != INDEX_NONE)
			{
				Blueprint->NewVariables[VarIndex].Category = MoveTemp(SetCategory);
			}
			else
			{
				
				if (USCS_Node* SCSNode = FBlueprintEditorUtils::FindSCS_Node(Blueprint, VarName))
				{
					SCSNode->Modify();
					SCSNode->CategoryName = MoveTemp(SetCategory);
				}
			}
		}
	}
	else if (InLocalVarScope)
	{
		UK2Node_FunctionEntry* OutFunctionEntryNode;
		if (FBPVariableDescription* LocalVariable = FindLocalVariable(Blueprint, InLocalVarScope, VarName, &OutFunctionEntryNode))
		{
			OutFunctionEntryNode->Modify();
			LocalVariable->SetMetaData(TEXT("Category"), *SetCategory.ToString());
			LocalVariable->Category = MoveTemp(SetCategory);
		}
	}

	if (bDontRecompile == false)
	{
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
	}
}

void FBlueprintEditorUtils::SetBlueprintFunctionOrMacroCategory(UEdGraph* Graph, const FText& InCategoryName, bool bDontRecompile)
{
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(Graph);
	if (FKismetUserDeclaredFunctionMetadata* MetaData = FBlueprintEditorUtils::GetGraphFunctionMetaData(Graph))
	{
		const FText& NewCategory = InCategoryName.IsEmpty() ? UEdGraphSchema_K2::VR_DefaultCategory : InCategoryName;
		if (!MetaData->Category.EqualTo(NewCategory))
		{
			FScopedTransaction Transaction(LOCTEXT("SetBlueprintFunctionOrMacroCategory", "Set Category"));

			FBlueprintEditorUtils::ModifyFunctionMetaData(Graph);

			UFunction* Function = nullptr;
			for (TFieldIterator<UFunction> FunctionIt(Blueprint->SkeletonGeneratedClass, EFieldIteratorFlags::IncludeSuper); FunctionIt; ++FunctionIt)
			{
				if (FunctionIt->GetName() == Graph->GetName())
				{
					Function = *FunctionIt;
					break;
				}
			}

			MetaData->Category = NewCategory;

			if (Function)
			{
				check(!Function->IsNative()); // Should never get here with a native function, as we wouldn't have been able to find metadata for it
				Function->Modify();
				Function->SetMetaData(FBlueprintMetadata::MD_FunctionCategory, *NewCategory.ToString());
			}

			if (!bDontRecompile)
			{
				FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
			}

			FBlueprintActionDatabase::Get().RefreshAssetActions(Blueprint);
		}
	}
}

UAnimGraphNode_Root* FBlueprintEditorUtils::GetAnimGraphRoot(UEdGraph* InGraph)
{
	if(InGraph->GetSchema()->IsA(UAnimationGraphSchema::StaticClass()))
	{
		TArray<UAnimGraphNode_Root*> Roots;
		InGraph->GetNodesOfClass<UAnimGraphNode_Root>(Roots);
		check(Roots.Num() == 1);

		return Roots[0];
	}
	return nullptr;
}

void FBlueprintEditorUtils::SetAnimationGraphLayerGroup(UEdGraph* InGraph, const FText& InGroupName)
{
	if(InGraph->GetSchema()->IsA(UAnimationGraphSchema::StaticClass()))
	{
		const FName NewGroup = InGroupName.IsEmpty() ? NAME_None : FName(*InGroupName.ToString());
		UAnimGraphNode_Root* Root = GetAnimGraphRoot(InGraph);
		if(NewGroup != Root->Node.GetGroup())
		{
			FScopedTransaction Transaction(LOCTEXT("SetAnimationGraphLayerGroup", "Set Group"));

			UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(InGraph);
		
			Root->Modify();

			UFunction* Function = nullptr;
			for (TFieldIterator<UFunction> FunctionIt(Blueprint->SkeletonGeneratedClass, EFieldIteratorFlags::IncludeSuper); FunctionIt; ++FunctionIt)
			{
				if (FunctionIt->GetName() == InGraph->GetName())
				{
					Function = *FunctionIt;
					break;
				}
			}

		
			Root->Node.SetGroup(NewGroup);

			if (Function)
			{
				Function->Modify();
				Function->SetMetaData(FBlueprintMetadata::MD_FunctionCategory, NewGroup == NAME_None ? TEXT("") : *NewGroup.ToString());
			}

			FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
		}
	}
}

int32 FBlueprintEditorUtils::FindIndexOfGraphInParent(UEdGraph* Graph)
{
	int32 Result = INDEX_NONE;

	if (UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(Graph))
	{
		Result = Blueprint->FunctionGraphs.IndexOfByKey(Graph);
		if (Result == INDEX_NONE)
		{
			Result = Blueprint->MacroGraphs.IndexOfByKey(Graph);
		}
	}

	return Result;
}

bool FBlueprintEditorUtils::MoveGraphBeforeOtherGraph(UEdGraph* Graph, int32 NewIndex, bool bDontRecompile)
{
	if (UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(Graph))
	{
		bool bModified = false;

		const int32 OldFunctionIndex = Blueprint->FunctionGraphs.IndexOfByKey(Graph);
		if (OldFunctionIndex != INDEX_NONE)
		{
			if ((OldFunctionIndex != NewIndex) && Blueprint->FunctionGraphs.IsValidIndex(NewIndex))
			{
				Blueprint->Modify();
				Blueprint->FunctionGraphs.Insert(Graph, NewIndex);
				Blueprint->FunctionGraphs.RemoveAt((OldFunctionIndex < NewIndex) ? OldFunctionIndex : (OldFunctionIndex + 1));
				bModified = true;
			}
		}

		const int32 OldMacroIndex = Blueprint->MacroGraphs.IndexOfByKey(Graph);
		if (OldMacroIndex != INDEX_NONE)
		{
			if ((OldMacroIndex != NewIndex) && Blueprint->MacroGraphs.IsValidIndex(NewIndex))
			{
				Blueprint->Modify();
				Blueprint->MacroGraphs.Insert(Graph, NewIndex);
				Blueprint->MacroGraphs.RemoveAt((OldMacroIndex < NewIndex) ? OldMacroIndex : (OldMacroIndex + 1));
				bModified = true;
			}
		}

		if (bModified)
		{
			if (!bDontRecompile)
			{
				FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
			}
			else
			{
				FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
			}
		}

		return bModified;
	}

	return false;
}


FText FBlueprintEditorUtils::GetBlueprintVariableCategory(UBlueprint* Blueprint, const FName& VarName, const UStruct* InLocalVarScope)
{
	FText CategoryName;
	UClass* SkeletonGeneratedClass = Blueprint->SkeletonGeneratedClass;
	FProperty* TargetProperty = FindFProperty<FProperty>(SkeletonGeneratedClass, VarName);
	if(TargetProperty != nullptr)
	{
		CategoryName = FObjectEditorUtils::GetCategoryText(TargetProperty);
	}
	else if(InLocalVarScope)
	{
		// Check to see if it is a local variable
		if(FBPVariableDescription* LocalVariable = FindLocalVariable(Blueprint, InLocalVarScope, VarName))
		{
			CategoryName = LocalVariable->Category;
		}
	}

	if(CategoryName.IsEmpty() && Blueprint->SimpleConstructionScript != nullptr)
	{
		// Look for the variable in the SCS (in case the Blueprint has not been compiled yet)
		if (USCS_Node* SCSNode = FBlueprintEditorUtils::FindSCS_Node(Blueprint, VarName))
		{
			CategoryName = SCSNode->CategoryName;
		}
	}

	return CategoryName;
}

uint64* FBlueprintEditorUtils::GetBlueprintVariablePropertyFlags(UBlueprint* Blueprint, const FName& VarName)
{
	const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(Blueprint, VarName);
	if (VarIndex != INDEX_NONE)
	{
		return &Blueprint->NewVariables[VarIndex].PropertyFlags;
	}
	return nullptr;
}

FBPVariableDescription* FBlueprintEditorUtils::GetVariableFromOnRepFunction(UBlueprint* Blueprint, FName FuncName)
{
	const TCHAR* OnRepPrefix = TEXT("OnRep_");
	FString FuncNameStr = FuncName.ToString();

	if (FuncNameStr.StartsWith(OnRepPrefix))
	{
		FName VarName(FuncNameStr.RightChop(FCString::Strlen(OnRepPrefix)));
		const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(Blueprint, VarName);
		if (VarIndex != INDEX_NONE)
		{
			if (Blueprint->NewVariables[VarIndex].RepNotifyFunc == FuncName)
			{
				return &Blueprint->NewVariables[VarIndex];
			}
		}
	}

	return nullptr;
}

FName FBlueprintEditorUtils::GetBlueprintVariableRepNotifyFunc(UBlueprint* Blueprint, const FName& VarName)
{
	const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(Blueprint, VarName);
	if (VarIndex != INDEX_NONE)
	{
		return Blueprint->NewVariables[VarIndex].RepNotifyFunc;
	}
	return NAME_None;
}

void FBlueprintEditorUtils::SetBlueprintVariableRepNotifyFunc(UBlueprint* Blueprint, const FName& VarName, const FName& RepNotifyFunc)
{
	const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(Blueprint, VarName);
	if (VarIndex != INDEX_NONE)
	{
		Blueprint->NewVariables[VarIndex].RepNotifyFunc = RepNotifyFunc;
	}
}

void FBlueprintEditorUtils::GetFunctionNameList(const UBlueprint* Blueprint, TSet<FName>& FunctionNames)
{
	if( UClass* SkeletonClass = Blueprint->SkeletonGeneratedClass )
	{
		for( TFieldIterator<UFunction> FuncIt(SkeletonClass, EFieldIteratorFlags::IncludeSuper); FuncIt; ++FuncIt )
		{
			FunctionNames.Add( (*FuncIt)->GetFName() );
		}
	}
}

void FBlueprintEditorUtils::GetDelegateNameList(const UBlueprint* Blueprint, TSet<FName>& FunctionNames)
{
	check(Blueprint);
	for (int32 It = 0; It < Blueprint->DelegateSignatureGraphs.Num(); It++)
	{
		if (UEdGraph* Graph = Blueprint->DelegateSignatureGraphs[It])
		{
			FunctionNames.Add(Graph->GetFName());
		}
	}
}

UEdGraph* FBlueprintEditorUtils::GetDelegateSignatureGraphByName(UBlueprint* Blueprint, FName FunctionName)
{
	if ((nullptr != Blueprint) && (FunctionName != NAME_None))
	{
		for (int32 It = 0; It < Blueprint->DelegateSignatureGraphs.Num(); It++)
		{
			if (UEdGraph* Graph = Blueprint->DelegateSignatureGraphs[It])
			{
				if(FunctionName == Graph->GetFName())
				{
					return Graph;
				}
			}
		}
	}
	return nullptr;
}

// Gets a list of pins that should hidden for a given function
void FBlueprintEditorUtils::GetHiddenPinsForFunction(UEdGraph const* Graph, UFunction const* Function, TSet<FName>& HiddenPins, TSet<FName>* OutInternalPins)
{
	check(Function != nullptr);
	TMap<FName, FString>* MetaData = FMetaData::GetMapForObject(Function);	
	if (MetaData != nullptr)
	{
		for (TMap<FName, FString>::TConstIterator It(*MetaData); It; ++It)
		{
			const FName& Key = It.Key();

			if (Key == FBlueprintMetadata::MD_LatentInfo)
			{
				HiddenPins.Add(*It.Value());
			}
			else if (Key == FBlueprintMetadata::MD_HidePin)
			{
				TArray<FString> HiddenPinNames;
				It.Value().ParseIntoArray(HiddenPinNames, TEXT(","));
				for (FString& HiddenPinName : HiddenPinNames)
				{
					HiddenPinName.TrimStartAndEndInline();
					HiddenPins.Add(*HiddenPinName);
				}
			}
			else if (Key == FBlueprintMetadata::MD_ExpandEnumAsExecs ||
					Key == FBlueprintMetadata::MD_ExpandBoolAsExecs)
			{
				TArray<FName> EnumPinNames;
				UK2Node_CallFunction::GetExpandEnumPinNames(Function, EnumPinNames);
				
				for (const FName& EnumName : EnumPinNames)
				{
					HiddenPins.Add(EnumName);
				}
			}
			else if (Key == FBlueprintMetadata::MD_InternalUseParam)
			{
				TArray<FString> HiddenPinNames;
				It.Value().ParseIntoArray(HiddenPinNames, TEXT(","));
				for (FString& HiddenPinName : HiddenPinNames)
				{
					HiddenPinName.TrimStartAndEndInline();

					FName HiddenPinFName(*HiddenPinName);
					HiddenPins.Add(HiddenPinFName);

					if (OutInternalPins)
					{
						OutInternalPins->Add(HiddenPinFName);
					}
				}
			}
			else if (Key == FBlueprintMetadata::MD_WorldContext)
			{
				const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
				if(!K2Schema->IsStaticFunctionGraph(Graph))
				{
					// if the blueprint has world context that we can lookup with "self", 
					// then we can hide this pin (and default it to self)
					if (!ShouldShowWorldContextPin(FindBlueprintForGraph(Graph)))
					{
						HiddenPins.Add(*It.Value());
					}
				}
			}
		}
	}
}

bool FBlueprintEditorUtils::IsPinTypeValid(const FEdGraphPinType& Type)
{
	if (const UUserDefinedStruct* UDStruct = Cast<const UUserDefinedStruct>(Type.PinSubCategoryObject.Get()))
	{
		if (EUserDefinedStructureStatus::UDSS_UpToDate != UDStruct->Status)
		{
			return false;
		}
	}
	return true;
}

void FBlueprintEditorUtils::ValidatePinConnections(const UEdGraphNode* Node, FCompilerResultsLog& MessageLog)
{
	if (Node)
	{
		const UEdGraphSchema* Schema = Node->GetSchema();
		check(Schema);

		// Validate that all pins with links are actually set to valid connections.
		// This is necessary because the user could change the type of the pin 
		for (UEdGraphPin* Pin : Node->Pins)
		{
			if (Pin && !Pin->bOrphanedPin)
			{
				for (UEdGraphPin* Link : Pin->LinkedTo)
				{
					const FPinConnectionResponse ConnectionResponse = Schema->CanCreateConnection(Pin, Link);
					if (Link && !Link->bOrphanedPin && Link != Pin && ConnectionResponse.Response == CONNECT_RESPONSE_DISALLOW)
					{
						const FString ErrorMessage = FText::Format(LOCTEXT("BadConnection_ErrorFmt", "Invalid pin connection from @@ and @@: {0} (You may have changed the type after the connections were made)"), ConnectionResponse.Message).ToString();
						MessageLog.Error(*ErrorMessage, Pin, Link);
					}
				}
			}
		}
	}
}

void FBlueprintEditorUtils::ValidateEditorOnlyNodes(const UK2Node* Node, FCompilerResultsLog& MessageLog)
{
	if(!Node)
	{
		return;
	}

	const UBlueprint* BP = Node->GetBlueprint();
	const UClass* NodeClass = Node->GetClass();
	const UPackage* NodeCDOPackage = NodeClass->GetDefaultObject(false) ? NodeClass->GetDefaultObject(false)->GetOutermost() : nullptr;
	
	if(NodeCDOPackage && BP)
	{
		const bool bIsEditorOnlyPackage = NodeCDOPackage->HasAllPackagesFlags(PKG_EditorOnly);
		const bool bIsUncookedOrDev = NodeCDOPackage->HasAnyPackageFlags(PKG_UncookedOnly | PKG_Developer);		

		const UClass* BlueprintClass = BP ? *BP->ParentClass : nullptr;
		const bool bIsEditorOnlyBlueprintBaseClass = BlueprintClass ? IsEditorOnlyObject(BlueprintClass) : false;

		// Check whether the blueprint itself, or its class is marked as editor-only
		if (!bIsUncookedOrDev && bIsEditorOnlyPackage && !(IsEditorOnlyObject(BP) || bIsEditorOnlyBlueprintBaseClass))
		{
			MessageLog.Warning(*LOCTEXT("EditorOnlyConflict_ErrorFmt", "The node '@@' is from an Editor Only module, but is placed in a runtime blueprint! K2 Nodes should only be defined in a Developer or UncookedOnly module.").ToString(), Node);
		}
	}
}

void FBlueprintEditorUtils::GetClassVariableList(const UBlueprint* Blueprint, TSet<FName>& VisibleVariables, bool bIncludePrivateVars) 
{
	// Existing variables in the parent class and above, when using the compilation manager the previous SkeletonGeneratedClass will have been cleared when
	// we're regenerating the SkeletonGeneratedClass. Using this function in the skeleton pass at all is highly dubious, but I am leaving it until the 
	// compilation manager is on full time:
	if (Blueprint->SkeletonGeneratedClass != nullptr)
	{
		for (TFieldIterator<FProperty> PropertyIt(Blueprint->SkeletonGeneratedClass, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
		{
			FProperty* Property = *PropertyIt;

			if ((!Property->HasAnyPropertyFlags(CPF_Parm) && (bIncludePrivateVars || Property->HasAllPropertyFlags(CPF_BlueprintVisible))))
			{
				VisibleVariables.Add(Property->GetFName());
			}
		}

		if (bIncludePrivateVars)
		{
			// Include SCS node variable names, timelines, and other member variables that may be pending compilation. Consider them to be "private" as they're not technically accessible for editing just yet.
			TArray<UBlueprintGeneratedClass*> ParentBPStack;
			UBlueprint::GetBlueprintHierarchyFromClass(Blueprint->SkeletonGeneratedClass, ParentBPStack);
			for (int32 StackIndex = ParentBPStack.Num() - 1; StackIndex >= 0; --StackIndex)
			{
				UBlueprintGeneratedClass* ParentPBGC = ParentBPStack[StackIndex];
				check(ParentPBGC);
				UBlueprint* ParentBP = Cast<UBlueprint>(ParentPBGC->ClassGeneratedBy);

				GetSCSVariableNameList(ParentPBGC, VisibleVariables);

				if (ParentBP)
				{
					for (const FBPVariableDescription& Variable : ParentBP->NewVariables)
					{
						VisibleVariables.Add(Variable.VarName);
					}
				}

				// Since we defer copying the timeline templates to the BPGC until compile time, 
				// we consider the BP (when present) to be authoritative.
				const TArray<TObjectPtr<UTimelineTemplate>>& Timelines = ParentBP ? ParentBP->Timelines : ParentPBGC->Timelines;
				for (UTimelineTemplate* Timeline : Timelines)
				{
					if (Timeline)
					{
						VisibleVariables.Add(Timeline->GetFName());
					}
				}
			}
		}
	}

	// "self" is reserved for all classes
	VisibleVariables.Add(NAME_Self);
}

void FBlueprintEditorUtils::GetNewVariablesOfType( const UBlueprint* Blueprint, const FEdGraphPinType& Type, TArray<FName>& OutVars )
{
	for(int32 i=0; i<Blueprint->NewVariables.Num(); i++)
	{
		const FBPVariableDescription& Var = Blueprint->NewVariables[i];
		if(Type == Var.VarType)
		{
			OutVars.Add(Var.VarName);
		}
	}
}

void FBlueprintEditorUtils::GetLocalVariablesOfType( const UEdGraph* Graph, const FEdGraphPinType& Type, TArray<FName>& OutVars)
{
	if (DoesSupportLocalVariables(Graph))
	{
		// Grab the function graph, so we can find the function entry node for local variables
		UEdGraph* FunctionGraph = FBlueprintEditorUtils::GetTopLevelGraph(Graph);

		TArray<UK2Node_FunctionEntry*> GraphNodes;
		FunctionGraph->GetNodesOfClass<UK2Node_FunctionEntry>(GraphNodes);

		// There should only be one entry node
		check(GraphNodes.Num() == 1);

		for (const FBPVariableDescription& LocalVar : GraphNodes[0]->LocalVariables)
		{
			if (LocalVar.VarType == Type)
			{
				OutVars.Add(LocalVar.VarName);
			}
		}
	}
}

// Adds a member variable to the blueprint.  It cannot mask a variable in any superclass.
bool FBlueprintEditorUtils::AddMemberVariable(UBlueprint* Blueprint, const FName& NewVarName, const FEdGraphPinType& NewVarType, const FString& DefaultValue/* = FString()*/)
{
	// Don't allow vars with empty names
	if(NewVarName == NAME_None)
	{
		return false;
	}

	// First we need to see if there is already a variable with that name, in this blueprint or parent class
	TSet<FName> CurrentVars;
	FBlueprintEditorUtils::GetClassVariableList(Blueprint, CurrentVars);
	if(CurrentVars.Contains(NewVarName))
	{
		return false; // fail
	}

	Blueprint->Modify();

	// Now create new variable
	FBPVariableDescription NewVar;

	NewVar.VarName = NewVarName;
	NewVar.VarGuid = FGuid::NewGuid();
	NewVar.FriendlyName = FName::NameToDisplayString( NewVarName.ToString(), (NewVarType.PinCategory == UEdGraphSchema_K2::PC_Boolean) ? true : false );
	NewVar.VarType = NewVarType;
	// default new vars to 'kismet read/write' and 'only editable on owning CDO' 
	NewVar.PropertyFlags |= (CPF_Edit | CPF_BlueprintVisible | CPF_DisableEditOnInstance);
	if(NewVarType.PinCategory == UEdGraphSchema_K2::PC_MCDelegate)
	{
		NewVar.PropertyFlags |= CPF_BlueprintAssignable | CPF_BlueprintCallable;
	}
	else
	{
		PostSetupObjectPinType(Blueprint, NewVar);
	}
	NewVar.ReplicationCondition = COND_None;
	NewVar.Category = UEdGraphSchema_K2::VR_DefaultCategory;
	NewVar.DefaultValue = DefaultValue;

	// user created variables should be none of these things
	NewVar.VarType.bIsConst       = false;
	NewVar.VarType.bIsWeakPointer = false;
	NewVar.VarType.bIsReference   = false;

	// Text variables, etc. should default to multiline
	if (NewVarType.PinCategory == UEdGraphSchema_K2::PC_String || NewVarType.PinCategory == UEdGraphSchema_K2::PC_Text)
	{
		NewVar.SetMetaData(TEXT("MultiLine"), TEXT("true"));
	}

	Blueprint->NewVariables.Add(NewVar);

	// Potentially adjust variable names for any child blueprints
	FBlueprintEditorUtils::ValidateBlueprintChildVariables(Blueprint, NewVarName);

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);

	return true;
}

// Removes a member variable if it was declared in this blueprint and not in a base class.
void FBlueprintEditorUtils::RemoveMemberVariable(UBlueprint* Blueprint, const FName VarName)
{
	const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(Blueprint, VarName);
	if (VarIndex != INDEX_NONE)
	{
		if ( Blueprint->NewVariables[VarIndex].HasMetaData(FBlueprintMetadata::MD_FieldNotify) && FBlueprintEditorUtils::ImplementsInterface(Blueprint, true, UNotifyFieldValueChanged::StaticClass()) )
		{
			RemoveFieldNotifyFromAllMetadata(Blueprint, VarName);
		}
		Blueprint->NewVariables.RemoveAt(VarIndex);
		FBlueprintEditorUtils::RemoveVariableNodes(Blueprint, VarName);
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
	}
}

// Removes the field name from the field notify metadata of all functions and variables
void FBlueprintEditorUtils::RemoveFieldNotifyFromAllMetadata(UBlueprint* Blueprint, const FName FieldName)
{
	// Go through all field notify variables and remove FieldName from their metadata.
	for (int32 i = 0; i < Blueprint->NewVariables.Num(); ++i)
	{
		FBPVariableDescription& Variable = Blueprint->NewVariables[i];
		if (Variable.HasMetaData(FBlueprintMetadata::MD_FieldNotify))
		{
			FString FieldNotifyValues = Variable.GetMetaData(FBlueprintMetadata::MD_FieldNotify);
			TArray<FString> ListOfFieldNotifies;
			const TCHAR* Delimiter = TEXT("|");

			FieldNotifyValues.ParseIntoArray(ListOfFieldNotifies, Delimiter);
			if (ListOfFieldNotifies.Contains(FieldName.ToString()))
			{
				ListOfFieldNotifies.Remove(FieldName.ToString());
				if (ListOfFieldNotifies.Num() > 0)
				{
					FBlueprintEditorUtils::SetBlueprintVariableMetaData(Blueprint, Variable.VarName, NULL, FBlueprintMetadata::MD_FieldNotify, FString::Join(ListOfFieldNotifies, Delimiter));
				}
				else
				{
					FBlueprintEditorUtils::SetBlueprintVariableMetaData(Blueprint, Variable.VarName, NULL, FBlueprintMetadata::MD_FieldNotify, TEXT(""));
				}
			}
		}
	}
}

void FBlueprintEditorUtils::BulkRemoveMemberVariables(UBlueprint* Blueprint, const TArray<FName>& VarNames)
{
	const FScopedTransaction Transaction( LOCTEXT("BulkRemoveMemberVariables", "Bulk Remove Member Variables") );
	Blueprint->Modify();

	bool bModified = false;
	for (int32 i = 0; i < VarNames.Num(); ++i)
	{
		const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(Blueprint, VarNames[i]);
		if (VarIndex != INDEX_NONE)
		{
			Blueprint->NewVariables.RemoveAt(VarIndex);
			FBlueprintEditorUtils::RemoveVariableNodes(Blueprint, VarNames[i]);
			bModified = true;
		}
	}

	if (bModified)
	{
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
	}
}

void FBlueprintEditorUtils::GetUsedAndUnusedVariables(UBlueprint* Blueprint, TArray<FProperty*>& OutUsedVariables, TArray<FProperty*>& OutUnusedVariables)
{
	TArray<FName> VariableNames;
	for (TFieldIterator<FProperty> PropertyIt(Blueprint->SkeletonGeneratedClass, EFieldIteratorFlags::ExcludeSuper); PropertyIt; ++PropertyIt)
	{
		FProperty* Property = *PropertyIt;
		// Don't show delegate properties, there is special handling for these
		const bool bDelegateProp = Property->IsA(FDelegateProperty::StaticClass()) || Property->IsA(FMulticastDelegateProperty::StaticClass());
		const bool bShouldShowProp = (!Property->HasAnyPropertyFlags(CPF_Parm) && Property->HasAllPropertyFlags(CPF_BlueprintVisible) && !bDelegateProp);

		if (bShouldShowProp)
		{
			FName VarName = Property->GetFName();

			const int32 VarInfoIndex = FBlueprintEditorUtils::FindNewVariableIndex(Blueprint, VarName);
			const bool bHasVarInfo = (VarInfoIndex != INDEX_NONE);

			const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(Property);
			bool bIsTimeline = ObjectProperty &&
				ObjectProperty->PropertyClass &&
				ObjectProperty->PropertyClass->IsChildOf(UTimelineComponent::StaticClass());
			if (!bIsTimeline && bHasVarInfo && !FBlueprintEditorUtils::IsVariableUsed(Blueprint, VarName))
			{
				OutUnusedVariables.Add(Property);
			}
			else
			{
				OutUsedVariables.Add(Property);
			}
		}
	}
}

FGuid FBlueprintEditorUtils::FindMemberVariableGuidByName(UBlueprint* InBlueprint, const FName InVariableName)
{
	while(InBlueprint)
	{
		const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(InBlueprint, InVariableName);
		if (VarIndex != INDEX_NONE)
		{
			return InBlueprint->NewVariables[VarIndex].VarGuid;
		}
		InBlueprint = Cast<UBlueprint>(InBlueprint->ParentClass->ClassGeneratedBy);
	}
	return FGuid(); 
}

FName FBlueprintEditorUtils::FindMemberVariableNameByGuid(UBlueprint* InBlueprint, const FGuid& InVariableGuid)
{
	while(InBlueprint)
	{
		for(FBPVariableDescription& Variable : InBlueprint->NewVariables)
		{
			if(Variable.VarGuid == InVariableGuid)
			{
				return Variable.VarName;
			}
		}

		InBlueprint = Cast<UBlueprint>(InBlueprint->ParentClass->ClassGeneratedBy);
	}
	return NAME_None;
}

void FBlueprintEditorUtils::RemoveVariableNodes(UBlueprint* Blueprint, const FName VarName, bool const bForSelfOnly, UEdGraph* LocalGraphScope)
{
	TArray<UEdGraph*> AllGraphs;
	Blueprint->GetAllGraphs(AllGraphs);

	for(TArray<UEdGraph*>::TConstIterator it(AllGraphs); it; ++it)
	{
		const UEdGraph* CurrentGraph = *it;

		TArray<UK2Node_Variable*> GraphNodes;
		CurrentGraph->GetNodesOfClass(GraphNodes);

		for( TArray<UK2Node_Variable*>::TConstIterator NodeIt(GraphNodes); NodeIt; ++NodeIt )
		{
			UK2Node_Variable* CurrentNode = *NodeIt;

			UClass* SelfClass = Blueprint->GeneratedClass;
			UClass* VariableParent = CurrentNode->VariableReference.GetMemberParentClass(SelfClass);

			if ((SelfClass == VariableParent) || !bForSelfOnly)
			{
				if(LocalGraphScope == CurrentNode->GetGraph() || LocalGraphScope == nullptr)
				{
					if(VarName == CurrentNode->GetVarName())
					{
						CurrentNode->DestroyNode();
					}
				}
			}
		}
	}
}

void FBlueprintEditorUtils::RenameComponentMemberVariable(UBlueprint* Blueprint, USCS_Node* Node, const FName NewName)
{
	// Should not allow renaming to "none" (UI should prevent this)
	check(!NewName.IsNone());

	if (!NewName.IsEqual(Node->GetVariableName(), ENameCase::CaseSensitive))
	{
		Blueprint->Modify();

		// Validate child blueprints and adjust variable names to avoid a potential name collision
		FBlueprintEditorUtils::ValidateBlueprintChildVariables(Blueprint, NewName);

		// Update the name
		const FName OldName = Node->GetVariableName();
		Node->Modify();
		Node->SetVariableName(NewName);

		// Rename Inheritable Component Templates
		{
			const FComponentKey Key(Node);
			TArray<UBlueprint*> Dependents;
			FindDependentBlueprints(Blueprint, Dependents);
			for (UBlueprint* DepBP : Dependents)
			{
				UInheritableComponentHandler* InheritableComponentHandler = DepBP ? DepBP->GetInheritableComponentHandler(false) : nullptr;
				if (InheritableComponentHandler && InheritableComponentHandler->GetOverridenComponentTemplate(Key))
				{
					InheritableComponentHandler->Modify();
					InheritableComponentHandler->RefreshTemplateName(Key);
					InheritableComponentHandler->MarkPackageDirty();
				}
			}
		}

		Node->NameWasModified();

		// Update any existing references to the old name
		if (OldName != NAME_None)
		{
			FBlueprintEditorUtils::ReplaceVariableReferences(Blueprint, OldName, NewName);
		}

		// And recompile
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
	}
}

void FBlueprintEditorUtils::RenameMemberVariable(UBlueprint* Blueprint, const FName OldName, const FName NewName)
{
	if (!NewName.IsNone() && !NewName.IsEqual(OldName, ENameCase::CaseSensitive))
	{
		const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(Blueprint, OldName);
		if (VarIndex != INDEX_NONE)
		{
			const FScopedTransaction Transaction( LOCTEXT("RenameVariable", "Rename Variable") );
			Blueprint->Modify();

			// Update the name
			FBPVariableDescription& Variable = Blueprint->NewVariables[VarIndex];
			Variable.VarName = NewName;

			// If the variable has an associated OnRep function, warn the user and break the association if the name is changed
			FName OnRepFuncName = Blueprint->NewVariables[VarIndex].RepNotifyFunc;
			if (OnRepFuncName != NAME_None)
			{
				if (!VerifyUserWantsRepNotifyVariableNameChanged(OldName, OnRepFuncName))
				{
					// Showing the warning dialog causes the variable name text box to lose focus, which can result in this function being called again.
					// The VarName is set before verifying to skip over the second call, preventing the dialog from appearing twice. 
					Variable.VarName = OldName;
					return;
				}
				Blueprint->NewVariables[VarIndex].RepNotifyFunc = NAME_None;
			}

			Variable.FriendlyName = FName::NameToDisplayString( NewName.ToString(), (Variable.VarType.PinCategory == UEdGraphSchema_K2::PC_Boolean) ? true : false );

			// Update any existing references to the old name
			FBlueprintEditorUtils::ReplaceVariableReferences(Blueprint, OldName, NewName);

			{
				//Grab property of blueprint's current CDO
				UClass* GeneratedClass = Blueprint->GeneratedClass;
				UObject* GeneratedCDO = GeneratedClass ? GeneratedClass->GetDefaultObject(false) : nullptr;
				if (GeneratedCDO)
				{
					FProperty* TargetProperty = FindFProperty<FProperty>(GeneratedCDO->GetClass(), OldName); // GeneratedCDO->GetClass() is used instead of GeneratedClass, because CDO could use REINST class.
					// Grab the address of where the property is actually stored (UObject* base, plus the offset defined in the property)
					void* OldPropertyAddr = TargetProperty ? TargetProperty->ContainerPtrToValuePtr<void>(GeneratedCDO) : nullptr;
					if (OldPropertyAddr)
					{
						// if there is a property for variable, it means the original default value was already copied, so it can be safely overridden
						Variable.DefaultValue.Empty();
						PropertyValueToString(TargetProperty, reinterpret_cast<const uint8*>(GeneratedCDO), Variable.DefaultValue);
					}
				}
				else
				{
					UE_LOGF(LogBlueprint, Warning, "Could not find default value of renamed variable '%ls' (previously '%ls') in %ls", *NewName.ToString(), *OldName.ToString(), *GetPathNameSafe(Blueprint));
				}

				// Validate child blueprints and adjust variable names to avoid a potential name collision
				FBlueprintEditorUtils::ValidateBlueprintChildVariables(Blueprint, NewName);

				// And recompile
				FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
			}

			{
				const bool bIsDelegateVar = (Variable.VarType.PinCategory == UEdGraphSchema_K2::PC_MCDelegate);
				if (UEdGraph* DelegateSignatureGraph = bIsDelegateVar ? GetDelegateSignatureGraphByName(Blueprint, OldName) : nullptr)
				{
					FBlueprintEditorUtils::RenameGraph(DelegateSignatureGraph, NewName.ToString());

					// this code should not be necessary, because the GUID remains valid, but let it be for backward compatibility.
					TArray<UK2Node_BaseMCDelegate*> NodeUsingDelegate;
					FBlueprintEditorUtils::GetAllNodesOfClass<UK2Node_BaseMCDelegate>(Blueprint, NodeUsingDelegate);
					for (UK2Node_BaseMCDelegate* FunctionNode : NodeUsingDelegate)
					{
						if (FunctionNode->DelegateReference.IsSelfContext() && (FunctionNode->DelegateReference.GetMemberName() == OldName))
						{
							FunctionNode->Modify();
							FunctionNode->DelegateReference.SetSelfMember(NewName);
						}
					}
				}
			}
		}
		else if (Blueprint && Blueprint->SimpleConstructionScript)
		{
			// Wasn't in the introduced variable list; try to find the associated SCS node
			//@TODO: The SCS-generated variables should be in the variable list and have a link back;
			// As it stands, you cannot do any metadata operations on a SCS variable, and you have to do icky code like the following
			TArray<USCS_Node*> Nodes = Blueprint->SimpleConstructionScript->GetAllNodes();
			for (TArray<USCS_Node*>::TConstIterator NodeIt(Nodes); NodeIt; ++NodeIt)
			{
				USCS_Node* CurrentNode = *NodeIt;
				if (CurrentNode->GetVariableName() == OldName)
				{
					RenameComponentMemberVariable(Blueprint, CurrentNode, NewName);
					break;
				}
			}
		}
	}
}

TArray<UK2Node*> FBlueprintEditorUtils::GetNodesForVariable(const FName& InVarName, const UBlueprint* InBlueprint, const UStruct* InScope/* = nullptr*/)
{
	TArray<UK2Node*> ReturnNodes;
	TArray<UK2Node*> Nodes;
	GetAllNodesOfClass<UK2Node>(InBlueprint, Nodes);

	bool bNodesPendingDeletion = false;
	for( TArray<UK2Node*>::TConstIterator NodeIt(Nodes); NodeIt; ++NodeIt )
	{
		UK2Node* CurrentNode = *NodeIt;
		if (CurrentNode->ReferencesVariable(InVarName, InScope))
		{
			ReturnNodes.Add(CurrentNode);
		}
	}
	return ReturnNodes;
}

bool FBlueprintEditorUtils::VerifyUserWantsVariableTypeChanged(const FName& InVarName)
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("VariableName"), FText::FromName(InVarName));

	FText ConfirmDelete = FText::Format(LOCTEXT( "ConfirmChangeVarType", "This could break connections, do you want to search all Variable '{VariableName}' instances, change its type, and recompile?"), Args );

	// Warn the user that this may result in data loss
	FSuppressableWarningDialog::FSetupInfo Info( ConfirmDelete, LOCTEXT("ChangeVariableType", "Change Variable Type"), "ChangeVariableType_Warning" );
	Info.ConfirmText = LOCTEXT( "ChangeVariableType_Yes", "Change Variable Type");
	Info.CancelText = LOCTEXT( "ChangeVariableType_No", "Do Nothing");	

	FSuppressableWarningDialog ChangeVariableType( Info );

	FSuppressableWarningDialog::EResult RetCode = ChangeVariableType.ShowModal();
	return RetCode == FSuppressableWarningDialog::Confirm || RetCode == FSuppressableWarningDialog::Suppressed;
}

bool FBlueprintEditorUtils::VerifyUserWantsToSearch(FText Reason, const FString& SearchString)
{
	const UBlueprintEditorSettings* LocalSettings = GetDefault<UBlueprintEditorSettings>();
	// if the user has requested search operations on variable type changes then trigger the 
	// search operation
	 if(LocalSettings->bSearchForReferencesWhenVariableTypeChanged)
	 {
		return true;
	 }

	// else just log the search string so the user can search themself if they care to:
	FText LocalizedMessage = FText::Format( 
		LOCTEXT( "SearchSkipped", "{0} - search skipped as requested by editor settings, consider searching \"{1}\""),
		Reason, FText::FromString(SearchString));
	UE_LOGF(LogBlueprint, Warning, "%ls", *LocalizedMessage.ToString());
	
	return false;
}

bool FBlueprintEditorUtils::VerifyUserWantsRepNotifyVariableNameChanged(const FName& InVarName, const FName& InFuncName)
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("VariableName"), FText::FromName(InVarName));
	Args.Add(TEXT("FuncName"), FText::FromName(InFuncName));

	FText ConfirmRename = FText::Format(LOCTEXT("ConfirmChangeRepNotifyVarName",
		"Variable '{VariableName}' is linked to the OnRep function '{FuncName}'. Renaming it will still allow this variable to be replicated, but the function will not be called. Do you wish to proceed?"), Args);

	FSuppressableWarningDialog::FSetupInfo Info(ConfirmRename, LOCTEXT("ChangeRepNotifyVariableName", "Change RepNotify Variable Name"), "ChangeRepNotifyVariableName_Warning");
	Info.ConfirmText = LOCTEXT("ChangeRepNotifyVariableName_Yes", "Yes");
	Info.CancelText = LOCTEXT("ChangeRepNotifyVariableName_No", "No");

	FSuppressableWarningDialog ChangeRepNotifyVariableName(Info);

	FSuppressableWarningDialog::EResult RetCode = ChangeRepNotifyVariableName.ShowModal();
	return RetCode == FSuppressableWarningDialog::Confirm || RetCode == FSuppressableWarningDialog::Suppressed;
}

void FBlueprintEditorUtils::GetLoadedChildBlueprints(UBlueprint* InBlueprint, TArray<UBlueprint*>& OutBlueprints)
{
	// Iterate over currently-loaded Blueprints and potentially adjust their variable names if they conflict with the parent
	for(TObjectIterator<UBlueprint> BlueprintIt; BlueprintIt; ++BlueprintIt)
	{
		UBlueprint* ChildBP = *BlueprintIt;
		if(ChildBP != nullptr && ChildBP->ParentClass != nullptr)
		{
			TArray<UBlueprint*> ParentBPArray;
			// Get the parent hierarchy
			UBlueprint::GetBlueprintHierarchyFromClass(ChildBP->ParentClass, ParentBPArray);

			// Also get any BP interfaces we use
			TArray<UClass*> ImplementedInterfaces;
			FindImplementedInterfaces(ChildBP, true, ImplementedInterfaces);
			for (UClass* ImplementedInterface : ImplementedInterfaces)
			{
				UBlueprint* BlueprintInterfaceClass = UBlueprint::GetBlueprintFromClass(ImplementedInterface);
				if(BlueprintInterfaceClass != nullptr)
				{
					ParentBPArray.Add(BlueprintInterfaceClass);
				}
			}

			if(ParentBPArray.Contains(InBlueprint))
			{
				OutBlueprints.Add(ChildBP);
			}
		}
	}
}

void FBlueprintEditorUtils::ChangeMemberVariableType(UBlueprint* Blueprint, const FName VariableName, const FEdGraphPinType& NewPinType)
{
	if (VariableName != NAME_None)
	{
		const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(Blueprint, VariableName);
		if (VarIndex != INDEX_NONE)
		{
			const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
			FBPVariableDescription& Variable = Blueprint->NewVariables[VarIndex];
			
			// Update the variable type only if it is different
			if (Variable.VarType != NewPinType)
			{
				TArray<UBlueprint*> ChildBPs;
				GetLoadedChildBlueprints(Blueprint, ChildBPs);

				TArray<UK2Node*> AllVariableNodes = GetNodesForVariable(VariableName, Blueprint);
				for(UBlueprint* ChildBP : ChildBPs)
				{
					TArray<UK2Node*> VariableNodes = GetNodesForVariable(VariableName, ChildBP);
					AllVariableNodes.Append(VariableNodes);
				}

				// TRUE if the user might be breaking variable connections
				bool bBreakingVariableConnections = false;

				// If there are variable nodes in place, warn the user of the consequences using a suppressible dialog
				if(AllVariableNodes.Num())
				{
					if(!VerifyUserWantsVariableTypeChanged(VariableName))
					{
						// User has decided to cancel changing the variable member type
						return;
					}
					bBreakingVariableConnections = true;
				}

				FText OperationIdentifier = LOCTEXT("ChangeVariableType", "Change Variable Type");
				const FScopedTransaction Transaction( OperationIdentifier );
				Blueprint->Modify();

				/** Only change the variable type if type selection is valid, some unloaded Blueprints will turn out to be bad */
				bool bChangeVariableType = true;

				if ((NewPinType.PinCategory == UEdGraphSchema_K2::PC_Object) || (NewPinType.PinCategory == UEdGraphSchema_K2::PC_Interface))
				{
					// if it's a PC_Object, then it should have an associated UClass object
					if(NewPinType.PinSubCategoryObject.IsValid())
					{
						const UClass* ClassObject = Cast<UClass>(NewPinType.PinSubCategoryObject.Get());
						check(ClassObject != nullptr);

						if (ClassObject->IsChildOf(AActor::StaticClass()))
						{
							// NOTE: Right now the code that stops hard AActor references from being set in unsafe places is tied to this flag
							Variable.PropertyFlags |= CPF_DisableEditOnTemplate;
						}
						else 
						{
							// clear the disable-default-value flag that might have been present (if this was an AActor variable before)
							Variable.PropertyFlags &= ~(CPF_DisableEditOnTemplate);
						}
					}
					else
					{
						bChangeVariableType = false;

						// Display a notification to inform the user that the variable type was invalid (likely due to corruption), it should no longer appear in the list.
						FNotificationInfo Info( LOCTEXT("InvalidUnloadedBP", "The selected type was invalid once loaded, it has been removed from the list!") );
						Info.ExpireDuration = 3.0f;
						Info.bUseLargeFont = false;
						TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
						if ( Notification.IsValid() )
						{
							Notification->SetCompletionState( SNotificationItem::CS_Fail );
						}
					}
				}
				else 
				{
					// clear the disable-default-value flag that might have been present (if this was an AActor variable before)
					Variable.PropertyFlags &= ~(CPF_DisableEditOnTemplate);
				}

				if(bChangeVariableType)
				{
					const bool bBecameBoolean = Variable.VarType.PinCategory != UEdGraphSchema_K2::PC_Boolean && NewPinType.PinCategory == UEdGraphSchema_K2::PC_Boolean;
					const bool bBecameNotBoolean = Variable.VarType.PinCategory == UEdGraphSchema_K2::PC_Boolean && NewPinType.PinCategory != UEdGraphSchema_K2::PC_Boolean;
					if (bBecameBoolean || bBecameNotBoolean)
					{
						Variable.FriendlyName = FName::NameToDisplayString(Variable.VarName.ToString(), bBecameBoolean);
					}

					Variable.VarType = NewPinType;

					if(Variable.VarType.IsSet() || Variable.VarType.IsMap())
					{
						// Make sure that the variable is no longer tagged for replication, and warn the user if the variable is no
						// longer going to be replicated:
						if(Variable.RepNotifyFunc != NAME_None || Variable.PropertyFlags & CPF_Net || Variable.PropertyFlags & CPF_RepNotify)
						{
							FNotificationInfo Warning( 
								FText::Format(
									LOCTEXT("InvalidReplicationSettings", "Maps and sets cannot be replicated - {0} has had its replication settings cleared"),
									FText::FromName(Variable.VarName) 
								) 
							);
							Warning.ExpireDuration = 5.0f;
							Warning.bFireAndForget = true;
							Warning.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
							FSlateNotificationManager::Get().AddNotification(Warning);

							Variable.PropertyFlags &= ~CPF_Net;
							Variable.PropertyFlags &= ~CPF_RepNotify;
							Variable.RepNotifyFunc = NAME_None;
							Variable.ReplicationCondition = COND_None;
						}
					}

					UClass* ParentClass = nullptr;
					FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);

					if(bBreakingVariableConnections)
					{
						for(UBlueprint* ChildBP : ChildBPs)
						{
							// Mark the Blueprint as structurally modified so we can reconstruct the node successfully
							FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(ChildBP);
						}

						// Reconstruct all variable nodes that reference the changing variable
						for(UK2Node* VariableNode : AllVariableNodes)
						{
							K2Schema->ReconstructNode(*VariableNode, true);
						}

						TSharedPtr<IToolkit> FoundAssetEditor = FToolkitManager::Get().FindEditorForAsset(Blueprint);
						if (FoundAssetEditor.IsValid())
						{
							TSharedRef<IBlueprintEditor> BlueprintEditor = StaticCastSharedRef<IBlueprintEditor>(FoundAssetEditor.ToSharedRef());

							UK2Node* FirstVariableNode = nullptr;
							for (UK2Node* VariableNode : AllVariableNodes)
							{
								if (VariableNode->IsA<UK2Node_Variable>())
								{
									FirstVariableNode = VariableNode;
									break;
								}
							}
							
							if (FirstVariableNode)
							{
								constexpr EGetFindReferenceSearchStringFlags Flags = EGetFindReferenceSearchStringFlags::UseSearchSyntax;
								const FString SearchString = FirstVariableNode->GetFindReferenceSearchString(Flags);
								// At scale indexing is slow, allow the user to disable it:
								if(VerifyUserWantsToSearch(OperationIdentifier, SearchString))
								{
									constexpr bool bSetFindWithinBlueprint = false;
									constexpr bool bSelectFirstResult = false;
									BlueprintEditor->SummonSearchUI(bSetFindWithinBlueprint, SearchString, bSelectFirstResult);
								}
							}
						}
					}
				}
			}
		}
	}
}

FName FBlueprintEditorUtils::DuplicateMemberVariable(UBlueprint* InFromBlueprint, UBlueprint* InToBlueprint, FName InVariableToDuplicate)
{
	FName DuplicatedVariableName;

	if (InVariableToDuplicate != NAME_None)
	{
		const FScopedTransaction Transaction(LOCTEXT("DuplicateVariable", "Duplicate Variable"));
		InToBlueprint->Modify();

		FBPVariableDescription NewVar;

		UBlueprint* SourceBlueprint;
		const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndexAndBlueprint(InFromBlueprint, InVariableToDuplicate, SourceBlueprint);
		if (VarIndex != INDEX_NONE)
		{
			FBPVariableDescription& Variable = SourceBlueprint->NewVariables[VarIndex];

			NewVar = DuplicateVariableDescription(SourceBlueprint, Variable);

			// We need to manually pull the DefaultValue from the FProperty to set it
			void* OldPropertyAddr = nullptr;

			//Grab property of blueprint's current CDO
			UClass* GeneratedClass = SourceBlueprint->GeneratedClass;
			UObject* GeneratedCDO = GeneratedClass->GetDefaultObject();
			FProperty* TargetProperty = FindFProperty<FProperty>(GeneratedClass, Variable.VarName);

			if (TargetProperty)
			{
				// Grab the address of where the property is actually stored (UObject* base, plus the offset defined in the property)
				OldPropertyAddr = TargetProperty->ContainerPtrToValuePtr<void>(GeneratedCDO);
				if (OldPropertyAddr)
				{
					// if there is a property for variable, it means the original default value was already copied, so it can be safely overridden
					NewVar.DefaultValue.Empty();
					TargetProperty->ExportTextItem_Direct(NewVar.DefaultValue, OldPropertyAddr, OldPropertyAddr, nullptr, PPF_SerializedAsImportText);
				}
			}

			// Add the new variable
			InToBlueprint->NewVariables.Add(NewVar);
		}

		if (NewVar.VarGuid.IsValid())
		{
			DuplicatedVariableName = NewVar.VarName;

			// Potentially adjust variable names for any child blueprints
			FBlueprintEditorUtils::ValidateBlueprintChildVariables(InToBlueprint, NewVar.VarName);

			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(InToBlueprint);
		}
	}

	return DuplicatedVariableName;
}

FName FBlueprintEditorUtils::DuplicateVariable(UBlueprint* InBlueprint, const UStruct* InScope, FName InVariableToDuplicate)
{
	FName DuplicatedVariableName;

	if (InVariableToDuplicate != NAME_None)
	{
		const FScopedTransaction Transaction(LOCTEXT("DuplicateVariable", "Duplicate Variable"));
		InBlueprint->Modify();

		DuplicatedVariableName = FBlueprintEditorUtils::DuplicateMemberVariable(InBlueprint, InBlueprint, InVariableToDuplicate);
		
		if (DuplicatedVariableName == NAME_None && InScope)
		{
			// It's probably a local variable

			UK2Node_FunctionEntry* FunctionEntry = nullptr;
			FBPVariableDescription* LocalVariable = FBlueprintEditorUtils::FindLocalVariable(InBlueprint, InScope, InVariableToDuplicate, &FunctionEntry);

			FBPVariableDescription NewVar;
			if (LocalVariable)
			{
				FunctionEntry->Modify();

				NewVar = DuplicateVariableDescription(InBlueprint, *LocalVariable);

				// Add the new variable
				FunctionEntry->LocalVariables.Add(NewVar);
			}

			if (NewVar.VarGuid.IsValid())
			{
				DuplicatedVariableName = NewVar.VarName;

				// Potentially adjust variable names for any child blueprints
				FBlueprintEditorUtils::ValidateBlueprintChildVariables(InBlueprint, NewVar.VarName);

				FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(InBlueprint);
			}
		}

	}

	return DuplicatedVariableName;
}

FBPVariableDescription FBlueprintEditorUtils::DuplicateVariableDescription(UBlueprint* InBlueprint, FBPVariableDescription& InVariableDescription)
{
	FName DuplicatedVariableName = InVariableDescription.VarName;

	if (FKismetNameValidator(InBlueprint).IsValid(DuplicatedVariableName) != EValidatorResult::Ok)
	{
		DuplicatedVariableName = FindUniqueKismetName(InBlueprint, InVariableDescription.VarName.GetPlainNameString());
	}

	// Now create new variable
	FBPVariableDescription NewVar = InVariableDescription;
	NewVar.VarName = DuplicatedVariableName;
	NewVar.FriendlyName = FName::NameToDisplayString( NewVar.VarName.ToString(), NewVar.VarType.PinCategory == UEdGraphSchema_K2::PC_Boolean);
	NewVar.VarGuid = FGuid::NewGuid();

	return NewVar;
}

bool FBlueprintEditorUtils::AddLocalVariable(UBlueprint* Blueprint, UEdGraph* InTargetGraph, const FName InNewVarName, const FEdGraphPinType& InNewVarType, const FString& DefaultValue/*= FString()*/)
{
	if(InTargetGraph != nullptr && InTargetGraph->GetSchema()->GetGraphType(InTargetGraph) == GT_Function)
	{
		// Ensure we have the top level graph for the function, in-case we are in a child graph
		UEdGraph* TargetGraph = FBlueprintEditorUtils::GetTopLevelGraph(InTargetGraph);

		const FScopedTransaction Transaction( LOCTEXT("AddLocalVariable", "Add Local Variable") );
		Blueprint->Modify();

		TArray<UK2Node_FunctionEntry*> FunctionEntryNodes;
		TargetGraph->GetNodesOfClass(FunctionEntryNodes);
		check(FunctionEntryNodes.Num());

		// Now create new variable
		FBPVariableDescription NewVar;

		NewVar.VarName = InNewVarName;
		NewVar.VarGuid = FGuid::NewGuid();
		NewVar.VarType = InNewVarType;
		NewVar.PropertyFlags |= CPF_BlueprintVisible;
		NewVar.FriendlyName = FName::NameToDisplayString( NewVar.VarName.ToString(), (NewVar.VarType.PinCategory == UEdGraphSchema_K2::PC_Boolean) ? true : false );
		NewVar.Category = UEdGraphSchema_K2::VR_DefaultCategory;
		NewVar.DefaultValue = DefaultValue;

		PostSetupObjectPinType(Blueprint, NewVar);

		FunctionEntryNodes[0]->Modify();
		FunctionEntryNodes[0]->LocalVariables.Add(NewVar);

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);

		return true;
	}
	return false;
}

void FBlueprintEditorUtils::RemoveLocalVariable(UBlueprint* InBlueprint, const UStruct* InScope, const FName InVarName)
{
	UEdGraph* ScopeGraph = FindScopeGraph(InBlueprint, InScope);

	if(ScopeGraph)
	{
		TArray<UK2Node_FunctionEntry*> GraphNodes;
		ScopeGraph->GetNodesOfClass<UK2Node_FunctionEntry>(GraphNodes);

		bool bFoundLocalVariable = false;

		// There is only ever 1 function entry
		check(GraphNodes.Num() == 1)
		for( int32 VarIdx = 0; VarIdx < GraphNodes[0]->LocalVariables.Num(); ++VarIdx )
		{
			if(GraphNodes[0]->LocalVariables[VarIdx].VarName == InVarName)
			{
				GraphNodes[0]->LocalVariables.RemoveAt(VarIdx);
				FBlueprintEditorUtils::RemoveVariableNodes(InBlueprint, InVarName, true, ScopeGraph);
				FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(InBlueprint);

				bFoundLocalVariable = true;

				// No other local variables will match, we are done
				break;
			}
		}

		// Check if we found the local variable, it is a problem if we do not.
		if(!bFoundLocalVariable)
		{
			UE_LOGF(LogBlueprint, Warning, "Could not find local variable '%ls'!", *InVarName.ToString());
		}
	}
}

FFunctionFromNodeHelper::FFunctionFromNodeHelper(const UObject* Obj) : Function(FunctionFromNode(Cast<UK2Node>(Obj))), Node(Cast<UK2Node>(Obj))
{

}

UFunction* FFunctionFromNodeHelper::FunctionFromNode(const UK2Node* Node)
{
	UFunction* Function = nullptr;
	UBlueprint* Blueprint = Node ? Node->GetBlueprint() : nullptr;
	const UClass* SearchScope = Blueprint ? *Blueprint->SkeletonGeneratedClass : nullptr;
	if (SearchScope)
	{
		if (const UK2Node_FunctionResult* ResultNode = Cast<UK2Node_FunctionResult>(Node))
		{
			// Function result nodes cannot resolve the UFunction, so find the entry node and use that for finding the UFunction
			TArray<UK2Node_FunctionEntry*> EntryNodes;
			ResultNode->GetGraph()->GetNodesOfClass(EntryNodes);

			check(EntryNodes.Num() == 1);
			Node = EntryNodes[0];
		}
		
		if (const UK2Node_FunctionEntry* FunctionNode = Cast<UK2Node_FunctionEntry>(Node))
		{
			const FName FunctionName = (FunctionNode->CustomGeneratedFunctionName != NAME_None) ? FunctionNode->CustomGeneratedFunctionName : FunctionNode->GetGraph()->GetFName();
			Function = SearchScope->FindFunctionByName(FunctionName);
		}
		else if (const UK2Node_Event* EventNode = Cast<UK2Node_Event>(Node))
		{
			// We need to search up the class hierarchy by name or functions like CanAddParentNode will fail:
			FName SearchName = EventNode->EventReference.GetMemberName();
			// If the function member name is none, then try the custom function name
			if (SearchName.IsNone())
			{
				SearchName = EventNode->CustomFunctionName;
			}
			Function = SearchScope->FindFunctionByName(SearchName);
		}
	}

	return Function;
}

UEdGraph* FBlueprintEditorUtils::FindScopeGraph(const UBlueprint* InBlueprint, const UStruct* InScope)
{
	UEdGraph* ScopeGraph = nullptr;

	TArray<UEdGraph*> AllGraphs;
	InBlueprint->GetAllGraphs(AllGraphs);

	for (UEdGraph* Graph : AllGraphs)
	{
		check(Graph != nullptr);
		if(Graph->GetFName() == InScope->GetFName())
		{
			// This graph should always be a function graph
			check(Graph->GetSchema()->GetGraphType(Graph) == GT_Function);
			
			ScopeGraph = Graph;
			break;
		}
	}

	if(!ScopeGraph)
	{
		FName UbergraphName = FBlueprintEditorUtils::GetUbergraphFunctionName(InBlueprint);
		
		bool bIsBlueprintEvent = false;
		if (const UFunction* AsFunction = Cast<UFunction>(InScope))
		{
			bIsBlueprintEvent = AsFunction->HasAnyFunctionFlags(FUNC_BlueprintEvent);
		}

		if(InScope->GetFName() == UbergraphName || bIsBlueprintEvent)
		{
			if(InBlueprint->UbergraphPages.Num() > 0)
			{
				ScopeGraph = InBlueprint->UbergraphPages[0];
			}
		}
	}

	return ScopeGraph;
}

void FBlueprintEditorUtils::RenameLocalVariable(UBlueprint* InBlueprint, const UStruct* InScope, const FName InOldName, const FName InNewName)
{
	if (!InNewName.IsNone() && !InNewName.IsEqual(InOldName, ENameCase::CaseSensitive))
	{
		UK2Node_FunctionEntry* FunctionEntry = nullptr;
		FBPVariableDescription* LocalVariable = FindLocalVariable(InBlueprint, InScope, InOldName, &FunctionEntry);
		const FProperty* OldProperty = FindFProperty<const FProperty>(InScope, InOldName);
		const FProperty* ExistingProperty = FindFProperty<const FProperty>(InScope, InNewName);
		const bool bHasExistingProperty = ExistingProperty && ExistingProperty != OldProperty;
		if (bHasExistingProperty)
		{
			UE_LOGF(LogBlueprint, Warning, "Cannot name local variable '%ls'. The name is already used.", *InNewName.ToString());
		}

		if (LocalVariable && !bHasExistingProperty)
		{
			const FScopedTransaction Transaction( LOCTEXT("RenameLocalVariable", "Rename Local Variable") );
			InBlueprint->Modify();
			FunctionEntry->Modify();

			// Update the name
			LocalVariable->VarName = InNewName;
			LocalVariable->FriendlyName = FName::NameToDisplayString( InNewName.ToString(), LocalVariable->VarType.PinCategory == UEdGraphSchema_K2::PC_Boolean );

			// Update any existing references to the old name
			if (RenameVariableReferencesInGraph(InBlueprint, InBlueprint->GeneratedClass, FindScopeGraph(InBlueprint, InScope), InOldName, InNewName))
			{
				MarkBlueprintAsModified(InBlueprint);
			}

			// Validate child blueprints and adjust variable names to avoid a potential name collision
			FBlueprintEditorUtils::ValidateBlueprintChildVariables(InBlueprint, InNewName);
			
			// And recompile
			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(InBlueprint);
		}
	}
}

FBPVariableDescription* FBlueprintEditorUtils::FindLocalVariable(const UBlueprint* InBlueprint, const UEdGraph* InScopeGraph, const FName InVariableName, class UK2Node_FunctionEntry** OutFunctionEntry)
{
	FBPVariableDescription* ReturnVariable = nullptr;
	if (DoesSupportLocalVariables(InScopeGraph))
	{
		UEdGraph* FunctionGraph = GetTopLevelGraph(InScopeGraph);
		TArray<UK2Node_FunctionEntry*> GraphNodes;
		FunctionGraph->GetNodesOfClass<UK2Node_FunctionEntry>(GraphNodes);

		bool bFoundLocalVariable = false;

		if (GraphNodes.Num() > 0)
		{
			// If there is an entry node, there should only be one
			check(GraphNodes.Num() == 1);

			for (int32 VarIdx = 0; VarIdx < GraphNodes[0]->LocalVariables.Num(); ++VarIdx)
			{
				if (GraphNodes[0]->LocalVariables[VarIdx].VarName == InVariableName)
				{
					if (OutFunctionEntry)
					{
						*OutFunctionEntry = GraphNodes[0];
					}
					ReturnVariable = &GraphNodes[0]->LocalVariables[VarIdx];
					break;
				}
			}
		}
	}

	return ReturnVariable;
}

FBPVariableDescription* FBlueprintEditorUtils::FindLocalVariable(const UBlueprint* InBlueprint, const UStruct* InScope, const FName InVariableName, class UK2Node_FunctionEntry** OutFunctionEntry)
{
	UEdGraph* ScopeGraph = FindScopeGraph(InBlueprint, InScope);

	return FindLocalVariable(InBlueprint, ScopeGraph, InVariableName, OutFunctionEntry);
}

FName FBlueprintEditorUtils::FindLocalVariableNameByGuid(const UBlueprint* InBlueprint, const FGuid& InVariableGuid)
{
	// Search through all function entry nodes for a local variable with the passed Guid
	TArray<UK2Node_FunctionEntry*> FunctionEntryNodes;
	GetAllNodesOfClass<UK2Node_FunctionEntry>(InBlueprint, FunctionEntryNodes);

	FName ReturnVariableName = NAME_None;
	for (UK2Node_FunctionEntry* const FunctionEntry : FunctionEntryNodes)
	{
		for( FBPVariableDescription& Variable : FunctionEntry->LocalVariables )
		{
			if(Variable.VarGuid == InVariableGuid)
			{
				ReturnVariableName = Variable.VarName;
				break;
			}
		}

		if(ReturnVariableName != NAME_None)
		{
			break;
		}
	}

	return ReturnVariableName;
}

FGuid FBlueprintEditorUtils::FindLocalVariableGuidByName(const UBlueprint* InBlueprint, const UStruct* InScope, const FName InVariableName)
{
	FGuid ReturnGuid;
	if(FBPVariableDescription* LocalVariable = FindLocalVariable(InBlueprint, InScope, InVariableName))
	{
		ReturnGuid = LocalVariable->VarGuid;
	}

	return ReturnGuid;
}

FGuid FBlueprintEditorUtils::FindLocalVariableGuidByName(const UBlueprint* InBlueprint, const UEdGraph* InScopeGraph, const FName InVariableName)
{
	FGuid ReturnGuid;
	if(FBPVariableDescription* LocalVariable = FindLocalVariable(InBlueprint, InScopeGraph, InVariableName))
	{
		ReturnGuid = LocalVariable->VarGuid;
	}

	return ReturnGuid;
}

void FBlueprintEditorUtils::ChangeLocalVariableType(UBlueprint* InBlueprint, const UStruct* InScope, const FName InVariableName, const FEdGraphPinType& NewPinType)
{
	if (InVariableName != NAME_None)
	{
		FString ActionCategory;
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

		UK2Node_FunctionEntry* FunctionEntry = nullptr;
		FBPVariableDescription* VariablePtr = FindLocalVariable(InBlueprint, InScope, InVariableName, &FunctionEntry);

		if(VariablePtr)
		{
			FBPVariableDescription& Variable = *VariablePtr;

			// Update the variable type only if it is different
			if (Variable.VarName == InVariableName && Variable.VarType != NewPinType)
			{
				TArray<UK2Node*> VariableNodes = GetNodesForVariable(InVariableName, InBlueprint, InScope);

				// If there are variable nodes in place, warn the user of the consequences using a suppressible dialog
				if(VariableNodes.Num())
				{
					if(!VerifyUserWantsVariableTypeChanged(InVariableName))
					{
						// User has decided to cancel changing the variable member type
						return;
					}
				}

				FText OperationIdentifier = LOCTEXT("ChangeLocalVariableType", "Change Local Variable Type");
				const FScopedTransaction Transaction( OperationIdentifier );
				InBlueprint->Modify();
				FunctionEntry->Modify();

				Variable.VarType = NewPinType;

				// Reset the default value
				Variable.DefaultValue.Empty();

				// Mark the Blueprint as structurally modified so we can reconstruct the node successfully
				FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(InBlueprint);

				if ((NewPinType.PinCategory == UEdGraphSchema_K2::PC_Object) || (NewPinType.PinCategory == UEdGraphSchema_K2::PC_Interface) || (NewPinType.PinCategory == UEdGraphSchema_K2::PC_SoftObject))
				{
					// if it's a PC_Object, then it should have an associated UClass object
					if(NewPinType.PinSubCategoryObject.IsValid())
					{
						const UClass* ClassObject = Cast<UClass>(NewPinType.PinSubCategoryObject.Get());
						check(ClassObject != nullptr);

						if (ClassObject->IsChildOf(AActor::StaticClass()))
						{
							// prevent Actor variables from having default values (because Blueprint templates are library elements that can 
							// bridge multiple levels and different levels might not have the actor that the default is referencing).
							Variable.PropertyFlags |= CPF_DisableEditOnTemplate;
						}
						else 
						{
							// clear the disable-default-value flag that might have been present (if this was an AActor variable before)
							Variable.PropertyFlags &= ~(CPF_DisableEditOnTemplate);
						}
					}
				}

				// Reconstruct all local variables referencing the modified one
				for(UK2Node* VariableNode : VariableNodes)
				{
					K2Schema->ReconstructNode(*VariableNode, true);
				}

				TSharedPtr<IToolkit> FoundAssetEditor = FToolkitManager::Get().FindEditorForAsset(InBlueprint);

				// No need to submit a search query if there are no nodes.
				if (FoundAssetEditor.IsValid() && VariableNodes.Num())
				{
					TSharedRef<IBlueprintEditor> BlueprintEditor = StaticCastSharedRef<IBlueprintEditor>(FoundAssetEditor.ToSharedRef());

					UK2Node* FirstVariableNode = nullptr;
					for (UK2Node* VariableNode : VariableNodes)
					{
						if (VariableNode->IsA<UK2Node_Variable>())
						{
							FirstVariableNode = VariableNode;
							break;
						}
					}

					// At scale indexing is slow, allow the user to disable it:
					if (FirstVariableNode)
					{
						constexpr EGetFindReferenceSearchStringFlags Flags = EGetFindReferenceSearchStringFlags::UseSearchSyntax;
						const FString SearchString = VariableNodes[0]->GetFindReferenceSearchString(Flags);
						// At scale indexing is slow, allow the user to disable it:
						if(VerifyUserWantsToSearch(OperationIdentifier, SearchString))
						{
							constexpr bool bSetFindWithinBlueprint = true;
							constexpr bool bSelectFirstResult = false;
							BlueprintEditor->SummonSearchUI(bSetFindWithinBlueprint, SearchString, bSelectFirstResult);
						}
					}
				}
			}
		}
	}
}

void FBlueprintEditorUtils::ReplaceVariableReferences(UBlueprint* Blueprint, const FName OldName, const FName NewName)
{
	check((OldName != NAME_None) && (NewName != NAME_None));

	FBlueprintEditorUtils::RenameVariableReferences(Blueprint, Blueprint->GeneratedClass, OldName, NewName);

	TArray<UBlueprint*> Dependents;
	FindDependentBlueprints(Blueprint, Dependents);

	for (UBlueprint* DependentBp : Dependents)
	{
		FBlueprintEditorUtils::RenameVariableReferences(DependentBp, Blueprint->GeneratedClass, OldName, NewName);
	}
}

void FBlueprintEditorUtils::ReplaceVariableReferences(UBlueprint* Blueprint, const FProperty* OldVariable, const FProperty* NewVariable)
{
	check((OldVariable != nullptr) && (NewVariable != nullptr));
	ReplaceVariableReferences(Blueprint, OldVariable->GetFName(), NewVariable->GetFName());
}

void FBlueprintEditorUtils::ReplaceFunctionReferences(UBlueprint* Blueprint, const FName OldName, const FName NewName)
{
	check((OldName != NAME_None) && (NewName != NAME_None));

	FBlueprintEditorUtils::RenameFunctionReferences(Blueprint, Blueprint->GeneratedClass, OldName, NewName);

	TArray<UBlueprint*> Dependents;
	FindDependentBlueprints(Blueprint, Dependents);

	for (UBlueprint* DependentBp : Dependents)
	{
		FBlueprintEditorUtils::RenameFunctionReferences(DependentBp, Blueprint->GeneratedClass, OldName, NewName);
	}
}

bool FBlueprintEditorUtils::IsVariableComponent(const FBPVariableDescription& Variable)
{
	// Find the variable in the list
	if( Variable.VarType.PinCategory == FName(TEXT("object")) )
	{
		const UClass* VarClass = Cast<const UClass>(Variable.VarType.PinSubCategoryObject.Get());
		return (VarClass && VarClass->HasAnyClassFlags(CLASS_DefaultToInstanced));
	}

	return false;
}

namespace UE::Blueprint::Private
{
	// Given a specified search criteria algorithm, walk the current blueprint with a specified scope.
	// When no explicit scope is provided, the asset registry will also be walked with the algorithm on additional blueprints.
	template<typename SearchFunc> static bool SearchBlueprintWithFunc(const SearchFunc& Func, const UBlueprint* Blueprint, const UEdGraph* LocalGraphScope)
	{
		// Search the initial blueprint
		if (Func(Blueprint))
		{
			return true;
		}

		// Optionally walk the asset registry for other blueprints
		if (!LocalGraphScope)
		{
			const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		
			// Discover additional packages which reference the initial blueprint package name
			FARFilter Filter;
			AssetRegistryModule.Get().GetReferencers(Blueprint->GetPackage()->GetFName(), Filter.PackageNames, UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Hard);

			if (Filter.PackageNames.Num() > 0)
			{
				GWarn->BeginSlowTask(LOCTEXT("LoadingReferencerAssets", "Loading Referencing Assets ..."), true);
				{
					Filter.TagsAndValues.Add(FBlueprintTags::IsDataOnly, TOptional<FString>(TEXT("false")));
					TArray<FAssetData> ReferencerAssetData;
					AssetRegistryModule.Get().GetAssets(Filter, ReferencerAssetData);

					// For each referencing asset
					for (const FAssetData& ReferencerData : ReferencerAssetData)
					{
						const UObject* AssetReferencer = ReferencerData.GetAsset();

						// Conditionally search the asset if it is a blueprint
						if (const UBlueprint* BlueprintReferencer = Cast<const UBlueprint>(AssetReferencer))
						{
							if (BlueprintReferencer && Func(BlueprintReferencer))
							{
								GWarn->EndSlowTask();
								return true;
							}
						}
						// Otherwise check to see if a corresponding world level blueprint is in scope to search
						else if (const UWorld* WorldReferencer = Cast<const UWorld>(AssetReferencer))
						{
							const auto& PersistentLevel = WorldReferencer->PersistentLevel;
							if (PersistentLevel && PersistentLevel->OwningWorld && Func(PersistentLevel->GetLevelScriptBlueprint()))
							{
								GWarn->EndSlowTask();
								return true;
							}
						}
					}
				}
				GWarn->EndSlowTask();
			}
		}

		return false;
	}

}

bool FBlueprintEditorUtils::IsVariableUsed(const UBlueprint* Blueprint, const FName& VariableName, const UEdGraph* LocalGraphScope /* = nullptr */)
{
	if (VariableName.IsNone())
	{
		return false;
	}

	// Retrieve the corresponding variable guid from the blueprint
	FGuid VariableGuid;

	if (LocalGraphScope)
	{
		VariableGuid = FindLocalVariableGuidByName(Blueprint, LocalGraphScope, VariableName);
	}
	else
	{
		UBlueprint::GetGuidFromClassByFieldName<FProperty>(Blueprint->SkeletonGeneratedClass, VariableName, VariableGuid);
	}
	
	if (!VariableGuid.IsValid())
	{
		return false;
	}

	// Blueprint variable search algorithm
	const auto SearchBlueprint = [VariableGuid, VariableName, Blueprint, LocalGraphScope](const UBlueprint* CurrentBlueprint) -> bool
	{
		TArray<UEdGraph*> AllGraphs;
		CurrentBlueprint->GetAllGraphs(AllGraphs);

		// For each blueprint subgraph
		for (TArray<UEdGraph*>::TConstIterator it(AllGraphs); it; ++it)
		{
			const UEdGraph* CurrentGraph = *it;

			// If the current graph is the specified scope or unbounded
			if (CurrentGraph && (CurrentGraph == LocalGraphScope || LocalGraphScope == nullptr))
			{
				// Check all variable nodes, ignoring connectivity
				TArray<UK2Node_Variable*> VariableNodes;
				CurrentGraph->GetNodesOfClass(VariableNodes);

				if (Algo::AnyOf(VariableNodes, [&VariableGuid, &VariableName](const UK2Node_Variable* VariableNode)
				{
					return VariableGuid == VariableNode->VariableReference.GetMemberGuid() && VariableName == VariableNode->GetVarName();
				}))
				{
					return true;
				}

				// Check all GetClassDefaults nodes that exposes the variable as an output pin connected to something
				TArray<UK2Node_GetClassDefaults*> ClassDefaultsNodes;
				CurrentGraph->GetNodesOfClass(ClassDefaultsNodes);

				if (Algo::AnyOf(ClassDefaultsNodes, [&VariableName, &Blueprint](const UK2Node_GetClassDefaults* GraphNode)
				{
					if (GraphNode->GetInputClass() == Blueprint->SkeletonGeneratedClass)
					{
						const UEdGraphPin* VarPin = GraphNode->FindPin(VariableName);
						if (VarPin && VarPin->Direction == EGPD_Output && VarPin->LinkedTo.Num() > 0)
						{
							return true;
						}
					}

					return false;
				}))
				{
					return true;
				}

				// Check for all component bound event nodes. This variable may be referenced by a bound event
				// (i.e. "On Component Hit" or any of the delegates you can add from the details panel with a "+" button)
				TArray<UK2Node_ComponentBoundEvent*> ComponentBoundEventNodes;
				CurrentGraph->GetNodesOfClass(ComponentBoundEventNodes);

				if (Algo::AnyOf(ComponentBoundEventNodes, [&VariableName](const UK2Node_ComponentBoundEvent* EventNode)
					{
						return EventNode->GetComponentPropertyName() == VariableName;
					}))
				{
					return true;
				}

				// Check all K2Node's which specify private/internal function referencing behavior
				TArray<const UK2Node*> GraphNodes;
				CurrentGraph->GetNodesOfClass(GraphNodes);

				if (Algo::AnyOf(GraphNodes, [&VariableName, &VariableGuid, &Blueprint](const UK2Node* GraphNode)
				{
					return GraphNode->ReferencesVariable(VariableName, Blueprint->SkeletonGeneratedClass);
				}))
				{
					return true;
				}

			}
		}

		return false;
	};

	// Given the specified variable search algorithm, walk the blueprint asset
	return UE::Blueprint::Private::SearchBlueprintWithFunc(SearchBlueprint, Blueprint, LocalGraphScope);
}

bool FBlueprintEditorUtils::IsFunctionUsed(const UBlueprint* Blueprint, const FName& FunctionName, const UEdGraph* LocalGraphScope /* = nullptr */)
{
	if (FunctionName.IsNone())
	{
		return false;
	}

	// Retrieve the corresponding function guid from the blueprint
	FGuid FunctionGuid;
	UBlueprint::GetFunctionGuidFromClassByFieldName(Blueprint->SkeletonGeneratedClass, FunctionName, FunctionGuid);

	if (!FunctionGuid.IsValid())
	{
		return false;
	}

	// Blueprint function search algorithm
	const auto SearchBlueprint = [&LocalGraphScope, &FunctionGuid, &FunctionName, &Blueprint](const UBlueprint* CurrentBlueprint) -> bool
	{
		TArray<UEdGraph*> BlueprintGraphs;
		CurrentBlueprint->GetAllGraphs(BlueprintGraphs);

		// For each blueprint subgraph
		for (TArray<UEdGraph*>::TConstIterator it(BlueprintGraphs); it; ++it)
		{
			const UEdGraph* CurrentGraph = *it;

			// If the current graph is the specified scope or unbounded
			if (CurrentGraph && (CurrentGraph == LocalGraphScope || !LocalGraphScope))
			{
				// Check all function graph nodes, ignoring connectivity
				TArray<UK2Node_CallFunction*> CallFunctionNodes;
				CurrentGraph->GetNodesOfClass(CallFunctionNodes);

				if (Algo::AnyOf(CallFunctionNodes, [&FunctionGuid, &FunctionName](const UK2Node_CallFunction* GraphNode)
				{
					return FunctionGuid == GraphNode->FunctionReference.GetMemberGuid() && FunctionName == GraphNode->GetFunctionName();
				}))
				{
					return true;
				}

				// Check all K2Nodes which specify internal function referencing behavior
				TArray<const UK2Node*> GraphNodes;
				CurrentGraph->GetNodesOfClass(GraphNodes);

				if (Algo::AnyOf(GraphNodes, [&FunctionName, &Blueprint](const UK2Node* GraphNode)
				{
					return GraphNode->ReferencesFunction(FunctionName, Blueprint->SkeletonGeneratedClass);
				}))
				{
					return true;
				}
			}
		}

		return false;
	};

	// Given the specified function search algorithm, walk the blueprint asset
	return UE::Blueprint::Private::SearchBlueprintWithFunc(SearchBlueprint, Blueprint, LocalGraphScope);
}

bool FBlueprintEditorUtils::ValidateAllMemberVariables(UBlueprint* InBlueprint, UBlueprint* InParentBlueprint, const FName InVariableName)
{
	for(int32 VariableIdx = 0; VariableIdx < InBlueprint->NewVariables.Num(); ++VariableIdx)
	{
		if(InBlueprint->NewVariables[VariableIdx].VarName == InVariableName)
		{
			FName NewChildName = FBlueprintEditorUtils::FindUniqueKismetName(InBlueprint, InVariableName.ToString(), InParentBlueprint ? InParentBlueprint->SkeletonGeneratedClass : InBlueprint->ParentClass);

			UE_LOGF(LogBlueprint, Warning, "Blueprint %ls (child of/implements %ls) has a member variable with a conflicting name (%ls). Changing to %ls.", *InBlueprint->GetName(), *GetNameSafe(InParentBlueprint), *InVariableName.ToString(), *NewChildName.ToString());

			FBlueprintEditorUtils::RenameMemberVariable(InBlueprint, InBlueprint->NewVariables[VariableIdx].VarName, NewChildName);
			return true;
		}
	}

	return false;
}

bool FBlueprintEditorUtils::ValidateAllComponentMemberVariables(UBlueprint* InBlueprint, UBlueprint* InParentBlueprint, const FName& InVariableName)
{
	if(InBlueprint->SimpleConstructionScript != nullptr)
	{
		TArray<USCS_Node*> ChildSCSNodes = InBlueprint->SimpleConstructionScript->GetAllNodes();
		for(int32 NodeIndex = 0; NodeIndex < ChildSCSNodes.Num(); ++NodeIndex)
		{
			USCS_Node* SCS_Node = ChildSCSNodes[NodeIndex];
			if(SCS_Node != nullptr && SCS_Node->GetVariableName() == InVariableName)
			{
				FName NewChildName = FBlueprintEditorUtils::FindUniqueKismetName(InBlueprint, InVariableName.ToString());

				UE_LOGF(LogBlueprint, Warning, "Blueprint %ls (child of/implements %ls) has a component variable with a conflicting name (%ls). Changing to %ls.", *InBlueprint->GetName(), *InParentBlueprint->GetName(), *InVariableName.ToString(), *NewChildName.ToString());

				FBlueprintEditorUtils::RenameComponentMemberVariable(InBlueprint, SCS_Node, NewChildName);
				return true;
			}
		}
	}
	return false;
}

bool FBlueprintEditorUtils::ValidateAllTimelines(UBlueprint* InBlueprint, UBlueprint* InParentBlueprint, const FName& InVariableName)
{
	for (int32 TimelineIndex=0; TimelineIndex < InBlueprint->Timelines.Num(); ++TimelineIndex)
	{
		UTimelineTemplate* TimelineTemplate = InBlueprint->Timelines[TimelineIndex];
		if( TimelineTemplate )
		{
			if( TimelineTemplate->GetFName() == InVariableName )
			{
				FName NewName = FBlueprintEditorUtils::FindUniqueKismetName(InBlueprint, TimelineTemplate->GetName());
				FBlueprintEditorUtils::RenameTimeline(InBlueprint, TimelineTemplate->GetFName(), NewName);

				UE_LOGF(LogBlueprint, Warning, "Blueprint %ls (child of/implements %ls) has a timeline with a conflicting name (%ls). Changing to %ls.", *InBlueprint->GetName(), *InParentBlueprint->GetName(), *InVariableName.ToString(), *NewName.ToString());
				return true;
			}
		}
	}
	return false;
}

bool FBlueprintEditorUtils::ValidateAllFunctionGraphs(UBlueprint* InBlueprint, UBlueprint* InParentBlueprint, const FName& InVariableName)
{
	for (int32 FunctionIndex=0; FunctionIndex < InBlueprint->FunctionGraphs.Num(); ++FunctionIndex)
	{
		UEdGraph* FunctionGraph = InBlueprint->FunctionGraphs[FunctionIndex];

		if( FunctionGraph->GetFName() == InVariableName )
		{
			FName NewName = FBlueprintEditorUtils::FindUniqueKismetName(InBlueprint, FunctionGraph->GetName());
			FBlueprintEditorUtils::RenameGraph(FunctionGraph, NewName.ToString());

			UE_LOGF(LogBlueprint, Warning, "Blueprint %ls (child of/implements %ls) has a function graph with a conflicting name (%ls). Changing to %ls.", *InBlueprint->GetName(), *InParentBlueprint->GetName(), *InVariableName.ToString(), *NewName.ToString());
			return true;
		}
	}
	return false;
}

void FBlueprintEditorUtils::FixupVariableDescription(UBlueprint* Blueprint, FBPVariableDescription& VarDesc)
{
	if ((VarDesc.PropertyFlags & CPF_Config) != 0 && Blueprint->GetLinkerCustomVersion(FBlueprintsObjectVersion::GUID) < FBlueprintsObjectVersion::DisallowObjectConfigVars)
	{
		// Synchronized with FBlueprintVarActionDetails::IsConfigCheckBoxEnabled
		const FEdGraphPinType& VarType = VarDesc.VarType;
		if (VarType.PinCategory == UEdGraphSchema_K2::PC_Object ||
			VarType.PinCategory == UEdGraphSchema_K2::PC_Interface)
		{
			VarDesc.PropertyFlags &= ~CPF_Config;
		}
	}

	// Remove bitflag enum type metadata if the enum type name is missing or if the enum type is no longer a bitflags type.
	if (VarDesc.HasMetaData(FBlueprintMetadata::MD_BitmaskEnum))
	{
		FString BitmaskEnumTypePath = VarDesc.GetMetaData(FBlueprintMetadata::MD_BitmaskEnum);
		if (!BitmaskEnumTypePath.IsEmpty())
		{
			const UEnum* BitflagsEnum = nullptr;
			
			// if the enum is saved by name (deprecated), find the associated enum and reserialize it as a long asset path
			if (FPackageName::IsShortPackageName(BitmaskEnumTypePath))
			{
				BitflagsEnum = FindFirstObject<UEnum>(GetData(BitmaskEnumTypePath));
				if (BitflagsEnum != nullptr)
				{
					BitmaskEnumTypePath = FTopLevelAssetPath(BitflagsEnum->GetPackage()->GetFName(), BitflagsEnum->GetFName()).ToString();
					VarDesc.SetMetaData(FBlueprintMetadata::MD_BitmaskEnum, BitmaskEnumTypePath);
				}
				else
				{
					UE_LOGF(LogBlueprint, Error, "Enum %ls cannot be loaded", *BitmaskEnumTypePath);
				}
			}
			else
			{
				BitflagsEnum = FindObject<UEnum>(nullptr, GetData(BitmaskEnumTypePath));
			}
			
			if (BitflagsEnum == nullptr || !BitflagsEnum->HasMetaData(*FBlueprintMetadata::MD_Bitflags.ToString()) || !UEdGraphSchema_K2::IsAllowableBlueprintVariableType(BitflagsEnum))
			{
				VarDesc.RemoveMetaData(FBlueprintMetadata::MD_BitmaskEnum);
			}
		}
		else
		{
			VarDesc.RemoveMetaData(FBlueprintMetadata::MD_BitmaskEnum);
		}
	}
}

void FBlueprintEditorUtils::ValidateBlueprintChildVariables(UBlueprint* InBlueprint, const FName InVariableName,
	TFunction<void(UBlueprint* InChildBP, const FName InVariableName, bool bValidatedVariable)> PostValidationCallback)
{
	// Iterate over currently-loaded Blueprints and potentially adjust their variable names if they conflict with the parent
	for(TObjectIterator<UBlueprint> BlueprintIt; BlueprintIt; ++BlueprintIt)
	{
		UBlueprint* ChildBP = *BlueprintIt;
		if(ChildBP != nullptr && ChildBP->ParentClass != nullptr)
		{
			TArray<UBlueprint*> ParentBPArray;
			// Get the parent hierarchy
			UBlueprint::GetBlueprintHierarchyFromClass(ChildBP->ParentClass, ParentBPArray);

			// Also get any BP interfaces we use
			TArray<UClass*> ImplementedInterfaces;
			FindImplementedInterfaces(ChildBP, true, ImplementedInterfaces);
			for (UClass* ImplementedInterface : ImplementedInterfaces)
			{
				UBlueprint* BlueprintInterfaceClass = UBlueprint::GetBlueprintFromClass(ImplementedInterface);
				if(BlueprintInterfaceClass != nullptr)
				{
					ParentBPArray.Add(BlueprintInterfaceClass);
				}
			}

			if(ParentBPArray.Contains(InBlueprint))
			{
				bool bValidatedVariable = false;

				bValidatedVariable = ValidateAllMemberVariables(ChildBP, InBlueprint, InVariableName);

				if(!bValidatedVariable)
				{
					bValidatedVariable = ValidateAllComponentMemberVariables(ChildBP, InBlueprint, InVariableName);
				}

				if(!bValidatedVariable)
				{
					bValidatedVariable = ValidateAllTimelines(ChildBP, InBlueprint, InVariableName);
				}

				if(!bValidatedVariable)
				{
					bValidatedVariable = ValidateAllFunctionGraphs(ChildBP, InBlueprint, InVariableName);
				}

				if (PostValidationCallback)
				{
					// Perform custom post-validation (if specified).
					PostValidationCallback(ChildBP, InVariableName, bValidatedVariable);
				}
			}
		}
	}
}

int32 FBlueprintEditorUtils::GetChildrenOfBlueprint(UBlueprint* InBlueprint, TArray<FAssetData>& OutChildren, bool bInRecursive /*= true*/)
{
	int32 Count = 0;
	const FAssetData ParentAsset(InBlueprint);
	TArray<FName> ParentNames;
	ParentNames.Add(ParentAsset.GetTagValueRef<FName>(FBlueprintTags::GeneratedClassPath));

	for (int32 ParentIdx = 0; ParentIdx < ParentNames.Num(); ++ParentIdx)
	{
		FARFilter Filter;
		Filter.TagsAndValues.Add(FBlueprintTags::ParentClassPath, ParentNames[ParentIdx].ToString());

		TArray<FAssetData> FoundAssets;
		if (FAssetRegistryModule::GetRegistry().GetAssets(Filter, FoundAssets) && FoundAssets.Num() > 0)
		{
			if (bInRecursive)
			{
				for (const FAssetData& Child : FoundAssets)
				{
					ParentNames.Add(Child.GetTagValueRef<FName>(FBlueprintTags::GeneratedClassPath));
				}
			}

			Count += FoundAssets.Num();
			OutChildren.Append(MoveTemp(FoundAssets));
		}
	}

	return Count;
}

void FBlueprintEditorUtils::MarkBlueprintChildrenAsModified(UBlueprint* InBlueprint)
{
	TArray<FAssetData> Children;
	if (GetChildrenOfBlueprint(InBlueprint, Children) > 0)
	{
		SIZE_T Unloaded = Algo::CountIf(Children,
			[](const FAssetData& Asset)
			{
				return !Asset.IsAssetLoaded();
			});


		// If there are any unloaded children, ask the user to verify
		EAppReturnType::Type DialogResponse = EAppReturnType::Yes;
		if (Unloaded > 0)
		{
			FText Message = FText::Format(LOCTEXT("LoadChildrenPopupMessage", "Load {0} unloaded child blueprints to fix up phantom references?"), FText::FromString(LexToString(Unloaded)));
			FText Title = LOCTEXT("LoadChildrenPopupTitle", "Load Unloaded Children?");
			DialogResponse = FMessageDialog::Open(EAppMsgType::YesNo, Message, Title);
		}

		// Conditionally Load Children and mark as modified 
		const bool bLoad = (DialogResponse == EAppReturnType::Yes);
		for (FAssetData& Child : Children)
		{
			if (UBlueprint* ChildBlueprint = Cast<UBlueprint>(Child.FastGetAsset(bLoad)))
			{
				MarkBlueprintAsModified(ChildBlueprint);
			}
		}
	}
}

//////////////////////////////////////////////////////////////////////////

/** Shared function for posting notification toasts */
static void ShowNotification(const FText& Message, EMessageSeverity::Type Severity)
{
	if(FApp::IsUnattended())
	{
		switch(Severity)
		{
		case EMessageSeverity::Error:
			UE_LOGF(LogBlueprint, Error, "%ls", *Message.ToString());
			break;
		case EMessageSeverity::PerformanceWarning:
		case EMessageSeverity::Warning:
			UE_LOGF(LogBlueprint, Warning, "%ls", *Message.ToString());
			break;
		case EMessageSeverity::Info:
			UE_LOGF(LogBlueprint, Log, "%ls", *Message.ToString());
			break;
		}
	}
	else
	{
		FNotificationInfo Warning(Message);
		Warning.ExpireDuration = 5.0f;
		Warning.bFireAndForget = true;
		switch(Severity)
		{
		case EMessageSeverity::Error:
			Warning.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Error"));
			break;
		case EMessageSeverity::PerformanceWarning:
		case EMessageSeverity::Warning:
			Warning.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
			break;
		case EMessageSeverity::Info:
			Warning.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Info"));
			break;
		}
	
		FSlateNotificationManager::Get().AddNotification(Warning);
	}
}

//////////////////////////////////////////////////////////////////////////
// Interfaces

FGuid FBlueprintEditorUtils::FindInterfaceGraphGuid(const FName& GraphName, const UClass* InterfaceClass)
{
	// check if this is a blueprint - only blueprint interfaces can have Guids
	check(InterfaceClass);
	const UBlueprint* InterfaceBlueprint = Cast<UBlueprint>(InterfaceClass->ClassGeneratedBy);
	if(InterfaceBlueprint != nullptr)
	{
		// find the graph for this function
		TArray<UEdGraph*> InterfaceGraphs;
		InterfaceBlueprint->GetAllGraphs(InterfaceGraphs);

		for (const UEdGraph* InterfaceGraph : InterfaceGraphs)
		{
			if(InterfaceGraph != nullptr && InterfaceGraph->GetFName() == GraphName)
			{
				return InterfaceGraph->GraphGuid;
			}
		}
	}

	return FGuid();
}

FGuid FBlueprintEditorUtils::FindInterfaceFunctionGuid(const UFunction* Function, const UClass* InterfaceClass)
{
	check(Function);
	return FindInterfaceGraphGuid(Function->GetFName(), InterfaceClass);
}

// Add a new interface, and member function graphs to the blueprint
bool FBlueprintEditorUtils::ImplementNewInterface(UBlueprint* Blueprint, FTopLevelAssetPath InterfaceClassPathName)
{
	check(!InterfaceClassPathName.IsNull());

	// Attempt to find the class we want to implement
	UClass* InterfaceClass = FindObject<UClass>(InterfaceClassPathName);

	// Make sure the class is found, and isn't native (since Blueprints don't necessarily generate native classes.
	check(InterfaceClass);

	// Check to make sure we haven't already implemented it
	for( int32 i = 0; i < Blueprint->ImplementedInterfaces.Num(); i++ )
	{
		if( Blueprint->ImplementedInterfaces[i].Interface == InterfaceClass )
		{
			ShowNotification(
				FText::Format(
					LOCTEXT("InterfaceAlreadyImplementedFmt", "Blueprint '{0}' already implements the interface '{1}'"),
					FText::FromString(Blueprint->GetName()),
					FText::FromString(InterfaceClassPathName.ToString())
				),
				EMessageSeverity::Warning
			);
			return false;
		}
	}

	// Make a new entry for this interface
	FBPInterfaceDescription NewInterface;
	NewInterface.Interface = InterfaceClass;

	bool bAllFunctionsAdded = true;

	// Add the graphs for the functions required by this interface
	for( TFieldIterator<UFunction> FunctionIter(InterfaceClass, EFieldIteratorFlags::IncludeSuper); FunctionIter; ++FunctionIter )
	{
		UFunction* Function = *FunctionIter;
		const bool bIsAnimFunction = Function->HasMetaData(FBlueprintMetadata::MD_AnimBlueprintFunction);
		if(bIsAnimFunction && !Blueprint->IsA<UAnimBlueprint>())
		{
			bAllFunctionsAdded = false;
			ShowNotification(
				FText::Format(
					LOCTEXT("AddedAnimFunctionToNonAnimBlueprint", "Blueprint '{0}' is not an animation blueprint and therefore cannot implement interface '{1}'"),
					FText::FromString(Blueprint->GetName()),
					FText::FromString(InterfaceClassPathName.ToString())
				),
				EMessageSeverity::Error
			);
			break;
		}
		if( (UEdGraphSchema_K2::CanKismetOverrideFunction(Function) && !UEdGraphSchema_K2::FunctionCanBePlacedAsEvent(Function)) || 
			bIsAnimFunction)
		{
			FName FunctionName = Function->GetFName();
			UEdGraph* FuncGraph = FindObject<UEdGraph>( Blueprint, *(FunctionName.ToString()) );
			if (FuncGraph != nullptr)
			{
				bAllFunctionsAdded = false;

				ShowNotification(
					FText::Format(
						LOCTEXT("InterfaceFunctionConflictsFmt", "Blueprint '{0}' has a function or graph which conflicts with function '{1}' in interface '{2}'"),
						FText::FromString(Blueprint->GetName()),
						FText::FromName(FunctionName),
						FText::FromString(InterfaceClassPathName.ToString())
					),
					EMessageSeverity::Error
				);
				break;
			}

			UEdGraph* NewGraph;
			if(bIsAnimFunction)
			{
				NewGraph = FBlueprintEditorUtils::CreateNewGraph(Blueprint, FunctionName, UAnimationGraph::StaticClass(), UAnimationGraphSchema::StaticClass());
			}
			else
			{
				NewGraph = FBlueprintEditorUtils::CreateNewGraph(Blueprint, FunctionName, UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass());
			}
			NewGraph->bAllowDeletion = false;
			NewGraph->InterfaceGuid = FindInterfaceFunctionGuid(Function, InterfaceClass);

			NewInterface.Graphs.Add(NewGraph);

			FBlueprintEditorUtils::AddInterfaceGraph(Blueprint, NewGraph, InterfaceClass);
		}
	}

	if (bAllFunctionsAdded)
	{
		Blueprint->ImplementedInterfaces.Add(NewInterface);
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
	}
	return bAllFunctionsAdded;
}

bool FBlueprintEditorUtils::ImplementNewInterface(UBlueprint* Blueprint, const FName& InterfaceClassName)
{
	FTopLevelAssetPath InterfaceClassPathName = UClass::TryConvertShortTypeNameToPathName<UStruct>(InterfaceClassName.ToString(), ELogVerbosity::Warning, TEXT("FBlueprintEditorUtils::ImplementNewInterface"));
	return ImplementNewInterface(Blueprint, InterfaceClassPathName);
}

// Gets the graphs currently in the blueprint associated with the specified interface
void FBlueprintEditorUtils::GetInterfaceGraphs(UBlueprint* Blueprint, FTopLevelAssetPath InterfaceClassPathName, TArray<UEdGraph*>& ChildGraphs)
{
	ChildGraphs.Empty();

	if (InterfaceClassPathName.IsNull())
	{
		return;
	}

	// Find the implemented interface
	for (const FBPInterfaceDescription& InterfaceDesc : Blueprint->ImplementedInterfaces)
	{
		if (InterfaceDesc.Interface && InterfaceDesc.Interface->GetClassPathName() == InterfaceClassPathName)
		{
			ChildGraphs = InterfaceDesc.Graphs;
			return;			
		}
	}
}

void FBlueprintEditorUtils::GetInterfaceGraphs(UBlueprint* Blueprint, const FName& InterfaceClassName, TArray<UEdGraph*>& ChildGraphs)
{
	FTopLevelAssetPath InterfaceClassPathName = UClass::TryConvertShortTypeNameToPathName<UStruct>(InterfaceClassName.ToString(), ELogVerbosity::Warning, TEXT("FBlueprintEditorUtils::GetInterfaceGraphs"));
	GetInterfaceGraphs(Blueprint, InterfaceClassPathName, ChildGraphs);
}

UFunction* FBlueprintEditorUtils::GetInterfaceFunction(UBlueprint* Blueprint, const FName FuncName)
{
	UFunction* Function = nullptr;

	// If that class is an interface class implemented by this function, then return true
	for (const FBPInterfaceDescription& I : Blueprint->ImplementedInterfaces)
	{
		if (I.Interface)
		{
			Function = FindUField<UFunction>(I.Interface, FuncName);
			if (Function)
			{
				// found it, done
				return Function;
			}
		}
	}

	// Check if it is in a native class or parent class
	for (UClass* TempClass = Blueprint->ParentClass; (nullptr != TempClass) && (nullptr == Function); TempClass = TempClass->GetSuperClass())
	{
		for (const FImplementedInterface& I : TempClass->Interfaces)
		{
			Function = FindUField<UFunction>(I.Class, FuncName);
			if (Function)
			{
				// found it, done
				return Function;
			}
		}
	}

	return nullptr;
}

bool FBlueprintEditorUtils::IsInterfaceFunction(UBlueprint* Blueprint, UFunction* Function)
{
	if (Blueprint == nullptr || Function == nullptr)
	{
		return false;
	}

	const FName FuncName = Function->GetFName();

	// Will return nullptr if the function isn't found
	return (GetInterfaceFunction(Blueprint, FuncName) != nullptr);
}

// Remove an implemented interface, and its associated member function graphs
void FBlueprintEditorUtils::RemoveInterface(UBlueprint* Blueprint, FTopLevelAssetPath InterfaceClassPathName, bool bPreserveFunctions /*= false*/)
{
	if (InterfaceClassPathName.IsNull())
	{
		return;
	}

	// Find the implemented interface
	int32 Idx = INDEX_NONE;
	for (int32 i = 0; i < Blueprint->ImplementedInterfaces.Num(); i++)
	{
		const FBPInterfaceDescription& InterfaceDesc = Blueprint->ImplementedInterfaces[i];
		if (InterfaceDesc.Interface && InterfaceDesc.Interface->GetClassPathName() == InterfaceClassPathName)
		{
			Idx = i;
			break;
		}
	}

	if (ensureMsgf(Idx != INDEX_NONE, TEXT("%s: No implementation was found for \'%s\'."), *Blueprint->GetName(), *InterfaceClassPathName.ToString()))
	{
		FBPInterfaceDescription& CurrentInterface = Blueprint->ImplementedInterfaces[Idx];
		const UClass* InterfaceClass = Blueprint->ImplementedInterfaces[Idx].Interface;
		const FScopedTransaction Transaction(LOCTEXT("RemoveInterface", "Remove interface"));
		Blueprint->Modify();

		// For every function and event in the interface...
		for (TFieldIterator<UFunction> FunctionIt(InterfaceClass); FunctionIt; ++FunctionIt)
		{
			UFunction* Function = *FunctionIt;
			const FName FunctionName = Function->GetFName();
			
			// If this function name is in the list of interface graphs, then handle it like a function
			if (FunctionName == UEdGraphSchema_K2::FN_ExecuteUbergraphBase || 
				FBlueprintEditorUtils::RemoveInterfaceFunction(Blueprint, CurrentInterface, Function, bPreserveFunctions))
			{
				continue;
			}

			// Find all events placed in the event graph so we can check if they belong to this interface
			TArray<UK2Node_Event*> AllEvents;
			FBlueprintEditorUtils::GetAllNodesOfClass(Blueprint, AllEvents);
			for (TArray<UK2Node_Event*>::TIterator NodeIt(AllEvents); NodeIt; ++NodeIt)
			{
				UK2Node_Event* EventNode = *NodeIt;
				if (EventNode->EventReference.GetMemberParentClass(EventNode->GetBlueprintClassFromNode()) == InterfaceClass)
				{
					UEdGraph* EventNodeGraph = EventNode->GetGraph();
					check(EventNodeGraph);
					EventNodeGraph->Modify();

					if (bPreserveFunctions)
					{
						// Create a custom event with the same name and signature
						const FVector2D PreviousNodePos = FVector2D(EventNode->NodePosX, EventNode->NodePosY);
						const FString PreviousNodeName = EventNode->EventReference.GetMemberName().ToString();
						const UFunction* PreviousSignatureFunction = EventNode->FindEventSignatureFunction();
						check(PreviousSignatureFunction);
						UK2Node_CustomEvent* NewEvent = UK2Node_CustomEvent::CreateFromFunction(PreviousNodePos, EventNode->GetGraph(), PreviousNodeName, PreviousSignatureFunction, false);
						const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

						// Move the pin links from the old pin to the new pin to preserve connections
						for (UEdGraphPin* CurrentPin : EventNode->Pins)
						{
							if (UEdGraphPin* TargetPin = NewEvent->FindPin(CurrentPin->PinName))
							{
								Schema->MovePinLinks(*CurrentPin, *TargetPin);
							}
							else
							{
								UE_LOGF(LogBlueprint, Warning,
									"Failed to transfer pin %ls on function %ls while removing interface from blueprint %ls",
									*CurrentPin->PinName.ToString(), *PreviousSignatureFunction->GetPathName(), *Blueprint->GetPathName());
							}
						}
					}

					EventNodeGraph->RemoveNode(EventNode);
					break;
				}
			}
		}

		// Then remove the interface from the list
		Blueprint->ImplementedInterfaces.RemoveAt(Idx, 1);
		
		// Refresh all the nodes to make sure that the references to "Self" are updated appropriately on any function calls  @see UE-78253
		FBlueprintEditorUtils::RefreshAllNodes(Blueprint);

		// Now recompile the blueprint (this needs to be done outside of RemoveGraph, after it's been removed from ImplementedInterfaces - otherwise it'll re-add it)
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);

		// Mark Child Blueprints as modified to fixup references to the Interface
		MarkBlueprintChildrenAsModified(Blueprint);
	}
}

void FBlueprintEditorUtils::RemoveInterface(UBlueprint* Blueprint, const FName& InterfaceClassName, bool bPreserveFunctions /*= false*/)
{
	FTopLevelAssetPath InterfaceClassPathName = UClass::TryConvertShortTypeNameToPathName<UStruct>(InterfaceClassName.ToString(), ELogVerbosity::Warning, TEXT("FBlueprintEditorUtils::RemoveInterface"));
	return RemoveInterface(Blueprint, InterfaceClassPathName, bPreserveFunctions);
}

bool FBlueprintEditorUtils::RemoveInterfaceFunction(UBlueprint* Blueprint, FBPInterfaceDescription& Interface, UFunction* Function, bool bPreserveFunction)
{
	for (auto it = Interface.Graphs.CreateIterator(); it; ++it)
	{
		UEdGraph* CurrentGraph = *it;
		if (Function->GetFName() == CurrentGraph->GetFName())
		{
			CurrentGraph->Modify();
			Blueprint->Modify();

			if (bPreserveFunction)
			{
				// Promote the graph if the user wants to preserve it:
				PromoteGraphFromInterfaceOverride(Blueprint, CurrentGraph);
				Blueprint->FunctionGraphs.Add(CurrentGraph);
			}

			FBlueprintEditorUtils::UpdateTransactionalFlags(Blueprint);

			if(!bPreserveFunction)
			{
				// Remove the interface graph
				FBlueprintEditorUtils::RemoveGraph(Blueprint, CurrentGraph, EGraphRemoveFlags::MarkTransient);	// Do not recompile, yet
			}

			return true;
		}
	}

	return false;
}

void FBlueprintEditorUtils::PromoteGraphFromInterfaceOverride(UBlueprint* InBlueprint, UEdGraph* InInterfaceGraph)
{
	InInterfaceGraph->bAllowDeletion = true;
	InInterfaceGraph->bAllowRenaming = true;
	InInterfaceGraph->bEditable = true;
	InInterfaceGraph->InterfaceGuid.Invalidate();

	// We need to flag the entry node to make sure that the compiled function is callable
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	Schema->AddExtraFunctionFlags(InInterfaceGraph, (FUNC_BlueprintCallable | FUNC_BlueprintEvent | FUNC_Public));
	Schema->MarkFunctionEntryAsEditable(InInterfaceGraph, true);

	// Move all non-exec pins from the function entry node to being user defined pins
	TArray<UK2Node_FunctionEntry*> FunctionEntryNodes;
	InInterfaceGraph->GetNodesOfClass(FunctionEntryNodes);
	if (FunctionEntryNodes.Num() > 0)
	{
		UK2Node_FunctionEntry* FunctionEntry = FunctionEntryNodes[0];
		FunctionEntry->PromoteFromInterfaceOverride();
	}

	// Move all non-exec pins from the function result node to being user defined pins
	TArray<UK2Node_FunctionResult*> FunctionResultNodes;
	InInterfaceGraph->GetNodesOfClass(FunctionResultNodes);
	if (FunctionResultNodes.Num() > 0)
	{
		UK2Node_FunctionResult* PrimaryFunctionResult = FunctionResultNodes[0];
		PrimaryFunctionResult->PromoteFromInterfaceOverride();

		// Reconstruct all result nodes so they update their pins accordingly
		for (UK2Node_FunctionResult* FunctionResult : FunctionResultNodes)
		{
			if (PrimaryFunctionResult != FunctionResult)
			{
				FunctionResult->PromoteFromInterfaceOverride(false);
			}
		}
	}

	// Promote any animation linked input poses
	TArray<UAnimGraphNode_LinkedInputPose*> LinkedInputPoseNodes;
	InInterfaceGraph->GetNodesOfClass(LinkedInputPoseNodes);
	for (UAnimGraphNode_LinkedInputPose* LinkedInputPoseNode : LinkedInputPoseNodes)
	{
		LinkedInputPoseNode->PromoteFromInterfaceOverride();
	}
}

void FBlueprintEditorUtils::CleanNullGraphReferencesRecursive(UEdGraph* Graph)
{
	for (int32 GraphIndex = 0; GraphIndex < Graph->SubGraphs.Num(); )
	{
		if (UEdGraph* ChildGraph = Graph->SubGraphs[GraphIndex])
		{
			CleanNullGraphReferencesRecursive(ChildGraph);
			++GraphIndex;
		}
		else
		{
			Graph->SubGraphs.RemoveAt(GraphIndex);
		}
	}
}

void FBlueprintEditorUtils::CleanNullGraphReferencesInArray(UBlueprint* Blueprint, TArray<UEdGraph*>& GraphArray)
{
	for (int32 GraphIndex = 0; GraphIndex < GraphArray.Num(); )
	{
		if (UEdGraph* Graph = GraphArray[GraphIndex])
		{
			CleanNullGraphReferencesRecursive(Graph);
			++GraphIndex;
		}
		else
		{
			GraphArray.RemoveAt(GraphIndex);
		}
	}
}

void FBlueprintEditorUtils::PurgeNullGraphs(UBlueprint* Blueprint)
{
	CleanNullGraphReferencesInArray(Blueprint, MutableView(Blueprint->UbergraphPages));
	CleanNullGraphReferencesInArray(Blueprint, MutableView(Blueprint->FunctionGraphs));
	CleanNullGraphReferencesInArray(Blueprint, MutableView(Blueprint->DelegateSignatureGraphs));
	CleanNullGraphReferencesInArray(Blueprint, MutableView(Blueprint->MacroGraphs));
}

void FBlueprintEditorUtils::RemoveInvalidStructureMemberVariableFromBlueprint(UBlueprint* Blueprint)
{
	if (!Blueprint)
	{
		return;
	}

	const UScriptStruct* FallbackStruct = GetFallbackStruct();

	const auto IsValidStruct = [FallbackStruct](const UScriptStruct* Struct)
	{
		if(Struct == nullptr || !IsValid(Struct))
		{
			return false;
		}

		if(Struct == FallbackStruct)
		{
			return false;
		}

		if(!Struct->IsNative() && Struct->HasAnyFlags(RF_Transient))
		{
			return false;
		}
		return true;
	};

	FString DisplayList;
	TArray<FName> ZombieMemberNames;
	for (int32 VarIndex = 0; VarIndex < Blueprint->NewVariables.Num(); ++VarIndex)
	{
		const FBPVariableDescription& Var = Blueprint->NewVariables[VarIndex];

		bool bIsInvalid = false;

		if (Var.VarType.PinCategory == UEdGraphSchema_K2::PC_Struct)
		{
			// The variable is invalid if the struct object is null, or it points to the fallback struct
			UScriptStruct* ScriptStruct = Cast<UScriptStruct>(Var.VarType.PinSubCategoryObject.Get());

			bIsInvalid = !IsValidStruct(ScriptStruct);
		}
		else if (Var.VarType.IsMap() && Var.VarType.PinValueType.TerminalCategory == UEdGraphSchema_K2::PC_Struct)
		{
			// If there is no ValueType object then the variable is invalid
			bIsInvalid = !IsValidStruct(Cast<UScriptStruct>(Var.VarType.PinValueType.TerminalSubCategoryObject.Get()));
		}

		// If this variable is invalid then display a warning
		if (bIsInvalid)
		{
			DisplayList += Var.FriendlyName.IsEmpty() ? Var.VarName.ToString() : Var.FriendlyName;
			DisplayList += TEXT("\n");
			ZombieMemberNames.Add(Var.VarName);
		}
	}

	if (ZombieMemberNames.Num())
	{
		UE_LOGF(LogBlueprint, Warning, "The following member variables in blueprint '%ls' have invalid type. Removing them.\n\n%ls", *Blueprint->GetFullName(), *DisplayList);

		Blueprint->Modify();

		for (const FName& Name : ZombieMemberNames)
		{
			Blueprint->NewVariables.RemoveAll([Name](const FBPVariableDescription& Desc){ return Desc.VarName == Name; });
			FBlueprintEditorUtils::RemoveVariableNodes(Blueprint, Name);
		}
	}
}

struct FConformCallsToParentFunctionUtils
{
	// Remove a parent function call node without breaking existing connections
	static void RemoveParentFunctionCallNode(UBlueprint* InBlueprint, UK2Node_CallParentFunction* InCallFunctionNode)
	{
		// Cache a reference to the output exec pin
		UEdGraphPin* OutputPin = InCallFunctionNode->GetThenPin();

		// We're going to destroy the existing parent function call node, but first we need to persist any existing connections
		for (int PinIndex = 0; PinIndex < InCallFunctionNode->Pins.Num(); ++PinIndex)
		{
			UEdGraphPin* InputPin = InCallFunctionNode->Pins[PinIndex];
			check(nullptr != InputPin);

			// If this is an input exec pin
			const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
			if (K2Schema && K2Schema->IsExecPin(*InputPin) && InputPin->Direction == EGPD_Input)
			{
				// Redirect any existing connections to the input exec pin to whatever pin(s) the output exec pin is connected to
				for (int InputLinkedToPinIndex = 0; InputLinkedToPinIndex < InputPin->LinkedTo.Num(); ++InputLinkedToPinIndex)
				{
					UEdGraphPin* InputLinkedToPin = InputPin->LinkedTo[InputLinkedToPinIndex];
					check(nullptr != InputLinkedToPin);

					// Break the existing link to the node we're about to remove
					InputLinkedToPin->BreakLinkTo(InputPin);

					// Redirect the input connection to the output connection(s)
					for (int OutputLinkedToPinIndex = 0; OutputLinkedToPinIndex < OutputPin->LinkedTo.Num(); ++OutputLinkedToPinIndex)
					{
						UEdGraphPin* OutputLinkedToPin = OutputPin->LinkedTo[OutputLinkedToPinIndex];
						check(nullptr != OutputLinkedToPin);

						// Make sure the output connection isn't linked to the node we're about to remove
						if (OutputLinkedToPin->LinkedTo.Contains(OutputPin))
						{
							OutputLinkedToPin->BreakLinkTo(OutputPin);
						}

						// Fix up the connection
						InputLinkedToPin->MakeLinkTo(OutputLinkedToPin);
					}
				}
			}
		}

		// Emit something to the log to indicate that we're making a change
		FFormatNamedArguments Args;
		Args.Add(TEXT("NodeTitle"), InCallFunctionNode->GetNodeTitle(ENodeTitleType::ListView));
		Args.Add(TEXT("FunctionNodeName"), FText::FromString(InCallFunctionNode->GetName()));
		InBlueprint->Message_Note(FText::Format(LOCTEXT("CallParentNodeRemoved_Note", "{NodeTitle} ({FunctionNodeName}) was not valid for this Blueprint - it has been removed."), Args).ToString());

		// Destroy the existing parent function call node (this will also break pin links and remove it from the graph)
		InCallFunctionNode->DestroyNode();
	}

	// Makes sure that all function overrides are valid, and replaces with local functions if not
	static void ConformParentFunctionOverrides(UBlueprint* InBlueprint)
	{
		TArray<UEdGraph*> AllGraphs;
		InBlueprint->GetAllGraphs(AllGraphs);
		for (UEdGraph* CurrentGraph : AllGraphs)
		{
			check(CurrentGraph);

			TArray<UK2Node_FunctionEntry*> FunctionEntryNodes;
			CurrentGraph->GetNodesOfClass<UK2Node_FunctionEntry>(FunctionEntryNodes);

			for (UK2Node_FunctionEntry* FunctionEntryNode : FunctionEntryNodes)
			{
				check(FunctionEntryNode);

				if (!FunctionEntryNode->FunctionReference.IsSelfContext())
				{
					UClass* SignatureClass = FunctionEntryNode->FunctionReference.GetMemberParentClass();
					if (const UBlueprint* SignatureClassBlueprint = UBlueprint::GetBlueprintFromClass(SignatureClass))
					{
						// Redirect to the skeleton class for Blueprint types.
						SignatureClass = SignatureClassBlueprint->SkeletonGeneratedClass;
					}

					if (SignatureClass)
					{
						const UFunction* Function = FunctionEntryNode->FunctionReference.ResolveMember<UFunction>(SignatureClass);
						if (Function == nullptr)
						{
							// Remove any calls to the parent class implementation
							TArray<UK2Node_CallParentFunction*> CallFunctionNodes;
							CurrentGraph->GetNodesOfClass<UK2Node_CallParentFunction>(CallFunctionNodes);
							for (UK2Node_CallParentFunction* CallFunctionNode : CallFunctionNodes)
							{
								RemoveParentFunctionCallNode(InBlueprint, CallFunctionNode);
							}

							// Execute the same conversion path we use for interface graphs
							FBlueprintEditorUtils::PromoteGraphFromInterfaceOverride(InBlueprint, CurrentGraph);

							// Emit something to the log to indicate that we've made a change
							FFormatNamedArguments Args;
							Args.Add(TEXT("NodeTitle"), FunctionEntryNode->GetNodeTitle(ENodeTitleType::ListView));
							Args.Add(TEXT("ParentClass"), FText::FromString(SignatureClass->GetName()));
							InBlueprint->Message_Note(FText::Format(LOCTEXT("ConvertedToLocalMemberFunction_Note", "Function '{NodeTitle}' was previously implemented as an override, but the function is no longer found in class '{ParentClass}'. As a result, it has been converted to a full member function."), Args).ToString());
						}
						else
						{
							if (FunctionEntryNode->bEnforceConstCorrectness)
							{
								// Sync the 'const' attribute with the original function, in case it has been changed
								const bool bIsConstFunction = Function->HasAllFunctionFlags(FUNC_Const);
								if (bIsConstFunction != FunctionEntryNode->HasAllExtraFlags(FUNC_Const))
								{
									int32 ExtraFlags = FunctionEntryNode->GetExtraFlags();

									FunctionEntryNode->Modify();
									FunctionEntryNode->SetExtraFlags(ExtraFlags ^ FUNC_Const);
								}
							}
						}
					}
				}

				// Rename the graph if it does not match the actual function name.
				const FName FunctionName = (FunctionEntryNode->CustomGeneratedFunctionName != NAME_None) ? FunctionEntryNode->CustomGeneratedFunctionName : FunctionEntryNode->FunctionReference.GetMemberName();
				if (FunctionEntryNode == FunctionEntryNodes[0]
					&& !FBlueprintEditorUtils::IsEventGraph(CurrentGraph)
					&& CurrentGraph->GetFName() != FunctionName)
				{
					FBlueprintEditorUtils::RenameGraph(CurrentGraph, FunctionName.ToString());
				}
			}
		}
	}
};

// Makes sure that calls to parent functions are valid, and removes them if not
void FBlueprintEditorUtils::ConformCallsToParentFunctions(UBlueprint* Blueprint)
{
	check(nullptr != Blueprint);

	// First, ensure that all function override implementations are up-to-date.
	FConformCallsToParentFunctionUtils::ConformParentFunctionOverrides(Blueprint);

	// Get the Blueprint's parent class.
	UClass* ParentClass = Blueprint->ParentClass;
	if (const UBlueprint* ParentBlueprint = UBlueprint::GetBlueprintFromClass(ParentClass))
	{
		// Defer to the skeleton class, as a Blueprint parent may not have been fully compiled yet (e.g. after a rename).
		ParentClass = ParentBlueprint->SkeletonGeneratedClass;
	}

	TArray<UEdGraph*> AllGraphs;
	Blueprint->GetAllGraphs(AllGraphs);
	for(int GraphIndex = 0; GraphIndex < AllGraphs.Num(); ++GraphIndex)
	{
		UEdGraph* CurrentGraph = AllGraphs[GraphIndex];
		check(nullptr != CurrentGraph);

		// Make sure the graph is loaded
		if(!CurrentGraph->HasAnyFlags(RF_NeedLoad|RF_NeedPostLoad))
		{
			TArray<UK2Node_CallParentFunction*> CallFunctionNodes;
			CurrentGraph->GetNodesOfClass<UK2Node_CallParentFunction>(CallFunctionNodes);

			// For each parent function call node in the graph
			for(int CallFunctionNodeIndex = 0; CallFunctionNodeIndex < CallFunctionNodes.Num(); ++CallFunctionNodeIndex)
			{
				UK2Node_CallParentFunction* CallFunctionNode = CallFunctionNodes[CallFunctionNodeIndex];
				check(nullptr != CallFunctionNode);

				// Make sure the node has already been loaded
				if(!CallFunctionNode->HasAnyFlags(RF_NeedLoad|RF_NeedPostLoad))
				{
					// Attempt to locate the function within the parent class
					FName MemberName = CallFunctionNode->FunctionReference.GetMemberName();
					const UFunction* TargetFunction = ParentClass ? ParentClass->FindFunctionByName(MemberName) : nullptr;
					if (TargetFunction == nullptr && ParentClass != nullptr)
					{
						// In case the function was renamed in the parent class, try to look up using the member's GUID.
						const FGuid MemberGuid = CallFunctionNode->FunctionReference.GetMemberGuid();
						if (MemberGuid.IsValid())
						{
							MemberName = UBlueprint::GetFieldNameFromClassByGuid<UFunction>(ParentClass, MemberGuid);
							if (MemberName != NAME_None)
							{
								TargetFunction = ParentClass->FindFunctionByName(MemberName);
							}
						}
					}
					
					if(TargetFunction != nullptr)
					{
						// Check to see if the function signature does not match the (authoritative) parent class.
						if(TargetFunction->GetOwnerClass()->GetAuthoritativeClass() != CallFunctionNode->FunctionReference.GetMemberParentClass(ParentClass->GetAuthoritativeClass()))
						{
							// Emit something to the log to indicate that we're making a change
							FFormatNamedArguments Args;
							Args.Add(TEXT("NodeTitle"), CallFunctionNode->GetNodeTitle(ENodeTitleType::ListView));
							Args.Add(TEXT("FunctionNodeName"), FText::FromString(CallFunctionNode->GetName()));
							Blueprint->Message_Note(FText::Format(LOCTEXT("CallParentFunctionSignatureFixed_Note", "{NodeTitle} ({FunctionNodeName}) had an invalid function signature - it has now been fixed."), Args).ToString() );

							// Redirect to the correct parent function. Note that for Blueprints, internally this will switch to the target function that's owned by the authoritative parent class.
							CallFunctionNode->SetFromFunction(TargetFunction);
						}
					}
					else
					{
						// Remove the parent function call node, preserving any existing connections.
						FConformCallsToParentFunctionUtils::RemoveParentFunctionCallNode(Blueprint, CallFunctionNode);
					}
				}
			}
		}
	}
}

namespace
{
	static bool ExtendedIsParent(const UClass* Parent, const UClass* Child)
	{
		if (Parent && Child)
		{
			if (Child->IsChildOf(Parent))
			{
				return true;
			}

			if (Parent->ClassGeneratedBy)
			{
				if (Parent->ClassGeneratedBy == Child->ClassGeneratedBy)
				{
					return true;
				}

				if (const UBlueprint* ParentBP = Cast<UBlueprint>(Parent->ClassGeneratedBy))
				{
					if (ParentBP->SkeletonGeneratedClass && Child->IsChildOf(ParentBP->SkeletonGeneratedClass))
					{
						return true;
					}

					if (ParentBP->GeneratedClass && Child->IsChildOf(ParentBP->GeneratedClass))
					{
						return true;
					}
				}
			}
		}
		return false;
	}

	static void FixOverriddenEventSignature(UK2Node_Event* EventNode, UBlueprint* Blueprint, UEdGraph* CurrentGraph)
	{
		check(EventNode && Blueprint && CurrentGraph);
		UClass* CurrentClass = EventNode->GetBlueprintClassFromNode();
		FMemberReference& FuncRef = EventNode->EventReference;
		const FName EventFuncName = FuncRef.GetMemberName();
		ensure(EventFuncName != NAME_None);
		ensure(!EventNode->IsA<UK2Node_CustomEvent>());

		const UFunction* TargetFunction = FuncRef.ResolveMember<UFunction>(CurrentClass);
		const UClass* FuncOwnerClass = FuncRef.GetMemberParentClass(CurrentClass);
		const bool bFunctionOwnerIsNotParentOfClass = !FuncOwnerClass || !ExtendedIsParent(FuncOwnerClass, CurrentClass);
		const bool bNeedsToBeFixed = !TargetFunction || bFunctionOwnerIsNotParentOfClass;
		if (bNeedsToBeFixed)
		{
			const UClass* SuperClass = CurrentClass->GetSuperClass();
			const UFunction* ActualTargetFunction = SuperClass ? SuperClass->FindFunctionByName(EventFuncName) : nullptr;
			if (ActualTargetFunction)
			{
				ensure(TargetFunction != ActualTargetFunction);
				if (!ensure(!TargetFunction || TargetFunction->IsSignatureCompatibleWith(ActualTargetFunction)))
				{
					UE_LOGF(LogBlueprint, Error
						, "FixOverriddenEventSignature function \"%ls\" is not compatible with \"%ls\" node \"%ls\""
						, *GetPathNameSafe(ActualTargetFunction), *GetPathNameSafe(TargetFunction), *GetPathNameSafe(EventNode));
				}

				ensure(GetDefault<UEdGraphSchema_K2>()->FunctionCanBePlacedAsEvent(ActualTargetFunction));
				FuncRef.SetFromField<UFunction>(ActualTargetFunction, false);

				// Emit something to the log to indicate that we've made a change
				FFormatNamedArguments Args;
				Args.Add(TEXT("NodeTitle"), EventNode->GetNodeTitle(ENodeTitleType::ListView));
				Args.Add(TEXT("EventNodeName"), FText::FromString(EventNode->GetName()));
				Blueprint->Message_Note(FText::Format(LOCTEXT("EventSignatureFixed_Note", "{NodeTitle} ({EventNodeName}) had an invalid function signature - it has now been fixed."), Args).ToString());
			}
			else
			{
				TSet<FName> DummyExtraNameList;
				UEdGraphNode* CustomEventNode = CurrentGraph->GetSchema()->CreateSubstituteNode(EventNode, CurrentGraph, nullptr, DummyExtraNameList);
				if (ensure(CustomEventNode))
				{
					// Destroy the old event node (this will also break all pin links and remove it from the graph)
					EventNode->DestroyNode();
					// Add the new custom event node to the graph
					CurrentGraph->Nodes.Add(CustomEventNode);
					// Emit something to the log to indicate that we've made a change
					FFormatNamedArguments Args;
					Args.Add(TEXT("NodeTitle"), EventNode->GetNodeTitle(ENodeTitleType::ListView));
					Args.Add(TEXT("EventNodeName"), FText::FromString(EventNode->GetName()));
					Blueprint->Message_Note(FText::Format(LOCTEXT("EventNodeReplaced_Note", "{NodeTitle} ({EventNodeName}) was not valid for this Blueprint - it has been converted to a custom event."), Args).ToString());
				}
			}
		}
	}
}

// Makes sure that all events we handle exist, and replace with custom events if not
void FBlueprintEditorUtils::ConformImplementedEvents(UBlueprint* Blueprint)
{
	check(nullptr != Blueprint);

	// Collect all implemented interface classes
	TArray<UClass*> ImplementedInterfaceClasses;
	FindImplementedInterfaces(Blueprint, true, ImplementedInterfaceClasses);

	TArray<UEdGraph*> AllGraphs;
	Blueprint->GetAllGraphs(AllGraphs);
	for(int GraphIndex = 0; GraphIndex < AllGraphs.Num(); ++GraphIndex)
	{
		UEdGraph* CurrentGraph = AllGraphs[GraphIndex];
		check(nullptr != CurrentGraph);

		// Make sure the graph is loaded
		if(!CurrentGraph->HasAnyFlags(RF_NeedLoad|RF_NeedPostLoad))
		{
			TArray<UK2Node_Event*> EventNodes;
			CurrentGraph->GetNodesOfClass<UK2Node_Event>(EventNodes);

			// For each event node in the graph
			for(int EventNodeIndex = 0; EventNodeIndex < EventNodes.Num(); ++EventNodeIndex)
			{
				UK2Node_Event* EventNode = EventNodes[EventNodeIndex];
				check(nullptr != EventNode);

				// If the event is loaded and is not a custom event
				if(!EventNode->HasAnyFlags(RF_NeedLoad|RF_NeedPostLoad) && EventNode->bOverrideFunction)
				{
					UClass* EventClass = EventNode->EventReference.GetMemberParentClass(EventNode->GetBlueprintClassFromNode());
					bool bEventNodeUsedByInterface = false;
					int32 Idx = 0;
					while (Idx != ImplementedInterfaceClasses.Num() && bEventNodeUsedByInterface == false)
					{
						const UClass* CurrentInterface = ImplementedInterfaceClasses[Idx];
						while (CurrentInterface)
						{
							if (EventClass == CurrentInterface )
							{
								bEventNodeUsedByInterface = true;
								break;
							}
							CurrentInterface = CurrentInterface->GetSuperClass();
						}
						++Idx;
					}
					if (Blueprint->GeneratedClass && !bEventNodeUsedByInterface)
					{
						FixOverriddenEventSignature(EventNode, Blueprint, CurrentGraph);
					}
				}
			}
		}
	}
}

/** Helper function for ConformImplementedInterfaces */
static void ConformInterfaceByGUID(const UBlueprint* Blueprint, const FBPInterfaceDescription& CurrentInterfaceDesc)
{
	// Conform anim layers before interface graphs as GUIDs may need to be set up in older assets.
	if (const UAnimBlueprint * AnimBlueprint = Cast<UAnimBlueprint>(Blueprint))
	{
		UAnimationGraphSchema::ConformAnimLayersByGuid(AnimBlueprint, CurrentInterfaceDesc);
	}

	// Attempt to conform by GUID if we have a blueprint interface
	// This just make sure that GUID-linked functions preserve their names
	const UBlueprint* InterfaceBlueprint = CastChecked<UBlueprint>(CurrentInterfaceDesc.Interface->ClassGeneratedBy);

	TArray<UEdGraph*> InterfaceGraphs;
	InterfaceBlueprint->GetAllGraphs(InterfaceGraphs);

	TArray<UEdGraph*> BlueprintGraphs;
	Blueprint->GetAllGraphs(BlueprintGraphs);
		
	for (UEdGraph* BlueprintGraph : BlueprintGraphs)
	{
		if(BlueprintGraph != nullptr && BlueprintGraph->InterfaceGuid.IsValid())
		{
			// valid interface Guid found, so fixup name if it is different
			for (const UEdGraph* InterfaceGraph : InterfaceGraphs)
			{
				if(InterfaceGraph != nullptr && InterfaceGraph->GraphGuid == BlueprintGraph->InterfaceGuid && InterfaceGraph->GetFName() != BlueprintGraph->GetFName())
				{
					FBlueprintEditorUtils::RenameGraph(BlueprintGraph, InterfaceGraph->GetFName().ToString());
					FBlueprintEditorUtils::RefreshGraphNodes(BlueprintGraph);
					break;
				}
			}
		}
	}
}

/** Helper function for ConformImplementedInterfaces */
static void ConformInterfaceByName(UBlueprint* Blueprint, FBPInterfaceDescription& CurrentInterfaceDesc, int32 InterfaceIndex, TArray<UK2Node_Event*>& ImplementedEvents, const TArray<FName>& VariableNamesUsedInBlueprint)
{
	// Iterate over all the functions in the interface, and create graphs that are in the interface, but missing in the blueprint
	if (CurrentInterfaceDesc.Interface)
	{
		// a interface could have since been added by the parent (or this blueprint could have been re-parented)
		if (IsInterfaceImplementedByParent(CurrentInterfaceDesc, Blueprint))
		{			
			// have to remove the interface before we promote it (in case this method is reentrant)
			FBPInterfaceDescription LocalInterfaceCopy = CurrentInterfaceDesc;
			Blueprint->ImplementedInterfaces.RemoveAt(InterfaceIndex, 1);

			// in this case, the interface needs to belong to the parent and not this
			// blueprint (we would have been prevented from getting in this state if we
			// had started with a parent that implemented this interface initially)
			PromoteInterfaceImplementationToOverride(LocalInterfaceCopy, Blueprint);
			return;
		}

		// check to make sure that there aren't any interface methods that we originally 
		// implemented as events, but have since switched to functions 
		TSet<FName> ExtraNameList;
		for (UK2Node_Event* EventNode : ImplementedEvents)
		{
			// if this event belongs to something other than this interface
			if (EventNode->EventReference.GetMemberParentClass(EventNode->GetBlueprintClassFromNode()) != CurrentInterfaceDesc.Interface)
			{
				continue;
			}

			UFunction* InterfaceFunction = EventNode->EventReference.ResolveMember<UFunction>(FBlueprintEditorUtils::GetMostUpToDateClass(CurrentInterfaceDesc.Interface));
			// if the function is still ok as an event, no need to try and fix it up
			if (UEdGraphSchema_K2::FunctionCanBePlacedAsEvent(InterfaceFunction))
			{
				continue;
			}

			UEdGraph* EventGraph = EventNode->GetGraph();
			// we've already implemented this interface function as an event (which we need to replace)
			UK2Node_CustomEvent* CustomEventNode = Cast<UK2Node_CustomEvent>(EventGraph->GetSchema()->CreateSubstituteNode(EventNode, EventGraph, nullptr, ExtraNameList));
			if (CustomEventNode == nullptr)
			{
				continue;
			}

			// grab the function's name before we delete the node
			FName const FunctionName = EventNode->EventReference.GetMemberName();
			// destroy the old event node (this will also break all pin links and remove it from the graph)
			EventNode->DestroyNode();

			if (InterfaceFunction)
			{
				// have to rename so it doesn't conflict with the graph we're about to add
				CustomEventNode->RenameCustomEventCloseToName();
			}
			EventGraph->Nodes.Add(CustomEventNode);

			// warn the user that their old functionality won't work (it's now connected 
			// to a custom node that isn't triggered anywhere)
			FText WarningMessageText;
			if (InterfaceFunction)
			{
				WarningMessageText = FText::Format(
					LOCTEXT("InterfaceEventNodeReplaced_WarnFmt", "'{0}' was promoted from an event to a function - it has been replaced by a custom event, which won't trigger unless you call it manually."),
					FText::FromName(FunctionName)
				);
			}
			else
			{
				WarningMessageText = FText::Format(
					LOCTEXT("InterfaceEventRemovedNodeReplaced_WarnFmt", "'{0}' was removed from its interface - it has been replaced by a custom event, which won't trigger unless you call it manually."),
					FText::FromName(FunctionName)
				);
			}

			Blueprint->Message_Warn(WarningMessageText.ToString());
		}

		// Cache off the graph names for this interface, for easier searching
		TMap<FName, UEdGraph*> InterfaceFunctionGraphs;
		for (int32 GraphIndex = 0; GraphIndex < CurrentInterfaceDesc.Graphs.Num(); GraphIndex++)
		{
			UEdGraph* CurrentGraph = CurrentInterfaceDesc.Graphs[GraphIndex];
			if( CurrentGraph )
			{
				InterfaceFunctionGraphs.Add(CurrentGraph->GetFName()) = CurrentGraph;
			}
		}

		// If this is a Blueprint interface, redirect to the skeleton class for function iteration
		const UClass* InterfaceClass = CurrentInterfaceDesc.Interface;
		if (InterfaceClass && InterfaceClass->ClassGeneratedBy)
		{
			InterfaceClass = CastChecked<UBlueprint>(InterfaceClass->ClassGeneratedBy)->SkeletonGeneratedClass;
		}

		// Iterate over all the functions in the interface, and create graphs that are in the interface, but missing in the blueprint
		for (TFieldIterator<UFunction> FunctionIter(InterfaceClass, EFieldIteratorFlags::IncludeSuper); FunctionIter; ++FunctionIter)
		{
			UFunction* Function = *FunctionIter;
			const FName FunctionName = Function->GetFName();
			if(!VariableNamesUsedInBlueprint.Contains(FunctionName))
			{
				if( UEdGraphSchema_K2::CanKismetOverrideFunction(Function) && !UEdGraphSchema_K2::FunctionCanBePlacedAsEvent(Function) )
				{
					if (UEdGraph** FunctionGraphPtr = InterfaceFunctionGraphs.Find(FunctionName))
					{
						const bool bIsConstInterfaceFunction = (Function->FunctionFlags & FUNC_Const) != 0;

						// Sync the 'const' attribute of the implementation with the interface function, in case it has been changed
						TArray<UK2Node_FunctionEntry*> FunctionEntryNodes;
						(*FunctionGraphPtr)->GetNodesOfClass<UK2Node_FunctionEntry>(FunctionEntryNodes);
						for (UK2Node_FunctionEntry* FunctionEntryNode : FunctionEntryNodes)
						{
							const bool bIsImplementedAsConstFunction = (FunctionEntryNode->GetExtraFlags() & FUNC_Const) != 0;
							if (bIsImplementedAsConstFunction != bIsConstInterfaceFunction)
							{
								FunctionEntryNode->Modify();
								if (bIsConstInterfaceFunction)
								{
									FunctionEntryNode->AddExtraFlags(FUNC_Const);
								}
								else
								{
									FunctionEntryNode->ClearExtraFlags(FUNC_Const);
								}
							}
						}
					}
					else
					{
						// interface methods initially create EventGraph stubs, so we need
						// to make sure we remove that entry so the new graph doesn't conflict (don't
						// worry, these are regenerated towards the end of a compile)
						for (UEdGraph* GraphStub : Blueprint->EventGraphs)
						{
							if (GraphStub->GetFName() == FunctionName)
							{
								FBlueprintEditorUtils::RemoveGraph(Blueprint, GraphStub, EGraphRemoveFlags::MarkTransient);
							}
						}

						// Check to see if we already have implemented 
						UEdGraph* NewGraph = FBlueprintEditorUtils::CreateNewGraph(Blueprint, FunctionName, UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass());
						NewGraph->bAllowDeletion = false;
						NewGraph->InterfaceGuid = FBlueprintEditorUtils::FindInterfaceFunctionGuid(Function, CurrentInterfaceDesc.Interface);
						CurrentInterfaceDesc.Graphs.Add(NewGraph);

						FBlueprintEditorUtils::AddInterfaceGraph(Blueprint, NewGraph, CurrentInterfaceDesc.Interface);
					}
				}
				else if(Function->HasMetaData(FBlueprintMetadata::MD_AnimBlueprintFunction))
				{
					if (UEdGraph** FunctionGraphPtr = InterfaceFunctionGraphs.Find(FunctionName))
					{
						UAnimationGraphSchema::ConformAnimGraphToInterface(Blueprint, *(*FunctionGraphPtr), Function);
					}
					// We perform the check here to avoid creating a graph if it isnt implemented in the full interface (note not the skeleton interface that we are iterating over)
					// this is to avoid creating it then removing the graph below if it isnt present in the full class, which will cause a name conflict second time around
					else if(FindUField<UFunction>(CurrentInterfaceDesc.Interface, FunctionName))
					{
						UEdGraph* NewGraph = FBlueprintEditorUtils::CreateNewGraph(Blueprint, FunctionName, UAnimationGraph::StaticClass(), UAnimationGraphSchema::StaticClass());
						NewGraph->bAllowDeletion = false;
						NewGraph->InterfaceGuid = FBlueprintEditorUtils::FindInterfaceFunctionGuid(Function, CurrentInterfaceDesc.Interface);
						CurrentInterfaceDesc.Graphs.Add(NewGraph);

						FBlueprintEditorUtils::AddInterfaceGraph(Blueprint, NewGraph, CurrentInterfaceDesc.Interface);
					}
				}
			}
			else
			{
				Blueprint->Status = BS_Error;
				const FString NewError = FText::Format(
					LOCTEXT("InterfaceNameCollision_ErrorFmt", "Interface name collision in blueprint: {0}, interface: {1}, name: {2}"),
					FText::FromString(Blueprint->GetFullName()),
					FText::FromString(CurrentInterfaceDesc.Interface->GetFullName()),
					FText::FromName(FunctionName)
				).ToString();
				Blueprint->Message_Error(NewError);
			}
		}

		// Iterate over all the graphs in the blueprint interface, and remove ones that no longer have functions 
		for (int32 GraphIndex = 0; GraphIndex < CurrentInterfaceDesc.Graphs.Num(); GraphIndex++)
		{
			// If we can't find the function associated with the graph, delete it
			UEdGraph* CurrentGraph = CurrentInterfaceDesc.Graphs[GraphIndex];

			if (!CurrentGraph || !FindUField<UFunction>(CurrentInterfaceDesc.Interface, CurrentGraph->GetFName()))
			{
				if(CurrentGraph)
				{
					CurrentGraph->GetSchema()->HandleGraphBeingDeleted(*CurrentGraph);

					// rename to free up the graph's name.. which may be needed by an inherited function
					// alternatively we could move this into the functions list?
					// REN_AllowPackageLinkerMismatch is passed to not have to force load completion before renaming
					CurrentGraph->Rename(
						nullptr,
						CurrentGraph->GetOuter(),
						REN_DoNotDirty | REN_DontCreateRedirectors | (Blueprint->bIsRegeneratingOnLoad ? REN_AllowPackageLinkerMismatch : 0));
					// removing from root, standalone, and public is defensive to make sure it is not saved:
					CurrentGraph->ClearFlags(RF_Standalone | RF_Public);
					CurrentGraph->RemoveFromRoot();
					// MarkAsGarbage could be used here, which would nicely trigger tab manager cleanup, but atm
					// use of MarkAsGarbage is causing SGraphPanel to have reference's nulled out (treated as weak)
					// by the GC. For now, I'm just going to flag this as no longer editable. This isn't a bad
					// out come as if the user has anything in the graph they might be able to copy it out
					CurrentGraph->bEditable = false;
				}
				
				CurrentInterfaceDesc.Graphs.RemoveAt(GraphIndex, 1);
				GraphIndex--;
			}
		}
	}
}

// Makes sure that all graphs for all interfaces we implement exist, and add if not
void FBlueprintEditorUtils::ConformImplementedInterfaces(UBlueprint* Blueprint)
{
	check(nullptr != Blueprint);
	FString ErrorStr;

	// Collect all variables names in current blueprint 
	TArray<FName> VariableNamesUsedInBlueprint;
	for (TFieldIterator<FProperty> VariablesIter(Blueprint->GeneratedClass); VariablesIter; ++VariablesIter)
	{
		VariableNamesUsedInBlueprint.Add(VariablesIter->GetFName());
	}
	for (const FBPVariableDescription& NewVariable : Blueprint->NewVariables)
	{
		VariableNamesUsedInBlueprint.AddUnique(NewVariable.VarName);
	}

	TArray<UEdGraph*> AllGraphs;
	Blueprint->GetAllGraphs(AllGraphs);
	// collect all existing event nodes, so we can find interface events that 
	// need to be converted to function graphs
	TArray<UK2Node_Event*> PotentialInterfaceEvents;
	for (UEdGraph const* Graph : AllGraphs)
	{
		UClass* InterfaceEventClass = UK2Node_Event::StaticClass();
		for (UEdGraphNode* GraphNode : Graph->Nodes)
		{
			// interface event nodes are only ever going to be implemented as 
			// explicit UK2Node_Events... using == instead of IsChildOf<> 
			// guarantees that we won't be catching any special node types that
			// users might have made (that maybe reference interface functions too)
			if (GraphNode && (GraphNode->GetClass() == InterfaceEventClass))
			{
				PotentialInterfaceEvents.Add(CastChecked<UK2Node_Event>(GraphNode));
			}
		}
	}
	for (int32 InterfaceIndex = 0; InterfaceIndex < Blueprint->ImplementedInterfaces.Num(); )
	{
		FBPInterfaceDescription& CurrentInterface = Blueprint->ImplementedInterfaces[InterfaceIndex];
		if (!CurrentInterface.Interface)
		{
			Blueprint->Status = BS_Error;
			Blueprint->ImplementedInterfaces.RemoveAt(InterfaceIndex, 1);
			continue;
		}

		// conform functions linked by Guids first
		if(CurrentInterface.Interface->ClassGeneratedBy != nullptr && CurrentInterface.Interface->ClassGeneratedBy->IsA(UBlueprint::StaticClass()))
		{
			ConformInterfaceByGUID(Blueprint, CurrentInterface);
		}

		// now try to conform by name/signature
		ConformInterfaceByName(Blueprint, CurrentInterface, InterfaceIndex, PotentialInterfaceEvents, VariableNamesUsedInBlueprint);

		// not going to remove this interface, so let's continue forward
		++InterfaceIndex;
	}
}

void FBlueprintEditorUtils::ConformDelegateSignatureGraphs(UBlueprint* Blueprint)
{
	Blueprint->DelegateSignatureGraphs.SetNum( Algo::RemoveIf(Blueprint->DelegateSignatureGraphs, 
		[Blueprint](UEdGraph* Graph)
		{
			const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(Blueprint, Graph->GetFName());
			return VarIndex == INDEX_NONE;
		}
	));
}

void FBlueprintEditorUtils::ConformAllowDeletionFlag(UBlueprint* Blueprint)
{
	for (UEdGraph* Graph : Blueprint->FunctionGraphs)
	{
		if (Graph->GetFName() != UEdGraphSchema_K2::FN_UserConstructionScript && Graph->GetFName() != UEdGraphSchema_K2::GN_AnimGraph)
		{
			Graph->bAllowDeletion = true;
		}
	}
}

/** Handle old Anim Blueprints (state machines in the wrong position, transition graphs with the wrong schema, etc...) */
void FBlueprintEditorUtils::UpdateOutOfDateAnimBlueprints(UBlueprint* InBlueprint)
{
	if (UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(InBlueprint))
	{
		// Ensure all transition graphs have the correct schema
		TArray<UAnimStateTransitionNode*> TransitionNodes;
		GetAllNodesOfClass<UAnimStateTransitionNode>(AnimBlueprint, /*out*/ TransitionNodes);
		for (UAnimStateTransitionNode* Node : TransitionNodes)
		{
			UEdGraph* TestGraph = Node->BoundGraph;
			if (TestGraph->Schema == UAnimationGraphSchema::StaticClass())
			{
				TestGraph->Schema = UAnimationTransitionSchema::StaticClass();
			}
		}

		// Handle a reparented anim blueprint that either needs or no longer needs an anim graph
		if(AnimBlueprint->BlueprintType != BPTYPE_Interface)
		{
			if (UAnimBlueprint::FindRootAnimBlueprint(AnimBlueprint) == nullptr)
			{
				// Add an anim graph if not present
				if (FindObject<UEdGraph>(AnimBlueprint, *(UEdGraphSchema_K2::GN_AnimGraph.ToString())) == nullptr)
				{
					UEdGraph* NewGraph = FBlueprintEditorUtils::CreateNewGraph(AnimBlueprint, UEdGraphSchema_K2::GN_AnimGraph, UAnimationGraph::StaticClass(), UAnimationGraphSchema::StaticClass());
					FBlueprintEditorUtils::AddDomainSpecificGraph(AnimBlueprint, NewGraph);
					AnimBlueprint->LastEditedDocuments.Add(NewGraph);
					NewGraph->bAllowDeletion = false;
				}
			}
			else
			{
				// Remove an anim graph if present
				for (int32 i = 0; i < AnimBlueprint->FunctionGraphs.Num(); ++i)
				{
					UEdGraph* FuncGraph = AnimBlueprint->FunctionGraphs[i];
					if ((FuncGraph != nullptr) && (FuncGraph->GetFName() == UEdGraphSchema_K2::GN_AnimGraph))
					{
						UE_LOGF(LogBlueprint, Log, "!!! Removing AnimGraph from %ls, because it has a parent anim blueprint that defines the AnimGraph", *AnimBlueprint->GetPathName());
						AnimBlueprint->FunctionGraphs.RemoveAt(i);
						break;
					}
				}
			}
		}
	}
}

void FBlueprintEditorUtils::UpdateOutOfDateCompositeNodes(UBlueprint* Blueprint)
{
	for (UEdGraph* UbergraphPage : Blueprint->UbergraphPages)
	{
		UpdateOutOfDateCompositeWithOuter(Blueprint, UbergraphPage);
	}
	for (UEdGraph* FunctionGraph : Blueprint->FunctionGraphs)
	{
		UpdateOutOfDateCompositeWithOuter(Blueprint, FunctionGraph);
	}
}

void FBlueprintEditorUtils::UpdateOutOfDateCompositeWithOuter(UBlueprint* Blueprint, UEdGraph* OuterGraph )
{
	check(OuterGraph != nullptr);
	check(FindBlueprintForGraphChecked(OuterGraph) == Blueprint);

	for (UEdGraphNode* Node : OuterGraph->Nodes)
	{
		//Is this node of a type that has a BoundGraph to update
		for(UEdGraph* BoundGraph : Node->GetSubGraphs())
		{
			if (BoundGraph)
			{
				// Check for out of date BoundGraph where outer is not the composite node
				if (BoundGraph->GetOuter() != Node)
				{
					// change the outer of the BoundGraph to be the composite node instead of the OuterGraph
					if (false == BoundGraph->Rename(*BoundGraph->GetName(), Node, (BoundGraph->HasAnyFlags(RF_NeedLoad | RF_NeedPostLoad) ? REN_AllowPackageLinkerMismatch : 0) | REN_DontCreateRedirectors))
					{
						UE_LOGF(LogBlueprintDebug, Log, "CompositeNode: On Blueprint '%ls' could not fix Outer() for BoundGraph of composite node '%ls'", *Blueprint->GetPathName(), *Node->GetName());
					}
				}
			}
		}
	}

	for (UEdGraph* SubGraph : OuterGraph->SubGraphs)
	{
		UpdateOutOfDateCompositeWithOuter(Blueprint, SubGraph);
	}
}

/** Ensure all component templates are in use */
void FBlueprintEditorUtils::UpdateComponentTemplates(UBlueprint* Blueprint)
{
	TArray<UActorComponent*> ReferencedTemplates;

	TArray<UK2Node_AddComponent*> AllComponents;
	FBlueprintEditorUtils::GetAllNodesOfClass(Blueprint, AllComponents);

	for (UK2Node_AddComponent* ComponentNode : AllComponents)
	{
		check(ComponentNode);

		UActorComponent* ActorComp = ComponentNode->GetTemplateFromNode();
		if (ActorComp)
		{
			ensure(Blueprint->ComponentTemplates.Contains(ActorComp));

			// fix up AddComponent nodes that don't have their own unique template objects
			if (ReferencedTemplates.Contains(ActorComp))
			{
				UE_LOGF(LogBlueprint, Warning,
					"Blueprint '%ls' has an AddComponent node '%ls' with a non-unique component template name (%ls). Moving it to a new template object with a unique name. Re-save the Blueprint to remove this warning on the next load.",
					*Blueprint->GetPathName(), *ComponentNode->GetPathName(), *ActorComp->GetName());

				ComponentNode->MakeNewComponentTemplate();
				ActorComp = ComponentNode->GetTemplateFromNode();
			}

			// fix up existing content to be sure these are flagged as archetypes and are transactional
			ActorComp->SetFlags(RF_ArchetypeObject|RF_Transactional);	
			ReferencedTemplates.Add(ActorComp);
		}
	}
	Blueprint->ComponentTemplates.Empty();
	Blueprint->ComponentTemplates.Append(ReferencedTemplates);
}

/** Ensures that the CDO root component reference is valid for Actor-based Blueprints */
void FBlueprintEditorUtils::UpdateRootComponentReference(UBlueprint* Blueprint)
{
	// The CDO's root component reference should match that of its parent class
	if(Blueprint && Blueprint->ParentClass && Blueprint->GeneratedClass)
	{
		AActor* ParentActorCDO = Cast<AActor>(Blueprint->ParentClass->GetDefaultObject(false));
		AActor* BlueprintActorCDO = Cast<AActor>(Blueprint->GeneratedClass->GetDefaultObject(false));
		if(ParentActorCDO && BlueprintActorCDO)
		{
			// If both CDOs are valid, check for a valid scene root component
			USceneComponent* ParentSceneRootComponent = ParentActorCDO->GetRootComponent();
			USceneComponent* BlueprintSceneRootComponent = BlueprintActorCDO->GetRootComponent();
			if((ParentSceneRootComponent == nullptr && BlueprintSceneRootComponent != nullptr)
				|| (ParentSceneRootComponent != nullptr && BlueprintSceneRootComponent == nullptr)
				|| (ParentSceneRootComponent != nullptr && BlueprintSceneRootComponent != nullptr && ParentSceneRootComponent->GetFName() != BlueprintSceneRootComponent->GetFName()))
			{
				// If the parent CDO has a valid scene root component
				if(ParentSceneRootComponent != nullptr)
				{
					// Search for a scene component with the same name in the Blueprint CDO's Components list
					TInlineComponentArray<USceneComponent*> SceneComponents;
					BlueprintActorCDO->GetComponents(SceneComponents);
					for(int i = 0; i < SceneComponents.Num(); ++i)
					{
						USceneComponent* SceneComp = SceneComponents[i];
						if(SceneComp && SceneComp->GetFName() == ParentSceneRootComponent->GetFName())
						{
							// We found a match, so make this the new scene root component
							BlueprintActorCDO->SetRootComponent(SceneComp);
							break;
						}
					}
				}
				else if(BlueprintSceneRootComponent != nullptr)
				{
					// The parent CDO does not have a valid scene root, so NULL out the Blueprint CDO reference to match
					BlueprintActorCDO->SetRootComponent(nullptr);
				}
			}
		}
	}
}

bool FBlueprintEditorUtils::IsSCSComponentProperty(FObjectProperty* MemberProperty)
{
	if (!MemberProperty->PropertyClass->IsChildOf<UActorComponent>())
	{
		return false;
	}


	UClass* OwnerClass = MemberProperty->GetOwnerClass();
	UBlueprintGeneratedClass* BpClassOwner = Cast<UBlueprintGeneratedClass>(OwnerClass);

	if (BpClassOwner == nullptr)
	{
		// if this isn't directly a blueprint property, then we check if it is a 
		// associated with a natively added component (which would still be  
		// accessible through the SCS tree)

		if ((OwnerClass == nullptr) || !OwnerClass->IsChildOf<AActor>())
		{
			return false;
		}
		else if (const AActor* ActorCDO = GetDefault<AActor>(OwnerClass))
		{
			const void* PropertyAddress = MemberProperty->ContainerPtrToValuePtr<void>(ActorCDO);
			UObject* PropertyValue = MemberProperty->GetObjectPropertyValue(PropertyAddress);

			for (UActorComponent* Component : ActorCDO->GetComponents())
			{
				if (Component && Component->GetClass()->IsChildOf(MemberProperty->PropertyClass))
				{
					if (PropertyValue == Component)
					{
						return true;
					}
				}
			}
		}
		return false;
	}

	FMemberReference MemberRef;
	MemberRef.SetFromField<FProperty>(MemberProperty, /*bIsConsideredSelfContext =*/false);
	bool const bIsGuidValid = MemberRef.GetMemberGuid().IsValid();

	if (BpClassOwner->SimpleConstructionScript != nullptr)
	{
		TArray<USCS_Node*> SCSNodes = BpClassOwner->SimpleConstructionScript->GetAllNodes();
		for (USCS_Node* ScsNode : SCSNodes)
		{
			if (bIsGuidValid && ScsNode->VariableGuid.IsValid())
			{
				if (ScsNode->VariableGuid == MemberRef.GetMemberGuid())
				{
					return true;
				}
			}
			else if (ScsNode->GetVariableName() == MemberRef.GetMemberName())
			{
				return true;
			}
		}
	}
	return false;
}

UActorComponent* FBlueprintEditorUtils::FindUCSComponentTemplate(const FComponentKey& ComponentKey, const FName& TemplateName)
{
	UActorComponent* FoundTemplate = nullptr;
	if (ComponentKey.IsValid() && ComponentKey.IsUCSKey())
	{
		if (UBlueprint* Blueprint = Cast<UBlueprint>(ComponentKey.GetComponentOwner()->ClassGeneratedBy))
		{
			if (UEdGraph* UCSGraph = FBlueprintEditorUtils::FindUserConstructionScript(Blueprint))
			{
				TArray<UK2Node_AddComponent*> ComponentNodes;
				UCSGraph->GetNodesOfClass<UK2Node_AddComponent>(ComponentNodes);

				for (UK2Node_AddComponent* UCSNode : ComponentNodes)
				{
					if (UCSNode->NodeGuid == ComponentKey.GetAssociatedGuid())
					{
						FoundTemplate = UCSNode->GetTemplateFromNode();
						break;
					}
				}
			}
		}
		else if(UBlueprintGeneratedClass* BPGC = Cast<UBlueprintGeneratedClass>(ComponentKey.GetComponentOwner()))
		{
			for (UActorComponent* ComponentTemplate : BPGC->ComponentTemplates)
			{
				if (ComponentTemplate->GetFName().IsEqual(TemplateName))
				{
					FoundTemplate = ComponentTemplate;
					break;
				}
			}
		}
	}
	return FoundTemplate;
}

/** Temporary fix for cut-n-paste error that failed to carry transactional flags */
void FBlueprintEditorUtils::UpdateTransactionalFlags(UBlueprint* Blueprint)
{
	TArray<UK2Node*> AllNodes;
	FBlueprintEditorUtils::GetAllNodesOfClass(Blueprint, AllNodes);

	for (UK2Node* K2Node : AllNodes)
	{
		check(K2Node);

		if (!K2Node->HasAnyFlags(RF_Transactional))
		{
			K2Node->SetFlags(RF_Transactional);
			Blueprint->Status = BS_Dirty;
		}
	}
}

void FBlueprintEditorUtils::UpdateStalePinWatches( UBlueprint* Blueprint )
{
	TSet<FBlueprintWatchedPin> AllPins;
	uint16 WatchCount = 0;
	
	// Find all unique pins being watched
	FKismetDebugUtilities::ForeachPinWatch(
		Blueprint,
		[&AllPins, &WatchCount](const FBlueprintWatchedPin& WatchedPin)
		{
			++WatchCount;
			UEdGraphPin* Pin = WatchedPin.Get();
			if (Pin == nullptr)
			{
				return; // ~continue
			}

			UEdGraphNode* OwningNode = Pin->GetOwningNode();
			// during node reconstruction, dead pins get moved to the transient 
			// package (so just in case this blueprint got saved with dead pin watches)
			if (OwningNode == nullptr)
			{
				return; // ~continue
			}

			if (!OwningNode->Pins.Contains(Pin))
			{
				return; // ~continue
			}

			AllPins.Add(WatchedPin);
		}
	);

	// Refresh watched pins with unique pins (throw away null or duplicate watches)
	if (WatchCount != AllPins.Num())
	{
		FKismetDebugUtilities::ClearPinWatches(Blueprint);
		for (FBlueprintWatchedPin& WatchedPin : AllPins)
		{
			FKismetDebugUtilities::AddPinWatch(Blueprint, MoveTemp(WatchedPin));
		}
	}
}

void FBlueprintEditorUtils::ClearMacroCosmeticInfoCache(UBlueprint* Blueprint)
{
	Blueprint->PRIVATE_CachedMacroInfo.Reset();
}

FBlueprintMacroCosmeticInfo FBlueprintEditorUtils::GetCosmeticInfoForMacro(UEdGraph* MacroGraph)
{
	if (UBlueprint* MacroOwnerBP = FBlueprintEditorUtils::FindBlueprintForGraph(MacroGraph))
	{
		checkSlow(MacroGraph->GetSchema()->GetGraphType(MacroGraph) == GT_Macro);
		
		// See if it's in the cache
		if (FBlueprintMacroCosmeticInfo* pCosmeticInfo = MacroOwnerBP->PRIVATE_CachedMacroInfo.Find(MacroGraph))
		{
			return *pCosmeticInfo;
		}
		else
		{
			FBlueprintMacroCosmeticInfo& CosmeticInfo = MacroOwnerBP->PRIVATE_CachedMacroInfo.Add(MacroGraph);
			CosmeticInfo.bContainsLatentNodes = FBlueprintEditorUtils::CheckIfGraphHasLatentFunctions(MacroGraph);

			return CosmeticInfo;
		}
	}

	return FBlueprintMacroCosmeticInfo();
}

void FBlueprintEditorUtils::ReplaceInvalidBlueprintNameCharacters(FString& InBaseName)
{
	for (TCHAR& TestChar : InBaseName)
	{
		for (TCHAR BadChar : UE_BLUEPRINT_INVALID_NAME_CHARACTERS)
		{
			if (TestChar == BadChar)
			{
				TestChar = TEXT('_');
				break;
			}
		}
	}
}

FName FBlueprintEditorUtils::FindUniqueKismetName(const UBlueprint* InBlueprint, const FString& InBaseName, UStruct* InScope/* = nullptr*/)
{
	int32 Count = 0;
	// If an empty string is given then we need to give a valid backup
	static const FString BackupKismetName = TEXT("K2Name");
	FString BaseName = InBaseName.IsEmpty() ? BackupKismetName : InBaseName;
	FString KismetName = InBaseName;
	TSharedPtr<FKismetNameValidator> NameValidator = MakeShareable(new FKismetNameValidator(InBlueprint, NAME_None, InScope));

	EValidatorResult Result = NameValidator->IsValid(KismetName);

	// Clean up BaseName to not contain any invalid characters, which will mean we can never find a legal name no matter how many numbers we add
	if (Result == EValidatorResult::ContainsInvalidCharacters)
	{
		ReplaceInvalidBlueprintNameCharacters(BaseName);
		KismetName = BaseName;
		Result = NameValidator->IsValid(KismetName);
	}

	while(Result != EValidatorResult::Ok)
	{
		// Calculate the number of digits in the number, adding 2 (1 extra to correctly count digits, another to account for the '_' that will be added to the name
		int32 CountLength = Count > 0? (int32)log((double)Count) + 2 : 2;

		// If the length of the final string will be too long, cut off the end so we can fit the number
		if(CountLength + BaseName.Len() > NameValidator->GetMaximumNameLength())
		{
			BaseName.LeftInline(NameValidator->GetMaximumNameLength() - CountLength, EAllowShrinking::No);
		}
		KismetName = FString::Printf(TEXT("%s_%d"), *BaseName, Count);
		Count++;
		Result = NameValidator->IsValid(KismetName);
	}

	return FName(*KismetName);
}

FName FBlueprintEditorUtils::FindUniqueCustomEventName(const UBlueprint* Blueprint)
{
	return FindUniqueKismetName(Blueprint, LOCTEXT("DefaultCustomEventName", "CustomEvent").ToString());
}

//////////////////////////////////////////////////////////////////////////
// Timeline

FName FBlueprintEditorUtils::FindUniqueTimelineName(const UBlueprint* Blueprint)
{
	return FindUniqueKismetName(Blueprint, LOCTEXT("DefaultTimelineName", "Timeline").ToString());
}

UTimelineTemplate* FBlueprintEditorUtils::AddNewTimeline(UBlueprint* Blueprint, const FName& TimelineVarName)
{
	// Early out if we don't support timelines in this class
	if( !FBlueprintEditorUtils::DoesSupportTimelines(Blueprint) )
	{
		return nullptr;
	}

	// First look to see if we already have a timeline with that name
	UTimelineTemplate* Timeline = Blueprint->FindTimelineTemplateByVariableName(TimelineVarName);
	if (Timeline != nullptr)
	{
		UE_LOGF(LogBlueprint, Log, "AddNewTimeline: Blueprint '%ls' already contains a timeline called '%ls'", *Blueprint->GetPathName(), *TimelineVarName.ToString());
		return nullptr;
	}
	else
	{
		Blueprint->Modify();
		check(nullptr != Blueprint->GeneratedClass);
		// Construct new graph with the supplied name
		const FName TimelineTemplateName = *UTimelineTemplate::TimelineVariableNameToTemplateName(TimelineVarName);
		Timeline = NewObject<UTimelineTemplate>(Blueprint->GeneratedClass, TimelineTemplateName, RF_Transactional);
		Blueprint->Timelines.Add(Timeline);

		// Potentially adjust variable names for any child blueprints
		FBlueprintEditorUtils::ValidateBlueprintChildVariables(Blueprint, TimelineVarName);

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
		return Timeline;
	}
}

void FBlueprintEditorUtils::RemoveTimeline(UBlueprint* Blueprint, UTimelineTemplate* Timeline, bool bDontRecompile)
{
	// Ensure objects are marked modified
	Timeline->Modify();
	Blueprint->Modify();

	Blueprint->Timelines.Remove(Timeline);
	Timeline->MarkAsGarbage();

	if( !bDontRecompile )
	{
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
	}
}

UK2Node_Timeline* FBlueprintEditorUtils::FindNodeForTimeline(UBlueprint* Blueprint, UTimelineTemplate* Timeline)
{
	check(Timeline);
	const FName TimelineVarName = Timeline->GetVariableName();

	TArray<UK2Node_Timeline*> TimelineNodes;
	FBlueprintEditorUtils::GetAllNodesOfClass<UK2Node_Timeline>(Blueprint, TimelineNodes);

	for(int32 i=0; i<TimelineNodes.Num(); i++)
	{
		UK2Node_Timeline* TestNode = TimelineNodes[i];
		if(TestNode->TimelineName == TimelineVarName)
		{
			return TestNode;
		}
	}

	return nullptr; // no node found
}

bool FBlueprintEditorUtils::RenameTimeline(UBlueprint* Blueprint, const FName OldNameRef, const FName NewName)
{
	check(Blueprint);

	bool bRenamed = false;

	// make a copy, in case we alter the value of what is referenced by 
	// OldNameRef through the course of this function
	FName OldName = OldNameRef;

	TSharedPtr<INameValidatorInterface> NameValidator = MakeShareable(new FKismetNameValidator(Blueprint));
	const FString NewTemplateName = UTimelineTemplate::TimelineVariableNameToTemplateName(NewName);
	// NewName should be already validated. But one must make sure that NewTemplateName is also unique.
	const bool bUniqueNameForTemplate = (EValidatorResult::Ok == NameValidator->IsValid(NewTemplateName));

	UTimelineTemplate* Template = Blueprint->FindTimelineTemplateByVariableName(OldName);
	UTimelineTemplate* NewTemplate = Blueprint->FindTimelineTemplateByVariableName(NewName);
	if ((!NewTemplate && bUniqueNameForTemplate) || NewTemplate == Template)
	{
		if (Template)
		{
			Blueprint->Modify();
			Template->Modify();

			if (UK2Node_Timeline* TimelineNode = FindNodeForTimeline(Blueprint, Template))
			{
				TimelineNode->Modify();
				TimelineNode->TimelineName = NewName;
			}

			TArray<UK2Node_Variable*> TimelineVarNodes;
			FBlueprintEditorUtils::GetAllNodesOfClass<UK2Node_Variable>(Blueprint, TimelineVarNodes);
			for(int32 It = 0; It < TimelineVarNodes.Num(); It++)
			{
				UK2Node_Variable* TestNode = TimelineVarNodes[It];
				if(TestNode && (OldName == TestNode->GetVarName()))
				{
					UEdGraphPin* TestPin = TestNode->FindPin(OldName);
					if(TestPin && (UTimelineComponent::StaticClass() == TestPin->PinType.PinSubCategoryObject.Get()))
					{
						TestNode->Modify();
						if(TestNode->VariableReference.IsSelfContext())
						{
							TestNode->VariableReference.SetSelfMember(NewName);
						}
						else
						{
							//TODO:
							UClass* ParentClass = TestNode->VariableReference.GetMemberParentClass((UClass*)nullptr);
							TestNode->VariableReference.SetExternalMember(NewName, ParentClass);
						}
						TestPin->Modify();
						TestPin->PinName = NewName;
					}
				}
			}

			Blueprint->Timelines.Remove(Template);
			
			UObject* ExistingObject = StaticFindObject(nullptr, Template->GetOuter(), *NewTemplateName, EFindObjectFlags::ExactClass);
			if (ExistingObject != Template && ExistingObject != nullptr)
			{
				ExistingObject->Rename(*MakeUniqueObjectName(ExistingObject->GetOuter(), ExistingObject->GetClass(), ExistingObject->GetFName()).ToString());
			}
			Template->Rename(*NewTemplateName, Template->GetOuter(), (Blueprint->bIsRegeneratingOnLoad ? REN_AllowPackageLinkerMismatch : REN_None));
			Blueprint->Timelines.Add(Template);

			// Validate child blueprints and adjust variable names to avoid a potential name collision
			FBlueprintEditorUtils::ValidateBlueprintChildVariables(Blueprint, NewName);

			// Refresh references and flush editors
			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
			bRenamed = true;
		}
	}
	return bRenamed;
}

//////////////////////////////////////////////////////////////////////////
// LevelScriptBlueprint

bool FBlueprintEditorUtils::FindReferencesToActorFromLevelScript(ULevelScriptBlueprint* LevelScriptBlueprint, AActor* InActor, TArray<UK2Node*>& ReferencedToActors)
{
	if (LevelScriptBlueprint == nullptr)
	{
		return false;
	}

	TArray<UEdGraph*> AllGraphs;
	LevelScriptBlueprint->GetAllGraphs(AllGraphs);

	for(TArray<UEdGraph*>::TConstIterator it(AllGraphs); it; ++it)
	{
		const UEdGraph* CurrentGraph = *it;

		TArray<UK2Node*> GraphNodes;
		CurrentGraph->GetNodesOfClass(GraphNodes);

		for(UK2Node* Node : GraphNodes)
		{
			if(Node != nullptr && Node->GetReferencedLevelActor() == InActor )
			{
				ReferencedToActors.Add(Node);
			}
		}
	}

	return ReferencedToActors.Num() > 0;
}

void FBlueprintEditorUtils::ReplaceAllActorRefrences(ULevelScriptBlueprint* InLevelScriptBlueprint, AActor* InOldActor, AActor* InNewActor)
{
	bool bBlueprintModified = false;
	auto BlueprintModify = [&bBlueprintModified, InLevelScriptBlueprint]()
	{
		if (!bBlueprintModified)
		{
			InLevelScriptBlueprint->Modify();
			FBlueprintEditorUtils::MarkBlueprintAsModified(InLevelScriptBlueprint);
			bBlueprintModified = true;
		}
	};

	// Literal nodes are the common "get" type nodes and need to be updated with the new object reference
	TArray< UK2Node_Literal* > LiteralNodes;
	FBlueprintEditorUtils::GetAllNodesOfClass(InLevelScriptBlueprint, LiteralNodes);

	for( UK2Node_Literal* LiteralNode : LiteralNodes )
	{
		if(LiteralNode->GetObjectRef() == InOldActor)
		{
			BlueprintModify();
			LiteralNode->Modify();
			LiteralNode->SetObjectRef(InNewActor);
			LiteralNode->ReconstructNode();
		}
	}

	// Actor Bound Events reference the actors as well and need to be updated
	TArray< UK2Node_ActorBoundEvent* > ActorEventNodes;
	FBlueprintEditorUtils::GetAllNodesOfClass(InLevelScriptBlueprint, ActorEventNodes);

	for( UK2Node_ActorBoundEvent* ActorEventNode : ActorEventNodes )
	{
		if(ActorEventNode->GetReferencedLevelActor() == InOldActor)
		{
			BlueprintModify();
			ActorEventNode->Modify();
			ActorEventNode->EventOwner = InNewActor;
			ActorEventNode->ReconstructNode();
		}
	}
}

void  FBlueprintEditorUtils::ModifyActorReferencedGraphNodes(ULevelScriptBlueprint* LevelScriptBlueprint, const AActor* InActor)
{
	TArray<UEdGraph*> AllGraphs;
	LevelScriptBlueprint->GetAllGraphs(AllGraphs);

	for(TArray<UEdGraph*>::TConstIterator it(AllGraphs); it; ++it)
	{
		const UEdGraph* CurrentGraph = *it;

		TArray<UK2Node*> GraphNodes;
		CurrentGraph->GetNodesOfClass(GraphNodes);

		for( TArray<UK2Node*>::TIterator NodeIt(GraphNodes); NodeIt; ++NodeIt )
		{
			UK2Node* CurrentNode = *NodeIt;
			if( CurrentNode != nullptr && CurrentNode->GetReferencedLevelActor() == InActor )
			{
				CurrentNode->Modify();
			}
		}
	}
}

void FBlueprintEditorUtils::FindActorsThatReferenceActor( AActor* InActor, TArray<UClass*>& InClassesToIgnore, TArray<AActor*>& OutReferencingActors )
{
	// Iterate all actors in the same world as InActor
	for ( FActorIterator ActorIt(InActor->GetWorld()); ActorIt; ++ActorIt )
	{
		AActor* CurrentActor = *ActorIt;
		if (( CurrentActor ) && ( CurrentActor != InActor ))
		{
			bool bShouldIgnore = false;
			// Ignore Actors that aren't in the same level as InActor - cross level references are not allowed, so it's safe to ignore.
			if ( !CurrentActor->IsInLevel(InActor->GetLevel() ) )
			{
				bShouldIgnore = true;
			}
			// Ignore Actors if they are of a type we were instructed to ignore.
			for ( int32 IgnoreIndex = 0; IgnoreIndex < InClassesToIgnore.Num() && !bShouldIgnore; IgnoreIndex++ )
			{
				if ( CurrentActor->IsA( InClassesToIgnore[IgnoreIndex] ) )
				{
					bShouldIgnore = true;
				}
			}

			if ( !bShouldIgnore )
			{
				// Get all references from CurrentActor and see if any are the Actor we're searching for
				TArray<UObject*> References;
				FReferenceFinder Finder( References );
				Finder.FindReferences( CurrentActor );

				if ( References.Contains( InActor ) )
				{
					OutReferencingActors.Add( CurrentActor );
				}
			}
		}
	}
};

class FActorMapReferenceProcessor : public FSimpleReferenceProcessorBase
{
	TArray<UObject*> PotentiallyReferencedActors;
	TMap<AActor*, TArray<AActor*>>& ReferencingActors;
public:
	FActorMapReferenceProcessor(UWorld* InWorld, TArray<UObject*>& OutPotentialReferencerObjects, const TArray<UClass*>& ClassesToIgnore, TMap<AActor*, TArray<AActor*>>& ReferencingActors)
		: ReferencingActors(ReferencingActors)
	{
		// Collect all actors in the world
		for (FActorIterator ActorIt(InWorld); ActorIt; ++ActorIt)
		{
			if (AActor* CurrentActor = *ActorIt)
			{
				bool bShouldIgnore = false;
				// Ignore actors if they belong to a class that's being ignored
				for (UClass* ClassToIgnore : ClassesToIgnore)
				{
					if (CurrentActor->IsA(ClassToIgnore))
					{
						bShouldIgnore = true;
						break;
					}
				}
				OutPotentialReferencerObjects.Add(Cast<UObject>(CurrentActor));

				// Collect all child elements of the actor and add them as potential referencer objects
				TArray<FTypedElementHandle> ChildElementHandles;
				UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();
				TTypedElement<ITypedElementHierarchyInterface> ElementHierarchyHandle = Registry->GetElement<ITypedElementHierarchyInterface>(UEngineElementsLibrary::AcquireEditorActorElementHandle(CurrentActor));
				ElementHierarchyHandle.GetChildElements(ChildElementHandles, true);
				// This is intentionally constructed in a way that will recursively get all child elements, by adding
				// to the array of handles while iterating it. Don't try to change to a range-based loop
				for (int i = 0; i < ChildElementHandles.Num(); ++i)
				{
					FTypedElementHandle ChildElementHandle = ChildElementHandles[i];
					if (TTypedElement<ITypedElementHierarchyInterface> ChildElementHierarchyInterface = Registry->GetElement<ITypedElementHierarchyInterface>(ChildElementHandle))
					{
						ChildElementHierarchyInterface.GetChildElements(ChildElementHandles, true);
					}
					if (TTypedElement<ITypedElementObjectInterface> ChildElementObjectInterface = Registry->GetElement<ITypedElementObjectInterface>(ChildElementHandle))
					{
						if (UObject* ChildObject = ChildElementObjectInterface.GetObject())
						{
							OutPotentialReferencerObjects.Add(ChildObject);
						}
					}
				}
				if (bShouldIgnore)
				{
					continue;
				}
				PotentiallyReferencedActors.Add(CurrentActor);
			}
		}
	}
	FORCEINLINE_DEBUGGABLE void HandleTokenStreamObjectReference(FGCArrayStruct& ObjectsToSerializeStruct, UObject* ReferencingObject, UObject*& Object, UE::GC::FTokenId, EGCTokenType, bool)
	{
		if (!ReferencingObject)
		{
			ReferencingObject = ObjectsToSerializeStruct.GetReferencingObject();
		}
		if (!Object || !ReferencingObject || Object == ReferencingObject || !Object->IsA<AActor>())
		{
			return;
		}

		AActor* Actor = CastChecked<AActor>(Object);
		if (!PotentiallyReferencedActors.Contains(Actor))
		{
			return;
		}
		// Ignore references from child objects
		if (ReferencingObject->IsInOuter(Object))
		{
			return;
		}
		// The object itself if it's an actor, or the actor that contains it (if that exists)
		if (AActor* ReferencingActor = ReferencingObject->IsA<AActor>() ? CastChecked<AActor>(ReferencingObject) : ReferencingObject->GetTypedOuter<AActor>())
		{
			// Don't record more than one reference from the same actor
			ReferencingActors.FindOrAdd(Actor).AddUnique(ReferencingActor);
		}
	}
};

void FBlueprintEditorUtils::GetActorReferenceMap(UWorld* InWorld, TArray<UClass*>& InClassesToIgnore, TMap<AActor*, TArray<AActor*>>& OutReferencingActors)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FBlueprintEditorUtils::GetActorReferenceMap);
	
	TArray<UObject*> InitialObjects;
	FActorMapReferenceProcessor Processor(InWorld, /* out */ InitialObjects, InClassesToIgnore, OutReferencingActors);

	UE::GC::FWorkerContext Context;
	Context.SetInitialObjectsUnpadded(InitialObjects);
	CollectReferences(Processor, Context);
}

void FBlueprintEditorUtils::FixLevelScriptActorBindings(ALevelScriptActor* LevelScriptActor, const ULevelScriptBlueprint* ScriptBlueprint)
{
	if( ScriptBlueprint->BlueprintType != BPTYPE_LevelScript )
	{
		return;
	}

	UPackage* ActorPackage = LevelScriptActor->GetOutermost();
	UPackage* BlueprintPkg = ScriptBlueprint->GetOutermost();
	// the nodes in the Blueprint are going to be bound to actors within the same
	// (level) package, they're the actors in the editor; if LevelScriptActor 
	// doesn't belong to that package, then it is likely a copy (for PIE), this guard 
	// prevents us from cross-binding instantiated (PIE) actors to editor objects
	if (ActorPackage != BlueprintPkg)
	{
		return;
	}

	TArray<UEdGraph*> AllGraphs;
	ScriptBlueprint->GetAllGraphs(AllGraphs);

	// Iterate over all graphs, and find all bound event nodes
	for( TArray<UEdGraph*>::TConstIterator GraphIt(AllGraphs); GraphIt; ++GraphIt )
	{
		TArray<UK2Node_ActorBoundEvent*> BoundEvents;
		(*GraphIt)->GetNodesOfClass(BoundEvents);

		for( TArray<UK2Node_ActorBoundEvent*>::TConstIterator NodeIt(BoundEvents); NodeIt; ++NodeIt )
		{
			UK2Node_ActorBoundEvent* EventNode = *NodeIt;

			// For each bound event node, verify that we have an entry point in the LSA, and add a delegate to the target
			if( EventNode && EventNode->EventOwner )
			{
				const FName TargetFunction = EventNode->CustomFunctionName;

				// Check to make sure the level scripting actor actually has the function defined
				if( LevelScriptActor->FindFunction(TargetFunction) )
				{
					// Grab the MC delegate we need to add to
					FMulticastDelegateProperty* TargetDelegate = EventNode->GetTargetDelegateProperty();
					if( TargetDelegate != nullptr && 
						TargetDelegate->GetOwnerClass() &&
						EventNode->EventOwner->GetClass()->IsChildOf(TargetDelegate->GetOwnerClass()))
					{
						// Create the delegate, and add it if it doesn't already exist
						FScriptDelegate Delegate;
						Delegate.BindUFunction(LevelScriptActor, TargetFunction);
						TargetDelegate->AddDelegate(MoveTemp(Delegate), EventNode->EventOwner);
					}
				}
			}
		}
	}
}

void FBlueprintEditorUtils::ListPackageContents(UPackage* Package, FOutputDevice& Ar)
{
	Ar.Logf(TEXT("Package %s contains:"), *Package->GetName());
	for (FThreadSafeObjectIterator ObjIt; ObjIt; ++ObjIt)
	{
		if (ObjIt->GetOuter() == Package)
		{
			Ar.Logf(TEXT("  %s (flags 0x%X)"), *ObjIt->GetFullName(), (int32)ObjIt->GetFlags());
		}
	}
}

bool FBlueprintEditorUtils::KismetDiagnosticExec(const TCHAR* InStream, FOutputDevice& Ar)
{
	const TCHAR* Str = InStream;

	if (FParse::Command(&Str, TEXT("FindBadBlueprintReferences")))
	{
		// Collect garbage first to remove any false positives
		CollectGarbage( GARBAGE_COLLECTION_KEEPFLAGS );

		UPackage* TransientPackage = GetTransientPackage();

		// Unique blueprints/classes that contain badness
		TSet<UObject*> ObjectsContainingBadness;
		TSet<UPackage*> BadPackages;

		// Run thru every object in the world
		for (FThreadSafeObjectIterator ObjectIt; ObjectIt; ++ObjectIt)
		{
			UObject* TestObj = *ObjectIt;

			// If the test object is itself transient, there is no concern
			CA_SUPPRESS(28182); // https://connect.microsoft.com/VisualStudio/feedback/details/3082622
			if (TestObj->HasAnyFlags(RF_Transient))
			{
				continue;
			}


			// Look for a containing scope (either a blueprint or a class)
			UObject* OuterScope = nullptr;
			for (UObject* TestOuter = TestObj; (TestOuter != nullptr) && (OuterScope == nullptr); TestOuter = TestOuter->GetOuter())
			{
				if (UClass* OuterClass = Cast<UClass>(TestOuter))
				{
					if (OuterClass->ClassGeneratedBy != nullptr)
					{
						OuterScope = OuterClass;
					}
				}
				else if (UBlueprint* OuterBlueprint = Cast<UBlueprint>(TestOuter))
				{
					OuterScope = OuterBlueprint;
				}
			}

			if ((OuterScope != nullptr) && !OuterScope->IsIn(TransientPackage))
			{
				// Find all references
				TArray<UObject*> ReferencedObjects;
				FReferenceFinder ObjectReferenceCollector(ReferencedObjects, nullptr, false, false, false, false);
				ObjectReferenceCollector.FindReferences(TestObj);

				for (UObject* ReferencedObj : ReferencedObjects)
				{
					// Ignore any references inside the outer blueprint or class; they're intrinsically safe
					if (!ReferencedObj->IsIn(OuterScope))
					{
						// If it's a public reference, that's fine
						if (!ReferencedObj->HasAnyFlags(RF_Public))
						{
							// It's a private reference outside of the parent object; not good!
							Ar.Logf(TEXT("%s has a reference to %s outside of it's container %s"),
								*TestObj->GetFullName(),
								*ReferencedObj->GetFullName(),
								*OuterScope->GetFullName()
								);
							ObjectsContainingBadness.Add(OuterScope);
							BadPackages.Add(OuterScope->GetOutermost());
						}
					}
				}
			}
		}

		// Report all the bad outers as text dumps so the exact property can be identified
		Ar.Logf(TEXT("Summary of assets containing objects that have bad references"));
		for (UObject* BadObj : ObjectsContainingBadness)
		{
			Ar.Logf(TEXT("\n\nObject %s referenced private objects outside of it's container asset inappropriately"), *BadObj->GetFullName());

			UBlueprint* Blueprint = Cast<UBlueprint>(BadObj);
			if (Blueprint == nullptr)
			{
				if (UClass* Class = Cast<UClass>(BadObj))
				{
					Blueprint = CastChecked<UBlueprint>(Class->ClassGeneratedBy);

					if (Blueprint->GeneratedClass == Class)
					{
						Ar.Logf(TEXT("  => GeneratedClass of %s"), *Blueprint->GetFullName());
					}
					else if (Blueprint->SkeletonGeneratedClass == Class)
					{
						Ar.Logf(TEXT("  => SkeletonGeneratedClass of %s"), *Blueprint->GetFullName());
					}
					else
					{
						Ar.Logf(TEXT("  => ***FALLEN BEHIND*** class generated by %s"), *Blueprint->GetFullName());
					}
					Ar.Logf(TEXT("  Has an associated CDO named %s"), *Class->GetDefaultObject()->GetFullName());
				}
			}

			// Export the asset to text
			if (true)
			{
				UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));
				FStringOutputDevice Archive;
				const FExportObjectInnerContext Context;
				UExporter::ExportToOutputDevice(&Context, BadObj, nullptr, Archive, TEXT("copy"), 0, /*PPF_IncludeTransient*/ /*PPF_ExportsNotFullyQualified|*/PPF_Copy, false, nullptr);
				FString ExportedText = Archive;

				Ar.Logf(TEXT("%s"), *ExportedText);
			}
		}

		// Report the contents of the bad packages
		for (UPackage* BadPackage : BadPackages)
		{
			Ar.Logf(TEXT("\nBad package %s contains:"), *BadPackage->GetName());
			for (FThreadSafeObjectIterator ObjIt; ObjIt; ++ObjIt)
			{
				if (ObjIt->GetOuter() == BadPackage)
				{
					Ar.Logf(TEXT("  %s"), *ObjIt->GetFullName());
				}
			}
		}

		Ar.Logf(TEXT("\nFinished listing illegal private references"));
	}
	else if (FParse::Command(&Str, TEXT("ListPackageContents")))
	{
		if (UPackage* Package = FindPackage(nullptr, Str))
		{
			FBlueprintEditorUtils::ListPackageContents(Package, Ar);
		}
		else
		{
			Ar.Logf(TEXT("Failed to find package %s"), Str);
		}
	}
	else if (FParse::Command(&Str, TEXT("RepairBlueprint")))
	{
		if (UBlueprint* Blueprint = FindFirstObject<UBlueprint>(Str, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("RepairBlueprint")))
		{
			IKismetCompilerInterface& Compiler = FModuleManager::LoadModuleChecked<IKismetCompilerInterface>(KISMET_COMPILER_MODULENAME);
			Compiler.RecoverCorruptedBlueprint(Blueprint);
		}
		else
		{
			Ar.Logf(TEXT("Failed to find blueprint %s"), Str);
		}
	}
	else if (FParse::Command(&Str, TEXT("ListOrphanClasses")))
	{
		UE_LOGF(LogBlueprintDebug, Log, "--- LISTING ORPHANED CLASSES ---");
		for( TObjectIterator<UClass> it; it; ++it )
		{
			UClass* CurrClass = *it;
			if( CurrClass->ClassGeneratedBy != nullptr && CurrClass->GetOutermost() != GetTransientPackage() )
			{
				if( UBlueprint* GeneratingBP = Cast<UBlueprint>(CurrClass->ClassGeneratedBy) )
				{
					if( CurrClass != GeneratingBP->GeneratedClass && CurrClass != GeneratingBP->SkeletonGeneratedClass )
					{
						UE_LOGF(LogBlueprintDebug, Log, " - %ls", *CurrClass->GetFullName());				
					}
				}	
			}
		}

		return true;
	}
	else if (FParse::Command(&Str, TEXT("ListRootSetObjects")))
	{
		UE_LOGF(LogBlueprintDebug, Log, "--- LISTING ROOTSET OBJ ---");
		for( FThreadSafeObjectIterator it; it; ++it )
		{
			UObject* CurrObj = *it;
			if( CurrObj->IsRooted() )
			{
				UE_LOGF(LogBlueprintDebug, Log, " - %ls", *CurrObj->GetFullName());
			}
		}
	}
	else
	{
		return false;
	}

	return true;
}


void FBlueprintEditorUtils::OpenReparentBlueprintMenu( UBlueprint* Blueprint, const TSharedRef<SWidget>& ParentContent, const FOnClassPicked& OnPicked)
{
	TArray< UBlueprint* > Blueprints;
	Blueprints.Add( Blueprint );
	OpenReparentBlueprintMenu( Blueprints, ParentContent, OnPicked );
}

class FBlueprintReparentFilter : public IClassViewerFilter
{
public:
	/** All children of these classes will be included unless filtered out by another setting. */
	TSet< const UClass* > AllowedChildrenOfClasses;

	/** Classes to not allow any children of into the Class Viewer/Picker. */
	TSet< const UClass* > DisallowedChildrenOfClasses;

	/** Classes to never show in this class viewer. */
	TSet< const UClass* > DisallowedClasses;

	/** Will limit the results to only native classes */
	bool bShowNativeOnly;

	FBlueprintReparentFilter()
		: bShowNativeOnly(false)
	{}

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs ) override
	{
		// If it appears on the allowed child-of classes list (or there is nothing on that list)
		//		AND it is NOT on the disallowed child-of classes list
		//		AND it is NOT on the disallowed classes list
		return InFilterFuncs->IfInChildOfClassesSet( AllowedChildrenOfClasses, InClass) != EFilterReturn::Failed && 
			InFilterFuncs->IfInChildOfClassesSet(DisallowedChildrenOfClasses, InClass) != EFilterReturn::Passed && 
			InFilterFuncs->IfInClassesSet(DisallowedClasses, InClass) != EFilterReturn::Passed &&
			!InClass->HasAnyClassFlags(CLASS_Deprecated) &&
			(!InClass->HasAnyFlags(RF_Transient) || InClass->IsNative()) && InClass->GetPackage() != GetTransientPackage() &&
			((bShowNativeOnly && InClass->HasAnyClassFlags(CLASS_Native)) || !bShowNativeOnly);
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		// If it appears on the allowed child-of classes list (or there is nothing on that list)
		//		AND it is NOT on the disallowed child-of classes list
		//		AND it is NOT on the disallowed classes list
		return InFilterFuncs->IfInChildOfClassesSet( AllowedChildrenOfClasses, InUnloadedClassData) != EFilterReturn::Failed && 
			InFilterFuncs->IfInChildOfClassesSet(DisallowedChildrenOfClasses, InUnloadedClassData) != EFilterReturn::Passed && 
			InFilterFuncs->IfInClassesSet(DisallowedClasses, InUnloadedClassData) != EFilterReturn::Passed &&
			!InUnloadedClassData->HasAnyClassFlags(CLASS_Deprecated)  &&
			((bShowNativeOnly && InUnloadedClassData->HasAnyClassFlags(CLASS_Native)) || !bShowNativeOnly);
	}
};

TSharedRef<SWidget> FBlueprintEditorUtils::ConstructBlueprintParentClassPicker( const TArray< UBlueprint* >& Blueprints, const FOnClassPicked& OnPicked)
{
	bool bIsActor = false;
	bool bIsAnimBlueprint = false;
	bool bIsLevelScriptActor = false;
	bool bIsComponentBlueprint = false;
	bool bIsEditorOnlyBlueprint = false;
	bool bIsWidgetBlueprint = false;
	TArray<UClass*> BlueprintClasses;
	for( auto BlueprintIter = Blueprints.CreateConstIterator(); (!bIsActor && !bIsAnimBlueprint) && BlueprintIter; ++BlueprintIter )
	{
		const UBlueprint* Blueprint = *BlueprintIter;
		bIsActor |= Blueprint->ParentClass && Blueprint->ParentClass->IsChildOf( AActor::StaticClass() );
		bIsAnimBlueprint |= Blueprint->IsA(UAnimBlueprint::StaticClass());
		bIsLevelScriptActor |= Blueprint->ParentClass && Blueprint->ParentClass->IsChildOf( ALevelScriptActor::StaticClass() );
		bIsComponentBlueprint |= Blueprint->ParentClass && Blueprint->ParentClass->IsChildOf( UActorComponent::StaticClass() );
		bIsEditorOnlyBlueprint |= IsEditorUtilityBlueprint(Blueprint);
		bIsWidgetBlueprint = Blueprint->IsA(UBaseWidgetBlueprint::StaticClass());
		if(Blueprint->GeneratedClass)
		{
			BlueprintClasses.Add(Blueprint->GeneratedClass);
		}
	}

	// Fill in options
	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;
	Options.bShowBackgroundBorder = false;

	TSharedPtr<FBlueprintReparentFilter> Filter = MakeShareable(new FBlueprintReparentFilter);
	Options.ClassFilters.Add(Filter.ToSharedRef());
	Options.ViewerTitleString = LOCTEXT("ReparentBlueprint", "Reparent blueprint");

	// Only allow parenting to base blueprints.
	Options.bIsBlueprintBaseOnly = true;
	Options.bEditorClassesOnly = bIsEditorOnlyBlueprint;

	// never allow parenting to Interface
	Filter->DisallowedChildrenOfClasses.Add( UInterface::StaticClass() );

	// never allow parenting to children of itself
	for (UClass* BPClass : BlueprintClasses)
	{
		Filter->DisallowedChildrenOfClasses.Add(BPClass);
	}

	for ( UBlueprint* Blueprint : Blueprints )
	{
		Blueprint->GetReparentingRules(Filter->AllowedChildrenOfClasses, Filter->DisallowedChildrenOfClasses);

		// Include a class viewer filter for imported namespaces if the class picker is being hosted in an editor context.
		TSharedPtr<IToolkit> AssetEditor = FToolkitManager::Get().FindEditorForAsset(Blueprint);
		if (AssetEditor.IsValid() && AssetEditor->IsBlueprintEditor())
		{
			TSharedPtr<IBlueprintEditor> BlueprintEditor = StaticCastSharedPtr<IBlueprintEditor>(AssetEditor);
			TSharedPtr<IClassViewerFilter> ImportedClassViewerFilter = BlueprintEditor->GetImportedClassViewerFilter();
			if (ImportedClassViewerFilter.IsValid())
			{
				Options.ClassFilters.AddUnique(ImportedClassViewerFilter.ToSharedRef());
			}
		}
	}

	if(bIsActor)
	{
		if(bIsLevelScriptActor)
		{
			// Don't allow conversion outside of the LevelScriptActor hierarchy
			Filter->AllowedChildrenOfClasses.Add( ALevelScriptActor::StaticClass() );
			Filter->bShowNativeOnly = true;
		}
		else
		{
			// Don't allow conversion outside of the Actor hierarchy
			{
				bool bHasAllowedActorClasses = false;
				for (const UClass* Class : Filter->AllowedChildrenOfClasses)
				{
					if (Class && Class->IsChildOf(AActor::StaticClass()))
					{
						// This blueprint has already defined explicit actor classes to use
						bHasAllowedActorClasses = true;
						break;
					}
				}
			
				if (!bHasAllowedActorClasses)
				{
					Filter->AllowedChildrenOfClasses.Add( AActor::StaticClass() );
				}
			}

			// Don't allow non-LevelScriptActor->LevelScriptActor conversion
			Filter->DisallowedChildrenOfClasses.Add( ALevelScriptActor::StaticClass() );
		}
	}
	else if (bIsAnimBlueprint)
	{
		// If it's an anim blueprint, do not allow conversion to non anim
		Filter->AllowedChildrenOfClasses.Add( UAnimInstance::StaticClass() );
	}
	else if(bIsComponentBlueprint)
	{
		// If it is a component blueprint, only allow classes under and including UActorComponent
		Filter->AllowedChildrenOfClasses.Add( UActorComponent::StaticClass() );
	}
	else if (bIsEditorOnlyBlueprint && !bIsWidgetBlueprint)
	{
		Filter->DisallowedChildrenOfClasses.Add(UWidget::StaticClass());
	}
	else
	{
		Filter->DisallowedChildrenOfClasses.Add( AActor::StaticClass() );
	}


	for( const UBlueprint* Blueprint : Blueprints)
	{
		// don't allow making me my own parent!
		if(Blueprint->GeneratedClass)
		{
			Filter->DisallowedClasses.Add(Blueprint->GeneratedClass);
		}

		// Disallow reparenting to any sub-class that has a different blueprint type.
		// for example, reparenting AActor to AEditorUtilityActor is illegal because AActor uses UBlueprint while
		// AEditorUtilityActor uses UEditorUtilityBlueprint
		const IKismetCompilerInterface& KismetCompilerModule = FModuleManager::LoadModuleChecked<IKismetCompilerInterface>("KismetCompiler");
		if (Blueprint->ParentClass) // ParentClass is rarely null. Allow all children in that case.
		{
			KismetCompilerModule.GetSubclassesWithDifferingBlueprintTypes(Blueprint->ParentClass, Filter->DisallowedChildrenOfClasses);
		}
	}

	return FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer").CreateClassViewer(Options, OnPicked);
}

void FBlueprintEditorUtils::OpenReparentBlueprintMenu( const TArray< UBlueprint* >& Blueprints, const TSharedRef<SWidget>& ParentContent, const FOnClassPicked& OnPicked)
{
	if ( Blueprints.Num() == 0 )
	{
		return;
	}

	TSharedRef<SWidget> ClassPicker = ConstructBlueprintParentClassPicker(Blueprints, OnPicked);

	TSharedRef<SBox> ClassPickerBox = 
		SNew(SBox)
		.WidthOverride(280.0f)
		.HeightOverride(400.0f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Menu.Background"))
			[
				ClassPicker
			]
		];

	// Show dialog to choose new parent class
	FSlateApplication::Get().PushMenu(
		ParentContent,
		FWidgetPath(),
		ClassPickerBox,
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect( FPopupTransitionEffect::ContextMenu),
		true);
}

/** Filter class for ClassPicker handling allowed interfaces for a Blueprint */
class FBlueprintInterfaceFilter : public IClassViewerFilter
{
public:
	/** All children of these classes will be included unless filtered out by another setting. */
	TSet< const UClass* > AllowedChildrenOfClasses;

	/** Classes to not allow any children of into the Class Viewer/Picker. */
	TSet< const UClass* > DisallowedChildrenOfClasses;

	/** Classes to never show in this class viewer. */
	TSet< const UClass* > DisallowedClasses;

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs ) override
	{
		// If it appears on the allowed child-of classes list (or there is nothing on that list)
		//		AND it is NOT on the disallowed child-of classes list
		//		AND it is NOT on the disallowed classes list
		return InFilterFuncs->IfInChildOfClassesSet( AllowedChildrenOfClasses, InClass) != EFilterReturn::Failed && 
			InFilterFuncs->IfInChildOfClassesSet(DisallowedChildrenOfClasses, InClass) != EFilterReturn::Passed && 
			InFilterFuncs->IfInClassesSet(DisallowedClasses, InClass) != EFilterReturn::Passed &&
			!InClass->HasAnyClassFlags(CLASS_Deprecated | CLASS_NewerVersionExists) &&
			FKismetEditorUtilities::IsClassABlueprintImplementableInterface(InClass);
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		// Unloaded interfaces mean they must be Blueprint Interfaces


		// If it appears on the allowed child-of classes list (or there is nothing on that list)
		//		AND it is NOT on the disallowed child-of classes list
		//		AND it is NOT on the disallowed classes list
		return InFilterFuncs->IfInChildOfClassesSet( AllowedChildrenOfClasses, InUnloadedClassData) != EFilterReturn::Failed && 
			InFilterFuncs->IfInChildOfClassesSet(DisallowedChildrenOfClasses, InUnloadedClassData) != EFilterReturn::Passed && 
			InFilterFuncs->IfInClassesSet(DisallowedClasses, InUnloadedClassData) != EFilterReturn::Passed &&
			!InUnloadedClassData->HasAnyClassFlags(CLASS_Deprecated | CLASS_NewerVersionExists) &&
			InUnloadedClassData->HasAnyClassFlags(CLASS_Interface);
	}
};

TSharedRef<SWidget> FBlueprintEditorUtils::ConstructBlueprintInterfaceClassPicker( const TArray< UBlueprint* >& Blueprints, const FOnClassPicked& OnPicked)
{
	TArray<UClass*> BlueprintClasses;
	for (const UBlueprint* Blueprint : Blueprints)
	{
		if(Blueprint->GeneratedClass)
		{
			BlueprintClasses.Add(Blueprint->GeneratedClass);
		}
	}

	// Fill in options
	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;
	Options.bShowBackgroundBorder = false;

	TSharedPtr<FBlueprintInterfaceFilter> Filter = MakeShareable(new FBlueprintInterfaceFilter);
	Options.ClassFilters.Add(Filter.ToSharedRef());
	Options.ViewerTitleString = LOCTEXT("ImplementInterfaceBlueprint", "Implement Interface");

	for (const UBlueprint* Blueprint : Blueprints)
	{
		// don't allow making me my own parent!
		if(Blueprint->GeneratedClass)
		{
			Filter->DisallowedClasses.Add(Blueprint->GeneratedClass);
		}
		
		UClass const* const ParentClass = Blueprint->ParentClass;
		// see if the parent class has any prohibited interfaces
		if ((ParentClass != nullptr) && ParentClass->HasMetaData(FBlueprintMetadata::MD_ProhibitedInterfaces))
		{
			FString const& ProhibitedList = Blueprint->ParentClass->GetMetaData(FBlueprintMetadata::MD_ProhibitedInterfaces);

			TArray<FString> ProhibitedInterfaceNames;
			ProhibitedList.ParseIntoArray(ProhibitedInterfaceNames, TEXT(","), true);

			// loop over all the prohibited interfaces
			for (int32 ExclusionIndex = 0; ExclusionIndex < ProhibitedInterfaceNames.Num(); ++ExclusionIndex)
			{
				ProhibitedInterfaceNames[ExclusionIndex].TrimStartInline();
				FString const& ProhibitedInterfaceName = ProhibitedInterfaceNames[ExclusionIndex].RightChop(1);
				UClass* ProhibitedInterface = UClass::TryFindTypeSlow<UClass>(ProhibitedInterfaceName);
				if(ProhibitedInterface)
				{
					Filter->DisallowedClasses.Add(ProhibitedInterface);
					Filter->DisallowedChildrenOfClasses.Add(ProhibitedInterface);
				}
			}
		}

		// Do not allow adding interfaces that are already added to the Blueprint
		TArray<UClass*> InterfaceClasses;
		FindImplementedInterfaces(Blueprint, true, InterfaceClasses);
		for(UClass* InterfaceClass : InterfaceClasses)
		{
			Filter->DisallowedClasses.Add(InterfaceClass);
		}

		// Include a class viewer filter for imported namespaces if the class picker is being hosted in an editor context
		TSharedPtr<IToolkit> AssetEditor = FToolkitManager::Get().FindEditorForAsset(Blueprint);
		if (AssetEditor.IsValid() && AssetEditor->IsBlueprintEditor())
		{
			TSharedPtr<IBlueprintEditor> BlueprintEditor = StaticCastSharedPtr<IBlueprintEditor>(AssetEditor);
			TSharedPtr<IClassViewerFilter> ImportedClassViewerFilter = BlueprintEditor->GetImportedClassViewerFilter();
			if (ImportedClassViewerFilter.IsValid())
			{
				Options.ClassFilters.AddUnique(ImportedClassViewerFilter.ToSharedRef());
			}
		}
	}

	// never allow parenting to children of itself
	for (UClass*  BPClass : BlueprintClasses)
	{
		Filter->DisallowedChildrenOfClasses.Add(BPClass);
	}

	return FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer").CreateClassViewer(Options, OnPicked);
}

/** Call PostEditChange() on any Actors that are based on this Blueprint */
void FBlueprintEditorUtils::PostEditChangeBlueprintActors(UBlueprint* Blueprint, bool bComponentEditChange)
{
	if (Blueprint->GeneratedClass && Blueprint->GeneratedClass->IsChildOf(AActor::StaticClass()))
	{
		// Get the selected Actor set in the level editor context
		bool bEditorSelectionChanged = false;
		const USelection* CurrentEditorActorSelection = GEditor ? GEditor->GetSelectedActors() : nullptr;
		const bool bIncludeDerivedClasses = false;

		TArray<UObject*> MatchingBlueprintObjects;
		GetObjectsOfClass(Blueprint->GeneratedClass, MatchingBlueprintObjects, bIncludeDerivedClasses, RF_ClassDefaultObject, EInternalObjectFlags::Garbage);

		for (UObject* MatchingObj : MatchingBlueprintObjects)
		{
			// We know the class was derived from AActor because we checked the Blueprint->GeneratedClass.
			AActor* Actor = static_cast<AActor*>(MatchingObj);
			
			// If worlds leak or GC hasn't been performed we'll have actors in invalid worlds and reconstructing
			// them in PostEditChange will crash
			UWorld* ActorWorld = Actor->GetWorld();
			if (ActorWorld && (!IsValid(ActorWorld) || ActorWorld->IsCleanedUp()))
			{
				continue;
			}

			Actor->PostEditChange();

			// Broadcast edit notification if necessary so that the level editor's detail panel is refreshed
			bEditorSelectionChanged |= CurrentEditorActorSelection && CurrentEditorActorSelection->IsSelected(Actor);
		}

		// Broadcast edit notifications if necessary so that level editor details are refreshed (e.g. components tree)
		if(bEditorSelectionChanged)
		{
			if(bComponentEditChange)
			{
				FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
				LevelEditor.BroadcastComponentsEdited();
			}
		}
	}	

	// Let the blueprint thumbnail renderer know that a blueprint has been modified so it knows to reinstance components for visualization
	FThumbnailRenderingInfo* RenderInfo = GUnrealEd->GetThumbnailManager()->GetRenderingInfo( Blueprint );
	if ( RenderInfo != nullptr )
	{
		UBlueprintThumbnailRenderer* BlueprintThumbnailRenderer = Cast<UBlueprintThumbnailRenderer>(RenderInfo->Renderer);
		if ( BlueprintThumbnailRenderer != nullptr )
		{
			BlueprintThumbnailRenderer->BlueprintChanged(Blueprint);
		}
	}
}

bool FBlueprintEditorUtils::IsPropertyPrivate(const FProperty* Property)
{
	return Property->HasAnyPropertyFlags(CPF_NativeAccessSpecifierPrivate) || Property->GetBoolMetaData(FBlueprintMetadata::MD_Private); 
}

FBlueprintEditorUtils::EPropertyWritableState FBlueprintEditorUtils::IsPropertyWritableInBlueprint(const UBlueprint* Blueprint, const FProperty* Property)
{
	if (Property)
	{
		if (!Property->HasAnyPropertyFlags(CPF_BlueprintVisible))
		{
			return EPropertyWritableState::NotBlueprintVisible;
		}
		if (Property->HasAnyPropertyFlags(CPF_BlueprintReadOnly))
		{
			return EPropertyWritableState::BlueprintReadOnly;
		}
		if (Property->GetBoolMetaData(FBlueprintMetadata::MD_Private))
		{
			const UClass* OwningClass = Property->GetOwnerChecked<UClass>();
			if (OwningClass->ClassGeneratedBy.Get() != Blueprint)
			{
				return EPropertyWritableState::Private;
			}
		}
	}
	return EPropertyWritableState::Writable;
}

FBlueprintEditorUtils::EPropertyReadableState FBlueprintEditorUtils::IsPropertyReadableInBlueprint(const UBlueprint* Blueprint, const FProperty* Property)
{
	if (Property)
	{
		if (!Property->HasAnyPropertyFlags(CPF_BlueprintVisible))
		{
			return EPropertyReadableState::NotBlueprintVisible;
		}
		if (Property->GetBoolMetaData(FBlueprintMetadata::MD_Private))
		{
			const UClass* OwningClass = Property->GetOwnerChecked<UClass>();
			if (OwningClass->ClassGeneratedBy.Get() != Blueprint)
			{
				return EPropertyReadableState::Private;
			}
		}
	}
	return EPropertyReadableState::Readable;
}

void FBlueprintEditorUtils::FindAndSetDebuggableBlueprintInstances()
{
	TMap< UBlueprint*, TArray< AActor* > > BlueprintsNeedingInstancesToDebug;

	// Find open blueprint editors that have no debug instances
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	TArray<UObject*> EditedAssets = AssetEditorSubsystem->GetAllEditedAssets();		
	for(int32 i=0; i<EditedAssets.Num(); i++)
	{
		UBlueprint* Blueprint = Cast<UBlueprint>( EditedAssets[i] );
		if( Blueprint != nullptr )
		{
			if (Blueprint->GetObjectPathToDebug().IsEmpty())
			{
				BlueprintsNeedingInstancesToDebug.FindOrAdd( Blueprint );
			}			
		}
	}

	// If we have blueprints with no debug objects selected try to find a suitable on to debug
	if( BlueprintsNeedingInstancesToDebug.Num() != 0 )
	{	
		// This will only assign currently selected objects of the right type, otherwise leave on default behavior to break on any
		USelection* Selected = GEditor->GetSelectedActors();
		const bool bDisAllowDerivedTypes = false;
		TArray< UBlueprint* > BlueprintsToRefresh;
		for (TMap< UBlueprint*, TArray< AActor* > >::TIterator ObjIt(BlueprintsNeedingInstancesToDebug); ObjIt; ++ObjIt)
		{	
			UBlueprint* EachBlueprint = ObjIt.Key();
			bool bFoundItemToDebug = false;

			if( Selected )
			{
				for (int32 iSelected = 0; iSelected < Selected->Num() ; iSelected++)
				{
					AActor* ObjectAsActor = Cast<AActor>( Selected->GetSelectedObject( iSelected ) );
					UWorld* ActorWorld = ObjectAsActor ? ObjectAsActor->GetWorld() : nullptr;
					if ((ActorWorld != nullptr) && (ActorWorld->WorldType != EWorldType::EditorPreview) && (ActorWorld->WorldType != EWorldType::Inactive))
					{
						if( IsObjectADebugCandidate(ObjectAsActor, EachBlueprint, true/*bInDisallowDerivedBlueprints*/ ) == true )
						{
							EachBlueprint->SetObjectBeingDebugged( ObjectAsActor );
							bFoundItemToDebug = true;
							BlueprintsToRefresh.Add( EachBlueprint );
							break;
						}
					}
				}
			}
		}

		// Refresh all blueprint windows that we have made a change to the debugging selection of
		for (int32 iRefresh = 0; iRefresh < BlueprintsToRefresh.Num() ; iRefresh++)
		{
			// Ensure its a blueprint editor !
			TSharedPtr< IToolkit > FoundAssetEditor = FToolkitManager::Get().FindEditorForAsset(BlueprintsToRefresh[ iRefresh]);
			if (FoundAssetEditor.IsValid() && FoundAssetEditor->IsBlueprintEditor())
			{
				TSharedPtr<IBlueprintEditor> BlueprintEditor = StaticCastSharedPtr<IBlueprintEditor>(FoundAssetEditor);
				BlueprintEditor->RefreshEditors();
			}
		}
	}
}

void FBlueprintEditorUtils::AnalyticsTrackNewNode( UEdGraphNode *NewNode )
{
	UBlueprint* Blueprint = FindBlueprintForNodeChecked(NewNode);
	TSharedPtr<IToolkit> FoundAssetEditor = FToolkitManager::Get().FindEditorForAsset(Blueprint);
	if (FoundAssetEditor.IsValid() && FoundAssetEditor->IsBlueprintEditor()) 
	{ 
		TSharedPtr<IBlueprintEditor> BlueprintEditor = StaticCastSharedPtr<IBlueprintEditor>(FoundAssetEditor);
		BlueprintEditor->AnalyticsTrackNodeEvent(Blueprint, NewNode, false);
	}
}

bool FBlueprintEditorUtils::IsObjectADebugCandidate( AActor* InActorObject, UBlueprint* InBlueprint, bool bInDisallowDerivedBlueprints )
{
	const bool bPassesFlags = !InActorObject->HasAnyFlags(RF_ClassDefaultObject) && IsValid(InActorObject);
	bool bCanDebugThisObject = false;
	if( bInDisallowDerivedBlueprints == true )
	{
		bCanDebugThisObject = InActorObject->GetClass()->ClassGeneratedBy == InBlueprint;
	}
	else if(InBlueprint->GeneratedClass)
	{
		bCanDebugThisObject = InActorObject->IsA( InBlueprint->GeneratedClass );
	}
	
	return bPassesFlags && bCanDebugThisObject;
}

bool FBlueprintEditorUtils::PropertyValueFromString(const FProperty* Property, const FString& StrValue, uint8* Container, UObject* OwningObject, int32 PortFlags)
{
	return PropertyValueFromString_Direct(Property, StrValue, Property->ContainerPtrToValuePtr<uint8>(Container), OwningObject, PortFlags);
}

bool FBlueprintEditorUtils::PropertyValueFromString_Direct(const FProperty* Property, const FString& StrValue, uint8* DirectValue, UObject* OwningObject, int32 PortFlags)
{
	bool bParseSucceeded = true;
	if (!Property->IsA(FStructProperty::StaticClass()))
	{
		if (Property->IsA(FIntProperty::StaticClass()))
		{
			int32 IntValue = 0;
			bParseSucceeded = FDefaultValueHelper::ParseInt(StrValue, IntValue);
			CastFieldChecked<const FIntProperty>(Property)->SetPropertyValue(DirectValue, IntValue);
		}
		else if (Property->IsA(FInt64Property::StaticClass()))
		{
			int64 IntValue = 0;
			bParseSucceeded = FDefaultValueHelper::ParseInt64(StrValue, IntValue);
			CastFieldChecked<const FInt64Property>(Property)->SetPropertyValue(DirectValue, IntValue);
		}
		else if (Property->IsA(FFloatProperty::StaticClass()))
		{
			float FloatValue = 0.0f;
			bParseSucceeded = FDefaultValueHelper::ParseFloat(StrValue, FloatValue);
			CastFieldChecked<const FFloatProperty>(Property)->SetPropertyValue(DirectValue, FloatValue);
		}
		else if (Property->IsA(FDoubleProperty::StaticClass()))
		{
			double DoubleValue = 0.0;
			bParseSucceeded = FDefaultValueHelper::ParseDouble(StrValue, DoubleValue);
			CastFieldChecked<const FDoubleProperty>(Property)->SetPropertyValue(DirectValue, DoubleValue);
		}
		else if (const FByteProperty* ByteProperty = CastField<const FByteProperty>(Property))
		{
			int64 IntValue = 0;
			if (const UEnum* Enum = ByteProperty->Enum)
			{
				if (StrValue.Len() < NAME_SIZE)
				{
					IntValue = Enum->GetValueByName(FName(*StrValue));
				}
				else
				{
					IntValue = INDEX_NONE;
				}
				bParseSucceeded = (INDEX_NONE != IntValue);

				// If the parse did not succeed, clear out the int to keep the enum value valid
				if (!bParseSucceeded)
				{
					IntValue = 0;
				}
			}
			else
			{
				bParseSucceeded = FDefaultValueHelper::ParseInt64(StrValue, IntValue);
			}

			bParseSucceeded = bParseSucceeded && (IntValue <= 255) && (IntValue >= 0);
			ByteProperty->SetPropertyValue(DirectValue, static_cast<uint8>(IntValue));
		}
		else if (const FEnumProperty* EnumProperty = CastField<const FEnumProperty>(Property))
		{
			bParseSucceeded = false;
			if (const UEnum* Enum = EnumProperty->GetEnum())
			{
				int64 IntValue = INDEX_NONE;
				if (StrValue.Len() < NAME_SIZE)
				{
					IntValue = Enum->GetValueByName(FName(*StrValue));
				}
				bParseSucceeded = (INDEX_NONE != IntValue);

				// If the parse did not succeed, clear out the int to keep the enum value valid
				if (!bParseSucceeded)
				{
					IntValue = 0;
				}
				bParseSucceeded = bParseSucceeded && (IntValue <= 255) && (IntValue >= 0);
				EnumProperty->GetUnderlyingProperty()->SetIntPropertyValue(DirectValue, IntValue);
			}
			else
			{
				UE_LOGF(
					LogBlueprint,
					Warning,
					"Member 'Enum' of EnumProperty is nullptr, copy   operation would fail. You could ignore this message if you moved the enum class. EnumProperty name:'%ls', OwningObject: '%ls', outer of OwningObject: '%ls', outer of outer: '%ls'",
					*EnumProperty->GetName(),
					*GetNameSafe(OwningObject),
					*GetNameSafe(OwningObject ? OwningObject->GetOuter() : nullptr),
					*GetNameSafe(OwningObject ? OwningObject->GetOuter() ? OwningObject->GetOuter()->GetOuter(): nullptr : nullptr)
				);
			}
		}
		else if (Property->IsA(FStrProperty::StaticClass()))
		{
			CastFieldChecked<const FStrProperty>(Property)->SetPropertyValue(DirectValue, StrValue);
		}
		else if (Property->IsA(FBoolProperty::StaticClass()))
		{
			CastFieldChecked<const FBoolProperty>(Property)->SetPropertyValue(DirectValue, StrValue.ToBool());
		}
		else if (Property->IsA(FNameProperty::StaticClass()))
		{
			CastFieldChecked<const FNameProperty>(Property)->SetPropertyValue(DirectValue, FName(*StrValue));
		}
		else if (Property->IsA(FTextProperty::StaticClass()))
		{
			FStringOutputDevice ImportError;
			const TCHAR* EndOfParsedBuff = Property->ImportText_Direct(*StrValue, DirectValue, OwningObject, PPF_SerializedAsImportText | PortFlags, &ImportError);
			bParseSucceeded = EndOfParsedBuff && ImportError.IsEmpty();
		}
		else
		{
			// Empty array-like properties need to use "()" in order to import correctly (as array properties export comma separated within a set of brackets)
			const TCHAR* const ValueToImport = (StrValue.IsEmpty() && (Property->IsA(FArrayProperty::StaticClass()) || Property->IsA(FMulticastDelegateProperty::StaticClass())))
				? TEXT("()")
				: *StrValue;

			FStringOutputDevice ImportError;
			const TCHAR* EndOfParsedBuff = Property->ImportText_Direct(*StrValue, DirectValue, OwningObject, PPF_SerializedAsImportText | PortFlags, &ImportError);
			bParseSucceeded = EndOfParsedBuff && ImportError.IsEmpty();
		}
	}
	else
	{
		static UScriptStruct* VectorStruct = TBaseStructure<FVector>::Get();
		static UScriptStruct* RotatorStruct = TBaseStructure<FRotator>::Get();
		static UScriptStruct* TransformStruct = TBaseStructure<FTransform>::Get();
		static UScriptStruct* LinearColorStruct = TBaseStructure<FLinearColor>::Get();

		const FStructProperty* StructProperty = CastFieldChecked<const FStructProperty>(Property);

		// Struct properties must be handled differently, unfortunately.  We only support FVector, FRotator, and FTransform
		if (StructProperty->Struct == VectorStruct)
		{
			FVector V = FVector::ZeroVector;
			bParseSucceeded = FDefaultValueHelper::ParseVector(StrValue, V);
			Property->CopySingleValue(DirectValue, &V);
		}
		else if (StructProperty->Struct == RotatorStruct)
		{
			FRotator R = FRotator::ZeroRotator;
			bParseSucceeded = FDefaultValueHelper::ParseRotator(StrValue, R);
			Property->CopySingleValue(DirectValue, &R);
		}
		else if (StructProperty->Struct == TransformStruct)
		{
			FTransform T = FTransform::Identity;
			bParseSucceeded = T.InitFromString(StrValue);
			Property->CopySingleValue(DirectValue, &T);
		}
		else if (StructProperty->Struct == LinearColorStruct)
		{
			FLinearColor Color;
			// Color form: "(R=%f,G=%f,B=%f,A=%f)"
			bParseSucceeded = Color.InitFromString(StrValue);
			Property->CopySingleValue(DirectValue, &Color);
		}
		else if (StructProperty->Struct)
		{
			const UScriptStruct* Struct = StructProperty->Struct;
			const int32 StructSize = Struct->GetStructureSize() * StructProperty->ArrayDim;
			StructProperty->InitializeValue(DirectValue);
			ensure(1 == StructProperty->ArrayDim);

			FStringOutputDevice ImportError;
			const TCHAR* EndOfParsedBuff = StructProperty->ImportText_Direct(StrValue.IsEmpty() ? TEXT("()") : *StrValue, DirectValue, OwningObject, PPF_SerializedAsImportText | PortFlags, &ImportError);
			bParseSucceeded &= EndOfParsedBuff && ImportError.IsEmpty();
		}
	}

	return bParseSucceeded;
}

bool FBlueprintEditorUtils::PropertyValueToString(const FProperty* Property, const uint8* Container, FString& OutForm, UObject* OwningObject, int32 PortFlags)
{
	return PropertyValueToString_Direct(Property, Property->ContainerPtrToValuePtr<const uint8>(Container), OutForm, OwningObject, PortFlags);
}

bool FBlueprintEditorUtils::PropertyValueToString_Direct(const FProperty* Property, const uint8* DirectValue, FString& OutForm, UObject* OwningObject, int32 PortFlags)
{
	check(Property && DirectValue);
	OutForm.Reset();

	const FStructProperty* StructProperty = CastField<FStructProperty>(Property);
	if (StructProperty)
	{
		static UScriptStruct* VectorStruct = TBaseStructure<FVector>::Get();
		static UScriptStruct* RotatorStruct = TBaseStructure<FRotator>::Get();
		static UScriptStruct* TransformStruct = TBaseStructure<FTransform>::Get();
		static UScriptStruct* LinearColorStruct = TBaseStructure<FLinearColor>::Get();

		// Struct properties must be handled differently, unfortunately.  We only support FVector, FRotator, and FTransform
		if (StructProperty->Struct == VectorStruct)
		{
			FVector Vector;
			Property->CopySingleValue(&Vector, DirectValue);
			OutForm = FString::Printf(TEXT("%f,%f,%f"), Vector.X, Vector.Y, Vector.Z);
		}
		else if (StructProperty->Struct == RotatorStruct)
		{
			FRotator Rotator;
			Property->CopySingleValue(&Rotator, DirectValue);
			OutForm = FString::Printf(TEXT("%f,%f,%f"), Rotator.Pitch, Rotator.Yaw, Rotator.Roll);
		}
		else if (StructProperty->Struct == TransformStruct)
		{
			FTransform Transform;
			Property->CopySingleValue(&Transform, DirectValue);
			OutForm = Transform.ToString();
		}
		else if (StructProperty->Struct == LinearColorStruct)
		{
			FLinearColor Color;
			Property->CopySingleValue(&Color, DirectValue);
			OutForm = Color.ToString();
		}
	}

	bool bSucceeded = true;
	if (OutForm.IsEmpty())
	{
		const uint8* DefaultValue = DirectValue;	
		bSucceeded = Property->ExportText_Direct(OutForm, DirectValue, DefaultValue, OwningObject, PPF_SerializedAsImportText | PortFlags);
	}
	return bSucceeded;
}

FName FBlueprintEditorUtils::GenerateUniqueGraphName(UObject* const InOuter, FString const& ProposedName)
{
	FName UniqueGraphName(*ProposedName);

	int32 CountPostfix = 1;
	while (!FBlueprintEditorUtils::IsGraphNameUnique(InOuter, UniqueGraphName))
	{
		UniqueGraphName = FName(*FString::Printf(TEXT("%s%i"), *ProposedName, CountPostfix));
		++CountPostfix;
	}

	return UniqueGraphName;
}

bool FBlueprintEditorUtils::CheckIfNodeConnectsToSelection(UEdGraphNode* InNode, const TSet<UEdGraphNode*>& InSelectionSet)
{
	for (int32 PinIndex = 0; PinIndex < InNode->Pins.Num(); ++PinIndex)
	{
		UEdGraphPin* Pin = InNode->Pins[PinIndex];
		if(Pin->Direction == EGPD_Input && Pin->PinType.PinCategory != UEdGraphSchema_K2::PC_Exec)
		{
			for (int32 LinkIndex = 0; LinkIndex < Pin->LinkedTo.Num(); ++LinkIndex)
			{
				UEdGraphPin* LinkedToPin = Pin->LinkedTo[LinkIndex];

				// The InNode, which is NOT in the new function, is checking if one of it's pins IS in the function, return true if it is. If not, check the node.
				if(InSelectionSet.Contains(LinkedToPin->GetOwningNode()))
				{
					return true;
				}

				// Check the node recursively to see if it is connected back with selection.
				if(CheckIfNodeConnectsToSelection(LinkedToPin->GetOwningNode(), InSelectionSet))
				{
					return true;
				}
			}
		}
	}

	return false;
}

bool FBlueprintEditorUtils::CheckIfSelectionIsCycling(const TSet<UEdGraphNode*>& InSelectionSet, FCompilerResultsLog& InMessageLog)
{
	for (UEdGraphNode* Node : InSelectionSet)
	{
		if (Node)
		{
			for (int32 PinIndex = 0; PinIndex < Node->Pins.Num(); ++PinIndex)
			{
				UEdGraphPin* Pin = Node->Pins[PinIndex];
				if(Pin->Direction == EGPD_Input && Pin->PinType.PinCategory != UEdGraphSchema_K2::PC_Exec)
				{
					for (int32 LinkIndex = 0; LinkIndex < Pin->LinkedTo.Num(); ++LinkIndex)
					{
						UEdGraphPin* LinkedToPin = Pin->LinkedTo[LinkIndex];

						// Check to see if this node, which is IN the selection, has any connections OUTSIDE the selection
						// If it does, check to see if those nodes have any connections IN the selection
						if(!InSelectionSet.Contains(LinkedToPin->GetOwningNode()))
						{
							if(CheckIfNodeConnectsToSelection(LinkedToPin->GetOwningNode(), InSelectionSet))
							{
								InMessageLog.Error(*LOCTEXT("DependencyCyleDetected_Error", "Dependency cycle detected, preventing node @@ from being scheduled").ToString(), LinkedToPin->GetOwningNode());
								return true;
							}
						}
					}
				}
			}
		}
	}

	return false;
}

bool FBlueprintEditorUtils::IsPaletteActionReadOnly(TSharedPtr<FEdGraphSchemaAction> ActionIn, TSharedPtr<FBlueprintEditor> const BlueprintEditorIn)
{
	check(BlueprintEditorIn.IsValid());
	bool bIsReadOnly = false;
	if(!BlueprintEditorIn->InEditingMode())
	{
		bIsReadOnly = true;
	}
	else
	{
		UBlueprint const* const BlueprintObj = BlueprintEditorIn->GetBlueprintObj();	
		if(ActionIn->GetTypeId() == FEdGraphSchemaAction_K2Graph::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2Graph* GraphAction = (FEdGraphSchemaAction_K2Graph*)ActionIn.Get();
			// No graph is evidence of an overridable function, don't let the user modify it
			if(GraphAction->EdGraph == nullptr)
			{
				bIsReadOnly = true;
			}
			else
			{
				// Graphs that cannot be deleted or re-named are read-only
				if ( !(GraphAction->EdGraph->bAllowDeletion || GraphAction->EdGraph->bAllowRenaming) )
				{
					bIsReadOnly = true;
				}
				else
				{
					if(GraphAction->GraphType == EEdGraphSchemaAction_K2Graph::Function)
					{
						// Check if the function is an override
						UFunction* OverrideFunc = FindUField<UFunction>(BlueprintObj->ParentClass, GraphAction->FuncName);
						if ( OverrideFunc != nullptr )
						{
							bIsReadOnly = true;
						}
					}
					else if(GraphAction->GraphType == EEdGraphSchemaAction_K2Graph::Interface)
					{
						// Interfaces cannot be renamed
						bIsReadOnly = true;
					}
				}
			}
		}
		else if(ActionIn->GetTypeId() == FEdGraphSchemaAction_K2Var::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2Var* VarAction = (FEdGraphSchemaAction_K2Var*)ActionIn.Get();

			bIsReadOnly = true;

			if( FBlueprintEditorUtils::FindNewVariableIndex(BlueprintObj, VarAction->GetVariableName()) != INDEX_NONE)
			{
				bIsReadOnly = false;
			}
			else if(BlueprintObj->FindTimelineTemplateByVariableName(VarAction->GetVariableName()))
			{
				bIsReadOnly = false;
			}
			else if(BlueprintEditorIn->CanAccessComponentsMode())
			{
				// Wasn't in the introduced variable list; try to find the associated SCS node
				//@TODO: The SCS-generated variables should be in the variable list and have a link back;
				// As it stands, you cannot do any metadata operations on a SCS variable, and you have to do icky code like the following
				TArray<USCS_Node*> Nodes = BlueprintObj->SimpleConstructionScript->GetAllNodes();
				for (TArray<USCS_Node*>::TConstIterator NodeIt(Nodes); NodeIt; ++NodeIt)
				{
					USCS_Node* CurrentNode = *NodeIt;
					if (CurrentNode->GetVariableName() == VarAction->GetVariableName())
					{
						bIsReadOnly = false;
						break;
					}
				}
			}
		}
		else if(ActionIn->GetTypeId() == FEdGraphSchemaAction_K2Delegate::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2Delegate* DelegateAction = (FEdGraphSchemaAction_K2Delegate*)ActionIn.Get();

			if( FBlueprintEditorUtils::FindNewVariableIndex(BlueprintObj, DelegateAction->GetDelegateName()) == INDEX_NONE)
			{
				bIsReadOnly = true;
			}
		}
		else if (ActionIn->GetTypeId() == FEdGraphSchemaAction_K2Event::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2Event* EventAction = (FEdGraphSchemaAction_K2Event*)ActionIn.Get();
			UK2Node* AssociatedNode = EventAction->NodeTemplate;

			bIsReadOnly = (AssociatedNode == nullptr) || (!AssociatedNode->GetCanRenameNode());
		}
		else if (ActionIn->GetTypeId() == FEdGraphSchemaAction_K2InputAction::StaticGetTypeId())
		{
			bIsReadOnly = true;
		}
	}

	return bIsReadOnly;
}

struct FUberGraphHelper
{
	static void GetAll(const UBlueprint* Blueprint, TArray<UEdGraph*>& OutGraphs)
	{
		for (UEdGraph* UberGraph : Blueprint->UbergraphPages)
		{
			OutGraphs.Add(UberGraph);
			UberGraph->GetAllChildrenGraphs(OutGraphs);
		}
	}
};

FName FBlueprintEditorUtils::GetFunctionNameFromClassByGuid(const UClass* InClass, const FGuid FunctionGuid)
{
	TArray<UBlueprint*> Blueprints;
	UBlueprint::GetBlueprintHierarchyFromClass(InClass, Blueprints);

	for (int32 BPIndex = 0; BPIndex < Blueprints.Num(); ++BPIndex)
	{
		UBlueprint* Blueprint = Blueprints[BPIndex];
		for (UEdGraph* FunctionGraph : Blueprint->FunctionGraphs)
		{
			if (FunctionGraph && FunctionGraph->GraphGuid == FunctionGuid)
			{
				return FunctionGraph->GetFName();
			}
		}

		for (UEdGraph* FunctionGraph : Blueprint->DelegateSignatureGraphs)
		{
			if (FunctionGraph && FunctionGraph->GraphGuid == FunctionGuid)
			{
				const FString Name = FunctionGraph->GetName() + HEADER_GENERATED_DELEGATE_SIGNATURE_SUFFIX;
				return FName(*Name);
			}
		}

		//FUNCTIONS BASED ON CUSTOM EVENTS:
		TArray<UEdGraph*> UberGraphs;
		FUberGraphHelper::GetAll(Blueprint, UberGraphs);
		for (const UEdGraph* UberGraph : UberGraphs)
		{
			TArray<UK2Node_CustomEvent*> CustomEvents;
			UberGraph->GetNodesOfClass(CustomEvents);
			for (const UK2Node_CustomEvent* CustomEvent : CustomEvents)
			{
				if (!CustomEvent->bOverrideFunction && (CustomEvent->NodeGuid == FunctionGuid))
				{
					ensure(CustomEvent->CustomFunctionName != NAME_None);
					return CustomEvent->CustomFunctionName;
				}
			}
		}
	}

	return NAME_None;
}

bool FBlueprintEditorUtils::GetFunctionGuidFromClassByFieldName(const UClass* InClass, const FName FunctionName, FGuid& FunctionGuid)
{
	if (FunctionName != NAME_None)
	{
		TArray<UBlueprint*> Blueprints;
		UBlueprint::GetBlueprintHierarchyFromClass(InClass, Blueprints);

		for (int32 BPIndex = 0; BPIndex < Blueprints.Num(); ++BPIndex)
		{
			UBlueprint* Blueprint = Blueprints[BPIndex];
			for (UEdGraph* FunctionGraph : Blueprint->FunctionGraphs)
			{
				if (FunctionGraph && FunctionGraph->GetFName() == FunctionName)
				{
					FunctionGuid = FunctionGraph->GraphGuid;
					return true;
				}
			}

			FString BaseDelegateSignatureName = FunctionName.ToString();
			if (BaseDelegateSignatureName.RemoveFromEnd(HEADER_GENERATED_DELEGATE_SIGNATURE_SUFFIX))
			{
				const FName GraphName(*BaseDelegateSignatureName);
				for (UEdGraph* FunctionGraph : Blueprint->DelegateSignatureGraphs)
				{
					if (FunctionGraph && FunctionGraph->GetFName() == GraphName)
					{
						FunctionGuid = FunctionGraph->GraphGuid;
						return true;
					}
				}
			}

			TArray<UEdGraph*> UberGraphs;
			FUberGraphHelper::GetAll(Blueprint, UberGraphs);
			for (const UEdGraph* UberGraph : UberGraphs)
			{
				TArray<UK2Node_Event*> EventNodes;
				UberGraph->GetNodesOfClass(EventNodes);
				for (const UK2Node_Event* EventNode : EventNodes)
				{
					if (EventNode->NodeGuid.IsValid())
					{
						if (const UK2Node_CustomEvent* CustomEventNode = Cast<UK2Node_CustomEvent>(EventNode))
						{
							if (CustomEventNode->CustomFunctionName == FunctionName)
							{
								FunctionGuid = EventNode->NodeGuid;
								return true;
							}
						}
						else if (EventNode->EventReference.GetMemberName() == FunctionName)
						{
							FunctionGuid = EventNode->NodeGuid;
							return true;
						}
					}
				}
			}
		}
	}

	FunctionGuid.Invalidate();

	return false;
}

UK2Node_EditablePinBase* FBlueprintEditorUtils::GetEntryNode(const UEdGraph* InGraph)
{
	UK2Node_EditablePinBase* Result = nullptr;
	if (InGraph)
	{
		TArray<UK2Node_FunctionEntry*> EntryNodes;
		InGraph->GetNodesOfClass(EntryNodes);
		if (EntryNodes.Num() > 0)
		{
			if (EntryNodes[0]->IsEditable())
			{
				Result = EntryNodes[0];
			}
		}
		else
		{
			TArray<UK2Node_Tunnel*> TunnelNodes;
			InGraph->GetNodesOfClass(TunnelNodes);

			if (TunnelNodes.Num() > 0)
			{
				// Iterate over the tunnel nodes, and try to find an entry and exit
				for (int32 i = 0; i < TunnelNodes.Num(); i++)
				{
					UK2Node_Tunnel* Node = TunnelNodes[i];
					// Composite nodes should never be considered for function entry / exit, since we're searching for a graph's terminals
					if (Node->IsEditable() && !Node->IsA(UK2Node_Composite::StaticClass()))
					{
						if (Node->bCanHaveOutputs)
						{
							Result = Node;
							break;
						}
					}
				}
			}
		}
	}
	return Result;
}

void FBlueprintEditorUtils::GetEntryAndResultNodes(const UEdGraph* InGraph, TWeakObjectPtr<class UK2Node_EditablePinBase>& OutEntryNode, TWeakObjectPtr<class UK2Node_EditablePinBase>& OutResultNode)
{
	if (InGraph)
	{
		// There are a few different potential configurations for editable graphs (FunctionEntry/Result, Tunnel Pairs, etc).
		// Step through each case until we find one that matches what appears to be in the graph.  This could be improved if
		// we want to add more robust typing to the graphs themselves

		// Case 1:  Function Entry / Result Pair ------------------
		TArray<UK2Node_FunctionEntry*> EntryNodes;
		InGraph->GetNodesOfClass(EntryNodes);

		if (EntryNodes.Num() > 0)
		{
			if (EntryNodes[0]->IsEditable())
			{
				OutEntryNode = EntryNodes[0];

				// Find a result node
				TArray<UK2Node_FunctionResult*> ResultNodes;
				InGraph->GetNodesOfClass(ResultNodes);

				UK2Node_FunctionResult* ResultNode = ResultNodes.Num() ? ResultNodes[0] : nullptr;
				// Note:  we assume that if the entry is editable, the result is too (since the entry node is guaranteed to be there on graph creation, but the result isn't)
				if( ResultNode )
				{
					OutResultNode = ResultNode;
				}
			}
		}
		else
		{
			// Case 2:  Tunnel Pair -----------------------------------
			TArray<UK2Node_Tunnel*> TunnelNodes;
			InGraph->GetNodesOfClass(TunnelNodes);

			if (TunnelNodes.Num() > 0)
			{
				// Iterate over the tunnel nodes, and try to find an entry and exit
				for (int32 i = 0; i < TunnelNodes.Num(); i++)
				{
					UK2Node_Tunnel* Node = TunnelNodes[i];
					// Composite nodes should never be considered for function entry / exit, since we're searching for a graph's terminals
					if (Node->IsEditable() && !Node->IsA(UK2Node_Composite::StaticClass()))
					{
						if (Node->bCanHaveOutputs)
						{
							ensure(!OutEntryNode.IsValid());
							OutEntryNode = Node;
						}
						else if (Node->bCanHaveInputs)
						{
							ensure(!OutResultNode.IsValid());
							OutResultNode = Node;
						}
					}
				}
			}
		}
	}
}

FKismetUserDeclaredFunctionMetadata* FBlueprintEditorUtils::GetGraphFunctionMetaData(const UEdGraph* InGraph)
{
	if (InGraph)
	{
		UK2Node_EditablePinBase* FunctionEntryNode = GetEntryNode(InGraph);
		if (UK2Node_FunctionEntry* TypedEntryNode = Cast<UK2Node_FunctionEntry>(FunctionEntryNode))
		{
			return &(TypedEntryNode->MetaData);
		}
		else if (UK2Node_Tunnel* TunnelNode = ExactCast<UK2Node_Tunnel>(FunctionEntryNode))
		{
			// Must be exactly a tunnel, not a macro instance
			return &(TunnelNode->MetaData);
		}
	}

	return nullptr;
}

void FBlueprintEditorUtils::ModifyFunctionMetaData(const UEdGraph* InGraph)
{
	if (InGraph)
	{
		UK2Node_EditablePinBase* FunctionEntryNode = GetEntryNode(InGraph);
		if (UK2Node_FunctionEntry* TypedEntryNode = Cast<UK2Node_FunctionEntry>(FunctionEntryNode))
		{
			TypedEntryNode->Modify();
		}
		else if (UK2Node_Tunnel* TunnelNode = ExactCast<UK2Node_Tunnel>(FunctionEntryNode))
		{
			TunnelNode->Modify();
		}
	}
}

FText FBlueprintEditorUtils::GetGraphDescription(const UEdGraph* InGraph)
{
	if (FKismetUserDeclaredFunctionMetadata* MetaData = GetGraphFunctionMetaData(InGraph))
	{
		 return MetaData->ToolTip;
	}

	return LOCTEXT( "NoGraphTooltip", "(None)" );
}

bool FBlueprintEditorUtils::CheckIfGraphHasLatentFunctions(UEdGraph* InGraph)
{
	struct Local
	{
		static bool CheckIfGraphHasLatentFunctions(UEdGraph* InGraphToCheck, TArray<UEdGraph*>& InspectedGraphList)
		{
			UK2Node_EditablePinBase* EntryNode = GetEntryNode(InGraphToCheck);

			UK2Node_Tunnel* TunnelNode = ExactCast<UK2Node_Tunnel>(EntryNode);
			if(!TunnelNode)
			{
				// No tunnel, no metadata.
				return false;
			}

			if(TunnelNode->MetaData.HasLatentFunctions != INDEX_NONE)
			{
				return TunnelNode->MetaData.HasLatentFunctions > 0;
			}
			else
			{
				// Add all graphs to the list of already inspected, this prevents circular inclusion issues.
				InspectedGraphList.Add(InGraphToCheck);

				for( const UEdGraphNode* Node : InGraphToCheck->Nodes )
				{
					// BP graphs can sometimes have non-UK2Nodes (eg: UEdGraphNode_Comment)
					const UK2Node* K2Node = Cast<UK2Node>(Node);
					if (K2Node)
					{
						if (K2Node->IsLatentForMacros())
						{
							TunnelNode->MetaData.HasLatentFunctions = 1;
							return true;
						}
						else if (const UK2Node_MacroInstance* MacroInstanceNode = Cast<UK2Node_MacroInstance>(K2Node))
						{
							// Any macro graphs that haven't already been checked need to be checked for latent function calls
							if (InspectedGraphList.Find(MacroInstanceNode->GetMacroGraph()) == INDEX_NONE)
							{
								if (CheckIfGraphHasLatentFunctions(MacroInstanceNode->GetMacroGraph(), InspectedGraphList))
								{
									TunnelNode->MetaData.HasLatentFunctions = 1;
									return true;
								}
							}
						}
						else if (const UK2Node_Composite* CompositeNode = Cast<UK2Node_Composite>(K2Node))
						{
							// Any collapsed graphs that haven't already been checked need to be checked for latent function calls
							if (InspectedGraphList.Find(CompositeNode->BoundGraph) == INDEX_NONE)
							{
								if (CheckIfGraphHasLatentFunctions(CompositeNode->BoundGraph, InspectedGraphList))
								{
									TunnelNode->MetaData.HasLatentFunctions = 1;
									return true;
								}
							}
						}
					}
				}

				TunnelNode->MetaData.HasLatentFunctions = 0;
				return false;
			}
		}
	};

	TArray<UEdGraph*> InspectedGraphList;
	return Local::CheckIfGraphHasLatentFunctions(InGraph, InspectedGraphList);
}

void FBlueprintEditorUtils::PostSetupObjectPinType(UBlueprint* InBlueprint, FBPVariableDescription& InOutVarDesc)
{
	if ((InOutVarDesc.VarType.PinCategory == UEdGraphSchema_K2::PC_Object) || (InOutVarDesc.VarType.PinCategory == UEdGraphSchema_K2::PC_Interface))
	{
		if (InOutVarDesc.VarType.PinSubCategory == UEdGraphSchema_K2::PSC_Self)
		{
			InOutVarDesc.VarType.PinSubCategory = NAME_None;
			InOutVarDesc.VarType.PinSubCategoryObject = *InBlueprint->GeneratedClass;
		}
		else if (!InOutVarDesc.VarType.PinSubCategoryObject.IsValid())
		{
			// Fall back to UObject if the given type is not valid. This can happen for example if a variable is removed from
			// a Blueprint parent class along with the variable's type and the user then attempts to recreate the missing variable
			// through a stale variable node's context menu in a child Blueprint graph.
			InOutVarDesc.VarType.PinSubCategory = NAME_None;
			InOutVarDesc.VarType.PinSubCategoryObject = UObject::StaticClass();
		}

		// if it's a PC_Object, then it should have an associated UClass object
		check(InOutVarDesc.VarType.PinSubCategoryObject.IsValid());
		const UClass* ClassObject = Cast<UClass>(InOutVarDesc.VarType.PinSubCategoryObject.Get());
		check(ClassObject != nullptr);

		if (ClassObject->IsChildOf(AActor::StaticClass()))
		{
			// prevent Actor variables from having default values (because Blueprint templates are library elements that can 
			// bridge multiple levels and different levels might not have the actor that the default is referencing).
			InOutVarDesc.PropertyFlags |= CPF_DisableEditOnTemplate;
		}
	}
}

const FSlateBrush* FBlueprintEditorUtils::GetIconFromPin( const FEdGraphPinType& PinType, bool bIsLarge )
{
	const FSlateBrush* IconBrush = FAppStyle::GetBrush(TEXT("Kismet.VariableList.TypeIcon"));
	const UObject* PinSubObject = PinType.PinSubCategoryObject.Get();
	if( PinType.IsArray() && PinType.PinCategory != UEdGraphSchema_K2::PC_Exec )
	{
		IconBrush = FAppStyle::GetBrush(TEXT("Kismet.VariableList.ArrayTypeIcon"));
	}
	else if (PinType.IsMap() && PinType.PinCategory != UEdGraphSchema_K2::PC_Exec)
	{
		IconBrush = FAppStyle::GetBrush(TEXT("Kismet.VariableList.MapKeyTypeIcon"));
	}
	else if (PinType.IsSet() && PinType.PinCategory != UEdGraphSchema_K2::PC_Exec)
	{
		if( bIsLarge )
		{
			IconBrush = FAppStyle::GetBrush(TEXT("Kismet.VariableList.SetTypeIconLarge"));
		}
		else
		{
			IconBrush = FAppStyle::GetBrush(TEXT("Kismet.VariableList.SetTypeIcon"));
		}
	}
	else if (PinType.PinCategory == UEdGraphSchema_K2::PC_MCDelegate)
	{
		IconBrush = FAppStyle::GetBrush(TEXT("GraphEditor.Delegate_16x"));
	}
	// FindObject will crash if called during save - and we have reported crashes here
	// due to the save progress dialog invoking this function somehow
	else if( PinSubObject && !UE::IsSavingPackage(nullptr)) 
	{
		UClass* VarClass = FindObject<UClass>(nullptr, *PinSubObject->GetFullName());
		if( VarClass )
		{
			IconBrush = FSlateIconFinder::FindIconBrushForClass( VarClass );
		}
	}
	return IconBrush;
}

const FSlateBrush* FBlueprintEditorUtils::GetSecondaryIconFromPin(const FEdGraphPinType& PinType)
{
	if (PinType.IsMap() && PinType.PinCategory != UEdGraphSchema_K2::PC_Exec)
	{
		return FAppStyle::GetBrush(TEXT("Kismet.VariableList.MapValueTypeIcon"));
	}
	return nullptr;
}

bool FBlueprintEditorUtils::HasGetTypeHash(const FEdGraphPinType& PinType)
{
	if(PinType.PinCategory == UEdGraphSchema_K2::PC_Boolean)
	{
		return false;
	}

	if (PinType.PinCategory == UEdGraphSchema_K2::PC_Text)
	{
		return false;
	}

	if (PinType.PinCategory != UEdGraphSchema_K2::PC_Struct)
	{
		// even object or class types can be hashed, no reason to investigate further
		return true;
	}

	const UScriptStruct* StructType = Cast<const UScriptStruct>(PinType.PinSubCategoryObject.Get());
	if( StructType )
	{
		return StructHasGetTypeHash(StructType);
	}
	return false;
}

bool FBlueprintEditorUtils::PropertyHasGetTypeHash(const FProperty* PropertyType)
{
	return PropertyType->HasAllPropertyFlags(CPF_HasGetValueTypeHash);
} 

bool FBlueprintEditorUtils::StructHasGetTypeHash(const UScriptStruct* StructType)
{
	if (StructType->IsNative())
	{
		return StructType->GetCppStructOps() && StructType->GetCppStructOps()->HasGetTypeHash();
	}
	else
	{
		// if every member can be hashed (or is a FBoolProperty, which is specially 
		// handled by UScriptStruct::GetStructTypeHash) then we can hash the struct:
		for (TFieldIterator<FProperty> It(StructType); It; ++It)
		{
			if (CastField<FBoolProperty>(*It))
			{
				continue;
			}
			else
			{
				if (!FBlueprintEditorUtils::PropertyHasGetTypeHash(*It) )
				{
					return false;
				}
			}
		}
		return true;
	}
}

FText FBlueprintEditorUtils::GetFriendlyClassDisplayName(const UClass* Class)
{
	if (Class != nullptr)
	{
		return Class->GetDisplayNameText();
	}
	else
	{
		return LOCTEXT("ClassIsNull", "None");
	}
}

FString FBlueprintEditorUtils::GetClassNameWithoutSuffix(const UClass* Class)
{
	if (Class != nullptr)
	{
		FString Result = Class->GetName();
		if (Class->ClassGeneratedBy != nullptr)
		{
			Result.RemoveFromEnd(TEXT("_C"), ESearchCase::CaseSensitive);
		}

		return Result;
	}
	else
	{
		return LOCTEXT("ClassIsNull", "None").ToString();
	}
}

FText FBlueprintEditorUtils::GetDeprecatedMemberMenuItemName(const FText& MemberName)
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("MemberName"), MemberName);
	return FText::Format(LOCTEXT("DeprecatedMemberMenuItemName", "{MemberName} (Deprecated)"), Args);
}

FText FBlueprintEditorUtils::GetDeprecatedMemberUsageNodeWarning(const FText& MemberName, const FText& DetailedMessage)
{
	static FText UnknownName = LOCTEXT("UnknownDeprecatedMemberName", "[unknown]");
	static FText DefaultMessage = LOCTEXT("DefaultDeprecatedMemberUsageDetails", "Please replace or remove it.");

	FFormatNamedArguments Args;
	Args.Add("MemberName", ensure(!MemberName.IsEmpty()) ? MemberName : UnknownName);
	Args.Add("DetailedMessage", DetailedMessage.IsEmpty() ? DefaultMessage : DetailedMessage);
	return FText::Format(LOCTEXT("DeprecatedMemberUsageNodeWarning", "@@: Usage of '{MemberName}' has been deprecated. {DetailedMessage}"), Args);
}

EEdGraphNodeDeprecationMessageType FBlueprintEditorUtils::GetDeprecatedMessageType(const FString& TypeString)
{
	if (TypeString.Equals(TEXT("None"), ESearchCase::IgnoreCase))
	{
		return EEdGraphNodeDeprecationMessageType::None;
	}
	else if (TypeString.Equals(TEXT("Note"), ESearchCase::IgnoreCase))
	{
		return EEdGraphNodeDeprecationMessageType::Note;
	}
	
	// Default to warning
	return EEdGraphNodeDeprecationMessageType::Warning;
}

UK2Node_FunctionResult* FBlueprintEditorUtils::FindOrCreateFunctionResultNode(UK2Node_EditablePinBase* InFunctionEntryNode)
{
	UK2Node_FunctionResult* FunctionResult = nullptr;

	if (InFunctionEntryNode)
	{
		UEdGraph* Graph = InFunctionEntryNode->GetGraph();

		TArray<UK2Node_FunctionResult*> ResultNode;
		if (Graph)
		{
			Graph->GetNodesOfClass(ResultNode);
		}

		if (Graph && ResultNode.Num() == 0)
		{
			FGraphNodeCreator<UK2Node_FunctionResult> ResultNodeCreator(*Graph);
			FunctionResult = ResultNodeCreator.CreateNode();

			const UEdGraphSchema_K2* Schema = Cast<const UEdGraphSchema_K2>(FunctionResult->GetSchema());
			FunctionResult->NodePosX = InFunctionEntryNode->NodePosX + InFunctionEntryNode->NodeWidth + 256;
			FunctionResult->NodePosY = InFunctionEntryNode->NodePosY;
			FunctionResult->bIsEditable = true;
			UEdGraphSchema_K2::SetNodeMetaData(FunctionResult, FNodeMetadata::DefaultGraphNode);
			ResultNodeCreator.Finalize();

			// Connect the function entry to the result node, if applicable
			UEdGraphPin* ThenPin = Schema->FindExecutionPin(*InFunctionEntryNode, EGPD_Output);
			UEdGraphPin* ReturnPin = Schema->FindExecutionPin(*FunctionResult, EGPD_Input);

			if(ThenPin->LinkedTo.Num() == 0) 
			{
				ThenPin->MakeLinkTo(ReturnPin);
			}
			else
			{
				// Bump the result node up a bit, so it's less likely to fall behind the node the entry is already connected to
				FunctionResult->NodePosY -= 100;
			}
		}
		else
		{
			FunctionResult = ResultNode[0];
		}
	}

	return FunctionResult;
}

void FBlueprintEditorUtils::HandleDisableEditableWhenInherited(UObject* ModifiedObject, TArray<UObject*>& ArchetypeInstances)
{
	for (int32 Index = ArchetypeInstances.Num() - 1; Index >= 0; --Index)
	{
		UObject* ArchetypeInstance = ArchetypeInstances[Index];
		if (ArchetypeInstance != ModifiedObject)
		{
			UBlueprintGeneratedClass* BPGC = Cast<UBlueprintGeneratedClass>(ArchetypeInstance->GetOuter());
			if (BPGC)
			{
				if (UInheritableComponentHandler* ICH = BPGC->GetInheritableComponentHandler(false))
				{
					ICH->RemoveOverridenComponentTemplate(ICH->FindKey(CastChecked<UActorComponent>(ArchetypeInstance)));
				}
			}
		}
	}
}

UClass* FBlueprintEditorUtils::GetNativeParent(const UBlueprint* BP)
{
	UClass* Ret = BP->ParentClass;
	while(Ret && !Ret->HasAnyClassFlags(CLASS_Native))
	{
		Ret = Ret->GetSuperClass();
	}
	return Ret;
}

UClass* FBlueprintEditorUtils::GetTypeForPin(const UEdGraphPin& Pin)
{
	UClass* Ret = Cast<UClass>(Pin.PinType.PinSubCategoryObject.Get());

	if(Ret == nullptr && Pin.PinType.PinSubCategory == UEdGraphSchema_K2::PSC_Self)
	{
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(Pin.GetOwningNode());
		Ret = (Blueprint->GeneratedClass != NULL) ? Blueprint->GeneratedClass : Blueprint->ParentClass;
	}

	return Ret;
}

bool FBlueprintEditorUtils::ImplementsGetWorld(const UBlueprint* BP)
{
	if(UClass* NativeParent = GetNativeParent(BP))
	{
		return NativeParent->GetDefaultObject()->ImplementsGetWorld();
	}
	return false;
}

bool FBlueprintEditorUtils::ShouldShowWorldContextPin(const UBlueprint* BP)
{
	const UClass* ParentClass = BP ? *BP->ParentClass : nullptr;

	if (ParentClass && ParentClass->HasMetaDataHierarchical(FBlueprintMetadata::MD_ShowWorldContextPin) != nullptr)
	{
		// Always show even if not needed
		return true;
	}

	if (BP && BP->BlueprintType == BPTYPE_FunctionLibrary)
	{
		// NOTE: This was the old behavior, setting it to false would hide the pin which could break existing blueprints
		return true;
	}

	return !ImplementsGetWorld(BP);
}

struct FComponentInstancingDataUtils
{
	// Recursively gathers properties that differ from class/struct defaults, and fills out the cooked property list structure.
	static void RecursivePropertyGather(UStruct* InStruct, const uint8* DataPtr, const uint8* DefaultDataPtr, FBlueprintCookedComponentInstancingData& OutData)
	{
		// Determines if a struct value matches the default value using native comparison only (if implemented).
		auto IsNativeIdentical = [](const UScriptStruct* InStruct, const uint8* InValuePtr, const uint8* InDefaultValuePtr, uint32 PortFlags)
		{
			bool bIsIdentical = false;
			UScriptStruct::ICppStructOps* CppStructOps = InStruct->GetCppStructOps();
			if (CppStructOps
				&& CppStructOps->HasIdentical()
				&& CppStructOps->Identical(InValuePtr, InDefaultValuePtr, PortFlags, bIsIdentical))
			{
				return bIsIdentical;
			}

			// Not bound or no actual comparison was performed at the native level.
			return true;
		};

		for (FProperty* Property = InStruct->PropertyLink; Property; Property = Property->PropertyLinkNext)
		{
			// Skip editor-only properties since they won't be compiled in a non-editor configuration. Also skip transient and deprecated properties since they won't be serialized on save/duplicate. 
			if (!Property->IsEditorOnlyProperty()
				&& !Property->HasAnyPropertyFlags(CPF_Transient | CPF_DuplicateTransient | CPF_NonPIEDuplicateTransient | CPF_Deprecated))
			{
				for (int32 Idx = 0; Idx < Property->ArrayDim; Idx++)
				{
					const uint8* PropertyValue = Property->ContainerPtrToValuePtr<uint8>(DataPtr, Idx);
					const uint8* DefaultPropertyValue = Property->ContainerPtrToValuePtrForDefaults<uint8>(InStruct, DefaultDataPtr, Idx);

					FBlueprintComponentChangedPropertyInfo ChangedPropertyInfo;
					ChangedPropertyInfo.PropertyName = Property->GetFName();
					ChangedPropertyInfo.ArrayIndex = Idx;
					ChangedPropertyInfo.PropertyScope = InStruct;

					if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
					{
						int32 NumChangedProperties = OutData.ChangedPropertyList.Num();

						RecursivePropertyGather(StructProperty->Struct, PropertyValue, DefaultPropertyValue, OutData);

						// Prepend the struct property only if there is at least one changed sub-property, or otherwise if there are any native differences.
						if (NumChangedProperties < OutData.ChangedPropertyList.Num()
							|| !IsNativeIdentical(StructProperty->Struct, PropertyValue, DefaultPropertyValue, PPF_None))
						{
							OutData.ChangedPropertyList.Insert(ChangedPropertyInfo, NumChangedProperties);
						}
					}
					else if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
					{
						FScriptArrayHelper ArrayValueHelper(ArrayProperty, PropertyValue);
						FScriptArrayHelper DefaultArrayValueHelper(ArrayProperty, DefaultPropertyValue);

						int32 NumChangedProperties = OutData.ChangedPropertyList.Num();
						FBlueprintComponentChangedPropertyInfo ChangedArrayPropertyInfo = ChangedPropertyInfo;

						for (int32 ArrayValueIndex = 0; ArrayValueIndex < ArrayValueHelper.Num(); ++ArrayValueIndex)
						{
							ChangedArrayPropertyInfo.ArrayIndex = ArrayValueIndex;
							const uint8* ArrayPropertyValue = ArrayValueHelper.GetRawPtr(ArrayValueIndex);

							if (ArrayValueIndex < DefaultArrayValueHelper.Num())
							{
								const uint8* DefaultArrayPropertyValue = DefaultArrayValueHelper.GetRawPtr(ArrayValueIndex);

								if (FStructProperty* InnerStructProperty = CastField<FStructProperty>(ArrayProperty->Inner))
								{
									int32 NumChangedArrayProperties = OutData.ChangedPropertyList.Num();

									RecursivePropertyGather(InnerStructProperty->Struct, ArrayPropertyValue, DefaultArrayPropertyValue, OutData);

									// Prepend the struct property only if there is at least one changed sub-property, or otherwise if there are any native differences.
									if (NumChangedArrayProperties < OutData.ChangedPropertyList.Num()
										|| !IsNativeIdentical(InnerStructProperty->Struct, ArrayPropertyValue, DefaultArrayPropertyValue, PPF_None))
									{
										OutData.ChangedPropertyList.Insert(ChangedArrayPropertyInfo, NumChangedArrayProperties);
									}
								}
								else if (!ArrayProperty->Inner->Identical(ArrayPropertyValue, DefaultArrayPropertyValue, PPF_None))
								{
									// Emit the index of the individual array value that differs from the default value
									OutData.ChangedPropertyList.Add(ChangedArrayPropertyInfo);
								}
							}
							else
							{
								// Emit the "end" of differences with the default value (signals that remaining values should be copied in full)
								ChangedArrayPropertyInfo.PropertyName = NAME_None;
								OutData.ChangedPropertyList.Add(ChangedArrayPropertyInfo);

								// Don't need to record anything else.
								break;
							}
						}

						// Prepend the array property as changed only if the sizes differ and/or if we also wrote out any of the inner value as changed.
						if (ArrayValueHelper.Num() != DefaultArrayValueHelper.Num() || NumChangedProperties < OutData.ChangedPropertyList.Num())
						{
							OutData.ChangedPropertyList.Insert(ChangedPropertyInfo, NumChangedProperties);
						}
					}
					else if (!Property->Identical(PropertyValue, DefaultPropertyValue, PPF_None))
					{
						OutData.ChangedPropertyList.Add(ChangedPropertyInfo);
					}
				}
			}
		}
	}
};

void FBlueprintEditorUtils::BuildComponentInstancingData(UActorComponent* ComponentTemplate, FBlueprintCookedComponentInstancingData& OutData, bool bUseTemplateArchetype)
{
	if (ComponentTemplate)
	{
		UClass* ComponentTemplateClass = ComponentTemplate->GetClass();
		const UObject* ComponentDefaults = bUseTemplateArchetype ? ComponentTemplate->GetArchetype() : ComponentTemplateClass->GetDefaultObject(false);

		// Gather the set of properties that differ from the defaults.
		OutData.ChangedPropertyList.Empty();
		FComponentInstancingDataUtils::RecursivePropertyGather(ComponentTemplateClass, (uint8*)ComponentTemplate, (uint8*)ComponentDefaults, OutData);

		// Flag that cooked data has been built and is now considered to be valid.
		OutData.bHasValidCookedData = true;
	}
}

namespace 
{
	// This structure provides the ability to find/update the nodes primary object.  This must be specialized based
	// on the type of the object being found/updated
	template <typename TObjectType, bool bIsFind>
	struct FFindOrUpdateNodeHelper
	{
		template <typename FindExisting>
		static bool FindOrUpdateNode(UK2Node* Node, FindExisting& InFindExisting);
	};

	template <bool bIsFind>
	struct FFindOrUpdateNodeHelper<UScriptStruct, bIsFind>
	{
		template <typename FindExisting>
		static bool FindOrUpdateNode(UK2Node* Node, FindExisting InFindExisting)
		{
			// If this is a struct operation node operation on the changed struct we must reconstruct
			if (UK2Node_StructOperation* StructOpNode = Cast<UK2Node_StructOperation>(Node))
			{
				if (UScriptStruct* StructInNode = Cast<UScriptStruct>(StructOpNode->StructType))
				{
					if (TOptional<UScriptStruct*> NewStructInNode = InFindExisting(StructInNode))
					{
						if (!bIsFind)
						{
							StructOpNode->StructType = *NewStructInNode;
						}
						return true;
					}
				}
			}
			return false;
		}
	};

	template <bool bIsFind>
	struct FFindOrUpdateNodeHelper<UEnum, bIsFind>
	{
		template <typename FindExisting>
		static bool FindOrUpdateNode(UK2Node* Node, FindExisting InFindExisting)
		{
			if (INodeDependingOnEnumInterface* EnumInterface = Cast<INodeDependingOnEnumInterface>(Node))
			{
				if (UEnum* EnumInNode = EnumInterface->GetEnum())
				{
					if (TOptional<UEnum*> NewEnumInNode = InFindExisting(EnumInNode))
					{
						if (!bIsFind)
						{
							EnumInterface->ReloadEnum(*NewEnumInNode);
						}
						return true;
					}
				}
			}
			return false;
		}
	};

	// Scan all the nodes looking for references to objects. 
	template <typename TObject, bool bIsFind, typename FindExisting>
	void FindOrUpdateNodes(FBlueprintEditorUtils::FOnNodeFoundOrUpdated InOnNodeFoundOrUpdated, FindExisting InFindExisting)
	{
		for (TObjectIterator<UK2Node> It(RF_Transient | RF_ClassDefaultObject, /** bIncludeDerivedClasses */ true, /** InternalExcludeFlags */ EInternalObjectFlags::Garbage); It; ++It)
		{
			UK2Node* Node = *It;

			if (Node && !Node->HasAnyFlags(RF_Transient) && IsValidChecked(Node))
			{
				bool bReconstruct = FFindOrUpdateNodeHelper<TObject, bIsFind>::FindOrUpdateNode(Node, InFindExisting);

				// Look through the nodes pins and if any of them are split and the type of the split pin is a something we need to reconstruct
				if (!bIsFind || !bReconstruct)
				{
					for (UEdGraphPin* Pin : Node->Pins)
					{
						if (TObject* Object = Cast<TObject>(Pin->PinType.PinSubCategoryObject.Get()))
						{
							if (TOptional<TObject*> NewObject = InFindExisting(Object))
							{
								bReconstruct = true;
								if (bIsFind)
								{
									break;
								}
								Pin->PinType.PinSubCategoryObject = *NewObject;
							}
						}
					}
				}

				if (bReconstruct)
				{
					UBlueprint* FoundBlueprint = Node->HasValidBlueprint() ? Node->GetBlueprint() : nullptr;
					InOnNodeFoundOrUpdated(FoundBlueprint, Node);
				}
			}
		}
	}
}

void FBlueprintEditorUtils::FindScriptStructsInNodes(const TSet<UScriptStruct*>& Structs, FOnNodeFoundOrUpdated InOnNodeFoundOrUpdated)
{
	if (Structs.Num() == 0)
	{
		return;
	}

	FindOrUpdateNodes<UScriptStruct, true>(InOnNodeFoundOrUpdated, [&Structs](UScriptStruct* ScriptStruct)
		{
			return Structs.Contains(ScriptStruct) ? TOptional(ScriptStruct) : TOptional<UScriptStruct*>();
		}
	);
}

void FBlueprintEditorUtils::FindEnumsInNodes(const TSet<UEnum*>& Enums, FOnNodeFoundOrUpdated InOnNodeFoundOrUpdated)
{
	if (Enums.Num() == 0)
	{
		return;
	}

	FindOrUpdateNodes<UEnum, true>(InOnNodeFoundOrUpdated, [&Enums](UEnum* Enum)
		{
			return Enums.Contains(Enum) ? TOptional(Enum) : TOptional<UEnum*>();
		}
	);
}

void FBlueprintEditorUtils::UpdateScriptStructsInNodes(const TMap<UScriptStruct*, UScriptStruct*>& Structs, FOnNodeFoundOrUpdated InOnNodeFoundOrUpdated)
{
	if (Structs.Num() == 0)
	{
		return;
	}

	FindOrUpdateNodes<UScriptStruct, false>(InOnNodeFoundOrUpdated, [&Structs] (UScriptStruct* ScriptStruct)
		{
			UScriptStruct* const* Found = Structs.Find(ScriptStruct);
			return Found ? TOptional(*Found) : TOptional<UScriptStruct*>();
		}
	);
}

void FBlueprintEditorUtils::UpdateEnumsInNodes(const TMap<UEnum*, UEnum*>& Enums, FOnNodeFoundOrUpdated InOnNodeFoundOrUpdated)
{
	if (Enums.Num() == 0)
	{
		return;
	}

	FindOrUpdateNodes<UEnum, false>(InOnNodeFoundOrUpdated, [&Enums](UEnum* Enum)
		{
			UEnum* const* Found = Enums.Find(Enum);
			return Found ? TOptional(*Found) : TOptional<UEnum*>();
		}
	);
}

void FBlueprintEditorUtils::RecombineNestedSubPins(UK2Node* Node)
{
	checkSlow(Node);

	TArray<UEdGraphPin*> NestedSplitPins;
	for (int32 i = Node->Pins.Num() - 1; i >= 0; --i)
	{
		UEdGraphPin* Pin = Node->Pins[i];
		if (Pin->ParentPin != nullptr && Pin->ParentPin->ParentPin != nullptr && !Pin->bOrphanedPin)
		{
			NestedSplitPins.Add(Pin);

			// If there was nothing connected to or changed about this pin, then skip it
			if (Pin->LinkedTo.Num() > 0 || !Pin->DoesDefaultValueMatchAutogenerated())
			{
				// Otherwise add an orphan pin so warning/connections are not silently lost
				UEdGraphPin* OrphanPin = Node->CreatePin(Pin->Direction, Pin->PinType, Pin->PinName);
				OrphanPin->bOrphanedPin = true;
				OrphanPin->bNotConnectable = true;
				OrphanPin->DefaultValue = Pin->DefaultValue;
				OrphanPin->DefaultObject = Pin->DefaultObject;

				for (UEdGraphPin* OldLink : Pin->LinkedTo)
				{
					OrphanPin->MakeLinkTo(OldLink);
				}
			}
		}
	}

	// Wait to recombine because otherwise we could end up combining pins that that haven't had their orphan created yet
	const UEdGraphSchema* Schema = Node->GetSchema();
	for (int32 i = NestedSplitPins.Num() - 1; i >= 0; --i)
	{
		Schema->RecombinePin(NestedSplitPins[i]);
	}
}

static FAutoConsoleCommand AuditThreadSafeFunctions(
	TEXT("bp.AuditThreadSafeFunctions"),
	TEXT("Audit currently loaded thread safe functions. Writes results to the log."),
	FConsoleCommandDelegate::CreateLambda([]()
		{
			UE_LOGF(LogBlueprint, Display, "--- BEGIN audit all BlueprintThreadSafe functions ---");
			UE_LOGF(LogBlueprint, Display, "Name, Path, Type, BPCallType");

			for (TObjectIterator<UFunction> It; It; ++It)
			{
				UFunction* Function = *It;
				if (FBlueprintEditorUtils::HasFunctionBlueprintThreadSafeMetaData(Function))
				{
					const TCHAR* Native = Function->HasAnyFunctionFlags(FUNC_Native) ? TEXT("Native") : TEXT("Script");
					const TCHAR* Purity = [Function]()
					{
						if (Function->HasAnyFunctionFlags(FUNC_BlueprintCallable))
						{
							return Function->HasAllFunctionFlags(FUNC_BlueprintPure | FUNC_BlueprintCallable) ? TEXT("Pure") : TEXT("Callable");
						}

						return TEXT("NotCallable");
					}();
					UE_LOGF(LogBlueprint, Display, "%ls, %ls, %ls, %ls", *Function->GetName(), *Function->GetPathName(), Native, Purity);
				}
			}

			UE_LOGF(LogBlueprint, Display, "--- END audit all BlueprintThreadSafe functions ---");
		}));

static FAutoConsoleCommand AuditFunctionCallsForBlueprint(
	TEXT("bp.AuditFunctionCallsForBlueprint"),
	TEXT("Audit all functions called by a specified blueprint. Single argument supplies the asset to audit. Writes results to the log."),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& InArgs)
		{
			if (InArgs.Num() != 1)
			{
				return;
			}

			// Find our Blueprint & load it
			UBlueprint* Blueprint = LoadObject<UBlueprint>(nullptr, *InArgs[0]);
			if (Blueprint == nullptr)
			{
				UE_LOGF(LogBlueprint, Warning, "--- Could not load Blueprint %ls ---", *InArgs[0]);
				return;
			}

			if (Blueprint->GeneratedClass == nullptr)
			{
				UE_LOGF(LogBlueprint, Warning, "--- Blueprint %ls as a null GeneratedClass ---", *InArgs[0]);
				return;
			}

			UE_LOGF(LogBlueprint, Display, "--- BEGIN audit function calls for Blueprint %ls ---", *InArgs[0]);
			UE_LOGF(LogBlueprint, Display, "Name, Path, Type, BPCallType");

			struct FFunctionReferenceProcessor : public FSimpleReferenceProcessorBase
			{
				FORCEINLINE void HandleTokenStreamObjectReference(FGCArrayStruct& ObjectsToSerializeStruct, UObject* ReferencingObject, UObject*& Object, UE::GC::FTokenId, EGCTokenType TokenType, bool)
				{
					if (UFunction* Function = Cast<UFunction>(Object))
					{
						const TCHAR* Native = Function->HasAnyFunctionFlags(FUNC_Native) ? TEXT("Native") : TEXT("Script");
						const TCHAR* Purity = [Function]()
						{
							if (Function->HasAnyFunctionFlags(FUNC_BlueprintCallable))
							{
								return Function->HasAllFunctionFlags(FUNC_BlueprintPure | FUNC_BlueprintCallable) ? TEXT("Pure") : TEXT("Callable");
							}

							return TEXT("NotCallable");
						}();
						UE_LOGF(LogBlueprint, Display, "%ls, %ls, %ls, %ls", *Function->GetName(), *Function->GetOuterUClass()->GetPathName(), Native, Purity);
					}
				}
			} Processor;

			
			TArray<UObject*> InitialObjects = {Blueprint->GeneratedClass};
			UE::GC::FWorkerContext Context;
			Context.SetInitialObjectsUnpadded(InitialObjects);
			CollectReferences(Processor, Context);

			UE_LOGF(LogBlueprint, Display, "--- END audit all BlueprintThreadSafe functions ---");
		}));


bool FBlueprintEditorUtils::HasFunctionBlueprintThreadSafeMetaData(const UFunction* InFunction)
{
	if(InFunction)
	{
		const bool bHasThreadSafeMetaData = InFunction->HasMetaData(FBlueprintMetadata::MD_ThreadSafe);
		const bool bHasNotThreadSafeMetaData = InFunction->HasMetaData(FBlueprintMetadata::MD_NotThreadSafe);
		const bool bClassHasThreadSafeMetaData = InFunction->GetOwnerClass() && InFunction->GetOwnerClass()->HasMetaData(FBlueprintMetadata::MD_ThreadSafe);

		// Native functions need to just have the correct class/function metadata
		const bool bThreadSafeNative = InFunction->HasAnyFunctionFlags(FUNC_Native) && (bHasThreadSafeMetaData || (bClassHasThreadSafeMetaData && !bHasNotThreadSafeMetaData));

		// Script functions get their flag propagated from their entry point, and dont pay heed to class metadata
		const bool bThreadSafeScript = !InFunction->HasAnyFunctionFlags(FUNC_Native) && bHasThreadSafeMetaData;
		
		return bThreadSafeNative || bThreadSafeScript;
	}
	
	return false;
}

bool FBlueprintEditorUtils::ShouldOpenWithDataOnlyEditor(const UBlueprint* Blueprint)
{
	return FBlueprintEditorUtils::IsDataOnlyBlueprint(Blueprint)
		&& !FBlueprintEditorUtils::IsLevelScriptBlueprint(Blueprint)
		&& !FBlueprintEditorUtils::IsInterfaceBlueprint(Blueprint)
		&& !Blueprint->bForceFullEditor
		&& !Blueprint->bIsNewlyCreated;
}

#undef LOCTEXT_NAMESPACE
