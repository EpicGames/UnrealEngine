// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet2/Breakpoint.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Breakpoint)

FBlueprintBreakpoint::FBlueprintBreakpoint()
{
	bEnabled = false;
	bStepOnce = false;
	bStepOnce_WasPreviouslyDisabled = false;
	bStepOnce_RemoveAfterHit = false;
}

FText FBlueprintBreakpoint::GetLocationDescription() const
{
#if WITH_EDITORONLY_DATA
	if (Node != NULL)
	{
		FString Result
#if WITH_EDITOR
			= Node->GetDescriptiveCompiledName()
#endif
			;
		if (!Node->NodeComment.IsEmpty())
		{
			Result += TEXT(" // ");
			Result += Node->NodeComment;
		}

		return FText::FromString(Result);
	}
	else
	{
		return NSLOCTEXT("UBreakpoint", "ErrorInvalidLocation", "Error: Invalid location");
	}
#else	//#if WITH_EDITORONLY_DATA
	return NSLOCTEXT("UBreakpoint", "NoEditorData", "--- NO EDITOR DATA! ---");
#endif	//#if WITH_EDITORONLY_DATA
}
