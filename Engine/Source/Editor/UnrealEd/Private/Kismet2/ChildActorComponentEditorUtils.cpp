// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet2/ChildActorComponentEditorUtils.h"
#include "Settings/BlueprintEditorProjectSettings.h"
#include "ToolMenus.h"
#include "SSCSEditor.h"
#include "SubobjectEditorMenuContext.h"
#include "SSubobjectEditor.h"

#define LOCTEXT_NAMESPACE "ChildActorComponentEditorUtils"

struct FLocalChildActorComponentEditorUtils
{
	static bool IsChildActorTreeViewVisualizationModeSet(UChildActorComponent* InChildActorComponent, EChildActorComponentTreeViewVisualizationMode InMode)
	{
		if (!InChildActorComponent)
		{
			return false;
		}

		const EChildActorComponentTreeViewVisualizationMode CurrentMode = InChildActorComponent->GetEditorTreeViewVisualizationMode();
		if (CurrentMode == EChildActorComponentTreeViewVisualizationMode::UseDefault)
		{
			return InMode == FChildActorComponentEditorUtils::GetProjectDefaultTreeViewVisualizationMode();
		}

		return InMode == CurrentMode;
	}

	static void OnSetChildActorTreeViewVisualizationMode(UChildActorComponent* InChildActorComponent, EChildActorComponentTreeViewVisualizationMode InMode, TWeakPtr<SSubobjectEditor> InWeakSubobjectEditorPtr)
	{
		if (!InChildActorComponent)
		{
			return;
		}

		InChildActorComponent->SetEditorTreeViewVisualizationMode(InMode);

		TSharedPtr<SSubobjectEditor> SubobjectEditorPtr = InWeakSubobjectEditorPtr.Pin();
		if (SubobjectEditorPtr.IsValid())
		{
			SubobjectEditorPtr->UpdateTree();
		}
	}

	static void CreateChildActorVisualizationModesSubMenu(UToolMenu* InSubMenu, UChildActorComponent* InChildActorComponent, TWeakPtr<SSubobjectEditor> InWeakSCSEditorPtr)
	{
		FToolMenuSection& SubMenuSection = InSubMenu->AddSection("ExpansionModes");
		SubMenuSection.AddMenuEntry(
			"ComponentOnly",
			LOCTEXT("ChildActorVisualizationModeLabel_ComponentOnly", "Component Only"),
			LOCTEXT("ChildActorVisualizationModeToolTip_ComponentOnly", "Visualize this child actor as a single component node. The child actor template/instance will not be included in the tree view."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(&FLocalChildActorComponentEditorUtils::OnSetChildActorTreeViewVisualizationMode, InChildActorComponent, EChildActorComponentTreeViewVisualizationMode::ComponentOnly, InWeakSCSEditorPtr),
				FCanExecuteAction(),
				FIsActionChecked::CreateStatic(&FLocalChildActorComponentEditorUtils::IsChildActorTreeViewVisualizationModeSet, InChildActorComponent, EChildActorComponentTreeViewVisualizationMode::ComponentOnly)
			),
			EUserInterfaceActionType::Check);
		SubMenuSection.AddMenuEntry(
			"ChildActorOnly",
			LOCTEXT("ChildActorVisualizationModeLabel_ChildActorOnly", "Child Actor Only"),
			LOCTEXT("ChildActorVisualizationModeToolTip_ChildActorOnly", "Visualize this child actor's template/instance as a subtree with a root actor node in place of the component node."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(&FLocalChildActorComponentEditorUtils::OnSetChildActorTreeViewVisualizationMode, InChildActorComponent, EChildActorComponentTreeViewVisualizationMode::ChildActorOnly, InWeakSCSEditorPtr),
				FCanExecuteAction(),
				FIsActionChecked::CreateStatic(&FLocalChildActorComponentEditorUtils::IsChildActorTreeViewVisualizationModeSet, InChildActorComponent, EChildActorComponentTreeViewVisualizationMode::ChildActorOnly)
			),
			EUserInterfaceActionType::Check);
		SubMenuSection.AddMenuEntry(
			"ComponentWithChildActor",
			LOCTEXT("ChildActorVisualizationModeLabel_ComponentWithChildActor", "Component with Attached Child Actor"),
			LOCTEXT("ChildActorVisualizationModeToolTip_ComponentWithChildActor", "Visualize this child actor's template/instance as a subtree with a root actor node that's parented to the component node."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(&FLocalChildActorComponentEditorUtils::OnSetChildActorTreeViewVisualizationMode, InChildActorComponent, EChildActorComponentTreeViewVisualizationMode::ComponentWithChildActor, InWeakSCSEditorPtr),
				FCanExecuteAction(),
				FIsActionChecked::CreateStatic(&FLocalChildActorComponentEditorUtils::IsChildActorTreeViewVisualizationModeSet, InChildActorComponent, EChildActorComponentTreeViewVisualizationMode::ComponentWithChildActor)
			),
			EUserInterfaceActionType::Check);
	}
};

bool FChildActorComponentEditorUtils::IsChildActorNode(TSharedPtr<const FSCSEditorTreeNode> InNodePtr)
{
	return InNodePtr.IsValid() && InNodePtr->GetNodeType() == FSCSEditorTreeNode::ENodeType::ChildActorNode;
}

bool FChildActorComponentEditorUtils::IsChildActorSubtreeNode(TSharedPtr<const FSCSEditorTreeNode> InNodePtr)
{
	return InNodePtr.IsValid() && IsChildActorNode(InNodePtr->GetActorRootNode());
}

bool FChildActorComponentEditorUtils::ContainsChildActorSubtreeNode(const TArray<TSharedPtr<FSCSEditorTreeNode>>& InNodePtrs)
{
	for (TSharedPtr<FSCSEditorTreeNode> NodePtr : InNodePtrs)
	{
		if (IsChildActorSubtreeNode(NodePtr))
		{
			return true;
		}
	}

	return false;
}

TSharedPtr<FSCSEditorTreeNode> FChildActorComponentEditorUtils::GetOuterChildActorComponentNode(TSharedPtr<const FSCSEditorTreeNode> InNodePtr)
{
	if (InNodePtr.IsValid())
	{
		FSCSEditorActorNodePtrType ActorTreeRootNode = InNodePtr->GetActorRootNode();
		if (IsChildActorNode(ActorTreeRootNode))
		{
			return ActorTreeRootNode->GetOwnerNode();
		}
	}

	return nullptr;
}

bool FChildActorComponentEditorUtils::IsChildActorTreeViewExpansionEnabled()
{
	const UBlueprintEditorProjectSettings* EditorProjectSettings = GetDefault<UBlueprintEditorProjectSettings>();
	return EditorProjectSettings->bEnableChildActorExpansionInTreeView;
}

EChildActorComponentTreeViewVisualizationMode FChildActorComponentEditorUtils::GetProjectDefaultTreeViewVisualizationMode()
{
	const UBlueprintEditorProjectSettings* EditorProjectSettings = GetDefault<UBlueprintEditorProjectSettings>();
	return EditorProjectSettings->DefaultChildActorTreeViewMode;
}

EChildActorComponentTreeViewVisualizationMode FChildActorComponentEditorUtils::GetChildActorTreeViewVisualizationMode(UChildActorComponent* ChildActorComponent, EChildActorComponentTreeViewVisualizationMode DefaultVisOverride)
{
	if (ChildActorComponent)
	{
		EChildActorComponentTreeViewVisualizationMode CurrentMode = ChildActorComponent->GetEditorTreeViewVisualizationMode();
		if (CurrentMode != EChildActorComponentTreeViewVisualizationMode::UseDefault)
		{
			return CurrentMode;
		}
	}

	return DefaultVisOverride == EChildActorComponentTreeViewVisualizationMode::UseDefault ? GetProjectDefaultTreeViewVisualizationMode() : DefaultVisOverride;
}

bool FChildActorComponentEditorUtils::ShouldExpandChildActorInTreeView(UChildActorComponent* ChildActorComponent, EChildActorComponentTreeViewVisualizationMode DefaultVisOverride)
{
	if (!ChildActorComponent)
	{
		return false;
	}

	if ((DefaultVisOverride == EChildActorComponentTreeViewVisualizationMode::UseDefault) && !IsChildActorTreeViewExpansionEnabled())
	{
		return false;
	}

	EChildActorComponentTreeViewVisualizationMode CurrentMode = GetChildActorTreeViewVisualizationMode(ChildActorComponent, DefaultVisOverride);
	return CurrentMode != EChildActorComponentTreeViewVisualizationMode::ComponentOnly;
}

bool FChildActorComponentEditorUtils::ShouldShowChildActorNodeInTreeView(UChildActorComponent* ChildActorComponent, EChildActorComponentTreeViewVisualizationMode DefaultVisOverride)
{
	if (!ShouldExpandChildActorInTreeView(ChildActorComponent, DefaultVisOverride))
	{
		return false;
	}

	EChildActorComponentTreeViewVisualizationMode CurrentMode = GetChildActorTreeViewVisualizationMode(ChildActorComponent, DefaultVisOverride);
	return CurrentMode == EChildActorComponentTreeViewVisualizationMode::ComponentWithChildActor;
}

void FChildActorComponentEditorUtils::FillComponentContextMenuOptions(UToolMenu* Menu, UChildActorComponent* ChildActorComponent)
{
	if (!ChildActorComponent)
	{
		return;
	}

	if (!IsChildActorTreeViewExpansionEnabled())
	{
		return;
	}

	FToolMenuSection& Section = Menu->AddSection("ChildActorComponent", LOCTEXT("ChildActorComponentHeading", "Child Actor Component"));
	{
		TWeakPtr<SSubobjectEditor> WeakEditorPtr;
		if (USubobjectEditorMenuContext* MenuContext = Menu->FindContext<USubobjectEditorMenuContext>())
		{
			WeakEditorPtr = MenuContext->SubobjectEditor;
		}

		Section.AddSubMenu(
			"ChildActorVisualizationModes",
			LOCTEXT("ChildActorVisualizationModesSubMenu_Label", "Visualization Mode"),
			LOCTEXT("ChildActorVisualizationModesSubMenu_ToolTip", "Choose how to visualize this child actor in the tree view."),
			FNewToolMenuDelegate::CreateStatic(&FLocalChildActorComponentEditorUtils::CreateChildActorVisualizationModesSubMenu, ChildActorComponent, WeakEditorPtr));
	}
}

void FChildActorComponentEditorUtils::FillChildActorContextMenuOptions(UToolMenu* Menu, TSharedPtr<const FSCSEditorTreeNode> InNodePtr)
{
	if (!IsChildActorTreeViewExpansionEnabled())
	{
		return;
	}

	if (!IsChildActorNode(InNodePtr))
	{
		return;
	}

	TSharedPtr<const FSCSEditorTreeNodeChildActor> ChildActorNodePtr = StaticCastSharedPtr<const FSCSEditorTreeNodeChildActor>(InNodePtr);
	check(ChildActorNodePtr.IsValid());

	UChildActorComponent* ChildActorComponent = ChildActorNodePtr->GetChildActorComponent();
	if (!ChildActorComponent)
	{
		return;
	}

	FToolMenuSection& Section = Menu->AddSection("ChildActor", LOCTEXT("ChildActorHeading", "Child Actor"));
	{
		TWeakPtr<SSubobjectEditor> WeakEditorPtr;
		if (USubobjectEditorMenuContext* MenuContext = Menu->FindContext<USubobjectEditorMenuContext>())
		{
			WeakEditorPtr = MenuContext->SubobjectEditor;
		}

		Section.AddMenuEntry(
			"SetChildActorOnlyMode",
			LOCTEXT("SetChildActorOnlyMode_Label", "Switch to Child Actor Only Mode"),
			LOCTEXT("SetChildActorOnlyMode_ToolTip", "Visualize this child actor's template/instance subtree in place of its parent component node."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(&FLocalChildActorComponentEditorUtils::OnSetChildActorTreeViewVisualizationMode, ChildActorComponent, EChildActorComponentTreeViewVisualizationMode::ChildActorOnly, WeakEditorPtr),
				FCanExecuteAction()
			)
		);
	}
}

#undef LOCTEXT_NAMESPACE
