// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet2/CompilerResultsLog.h"
#include "Engine/Blueprint.h"
#include "Modules/ModuleManager.h"
#include "Misc/PackageName.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "MessageLogModule.h"
#include "Logging/MessageLog.h"
#include "Logging/StructuredLog.h"
#include "Misc/UObjectToken.h"
#include "SourceCodeNavigation.h"
#include "IHotReload.h"
#include "EngineLogs.h"
#include "Engine/Blueprint.h"
#include "IMessageLogListing.h"
#include "Settings/BlueprintEditorProjectSettings.h"

#if WITH_EDITOR

#define LOCTEXT_NAMESPACE "Editor.Stats"

const FName FCompilerResultsLog::Name(TEXT("CompilerResultsLog"));
FDelegateHandle FCompilerResultsLog::GetGlobalModuleCompilerDumpDelegateHandle;

//////////////////////////////////////////////////////////////////////////
// FBacktrackMap

void FBacktrackMap::NotifyIntermediateObjectCreation(UObject* NewObject, UObject* SourceObject)
{
	// Chase the source to make sure it's really a top-level ('source code') node
	while (UObject** SourceOfSource = SourceBacktrackMap.Find(SourceObject))
	{
		SourceObject = *SourceOfSource;
	}

	// Record the backtrack link
	SourceBacktrackMap.Add(NewObject, SourceObject);
}

void FBacktrackMap::NotifyIntermediatePinCreation(UEdGraphPin* NewPin, UEdGraphPin* SourcePin)
{
	check(NewPin->GetOwningNode() && SourcePin->GetOwningNode());
	// Chase the source to make sure it's really a top-level ('source code') node
	while (UEdGraphPin** SourceOfSource = PinSourceBacktrackMap.Find(SourcePin))
	{
		SourcePin = *SourceOfSource;
	}

	// Record the backtrack link
	PinSourceBacktrackMap.Add(NewPin, SourcePin);
}

UObject* FBacktrackMap::FindSourceObject(UObject* PossiblyDuplicatedObject)
{
	UObject** RemappedIfExisting = SourceBacktrackMap.Find(PossiblyDuplicatedObject);
	if (RemappedIfExisting != nullptr)
	{
		return *RemappedIfExisting;
	}
	else
	{
		// Not in the map, must be an unduplicated object
		return PossiblyDuplicatedObject;
	}
}

UObject const* FBacktrackMap::FindSourceObject(UObject const* PossiblyDuplicatedObject) const
{
	UObject *const* RemappedIfExisting = SourceBacktrackMap.Find(PossiblyDuplicatedObject);
	if (RemappedIfExisting != nullptr)
	{
		return *RemappedIfExisting;
	}
	else
	{
		// Not in the map, must be an unduplicated object
		return PossiblyDuplicatedObject;
	}
}

UEdGraphPin* FBacktrackMap::FindSourcePin(UEdGraphPin* PossiblyDuplicatedPin)
{
	UEdGraphPin** RemappedIfExisting = PinSourceBacktrackMap.Find(PossiblyDuplicatedPin);
	if (RemappedIfExisting != nullptr)
	{
		return *RemappedIfExisting;
	}
	else
	{
		// Not in the map, maybe its owning node was duplicated - and then maybe the GUID matches
		// some node on the original node:
		if (PossiblyDuplicatedPin)
		{
			if (UObject* OriginalOwner = FindSourceObject(PossiblyDuplicatedPin->GetOwningNode()))
			{
				if (UEdGraphNode* AsNode = Cast<UEdGraphNode>(OriginalOwner))
				{
					FGuid TargetGuid = PossiblyDuplicatedPin->PinId;
					UEdGraphPin** ExistingPin = AsNode->Pins.FindByPredicate([TargetGuid](const UEdGraphPin* TargetPin) { return TargetPin && TargetPin->PinId == TargetGuid; });
					if (ExistingPin != nullptr)
					{
						return *ExistingPin;
					}
				}
			}
		}
		
		// No source object found, just return itself:
		return PossiblyDuplicatedPin;
	}
}

UEdGraphPin const* FBacktrackMap::FindSourcePin(UEdGraphPin const* PossiblyDuplicatedPin) const
{
	return const_cast<FBacktrackMap*>(this)->FindSourcePin(const_cast<UEdGraphPin*>(PossiblyDuplicatedPin));
}


//////////////////////////////////////////////////////////////////////////
// FCompilerResultsLog

FCompilerResultsLog::FCompilerResultsLog(bool bIsCompatibleWithEvents/* = true*/)
	: NumErrors(0)
	, NumWarnings(0)
	, bSilentMode(false)
	, bLogInfoOnly(false)
	, bAnnotateMentionedNodes(true)
	, bLogDetailedResults(false)
	, EventDisplayThresholdMs(0)
{
	CurrentEventScope = nullptr;
}

FCompilerResultsLog::~FCompilerResultsLog()
{
}

void FCompilerResultsLog::Register()
{
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	MessageLogModule.RegisterLogListing(Name, LOCTEXT("CompilerLog", "Compiler Log"));

	GetGlobalModuleCompilerDumpDelegateHandle = IHotReloadModule::Get().OnModuleCompilerFinished().AddStatic( &FCompilerResultsLog::GetGlobalModuleCompilerDump );
}

void FCompilerResultsLog::Unregister()
{
	IHotReloadModule::Get().OnModuleCompilerFinished().Remove( GetGlobalModuleCompilerDumpDelegateHandle );

	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	MessageLogModule.UnregisterLogListing(Name);
}

void FCompilerResultsLog::InternalLogEvent(const FCompilerEvent& InEvent, int32 InDepth)
{
	const int EventTimeMs = (int)((InEvent.FinishTime - InEvent.StartTime) * 1000);
	if(EventTimeMs >= EventDisplayThresholdMs)
	{
		// Skip display of the top-most event since that time has already been logged
		if(InDepth > 0)
		{
			FString EventString = FString::Printf(TEXT("- %s"), *InEvent.Name);
			if(InEvent.Counter > 0)
			{
				EventString.Append(FString::Printf(TEXT(" (%d)"), InEvent.Counter + 1));
			}

			FFormatNamedArguments Args;
			Args.Add(TEXT("EventTimeMs"), EventTimeMs);
			EventString.Append(FText::Format(LOCTEXT("PerformanceSummaryEventTime", " [{EventTimeMs} ms]"), Args).ToString());

			FString IndentString = FString::Printf(TEXT("%*s"), (InDepth - 1) << 1, TEXT(""));
			Note(*FString::Printf(TEXT("%s%s"), *IndentString, *EventString));
		}

		const int32 NumChildEvents = InEvent.ChildEvents.Num();
		for(int32 i = 0; i < NumChildEvents; ++i)
		{
			InternalLogEvent(InEvent.ChildEvents[i].Get(), InDepth + 1);
		}
	}
}

bool FCompilerResultsLog::IsMessageEnabled(FName ID)
{
	if(ID == NAME_None)
	{
		return true;
	}

	const UBlueprintEditorProjectSettings* EditorProjectSettings = GetDefault<UBlueprintEditorProjectSettings>();
	if(IsRunningCommandlet() || !GIsEditor)
	{
		if(	EditorProjectSettings->DisabledCompilerMessagesExceptEditor.Contains(ID))
		{
			return false;
		}
	}

	if( EditorProjectSettings->DisabledCompilerMessages.Contains(ID) )
	{
		return false;
	}

	return true;
}

void FCompilerResultsLog::FEdGraphToken_Create(const UObject* InObject, FTokenizedMessage& OutMessage, TArray<UEdGraphNode*>& OutSourceNodes)
{
	FEdGraphToken::Create(InObject, this, OutMessage, OutSourceNodes);
}

void FCompilerResultsLog::FEdGraphToken_Create(const UEdGraphPin* InPin, FTokenizedMessage& OutMessage, TArray<UEdGraphNode*>& OutSourceNodes)
{
	FEdGraphToken::Create(InPin, this, OutMessage, OutSourceNodes);
}

void FCompilerResultsLog::FEdGraphToken_Create(const TCHAR* String, FTokenizedMessage& OutMessage, TArray<UEdGraphNode*>& OutSourceNodes)
{
	FEdGraphToken::Create(String, this, OutMessage, OutSourceNodes);
}

void FCompilerResultsLog::FEdGraphToken_Create(const FField* InField, FTokenizedMessage& OutMessage, TArray<UEdGraphNode*>& OutSourceNodes)
{
	FEdGraphToken::Create(InField, this, OutMessage, OutSourceNodes);
}

void FCompilerResultsLog::InternalLogSummary()
{
	if(CurrentEventScope.IsValid())
	{
		const double CompileStartTime = CurrentEventScope->StartTime;
		const double CompileFinishTime = CurrentEventScope->FinishTime;

		FNumberFormattingOptions TimeFormat;
		TimeFormat.MaximumFractionalDigits = 2;
		TimeFormat.MinimumFractionalDigits = 2;
		TimeFormat.MaximumIntegralDigits = 4;
		TimeFormat.MinimumIntegralDigits = 4;
		TimeFormat.UseGrouping = false;

		FFormatOrderedArguments Args;
		Args.Add(FText::AsNumber(CompileFinishTime - GStartTime, &TimeFormat)); // current time {0}
		Args.Add(FText::FromString(FPackageName::ObjectPathToObjectName(SourcePath))); // source name {1}
		Args.Add(FText::FromString(SourcePath)); // source path {2}
		Args.Add((int)((CompileFinishTime - CompileStartTime) * 1000)); // compile time {3}

		if (NumErrors > 0)
		{
			Args.Add(NumErrors); // num errors {4}
			Args.Add(NumWarnings); // num warnings {5}
			Note(*FText::Format(LOCTEXT("CompileFailed", "[{0}] Compile of {1} failed. {4} Fatal Issue(s) {5} Warning(s) [in {3} ms] ({2})"), MoveTemp(Args)).ToString());
		}
		else if(NumWarnings > 0)
		{
			Args.Add(NumWarnings); // num warnings {4}
			Note(*FText::Format(LOCTEXT("CompileWarning", "[{0}] Compile of {1} successful, but with {4} Warning(s) [in {3} ms] ({2})"), MoveTemp(Args)).ToString());
		}
		else
		{
			Note(*FText::Format(LOCTEXT("CompileSuccess", "[{0}] Compile of {1} successful! [in {3} ms] ({2})"), MoveTemp(Args)).ToString());
		}

		if(bLogDetailedResults)
		{
			// This will not do anything for the default BP compiler log
			Note(*LOCTEXT("PerformanceSummaryHeading", "Performance summary:").ToString());
			InternalLogEvent(*CurrentEventScope.Get());
		}
	}
}

bool FCompilerResultsLog::CommitPotentialMessages(UEdGraphNode* Source)
{
	TArray<TSharedRef<FTokenizedMessage>> FoundMessages;
	if (PotentialMessages.RemoveAndCopyValue(Source, FoundMessages))
	{
		for (const TSharedRef<FTokenizedMessage>& Message : FoundMessages)
		{
			switch (Message->GetSeverity())
			{
			case EMessageSeverity::Error:
				++NumErrors;
				break;

			case EMessageSeverity::Warning:
				++NumWarnings;
				break;

			default:
				break;
			}

			// build nodes list:
			TArray<UEdGraphNode*> Nodes;
			GetNodesFromTokens(Message->GetMessageTokens(), Nodes);

			Nodes.Add(Source);
			InternalLogMessage(Message->GetIdentifier(), Message, Nodes);
		}
		return true;
	}
	return false;
}

void FCompilerResultsLog::NotifyIntermediateObjectCreation(UObject* NewObject, UObject* SourceObject)
{
	SourceBacktrackMap.NotifyIntermediateObjectCreation(NewObject, SourceObject);
}

void FCompilerResultsLog::NotifyIntermediatePinCreation(UEdGraphPin* NewPin, UEdGraphPin* SourcePPin)
{
	SourceBacktrackMap.NotifyIntermediatePinCreation(NewPin, SourcePPin);
}

UObject* FCompilerResultsLog::FindSourceObject(UObject* PossiblyDuplicatedObject)
{
	return SourceBacktrackMap.FindSourceObject(PossiblyDuplicatedObject);
}

UObject const* FCompilerResultsLog::FindSourceObject(UObject const* PossiblyDuplicatedObject) const
{
	return SourceBacktrackMap.FindSourceObject(PossiblyDuplicatedObject);
}

UObject* FCompilerResultsLog::FindSourceMacroInstance(const UEdGraphNode* IntermediateNode) const
{
	UObject* Result = nullptr;
	const UEdGraphNode* Iter = IntermediateNode;
	while (const TWeakObjectPtr<UEdGraphNode>* SourceInstanceNode = FullMacroBacktrackMap.Find(Iter))
	{
		Iter = SourceInstanceNode->Get();
	}

	Result = const_cast<UEdGraphNode*>(Iter);

	if(Result)
	{
		Result = const_cast<FCompilerResultsLog*>(this)->FindSourceObject(Result);
	}

	return Result;
}

void FCompilerResultsLog::NotifyIntermediateTunnelNode(const UEdGraphNode* Node, const UEdGraphNode* OuterTunnelInstance)
{
	IntermediateTunnelNodeToTunnelInstanceMap.Add(Node, OuterTunnelInstance);
}

void FCompilerResultsLog::NotifyIntermediateMacroNode(UEdGraphNode* SourceNode, const UEdGraphNode* IntermediateNode)
{
	if(ensure(SourceNode != IntermediateNode))
	{
		FullMacroBacktrackMap.Add(IntermediateNode, SourceNode);
	}
}

const UEdGraphNode* FCompilerResultsLog::GetIntermediateTunnelInstance(const UEdGraphNode* IntermediateNode) const
{
	TWeakObjectPtr<const UEdGraphNode> Result;
	if (const TWeakObjectPtr<const UEdGraphNode>* IntermediateTunnelInstanceNode = IntermediateTunnelNodeToTunnelInstanceMap.Find(IntermediateNode))
	{
		Result = *IntermediateTunnelInstanceNode;
	}
	return Result.Get();
}

int32 FCompilerResultsLog::CalculateStableIdentifierForLatentActionManager( const UEdGraphNode* Node )
{
	/* 
		The name of this function is meant to instill a bit of caution:
		1. The Latent Action Manager uses uint32s to identify actions, so there
			is some risk of collision, increasing if we aren't able to distribute
			keys across the whole range of uint32
		2. We need these identifiers to be stable across blueprint compiles, meaning
			we can't just create a GUID and hash it

		Meeting these two requirements has proved difficult. The edge cases involve
		macros and nodes that implement UK2Node::ExpandNode, e.g. LoadAsset/LoadAssetClass
		nodes in Macros. In order to handle that case we use the source backtrack map.
		Typically an intermediate node has a dynamic GUID, which is useless, but source 
		nodes that came from tunnel expansions have stable GUIDs, and can be used 
	*/

	int32 LatentUUID = 0;
	const UEdGraphNode* OriginalNode = Node;
	const UEdGraphNode* OuterTunnelInstance = GetIntermediateTunnelInstance(Node);

	// first search for a node with a stable GUID (e.g., not a node that was created via SpawnIntermediateNode, 
	// but we want to include nodes that are created via macro instantiation):
	if (!OuterTunnelInstance)
	{
		Node = Cast<UEdGraphNode>(SourceBacktrackMap.FindSourceObject(Node));
		if (Node && Node->HasAnyFlags(RF_Transient))
		{
			// we failed to find a source node, bail
			Node = nullptr;
		}
	}
	
	if(Node)
	{
		LatentUUID = GetTypeHash(Node->NodeGuid);

		const UEdGraphNode* ResultNode = Node;
		while (OuterTunnelInstance && OuterTunnelInstance != ResultNode)
		{
			if (OuterTunnelInstance->NodeGuid.IsValid())
			{
				LatentUUID = HashCombine(LatentUUID, GetTypeHash(OuterTunnelInstance->NodeGuid));
			}
			ResultNode = OuterTunnelInstance;
			OuterTunnelInstance = GetIntermediateTunnelInstance(ResultNode);
		}
	}
	else
	{
		Warning(
			*LOCTEXT("UUIDDeterministicCookWarn", "Failed to produce a deterministic UUID for a node's latent action: @@").ToString(),
			OriginalNode
		);

		static int32 FallbackUUID = 0;
		LatentUUID = FallbackUUID++;
	}

	return LatentUUID;
}

UEdGraphPin* FCompilerResultsLog::FindSourcePin(UEdGraphPin* PossiblyDuplicatedPin)
{
	return SourceBacktrackMap.FindSourcePin(PossiblyDuplicatedPin);
}

const UEdGraphPin* FCompilerResultsLog::FindSourcePin(const UEdGraphPin* PossiblyDuplicatedPin) const
{
	return SourceBacktrackMap.FindSourcePin(PossiblyDuplicatedPin);
}

void FCompilerResultsLog::InternalLogMessage(FName MessageID, const TSharedRef<FTokenizedMessage>& Message, const TArray<UEdGraphNode*>& SourceNodes)
{
	const EMessageSeverity::Type Severity = Message->GetSeverity();
	Messages.Add(Message);
	AnnotateNode(SourceNodes, Message);
	Message->SetIdentifier(MessageID);

	if (!bSilentMode && (!bLogInfoOnly || (Severity == EMessageSeverity::Info)))
	{
		if (Severity == EMessageSeverity::Error)
		{
			UE_LOGFMT_NSLOC(LogBlueprint, Error, "CompilerResultsLog", "InternalLogMessage", "[AssetLog] {Asset}: [Compiler] {Message}",
				("Asset", UE::FAssetLog(*SourcePath)), ("Message", Message->ToText().ToString()));
		}
		else if (Severity == EMessageSeverity::Warning || Severity == EMessageSeverity::PerformanceWarning)
		{
			UE_LOGFMT_NSLOC(LogBlueprint, Warning, "CompilerResultsLog", "InternalLogMessage", "[AssetLog] {Asset}: [Compiler] {Message}",
				("Asset", UE::FAssetLog(*SourcePath)), ("Message", Message->ToText().ToString()));
		}
		else
		{
			UE_LOGFMT_NSLOC(LogBlueprint, Log, "CompilerResultsLog", "InternalLogMessage", "[AssetLog] {Asset}: [Compiler] {Message}",
				("Asset", UE::FAssetLog(*SourcePath)), ("Message", Message->ToText().ToString()));
		}
	}
}

void FCompilerResultsLog::AnnotateNode(const TArray<UEdGraphNode*>& Nodes, TSharedRef<FTokenizedMessage> LogLine)
{
	for(UEdGraphNode* Node : Nodes)
	{
		if (bAnnotateMentionedNodes)
		{
			// Determine if this message is the first or more important than the previous one (only showing one error/warning per node for now)
			bool bUpdateMessage = true;
			if (Node->bHasCompilerMessage)
			{
				// Already has a message, see if we meet or trump the severity
				bUpdateMessage = LogLine->GetSeverity() <= Node->ErrorType;
			}
			else
			{
				Node->ErrorMsg.Empty();
			}

			// Update the message
			if (bUpdateMessage)
			{
				Node->ErrorType = (int32)LogLine->GetSeverity();
				Node->bHasCompilerMessage = true;

				FText FullMessage = LogLine->ToText();

				if (Node->ErrorMsg.IsEmpty())
				{
					Node->ErrorMsg = FullMessage.ToString();
				}
				else
				{
					FFormatNamedArguments Args;
					Args.Add(TEXT("PreviousMessage"), FText::FromString(Node->ErrorMsg));
					Args.Add(TEXT("NewMessage"), FullMessage);
					Node->ErrorMsg = FText::Format(LOCTEXT("AggregateMessagesFormatter", "{PreviousMessage}\n{NewMessage}"), Args).ToString();
				}

				AnnotatedNodes.Add(Node);
			}
		}
	}	
}

TArray< TSharedRef<FTokenizedMessage> > FCompilerResultsLog::ParseCompilerLogDump(const FString& LogDump)
{
	TArray< TSharedRef<FTokenizedMessage> > Messages;

	TArray< FString > MessageLines;
	LogDump.ParseIntoArray(MessageLines, TEXT("\n"), false);

	// delete any trailing empty lines
	for (int32 i = MessageLines.Num()-1; i >= 0; --i)
	{
		if (!MessageLines[i].IsEmpty())
		{
			if (i < MessageLines.Num() - 1)
			{
				MessageLines.RemoveAt(i+1, MessageLines.Num() - (i+1));
			}
			break;
		}
	}

	for (int32 i = 0; i < MessageLines.Num(); ++i)
	{
		FString Line = MessageLines[i];
		if (Line.EndsWith(TEXT("\r"), ESearchCase::CaseSensitive))
		{
			Line.LeftChopInline(1, EAllowShrinking::No);
		}
		Line.ConvertTabsToSpacesInline(4);
		Line.TrimEndInline();

		// handle output line error message if applicable
		// @todo Handle case where there are parenthesis in path names
		// @todo Handle errors reported by Clang
		FString LeftStr, RightStr;
		FString FullPath, LineNumberString;
		if (Line.Split(TEXT(")"), &LeftStr, &RightStr, ESearchCase::CaseSensitive) &&
			LeftStr.Split(TEXT("("), &FullPath, &LineNumberString, ESearchCase::CaseSensitive) &&
			LineNumberString.IsNumeric() && (FCString::Strtoi(*LineNumberString, NULL, 10) > 0) &&
			RightStr.Contains(TEXT(": error")))
		{
			EMessageSeverity::Type Severity = EMessageSeverity::Error;
			FString FullPathTrimmed = FullPath;
			FullPathTrimmed.TrimStartInline();
			if (FullPathTrimmed.Len() != FullPath.Len()) // check for leading whitespace
			{
				Severity = EMessageSeverity::Info;
			}

			TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create( Severity );
			if ( Severity == EMessageSeverity::Info )	// add whitespace now
			{
				FString Whitespace = FullPath.Left(FullPath.Len() - FullPathTrimmed.Len());
				Message->AddToken( FTextToken::Create( FText::FromString( Whitespace ) ) );
				FullPath = FullPathTrimmed;
			}

			FString Link = FullPath + TEXT("(") + LineNumberString + TEXT(")");
			Message->AddToken( FTextToken::Create( FText::FromString( Link ) )->OnMessageTokenActivated(FOnMessageTokenActivated::CreateStatic(&FCompilerResultsLog::OnGotoError) ) );
			Message->AddToken( FTextToken::Create( FText::FromString( RightStr ) ) );
			Messages.Add(Message);

			if (Severity == EMessageSeverity::Error)
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("%s"), *Line);
			}
		}
		else
		{
			EMessageSeverity::Type Severity = EMessageSeverity::Info;
			if (Line.Contains(TEXT("error LNK"), ESearchCase::CaseSensitive))
			{
				Severity = EMessageSeverity::Error;
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("%s"), *Line);
			}

			TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create( Severity );
			Message->AddToken( FTextToken::Create( FText::FromString( Line ) ) );
			Messages.Add(Message);
		}
	}

	return Messages;
}

void FCompilerResultsLog::OnGotoError(const TSharedRef<IMessageToken>& Token)
{
	FString FullPath, LineNumberString;
	if (Token->ToText().ToString().Split(TEXT("("), &FullPath, &LineNumberString, ESearchCase::CaseSensitive))
	{
		LineNumberString.LeftChopInline(1, EAllowShrinking::No); // remove right parenthesis
		int32 LineNumber = FCString::Strtoi(*LineNumberString, NULL, 10);

		FSourceCodeNavigation::OpenSourceFileAsync( FullPath, LineNumber );
	}
}

void FCompilerResultsLog::GetGlobalModuleCompilerDump(const FString& LogDump, ECompilationResult::Type CompilationResult, bool bShowLog)
{
	FMessageLog MessageLog(Name);

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("TimeStamp"), FText::AsDateTime(FDateTime::UtcNow()));
	MessageLog.NewPage(FText::Format(LOCTEXT("CompilerLogPage", "Compilation - {TimeStamp}"), Arguments));

	if ( bShowLog )
	{
		MessageLog.Open(EMessageSeverity::Info, GetDefault<UEditorPerProjectUserSettings>()->bShowCompilerLogOnCompileError);
	}

	MessageLog.AddMessages(ParseCompilerLogDump(LogDump));
}
		
void FCompilerResultsLog::GetNodesFromTokens(const TArray<TSharedRef<IMessageToken> >& MessageTokens, TArray<UEdGraphNode*>& OutOwnerNodes)
{
	for (TSharedRef<IMessageToken> const& Token : MessageTokens)
	{
		if (Token->GetType() == EMessageToken::Object)
		{
			FWeakObjectPtr ObjectPtr = ((FUObjectToken&)Token.Get()).GetObject();
			if (!ObjectPtr.IsValid())
			{
				continue;
			}
			UObject* ObjectArgument = ObjectPtr.Get();
			if(UEdGraphNode* Node = Cast<UEdGraphNode>(ObjectArgument))
			{
				OutOwnerNodes.AddUnique(Node);
			}
		}
		else if (Token->GetType() == EMessageToken::EdGraph)
		{
			FEdGraphToken& AsEdGraphToken = ((FEdGraphToken&)Token.Get());
			const UEdGraphPin* PinBeingReferenced = AsEdGraphToken.GetPin();
			UEdGraphNode* OwnerNode = Cast<UEdGraphNode>((UObject*)AsEdGraphToken.GetGraphObject());
			if (OwnerNode == nullptr)
			{
				if (PinBeingReferenced)
				{
					OutOwnerNodes.AddUnique(Cast<UEdGraphNode>(PinBeingReferenced->GetOwningNodeUnchecked()));
				}
			}
			else
			{
				OutOwnerNodes.AddUnique(OwnerNode);
			}
		}
	}
}

void FCompilerResultsLog::Append(FCompilerResultsLog const& Other, bool bWriteToSystemLog)
{
	for (TSharedRef<FTokenizedMessage> const& Message : Other.Messages)
	{
		if (Messages.Contains(Message))
		{
			continue;
		}

		switch (Message->GetSeverity())
		{
		case EMessageSeverity::Warning:
			{
				++NumWarnings;
				break;
			}
				
		case EMessageSeverity::Error:
			{
				++NumErrors;
				break;
			}
		}

		TArray<UEdGraphNode*> OwnerNodes;
		GetNodesFromTokens(Message->GetMessageTokens(), OwnerNodes);

		if (bWriteToSystemLog)
		{
			InternalLogMessage(Message->GetIdentifier(), Message, OwnerNodes);
		}
		else
		{
			Messages.Add(Message);
			AnnotateNode(OwnerNodes, Message);
		}
	}
}

void FCompilerResultsLog::BeginEvent(const TCHAR* InName)
{
	// Create a new event
	TSharedPtr<FCompilerEvent> ParentEventScope = CurrentEventScope;
	CurrentEventScope = MakeShareable(new FCompilerEvent(ParentEventScope));

	// Start event with given name
	CurrentEventScope->Start(InName);
}

void FCompilerResultsLog::EndEvent()
{
	if(CurrentEventScope.IsValid())
	{
		// Mark finish time
		CurrentEventScope->Finish();

		// Get the parent event scope
		TSharedPtr<FCompilerEvent> ParentEventScope = CurrentEventScope->ParentEventScope;
		if(ParentEventScope.IsValid())
		{
			// Aggregate the current event into the parent event scope
			TSharedRef<FCompilerEvent> ChildEvent = CurrentEventScope.ToSharedRef();
			AddChildEvent(ParentEventScope, ChildEvent);

			// Move current event scope back up to parent
			CurrentEventScope = ParentEventScope;
		}
		else
		{
			// Log results summary once we've ended the top-level event
			InternalLogSummary();

			// Reset current event scope
			CurrentEventScope = nullptr;
		}
	}
}

void FCompilerResultsLog::AddChildEvent(TSharedPtr<FCompilerEvent>& ParentEventScope, TSharedRef<FCompilerEvent>& ChildEventScope)
{
	if(ParentEventScope.IsValid())
	{
		// If we already have a matching parent scope, aggregate child events into the current parent scope
		if(ParentEventScope->Name == ChildEventScope->Name)
		{
			for(int i = 0; i < ChildEventScope->ChildEvents.Num(); ++i)
			{
				AddChildEvent(ParentEventScope, ChildEventScope->ChildEvents[i]);
			}
		}
		else
		{
			// Look for a matching sibling event under the current parent scope
			bool bMatchFound = false;
			for(int i = ParentEventScope->ChildEvents.Num() - 1; i >= 0 && !bMatchFound; --i)
			{
				// If we find a matching scope, combine this event with the existing one
				TSharedPtr<FCompilerEvent> ExistingChildEvent = ParentEventScope->ChildEvents[i];
				if(ExistingChildEvent->Name == ChildEventScope->Name)
				{
					bMatchFound = true;

					// Append timing and child event data to the existing event to create an aggregate event
					ExistingChildEvent->Counter += 1;
					ExistingChildEvent->FinishTime += (ChildEventScope->FinishTime - ChildEventScope->StartTime);
					for(int j = 0; j < ChildEventScope->ChildEvents.Num(); ++j)
					{
						AddChildEvent(ExistingChildEvent, ChildEventScope->ChildEvents[j]);
					}
				}
			}

			if(!bMatchFound)
			{
				// If we didn't find a matching event, append the current event to the list of child events under the current parent scope
				ParentEventScope->ChildEvents.Add(ChildEventScope);
			}
		}
	}
}

static FName GetBlueprintMessageLogName(UBlueprint* InBlueprint)
{
	FName LogListingName;
	if (InBlueprint != nullptr)
	{
		LogListingName = *FString::Printf(TEXT("%s_%s_CompilerResultsLog"), *InBlueprint->GetBlueprintGuid().ToString(), *InBlueprint->GetName());
	}
	else
	{
		LogListingName = "BlueprintCompiler";
	}
	return LogListingName;
}

static TSharedRef<IMessageLogListing> RegisterBlueprintMessageLog(UBlueprint* InBlueprint)
{
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");

	const FName LogName = GetBlueprintMessageLogName(InBlueprint);

	// Register the log (this will return an existing log if it has been used before)
	FMessageLogInitializationOptions LogInitOptions;
	LogInitOptions.bShowInLogWindow = false;
	MessageLogModule.RegisterLogListing(LogName, LOCTEXT("BlueprintCompilerLogLabel", "BlueprintCompiler"), LogInitOptions);
	return MessageLogModule.GetLogListing(LogName);
}

TSharedRef<IMessageLogListing> FCompilerResultsLog::GetBlueprintMessageLog(UBlueprint* InBlueprint)
{
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");

	const FName LogName = GetBlueprintMessageLogName(InBlueprint);

	// Reuse any existing log, or create a new one (that is not held onto bey the message log system)
	if(MessageLogModule.IsRegisteredLogListing(LogName))
	{
		return MessageLogModule.GetLogListing(LogName);
	}
	else
	{
		FMessageLogInitializationOptions LogInitOptions;
		LogInitOptions.bShowInLogWindow = false;
		return MessageLogModule.CreateLogListing(LogName, LogInitOptions);
	}
}

FScopedBlueprintMessageLog::FScopedBlueprintMessageLog(UBlueprint* InBlueprint)
	: Log(RegisterBlueprintMessageLog(InBlueprint))
{
}

FScopedBlueprintMessageLog::~FScopedBlueprintMessageLog()
{
	// Unregister the log so it will be ref-counted to zero if it has no messages
	if(Log->NumMessages(EMessageSeverity::Info) == 0)
	{
		FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
		MessageLogModule.UnregisterLogListing(Log->GetName());
	}
}

#undef LOCTEXT_NAMESPACE

#endif	//#if WITH_EDITOR
