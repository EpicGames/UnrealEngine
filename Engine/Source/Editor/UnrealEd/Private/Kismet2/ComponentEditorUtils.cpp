// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet2/ComponentEditorUtils.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "UObject/PropertyPortFlags.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Styling/AppStyle.h"
#include "Components/PrimitiveComponent.h"
#include "Components/MeshComponent.h"
#include "Exporters/Exporter.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Misc/StringOutputDevice.h"
#include "Components/DecalComponent.h"
#include "ScopedTransaction.h"
#include "EdGraphSchema_K2.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Factories.h"
#include "UnrealExporter.h"
#include "Framework/Commands/GenericCommands.h"
#include "SourceCodeNavigation.h"
#include "Styling/SlateIconFinder.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Editor/UnrealEdEngine.h"
#include "Preferences/UnrealEdOptions.h"
#include "UnrealEdGlobals.h"

#include "HAL/PlatformApplicationMisc.h"
#include "ToolMenus.h"
#include "Kismet2/ComponentEditorContextMenuContex.h"
#include "Subsystems/AssetEditorSubsystem.h"

#define LOCTEXT_NAMESPACE "ComponentEditorUtils"

// Text object factory for pasting components
struct FComponentObjectTextFactory : public FCustomizableTextObjectFactory
{
	// Child->Parent name map
	TMap<FName, FName> ParentMap;

	// Name->Instance object mapping
	TMap<FName, UActorComponent*> NewObjectMap;

	// Determine whether or not scene components in the new object set can be attached to the given scene root component
	bool CanAttachComponentsTo(const USceneComponent* InRootComponent)
	{
		check(InRootComponent);

		// For each component in the set, check against the given root component and break if we fail to validate
		bool bCanAttachToRoot = true;
		for (auto NewComponentIt = NewObjectMap.CreateConstIterator(); NewComponentIt && bCanAttachToRoot; ++NewComponentIt)
		{
			// If this is a scene component, and it does not already have a parent within the set
			const USceneComponent* SceneComponent = Cast<USceneComponent>(NewComponentIt->Value);
			if (SceneComponent && !ParentMap.Contains(SceneComponent->GetFName()))
			{
				// Determine if we are allowed to attach the scene component to the given root component
				bCanAttachToRoot = InRootComponent->CanAttachAsChild(SceneComponent, NAME_None)
					&& SceneComponent->Mobility >= InRootComponent->Mobility
					&& ( !InRootComponent->IsEditorOnly() || SceneComponent->IsEditorOnly() );
			}
		}

		return bCanAttachToRoot;
	}

	// Constructs a new object factory from the given text buffer
	static TSharedRef<FComponentObjectTextFactory> Get(const FString& InTextBuffer, bool bPasteAsArchetypes = false)
	{
		// Construct a new instance
		TSharedPtr<FComponentObjectTextFactory> FactoryPtr = MakeShareable(new FComponentObjectTextFactory());
		check(FactoryPtr.IsValid());

		// Create new objects if we're allowed to
		if (FactoryPtr->CanCreateObjectsFromText(InTextBuffer))
		{
			EObjectFlags ObjectFlags = RF_Transactional;
			if (bPasteAsArchetypes)
			{
				ObjectFlags |= RF_ArchetypeObject | RF_Public;
			}

			// Use the transient package initially for creating the objects, since the variable name is used when copying
			FactoryPtr->ProcessBuffer(GetTransientPackage(), ObjectFlags, InTextBuffer);
		}

		return FactoryPtr.ToSharedRef();
	}

	virtual ~FComponentObjectTextFactory() {}

protected:
	// Constructor; protected to only allow this class to instance itself
	FComponentObjectTextFactory()
		:FCustomizableTextObjectFactory(GWarn)
	{
	}

	// FCustomizableTextObjectFactory implementation

	virtual bool CanCreateClass(UClass* ObjectClass, bool& bOmitSubObjs) const override
	{
		// Allow actor component types to be created
		bool bCanCreate = ObjectClass && ObjectClass->IsChildOf(UActorComponent::StaticClass());

		if (!bCanCreate)
		{
			// Also allow Blueprint-able actor types to pass, in order to enable proper creation of actor component types as subobjects. The actor instance will be discarded after processing.
			bCanCreate = ObjectClass && ObjectClass->IsChildOf(AActor::StaticClass()) && FKismetEditorUtilities::CanCreateBlueprintOfClass(ObjectClass);
		}
		else
		{
			// Actor component classes should not be abstract and must also be tagged as BlueprintSpawnable
			bCanCreate = FKismetEditorUtilities::IsClassABlueprintSpawnableComponent(ObjectClass);
		}

		return bCanCreate;
	}

	virtual void ProcessConstructedObject(UObject* NewObject) override
	{
		check(NewObject);

		TInlineComponentArray<UActorComponent*> ActorComponents;
		if (UActorComponent* NewActorComponent = Cast<UActorComponent>(NewObject))
		{
			ActorComponents.Add(NewActorComponent);
		}
		else if (AActor* NewActor = Cast<AActor>(NewObject))
		{
			if (USceneComponent* RootComponent = NewActor->GetRootComponent())
			{
				RootComponent->SetWorldLocationAndRotationNoPhysics(FVector(0.f),FRotator(0.f));
			}
			NewActor->GetComponents(ActorComponents);
		}

		for(UActorComponent* ActorComponent : ActorComponents)
		{
			// Add it to the new object map
			NewObjectMap.Add(ActorComponent->GetFName(), ActorComponent);

			// If this is a scene component and it has a parent
			USceneComponent* SceneComponent = Cast<USceneComponent>(ActorComponent);
			if (SceneComponent && SceneComponent->GetAttachParent())
			{
				// Add an entry to the child->parent name map
				ParentMap.Add(ActorComponent->GetFName(), SceneComponent->GetAttachParent()->GetFName());

				// Clear this so it isn't used when constructing the new SCS node
				SceneComponent->SetupAttachment(nullptr);
			}
		}
	}

	// FCustomizableTextObjectFactory (end)
};

bool FComponentEditorUtils::CanEditComponentInstance(const UActorComponent* ActorComp, const UActorComponent* ParentSceneComp, bool bAllowUserContructionScript)
{
	// Exclude nested DSOs attached to BP-constructed instances, which are not mutable.
	return (ActorComp != nullptr
		&& (!ActorComp->IsVisualizationComponent())
		&& (ActorComp->CreationMethod != EComponentCreationMethod::UserConstructionScript || bAllowUserContructionScript)
		&& (ParentSceneComp == nullptr || !ParentSceneComp->IsCreatedByConstructionScript() || !ActorComp->HasAnyFlags(RF_DefaultSubObject)))
		&& (ActorComp->CreationMethod != EComponentCreationMethod::Native || FComponentEditorUtils::GetPropertyForEditableNativeComponent(ActorComp));
}

FProperty* FComponentEditorUtils::GetPropertyForEditableNativeComponent(const UActorComponent* NativeComponent)
{
	// A native component can be edited if it is bound to a member variable and that variable is marked as visible in the editor
	// Note: We aren't concerned with whether the component is marked editable - the component itself is responsible for determining which of its properties are editable	
	UObject* ComponentOuter = (NativeComponent ? NativeComponent->GetOuter() : nullptr);
	UClass* OwnerClass = (ComponentOuter ? ComponentOuter->GetClass() : nullptr);

	if (OwnerClass != nullptr)
	{
		for (TFieldIterator<FObjectProperty> It(OwnerClass); It; ++It)
		{
			FObjectProperty* ObjectProp = *It;

			// Must be visible - note CPF_Edit is set for all properties that should be visible, not just those that are editable
			if ((ObjectProp->PropertyFlags & (CPF_Edit)) == 0)
			{
				continue;
			}

			UObject* Object = ObjectProp->GetObjectPropertyValue(ObjectProp->ContainerPtrToValuePtr<void>(ComponentOuter));
			if (Object != nullptr && Object->GetFName() == NativeComponent->GetFName())
			{
				return ObjectProp;
			}
		}	
	
		// We have to check for array properties as well because they are not FObjectProperties and we want to be able to
		// edit the inside of it
		if (ComponentOuter != nullptr)
		{
			for (TFieldIterator<FArrayProperty> PropIt(OwnerClass, EFieldIteratorFlags::IncludeSuper); PropIt; ++PropIt)
			{
				FArrayProperty* TestProperty = *PropIt;
				void* ArrayPropInstAddress = TestProperty->ContainerPtrToValuePtr<void>(ComponentOuter);

				// Ensure that this property is valid
				FObjectProperty* ArrayEntryProp = CastField<FObjectProperty>(TestProperty->Inner);
				if ((ArrayEntryProp == nullptr) || !ArrayEntryProp->PropertyClass->IsChildOf<UActorComponent>() || ((TestProperty->PropertyFlags & CPF_Edit) == 0))
				{
					continue;
				}

				// For each object in this array
				FScriptArrayHelper ArrayHelper(TestProperty, ArrayPropInstAddress);
				for (int32 ComponentIndex = 0; ComponentIndex < ArrayHelper.Num(); ++ComponentIndex)
				{
					// If this object is the native component we are looking for, then we should be allowed to edit it
					const uint8* ArrayValRawPtr = ArrayHelper.GetRawPtr(ComponentIndex);
					UObject* ArrayElement = ArrayEntryProp->GetObjectPropertyValue(ArrayValRawPtr);			
					
					if (ArrayElement != nullptr && ArrayElement->GetFName() == NativeComponent->GetFName())
					{
						return ArrayEntryProp;
					}
				}				
			}			

			// Check for map properties as well
			for (TFieldIterator<FMapProperty> PropIt(OwnerClass, EFieldIteratorFlags::IncludeSuper); PropIt; ++PropIt)
			{
				FMapProperty* TestProperty = *PropIt;
				void* MapPropInstAddress = TestProperty->ContainerPtrToValuePtr<void>(ComponentOuter);
				
				// Ensure that this property is valid and that it is marked as visible in the editor
				FObjectProperty* MapValProp = CastField<FObjectProperty>(TestProperty->ValueProp);
				if ((MapValProp == nullptr) || !MapValProp->PropertyClass->IsChildOf<UActorComponent>() || ((TestProperty->PropertyFlags & CPF_Edit) == 0))
				{
					continue;
				}

				FScriptMapHelper MapHelper(TestProperty, MapPropInstAddress);
				for (FScriptMapHelper::FIterator It(MapHelper); It; ++It)
				{
					// For each value in the map (don't bother checking the keys, they won't be what the user can edit in this case)
					const uint8* MapValueData = MapHelper.GetValuePtr(It);
					UObject* ValueElement = MapValProp->GetObjectPropertyValue(MapValueData);
					if (ValueElement != nullptr && ValueElement->GetFName() == NativeComponent->GetFName())
					{
						return MapValProp;
					}
				}
			}

			// Finally we should check for set properties
			for (TFieldIterator<FSetProperty> PropIt(OwnerClass, EFieldIteratorFlags::IncludeSuper); PropIt; ++PropIt)
			{
				FSetProperty* TestProperty = *PropIt;
				void* SetPropInstAddress = TestProperty->ContainerPtrToValuePtr<void>(ComponentOuter);

				// Ensure that this property is valid and that it is marked visible
				FObjectProperty* SetValProp = CastField<FObjectProperty>(TestProperty->ElementProp);
				if ((SetValProp == nullptr) || !SetValProp->PropertyClass->IsChildOf<UActorComponent>() || ((TestProperty->PropertyFlags & CPF_Edit) == 0))
				{
					continue;
				}
				
				// For each item in the set
				FScriptSetHelper SetHelper(TestProperty, SetPropInstAddress);
				for (FScriptSetHelper::FIterator It(SetHelper); It; ++It)
				{
					const uint8* SetValData = SetHelper.GetElementPtr(It);
					UObject* SetValueElem = SetValProp->GetObjectPropertyValue(SetValData);

					if (SetValueElem != nullptr && SetValueElem->GetFName() == NativeComponent->GetFName())
					{
						return SetValProp;
					}
				}
			}
		}
	}	

	return nullptr;
}

bool FComponentEditorUtils::IsValidVariableNameString(const UActorComponent* InComponent, const FString& InString)
{
	// First test to make sure the string is not empty and does not equate to the DefaultSceneRoot node name
	bool bIsValid = !InString.IsEmpty() && !InString.Equals(USceneComponent::GetDefaultSceneRootVariableName().ToString());
	if(bIsValid && InComponent != NULL)
	{
		// Next test to make sure the string doesn't conflict with the format that MakeUniqueObjectName() generates
		const FString ClassNameThatWillBeUsedInGenerator = FBlueprintEditorUtils::GetClassNameWithoutSuffix(InComponent->GetClass());
		const FString MakeUniqueObjectNamePrefix = FString::Printf(TEXT("%s_"), *ClassNameThatWillBeUsedInGenerator);
		if(InString.StartsWith(MakeUniqueObjectNamePrefix))
		{
			bIsValid = !InString.Replace(*MakeUniqueObjectNamePrefix, TEXT("")).IsNumeric();
		}
	}

	return bIsValid;
}

bool FComponentEditorUtils::IsComponentNameAvailable(const FString& InString, UObject* ComponentOwner, const UActorComponent* ComponentToIgnore)
{
	UObject* Object = FindObjectFast<UObject>(ComponentOwner, *InString);

	bool bNameIsAvailable = Object == nullptr || Object == ComponentToIgnore;

	return bNameIsAvailable;
}

FString FComponentEditorUtils::GenerateValidVariableName(TSubclassOf<UActorComponent> ComponentClass, AActor* ComponentOwner)
{
	check(ComponentOwner);

	FString ComponentTypeName = FBlueprintEditorUtils::GetClassNameWithoutSuffix(ComponentClass);

	// Strip off 'Component' if the class ends with that.  It just looks better in the UI.
	FString SuffixToStrip( TEXT( "Component" ) );
	if( ComponentTypeName.EndsWith( SuffixToStrip ) )
	{
		ComponentTypeName.LeftInline( ComponentTypeName.Len() - SuffixToStrip.Len(), EAllowShrinking::No );
	}

	// Strip off 'Actor' if the class ends with that so as not to confuse actors with components
	SuffixToStrip = TEXT( "Actor" );
	if( ComponentTypeName.EndsWith( SuffixToStrip ) )
	{
		ComponentTypeName.LeftInline( ComponentTypeName.Len() - SuffixToStrip.Len(), EAllowShrinking::No );
	}

	// Try to create a name without any numerical suffix first
	int32 Counter = 1;
	FString ComponentInstanceName = ComponentTypeName;
	while (!IsComponentNameAvailable(ComponentInstanceName, ComponentOwner))
	{
		// Assign the lowest possible numerical suffix
		ComponentInstanceName = FString::Printf(TEXT("%s%d"), *ComponentTypeName, Counter++);
	}

	return ComponentInstanceName;
}

FString FComponentEditorUtils::GenerateValidVariableNameFromAsset(UObject* Asset, AActor* ComponentOwner)
{
	int32 Counter = 1;
	FString AssetName = Asset->GetName();

	if (UClass* Class = Cast<UClass>(Asset))
	{
		if (!Class->HasAnyClassFlags(CLASS_CompiledFromBlueprint))
		{
			AssetName.RemoveFromEnd(TEXT("Component"));
		}
		else
		{
			AssetName.RemoveFromEnd("_C");
		}
	}
	else if (UActorComponent* Comp = Cast <UActorComponent>(Asset))
	{
		AssetName.RemoveFromEnd(UActorComponent::ComponentTemplateNameSuffix);
	}

	// Try to create a name without any numerical suffix first
	FString NewName = AssetName;

	auto BuildNewName = [&Counter, &AssetName]()
	{
		return FString::Printf(TEXT("%s%d"), *AssetName, Counter++);
	};

	if (ComponentOwner)
	{
		// If a desired name is supplied then walk back and find any numeric suffix so we can increment it nicely
		int32 Index = AssetName.Len();
		while (Index > 0 && AssetName[Index - 1] >= '0' && AssetName[Index - 1] <= '9')
		{
			--Index;
		}

		if (Index < AssetName.Len())
		{
			FString NumericSuffix = AssetName.RightChop(Index);
			Counter = FCString::Atoi(*NumericSuffix);
			NumericSuffix = FString::Printf(TEXT("%d"), Counter); // Restringify the counter to account for leading 0s that we don't want to remove
			AssetName.RemoveAt(AssetName.Len() - NumericSuffix.Len(), NumericSuffix.Len(), EAllowShrinking::No);
			++Counter;
			NewName = BuildNewName();
		}

		while (!IsComponentNameAvailable(NewName, ComponentOwner))
		{
			NewName = BuildNewName();
		}
	}

	return NewName;
}

USceneComponent* FComponentEditorUtils::FindClosestParentInList(UActorComponent* ChildComponent, const TArray<UActorComponent*>& ComponentList)
{
	// Find the most recent parent that is part of the ComponentList
	if (USceneComponent* ChildAsScene = Cast<USceneComponent>(ChildComponent))
	{
		for (USceneComponent* Parent = ChildAsScene->GetAttachParent(); Parent != nullptr; Parent = Parent->GetAttachParent())
		{
			if (ComponentList.Contains(Parent))
			{
				return Parent;
			}
		}
	}
	return nullptr;
}

bool FComponentEditorUtils::CanCopyComponent(const UActorComponent* ComponentToCopy)
{
	if (ComponentToCopy != nullptr && ComponentToCopy->GetFName() != USceneComponent::GetDefaultSceneRootVariableName())
	{
		UClass* ComponentClass = ComponentToCopy->GetClass();
		check(ComponentClass != nullptr);

		// Component class cannot be abstract and must also be tagged as BlueprintSpawnable
		return FKismetEditorUtilities::IsClassABlueprintSpawnableComponent(ComponentClass);
	}

	return false;
}

bool FComponentEditorUtils::CanCopyComponents(const TArray<UActorComponent*>& ComponentsToCopy)
{
	bool bCanCopy = ComponentsToCopy.Num() > 0;
	if (bCanCopy)
	{
		for (int32 i = 0; i < ComponentsToCopy.Num() && bCanCopy; ++i)
		{
			// Check for the default scene root; that cannot be copied/duplicated
			UActorComponent* Component = ComponentsToCopy[i];
			bCanCopy = CanCopyComponent(Component);
		}
	}

	return bCanCopy;
}

void FComponentEditorUtils::CopyComponents(const TArray<UActorComponent*>& ComponentsToCopy, FString* DestinationData)
{
	FStringOutputDevice Archive;

	// Clear the mark state for saving.
	UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));

	// Duplicate the selected component templates into temporary objects that we can modify
	TMap<FName, FName> ParentMap;
	TMap<FName, UActorComponent*> ObjectMap;
	for (UActorComponent* Component : ComponentsToCopy)
	{
		// Duplicate the component into a temporary object
		UObject* DuplicatedComponent = StaticDuplicateObject(Component, GetTransientPackage());
		if (DuplicatedComponent)
		{
			// If the duplicated component is a scene component, wipe its attach parent (to prevent log warnings for referencing a private object in an external package)
			if (USceneComponent* DuplicatedCompAsSceneComp = Cast<USceneComponent>(DuplicatedComponent))
			{
				DuplicatedCompAsSceneComp->SetupAttachment(nullptr);

				AActor* Owner = Component->GetOwner();
				if (Owner && Component == Owner->GetRootComponent())
				{
					DuplicatedCompAsSceneComp->SetRelativeTransform_Direct(FTransform::Identity);
				}
			}

			// Find the closest parent component of the current component within the list of components to copy
			USceneComponent* ClosestSelectedParent = FindClosestParentInList(Component, ComponentsToCopy);
			if (ClosestSelectedParent)
			{
				// If the parent is included in the list, record it into the node->parent map
				ParentMap.Add(Component->GetFName(), ClosestSelectedParent->GetFName());
			}

			// Record the temporary object into the name->object map
			ObjectMap.Add(Component->GetFName(), CastChecked<UActorComponent>(DuplicatedComponent));
		}
	}

	const FExportObjectInnerContext Context;

	// Export the component object(s) to text for copying
	for (const TPair<FName, UActorComponent*>& ObjectPair : ObjectMap)
	{
		// Get the component object to be copied
		UActorComponent* ComponentToCopy = ObjectPair.Value;
		check(ComponentToCopy);

		// If this component object had a parent within the selected set
		if (ParentMap.Contains(ComponentToCopy->GetFName()))
		{
			// Get the name of the parent component
			FName ParentName = ParentMap[ComponentToCopy->GetFName()];
			if (ObjectMap.Contains(ParentName))
			{
				// Ensure that this component is a scene component
				USceneComponent* SceneComponent = Cast<USceneComponent>(ComponentToCopy);
				if (SceneComponent)
				{
					// Set the attach parent to the matching parent object in the temporary set. This allows us to preserve hierarchy in the copied set.
					SceneComponent->SetupAttachment(Cast<USceneComponent>(ObjectMap[ParentName]));
				}
			}
		}

		// Export the component object to the given string
		UExporter::ExportToOutputDevice(&Context, ComponentToCopy, NULL, Archive, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false, ComponentToCopy->GetOuter());
	}

	// Copy text to clipboard
	if (DestinationData)
	{
		*DestinationData = MoveTemp(Archive);
	}
	else
	{
		FPlatformApplicationMisc::ClipboardCopy(*Archive);
	}
}

bool FComponentEditorUtils::CanPasteComponents(const USceneComponent* RootComponent, bool bOverrideCanAttach, bool bPasteAsArchetypes, const FString* SourceData)
{
	FString ClipboardContent;
	if (SourceData)
	{
		ClipboardContent = *SourceData;
	}
	else
	{
		FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);
	}

	// Obtain the component object text factory for the clipboard content and return whether or not we can use it
	TSharedRef<FComponentObjectTextFactory> Factory = FComponentObjectTextFactory::Get(ClipboardContent, bPasteAsArchetypes);
	return Factory->NewObjectMap.Num() > 0 && ( bOverrideCanAttach || Factory->CanAttachComponentsTo(RootComponent) );
}

void FComponentEditorUtils::PasteComponents(TArray<UActorComponent*>& OutPastedComponents, AActor* TargetActor, USceneComponent* TargetComponent, const FString* SourceData)
{
	check(TargetActor);

	// Get the text from the clipboard
	FString TextToImport;
	if (SourceData)
	{
		TextToImport = *SourceData;
	}
	else
	{
		FPlatformApplicationMisc::ClipboardPaste(TextToImport);
	}

	// Get a new component object factory for the clipboard content
	TSharedRef<FComponentObjectTextFactory> Factory = FComponentObjectTextFactory::Get(TextToImport);

	TargetActor->Modify();

	USceneComponent* TargetParent = nullptr;
	if (TargetComponent)
	{
		checkSlow(TargetActor == TargetComponent->GetOwner());
		if (TargetActor->GetRootComponent() == TargetComponent)
		{
			TargetParent = TargetComponent;
		}
		else
		{
			TargetParent = TargetComponent->GetAttachParent();
		}
	}
	for (const TPair<FName, UActorComponent*>& NewObjectPair : Factory->NewObjectMap)
	{
		// Get the component object instance
		UActorComponent* NewActorComponent = NewObjectPair.Value;
		check(NewActorComponent);

		// Relocate the instance from the transient package to the Actor and assign it a unique object name
		FString NewComponentName = FComponentEditorUtils::GenerateValidVariableNameFromAsset(NewActorComponent, TargetActor);
		NewActorComponent->Rename(*NewComponentName, TargetActor, REN_DontCreateRedirectors | REN_DoNotDirty);

		if (USceneComponent* NewSceneComponent = Cast<USceneComponent>(NewActorComponent))
		{
			// Default to attaching to the target component's parent if possible, otherwise attach to the root
			USceneComponent* NewComponentParent = TargetParent ? TargetParent : TargetActor->GetRootComponent();
			
			// Check to see if there's an entry for the current component in the set of parent components
			if (Factory->ParentMap.Contains(NewObjectPair.Key))
			{
				// Get the parent component name
				FName ParentName = Factory->ParentMap[NewObjectPair.Key];
				if (Factory->NewObjectMap.Contains(ParentName))
				{
					// The parent should by definition be a scene component
					NewComponentParent = CastChecked<USceneComponent>(Factory->NewObjectMap[ParentName]);
				}
			}

			//@todo: Fix pasting when the pasted component was a root
			//NewSceneComponent->UpdateComponentToWorld();
			if (NewComponentParent)
			{
				// Reattach the current node to the parent node
				NewSceneComponent->AttachToComponent(NewComponentParent, FAttachmentTransformRules::KeepRelativeTransform);
			}
			else
			{
				// There is no root component and this component isn't the child of another component in the map, so make it the root
				TargetActor->SetRootComponent(NewSceneComponent);
			}
		}

		TargetActor->AddInstanceComponent(NewActorComponent);
		NewActorComponent->RegisterComponent();

		OutPastedComponents.Add(NewActorComponent);
	}

	// Rerun construction scripts
	TargetActor->RerunConstructionScripts();
}

void FComponentEditorUtils::GetComponentsFromClipboard(TMap<FName, FName>& OutParentMap, TMap<FName, UActorComponent*>& OutNewObjectMap, bool bGetComponentsAsArchetypes)
{
	// Get the text from the clipboard
	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);

	// Get a new component object factory for the clipboard content
	TSharedRef<FComponentObjectTextFactory> Factory = FComponentObjectTextFactory::Get(TextToImport, bGetComponentsAsArchetypes);

	// Return the created component mappings
	OutParentMap = MoveTemp(Factory->ParentMap);
	OutNewObjectMap = MoveTemp(Factory->NewObjectMap);
}

bool FComponentEditorUtils::CanDeleteComponent(const UActorComponent* ComponentToDelete)
{
	// We can't delete non-instance components or the default scene root
	return ComponentToDelete->CreationMethod == EComponentCreationMethod::Instance
		&& ComponentToDelete->GetFName() != USceneComponent::GetDefaultSceneRootVariableName();
}

bool FComponentEditorUtils::CanDeleteComponents(const TArray<UActorComponent*>& ComponentsToDelete)
{
	for (const UActorComponent* ComponentToDelete : ComponentsToDelete)
	{
		if (!CanDeleteComponent(ComponentToDelete))
		{
			return false;
		}
	}
	return true;
}

int32 FComponentEditorUtils::DeleteComponents(const TArray<UActorComponent*>& ComponentsToDelete, UActorComponent*& OutComponentToSelect)
{
	int32 NumDeletedComponents = 0;

	TArray<AActor*> ActorsToReconstruct;

	for (UActorComponent* ComponentToDelete : ComponentsToDelete)
	{
		if (ComponentToDelete->CreationMethod != EComponentCreationMethod::Instance)
		{
			// We can only delete instance components, so retain selection on the un-deletable component
			OutComponentToSelect = ComponentToDelete;
			continue;
		}

		AActor* Owner = ComponentToDelete->GetOwner();
		check(Owner != nullptr);

		// If necessary, determine the component that should be selected following the deletion of the indicated component
		if (!OutComponentToSelect || ComponentToDelete == OutComponentToSelect)
		{
			USceneComponent* RootComponent = Owner->GetRootComponent();
			if (RootComponent != ComponentToDelete)
			{
				// Worst-case, the root can be selected
				OutComponentToSelect = RootComponent;

				if (USceneComponent* ComponentToDeleteAsSceneComp = Cast<USceneComponent>(ComponentToDelete))
				{
					if (USceneComponent* ParentComponent = ComponentToDeleteAsSceneComp->GetAttachParent())
					{
						// The component to delete has a parent, so we select that in the absence of an appropriate sibling
						OutComponentToSelect = ParentComponent;

						// Try to select the sibling that immediately precedes the component to delete
						TArray<USceneComponent*> Siblings;
						ParentComponent->GetChildrenComponents(false, Siblings);
						for (int32 i = 0; i < Siblings.Num() && ComponentToDelete != Siblings[i]; ++i)
						{
							if (IsValid(Siblings[i]))
							{
								OutComponentToSelect = Siblings[i];
							}
						}
					}
				}
				else
				{
					// For a non-scene component, try to select the preceding non-scene component
					for (UActorComponent* Component : Owner->GetComponents())
					{
						if (Component != nullptr)
						{
							if (Component == ComponentToDelete)
							{
								break;
							}
							else if (!Component->IsA<USceneComponent>())
							{
								OutComponentToSelect = Component;
							}
						}
					}
				}
			}
			else
			{
				OutComponentToSelect = nullptr;
			}
		}

		// Defer reconstruction
		ActorsToReconstruct.AddUnique(Owner);

		// Actually delete the component
		ComponentToDelete->Modify();
		ComponentToDelete->DestroyComponent(true);
		NumDeletedComponents++;
	}

	// Non-native components will be reinstanced, so we have to update the ptr after reconstruction
	// in order to avoid pointing at an invalid (trash) instance after re-running construction scripts.
	FName ComponentToSelectName;
	const AActor* ComponentToSelectOwner = nullptr;
	if (OutComponentToSelect && OutComponentToSelect->CreationMethod != EComponentCreationMethod::Native)
	{
		// Keep track of the pending selection's name and owner
		ComponentToSelectName = OutComponentToSelect->GetFName();
		ComponentToSelectOwner = OutComponentToSelect->GetOwner();

		// Reset the ptr value - we'll reassign it after reconstruction
		OutComponentToSelect = nullptr;
	}

	// Reconstruct owner instance(s) after deletion
	for(AActor* ActorToReconstruct : ActorsToReconstruct)
	{
		check(ActorToReconstruct != nullptr);
		ActorToReconstruct->RerunConstructionScripts();

		// If this actor matches the owner of the component to be selected, find the new instance of the component in the actor
		if (ComponentToSelectName != NAME_None && OutComponentToSelect == nullptr && ActorToReconstruct == ComponentToSelectOwner)
		{
			TInlineComponentArray<UActorComponent*> ActorComponents;
			ActorToReconstruct->GetComponents(ActorComponents);
			for (UActorComponent* ActorComponent : ActorComponents)
			{
				if (ActorComponent->GetFName() == ComponentToSelectName)
				{
					OutComponentToSelect = ActorComponent;
					break;
				}
			}
		}
	}

	return NumDeletedComponents;
}

UActorComponent* FComponentEditorUtils::DuplicateComponent(UActorComponent* TemplateComponent)
{
	check(TemplateComponent);

	UActorComponent* NewCloneComponent = nullptr;
	AActor* Actor = TemplateComponent->GetOwner();
	if (!TemplateComponent->IsVisualizationComponent() && Actor)
	{
		Actor->Modify();
		FName NewComponentName = *FComponentEditorUtils::GenerateValidVariableNameFromAsset(TemplateComponent, Actor);

		bool bKeepWorldLocationOnAttach = false;

		const bool bTemplateTransactional = TemplateComponent->HasAllFlags(RF_Transactional);
		TemplateComponent->SetFlags(RF_Transactional);

		NewCloneComponent = DuplicateObject<UActorComponent>(TemplateComponent, Actor, NewComponentName );
		NewCloneComponent->ClearFlags(RF_DefaultSubObject);

		if (!bTemplateTransactional)
		{
			TemplateComponent->ClearFlags(RF_Transactional);
		}
			
		USceneComponent* NewSceneComponent = Cast<USceneComponent>(NewCloneComponent);
		if (NewSceneComponent)
		{
			// Ensure the clone doesn't think it has children
			FDirectAttachChildrenAccessor::Get(NewSceneComponent).Empty();

			// If the clone is a scene component without an attach parent, attach it to the root (can happen when duplicating the root component)
			if (!NewSceneComponent->GetAttachParent())
			{
				USceneComponent* RootComponent = Actor->GetRootComponent();
				check(RootComponent);

				// GetComponentTransform() is not a UPROPERTY, so make sure the clone has calculated it properly before attachment
				NewSceneComponent->SetRelativeTransform_Direct(FTransform::Identity);
				NewSceneComponent->UpdateComponentToWorld();

				NewSceneComponent->SetupAttachment(RootComponent);
			}
		}

		NewCloneComponent->OnComponentCreated();

		// Add to SerializedComponents array so it gets saved
		Actor->AddInstanceComponent(NewCloneComponent);
		
		// Register the new component
		NewCloneComponent->RegisterComponent();

		// Rerun construction scripts
		Actor->RerunConstructionScripts();
	}

	return NewCloneComponent;
}

void FComponentEditorUtils::AdjustComponentDelta(const USceneComponent* Component, FVector& Drag, FRotator& Rotation)
{
	if (const USceneComponent* ParentSceneComp = Component->GetAttachParent())
	{
		const FTransform ParentToWorldSpace = ParentSceneComp->GetSocketTransform(Component->GetAttachSocketName());

		if (!Component->IsUsingAbsoluteLocation())
		{
			//transform the drag vector in relative to the parent transform
			Drag = ParentToWorldSpace.InverseTransformVectorNoScale(Drag);
			//Now that we have a global drag we can apply the parent scale
			Drag = Drag * ParentToWorldSpace.Inverse().GetScale3D();
		}

		if (!Component->IsUsingAbsoluteRotation())
		{
			Rotation = ( ParentToWorldSpace.Inverse().GetRotation() * Rotation.Quaternion() * ParentToWorldSpace.GetRotation() ).Rotator();
		}
	}
}

void FComponentEditorUtils::BindComponentSelectionOverride(USceneComponent* SceneComponent, bool bBind)
{
	if (SceneComponent)
	{
		// If the scene component is a primitive component, set the override for it
		UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(SceneComponent);
		if (PrimitiveComponent && PrimitiveComponent->SelectionOverrideDelegate.IsBound() != bBind)
		{
			if (bBind)
			{
				PrimitiveComponent->SelectionOverrideDelegate = UPrimitiveComponent::FSelectionOverride::CreateUObject(GUnrealEd, &UUnrealEdEngine::IsComponentSelected);
			}
			else
			{
				PrimitiveComponent->SelectionOverrideDelegate.Unbind();
			}
		}
		else
		{
			TArray<UPrimitiveComponent*> ComponentsToBind;

			if (UChildActorComponent* ChildActorComponent = Cast<UChildActorComponent>(SceneComponent))
			{
				if (AActor* ChildActor = ChildActorComponent->GetChildActor())
				{
					ChildActor->GetComponents(ComponentsToBind,true);
				}
			}

			// Otherwise, make sure the override is set properly on any attached editor-only primitive components (ex: billboards)
			for (USceneComponent* Component : SceneComponent->GetAttachChildren())
			{
				if (UPrimitiveComponent* PrimComponent = Cast<UPrimitiveComponent>(Component))
				{
					if (PrimComponent->IsEditorOnly())
					{
						ComponentsToBind.Add(PrimComponent);
					}
				}
			}

			for (UPrimitiveComponent* PrimComponent : ComponentsToBind)
			{
				if (PrimComponent->SelectionOverrideDelegate.IsBound() != bBind)
				{
					if (bBind)
					{
						PrimComponent->SelectionOverrideDelegate = UPrimitiveComponent::FSelectionOverride::CreateUObject(GUnrealEd, &UUnrealEdEngine::IsComponentSelected);
					}
					else
					{
						PrimComponent->SelectionOverrideDelegate.Unbind();
					}
				}
			}
		}
	}
}

bool FComponentEditorUtils::AttemptApplyMaterialToComponent(USceneComponent* SceneComponent, UMaterialInterface* MaterialToApply, int32 OptionalMaterialSlot)
{
	bool bResult = false;
	
	UMeshComponent* MeshComponent = Cast<UMeshComponent>(SceneComponent);
	UDecalComponent* DecalComponent = Cast<UDecalComponent>(SceneComponent);

	UMaterial* BaseMaterial = MaterialToApply->GetBaseMaterial();

	bool bCanApplyToComponent = DecalComponent || ( MeshComponent && BaseMaterial &&  BaseMaterial->MaterialDomain != MD_DeferredDecal && BaseMaterial->MaterialDomain != MD_UI );
	// We can only apply a material to a mesh or a decal
	if (bCanApplyToComponent && (MeshComponent || DecalComponent) )
	{
		bResult = true;
		const FScopedTransaction Transaction(LOCTEXT("DropTarget_UndoSetComponentMaterial", "Assign Material to Component (Drag and Drop)"));
		FProperty* Property = FindFProperty<FProperty>(SceneComponent->GetClass(), MeshComponent ? "OverrideMaterials" : "DecalMaterial");
		SceneComponent->Modify();
		SceneComponent->PreEditChange(Property);

		if (MeshComponent)
		{
			// OK, we need to figure out how many material slots this mesh component/static mesh has.
			// Start with the actor's material count, then drill into the static/skeletal mesh to make sure 
			// we have the right total.
			int32 MaterialCount = FMath::Max(MeshComponent->OverrideMaterials.Num(), MeshComponent->GetNumMaterials());

			// Do we have an overridable material at the appropriate slot?
			if (MaterialCount > 0 && OptionalMaterialSlot < MaterialCount)
			{
				if (OptionalMaterialSlot == -1)
				{
					// Apply the material to every slot.
					for (int32 CurMaterialIndex = 0; CurMaterialIndex < MaterialCount; ++CurMaterialIndex)
					{
						MeshComponent->SetMaterial(CurMaterialIndex, MaterialToApply);
					}
				}
				else
				{
					// Apply only to the indicated slot.
					MeshComponent->SetMaterial(OptionalMaterialSlot, MaterialToApply);
				}
			}
		}
		else
		{
			DecalComponent->SetMaterial(0, MaterialToApply);
		}

		SceneComponent->MarkRenderStateDirty();
		FPropertyChangedEvent PropertyChangedEvent(Property, EPropertyChangeType::ValueSet, { SceneComponent });
		SceneComponent->PostEditChangeProperty(PropertyChangedEvent);
		GEditor->OnSceneMaterialsModified();
	}

	return bResult;
}

FName FComponentEditorUtils::FindVariableNameGivenComponentInstance(const UActorComponent* ComponentInstance)
{
	check(ComponentInstance != nullptr);

	// When names mismatch, try finding a differently named variable pointing to the the component (the mismatch should only be possible for native components)
	auto FindPropertyReferencingComponent = [](const UActorComponent* Component) -> FProperty*
	{
		if (AActor* OwnerActor = Component->GetOwner())
		{
			UClass* OwnerClass = OwnerActor->GetClass();
			AActor* OwnerCDO = CastChecked<AActor>(OwnerClass->GetDefaultObject());
			check(OwnerCDO->HasAnyFlags(RF_ClassDefaultObject));

			for (TFieldIterator<FObjectProperty> PropIt(OwnerClass, EFieldIteratorFlags::IncludeSuper); PropIt; ++PropIt)
			{
				FObjectProperty* TestProperty = *PropIt;
				if (Component->GetClass()->IsChildOf(TestProperty->PropertyClass))
				{
					void* TestPropertyInstanceAddress = TestProperty->ContainerPtrToValuePtr<void>(OwnerCDO);
					UObject* ObjectPointedToByProperty = TestProperty->GetObjectPropertyValue(TestPropertyInstanceAddress);
					if (ObjectPointedToByProperty == Component)
					{
						// This property points to the component archetype, so it's an anchor even if it was named wrong
						return TestProperty;
					}
				}
			}

			for (TFieldIterator<FArrayProperty> PropIt(OwnerClass, EFieldIteratorFlags::IncludeSuper); PropIt; ++PropIt)
			{
				FArrayProperty* TestProperty = *PropIt;
				void* ArrayPropInstAddress = TestProperty->ContainerPtrToValuePtr<void>(OwnerCDO);

				FObjectProperty* ArrayEntryProp = CastField<FObjectProperty>(TestProperty->Inner);
				if ((ArrayEntryProp == nullptr) || !ArrayEntryProp->PropertyClass->IsChildOf<UActorComponent>())
				{
					continue;
				}

				FScriptArrayHelper ArrayHelper(TestProperty, ArrayPropInstAddress);
				for (int32 ComponentIndex = 0; ComponentIndex < ArrayHelper.Num(); ++ComponentIndex)
				{
					UObject* ArrayElement = ArrayEntryProp->GetObjectPropertyValue(ArrayHelper.GetRawPtr(ComponentIndex));
					if (ArrayElement == Component)
					{
						return TestProperty;
					}
				}
			}
		}

		return nullptr;
	};

	// First see if the name just works
	if (AActor* OwnerActor = ComponentInstance->GetOwner())
	{
		UClass* OwnerActorClass = OwnerActor->GetClass();
		if (FObjectProperty* TestProperty = FindFProperty<FObjectProperty>(OwnerActorClass, ComponentInstance->GetFName()))
		{
			if (ComponentInstance->GetClass()->IsChildOf(TestProperty->PropertyClass))
					{
						return TestProperty->GetFName();
					}
				}

		if (FProperty* ReferencingProp = FindPropertyReferencingComponent(ComponentInstance))
		{
			return ReferencingProp->GetFName();
		}
			}

	if (UActorComponent* Archetype = Cast<UActorComponent>(ComponentInstance->GetArchetype()))
	{
		if (FProperty* ReferencingProp = FindPropertyReferencingComponent(Archetype))
		{
			return ReferencingProp->GetFName();
		}
	}

	return NAME_None;
}

void FComponentEditorUtils::FillComponentContextMenuOptions(UToolMenu* Menu, const TArray<UActorComponent*>& SelectedComponents)
{
	// Basic commands
	{
		FToolMenuSection& Section = Menu->AddSection("EditComponent", LOCTEXT("EditComponentHeading", "Edit"));
		Section.AddMenuEntry(FGenericCommands::Get().Cut);
		Section.AddMenuEntry(FGenericCommands::Get().Copy);
		Section.AddMenuEntry(FGenericCommands::Get().Paste);
		Section.AddMenuEntry(FGenericCommands::Get().Duplicate);
		Section.AddMenuEntry(FGenericCommands::Get().Delete);
		Section.AddMenuEntry(FGenericCommands::Get().Rename);
	}

	if (SelectedComponents.Num() == 1)
	{
		UActorComponent* Component = SelectedComponents[0];

		if (Component->GetClass()->ClassGeneratedBy)
		{
			{
				FToolMenuSection& Section = Menu->AddSection("ComponentAsset", LOCTEXT("ComponentAssetHeading", "Asset"));
				Section.AddMenuEntry(
					"GoToBlueprintForComponent",
					FText::Format(LOCTEXT("GoToBlueprintForComponent", "Edit {0}"), FText::FromString(Component->GetClass()->ClassGeneratedBy->GetName())),
					LOCTEXT("EditBlueprintForComponent_ToolTip", "Edits the Blueprint Class that defines this component."),
					FSlateIconFinder::FindIconForClass(Component->GetClass()),
					FUIAction(
										FExecuteAction::CreateStatic(&FComponentEditorUtils::OnEditBlueprintComponent, Component->GetClass()->ClassGeneratedBy.Get()),
					FCanExecuteAction()));

				Section.AddMenuEntry(
					"GoToAssetForComponent",
					LOCTEXT("GoToAssetForComponent", "Find Class in Content Browser"),
					LOCTEXT("GoToAssetForComponent_ToolTip", "Summons the content browser and goes to the class for this component."),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "SystemWideCommands.FindInContentBrowser"),
					FUIAction(
										FExecuteAction::CreateStatic(&FComponentEditorUtils::OnGoToComponentAssetInBrowser, Component->GetClass()->ClassGeneratedBy.Get()),
					FCanExecuteAction()));
			}
		}
		else
		{
			if (ensure(GUnrealEd) && GUnrealEd->GetUnrealEdOptions()->IsCPPAllowed())
			{
				FToolMenuSection& Section = Menu->AddSection("ComponentCode", LOCTEXT("ComponentCodeHeading", "C++"));
				if (FSourceCodeNavigation::IsCompilerAvailable())
				{
					FString ClassHeaderPath;
					if (FSourceCodeNavigation::FindClassHeaderPath(Component->GetClass(), ClassHeaderPath) && IFileManager::Get().FileSize(*ClassHeaderPath) != INDEX_NONE)
					{
						const FString CodeFileName = FPaths::GetCleanFilename(*ClassHeaderPath);

						Section.AddMenuEntry(
							"GoToCodeForComponent",
							FText::Format(LOCTEXT("GoToCodeForComponent", "Open {0}"), FText::FromString(CodeFileName)),
							FText::Format(LOCTEXT("GoToCodeForComponent_ToolTip", "Opens the header file for this component ({0}) in a code editing program"), FText::FromString(CodeFileName)),
							FSlateIcon(),
							FUIAction(
							FExecuteAction::CreateStatic(&FComponentEditorUtils::OnOpenComponentCodeFile, ClassHeaderPath),
							FCanExecuteAction()));
					}

					Section.AddMenuEntry(
						"GoToAssetForComponent",
						LOCTEXT("GoToAssetForComponent", "Find Class in Content Browser"),
						LOCTEXT("GoToAssetForComponent_ToolTip", "Summons the content browser and goes to the class for this component."),
						FSlateIcon(FAppStyle::GetAppStyleSetName(), "SystemWideCommands.FindInContentBrowser"),
						FUIAction(
						FExecuteAction::CreateStatic(&FComponentEditorUtils::OnGoToComponentAssetInBrowser, (UObject*)Component->GetClass()),
						FCanExecuteAction()));
				}
			}
		}
	}
}

UActorComponent* FComponentEditorUtils::FindMatchingComponent(const UActorComponent* ComponentInstance, const TInlineComponentArray<UActorComponent*>& ComponentList)
{
	if (ComponentInstance == nullptr)
	{
		return nullptr;
	}

	TInlineComponentArray<UActorComponent*> FoundComponents;
	UActorComponent* LastFoundComponent = nullptr;
	for (UActorComponent* Component : ComponentList)
	{
		// Early out on pointer match
		if (ComponentInstance == Component)
		{
			return Component;
		}

		if (ComponentInstance->GetFName() == Component->GetFName())
		{
			FoundComponents.Add(Component);
			LastFoundComponent = Component;
		}
	}

	// No match or 1 match avoid sorting
	if (FoundComponents.Num() <= 1)
	{
		return LastFoundComponent;
	}

	if (const USceneComponent* CurrentSceneComponent = Cast<USceneComponent>(ComponentInstance))
	{
		// Sort by matching hierarchy
		FoundComponents.Sort([&](const UActorComponent& ComponentA, const UActorComponent& ComponentB)
		{
			const USceneComponent* SceneComponentA = Cast<USceneComponent>(&ComponentA);
			const USceneComponent* SceneComponentB = Cast<USceneComponent>(&ComponentB);
			if (SceneComponentB == nullptr)
			{
				return true;
			}
			else if (SceneComponentA == nullptr)
			{
				return false;
			}

			const USceneComponent* AttachParentA = SceneComponentA->GetAttachParent();
			const USceneComponent* AttachParentB = SceneComponentB->GetAttachParent();
			const USceneComponent* CurrentParent = CurrentSceneComponent->GetAttachParent();
			// No parents...
			if (CurrentParent == nullptr)
			{
				return AttachParentA == nullptr;
			}

			bool MatchA = AttachParentA != nullptr && AttachParentA->GetFName() == CurrentParent->GetFName();
			bool MatchB = AttachParentB != nullptr && AttachParentB->GetFName() == CurrentParent->GetFName();
			while (MatchA && MatchB)
			{
				AttachParentA = AttachParentA->GetAttachParent();
				AttachParentB = AttachParentB->GetAttachParent();
				CurrentParent = CurrentParent->GetAttachParent();
				if (CurrentParent == nullptr)
				{
					return AttachParentA == nullptr;
				}

				MatchA = AttachParentA != nullptr && AttachParentA->GetFName() == CurrentParent->GetFName();
				MatchB = AttachParentB != nullptr && AttachParentB->GetFName() == CurrentParent->GetFName();
			}

			return MatchA;
		});
	}

	return FoundComponents[0];
}

FComponentReference FComponentEditorUtils::MakeComponentReference(const AActor* InExpectedComponentOwner, const UActorComponent* InComponent)
{
	FComponentReference Result;
	if (InComponent)
	{
		AActor* ComponentOwner = InComponent->GetOwner();
		const AActor* Owner = InExpectedComponentOwner;
		if (ComponentOwner && ComponentOwner != Owner)
		{
			Result.OtherActor = ComponentOwner;
			Owner = ComponentOwner;
		}

		if (InComponent->CreationMethod == EComponentCreationMethod::Native || InComponent->CreationMethod == EComponentCreationMethod::SimpleConstructionScript)
		{
			Result.ComponentProperty = FComponentEditorUtils::FindVariableNameGivenComponentInstance(InComponent);
		}
		if (Result.ComponentProperty.IsNone() && InComponent->CreationMethod != EComponentCreationMethod::UserConstructionScript)
		{
			Result.PathToComponent = InComponent->GetPathName(ComponentOwner);
		}
	}
	return Result;
}

void FComponentEditorUtils::OnGoToComponentAssetInBrowser(UObject* Asset)
{
	TArray<UObject*> Objects;
	Objects.Add(Asset);
	GEditor->SyncBrowserToObjects(Objects);
}

void FComponentEditorUtils::OnOpenComponentCodeFile(const FString CodeFileName)
{
	const FString AbsoluteHeaderPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*CodeFileName);
	FSourceCodeNavigation::OpenSourceFileAsync(AbsoluteHeaderPath);
}

void FComponentEditorUtils::OnEditBlueprintComponent(UObject* Blueprint)
{
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Blueprint);
}

#undef LOCTEXT_NAMESPACE
