// Copyright Epic Games, Inc. All Rights Reserved.


#include "Kismet2/DebuggerCommands.h"
#include "Misc/Paths.h"
#include "Misc/MessageDialog.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "Null/NullPlatformApplicationMisc.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SSpinBox.h"
#include "Framework/Docking/TabManager.h"
#include "Styling/AppStyle.h"
#include "GameFramework/Actor.h"
#include "Settings/LevelEditorPlaySettings.h"
#include "Editor/UnrealEdEngine.h"
#include "Settings/EditorExperimentalSettings.h"
#include "GameFramework/PlayerStart.h"
#include "Components/CapsuleComponent.h"
#include "LevelEditorViewport.h"
#include "UnrealEdGlobals.h"
#include "EditorAnalytics.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/KismetDebugUtilities.h"

#include "Interfaces/TargetDeviceId.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Interfaces/ITurnkeySupportModule.h"
#include "ITargetDeviceProxy.h"
#include "ITargetDeviceServicesModule.h"
#include "ISettingsModule.h"
#include "Interfaces/IMainFrameModule.h"

#include "EngineAnalytics.h"
#include "AnalyticsEventAttribute.h"
#include "Interfaces/IAnalyticsProvider.h"

#include "GameProjectGenerationModule.h"
#if WITH_UNREAL_TARGET_DEVELOPER_TOOLS
#include "Interfaces/IProjectTargetPlatformEditorModule.h"
#endif
#include "PlatformInfo.h"

#include "IHeadMountedDisplay.h"
#include "IVREditorModule.h"
#include "IXRTrackingSystem.h"
#include "Editor.h"

//@TODO: Remove this dependency
#include "EngineGlobals.h"
#include "LevelEditor.h"
#include "IAssetViewport.h"

#include "Logging/TokenizedMessage.h"
#include "Logging/MessageLog.h"

#include "Interfaces/IProjectManager.h"

#include "InstalledPlatformInfo.h"
#include "IDesktopPlatform.h"
#include "DesktopPlatformModule.h"
#include "CookerSettings.h"
#include "HAL/PlatformFileManager.h"
#include "SourceControlHelpers.h"
#include "ISourceControlModule.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#include "ToolMenus.h"
#include "SBlueprintEditorToolbar.h"
#include "SEnumCombo.h"
#include "Dialogs/Dialogs.h"

#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "IUATHelperModule.h"
#include "ISettingsEditorModule.h"
#include "Async/Async.h"
#include "Misc/FileHelper.h"
#include "Settings/ProjectPackagingSettings.h"


#define LOCTEXT_NAMESPACE "DebuggerCommands"

DEFINE_LOG_CATEGORY_STATIC(LogDebuggerCommands, Log, All);

namespace DebuggerCommands
{
	static bool bAllowPlayWorldFeature = true;
	static FAutoConsoleVariableRef AllowPlayWorldFeatureCVar(TEXT("Editor.AllowPlayWorldFeature"), bAllowPlayWorldFeature, TEXT("When true play world is allowed."));
	static bool AllowPlayWorldFeature();

	static bool IsVREditorActive()
	{
		if (IVREditorModule::IsAvailable() && IVREditorModule::Get().GetVRModeBase() != nullptr)
		{
			return true;
		}

		return false;
	}
}

void SGlobalPlayWorldActions::Construct(const FArguments& InArgs)
{
	// Always keep track of the current active play world actions widget so we later set user focus on it
	FPlayWorldCommands::SetActiveGlobalPlayWorldActionsWidget(SharedThis(this));

	ChildSlot
		[
			InArgs._Content.Widget
		];
}

FReply SGlobalPlayWorldActions::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	// Always keep track of the current active play world actions widget so we later set user focus on it
	FPlayWorldCommands::SetActiveGlobalPlayWorldActionsWidget(SharedThis(this));

	if (FPlayWorldCommands::GlobalPlayWorldActions->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	else
	{
		FPlayWorldCommands::SetActiveGlobalPlayWorldActionsWidget(TSharedPtr<SGlobalPlayWorldActions>());
		return FReply::Unhandled();
	}

}

bool SGlobalPlayWorldActions::SupportsKeyboardFocus() const
{
	return true;
}

// Put internal callbacks that we don't need to expose here in order to avoid unnecessary build dependencies outside of this module
class FInternalPlayWorldCommandCallbacks : public FPlayWorldCommandCallbacks
{
public:

	// Play In
	static void RepeatLastPlay_Clicked();
	static bool RepeatLastPlay_CanExecute();
	static FText GetRepeatLastPlayToolTip();
	static FSlateIcon GetRepeatLastPlayIcon();

	static void Simulate_Clicked();
	static bool Simulate_CanExecute();
	static bool Simulate_IsChecked();

	static void PlayInViewport_Clicked();
	static bool PlayInViewport_CanExecute();
	static void PlayInEditorFloating_Clicked();
	static bool PlayInEditorFloating_CanExecute();
	static void PlayInNewProcess_Clicked(EPlayModeType PlayModeType);
	static bool PlayInNewProcess_CanExecute();
	static void PlayInVR_Clicked();
	static bool PlayInVR_CanExecute();
	static bool PlayInModeIsChecked(EPlayModeType PlayMode);

	static bool PlayInLocation_CanExecute(EPlayModeLocations Location);
	static void PlayInLocation_Clicked(EPlayModeLocations Location);
	static bool PlayInLocation_IsChecked(EPlayModeLocations Location);

	static void PlayInSettings_Clicked();

	static void HandleShowSDKTutorial(FString PlatformName, FString NotInstalledDocLink);

	static void StopPlaySession_Clicked();
	static void LateJoinSession_Clicked();
	static void SingleFrameAdvance_Clicked();

	static void ShowCurrentStatement_Clicked();
	static void AbortExecution_Clicked();
	static void ContinueExecution_Clicked();
	static void StepInto_Clicked();
	static void StepOver_Clicked();
	static void StepOut_Clicked();

	static void TogglePlayPause_Clicked();

	// Mouse control
	static void GetMouseControlExecute();

	static void PossessEjectPlayer_Clicked();
	static bool CanPossessEjectPlayer();
	static FText GetPossessEjectLabel();
	static FText GetPossessEjectTooltip();
	static FSlateIcon GetPossessEjectImage();

	static bool CanLateJoin();
	static bool CanShowLateJoinButton();

	static bool IsStoppedAtBreakpoint();
	static bool IsStoppedAtBreakpoint_InEngineMode();

	static bool CanShowNonPlayWorldOnlyActions();
	static bool CanShowVROnlyActions();

	static int32 GetNumberOfClients();
	static void SetNumberOfClients(int32 NumClients, ETextCommit::Type CommitInfo = ETextCommit::Default);

	static int32 GetNetPlayMode();
	static void SetNetPlayMode(int32 Value);

protected:

	static void PlayInNewProcess(EPlayModeType PlayModeType, FString PIEPreviewDeviceName);

	/**
	 * Adds a message to the message log.
	 *
	 * @param Text The main message text.
	 * @param Detail The detailed description.
	 * @param TutorialLink A link to an associated tutorial.
	 * @param DocumentationLink A link to documentation.
	 */
	static void AddMessageLog(const FText& Text, const FText& Detail, const FString& TutorialLink, const FString& DocumentationLink);

	/**
	 * Checks whether the specified platform has a default device that can be launched on.
	 *
	 * @param PlatformName - The name of the platform to check.
	 *
	 * @return true if the platform can be played on, false otherwise.
	 */
	static bool CanLaunchOnDevice(const FString& DeviceName);

	/**
	 * Starts a game session on the default device of the specified platform.
	 *
	 * @param PlatformName - The name of the platform to play the game on.
	 */
	static void LaunchOnDevice(const FString& DeviceId, const FString& DeviceName, bool bUseTurnkey);

	/** Get the player start location to use when starting PIE */
	static EPlayModeLocations GetPlayModeLocation();

	/** checks to see if we have everything needed to launch a build to device */
	static bool IsReadyToLaunchOnDevice(FString DeviceId);
};

namespace DebuggerCommands
{
	bool AllowPlayWorldFeature()
	{
		return bAllowPlayWorldFeature && !FInternalPlayWorldCommandCallbacks::IsStoppedAtBreakpoint_InEngineMode();
	}
}

/**
 * Called to leave K2 debugging mode
 */
static void LeaveDebuggingMode()
{
	GUnrealEd->SetPIEWorldsPaused(false);

	// Determine whether or not we are resuming play.
	const bool bIsResumingPlay = !FKismetDebugUtilities::IsSingleStepping() && !GEditor->ShouldEndPlayMap();

	if (FSlateApplication::Get().InKismetDebuggingMode() && bIsResumingPlay)
	{
		// Focus the game view port when resuming from debugging.
		FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor").FocusPIEViewport();
	}

	// Tell the application to stop ticking in this stack frame. The parameter controls whether or not to recapture the mouse to the game viewport.
	FSlateApplication::Get().LeaveDebuggingMode(!bIsResumingPlay);
}


//////////////////////////////////////////////////////////////////////////
// FPlayWorldCommands

UE_DEFINE_TCOMMANDS(FPlayWorldCommands)

TSharedPtr<FUICommandList> FPlayWorldCommands::GlobalPlayWorldActions(new FUICommandList());

TWeakPtr<SGlobalPlayWorldActions> FPlayWorldCommands::ActiveGlobalPlayWorldActionsWidget;

TWeakPtr<SGlobalPlayWorldActions> FPlayWorldCommands::GetActiveGlobalPlayWorldActionsWidget()
{
	return FPlayWorldCommands::ActiveGlobalPlayWorldActionsWidget;
}

void FPlayWorldCommands::SetActiveGlobalPlayWorldActionsWidget(TWeakPtr<SGlobalPlayWorldActions> ActiveWidget)
{
	FPlayWorldCommands::ActiveGlobalPlayWorldActionsWidget = ActiveWidget;
}

FPlayWorldCommands::FPlayWorldCommands()
	: TCommands<FPlayWorldCommands>("PlayWorld", LOCTEXT("PlayWorld", "Play World (PIE/SIE)"), "MainFrame", FAppStyle::GetAppStyleSetName())
{
	ULevelEditorPlaySettings* PlaySettings = GetMutableDefault<ULevelEditorPlaySettings>();

	// initialize default Play device
	if (PlaySettings->LastExecutedLaunchName.IsEmpty())
	{
		FString RunningPlatformName = GetTargetPlatformManagerRef().GetRunningTargetPlatform()->PlatformName();
		FString PlayPlatformName;

		if (RunningPlatformName == TEXT("WindowsEditor"))
		{
			PlayPlatformName = TEXT("Windows");
		}
		else if (RunningPlatformName == TEXT("MacEditor"))
		{
			PlayPlatformName = TEXT("Mac");
		}
		else if (RunningPlatformName == TEXT("LinuxEditor"))
		{
			PlayPlatformName = TEXT("Linux");
		}

		if (!PlayPlatformName.IsEmpty())
		{
			ITargetPlatform* PlayPlatform = GetTargetPlatformManagerRef().FindTargetPlatform(PlayPlatformName);

			if (PlayPlatform != nullptr)
			{
				ITargetDevicePtr PlayDevice = PlayPlatform->GetDefaultDevice();

				if (PlayDevice.IsValid())
				{
					PlaySettings->LastExecutedLaunchDevice = PlayDevice->GetId().ToString();
					PlaySettings->LastExecutedLaunchName = PlayDevice->GetName();
					PlaySettings->SaveConfig();
				}
			}
		}
	}
}


void FPlayWorldCommands::RegisterCommands()
{
	// SIE
	UI_COMMAND(Simulate, "Simulate", "Start simulating the game", EUserInterfaceActionType::Check, FInputChord(EKeys::S, EModifierKey::Alt));

	// PIE
	UI_COMMAND(RepeatLastPlay, "Play", "Launches a game preview session in the same mode as the last game preview session launched from the Game Preview Modes dropdown next to the Play button on the level editor toolbar", EUserInterfaceActionType::Button, FInputChord(EKeys::P, EModifierKey::Alt))
	UI_COMMAND(PlayInViewport, "Selected Viewport", "Play this level in the active level editor viewport", EUserInterfaceActionType::Check, FInputChord());
	UI_COMMAND(PlaySimulate, "Start simulation", "Simulate this level in the active level editor viewport", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(PlayInEditorFloating, "New Editor Window (PIE)", "Play this level in a new window", EUserInterfaceActionType::Check, FInputChord());
	UI_COMMAND(PlayInVR, "VR Preview", "Play this level in VR", EUserInterfaceActionType::Check, FInputChord());
	UI_COMMAND(PlayInMobilePreview, "Standalone Game Mobile", "Play this level as a mobile device preview in Mobile mode (runs in its own process)", EUserInterfaceActionType::Check, FInputChord());
	UI_COMMAND(PlayInNewProcess, "Standalone Game", "Play this level in a new window that runs in its own process", EUserInterfaceActionType::Check, FInputChord());
	UI_COMMAND(PlayInCameraLocation, "Current Camera Location", "Spawn the player at the current camera location", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(PlayInDefaultPlayerStart, "Default Player Start", "Spawn the player at the map's default player start", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(PlayInNetworkSettings, "Network Settings...", "Open the settings for the 'Play In' feature", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(PlayInSettings, "Advanced Settings...", "Open the settings for the 'Play In' feature", EUserInterfaceActionType::Button, FInputChord());

	// SIE & PIE controls
	UI_COMMAND(StopPlaySession, "Stop", "Stop", EUserInterfaceActionType::Button, FInputChord(EKeys::Escape));
	UI_COMMAND(ResumePlaySession, "Resume", "Resume", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(PausePlaySession, "Pause", "Pause", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(GetMouseControl, "Mouse Control", "Get mouse cursor while in PIE", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::F1));
	UI_COMMAND(LateJoinSession, "Add Client", "Add another client", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SingleFrameAdvance, "Skip", "Advances a single frame", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(TogglePlayPauseOfPlaySession, "Toggle Play/Pause", "Resume playing if paused, or pause if playing", EUserInterfaceActionType::Button, FInputChord(EKeys::Pause));
	UI_COMMAND(PossessEjectPlayer, "Possess or Eject Player", "Possesses or ejects the player from the camera", EUserInterfaceActionType::Button, FInputChord(EKeys::F8));
	UI_COMMAND(ShowCurrentStatement, "Locate", "Locate the currently active node", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AbortExecution, "Abort", "Abort the execution", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ContinueExecution, "Continue", "Continue the execution", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt | EModifierKey::Shift, EKeys::F10));
	UI_COMMAND(StepInto, "Step Into", "Step Into the next node to be executed", EUserInterfaceActionType::Button, PLATFORM_MAC ? FInputChord(EModifierKey::Control, EKeys::F11) : FInputChord(EKeys::F11));
	UI_COMMAND(StepOver, "Step Over", "Step to the next node to be executed in the current graph", EUserInterfaceActionType::Button, FInputChord(EKeys::F10));
	UI_COMMAND(StepOut, "Step Out", "Step Out to the next node to be executed in the parent graph", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt | EModifierKey::Shift, EKeys::F11));
}

void FPlayWorldCommands::BindGlobalPlayWorldCommands()
{
	static bool bInitializedGlobalPlayWorldCommands = false;
	check(!bInitializedGlobalPlayWorldCommands);
	bInitializedGlobalPlayWorldCommands = true;

	const FPlayWorldCommands& Commands = FPlayWorldCommands::Get();
	FUICommandList& ActionList = *GlobalPlayWorldActions;

	// SIE
	ActionList.MapAction(Commands.Simulate,
		FExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::Simulate_Clicked),
		FCanExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::Simulate_CanExecute),
		FIsActionChecked::CreateStatic(&FInternalPlayWorldCommandCallbacks::PlayInModeIsChecked, PlayMode_Simulate),
		FIsActionButtonVisible::CreateStatic(&FInternalPlayWorldCommandCallbacks::CanShowNonPlayWorldOnlyActions)
	);

	// PIE
	ActionList.MapAction(Commands.RepeatLastPlay,
		FExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::RepeatLastPlay_Clicked),
		FCanExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::RepeatLastPlay_CanExecute),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateStatic(&FInternalPlayWorldCommandCallbacks::CanShowNonPlayWorldOnlyActions)
	);

	ActionList.MapAction(
		Commands.PlayInViewport,
		FExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::PlayInViewport_Clicked),
		FCanExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::PlayInViewport_CanExecute),
		FIsActionChecked::CreateLambda(
			[]() -> bool
			{
				// The toolbar play button launches PIE whenever we last ran SIE or PIE.
				// This logic makes sure the checked state in the ... submenu also indicates the same.
				return FInternalPlayWorldCommandCallbacks::PlayInModeIsChecked(PlayMode_InViewPort)
					|| FInternalPlayWorldCommandCallbacks::PlayInModeIsChecked(PlayMode_Simulate);
			}
		),
		FIsActionButtonVisible::CreateStatic(&FInternalPlayWorldCommandCallbacks::CanShowNonPlayWorldOnlyActions)
	);

	ActionList.MapAction(
		Commands.PlaySimulate,
		FExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::Simulate_Clicked),
		FCanExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::Simulate_CanExecute),
		FIsActionChecked::CreateStatic(&FInternalPlayWorldCommandCallbacks::PlayInModeIsChecked, PlayMode_Simulate),
		FIsActionButtonVisible::CreateStatic(&FInternalPlayWorldCommandCallbacks::CanShowNonPlayWorldOnlyActions)
	);

	ActionList.MapAction(Commands.PlayInEditorFloating,
		FExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::PlayInEditorFloating_Clicked),
		FCanExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::PlayInEditorFloating_CanExecute),
		FIsActionChecked::CreateStatic(&FInternalPlayWorldCommandCallbacks::PlayInModeIsChecked, PlayMode_InEditorFloating),
		FIsActionButtonVisible::CreateStatic(&FInternalPlayWorldCommandCallbacks::CanShowNonPlayWorldOnlyActions)
	);

	ActionList.MapAction(Commands.PlayInVR,
		FExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::PlayInVR_Clicked),
		FCanExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::PlayInVR_CanExecute),
		FIsActionChecked::CreateStatic(&FInternalPlayWorldCommandCallbacks::PlayInModeIsChecked, PlayMode_InVR),
		FIsActionButtonVisible::CreateStatic(&FInternalPlayWorldCommandCallbacks::CanShowVROnlyActions)
	);

	ActionList.MapAction(Commands.PlayInMobilePreview,
		FExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::PlayInNewProcess_Clicked, PlayMode_InMobilePreview),
		FCanExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::PlayInNewProcess_CanExecute),
		FIsActionChecked::CreateStatic(&FInternalPlayWorldCommandCallbacks::PlayInModeIsChecked, PlayMode_InMobilePreview),
		FIsActionButtonVisible::CreateStatic(&FInternalPlayWorldCommandCallbacks::CanShowNonPlayWorldOnlyActions)
	);

	ActionList.MapAction(Commands.PlayInNewProcess,
		FExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::PlayInNewProcess_Clicked, PlayMode_InNewProcess),
		FCanExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::PlayInNewProcess_CanExecute),
		FIsActionChecked::CreateStatic(&FInternalPlayWorldCommandCallbacks::PlayInModeIsChecked, PlayMode_InNewProcess),
		FIsActionButtonVisible::CreateStatic(&FInternalPlayWorldCommandCallbacks::CanShowNonPlayWorldOnlyActions)
	);

	ActionList.MapAction(Commands.PlayInCameraLocation,
		FExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::PlayInLocation_Clicked, PlayLocation_CurrentCameraLocation),
		FCanExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::PlayInLocation_CanExecute, PlayLocation_CurrentCameraLocation),
		FIsActionChecked::CreateStatic(&FInternalPlayWorldCommandCallbacks::PlayInLocation_IsChecked, PlayLocation_CurrentCameraLocation),
		FIsActionButtonVisible::CreateStatic(&FInternalPlayWorldCommandCallbacks::CanShowNonPlayWorldOnlyActions)
	);

	ActionList.MapAction(Commands.PlayInDefaultPlayerStart,
		FExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::PlayInLocation_Clicked, PlayLocation_DefaultPlayerStart),
		FCanExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::PlayInLocation_CanExecute, PlayLocation_DefaultPlayerStart),
		FIsActionChecked::CreateStatic(&FInternalPlayWorldCommandCallbacks::PlayInLocation_IsChecked, PlayLocation_DefaultPlayerStart),
		FIsActionButtonVisible::CreateStatic(&FInternalPlayWorldCommandCallbacks::CanShowNonPlayWorldOnlyActions)
	);

	ActionList.MapAction(Commands.PlayInSettings,
		FExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::PlayInSettings_Clicked)
	);


	// Stop play session
	ActionList.MapAction(Commands.StopPlaySession,
		FExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::StopPlaySession_Clicked),
		FCanExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::HasPlayWorld),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateStatic(&DebuggerCommands::AllowPlayWorldFeature)
	);

	// Late join session
	ActionList.MapAction(Commands.LateJoinSession,
		FExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::LateJoinSession_Clicked),
		FCanExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::CanLateJoin),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateStatic(&FInternalPlayWorldCommandCallbacks::CanShowLateJoinButton)
	);

	// Play, Pause, Toggle between play and pause
	ActionList.MapAction(Commands.ResumePlaySession,
		FExecuteAction::CreateStatic(&FPlayWorldCommandCallbacks::ResumePlaySession_Clicked),
		FCanExecuteAction::CreateStatic(&FPlayWorldCommandCallbacks::HasPlayWorldAndPaused),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateStatic(&FPlayWorldCommandCallbacks::HasPlayWorldAndPaused)
	);

	ActionList.MapAction(Commands.PausePlaySession,
		FExecuteAction::CreateStatic(&FPlayWorldCommandCallbacks::PausePlaySession_Clicked),
		FCanExecuteAction::CreateStatic(&FPlayWorldCommandCallbacks::HasPlayWorldAndRunning),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateStatic(&FPlayWorldCommandCallbacks::HasPlayWorldAndRunning)
	);

	ActionList.MapAction(Commands.SingleFrameAdvance,
		FExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::SingleFrameAdvance_Clicked),
		FCanExecuteAction::CreateStatic(&FPlayWorldCommandCallbacks::HasPlayWorldAndPaused),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateStatic(&DebuggerCommands::AllowPlayWorldFeature)
	);

	ActionList.MapAction(Commands.TogglePlayPauseOfPlaySession,
		FExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::TogglePlayPause_Clicked),
		FCanExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::HasPlayWorld),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateStatic(&FInternalPlayWorldCommandCallbacks::HasPlayWorld)
	);

	// Get mouse control from PIE
	ActionList.MapAction(Commands.GetMouseControl,
		FExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::GetMouseControlExecute),
		FCanExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::HasPlayWorld),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateStatic(&FInternalPlayWorldCommandCallbacks::HasPlayWorld)
	);

	// Toggle PIE/SIE, Eject (PIE->SIE), and Possess (SIE->PIE)
	ActionList.MapAction(
		Commands.PossessEjectPlayer,
		FExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::PossessEjectPlayer_Clicked),
		FCanExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::CanPossessEjectPlayer),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateStatic(&FInternalPlayWorldCommandCallbacks::CanPossessEjectPlayer)
	);

	// Breakpoint-only commands
	ActionList.MapAction(Commands.ShowCurrentStatement,
		FExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::ShowCurrentStatement_Clicked),
		FCanExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::IsStoppedAtBreakpoint),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateStatic(&FInternalPlayWorldCommandCallbacks::IsStoppedAtBreakpoint)
	);

	ActionList.MapAction(Commands.AbortExecution,
		FExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::AbortExecution_Clicked),
		FCanExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::IsStoppedAtBreakpoint_InEngineMode),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateStatic(&FInternalPlayWorldCommandCallbacks::IsStoppedAtBreakpoint_InEngineMode)
	);

	ActionList.MapAction(Commands.ContinueExecution,
		FExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::ContinueExecution_Clicked),
		FCanExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::IsStoppedAtBreakpoint),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateStatic(&FInternalPlayWorldCommandCallbacks::IsStoppedAtBreakpoint)
	);
	
	ActionList.MapAction(Commands.StepInto,
		FExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::StepInto_Clicked),
		FCanExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::IsStoppedAtBreakpoint),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateStatic(&FInternalPlayWorldCommandCallbacks::IsStoppedAtBreakpoint)
	);

	ActionList.MapAction(Commands.StepOver,
		FExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::StepOver_Clicked),
		FCanExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::IsStoppedAtBreakpoint),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateStatic(&FInternalPlayWorldCommandCallbacks::IsStoppedAtBreakpoint)
	);

	ActionList.MapAction(Commands.StepOut,
		FExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::StepOut_Clicked),
		FCanExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::IsStoppedAtBreakpoint),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateStatic(&FInternalPlayWorldCommandCallbacks::IsStoppedAtBreakpoint)
	);
}

void FPlayWorldCommands::BuildToolbar(FToolMenuSection& InSection)
{
	FToolMenuEntry PlayMenuEntry =
		FToolMenuEntry::InitToolBarButton(
			FPlayWorldCommands::Get().RepeatLastPlay,
			TAttribute<FText>(),
			TAttribute< FText >::Create(TAttribute< FText >::FGetter::CreateStatic(&FInternalPlayWorldCommandCallbacks::GetRepeatLastPlayToolTip)),
			TAttribute< FSlateIcon >::Create(TAttribute< FSlateIcon >::FGetter::CreateStatic(&FInternalPlayWorldCommandCallbacks::GetRepeatLastPlayIcon)),
			FName(TEXT("LevelToolbarPlay")));
	PlayMenuEntry.StyleNameOverride = FName("Toolbar.BackplateLeftPlay");

	// Play
	InSection.AddEntry(PlayMenuEntry);

	// First slot (only one is visible at a time.)
	{
		{
			FToolMenuEntry ResumeEntry = FToolMenuEntry::InitToolBarButton(
				FPlayWorldCommands::Get().ResumePlaySession,
				TAttribute<FText>(),
				TAttribute<FText>(),
				TAttribute<FSlateIcon>::CreateLambda(
					[]() -> FSlateIcon
					{
						if (FPlayWorldCommandCallbacks::IsInPIE())
						{
							return FSlateIcon(FAppStyle::GetAppStyleSetName(), "PlayWorld.PlayInViewport");
						}
						else
						{
							return FSlateIcon(FAppStyle::GetAppStyleSetName(), "PlayWorld.PlaySimulate");
						}
					}
				),
				FName(TEXT("ResumePlaySession"))
			);
			ResumeEntry.StyleNameOverride = FName("Toolbar.BackplateLeft");
			InSection.AddEntry(ResumeEntry);
		}

		{
			FToolMenuEntry PauseEntry = FToolMenuEntry::InitToolBarButton(
				FPlayWorldCommands::Get().PausePlaySession,
				TAttribute<FText>(),
				TAttribute<FText>(),
				TAttribute<FSlateIcon>(),
				FName(TEXT("PausePlaySession"))
			);
			PauseEntry.StyleNameOverride = FName("Toolbar.BackplateLeft");
			InSection.AddEntry(PauseEntry);
		}
	}

	FToolMenuEntry SingleFrameAdvanceEntry = FToolMenuEntry::InitToolBarButton(FPlayWorldCommands::Get().SingleFrameAdvance, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>(), FName(TEXT("SingleFrameAdvance")));
	SingleFrameAdvanceEntry.StyleNameOverride = FName("Toolbar.BackplateCenter");
	InSection.AddEntry(SingleFrameAdvanceEntry);

	// Stop
	FToolMenuEntry StopEntry = FToolMenuEntry::InitToolBarButton(
		FPlayWorldCommands::Get().StopPlaySession,
		TAttribute<FText>(),
		TAttribute<FText>(),
		TAttribute<FSlateIcon>(),
		FName(TEXT("StopPlaySession"))
	);
	StopEntry.StyleNameOverride = FName("Toolbar.BackplateCenterStop");

	InSection.AddEntry(StopEntry);

	// Late Join
	FToolMenuEntry LateJoinEntry = FToolMenuEntry::InitToolBarButton(FPlayWorldCommands::Get().LateJoinSession, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>(), FName(TEXT("LateJoinSession")));
	LateJoinEntry.StyleNameOverride = FName("Toolbar.BackplateCenter");
	InSection.AddEntry(LateJoinEntry);

	// Fourth slot (only one is visible at a time.)
	{
		{
			FToolMenuEntry PlaySimulateEntry = FToolMenuEntry::InitToolBarButton(
				FPlayWorldCommands::Get().PlaySimulate,
				TAttribute<FText>(),
				TAttribute<FText>(),
				TAttribute<FSlateIcon>(),
				FName(TEXT("PlaySimulate"))
			);
			PlaySimulateEntry.StyleNameOverride = FName("Toolbar.BackplateCenter");
			InSection.AddEntry(PlaySimulateEntry);
		}

		{
			FToolMenuEntry PossessEjectEntry = FToolMenuEntry::InitToolBarButton(
				FPlayWorldCommands::Get().PossessEjectPlayer,
				TAttribute<FText>::Create(
					TAttribute<FText>::FGetter::CreateStatic(&FInternalPlayWorldCommandCallbacks::GetPossessEjectLabel)
				),
				TAttribute<FText>::Create(
					TAttribute<FText>::FGetter::CreateStatic(&FInternalPlayWorldCommandCallbacks::GetPossessEjectTooltip)
				),
				TAttribute<FSlateIcon>::Create(
					TAttribute<FSlateIcon>::FGetter::CreateStatic(&FInternalPlayWorldCommandCallbacks::GetPossessEjectImage)
				),
				FName(TEXT("PossessEjectPlayer"))
			);
			PossessEjectEntry.StyleNameOverride = FName("Toolbar.BackplateCenter");
			InSection.AddEntry(PossessEjectEntry);
		}
	}

	FUIAction SpecialPIEOptionsMenuAction;
	SpecialPIEOptionsMenuAction.CanExecuteAction = FCanExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::CanShowNonPlayWorldOnlyActions);
	SpecialPIEOptionsMenuAction.IsActionVisibleDelegate = FIsActionButtonVisible::CreateStatic(&DebuggerCommands::AllowPlayWorldFeature);

	FToolMenuEntry PIEComboEntry = FToolMenuEntry::InitComboButton("PIECombo", SpecialPIEOptionsMenuAction, FOnGetContent::CreateStatic(&GeneratePlayMenuContent, GlobalPlayWorldActions.ToSharedRef()), LOCTEXT("PIEComboLabel","Play Options"), LOCTEXT("PIEComboToolTip", "Change Play Mode and Play Settings"));
	PIEComboEntry.StyleNameOverride = FName("Toolbar.BackplateRightCombo");

	InSection.AddEntry(PIEComboEntry);

	// Single-stepping only buttons
	
	FToolMenuEntry ShowCurrentStatementEntry = FToolMenuEntry::InitToolBarButton(FPlayWorldCommands::Get().ShowCurrentStatement, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>(), FName(TEXT("ShowCurrentStatement")));
	ShowCurrentStatementEntry.StyleNameOverride = FName("Toolbar.BackplateLeft");

	FToolMenuEntry ContinueEntry = FToolMenuEntry::InitToolBarButton(FPlayWorldCommands::Get().ContinueExecution, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>(), FName(TEXT("ContinueExecution")));
	ContinueEntry.StyleNameOverride = FName("Toolbar.BackplateCenter");

	FToolMenuEntry AbortEntry = FToolMenuEntry::InitToolBarButton(FPlayWorldCommands::Get().AbortExecution, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>(), FName(TEXT("AbortExecution")));
	AbortEntry.StyleNameOverride = FName("Toolbar.BackplateCenter");
	
	FToolMenuEntry StepIntoEntry = FToolMenuEntry::InitToolBarButton(FPlayWorldCommands::Get().StepInto, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>(), FName(TEXT("StepInto")));
	StepIntoEntry.StyleNameOverride = FName("Toolbar.BackplateCenter");

	FToolMenuEntry StepOverEntry = FToolMenuEntry::InitToolBarButton(FPlayWorldCommands::Get().StepOver, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>(), FName(TEXT("StepOver")));
	StepOverEntry.StyleNameOverride = FName("Toolbar.BackplateCenter");

	FToolMenuEntry StepOutEntry = FToolMenuEntry::InitToolBarButton(FPlayWorldCommands::Get().StepOut, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>(), FName(TEXT("StepOut")));
	StepOutEntry.StyleNameOverride = FName("Toolbar.BackplateRight");

	InSection.AddEntry(ShowCurrentStatementEntry);
	InSection.AddEntry(ContinueEntry);
	InSection.AddEntry(AbortEntry);
	InSection.AddEntry(StepIntoEntry);	
	InSection.AddEntry(StepOverEntry);	
	InSection.AddEntry(StepOutEntry);
}

void SetLastExecutedPlayMode(EPlayModeType PlayMode)
{
	ULevelEditorPlaySettings* PlaySettings = GetMutableDefault<ULevelEditorPlaySettings>();
	PlaySettings->LastExecutedPlayModeType = PlayMode;

	FPropertyChangedEvent PropChangeEvent(ULevelEditorPlaySettings::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ULevelEditorPlaySettings, LastExecutedPlayModeType)));
	PlaySettings->PostEditChangeProperty(PropChangeEvent);

	PlaySettings->SaveConfig();
}


static void RememberQuickLaunch(FString DeviceId)
{
	// remember that clicking Play should launch
	SetLastExecutedPlayMode(EPlayModeType::PlayMode_QuickLaunch);

	// store the device name in the play settings for next click/run
	ULevelEditorPlaySettings* PlaySettings = GetMutableDefault<ULevelEditorPlaySettings>();

	PlaySettings->LastExecutedLaunchName = DeviceId;

	FPropertyChangedEvent PropChangeEvent(ULevelEditorPlaySettings::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ULevelEditorPlaySettings, LastExecutedLaunchName)));
	PlaySettings->PostEditChangeProperty(PropChangeEvent);
	PlaySettings->SaveConfig();
}

TSharedRef< SWidget > FPlayWorldCommands::GeneratePlayMenuContent(TSharedRef<FUICommandList> InCommandList)
{
	static const FName MenuName("UnrealEd.PlayWorldCommands.PlayMenu");

	if (!UToolMenus::Get()->IsMenuRegistered(MenuName))
	{
		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(MenuName);

		struct FLocal
		{
			static void AddPlayModeMenuEntry(FToolMenuSection& Section, EPlayModeType PlayMode)
			{
				TSharedPtr<FUICommandInfo> PlayModeCommand;

				switch (PlayMode)
				{
				case PlayMode_InEditorFloating:
					PlayModeCommand = FPlayWorldCommands::Get().PlayInEditorFloating;
					break;

				case PlayMode_InMobilePreview:
					PlayModeCommand = FPlayWorldCommands::Get().PlayInMobilePreview;
					break;

				case PlayMode_InNewProcess:
					PlayModeCommand = FPlayWorldCommands::Get().PlayInNewProcess;
					break;

				case PlayMode_InViewPort:
					PlayModeCommand = FPlayWorldCommands::Get().PlayInViewport;
					break;

				case PlayMode_InVR:
					PlayModeCommand = FPlayWorldCommands::Get().PlayInVR;
					break;
				}

				if (PlayModeCommand.IsValid())
				{
					Section.AddMenuEntry(PlayModeCommand);
				}
			}
		};

		// play in view port
		{
			FToolMenuSection& Section = Menu->AddSection("LevelEditorPlayModes", LOCTEXT("PlayButtonModesSection", "Modes"));
			FLocal::AddPlayModeMenuEntry(Section, PlayMode_InViewPort);
			FLocal::AddPlayModeMenuEntry(Section, PlayMode_InEditorFloating);
			FLocal::AddPlayModeMenuEntry(Section, PlayMode_InVR);
			FLocal::AddPlayModeMenuEntry(Section, PlayMode_InMobilePreview);
			FLocal::AddPlayModeMenuEntry(Section, PlayMode_InNewProcess);

			Section.AddSeparator("SimulateSeparator");

			if (FPlayWorldCommands::Get().Simulate)
			{
				Section.AddMenuEntry(
					"Simulate",
					FPlayWorldCommands::Get().Simulate->GetLabel(),
					FPlayWorldCommands::Get().Simulate->GetDescription(),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "PlayWorld.PlaySimulate"),
					FExecuteAction::CreateLambda(
						[]()
						{
							FPlayWorldCommands::GlobalPlayWorldActions->ExecuteAction(
								FPlayWorldCommands::Get().Simulate.ToSharedRef()
							);
						}
					)
				);
			}
		}

		// tip section
		{
			FToolMenuSection& Section = Menu->AddSection("LevelEditorPlayTip");
			Section.AddSeparator(NAME_None);
			Section.AddEntry(FToolMenuEntry::InitWidget(
				"PlayIn",
				SNew(SBox)
				.Padding(FMargin(16.0f, 3.0f))
				[
					SNew(STextBlock)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					.Text(LOCTEXT("PlayInTip", "Launching a game (preview or on device) with a different mode will change your default 'Play' mode in the toolbar"))
					.WrapTextAt(250)
				],
				FText::GetEmpty()));
		}

		// player start selection
		{
			FToolMenuSection& Section = Menu->AddSection("LevelEditorPlayPlayerStart", LOCTEXT("PlayButtonLocationSection", "Spawn player at..."));
			Section.AddMenuEntry(FPlayWorldCommands::Get().PlayInCameraLocation);
			Section.AddMenuEntry(FPlayWorldCommands::Get().PlayInDefaultPlayerStart);
		}

		// Basic network options
		const ULevelEditorPlaySettings* PlayInSettings = GetDefault<ULevelEditorPlaySettings>();
		{
			FToolMenuSection& Section = Menu->AddSection("LevelEditorPlayInWindowNetwork", LOCTEXT("LevelEditorPlayInWindowNetworkSection", "Multiplayer Options"));
			// Num Clients
			{
				TSharedRef<SWidget> NumPlayers = SNew(SSpinBox<int32>)	// Copy limits from PlayNumberOfClients meta data
					.MinValue(1)
					.MaxValue(MAX_PIE_INSTANCES - 1)
					.MinSliderValue(1)
					.MaxSliderValue(MAX_PIE_INSTANCES - 1)
					.Delta(1)
					.ToolTipText(LOCTEXT("NumberOfClientsToolTip", "How many client instances do you want to create? The first instance respects the Play Mode location (PIE/PINW) and additional instances respect the RunUnderOneProcess setting."))
					.Value_Static(&FInternalPlayWorldCommandCallbacks::GetNumberOfClients)
					.OnValueCommitted_Static(&FInternalPlayWorldCommandCallbacks::SetNumberOfClients)
					.OnValueChanged_Lambda([](int32 InNumClients) { FInternalPlayWorldCommandCallbacks::SetNumberOfClients(InNumClients, ETextCommit::Default); });

				Section.AddEntry(FToolMenuEntry::InitWidget("NumPlayers", NumPlayers, LOCTEXT("NumberOfClientsMenuWidget", "Number of Players")));
			}
			// Net Mode
			{
				Section.AddSubMenu(
					"NetMode",
					LOCTEXT("NetworkModeMenu", "Net Mode"),
					LOCTEXT("NetworkModeToolTip", "Which network mode should the clients launch in? A server will automatically be started if needed."),
					FNewMenuDelegate::CreateLambda([](FMenuBuilder& InMenuBuilder)
						{
							const UEnum* PlayNetModeEnum = FindObject<UEnum>(nullptr, TEXT("/Script/UnrealEd.EPlayNetMode"));

							for (int32 i = 0; i < PlayNetModeEnum->NumEnums() - 1; i++)
							{
								if (PlayNetModeEnum->HasMetaData(TEXT("Hidden"), i) == false)
								{
									FUIAction Action(FExecuteAction::CreateStatic(&FInternalPlayWorldCommandCallbacks::SetNetPlayMode, i), FCanExecuteAction(), FIsActionChecked::CreateLambda([](int32 Index) {return FInternalPlayWorldCommandCallbacks::GetNetPlayMode() == Index; }, i));
									InMenuBuilder.AddMenuEntry(PlayNetModeEnum->GetDisplayNameTextByIndex(i), PlayNetModeEnum->GetToolTipTextByIndex(i), FSlateIcon(), Action, NAME_None, EUserInterfaceActionType::RadioButton);
								}
							}
						})
						,false);
			}
		}

		// settings
		{
			FToolMenuSection& Section = Menu->AddSection("LevelEditorPlaySettings");
			Section.AddMenuEntry(FPlayWorldCommands::Get().PlayInSettings);
		}
	}

	// Get all menu extenders for this context menu from the level editor module
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
	TSharedPtr<FExtender> MenuExtender = LevelEditorModule.AssembleExtenders(InCommandList, LevelEditorModule.GetAllLevelEditorToolbarPlayMenuExtenders());
	FToolMenuContext MenuContext(InCommandList, MenuExtender);
	return UToolMenus::Get()->GenerateWidget(MenuName, MenuContext);
}


//////////////////////////////////////////////////////////////////////////
// FPlayWorldCommandCallbacks

void FPlayWorldCommandCallbacks::StartPlayFromHere()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	FStartPlayFromHereOptions Options;
	Options.Mode = EPlayFromHereMode::Click;
	Options.DestinationSlateViewport = LevelEditorModule.GetFirstActiveViewport();
	StartPlayFromHere(Options);
}

void FPlayWorldCommandCallbacks::StartPlayFromHere(const TOptional<FVector>& Location, const TOptional<FRotator>& Rotation, const TSharedPtr<IAssetViewport>& ActiveLevelViewport)
{
	FStartPlayFromHereOptions Options;
	Options.Mode = EPlayFromHereMode::Custom;
	Options.Location = Location;
	Options.Rotation = Rotation;
	Options.DestinationSlateViewport = ActiveLevelViewport;
	StartPlayFromHere(Options);
}

bool FPlayWorldCommandCallbacks::StartPlayFromHere(const FStartPlayFromHereOptions& InOptions)
{
	if (FInternalPlayWorldCommandCallbacks::IsStoppedAtBreakpoint_InEngineMode() || DebuggerCommands::IsVREditorActive())
	{
		return false;
	}

	TOptional<FVector> StartLocation = InOptions.Location;
	TOptional<FRotator> StartRotation = InOptions.Rotation;

	bool bCanProceed = InOptions.Mode == EPlayFromHereMode::Custom;
	if (InOptions.Mode == EPlayFromHereMode::Click)
	{
		// Figure out the start location of the player
		const UClass* const PlayerStartClass = GUnrealEd->PlayFromHerePlayerStartClass ? (UClass*)GUnrealEd->PlayFromHerePlayerStartClass : APlayerStart::StaticClass();
		const UCapsuleComponent* DefaultCollisionComponent = CastChecked<UCapsuleComponent>(PlayerStartClass->GetDefaultObject<AActor>()->GetRootComponent());
		const FVector CollisionExtent = FVector(DefaultCollisionComponent->GetScaledCapsuleRadius(), DefaultCollisionComponent->GetScaledCapsuleRadius(), DefaultCollisionComponent->GetScaledCapsuleHalfHeight());
		StartLocation = GEditor->UnsnappedClickLocation + GEditor->ClickPlane * (FVector::BoxPushOut(GEditor->ClickPlane, CollisionExtent) + 0.1f);
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");

		const TSharedPtr<IAssetViewport> ActiveLevelViewport = LevelEditorModule.GetFirstActiveViewport();
		if (ActiveLevelViewport.IsValid() && ActiveLevelViewport->GetAssetViewportClient().IsPerspective())
		{
			StartRotation = ActiveLevelViewport->GetAssetViewportClient().GetViewRotation();
		}

		bCanProceed = true;
	}
	else if (InOptions.Mode == EPlayFromHereMode::SelectedActor)
	{
		if (GEditor->GetSelectedActorCount() == 1)
		{
			if (AActor* Actor = Cast<AActor>(*GEditor->GetSelectedActorIterator()))
			{
				if (const UWorld* ActorWorld = Actor->GetWorld())
				{
					if (ActorWorld->PersistentLevel)
					{
						ActorWorld->PersistentLevel->PlayFromHereActor = Actor;
					}
				}
				StartLocation = Actor->GetActorLocation();
				StartRotation = Actor->GetActorRotation();
				bCanProceed = true;
			}
		}
	}
	
	if (!bCanProceed)
	{
		return false;
	}

	FRequestPlaySessionParams SessionParams;
	SessionParams.StartLocation = StartLocation;
	SessionParams.StartRotation = StartRotation;

	if (InOptions.DestinationSlateViewport.IsValid() && InOptions.DestinationSlateViewport->GetAssetViewportClient().IsPerspective())
	{
		// If there is no level viewport, a new window will be spawned to play in.
		SessionParams.DestinationSlateViewport = InOptions.DestinationSlateViewport;
	}

	// Is a PIE session already running?  If so we close it first
	if (GUnrealEd->PlayWorld != nullptr)
	{
		GUnrealEd->EndPlayMap();
	}

	GUnrealEd->RequestPlaySession(SessionParams);

	return true;
}

void FPlayWorldCommandCallbacks::ResumePlaySession_Clicked()
{
	if (FInternalPlayWorldCommandCallbacks::IsStoppedAtBreakpoint_InEngineMode())
	{
		return;
	}

	if (HasPlayWorld())
	{
		LeaveDebuggingMode();
		GUnrealEd->PlaySessionResumed();
		uint32 UserIndex = 0;
		FSlateApplication::Get().SetUserFocusToGameViewport(UserIndex);
	}
}


void FPlayWorldCommandCallbacks::PausePlaySession_Clicked()
{
	if (FInternalPlayWorldCommandCallbacks::IsStoppedAtBreakpoint_InEngineMode())
	{
		return;
	}

	if (HasPlayWorld())
	{
		GUnrealEd->SetPIEWorldsPaused(true);
		GUnrealEd->PlaySessionPaused();
		if (IsInPIE())
		{
			FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::SetDirectly);
			FSlateApplication::Get().ResetToDefaultInputSettings();

			TWeakPtr<SGlobalPlayWorldActions> ActiveGlobalPlayWorldWidget = FPlayWorldCommands::GetActiveGlobalPlayWorldActionsWidget();
			if (ActiveGlobalPlayWorldWidget.IsValid())
			{
				uint32 UserIndex = 0;
				FSlateApplication::Get().SetUserFocus(UserIndex, ActiveGlobalPlayWorldWidget.Pin());
			}
		}
	}
}


void FPlayWorldCommandCallbacks::SingleFrameAdvance_Clicked()
{
	if (FInternalPlayWorldCommandCallbacks::IsStoppedAtBreakpoint_InEngineMode())
	{
		return;
	}

	if (HasPlayWorld())
	{
		FInternalPlayWorldCommandCallbacks::SingleFrameAdvance_Clicked();
	}
}

bool FPlayWorldCommandCallbacks::IsInSIE()
{
	return GEditor->bIsSimulatingInEditor;
}


bool FPlayWorldCommandCallbacks::IsInPIE()
{
	return (GEditor->PlayWorld != NULL) && (!GEditor->bIsSimulatingInEditor);
}


bool FPlayWorldCommandCallbacks::IsInSIE_AndRunning()
{
	return IsInSIE() && ((GEditor->PlayWorld == NULL) || !(GEditor->PlayWorld->bDebugPauseExecution));
}


bool FPlayWorldCommandCallbacks::IsInPIE_AndRunning()
{
	return IsInPIE() && ((GEditor->PlayWorld == NULL) || !(GEditor->PlayWorld->bDebugPauseExecution));
}

bool FPlayWorldCommandCallbacks::HasPlayWorld()
{
	return GEditor->PlayWorld != NULL;
}


bool FPlayWorldCommandCallbacks::HasPlayWorldAndPaused()
{
	return HasPlayWorld() && GUnrealEd->PlayWorld->bDebugPauseExecution;
}


bool FPlayWorldCommandCallbacks::HasPlayWorldAndRunning()
{
	return HasPlayWorld() && !GUnrealEd->PlayWorld->bDebugPauseExecution;
}


//////////////////////////////////////////////////////////////////////////
// FInternalPlayWorldCommandCallbacks

FText FInternalPlayWorldCommandCallbacks::GetPossessEjectLabel()
{
	if (IsInPIE())
	{
		return LOCTEXT("EjectLabel", "Eject");
	}
	else if (IsInSIE())
	{
		return LOCTEXT("PossessLabel", "Possess");
	}
	else
	{
		return LOCTEXT("ToggleBetweenPieAndSIELabel", "Toggle Between PIE and SIE");
	}
}


FText FInternalPlayWorldCommandCallbacks::GetPossessEjectTooltip()
{
	if (IsInPIE())
	{
		return LOCTEXT("EjectToolTip", "Detaches from the player controller, allowing regular editor controls");
	}
	else if (IsInSIE())
	{
		return LOCTEXT("PossessToolTip", "Attaches to the player controller, allowing normal gameplay controls");
	}
	else
	{
		return LOCTEXT("ToggleBetweenPieAndSIEToolTip", "Toggles the current play session between play in editor and simulate in editor");
	}
}


FSlateIcon FInternalPlayWorldCommandCallbacks::GetPossessEjectImage()
{
	if (IsInSIE())
	{
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), "PlayWorld.PlayPossess");
	}
	else
	{
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), "PlayWorld.EjectFromPlayer");
	}
}

bool FInternalPlayWorldCommandCallbacks::CanLateJoin()
{
	if (FInternalPlayWorldCommandCallbacks::IsStoppedAtBreakpoint_InEngineMode())
	{
		return false;
	}

	return HasPlayWorld();
}

bool FInternalPlayWorldCommandCallbacks::CanShowLateJoinButton()
{
	return GetDefault<UEditorExperimentalSettings>()->bAllowLateJoinInPIE && HasPlayWorld();
}


void FInternalPlayWorldCommandCallbacks::Simulate_Clicked()
{
	if (IsStoppedAtBreakpoint_InEngineMode() || DebuggerCommands::IsVREditorActive())
	{
		return;
	}

	// Is a simulation session already running?  If so, do nothing
	if (HasPlayWorld() && GUnrealEd->bIsSimulatingInEditor)
	{
		return;
	}

	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));

	TSharedPtr<IAssetViewport> ActiveLevelViewport = LevelEditorModule.GetFirstActiveViewport();
	if (ActiveLevelViewport.IsValid())
	{
		// Start a new simulation session!
		if (!HasPlayWorld())
		{
			if (FEngineAnalytics::IsAvailable())
			{
				FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.SimulateInEditor"));
			}
			SetLastExecutedPlayMode(PlayMode_Simulate);
			FRequestPlaySessionParams SessionParams;
			SessionParams.WorldType = EPlaySessionWorldType::SimulateInEditor;
			SessionParams.DestinationSlateViewport = ActiveLevelViewport;

			GUnrealEd->RequestPlaySession(SessionParams);
		}
		else if (ActiveLevelViewport->HasPlayInEditorViewport())
		{
			GUnrealEd->RequestToggleBetweenPIEandSIE();
		}
	}
}


bool FInternalPlayWorldCommandCallbacks::Simulate_CanExecute()
{
	if (IsStoppedAtBreakpoint_InEngineMode() || DebuggerCommands::IsVREditorActive())
	{
		return false;
	}

	// Can't simulate while already simulating; PIE is fine as we toggle to simulate
	return !(HasPlayWorld() && GUnrealEd->bIsSimulatingInEditor) && !GEditor->IsLightingBuildCurrentlyRunning();
}


bool FInternalPlayWorldCommandCallbacks::Simulate_IsChecked()
{
	return HasPlayWorld() && GUnrealEd->bIsSimulatingInEditor;
}


const TSharedRef < FUICommandInfo > GetLastPlaySessionCommand()
{
	const ULevelEditorPlaySettings* PlaySettings = GetDefault<ULevelEditorPlaySettings>();

	const FPlayWorldCommands& Commands = FPlayWorldCommands::Get();
	TSharedRef < FUICommandInfo > Command = Commands.PlayInViewport.ToSharedRef();

	switch (PlaySettings->LastExecutedPlayModeType)
	{
	case PlayMode_Simulate:
	case PlayMode_InViewPort:
		Command = Commands.PlayInViewport.ToSharedRef();
		break;

	case PlayMode_InEditorFloating:
		Command = Commands.PlayInEditorFloating.ToSharedRef();
		break;

	case PlayMode_InMobilePreview:
		Command = Commands.PlayInMobilePreview.ToSharedRef();
		break;

	case PlayMode_InNewProcess:
		Command = Commands.PlayInNewProcess.ToSharedRef();
		break;

	case PlayMode_InVR:
		Command = Commands.PlayInVR.ToSharedRef();
		break;
	}

	return Command;
}



/** Report PIE usage to engine analytics */
void RecordLastExecutedPlayMode()
{
	if (FEngineAnalytics::IsAvailable())
	{
		const ULevelEditorPlaySettings* PlaySettings = GetDefault<ULevelEditorPlaySettings>();

		// play location
		FString PlayLocationString;

		switch (PlaySettings->LastExecutedPlayModeLocation)
		{
		case PlayLocation_CurrentCameraLocation:
			PlayLocationString = TEXT("CurrentCameraLocation");
			break;

		case PlayLocation_DefaultPlayerStart:
			PlayLocationString = TEXT("DefaultPlayerStart");
			break;

		default:
			PlayLocationString = TEXT("<UNKNOWN>");
		}

		// play mode
		FString PlayModeString;

		switch (PlaySettings->LastExecutedPlayModeType)
		{
		case PlayMode_InViewPort:
			PlayModeString = TEXT("InViewPort");
			break;

		case PlayMode_InEditorFloating:
			PlayModeString = TEXT("InEditorFloating");
			break;

		case PlayMode_InMobilePreview:
			PlayModeString = TEXT("InMobilePreview");
			break;

		case PlayMode_InNewProcess:
			PlayModeString = TEXT("InNewProcess");
			break;

		case PlayMode_InVR:
			PlayModeString = TEXT("InVR");
			break;

		case PlayMode_Simulate:
			PlayModeString = TEXT("Simulate");
			break;

		default:
			PlayModeString = TEXT("<UNKNOWN>");
		}

		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.PIE"), TEXT("PlayLocation"), PlayLocationString);
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.PIE"), TEXT("PlayMode"), PlayModeString);
	}
}


void SetLastExecutedLaunchMode(ELaunchModeType LaunchMode)
{
	ULevelEditorPlaySettings* PlaySettings = GetMutableDefault<ULevelEditorPlaySettings>();
	PlaySettings->LastExecutedLaunchModeType = LaunchMode;

	PlaySettings->PostEditChange();

	PlaySettings->SaveConfig();
}


void FInternalPlayWorldCommandCallbacks::RepeatLastPlay_Clicked()
{
	if (!DebuggerCommands::AllowPlayWorldFeature())
	{
		return;
	}

	ULevelEditorPlaySettings* PlaySettings = GetMutableDefault<ULevelEditorPlaySettings>();
	PlaySettings->PostEditChange();

	// hand over to Turnkey module
	if (PlaySettings->LastExecutedPlayModeType == EPlayModeType::PlayMode_QuickLaunch)
	{
#if UE_WITH_TURNKEY_SUPPORT
		ITurnkeySupportModule::Get().RepeatQuickLaunch(PlaySettings->LastExecutedLaunchName);
#endif
	}
	else
	{
		// Grab the play command and execute it
		TSharedRef<FUICommandInfo> LastCommand = GetLastPlaySessionCommand();
		UE_LOGF(LogDebuggerCommands, Log, "Repeating last play command: %ls", *LastCommand->GetLabel().ToString());

		FPlayWorldCommands::GlobalPlayWorldActions->ExecuteAction(LastCommand);
	}
}


bool FInternalPlayWorldCommandCallbacks::RepeatLastPlay_CanExecute()
{
	if (!DebuggerCommands::AllowPlayWorldFeature())
	{
		return false;
	}

	const ULevelEditorPlaySettings* PlaySettings = GetDefault<ULevelEditorPlaySettings>();
	if (PlaySettings->LastExecutedPlayModeType == EPlayModeType::PlayMode_QuickLaunch)
	{
		// return true, and let Turnkey module determine if it's still usable, and show an error if not
		return true;
	}

	return FPlayWorldCommands::GlobalPlayWorldActions->CanExecuteAction(GetLastPlaySessionCommand());
}


FText FInternalPlayWorldCommandCallbacks::GetRepeatLastPlayToolTip()
{
	const ULevelEditorPlaySettings* PlaySettings = GetDefault<ULevelEditorPlaySettings>();
	if (PlaySettings->LastExecutedPlayModeType == EPlayModeType::PlayMode_QuickLaunch)
	{
		// @todo make a proper tooltip!
		return FText::FromString(PlaySettings->LastExecutedLaunchName);
	}

	return GetLastPlaySessionCommand()->GetDescription();
}


FSlateIcon FInternalPlayWorldCommandCallbacks::GetRepeatLastPlayIcon()
{
	// get platform icon for Quick Launch mode
	const ULevelEditorPlaySettings* PlaySettings = GetDefault<ULevelEditorPlaySettings>();
	if (PlaySettings->LastExecutedPlayModeType == EPlayModeType::PlayMode_QuickLaunch)
	{
		FTargetDeviceId DeviceId;
		FTargetDeviceId::Parse(PlaySettings->LastExecutedLaunchName, DeviceId);

		// get platform name from DeviceId
		
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), FDataDrivenPlatformInfoRegistry::GetPlatformInfo(DeviceId.GetPlatformName()).GetIconStyleName(EPlatformIconSize::Normal));
	}

	return GetLastPlaySessionCommand()->GetIcon();
}


void FInternalPlayWorldCommandCallbacks::PlayInViewport_Clicked()
{
	if (IsStoppedAtBreakpoint_InEngineMode() || DebuggerCommands::IsVREditorActive())
	{
		return;
	}

	TRACE_BOOKMARK(TEXT("PIE"));

	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));

	/** Set PlayInViewPort as the last executed play command */
	const FPlayWorldCommands& Commands = FPlayWorldCommands::Get();

	SetLastExecutedPlayMode(PlayMode_InViewPort);

	RecordLastExecutedPlayMode();

	TSharedPtr<IAssetViewport> ActiveLevelViewport = LevelEditorModule.GetFirstActiveViewport();

	const bool bAtPlayerStart = (GetPlayModeLocation() == PlayLocation_DefaultPlayerStart);

	FRequestPlaySessionParams SessionParams;

	// Make sure we can find a path to the view port.  This will fail in cases where the view port widget
	// is in a backgrounded tab, etc.  We can't currently support starting PIE in a backgrounded tab
	// due to how PIE manages focus and requires event forwarding from the application.
	if (ActiveLevelViewport.IsValid() && FSlateApplication::Get().FindWidgetWindow(ActiveLevelViewport->AsWidget()).IsValid())
	{
		SessionParams.DestinationSlateViewport = ActiveLevelViewport;
		if (!bAtPlayerStart)
		{
			// Start the player where the camera is if not forcing from player start
			SessionParams.StartLocation = ActiveLevelViewport->GetAssetViewportClient().GetViewLocation();
			SessionParams.StartRotation = ActiveLevelViewport->GetAssetViewportClient().GetViewRotation();
		}
	}

	if (!HasPlayWorld())
	{
		// If there is an active level view port, play the game in it, otherwise make a new window.
		GUnrealEd->RequestPlaySession(SessionParams);
	}
	else
	{
		// There is already a play world active which means simulate in editor is happening
		// Toggle to PIE
		check(!GIsPlayInEditorWorld);
		GUnrealEd->RequestToggleBetweenPIEandSIE();
	}
}

bool FInternalPlayWorldCommandCallbacks::PlayInViewport_CanExecute()
{
	if (IsStoppedAtBreakpoint_InEngineMode() || DebuggerCommands::IsVREditorActive())
	{
		return false;
	}

	// Disallow PIE when compiling in the editor
	if (GEditor->bIsCompiling)
	{
		return false;
	}

	// Allow PIE if we don't already have a play session or the play session is simulate in editor (which we can toggle to PIE)
	return (!GEditor->IsPlaySessionInProgress() && !HasPlayWorld() && !GEditor->IsLightingBuildCurrentlyRunning()) || GUnrealEd->IsSimulateInEditorInProgress();
}


void FInternalPlayWorldCommandCallbacks::PlayInEditorFloating_Clicked()
{
	if (IsStoppedAtBreakpoint_InEngineMode() || DebuggerCommands::IsVREditorActive())
	{
		return;
	}

	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));

	SetLastExecutedPlayMode(PlayMode_InEditorFloating);

	FRequestPlaySessionParams SessionParams;

	// Is a PIE session already running?  If not, then we'll kick off a new one
	if (!HasPlayWorld())
	{
		RecordLastExecutedPlayMode();

		const bool bAtPlayerStart = (GetPlayModeLocation() == PlayLocation_DefaultPlayerStart);
		if (!bAtPlayerStart)
		{
			TSharedPtr<IAssetViewport> ActiveLevelViewport = LevelEditorModule.GetFirstActiveViewport();

			// Make sure we can find a path to the view port.  This will fail in cases where the view port widget
			// is in a backgrounded tab, etc.  We can't currently support starting PIE in a backgrounded tab
			// due to how PIE manages focus and requires event forwarding from the application.
			if (ActiveLevelViewport.IsValid() &&
				FSlateApplication::Get().FindWidgetWindow(ActiveLevelViewport->AsWidget()).IsValid())
			{
				// Start the player where the camera is if not forcing from player start
				SessionParams.StartLocation = ActiveLevelViewport->GetAssetViewportClient().GetViewLocation();
				SessionParams.StartRotation = ActiveLevelViewport->GetAssetViewportClient().GetViewRotation();
			}
		}

		// Spawn a new window to play in.
		GUnrealEd->RequestPlaySession(SessionParams);
	}
	else
	{
		// Terminate existing session.  This is deferred because we could be processing this from the play world and we should not clear the play world while in it.
		GUnrealEd->RequestEndPlayMap();
	}
}


bool FInternalPlayWorldCommandCallbacks::PlayInEditorFloating_CanExecute()
{
	if (IsStoppedAtBreakpoint_InEngineMode() || DebuggerCommands::IsVREditorActive())
	{
		return false;
	}

	return (!HasPlayWorld() || !GUnrealEd->bIsSimulatingInEditor) && !GEditor->IsLightingBuildCurrentlyRunning();
}

void FInternalPlayWorldCommandCallbacks::PlayInVR_Clicked()
{
	if (IsStoppedAtBreakpoint_InEngineMode() || DebuggerCommands::IsVREditorActive())
	{
		return;
	}

	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));

	SetLastExecutedPlayMode(PlayMode_InVR);
	FRequestPlaySessionParams SessionParams;

	// Is a PIE session already running?  If not, then we'll kick off a new one
	if (!HasPlayWorld())
	{
		RecordLastExecutedPlayMode();

		const bool bAtPlayerStart = (GetPlayModeLocation() == PlayLocation_DefaultPlayerStart);
		if (!bAtPlayerStart)
		{
			TSharedPtr<IAssetViewport> ActiveLevelViewport = LevelEditorModule.GetFirstActiveViewport();

			// Make sure we can find a path to the view port.  This will fail in cases where the view port widget
			// is in a backgrounded tab, etc.  We can't currently support starting PIE in a backgrounded tab
			// due to how PIE manages focus and requires event forwarding from the application.
			if (ActiveLevelViewport.IsValid() &&
				FSlateApplication::Get().FindWidgetWindow(ActiveLevelViewport->AsWidget()).IsValid())
			{
				// Start the player where the camera is if not forcing from player start
				SessionParams.StartLocation = ActiveLevelViewport->GetAssetViewportClient().GetViewLocation();
				SessionParams.StartRotation = ActiveLevelViewport->GetAssetViewportClient().GetViewRotation();
			}
		}

		SessionParams.SessionPreviewTypeOverride = EPlaySessionPreviewType::VRPreview;

		// Spawn a new window to play in.
		GUnrealEd->RequestPlaySession(SessionParams);
	}
}


bool FInternalPlayWorldCommandCallbacks::PlayInVR_CanExecute()
{
	if (IsStoppedAtBreakpoint_InEngineMode() || DebuggerCommands::IsVREditorActive())
	{
		return false;
	}

	return (!HasPlayWorld() || !GUnrealEd->bIsSimulatingInEditor) && !GEditor->IsLightingBuildCurrentlyRunning() &&
		GEngine && GEngine->XRSystem.IsValid() && GEngine->XRSystem->GetHMDDevice() && GEngine->XRSystem->GetHMDDevice()->IsHMDConnected();
}

void SetLastExecutedPIEPreviewDevice(FString PIEPreviewDevice)
{
	ULevelEditorPlaySettings* PlaySettings = GetMutableDefault<ULevelEditorPlaySettings>();
	PlaySettings->LastExecutedPIEPreviewDevice = PIEPreviewDevice;
	FPropertyChangedEvent PropChangeEvent(ULevelEditorPlaySettings::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ULevelEditorPlaySettings, LastExecutedPIEPreviewDevice)));
	PlaySettings->PostEditChangeProperty(PropChangeEvent);
	PlaySettings->SaveConfig();
}

void FInternalPlayWorldCommandCallbacks::PlayInNewProcess_Clicked(EPlayModeType PlayModeType)
{
	check(PlayModeType == PlayMode_InNewProcess 
		|| PlayModeType == PlayMode_InMobilePreview);

	if (IsStoppedAtBreakpoint_InEngineMode() || DebuggerCommands::IsVREditorActive())
	{
		return;
	}

	SetLastExecutedPlayMode(PlayModeType);
	FRequestPlaySessionParams SessionParams;

	if (!HasPlayWorld())
	{
		RecordLastExecutedPlayMode();

		const bool bAtPlayerStart = (GetPlayModeLocation() == PlayLocation_DefaultPlayerStart);
		if (!bAtPlayerStart)
		{
			FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
			TSharedPtr<IAssetViewport> ActiveLevelViewport = LevelEditorModule.GetFirstActiveViewport();

			if (ActiveLevelViewport.IsValid() && FSlateApplication::Get().FindWidgetWindow(ActiveLevelViewport->AsWidget()).IsValid())
			{
				SessionParams.StartLocation = ActiveLevelViewport->GetAssetViewportClient().GetViewLocation();
				SessionParams.StartRotation = ActiveLevelViewport->GetAssetViewportClient().GetViewRotation();
			}
		}

		if (PlayModeType == PlayMode_InMobilePreview)
		{
			SessionParams.SessionPreviewTypeOverride = EPlaySessionPreviewType::MobilePreview;
		}

		SessionParams.SessionDestination = EPlaySessionDestinationType::NewProcess;

		// Spawn a new window to play in.
		GUnrealEd->RequestPlaySession(SessionParams);
	}
	else
	{
		GUnrealEd->EndPlayMap();
	}
}


bool FInternalPlayWorldCommandCallbacks::PlayInNewProcess_CanExecute()
{
	if (IsStoppedAtBreakpoint_InEngineMode() || DebuggerCommands::IsVREditorActive())
	{
		return false;
	}
	else if (FNullPlatformApplicationMisc::IsUsingNullApplication())
	{
		return false;
	}

	return true;
}


bool FInternalPlayWorldCommandCallbacks::PlayInModeIsChecked(EPlayModeType PlayMode)
{
	return (PlayMode == GetDefault<ULevelEditorPlaySettings>()->LastExecutedPlayModeType);
}


bool FInternalPlayWorldCommandCallbacks::PlayInLocation_CanExecute(EPlayModeLocations Location)
{
	if (IsStoppedAtBreakpoint_InEngineMode() || DebuggerCommands::IsVREditorActive())
	{
		return false;
	}

	switch (Location)
	{
	case PlayLocation_CurrentCameraLocation:
		return true;

	case PlayLocation_DefaultPlayerStart:
		return (GEditor->CheckForPlayerStart() != nullptr);

	default:
		return false;
	}
}


void FInternalPlayWorldCommandCallbacks::PlayInLocation_Clicked(EPlayModeLocations Location)
{
	ULevelEditorPlaySettings* PlaySettings = GetMutableDefault<ULevelEditorPlaySettings>();
	PlaySettings->LastExecutedPlayModeLocation = Location;
	PlaySettings->PostEditChange();
	PlaySettings->SaveConfig();
}


bool FInternalPlayWorldCommandCallbacks::PlayInLocation_IsChecked(EPlayModeLocations Location)
{
	switch (Location)
	{
	case PlayLocation_CurrentCameraLocation:
		return ((GetDefault<ULevelEditorPlaySettings>()->LastExecutedPlayModeLocation == PlayLocation_CurrentCameraLocation) || (GEditor->CheckForPlayerStart() == nullptr));

	case PlayLocation_DefaultPlayerStart:
		return ((GetDefault<ULevelEditorPlaySettings>()->LastExecutedPlayModeLocation == PlayLocation_DefaultPlayerStart) && (GEditor->CheckForPlayerStart() != nullptr));
	}

	return false;
}


void FInternalPlayWorldCommandCallbacks::PlayInSettings_Clicked()
{
	FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Editor", "LevelEditor", "PlayIn");
}




void FInternalPlayWorldCommandCallbacks::GetMouseControlExecute()
{
	if (IsInPIE()) {
		FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::SetDirectly);
		FSlateApplication::Get().ResetToDefaultInputSettings();

		TWeakPtr<SGlobalPlayWorldActions> ActiveGlobalPlayWorldWidget = FPlayWorldCommands::GetActiveGlobalPlayWorldActionsWidget();
		if (ActiveGlobalPlayWorldWidget.IsValid())
		{
			uint32 UserIndex = 0;
			FSlateApplication::Get().SetUserFocus(UserIndex, ActiveGlobalPlayWorldWidget.Pin());
		}
	}
}

void FInternalPlayWorldCommandCallbacks::SingleFrameAdvance_Clicked()
{
	if (IsStoppedAtBreakpoint_InEngineMode())
	{
		return;
	}

	// We want to function just like Single stepping where we will stop at a breakpoint if one is encountered but we also want to stop after 1 tick if a breakpoint is not encountered.
	FKismetDebugUtilities::RequestSingleStepIn();
	if (HasPlayWorld())
	{
		GUnrealEd->PlayWorld->bDebugFrameStepExecution = true;
		LeaveDebuggingMode();
		GUnrealEd->PlaySessionSingleStepped();
	}
}


void FInternalPlayWorldCommandCallbacks::StopPlaySession_Clicked()
{
	if (IsStoppedAtBreakpoint_InEngineMode())
	{
		return;
	}

	if (HasPlayWorld())
	{
		GEditor->RequestEndPlayMap();
		LeaveDebuggingMode();
	}
}

void FInternalPlayWorldCommandCallbacks::LateJoinSession_Clicked()
{
	if (HasPlayWorld())
	{
		GEditor->RequestLateJoin();
	}
}

void FInternalPlayWorldCommandCallbacks::ShowCurrentStatement_Clicked()
{
	UEdGraphNode* CurrentInstruction = FKismetDebugUtilities::GetCurrentInstruction();
	if (CurrentInstruction != NULL)
	{
		FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(CurrentInstruction);
	}
}

void FInternalPlayWorldCommandCallbacks::ContinueExecution_Clicked()
{
	if (FKismetDebugUtilities::GetCurrentDebuggingWorld())
	{
		LeaveDebuggingMode();
		GUnrealEd->PlaySessionSingleStepped();
	}
	else
	{
		const bool bIsResumingPlay = !FKismetDebugUtilities::IsSingleStepping();
		FSlateApplication::Get().LeaveDebuggingMode(!bIsResumingPlay);
	}
}

void FInternalPlayWorldCommandCallbacks::AbortExecution_Clicked()
{
	FKismetDebugUtilities::RequestAbortingExecution();
	ContinueExecution_Clicked();
}

void FInternalPlayWorldCommandCallbacks::StepInto_Clicked()
{
	FKismetDebugUtilities::RequestSingleStepIn();
	ContinueExecution_Clicked();
}

void FInternalPlayWorldCommandCallbacks::StepOver_Clicked()
{
	FKismetDebugUtilities::RequestStepOver();
	ContinueExecution_Clicked();
}

void FInternalPlayWorldCommandCallbacks::StepOut_Clicked()
{
	FKismetDebugUtilities::RequestStepOut();
	ContinueExecution_Clicked();
}

void FInternalPlayWorldCommandCallbacks::TogglePlayPause_Clicked()
{
	if (HasPlayWorld())
	{
		if (GUnrealEd->PlayWorld->IsPaused())
		{
			LeaveDebuggingMode();
			GUnrealEd->PlaySessionResumed();
			uint32 UserIndex = 0;
			FSlateApplication::Get().SetUserFocusToGameViewport(UserIndex);
		}
		else
		{
			GUnrealEd->SetPIEWorldsPaused(true);
			GUnrealEd->PlaySessionPaused();
			if (IsInPIE()) 
			{
				FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::SetDirectly);
				FSlateApplication::Get().ResetToDefaultInputSettings();

				TWeakPtr<SGlobalPlayWorldActions> ActiveGlobalPlayWorldWidget = FPlayWorldCommands::GetActiveGlobalPlayWorldActionsWidget();
				if (ActiveGlobalPlayWorldWidget.IsValid())
				{
					uint32 UserIndex = 0;
					FSlateApplication::Get().SetUserFocus(UserIndex, ActiveGlobalPlayWorldWidget.Pin());
				}
			}
		}
	}
}

bool FInternalPlayWorldCommandCallbacks::CanShowNonPlayWorldOnlyActions()
{
	return !HasPlayWorld() && DebuggerCommands::AllowPlayWorldFeature();
}

bool FInternalPlayWorldCommandCallbacks::CanShowVROnlyActions()
{
	return !HasPlayWorld() && DebuggerCommands::AllowPlayWorldFeature();
}

int32 FInternalPlayWorldCommandCallbacks::GetNumberOfClients()
{
	const ULevelEditorPlaySettings* PlayInSettings = GetDefault<ULevelEditorPlaySettings>();
	int32 PlayNumberOfClients(0);
	PlayInSettings->GetPlayNumberOfClients(PlayNumberOfClients);	// Ignore 'state' of option (handled externally)
	return PlayNumberOfClients;
}


void FInternalPlayWorldCommandCallbacks::SetNumberOfClients(int32 NumClients, ETextCommit::Type CommitInfo)
{
	ULevelEditorPlaySettings* PlayInSettings = GetMutableDefault<ULevelEditorPlaySettings>();
	PlayInSettings->SetPlayNumberOfClients(NumClients);

	PlayInSettings->PostEditChange();
	// ETextCommit::Default is used when the value is changed during an interactive event like mouse drag or mouse wheel, vs. when the value is actually committed
	if(CommitInfo != ETextCommit::Default)
	{
		PlayInSettings->SaveConfig();
	}
}


int32 FInternalPlayWorldCommandCallbacks::GetNetPlayMode()
{
	const ULevelEditorPlaySettings* PlayInSettings = GetDefault<ULevelEditorPlaySettings>();
	EPlayNetMode NetMode;
	PlayInSettings->GetPlayNetMode(NetMode);

	return (int32)NetMode;
}

void FInternalPlayWorldCommandCallbacks::SetNetPlayMode(int32 Value)
{
	ULevelEditorPlaySettings* PlayInSettings = GetMutableDefault<ULevelEditorPlaySettings>();
	PlayInSettings->SetPlayNetMode((EPlayNetMode)Value);

	PlayInSettings->PostEditChange();
	PlayInSettings->SaveConfig();
}


bool FInternalPlayWorldCommandCallbacks::IsStoppedAtBreakpoint()
{
	return GIntraFrameDebuggingGameThread;
}

bool FInternalPlayWorldCommandCallbacks::IsStoppedAtBreakpoint_InEngineMode()
{
	return GIntraFrameDebuggingGameThread && FKismetDebugUtilities::GetCurrentDebuggingWorld() == nullptr;
}

void FInternalPlayWorldCommandCallbacks::PossessEjectPlayer_Clicked()
{
	GEditor->RequestToggleBetweenPIEandSIE();
}


bool FInternalPlayWorldCommandCallbacks::CanPossessEjectPlayer()
{
	if ((IsInSIE() || IsInPIE()) && !IsStoppedAtBreakpoint())
	{
		if (auto It = GUnrealEd->SlatePlayInEditorMap.CreateIterator(); It)
		{
			return It.Value().DestinationSlateViewport.IsValid();
		}
	}
	return false;
}

void FInternalPlayWorldCommandCallbacks::AddMessageLog(const FText& Text, const FText& Detail, const FString& TutorialLink, const FString& DocumentationLink)
{
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error);
	Message->AddToken(FTextToken::Create(Text));
	Message->AddToken(FTextToken::Create(Detail));
	if (!TutorialLink.IsEmpty())
	{
		Message->AddToken(FTutorialToken::Create(TutorialLink));
	}
	if (!DocumentationLink.IsEmpty())
	{
		Message->AddToken(FDocumentationToken::Create(DocumentationLink));
	}
	FMessageLog MessageLog("PackagingResults");
	MessageLog.AddMessage(Message);
	MessageLog.Open();
}


EPlayModeLocations FInternalPlayWorldCommandCallbacks::GetPlayModeLocation()
{
	// We can't use PlayLocation_DefaultPlayerStart without a player start position
	return GEditor->CheckForPlayerStart()
		? static_cast<EPlayModeLocations>(GetDefault<ULevelEditorPlaySettings>()->LastExecutedPlayModeLocation)
		: PlayLocation_CurrentCameraLocation;
}

#undef LOCTEXT_NAMESPACE
