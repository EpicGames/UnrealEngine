// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet2/EnumEditorUtils.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealType.h"
#include "UObject/EnumProperty.h"
#include "EdGraph/EdGraphNode.h"
#include "Engine/Blueprint.h"
#include "EdGraph/EdGraph.h"
#include "EdGraphSchema_K2.h"
#include "K2Node.h"
#include "K2Node_Variable.h"
#include "NodeDependingOnEnumInterface.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Internationalization/TextNamespaceUtil.h"
#include "Internationalization/TextPackageNamespaceUtil.h"
#include "ScopedTransaction.h"
#include "UObject/PropertyIterator.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Kismet2/StructureEditorUtils.h"

#define LOCTEXT_NAMESPACE "Enum"

struct FEnumEditorUtilsHelper
{
	 static const TCHAR* DisplayName() { return TEXT("DisplayName"); }
	 static const TCHAR* InvalidName() { return TEXT("(INVALID)"); }
};

//////////////////////////////////////////////////////////////////////////
// FEnumEditorManager
FEnumEditorUtils::FEnumEditorManager& FEnumEditorUtils::FEnumEditorManager::Get()
{
	static TSharedRef< FEnumEditorManager > EnumEditorManager( new FEnumEditorManager() );
	return *EnumEditorManager;
}

//////////////////////////////////////////////////////////////////////////
// User defined enumerations

UEnum* FEnumEditorUtils::CreateUserDefinedEnum(UObject* InParent, FName EnumName, EObjectFlags Flags)
{
	ensure(0 != (RF_Public & Flags));

	UEnum* Enum = NewObject<UUserDefinedEnum>(InParent, EnumName, Flags);

	if (NULL != Enum)
	{
		TArray<TPair<FName, int64>> EmptyNames;
		Enum->SetEnums(EmptyNames, UEnum::ECppForm::Namespaced, UEnum::EUnderlyingType::uint8, EEnumFlags::None, UEnum::EAddMaxKeyIfMissing::Yes);
		Enum->SetMetaData(TEXT("BlueprintType"), TEXT("true"));
	}

	return Enum;
}

bool FEnumEditorUtils::IsNameAvailebleForUserDefinedEnum(FName Name)
{
	return true;
}

void FEnumEditorUtils::UpdateAfterPathChanged(UEnum* Enum)
{
	check(NULL != Enum);

	TArray<TPair<FName, int64>> NewEnumeratorsNames;
	const int32 EnumeratorsToCopy = Enum->NumEnums() - 1; // skip _MAX
	for (int32 Index = 0; Index < EnumeratorsToCopy; Index++)
	{
		const FString ShortName = Enum->GetNameStringByIndex(Index);
		const FString NewFullName = Enum->GenerateFullEnumName(*ShortName);
		NewEnumeratorsNames.Emplace(*NewFullName, Index);
	}

	Enum->SetEnums(NewEnumeratorsNames, UEnum::ECppForm::Namespaced, UEnum::EUnderlyingType::uint8, EEnumFlags::None, UEnum::EAddMaxKeyIfMissing::Yes);
}

//////////////////////////////////////////////////////////////////////////
// Enumerators

void FEnumEditorUtils::CopyEnumeratorsWithoutMax(const UEnum* Enum, TArray<TPair<FName, int64>>& OutEnumNames)
{
	if (Enum == nullptr)
	{
		return;
	}

	const int32 EnumeratorsToCopy = Enum->NumEnums() - 1;
	for	(int32 Index = 0; Index < EnumeratorsToCopy; Index++)
	{
		OutEnumNames.Emplace(Enum->GetNameByIndex(Index), Enum->GetValueByIndex(Index));
	}
}

/** adds new enumerator (with default unique name) for user defined enum */
void FEnumEditorUtils::AddNewEnumeratorForUserDefinedEnum(UUserDefinedEnum* Enum)
{
	if (!Enum)
	{
		return;
	}

	const FScopedTransaction Transaction(NSLOCTEXT("EnumEditor", "AddNewEnumerator", "Add Enumerator"));

	PrepareForChange(Enum);

	TArray<TPair<FName, int64>> OldNames, Names;
	CopyEnumeratorsWithoutMax(Enum, OldNames);
	Names = OldNames;

	FString EnumNameString = Enum->GenerateNewEnumeratorName();
	const FString FullNameStr = Enum->GenerateFullEnumName(*EnumNameString);
	Names.Emplace(*FullNameStr, Enum->GetMaxEnumValue());

	// Clean up enum values.
	for (int32 i = 0; i < Names.Num(); ++i)
	{
		Names[i].Value = i;
	}
	const UEnum::ECppForm EnumType = Enum->GetCppForm();
	const UEnum::EUnderlyingType UnderlyingType = Enum->GetUnderlyingType();
	Enum->SetEnums(Names, EnumType, UnderlyingType, EEnumFlags::None, UEnum::EAddMaxKeyIfMissing::Yes);
	EnsureAllDisplayNamesExist(Enum);

	BroadcastChanges(Enum, OldNames);

	Enum->MarkPackageDirty();
}

void FEnumEditorUtils::RemoveEnumeratorFromUserDefinedEnum(UUserDefinedEnum* Enum, int32 EnumeratorIndex)
{
	if (!(Enum && (Enum->GetNameByIndex(EnumeratorIndex) != NAME_None)))
	{
		return;
	}

	const FScopedTransaction Transaction(NSLOCTEXT("EnumEditor", "RemoveEnumerator", "Remove Enumerator"));

	PrepareForChange(Enum);

	TArray<TPair<FName, int64>> OldNames, Names;
	CopyEnumeratorsWithoutMax(Enum, OldNames);
	Names = OldNames;

	Names.RemoveAt(EnumeratorIndex);

	// Clean up enum values.
	for (int32 i = 0; i < Names.Num(); ++i)
	{
		Names[i].Value = i;
	}
	const UEnum::ECppForm EnumType = Enum->GetCppForm();
	const UEnum::EUnderlyingType UnderlyingType = Enum->GetUnderlyingType();
	Enum->SetEnums(Names, EnumType, UnderlyingType, EEnumFlags::None, UEnum::EAddMaxKeyIfMissing::Yes);
	EnsureAllDisplayNamesExist(Enum);
	BroadcastChanges(Enum, OldNames);

	Enum->MarkPackageDirty();
}

bool FEnumEditorUtils::IsEnumeratorBitflagsType(UUserDefinedEnum* Enum)
{
	return Enum && Enum->HasMetaData(*FBlueprintMetadata::MD_Bitflags.ToString());
}

void FEnumEditorUtils::SetEnumeratorBitflagsTypeState(UUserDefinedEnum* Enum, bool bBitflagsType)
{
	if (Enum)
	{
		const FScopedTransaction Transaction(NSLOCTEXT("EnumEditor", "SetEnumeratorBitflagsTypeState", "Set Bitflag Type State"));

		PrepareForChange(Enum);

		if (bBitflagsType)
		{
			Enum->SetMetaData(*FBlueprintMetadata::MD_Bitflags.ToString(), TEXT(""));
		}
		else
		{
			Enum->RemoveMetaData(*FBlueprintMetadata::MD_Bitflags.ToString());
		}

		TArray<TPair<FName, int64>> Names;
		CopyEnumeratorsWithoutMax(Enum, Names);
		BroadcastChanges(Enum, Names);

		Enum->MarkPackageDirty();
	}
}

void FEnumEditorUtils::MoveEnumeratorInUserDefinedEnum(UUserDefinedEnum* Enum, int32 InitialEnumeratorIndex, int32 TargetIndex)
{
	if (!Enum || Enum->GetNameByIndex(InitialEnumeratorIndex) == NAME_None || TargetIndex < 0 || TargetIndex >= Enum->NumEnums())
	{
		return;
	}

	const FScopedTransaction Transaction(NSLOCTEXT("EnumEditor", "MoveEnumeratorInUserDefinedEnum", "Reorder Enumerator"));

	PrepareForChange(Enum);

	TArray<TPair<FName, int64>> OldNames, Names;
	CopyEnumeratorsWithoutMax(Enum, OldNames);
	Names = OldNames;

	TPair<FName, int64> EnumeratorToMove = Names[InitialEnumeratorIndex];
	Names.RemoveAt(InitialEnumeratorIndex);
	Names.Insert(EnumeratorToMove, TargetIndex);

	// Clean up enum values.
	for (int32 i = 0; i < Names.Num(); ++i)
	{
		Names[i].Value = i;
	}
	const UEnum::ECppForm EnumType = Enum->GetCppForm();
	const UEnum::EUnderlyingType UnderlyingType = Enum->GetUnderlyingType();
	Enum->SetEnums(Names, EnumType, UnderlyingType, EEnumFlags::None, UEnum::EAddMaxKeyIfMissing::Yes);
	EnsureAllDisplayNamesExist(Enum);
	BroadcastChanges(Enum, OldNames);

	Enum->MarkPackageDirty();
}

bool FEnumEditorUtils::IsProperNameForUserDefinedEnumerator(const UEnum* Enum, FString NewName)
{
	if (Enum && !UEnum::IsFullEnumName(*NewName))
	{
		check(Enum->GetFName().IsValidXName());

		const FName ShortName(*NewName);
		const bool bValidName = ShortName.IsValidXName(INVALID_OBJECTNAME_CHARACTERS);

		const FString TrueNameStr = Enum->GenerateFullEnumName(*NewName);
		const FName TrueName(*TrueNameStr);
		check(!bValidName || TrueName.IsValidXName());

		const bool bNameNotUsed = (INDEX_NONE == Enum->GetIndexByName(TrueName));
		return bNameNotUsed && bValidName;
	}
	return false;
}

class FArchiveEnumeratorResolver : public FArchiveUObject
{
public:
	const UEnum* Enum;
	const TArray<TPair<FName, int64>>& OldNames;

	FArchiveEnumeratorResolver(const UEnum* InEnum, const TArray<TPair<FName, int64>>& InOldNames)
		: FArchiveUObject(), Enum(InEnum), OldNames(InOldNames)
	{
	}

	virtual bool UseToResolveEnumerators() const override
	{
		return true;
	}
};

void FEnumEditorUtils::PrepareForChange(UUserDefinedEnum* Enum)
{
	FEnumEditorManager::Get().PreChange(Enum, EEnumEditorChangeInfo::Changed);
	Enum->Modify();
}

void FEnumEditorUtils::PostEditUndo(UUserDefinedEnum* Enum)
{
	UpdateAfterPathChanged(Enum);
	BroadcastChanges(Enum, TArray<TPair<FName, int64>>(), false);
}

void FEnumEditorUtils::BroadcastChanges(const UUserDefinedEnum* Enum, const TArray<TPair<FName, int64>>& OldNames, bool bResolveData)
{
	check(nullptr != Enum);
	if (bResolveData)
	{
		FArchiveEnumeratorResolver EnumeratorResolver(Enum, OldNames);

		// Track any user defined structs that may have been modified during this enum update
		TSet<UUserDefinedStruct*> EffectedUserStructs;
		
		// Helper lambda to get the class of a UStruct off of a property
		auto GetStructClass = [&EffectedUserStructs](const FProperty* Prop) -> UClass*
		{
			if(Prop)
			{
				// Attempt to use this property's default owner class
				if(UClass* OwnerClass = Prop->GetOwnerClass())
				{
					return OwnerClass;	
				}
				// Otherwise check for UserDefinedStructs that may have this property
				else if (UUserDefinedStruct* UserStruct = Cast<UUserDefinedStruct>(Prop->GetOwnerStruct()))
				{
					// Exclude UDS compiler artifacts; they don't require regeneration.
					if (!UserStruct->PrimaryStruct.IsValid())
					{
						EffectedUserStructs.Add(UserStruct);
					}
				}
			}
			return nullptr;
		};

		TSet<UClass*> ClassesToCheck;

		for (TPropertyIterator<FByteProperty> It; It; ++It)
		{
			const FByteProperty* ByteProperty = *It;
			if (ByteProperty && (Enum == ByteProperty->GetIntPropertyEnum()))
			{
				if(UClass* StructClass = GetStructClass(ByteProperty))
				{
					ClassesToCheck.Add(StructClass);
				}
			}
		}

		for (TPropertyIterator<FEnumProperty> It; It; ++It)
		{
			const FEnumProperty* EnumProperty = *It;
			if (EnumProperty && (Enum == EnumProperty->GetEnum()))
			{
				// Check for user defined structs that may have this property
				if (UClass* StructClass = GetStructClass(EnumProperty))
				{
					ClassesToCheck.Add(StructClass);
				}
			}
		}

		for (FThreadSafeObjectIterator ObjIter; ObjIter; ++ObjIter)
		{
			for (UClass* Class : ClassesToCheck)
			{
				if (ObjIter->IsA(Class))
				{
					ObjIter->Serialize(EnumeratorResolver);
					break;
				}
			}
		}

		// User defined structs have to be notified that their structure has been changed after 
		// serialization in order for the structure editor to get updated properly
		for(UUserDefinedStruct* Struct : EffectedUserStructs)
		{
			Struct->Serialize(EnumeratorResolver);

			FStructureEditorUtils::ModifyStructData(Struct);
			FStructureEditorUtils::OnStructureChanged(Struct);
		}
	}

	struct FNodeValidatorHelper
	{
		static bool IsValid(UK2Node* Node)
		{
			return ::IsValid(Node)
				&& (NULL != Cast<UEdGraph>(Node->GetOuter()))
				&& !Node->HasAnyFlags(RF_Transient);
		}
	};

	TSet<UBlueprint*> BlueprintsToRefresh;

	{
		//CUSTOM NODES DEPENTENT ON ENUM

		for (TObjectIterator<UK2Node> It(RF_Transient); It; ++It)
		{
			UK2Node* Node = *It;
			INodeDependingOnEnumInterface* NodeDependingOnEnum = Cast<INodeDependingOnEnumInterface>(Node);
			if (FNodeValidatorHelper::IsValid(Node) && NodeDependingOnEnum && (Enum == NodeDependingOnEnum->GetEnum()))
			{
				if (Node->HasValidBlueprint())
				{
					if (NodeDependingOnEnum->ShouldBeReconstructedAfterEnumChanged())
					{
						Node->ReconstructNode();
					}
					BlueprintsToRefresh.Add(Node->GetBlueprint());
				}
			}
		}
	}

	for (TObjectIterator<UEdGraphNode> It(RF_Transient); It; ++It)
	{
		for (UEdGraphPin* Pin : It->Pins)
		{
			if (Pin && (Pin->PinType.PinSubCategory != UEdGraphSchema_K2::PSC_Bitmask) && (Enum == Pin->PinType.PinSubCategoryObject.Get()) && (EEdGraphPinDirection::EGPD_Input == Pin->Direction))
			{
				UK2Node* Node = Cast<UK2Node>(Pin->GetOuter());
				if (FNodeValidatorHelper::IsValid(Node))
				{
					if (Node->HasValidBlueprint())
					{
						UBlueprint* Blueprint = Node->GetBlueprint();
						if (INDEX_NONE == Enum->GetIndexByNameString(Pin->DefaultValue))
						{
							Pin->Modify();
							if (Blueprint->BlueprintType == BPTYPE_Interface)
							{
								Pin->DefaultValue = Enum->GetNameStringByIndex(0);
							}
							else
							{
								Pin->DefaultValue = FEnumEditorUtilsHelper::InvalidName();
							}
							Node->PinDefaultValueChanged(Pin);
							BlueprintsToRefresh.Add(Blueprint);
						}
					}
				}
			}
		}
	}

	// Modify any properties that are using the enum as a bitflags type for bitmask values inside a Blueprint class.
	for (TPropertyIterator<FIntProperty> PropertyIter; PropertyIter; ++PropertyIter)
	{
		const FIntProperty* IntProperty = *PropertyIter;
		if (IntProperty && IntProperty->HasMetaData(*FBlueprintMetadata::MD_Bitmask.ToString()))
		{
			UClass* OwnerClass = IntProperty->GetOwnerClass();
			if (OwnerClass)
			{
				// Note: We only need to consider the skeleton class here.
				UBlueprint* Blueprint = Cast<UBlueprint>(OwnerClass->ClassGeneratedBy);
				if (Blueprint && OwnerClass == Blueprint->SkeletonGeneratedClass)
				{
					const FString& BitmaskEnumName = IntProperty->GetMetaData(FBlueprintMetadata::MD_BitmaskEnum);
					if (BitmaskEnumName == Enum->GetName() && !Enum->HasMetaData(*FBlueprintMetadata::MD_Bitflags.ToString()))
					{
						FName VarName = IntProperty->GetFName();

						// This will remove the metadata key from both the skeleton & full class.
						FBlueprintEditorUtils::RemoveBlueprintVariableMetaData(Blueprint, VarName, nullptr, FBlueprintMetadata::MD_BitmaskEnum);

						// Need to reassign the property since the skeleton class will have been regenerated at this point.
						IntProperty = FindFieldChecked<FIntProperty>(Blueprint->SkeletonGeneratedClass, VarName);

						// Reconstruct any nodes that reference the variable that was just modified.
						for (TObjectIterator<UK2Node_Variable> VarNodeIt; VarNodeIt; ++VarNodeIt)
						{
							UK2Node_Variable* VarNode = *VarNodeIt;
							if (VarNode && VarNode->GetPropertyForVariable() == IntProperty)
							{
								VarNode->ReconstructNode();

								if (VarNode->HasValidBlueprint())
								{
									BlueprintsToRefresh.Add(VarNode->GetBlueprint());
								}
							}
						}

						BlueprintsToRefresh.Add(Blueprint);
					}
				}
			}
		}
	}

	for (auto It = BlueprintsToRefresh.CreateIterator(); It; ++It)
	{
		FBlueprintEditorUtils::MarkBlueprintAsModified(*It);
		(*It)->BroadcastChanged();
	}

	FEnumEditorManager::Get().PostChange(Enum, EEnumEditorChangeInfo::Changed);
}

int64 FEnumEditorUtils::ResolveEnumerator(const UEnum* Enum, FArchive& Ar, int64 EnumeratorValue)
{
	check(Ar.UseToResolveEnumerators());
	const FArchiveEnumeratorResolver* EnumeratorResolver = (FArchiveEnumeratorResolver*)(&Ar);
	if(Enum == EnumeratorResolver->Enum)
	{
		for (TPair<FName, int64> OldName : EnumeratorResolver->OldNames)
		{
			if (OldName.Value == EnumeratorValue)
			{
				const FName EnumeratorName = OldName.Key;
				const int64 NewEnumValue = Enum->GetValueByName(EnumeratorName);
				if(INDEX_NONE != NewEnumValue)
				{
					return NewEnumValue;
				}
			}
		}
		return Enum->GetMaxEnumValue();
	}
	return EnumeratorValue;
}

bool FEnumEditorUtils::SetEnumeratorDisplayName(UUserDefinedEnum* Enum, int32 EnumeratorIndex, FText NewDisplayName)
{
	if (Enum && (EnumeratorIndex >= 0) && (EnumeratorIndex < Enum->NumEnums()))
	{
		if (IsEnumeratorDisplayNameValid(Enum, EnumeratorIndex, NewDisplayName))
		{
			const FScopedTransaction Transaction(NSLOCTEXT("EnumEditor", "SetEnumeratorDisplayName", "Set Display Name"));

			PrepareForChange(Enum);
			
			const FName EnumEntryName = *Enum->GetNameStringByIndex(EnumeratorIndex);

			FText DisplayNameToSet = NewDisplayName;

#if USE_STABLE_LOCALIZATION_KEYS
			// Make sure the package namespace for this display name is up-to-date
			{
				const FString PackageNamespace = TextNamespaceUtil::EnsurePackageNamespace(Enum);
				if (!PackageNamespace.IsEmpty())
				{
					const FString DisplayNameNamespace = FTextInspector::GetNamespace(DisplayNameToSet).Get(FString());
					const FString FullNamespace = TextNamespaceUtil::BuildFullNamespace(DisplayNameNamespace, PackageNamespace);
					if (!DisplayNameNamespace.Equals(FullNamespace, ESearchCase::CaseSensitive))
					{
						// We may assign a new key when if we don't have the correct package namespace in order to avoid identity conflicts when instancing
						DisplayNameToSet = FText::ChangeKey(FullNamespace, FGuid::NewGuid().ToString(), DisplayNameToSet);
					}
				}
			}
#endif // USE_STABLE_LOCALIZATION_KEYS

			Enum->DisplayNameMap.Add(EnumEntryName, DisplayNameToSet);

			BroadcastChanges(Enum, TArray<TPair<FName, int64>>(), false);
			return true;
		}
	}

	return false;
}

bool FEnumEditorUtils::IsEnumeratorDisplayNameValid(const UUserDefinedEnum* Enum, int32 EnumeratorIndex, const FText NewDisplayName)
{
	if (!Enum || NewDisplayName.IsEmptyOrWhitespace() || NewDisplayName.ToString() == FEnumEditorUtilsHelper::InvalidName())
	{
		return false;
	}

	for	(int32 Index = 0; Index < Enum->NumEnums(); Index++)
	{
		if (Index != EnumeratorIndex && NewDisplayName.ToString() == Enum->GetDisplayNameTextByIndex(Index).ToString())
		{
			return false;
		}
	}

	return true;
}

void FEnumEditorUtils::EnsureAllDisplayNamesExist(UUserDefinedEnum* Enum)
{
	if (Enum)
	{
		const int32 EnumeratorsToEnsure = FMath::Max(Enum->NumEnums() - 1, 0);

		// Remove any stale display names
		{
			TSet<FName> KnownEnumEntryNames;
			KnownEnumEntryNames.Reserve(EnumeratorsToEnsure);

			for (int32 Index = 0; Index < EnumeratorsToEnsure; ++Index)
			{
				const FName EnumEntryName = *Enum->GetNameStringByIndex(Index);
				KnownEnumEntryNames.Add(EnumEntryName);
			}

			for (auto DisplayNameIt = Enum->DisplayNameMap.CreateIterator(); DisplayNameIt; ++DisplayNameIt)
			{
				if (!KnownEnumEntryNames.Contains(DisplayNameIt->Key))
				{
					DisplayNameIt.RemoveCurrent();
				}
			}
		}

		Enum->DisplayNameMap.Reserve(EnumeratorsToEnsure);

		// Add any missing display names
		for	(int32 Index = 0; Index < EnumeratorsToEnsure; ++Index)
		{
			const FName EnumEntryName = *Enum->GetNameStringByIndex(Index);

			if (!Enum->DisplayNameMap.Contains(EnumEntryName))
			{
				FText DisplayNameToSet;

#if USE_STABLE_LOCALIZATION_KEYS
				// Give the new text instance the full package-based namespace, and give it a brand new key; these will become its new stable identity
				{
					const FString PackageNamespace = GIsEditor ? TextNamespaceUtil::EnsurePackageNamespace(Enum) : TextNamespaceUtil::GetPackageNamespace(Enum);
					const FString TextNamespace = TextNamespaceUtil::BuildFullNamespace(FString(), PackageNamespace, true);
					const FString TextKey = FGuid::NewGuid().ToString();
					DisplayNameToSet = FText::AsLocalizable_Advanced(TextNamespace, TextKey, EnumEntryName.ToString());
				}
#else  // USE_STABLE_LOCALIZATION_KEYS
				DisplayNameToSet = FText::FromName(EnumEntryName);
#endif // USE_STABLE_LOCALIZATION_KEYS

				Enum->DisplayNameMap.Add(EnumEntryName, DisplayNameToSet);
			}
		}
	}
}

void FEnumEditorUtils::UpgradeDisplayNamesFromMetaData(UUserDefinedEnum* Enum)
{
	if (Enum)
	{
		const int32 EnumeratorsToEnsure = FMath::Max(Enum->NumEnums() - 1, 0);
		Enum->DisplayNameMap.Empty(EnumeratorsToEnsure);

		bool bDidUpgradeDisplayNames = false;
		for (int32 Index = 0; Index < EnumeratorsToEnsure; ++Index)
		{
			const FString& MetaDataEntryDisplayName = Enum->GetMetaData(FEnumEditorUtilsHelper::DisplayName(), Index);
			if (!MetaDataEntryDisplayName.IsEmpty())
			{
				bDidUpgradeDisplayNames = true;

				const FName EnumEntryName = *Enum->GetNameStringByIndex(Index);

				FText DisplayNameToSet;

#if USE_STABLE_LOCALIZATION_KEYS
				// Give the new text instance the full package-based namespace, and give it a brand new key; these will become its new stable identity
				{
					const FString PackageNamespace = GIsEditor ? TextNamespaceUtil::EnsurePackageNamespace(Enum) : TextNamespaceUtil::GetPackageNamespace(Enum);
					const FString TextNamespace = TextNamespaceUtil::BuildFullNamespace(TEXT(""), PackageNamespace, true);
					const FString TextKey = FGuid::NewGuid().ToString();
					DisplayNameToSet = FText::AsLocalizable_Advanced(TextNamespace, TextKey, MetaDataEntryDisplayName);
				}
#else  // USE_STABLE_LOCALIZATION_KEYS
				DisplayNameToSet = FText::FromName(EnumEntryName);
#endif // USE_STABLE_LOCALIZATION_KEYS

				Enum->DisplayNameMap.Add(EnumEntryName, DisplayNameToSet);
			}
		}

#if USE_STABLE_LOCALIZATION_KEYS
		if (bDidUpgradeDisplayNames)
		{
			UE_LOGF(LogClass, Warning, "Enum '%ls' was upgraded to use FText to store its display name data. Please re-save this asset to avoid issues with localization and determinstic cooking.", *Enum->GetPathName());
		}
#endif // USE_STABLE_LOCALIZATION_KEYS
	}
}

#undef LOCTEXT_NAMESPACE
