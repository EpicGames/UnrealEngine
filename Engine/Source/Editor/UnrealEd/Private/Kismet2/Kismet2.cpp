// Copyright Epic Games, Inc. All Rights Reserved.


#include "CoreMinimal.h"

#include "BlueprintCompilationManager.h"
#include "Misc/CoreMisc.h"
#include "Stats/StatsMisc.h"
#include "Stats/Stats.h"
#include "Modules/ModuleManager.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Serialization/ArchiveUObject.h"
#include "UObject/GarbageCollection.h"
#include "UObject/Class.h"
#include "UObject/Package.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/MetaData.h"
#include "Templates/SubclassOf.h"
#include "UObject/UnrealType.h"
#include "UObject/UObjectHash.h"
#include "Serialization/FindObjectReferencers.h"
#include "Serialization/ArchiveReplaceObjectRef.h"
#include "String/ParseTokens.h"
#include "Misc/PackageName.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "Components/ActorComponent.h"
#include "Components/SceneComponent.h"
#include "GameFramework/Actor.h"
#include "Engine/Blueprint.h"
#include "EdGraph/EdGraph.h"
#include "Editor/EditorEngine.h"
#include "Settings/EditorExperimentalSettings.h"
#include "Animation/AnimBlueprint.h"
#include "Engine/MemberReference.h"
#include "GeneralProjectSettings.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/SimpleConstructionScript.h"
#include "Engine/LevelScriptBlueprint.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphSchema_K2_Actions.h"
#include "K2Node_Event.h"
#include "K2Node_ActorBoundEvent.h"
#include "K2Node_CallParentFunction.h"
#include "K2Node_ComponentBoundEvent.h"
#include "K2Node_Composite.h"
#include "K2Node_FunctionEntry.h"
#include "AnimationGraph.h"
#include "AnimationGraphSchema.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/KismetReinstanceUtilities.h"
#include "Engine/SCS_Node.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetDebugUtilities.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Editor.h"
#include "Editor/Transactor.h"

#include "BlueprintEditorModule.h"
#include "FindInBlueprintManager.h"
#include "Toolkits/ToolkitManager.h"
#include "KismetCompilerModule.h"
#include "Kismet2/CompilerResultsLog.h"
#include "IAssetTools.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Layers/LayersSubsystem.h"
#include "ScopedTransaction.h"
#include "AssetToolsModule.h"
#include "EngineAnalytics.h"
#include "AnalyticsEventAttribute.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "ActorEditorUtils.h"
#include "ObjectEditorUtils.h"
#include "Dialogs/DlgPickAssetPath.h"
#include "ComponentAssetBroker.h"
#include "BlueprintEditorSettings.h"
#include "PackageTools.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Engine/InheritableComponentHandler.h"
#include "Stats/StatsHierarchical.h"
#include "Settings/EditorStyleSettings.h"
#include "ToolMenus.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Subsystems/AssetEditorSubsystem.h"

#define LOCTEXT_NAMESPACE "UnrealEd.Editor"

namespace UE::Private
{
	/** Flag to indicate whether a log should be produced when users explicitly request a compilation of a blueprint class */
	bool bLogExplicitCompilationRequests = true;
	static FAutoConsoleVariableRef CVarLogExplicitCompilationRequests(
		TEXT("BP.bLogExplicitCompilationRequests"), bLogExplicitCompilationRequests,
		TEXT("If true, CompileBlueprint will emit a log indicating the class getting recompiled."),
		ECVF_Default);

	static bool ExternalDataLayerAssetCopyPropertyFilter(FProperty& Property, UObject& Object)
	{
		static FName ExternalDataLayerAssetPropertyName(AActor::GetExternalDataLayerAssetPropertyName());
		return !Object.IsA<AActor>() || (Property.GetFName() != ExternalDataLayerAssetPropertyName);
	};
}

//////////////////////////////////////////////////////////////////////////
// FArchiveInvalidateTransientRefs

/**
 * Archive built to go through and find any references to objects in the transient package, and then NULL those references
 */
class FArchiveInvalidateTransientRefs : public FArchiveUObject
{
public:
	FArchiveInvalidateTransientRefs()
	{
		ArIsObjectReferenceCollector = true;
		this->SetIsPersistent(false);
		ArIgnoreArchetypeRef = false;
	}
protected:
	/** 
	 * UObject serialize operator implementation
	 *
	 * @param Object	reference to Object reference
	 * @return reference to instance of this class
	 */
	FArchive& operator<<( UObject*& Object )
	{
		// Check if this is a reference to an object existing in the transient package, and if so, NULL it.
		if ((Object != NULL) && (Object->GetOutermost() == GetTransientPackage()) )
		{
			check( Object->IsValidLowLevel() );
			Object = NULL;
		}

		return *this;
	}
};


//////////////////////////////////////////////////////////////////////////
// FBlueprintObjectsBeingDebuggedIterator

FBlueprintObjectsBeingDebuggedIterator::FBlueprintObjectsBeingDebuggedIterator(UBlueprint* InBlueprint)
	: Blueprint(InBlueprint)
{
}

UObject* FBlueprintObjectsBeingDebuggedIterator::operator* () const
{
	return Blueprint->GetObjectBeingDebugged();
}

UObject* FBlueprintObjectsBeingDebuggedIterator::operator-> () const
{
	return Blueprint->GetObjectBeingDebugged();
}

FBlueprintObjectsBeingDebuggedIterator& FBlueprintObjectsBeingDebuggedIterator::operator++()
{
	Blueprint = NULL;
	return *this;
}

bool FBlueprintObjectsBeingDebuggedIterator::IsValid() const
{
	return Blueprint != NULL;
}



//////////////////////////////////////////////////////////////////////////
// FObjectsBeingDebuggedIterator

FObjectsBeingDebuggedIterator::FObjectsBeingDebuggedIterator()
	: SelectedActorsIter(*GEditor->GetSelectedActors())
	, LevelScriptActorIndex(INDEX_NONE)
{
	FindNextLevelScriptActor();
}

UWorld* FObjectsBeingDebuggedIterator::GetWorld() const
{
	return (GEditor->PlayWorld != NULL) ? GEditor->PlayWorld : GWorld;
}

UObject* FObjectsBeingDebuggedIterator::operator* () const
{
	return SelectedActorsIter ? *SelectedActorsIter : (UObject*)(GetWorld()->GetLevel(LevelScriptActorIndex)->GetLevelScriptActor());
}

UObject* FObjectsBeingDebuggedIterator::operator-> () const
{
	return SelectedActorsIter ? *SelectedActorsIter : (UObject*)(GetWorld()->GetLevel(LevelScriptActorIndex)->GetLevelScriptActor());
}

FObjectsBeingDebuggedIterator& FObjectsBeingDebuggedIterator::operator++()
{
	if (SelectedActorsIter)
	{
		++SelectedActorsIter;
	}
	else
	{
		FindNextLevelScriptActor();
	}

	return *this;
}

bool FObjectsBeingDebuggedIterator::IsValid() const
{
	return SelectedActorsIter || (LevelScriptActorIndex < GetWorld()->GetNumLevels());
}

void FObjectsBeingDebuggedIterator::FindNextLevelScriptActor()
{
	while (++LevelScriptActorIndex < GetWorld()->GetNumLevels())
	{
		ULevel* Level = GetWorld()->GetLevel(LevelScriptActorIndex);
		if ((Level != NULL) && (Level->GetLevelScriptActor() != NULL))
		{
			break;
		}
	}
}

//////////////////////////////////////////////////////////////////////////
// FBlueprintUnloader

/** Utility struct, used to aid in unloading and replacing a specific blueprint. */
struct FBlueprintUnloader
{
public:
	FBlueprintUnloader(UBlueprint* OldBlueprint);

	/** 
	 * Unloads the specified Blueprint (marking it pending-kill, and removing it 
	 * from its outer package). Optionally, will unload the package as well.
	 *
	 * @param  bResetPackage	Whether or not this should unload the entire package.
	 */
	void UnloadBlueprint(const bool bResetPackage);
	
	/** 
	 * Replaces all old references to the original blueprints (its class/CDO/etc.)
	 * @param  NewBlueprint	The blueprint to replace old references with
	 */
	void ReplaceStaleRefs(UBlueprint* NewBlueprint);

private:
	TWeakObjectPtr<UBlueprint> OldBlueprint;
	UClass*  OldGeneratedClass;
	UObject* OldCDO;
	UClass*  OldSkeletonClass;
	UObject* OldSkelCDO;
};

FBlueprintUnloader::FBlueprintUnloader(UBlueprint* OldBlueprintIn)
	: OldBlueprint(OldBlueprintIn)
	, OldGeneratedClass(OldBlueprint->GeneratedClass)
	, OldCDO(nullptr)
	, OldSkeletonClass(OldBlueprint->SkeletonGeneratedClass)
	, OldSkelCDO(nullptr)
{
	if (OldGeneratedClass != nullptr)
	{
		OldCDO = OldGeneratedClass->GetDefaultObject(/*bCreateIfNeeded =*/false);
	}
	if (OldSkeletonClass != nullptr)
	{
		OldSkelCDO = OldSkeletonClass->GetDefaultObject(/*bCreateIfNeeded =*/false);
	}
	OldBlueprint = OldBlueprintIn;
}

void FBlueprintUnloader::UnloadBlueprint(const bool bResetPackage)
{
	if (OldBlueprint.IsValid())
	{
		UBlueprint* UnloadingBp = OldBlueprint.Get();

		UPackage* const OldPackage = UnloadingBp->GetOutermost();
		bool const bIsDirty = OldPackage->IsDirty();

		UPackage* const TransientPackage = GetTransientPackage();
		check(OldPackage != TransientPackage); // is the blueprint already unloaded?
		
		FName const BlueprintName = UnloadingBp->GetFName();
		// move the blueprint to the transient package (to be picked up by garbage collection later)
		FName UnloadedName = MakeUniqueObjectName(TransientPackage, UBlueprint::StaticClass(), BlueprintName);
		UnloadingBp->Rename(*UnloadedName.ToString(), TransientPackage, REN_DontCreateRedirectors | REN_DoNotDirty);
		// @TODO: currently, REN_DoNotDirty does not guarantee that the package 
		//        will not be marked dirty
		OldPackage->SetDirtyFlag(bIsDirty);

		// make sure the blueprint is properly trashed (remove it from the package)
		UnloadingBp->SetFlags(RF_Transient);
		UnloadingBp->ClearFlags(RF_Standalone | RF_Transactional);
		UnloadingBp->RemoveFromRoot();
		UnloadingBp->MarkAsGarbage();
		// if it's in the undo buffer, then we have to clear that...
		if (FKismetEditorUtilities::IsReferencedByUndoBuffer(UnloadingBp))
		{
			GEditor->Trans->Reset(LOCTEXT("UnloadedBlueprint", "Unloaded Blueprint"));
		}

		if (bResetPackage)
		{
			TArray<UPackage*> PackagesToUnload;
			PackagesToUnload.Add(OldPackage);

			FText PackageUnloadError;
			UPackageTools::UnloadPackages(PackagesToUnload, PackageUnloadError);

			if (!PackageUnloadError.IsEmpty())
			{
				const FText ErrorMessage = FText::Format(LOCTEXT("UnloadBpPackageError", "Failed to unload Bluprint '{0}': {1}"),
					FText::FromName(BlueprintName), PackageUnloadError);
				FSlateNotificationManager::Get().AddNotification(FNotificationInfo(ErrorMessage));

				// fallback to manually setting up the package so it can reload 
				// the blueprint 
				ResetLoaders(OldPackage);
				OldPackage->ClearFlags(RF_WasLoaded);
				OldPackage->bHasBeenFullyLoaded = false;
				OldPackage->GetMetaData().RemoveMetaDataOutsidePackage(OldPackage);
			}
		}

		UnloadingBp->ClearEditorReferences();

		// handled in FBlueprintEditor (from the OnBlueprintUnloaded event)
// 		IAssetEditorInstance* EditorInst = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(UnloadingBp, /*bFocusIfOpen =*/false);
// 		if (EditorInst != nullptr)
// 		{
// 			EditorInst->CloseWindow();
// 		}
	}
}

void FBlueprintUnloader::ReplaceStaleRefs(UBlueprint* NewBlueprint)
{
	//--------------------------------------
	// Construct redirects
	//--------------------------------------

	TMap<UObject*, UObject*> Redirects;
	TArray<UObject*> OldObjsNeedingReplacing;

	if (OldBlueprint.IsValid(/*bEvenIfPendingKill =*/true))
	{
		UBlueprint* ToBeReplaced = OldBlueprint.Get(/*bEvenIfPendingKill =*/true);
		if (OldGeneratedClass != nullptr)
		{
			OldObjsNeedingReplacing.Add(OldGeneratedClass);
			Redirects.Add(OldGeneratedClass, NewBlueprint->GeneratedClass);
		}
		if (OldCDO != nullptr)
		{
			OldObjsNeedingReplacing.Add(OldCDO);
			Redirects.Add(OldCDO, NewBlueprint->GeneratedClass->GetDefaultObject());
		}
		if (OldSkeletonClass != nullptr)
		{
			OldObjsNeedingReplacing.Add(OldSkeletonClass);
			Redirects.Add(OldSkeletonClass, NewBlueprint->SkeletonGeneratedClass);
		}
		if (OldSkelCDO != nullptr)
		{
			OldObjsNeedingReplacing.Add(OldSkelCDO);
			Redirects.Add(OldSkelCDO, NewBlueprint->SkeletonGeneratedClass->GetDefaultObject());
		}

		OldObjsNeedingReplacing.Add(ToBeReplaced);
		Redirects.Add(ToBeReplaced, NewBlueprint);

		// clear the object being debugged; otherwise ReplaceInstancesOfClass()  
		// trys to reset it with a new level instance, and OldBlueprint won't 
		// match the new instance's type (it's now a NewBlueprint)
		ToBeReplaced->SetObjectBeingDebugged(nullptr);
	}

	//--------------------------------------
	// Replace old references
	//--------------------------------------

	TArray<UObject*> Referencers;
	// find all objects, still referencing the old blueprint/class/cdo/etc.
	for (auto Referencer : TFindObjectReferencers<UObject>(OldObjsNeedingReplacing, /*PackageToCheck =*/nullptr, /*bIgnoreTemplates =*/false))
	{
		Referencers.Add(Referencer.Value);
	}

	FBlueprintCompileReinstancer::ReplaceInstancesOfClass(OldGeneratedClass, NewBlueprint->GeneratedClass, FReplaceInstancesOfClassParameters());

	for (UObject* Referencer : Referencers)
	{
		FArchiveReplaceObjectRef<UObject>(Referencer, Redirects);
	}
}

//////////////////////////////////////////////////////////////////////////

// Static variable definition
TArray<FString> FKismetEditorUtilities::TrackedBlueprintParentList;
FKismetEditorUtilities::FOnBlueprintUnloaded FKismetEditorUtilities::OnBlueprintUnloaded;
FKismetEditorUtilities::FOnBlueprintGeneratedClassUnloaded FKismetEditorUtilities::OnBlueprintGeneratedClassUnloaded;
TMultiMap<void*, FKismetEditorUtilities::FDefaultEventNodeData> FKismetEditorUtilities::AutoGeneratedDefaultEventsMap;
TMultiMap<void*, FKismetEditorUtilities::FOnBlueprintCreatedData> FKismetEditorUtilities::OnBlueprintCreatedCallbacks;

/** Create the correct event graphs for this blueprint */
void FKismetEditorUtilities::CreateDefaultEventGraphs(UBlueprint* Blueprint)
{
	UEdGraph* Ubergraph = FBlueprintEditorUtils::CreateNewGraph(Blueprint, UEdGraphSchema_K2::GN_EventGraph, UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass());
	Ubergraph->bAllowDeletion = false; //@TODO: Really, just want to make sure we never drop below 1, not that you cannot delete any particular one!
	FBlueprintEditorUtils::AddUbergraphPage(Blueprint, Ubergraph);

	Blueprint->LastEditedDocuments.AddUnique(Ubergraph);
}

/** Create a new Blueprint and initialize it to a valid state but uses the associated blueprint types. */
UBlueprint* FKismetEditorUtilities::CreateBlueprint(UClass* ParentClass, UObject* Outer, const FName NewBPName,	EBlueprintType BlueprintType, FName CallingContext)
{
	UClass* BlueprintClassType = UBlueprint::StaticClass();
	UClass* BlueprintGeneratedClassType = UBlueprintGeneratedClass::StaticClass();
	const IKismetCompilerInterface& KismetCompilerModule = FModuleManager::LoadModuleChecked<IKismetCompilerInterface>("KismetCompiler");
	KismetCompilerModule.GetBlueprintTypesForClass(ParentClass, BlueprintClassType, BlueprintGeneratedClassType);

	return CreateBlueprint(ParentClass, Outer, NewBPName, BlueprintType, BlueprintClassType, BlueprintGeneratedClassType, CallingContext);
}

/** Create a new Blueprint and initialize it to a valid state. */
UBlueprint* FKismetEditorUtilities::CreateBlueprint(UClass* ParentClass, UObject* Outer, const FName NewBPName, EBlueprintType BlueprintType, TSubclassOf<UBlueprint> BlueprintClassType, TSubclassOf<UBlueprintGeneratedClass> BlueprintGeneratedClassType, FName CallingContext)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CreateBlueprint);
	check(FindObject<UBlueprint>(Outer, *NewBPName.ToString()) == NULL); 

	// Not all types are legal for all parent classes, if the parent class is const then the blueprint cannot be an ubergraph-bearing one
	if ((BlueprintType == BPTYPE_Normal) && (ParentClass->HasAnyClassFlags(CLASS_Const)))
	{
		BlueprintType = BPTYPE_Const;
	}

	const UBlueprintEditorSettings* Settings = GetDefault<UBlueprintEditorSettings>();
	check(Settings);
	
	// Create new UBlueprint object
	UBlueprint* NewBP = NewObject<UBlueprint>(Outer, *BlueprintClassType, NewBPName, RF_Public | RF_Standalone | RF_Transactional | RF_LoadCompleted);
	NewBP->Status = BS_BeingCreated;
	NewBP->BlueprintType = BlueprintType;
	NewBP->ParentClass = ParentClass;
	NewBP->BlueprintSystemVersion = UBlueprint::GetCurrentBlueprintSystemVersion();
	NewBP->bIsNewlyCreated = true;
	NewBP->bLegacyNeedToPurgeSkelRefs = false;
	NewBP->GenerateNewGuid();

	// Create SimpleConstructionScript and UserConstructionScript
	if (FBlueprintEditorUtils::SupportsConstructionScript(NewBP))
	{ 
		// >>> Temporary workaround, before a BlueprintGeneratedClass is the main asset.
		FName NewSkelClassName, NewGenClassName;
		NewBP->GetBlueprintClassNames(NewGenClassName, NewSkelClassName);
		UBlueprintGeneratedClass* NewClass = NewObject<UBlueprintGeneratedClass>(
			NewBP->GetOutermost(), *BlueprintGeneratedClassType, NewGenClassName, RF_Public | RF_Transactional);
		NewBP->GeneratedClass = NewClass;
		NewClass->ClassGeneratedBy = NewBP;
		NewClass->SetSuperStruct(ParentClass);
		// <<< Temporary workaround

		NewBP->SimpleConstructionScript = NewObject<USimpleConstructionScript>(NewClass);
		NewBP->SimpleConstructionScript->SetFlags(RF_Transactional);
		NewBP->LastEditedDocuments.Add(ToRawPtr(NewBP->SimpleConstructionScript));

		// Note: UCS graph creation may be restricted due to editor permissions.
		if (Settings->IsFunctionAllowed(NewBP, UEdGraphSchema_K2::FN_UserConstructionScript))
		{
			UEdGraph* UCSGraph = FKismetEditorUtilities::CreateUserConstructionScript(NewBP);

			NewBP->LastEditedDocuments.Add(UCSGraph);
			UCSGraph->bAllowDeletion = false;
		}
	}

	// Create default event graph(s)
	if (FBlueprintEditorUtils::DoesSupportEventGraphs(NewBP) && NewBP->SupportsEventGraphs())
	{
		check(NewBP->UbergraphPages.Num() == 0);
		CreateDefaultEventGraphs(NewBP);
	}

	//@TODO: ANIMREFACTOR 1: This kind of code should be on a per-blueprint basis; not centralized here
	if (UAnimBlueprint* AnimBP = Cast<UAnimBlueprint>(NewBP))
	{
		UAnimBlueprint* RootAnimBP = UAnimBlueprint::FindRootAnimBlueprint(AnimBP);
		if (RootAnimBP == nullptr)
		{
			// Interfaces dont have default graphs, only 'function' anim graphs
			if(AnimBP->BlueprintType != BPTYPE_Interface)
			{
				// Only allow an anim graph if there isn't one in a parent blueprint
				UEdGraph* NewGraph = FBlueprintEditorUtils::CreateNewGraph(AnimBP, UEdGraphSchema_K2::GN_AnimGraph, UAnimationGraph::StaticClass(), UAnimationGraphSchema::StaticClass());
				FBlueprintEditorUtils::AddDomainSpecificGraph(NewBP, NewGraph);
				NewBP->LastEditedDocuments.Add(NewGraph);
				NewGraph->bAllowDeletion = false;
			}
		}
		else
		{
			// Make sure the anim blueprint targets the same skeleton as the parent
			AnimBP->TargetSkeleton = RootAnimBP->TargetSkeleton;
		}
	}

	// Create initial UClass
	IKismetCompilerInterface& Compiler = FModuleManager::LoadModuleChecked<IKismetCompilerInterface>(KISMET_COMPILER_MODULENAME);

	// Skip validation of the class default object here, because (a) the new CDO may fail validation since this
	// is a new Blueprint that the user has not had a chance to modify any defaults for yet, and (b) in some cases,
	// default value propagation to the new Blueprint's CDO may be deferred until after compilation (e.g. reparenting).
	// Also skip the Blueprint search data update, as that will be handled by an OnAssetAdded() delegate in the FiB manager.
	const EBlueprintCompileOptions CompileOptions =
		EBlueprintCompileOptions::SkipGarbageCollection |
		EBlueprintCompileOptions::SkipDefaultObjectValidation |
		EBlueprintCompileOptions::SkipFiBSearchMetaUpdate;

	FBlueprintCompilationManager::CompileSynchronously(
		FBPCompileRequest(NewBP, CompileOptions, nullptr)
	);

	// Mark the BP as being regenerated, so it will not be confused as needing to be loaded and regenerated when a referenced BP loads.
	NewBP->bHasBeenRegenerated = true;

	if(Settings->bSpawnDefaultBlueprintNodes)
	{
		// Only add default events if there is an ubergraph and they are supported
		if(NewBP->UbergraphPages.Num() && FBlueprintEditorUtils::DoesSupportEventGraphs(NewBP))
		{
			// Based on the Blueprint type we are constructing, place some starting events.
			// Note, this cannot happen in the Factories for constructing these Blueprint types due to the fact that creating child BPs circumvent the factories
			UClass* WidgetClass = FindObject<UClass>(nullptr, TEXT("/Script/UMG.UserWidget"));
			UClass* GameplayAbilityClass = FindObject<UClass>(nullptr, TEXT("/Script/GameplayAbilities.GameplayAbility"));

			TArray<FName> AutoSpawnedEventNames;
			int32 NodePositionY = 0;

			// Spawn any defined auto generated default events for the class.  Only do this for the most senior class specified, so
			// that subclasses may have an entirely different set of default nodes if they wish.
			UClass* DefaultNodesClass = NewBP->GeneratedClass;
			while ( DefaultNodesClass )
			{
				bool bFoundDefaultNodes = false;
				for ( TMultiMap<void*, FDefaultEventNodeData>::TIterator DataIt(AutoGeneratedDefaultEventsMap); DataIt; ++DataIt )
				{
					FDefaultEventNodeData Data = DataIt.Value();
					if ( DefaultNodesClass == Data.TargetClass )
					{
						bFoundDefaultNodes = true;
						FKismetEditorUtilities::AddDefaultEventNode(NewBP, NewBP->UbergraphPages[0], Data.EventName, Data.TargetClass, NodePositionY);
					}
				}

				if ( bFoundDefaultNodes )
				{
					break;
				}

				DefaultNodesClass = DefaultNodesClass->GetSuperClass();
			}
		}

		// Give anyone who wants to do more advanced BP modification post-creation a chance to do so.
		// Anim Blueprints, for example, adds a non-event node to the main ubergraph.
		for (TMultiMap<void*, FKismetEditorUtilities::FOnBlueprintCreatedData>::TIterator DataIt(OnBlueprintCreatedCallbacks); DataIt; ++DataIt)
		{
			FOnBlueprintCreatedData Data = DataIt.Value();
			if (NewBP->GeneratedClass && NewBP->GeneratedClass->IsChildOf(Data.TargetClass))
			{
				FKismetEditorUtilities::FOnBlueprintCreated BlueprintCreatedDelegate = Data.OnBlueprintCreated;
				BlueprintCreatedDelegate.Execute(NewBP);
			}
		}
	}
	
	// Create the sparse class data and set the flag saying it is safe to serialize it
	UBlueprintGeneratedClass* BlueprintGeneratedClass = CastChecked<UBlueprintGeneratedClass>(NewBP->GeneratedClass);
	void* SparseDataPtr = BlueprintGeneratedClass->GetOrCreateSparseClassData();
	BlueprintGeneratedClass->bIsSparseClassDataSerializable = SparseDataPtr != nullptr;

	// Report blueprint creation to analytics
	if (FEngineAnalytics::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> Attribs;

		// translate the CallingContext into a string for analytics
		if (CallingContext != NAME_None)
		{
			Attribs.Add(FAnalyticsEventAttribute(FString("Context"), CallingContext.ToString()));
		}
		
		Attribs.Add(FAnalyticsEventAttribute(FString("ParentType"), ParentClass->ClassGeneratedBy == NULL ? FString("Native") : FString("Blueprint")));

		if(IsTrackedBlueprintParent(ParentClass))
		{
			Attribs.Add(FAnalyticsEventAttribute(FString("ParentClass"), ParentClass->GetName()));
		}

		const UGeneralProjectSettings& ProjectSettings = *GetDefault<UGeneralProjectSettings>();
		Attribs.Add(FAnalyticsEventAttribute(FString("ProjectId"), ProjectSettings.ProjectID.ToString()));
		Attribs.Add(FAnalyticsEventAttribute(FString("BlueprintId"), NewBP->GetBlueprintGuid().ToString()));

		FEngineAnalytics::GetProvider().RecordEvent(FString("Editor.Usage.BlueprintCreated"), Attribs);
	}

	return NewBP;
}

UEdGraph* FKismetEditorUtilities::CreateUserConstructionScript(UBlueprint* NewBP)
{
	UEdGraph* UCSGraph = FBlueprintEditorUtils::CreateNewGraph(NewBP, UEdGraphSchema_K2::FN_UserConstructionScript, UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass());
	FBlueprintEditorUtils::AddFunctionGraph(NewBP, UCSGraph, /*bIsUserCreated=*/ false, AActor::StaticClass());

	// If the blueprint is derived from another blueprint, add in a super-call automatically
	if (NewBP->ParentClass && NewBP->ParentClass->HasAnyClassFlags(CLASS_CompiledFromBlueprint))
	{
		check(UCSGraph->Nodes.Num() > 0);
		UK2Node_FunctionEntry* UCSEntry = CastChecked<UK2Node_FunctionEntry>(UCSGraph->Nodes[0]);
		FGraphNodeCreator<UK2Node_CallParentFunction> FunctionNodeCreator(*UCSGraph);
		UK2Node_CallParentFunction* ParentFunctionNode = FunctionNodeCreator.CreateNode();
		ParentFunctionNode->FunctionReference.SetExternalMember(UEdGraphSchema_K2::FN_UserConstructionScript, NewBP->ParentClass);
		ParentFunctionNode->NodePosX = 200;
		ParentFunctionNode->NodePosY = 0;
		ParentFunctionNode->AllocateDefaultPins();
		FunctionNodeCreator.Finalize();

		// Wire up the new node
		UEdGraphPin* ExecPin = UCSEntry->FindPin(UEdGraphSchema_K2::PN_Then);
		UEdGraphPin* SuperPin = ParentFunctionNode->FindPin(UEdGraphSchema_K2::PN_Execute);
		ExecPin->MakeLinkTo(SuperPin);
	}

	return UCSGraph;
}

void FKismetEditorUtilities::RegisterAutoGeneratedDefaultEvent(void* InOwner, UClass* InTargetClass, FName InEventName)
{
	FDefaultEventNodeData Data;
	Data.TargetClass = InTargetClass;
	Data.EventName = InEventName;
	AutoGeneratedDefaultEventsMap.Add(InOwner, Data);
}

void FKismetEditorUtilities::RegisterOnBlueprintCreatedCallback(void* InOwner, UClass* InTargetClass, FOnBlueprintCreated InOnBlueprintCreatedCallback)
{
	FOnBlueprintCreatedData Data;
	Data.TargetClass = InTargetClass;
	Data.OnBlueprintCreated = InOnBlueprintCreatedCallback;
	OnBlueprintCreatedCallbacks.Add(InOwner, Data);
}

void FKismetEditorUtilities::UnregisterAutoBlueprintNodeCreation(void* InOwner)
{
	AutoGeneratedDefaultEventsMap.Remove(InOwner);
	OnBlueprintCreatedCallbacks.Remove(InOwner);
}

UK2Node_Event* FKismetEditorUtilities::AddDefaultEventNode(UBlueprint* InBlueprint, UEdGraph* InGraph, FName InEventName, UClass* InEventClass, int32& InOutNodePosY)
{
	UK2Node_Event* NewEventNode = nullptr;

	FMemberReference EventReference;
	EventReference.SetExternalMember(InEventName, InEventClass);

	// Prevent events that are hidden in the Blueprint's class from being auto-generated.
	const bool bIsFunctionVisible = !FObjectEditorUtils::IsFunctionHiddenFromClass(EventReference.ResolveMember<UFunction>(InBlueprint), InBlueprint->ParentClass);
	
	// Prevent events that are not allowed in the editor (due to permission settings).
	const bool bIsFunctionAllowed = GetDefault<UBlueprintEditorSettings>()->IsFunctionAllowed(InBlueprint, InEventName);

	if(bIsFunctionVisible && bIsFunctionAllowed)
	{
		const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

		// Add the event
		NewEventNode = NewObject<UK2Node_Event>(InGraph);
		NewEventNode->EventReference = EventReference;

		// Snap the new position to the grid
		const UEditorStyleSettings* StyleSettings = GetDefault<UEditorStyleSettings>();
		if (StyleSettings)
		{
			const uint32 GridSnapSize = StyleSettings->GridSnapSize;
			InOutNodePosY = GridSnapSize * FMath::RoundFromZero(InOutNodePosY / (float)GridSnapSize);
		}
		
		// add update event graph
		NewEventNode->bOverrideFunction=true;
		NewEventNode->CreateNewGuid();
		NewEventNode->PostPlacedNewNode();
		NewEventNode->SetFlags(RF_Transactional);
		NewEventNode->AllocateDefaultPins();
		NewEventNode->bCommentBubblePinned = true;
		NewEventNode->bCommentBubbleVisible = true;
		NewEventNode->NodePosY = InOutNodePosY;
		UEdGraphSchema_K2::SetNodeMetaData(NewEventNode, FNodeMetadata::DefaultGraphNode);
		InOutNodePosY = NewEventNode->NodePosY + NewEventNode->NodeHeight + 200;

		InGraph->AddNode(NewEventNode);

		// Get the function that the event node or function entry represents
		FFunctionFromNodeHelper FunctionFromNode(NewEventNode);
		if (FunctionFromNode.Function && Schema->GetCallableParentFunction(FunctionFromNode.Function))
		{
			UFunction* ValidParent = Schema->GetCallableParentFunction(FunctionFromNode.Function);
			FGraphNodeCreator<UK2Node_CallParentFunction> FunctionNodeCreator(*InGraph);
			UK2Node_CallParentFunction* ParentFunctionNode = FunctionNodeCreator.CreateNode();
			ParentFunctionNode->SetFromFunction(ValidParent);
			ParentFunctionNode->AllocateDefaultPins();

			for (UEdGraphPin* EventPin : NewEventNode->Pins)
			{
				if (UEdGraphPin* ParentPin = ParentFunctionNode->FindPin(EventPin->PinName, EGPD_Input))
				{
					ParentPin->MakeLinkTo(EventPin);
				}
			}
			ParentFunctionNode->GetExecPin()->MakeLinkTo(NewEventNode->FindPin(UEdGraphSchema_K2::PN_Then));

			ParentFunctionNode->NodePosX = FunctionFromNode.Node->NodePosX + FunctionFromNode.Node->NodeWidth + 200;
			ParentFunctionNode->NodePosY = FunctionFromNode.Node->NodePosY;
			UEdGraphSchema_K2::SetNodeMetaData(ParentFunctionNode, FNodeMetadata::DefaultGraphNode);
			FunctionNodeCreator.Finalize();

			ParentFunctionNode->MakeAutomaticallyPlacedGhostNode();
		}

		NewEventNode->MakeAutomaticallyPlacedGhostNode();
	}

	return NewEventNode;
}

UBlueprint* FKismetEditorUtilities::ReplaceBlueprint(UBlueprint* Target, UBlueprint const* ReplacementArchetype)
{
	if (Target == ReplacementArchetype)
	{
		return Target;
	}

	FName DesiredName = Target->GetFName();

	UPackage* BlueprintPackage = Target->GetOutermost();
	check(BlueprintPackage != GetTransientPackage());
	
	// Need to close the editor now, it won't work once it's been garbage collected during the reload
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(Target);

	FBlueprintUnloader Unloader(Target);
	Unloader.UnloadBlueprint(/*bResetPackage =*/false);

	UBlueprint* Replacement = Cast<UBlueprint>(StaticDuplicateObject(ReplacementArchetype, BlueprintPackage, DesiredName));
	
	Unloader.ReplaceStaleRefs(Replacement);
	return Replacement;
}

bool FKismetEditorUtilities::IsReferencedByUndoBuffer(UBlueprint* Blueprint)
{
	UObject* BlueprintObj = Blueprint;
	FReferencerInformationList ReferencesIncludingUndo;
	IsReferenced(BlueprintObj, GARBAGE_COLLECTION_KEEPFLAGS, EInternalObjectFlags_GarbageCollectionKeepFlags, /*bCheckSubObjects =*/true, &ReferencesIncludingUndo);

	FReferencerInformationList ReferencesExcludingUndo;
	// Determine the in-memory references, *excluding* the undo buffer

	UTransactor* Transactor = GEditor ? ToRawPtr(GEditor->Trans) : nullptr;
	if (Transactor)
	{
		Transactor->DisableObjectSerialization();
	}
	IsReferenced(BlueprintObj, GARBAGE_COLLECTION_KEEPFLAGS, EInternalObjectFlags_GarbageCollectionKeepFlags, /*bCheckSubObjects =*/true, &ReferencesExcludingUndo);
	if (Transactor)
	{
		Transactor->EnableObjectSerialization();
	}

	// see if this object is the transaction buffer - set a flag so we know we need to clear the undo stack
	const int32 TotalReferenceCount   = ReferencesIncludingUndo.ExternalReferences.Num() + ReferencesIncludingUndo.InternalReferences.Num();
	const int32 NonUndoReferenceCount = ReferencesExcludingUndo.ExternalReferences.Num() + ReferencesExcludingUndo.InternalReferences.Num();

	return (TotalReferenceCount > NonUndoReferenceCount);
}

void FKismetEditorUtilities::CompileBlueprint(UBlueprint* BlueprintObj, EBlueprintCompileOptions CompileFlags, FCompilerResultsLog* pResults)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	if (UE::Private::bLogExplicitCompilationRequests
		&& !EnumHasAnyFlags(CompileFlags, EBlueprintCompileOptions::IsRegeneratingOnLoad))
	{
		LOG_SCOPE_VERBOSITY_OVERRIDE(LogBlueprint, ELogVerbosity::Log);
		UE_LOGF(LogBlueprint, Log, "Compiling Blueprint '%ls'", *GetPathNameSafe(BlueprintObj));
	}
	FBlueprintCompilationManager::CompileSynchronously(FBPCompileRequest(BlueprintObj, CompileFlags, pResults));
}

/** Generates a blueprint skeleton only.  Minimal compile, no notifications will be sent, no GC, etc.  Only successful if there isn't already a skeleton generated */
bool FKismetEditorUtilities::GenerateBlueprintSkeleton(UBlueprint* BlueprintObj, bool bForceRegeneration)
{
	bool bRegeneratedSkeleton = false;
	check(BlueprintObj);

	if( BlueprintObj->SkeletonGeneratedClass == NULL || bForceRegeneration )
	{
		FBlueprintCompilationManager::CompileSynchronously(
			FBPCompileRequest(BlueprintObj, EBlueprintCompileOptions::RegenerateSkeletonOnly, nullptr)
		);
	}
	return bRegeneratedSkeleton;
}

namespace ConformComponentsUtils
{
	static void ConformRemovedNativeComponents(UObject* BpCdo);
	static UObject* FindNativeArchetype(const UObject* NativeCDO, UActorComponent* Component);
};

static void ConformComponentsUtils::ConformRemovedNativeComponents(UObject* BpCdo)
{
	UClass* BlueprintClass = BpCdo->GetClass();
	check(BpCdo->HasAnyFlags(RF_ClassDefaultObject) && BlueprintClass->HasAnyClassFlags(CLASS_CompiledFromBlueprint));

	AActor* ActorCDO = Cast<AActor>(BpCdo);
	if (ActorCDO == nullptr)
	{
		return;
	}

	UClass* const NativeSuperClass = FBlueprintEditorUtils::FindFirstNativeClass(BlueprintClass);
	const AActor* NativeCDO = GetDefault<AActor>(NativeSuperClass);

	TInlineComponentArray<UActorComponent*> OldNativeComponents;
	TInlineComponentArray<UActorComponent*> NewNativeComponents;
	ActorCDO->GetComponents(OldNativeComponents);
	USceneComponent* OldNativeRootComponent = ActorCDO->GetRootComponent();

	TSet<UObject*> DestroyedComponents;
	for (UActorComponent* Component : OldNativeComponents)
	{
		UObject* NativeArchetype = FindNativeArchetype(NativeCDO, Component);
		if ((NativeArchetype == nullptr) || !NativeArchetype->HasAnyFlags(RF_ClassDefaultObject))
		{
			// Keep track of components inherited from the native super class that are still valid.
			NewNativeComponents.Add(Component);
			continue;
		}

		// If we have overriden the class of a native component then ensure that the component still exists and that the overriden class is a valid subclass of it
		if (GetAllowNativeComponentClassOverrides())
		{
			if (const FBPComponentClassOverride* BPCO = CastChecked<UBlueprintGeneratedClass>(BlueprintClass)->ComponentClassOverrides.FindByKey(Component->GetFName()))
			{
				if (BPCO->ComponentClass == Component->GetClass())
				{
					if (UObject* OverridenComponent = (UObject*)FindObjectWithOuter(NativeCDO, UActorComponent::StaticClass(), Component->GetFName()))
					{
						if (Component->IsA(OverridenComponent->GetClass()))
						{
							NewNativeComponents.Add(Component);
							continue;
						}
					}
				}
			}
		}

		// else, the component has been removed from our native super class

		Component->DestroyComponent(/*bPromoteChildren =*/false);
		if (Component->HasAnyInternalFlags(EInternalObjectFlags_AsyncLoading))
		{
			// Async loading components cannot be pending kill, or the async loading code will assert when trying to postload them.
			Component->ClearGarbage();
			FLinkerLoad::InvalidateExport(Component);
		}
		DestroyedComponents.Add(Component);

		// The DestroyComponent() call above will clear the RootComponent value in this case.
		if(Component == OldNativeRootComponent)
		{
			// Restore it here so that it will be reassigned to match the native CDO's value below.
			ActorCDO->SetRootComponent(OldNativeRootComponent);
		}

		UClass* ComponentClass = Component->GetClass();
		for (TFieldIterator<FArrayProperty> ArrayPropIt(NativeSuperClass); ArrayPropIt; ++ArrayPropIt)
		{
			FArrayProperty* ArrayProp = *ArrayPropIt;

			FObjectProperty* ObjInnerProp = CastField<FObjectProperty>(ArrayProp->Inner);
			if ((ObjInnerProp == nullptr) || !ComponentClass->IsChildOf(ObjInnerProp->PropertyClass))
			{
				continue;
			}

			uint8* BpArrayPtr = ArrayProp->ContainerPtrToValuePtr<uint8>(ActorCDO);
			FScriptArrayHelper BpArrayHelper(ArrayProp, BpArrayPtr);
			// iterate backwards so we can remove as we go
			for (int32 ArrayIndex = BpArrayHelper.Num()-1; ArrayIndex >= 0; --ArrayIndex)
			{
				uint8* BpEntryPtr = BpArrayHelper.GetRawPtr(ArrayIndex);
				UObject* ObjEntryValue = ObjInnerProp->GetObjectPropertyValue(BpEntryPtr);

				if (ObjEntryValue == Component)
				{
					// NOTE: until we fixup UE-15224, then this may be undesirably diverging from the natively defined 
					//       array (think delta serialization); however, I think from Blueprint creation on we treat 
					//       instanced sub-object arrays as differing (just may be confusing to the user)
					BpArrayHelper.RemoveValues(ArrayIndex);
				}
			}
		}

		// @TODO: have to also remove from map properties now that they're available
	}

	auto FindComponentTemplateByNameInActorCDO = [&NewNativeComponents](FName ToFind) -> UActorComponent**
	{
		return NewNativeComponents.FindByPredicate([ToFind](const UActorComponent* ActorComponent) -> bool
		{
			return ActorComponent && ActorComponent->GetFName() == ToFind;
		});
	};

	// 
	if (DestroyedComponents.Num() > 0)
	{
		for (TFieldIterator<FObjectProperty> ObjPropIt(NativeSuperClass); ObjPropIt; ++ObjPropIt)
		{
			FObjectProperty* ObjectProp = *ObjPropIt;
			UObject* PropObjValue = ObjectProp->GetObjectPropertyValue_InContainer(ActorCDO);

			if (DestroyedComponents.Contains(PropObjValue))
			{
				// Get the "new" value that's currently set on the native parent CDO. We need the Blueprint CDO to reflect this update in property value.
				UObject* SuperObjValue = ObjectProp->GetObjectPropertyValue_InContainer(NativeCDO);
				if (SuperObjValue && SuperObjValue->IsA<UActorComponent>())
				{
					// For components, make sure we use the instance that's owned by the Blueprint CDO and not the native parent CDO's instance.
					if (UActorComponent** ComponentTemplatePtr = FindComponentTemplateByNameInActorCDO(SuperObjValue->GetFName()))
					{
						SuperObjValue = *ComponentTemplatePtr;
					}
				}
			
				// Update the Blueprint CDO to match the native parent CDO.
				ObjectProp->SetObjectPropertyValue_InContainer(ActorCDO, SuperObjValue);
			}
		}
	}

	// Fix up the attachment hierarchy for inherited scene components that are still valid.
	for (UActorComponent* Component : NewNativeComponents)
	{
		if (USceneComponent* SceneComponent = Cast<USceneComponent>(Component))
		{
			// If the component in the Blueprint CDO was attached to a component that's been removed, update the Blueprint's component instance to match the archetype in the native parent CDO.
			if (DestroyedComponents.Contains(SceneComponent->GetAttachParent()))
			{
				if (USceneComponent* NativeArchetype = Cast<USceneComponent>(FindNativeArchetype(NativeCDO, SceneComponent)))
				{
					USceneComponent* NewAttachParent = NativeArchetype->GetAttachParent();
					if (NewAttachParent)
					{
						// Make sure we use the instance that's owned by the Blueprint CDO and not the native parent CDO's instance.
						if (UActorComponent** ComponentTemplatePtr = FindComponentTemplateByNameInActorCDO(NewAttachParent->GetFName()))
						{
							NewAttachParent = CastChecked<USceneComponent>(*ComponentTemplatePtr);
						}
					}

					SceneComponent->SetupAttachment(NewAttachParent);
				}
			}
		}
	}
}

static UObject* ConformComponentsUtils::FindNativeArchetype(const UObject* NativeCDO, UActorComponent* Component)
{
	FSoftObjectPath Path(Component);
	const FString& SubPathString = Path.GetSubPathString();
	const FStringView SubobjectDelim = FStringView(TEXT("."));
	const UE::String::EParseTokensOptions ParseOptions = UE::String::EParseTokensOptions::None;
	UObject* Iterator = const_cast<UObject*>(NativeCDO);
	UE::String::ParseTokens(SubPathString, SubobjectDelim,
		[&Iterator](FStringView Token)
		{
			if (Iterator == nullptr)
			{
				return;
			}

			Iterator = StaticFindObjectFast(UObject::StaticClass(), Iterator, FName(Token));
		},
		ParseOptions);

	UObject* NativeSubobject = const_cast<UObject*>(Iterator);

	if (!NativeSubobject)
	{
		return Component->GetClass()->GetDefaultObject(false);
	}
	else if (!Component->IsA(NativeSubobject->GetClass()))
	{
		return nullptr;
	}
	else
	{
		return NativeSubobject;
	}
}

/** Tries to make sure that a blueprint is conformed to its native parent, in case any native class flags have changed */
void FKismetEditorUtilities::ConformBlueprintFlagsAndComponents(UBlueprint* BlueprintObj)
{
	// Propagate native class flags to the children class.  This fixes up cases where native instanced components get added after BP creation, etc
	const UClass* ParentClass = BlueprintObj->ParentClass;

	if( UClass* SkelClass = BlueprintObj->SkeletonGeneratedClass )
	{
		SkelClass->ClassFlags |= (ParentClass->ClassFlags & CLASS_ScriptInherit);
		SkelClass->ClassConfigName = ParentClass->ClassConfigName;
		SkelClass->ClassWithin = ParentClass->ClassWithin;
		UObject* SkelCDO = SkelClass->GetDefaultObject();
		// NOTE: we don't need to call ConformRemovedNativeComponents() for skel
		//       classes, as they're generated on load (and not saved with stale 
		//       components)
		SkelCDO->InstanceSubobjectTemplates();
	}

	if( UClass* GenClass = BlueprintObj->GeneratedClass )
	{
		GenClass->ClassFlags |= (ParentClass->ClassFlags & CLASS_ScriptInherit);
		GenClass->ClassConfigName = ParentClass->ClassConfigName;
		GenClass->ClassWithin = ParentClass->ClassWithin;
		if (UObject* GenCDO = GenClass->GetDefaultObject(false))
		{
			ConformComponentsUtils::ConformRemovedNativeComponents(GenCDO);
			GenCDO->InstanceSubobjectTemplates();
		}
	}

	UInheritableComponentHandler* InheritableComponentHandler = BlueprintObj->GetInheritableComponentHandler(false);
	if (InheritableComponentHandler)
	{
		InheritableComponentHandler->ValidateTemplates();
	}
}

/** @return		true is it's possible to create a blueprint from the specified class */
bool FKismetEditorUtilities::CanCreateBlueprintOfClass(const UClass* Class)
{
	bool bCanCreateBlueprint = false;
	
	if (Class)
	{
		bool bAllowDerivedBlueprints = false;
		GConfig->GetBool(TEXT("Kismet"), TEXT("AllowDerivedBlueprints"), /*out*/ bAllowDerivedBlueprints, GEngineIni);

		bCanCreateBlueprint = !Class->HasAnyClassFlags(CLASS_Deprecated)
			&& !Class->HasAnyClassFlags(CLASS_NewerVersionExists)
			&& (!Class->ClassGeneratedBy || (bAllowDerivedBlueprints && !IsClassABlueprintSkeleton(Class)));

		const bool bIsBPGC = (Cast<UBlueprintGeneratedClass>(Class) != nullptr);

		const bool bIsValidClass = Class->GetBoolMetaDataHierarchical(FBlueprintMetadata::MD_IsBlueprintBase)
			|| (Class == UObject::StaticClass())
			|| (Class == USceneComponent::StaticClass() || Class == UActorComponent::StaticClass())
			|| bIsBPGC;  // BPs are always considered inheritable
			
		bCanCreateBlueprint &= bIsValidClass;
	}
	
	return bCanCreateBlueprint;
}

UPackage* CreateBlueprintPackage(const FString& Path, FString& OutAssetName)
{
	// Create a blueprint
	FString PackageName;
	OutAssetName = FPackageName::GetLongPackageAssetName(Path);

	// If no AssetName was found, generate a unique asset name.
	if (OutAssetName.Len() == 0)
	{
		PackageName = FPackageName::GetLongPackagePath(Path);
		FString BasePath = PackageName + TEXT("/") + LOCTEXT("BlueprintName_Default", "NewBlueprint").ToString();
		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
		AssetToolsModule.Get().CreateUniqueAssetName(BasePath, TEXT(""), PackageName, OutAssetName);
	}
	else
	{
		PackageName = Path;
	}

	return CreatePackage( *PackageName);
}

UBlueprint* FKismetEditorUtilities::CreateBlueprintFromActor(const FString& Path, AActor* Actor, const FKismetEditorUtilities::FCreateBlueprintFromActorParams& Params)
{
	UBlueprint* NewBlueprint = nullptr;
	FString AssetName;

	if (UPackage* Package = CreateBlueprintPackage(Path, AssetName))
	{
		NewBlueprint = CreateBlueprintFromActor(FName(*AssetName), Package, Actor, Params);
	}

	return NewBlueprint;
}

UBlueprint* FKismetEditorUtilities::CreateBlueprintFromActors(const FString& Path, const FKismetEditorUtilities::FCreateBlueprintFromActorsParams& Params)
{
	UBlueprint* NewBlueprint = nullptr;
	FString AssetName;

	if (UPackage* Package = CreateBlueprintPackage(Path, AssetName))
	{
		NewBlueprint = CreateBlueprintFromActors(FName(*AssetName), Package, Params);
	}

	return NewBlueprint;
}

void FKismetEditorUtilities::AddComponentsToBlueprint(UBlueprint* Blueprint, const TArray<UActorComponent*>& Components, const FKismetEditorUtilities::FAddComponentsToBlueprintParams& Params)
{
	USimpleConstructionScript* SCS = Blueprint->SimpleConstructionScript;

	USceneComponent* RootTemplate = nullptr;
	USCS_Node* RootSCSNode = nullptr;
	if (Params.OptionalNewRootNode == nullptr)
	{
		RootTemplate = SCS->GetSceneRootComponentTemplate(false, &RootSCSNode);
	}

	TArray<UBlueprint*> ParentBPStack;
	UBlueprint::GetBlueprintHierarchyFromClass(Blueprint->GeneratedClass, ParentBPStack);

	TMap<USceneComponent*, USCS_Node*> InstanceComponentToNodeMap;

	auto AddChildToSCSRootNodeLambda = [SCS, &Params, RootTemplate, RootSCSNode](USCS_Node* InSCSNode)
	{
		if (Params.OptionalNewRootNode)
		{
			Params.OptionalNewRootNode->AddChildNode(InSCSNode);
		}
		else if (RootTemplate)
		{
			// If we have a known root template and it is from the same blueprint, add the SCSNode as a child
			// of this node, otherwise set the template as parent and add the SCSNode as a RootNode for this BP
			if (RootSCSNode && RootSCSNode->GetSCS() == SCS)
			{
				RootSCSNode->AddChildNode(InSCSNode);
			}
			else
			{
				InSCSNode->SetParent(RootTemplate);
				SCS->AddNode(InSCSNode);
			}
		}
		else
		{
			// Otherwise if there are existing root nodes, attach the SCS Node to the first of those, 
			// and if there are no root nodes, our final fallback is to just be a RootNode
			const TArray<USCS_Node*>& RootNodes = SCS->GetRootNodes();
			if (RootNodes.Num() > 0)
			{
				RootNodes[0]->AddChildNode(InSCSNode);
			}
			else
			{
				SCS->AddNode(InSCSNode);
			}
		}
	};

	/** 
	 * Creates a new USCS_Node in the TargetSCS, duplicating the specified 
	 * component (leaving the new node unattached). If a copy was already 
	 * made (found  in NewSceneComponents) then that will be returned instead.
	 */	
	auto MakeComponentCopy = [SCS, &Params, &InstanceComponentToNodeMap](UActorComponent* ActorComponent)
	{
		USceneComponent* AsSceneComponent = Cast<USceneComponent>(ActorComponent);
		if (AsSceneComponent != nullptr)
		{
			USCS_Node** ExistingCopy = InstanceComponentToNodeMap.Find(AsSceneComponent);
			if (ExistingCopy != nullptr)
			{
				return *ExistingCopy;
			}
		}

		FName NewComponentName = ActorComponent->GetFName();
		if (Params.HarvestMode == EAddComponentToBPHarvestMode::Havest_AppendOwnerName)
		{
			if (AActor* Owner = ActorComponent->GetOwner())
			{
				NewComponentName = *(Owner->GetActorLabel() + TEXT("_") + ActorComponent->GetName());
			}
		}

		USCS_Node* NewSCSNode = SCS->CreateNode(ActorComponent->GetClass(), NewComponentName);
		UEditorEngine::FCopyPropertiesForUnrelatedObjectsParams CPFUOParams;
		CPFUOParams.bDoDelta = false; // We need a deep copy of parameters here so the CDO values get copied as well
		UEditorEngine::CopyPropertiesForUnrelatedObjects(ActorComponent, NewSCSNode->ComponentTemplate, CPFUOParams);

		// Clear the instance component flag
		NewSCSNode->ComponentTemplate->CreationMethod = EComponentCreationMethod::Native;

		if (AsSceneComponent != nullptr)
		{
			InstanceComponentToNodeMap.Add(AsSceneComponent, NewSCSNode);
			if (!Params.bKeepMobility)
			{
				Cast<USceneComponent>(NewSCSNode->ComponentTemplate)->SetMobility(EComponentMobility::Movable);
			}
		}

		if (Params.OutNodes)
		{
			Params.OutNodes->Add(NewSCSNode);
		}

		return NewSCSNode;
	};

	// Associate a scene node to its first attached parent if there is one.
	struct FSceneComponentAndFirstParent
	{
		FSceneComponentAndFirstParent(USceneComponent* InSceneComponent, USceneComponent* InFirstAttachParent)
			: SceneComponent(InSceneComponent)
			, FirstAttachParent(InFirstAttachParent)
		{}

		USceneComponent* SceneComponent;

		// The first attach parent is valid when we find a parent that is part of the components array.
		USceneComponent* FirstAttachParent;
	};

	// Array of FSceneComponentAndFirstParent that will be filled in a way that the parents are always before their children
	TArray<FSceneComponentAndFirstParent> SceneComponentNodes;
	SceneComponentNodes.Reserve(Components.Num());

	// Array of the valid ActorComponents
	TArray<UActorComponent*> ActorComponents;
	ActorComponents.Reserve(Components.Num());

	// The boolean indicate if the scene component was added to the SceneComponentNodes array
	TMap<USceneComponent*, bool> SceneComponentsMap;
	SceneComponentsMap.Reserve(Components.Num());

	for (int32 CompIndex = 0; CompIndex < Components.Num(); ++CompIndex)
	{
		UActorComponent* ActorComponent = Components[CompIndex];

		// Filter out nulls and the components we won't be able to create.
		if (ActorComponent && FKismetEditorUtilities::IsClassABlueprintSpawnableComponent(ActorComponent->GetClass()))
		{
			USceneComponent* SceneComponent = Cast<USceneComponent>(Components[CompIndex]);
			if (SceneComponent)
			{
				SceneComponentsMap.Add(SceneComponent, false);
			}
			else
			{
				ActorComponents.Add(ActorComponent);
			}
		}
	}

	TFunction<void (USceneComponent*)> ConstructSceneComponentNodes;
	// Fill the SceneComponentNodes array
	ConstructSceneComponentNodes = [&SceneComponentsMap, &SceneComponentNodes, &ConstructSceneComponentNodes, RootTemplate] (USceneComponent* SceneComponent)
	{
		// The scene component should always be present in the map
		if (*SceneComponentsMap.Find(SceneComponent))
		{
			// If the node for a scene component is already part of the array just return
			return;
		}
		else
		{
			USceneComponent* Parent = SceneComponent->GetAttachParent();
			while (Parent)
			{
				if (SceneComponentsMap.Contains(Parent))
				{
					// always add the parent first
					ConstructSceneComponentNodes(Parent);
					break;
				}
				Parent = Parent->GetAttachParent();
			}

			if (Parent == nullptr && RootTemplate)
			{
				AActor* Owner = SceneComponent->GetOwner();
				const bool bIsRootComponent = (Owner == nullptr || Owner->GetRootComponent() == SceneComponent);
				if (bIsRootComponent)
				{
					Parent = RootTemplate;
				}
			}

			SceneComponentNodes.Emplace(SceneComponent, Parent);
			SceneComponentsMap.Add(SceneComponent, true);
		}
	};

	TArray<USceneComponent*> SceneComponents;
	SceneComponentsMap.GetKeys(SceneComponents);
	for (USceneComponent* SceneComponent : SceneComponents)
	{
		ConstructSceneComponentNodes(SceneComponent);
	}

	if (Params.OutNodes)
	{
		Params.OutNodes->Reserve(Components.Num());
	}

	// The easy part to add the non-scene components.
	for (UActorComponent* ActorComponent : ActorComponents)
	{
		USCS_Node* SCSNode = MakeComponentCopy(ActorComponent);
		SCS->AddNode(SCSNode);
	}

	// The loop to add the scene components
	for (const FSceneComponentAndFirstParent& ComponentNode : SceneComponentNodes)
	{
		USceneComponent* SceneComponent = ComponentNode.SceneComponent;
		
		USCS_Node* SCSNode = MakeComponentCopy(SceneComponent);

		USceneComponent* FirstAttachParent = ComponentNode.FirstAttachParent;
		
		if (FirstAttachParent)
		{
			// If the parent we're going to be attached to isn't the original attach parent, then adjust the
			// relative transform so that the end result will be consistent with current relationship
			if (FirstAttachParent != SceneComponent->GetAttachParent())
			{
				USceneComponent* SceneComponentTemplate = CastChecked<USceneComponent>(SCSNode->ComponentTemplate);

				const FTransform ComponentToWorld = SceneComponent->GetComponentTransform();
				const FTransform RelativeTransform = ComponentToWorld.GetRelativeTransform(FirstAttachParent->GetComponentTransform());
				if (!SceneComponent->IsUsingAbsoluteLocation())
				{
					SceneComponentTemplate->SetRelativeLocation_Direct(RelativeTransform.GetLocation());
				}
				if (!SceneComponent->IsUsingAbsoluteRotation())
				{
					SceneComponentTemplate->SetRelativeRotation_Direct(RelativeTransform.GetRotation().Rotator());
				}
				if (!SceneComponent->IsUsingAbsoluteScale())
				{
					SceneComponentTemplate->SetRelativeScale3D_Direct(RelativeTransform.GetScale3D());
				}
			}
		}

		AActor* Actor = SceneComponent->GetOwner();
		if (((Actor == nullptr) || (SceneComponent == Actor->GetRootComponent())) && (SceneComponent->GetAttachParent() == nullptr || FirstAttachParent == RootTemplate))
		{
			if (Params.OptionalNewRootNode != nullptr)
			{
				Params.OptionalNewRootNode->AddChildNode(SCSNode);
			}
			else
			{
				if (RootTemplate)
				{
					SCSNode->SetParent(RootTemplate);
				}
				SCS->AddNode(SCSNode);
			}
		}
		// If we're not attached to a blueprint component, add ourself to the root node or the SCS root component:
		else if (SceneComponent->GetAttachParent() == nullptr)
		{
			AddChildToSCSRootNodeLambda(SCSNode);
		}
		// If we're attached to a blueprint component look it up as the variable name is the component name
		else if (SceneComponent->GetAttachParent()->IsCreatedByConstructionScript())
		{
			USCS_Node* ParentSCSNode = nullptr;
			if (FirstAttachParent)
			{
				// If we are using the root template then it will not be in the component node map, the scene component will be
				if(SceneComponent->GetAttachParent() == nullptr || FirstAttachParent == RootTemplate)
				{
					ParentSCSNode = InstanceComponentToNodeMap.FindChecked(SceneComponent);
				}
				else
				{
					ParentSCSNode = InstanceComponentToNodeMap.FindChecked(FirstAttachParent);
				}
			}
			else
			{
				for (UBlueprint* ParentBlueprint : ParentBPStack)
				{
					if (ParentBlueprint->SimpleConstructionScript)
					{
						ParentSCSNode = ParentBlueprint->SimpleConstructionScript->FindSCSNode(SceneComponent->GetAttachParent()->GetFName());
						if (ParentSCSNode)
						{
							break;
						}
					}
				}
			}
			check(ParentSCSNode);

			if (ParentSCSNode->GetSCS() != SCS)
			{
				SCS->AddNode(SCSNode);
				SCSNode->SetParent(ParentSCSNode);
			}
			else
			{
				ParentSCSNode->AddChildNode(SCSNode);
			}
		}
		else if ((SceneComponent->GetAttachParent()->CreationMethod == EComponentCreationMethod::Native) && (Params.HarvestMode == EAddComponentToBPHarvestMode::None))
		{
			// If we're attached to a component that will be native in the new blueprint
			SCS->AddNode(SCSNode);
			SCSNode->SetParent(SceneComponent->GetAttachParent());
		}
		else
		{
			// Otherwise we will already have created the parents' new SCS node, so attach to that
			if (USCS_Node** ParentSCSNode = InstanceComponentToNodeMap.Find(FirstAttachParent))
			{
				(*ParentSCSNode)->AddChildNode(SCSNode);
			}
			else
			{
				AddChildToSCSRootNodeLambda(SCSNode);
			}
		}
	}
}

struct FResetSceneComponentAfterCopy
{
private:
	static void Reset(USceneComponent* Component)
	{
		Component->SetRelativeLocation_Direct(FVector::ZeroVector);
		Component->SetRelativeRotation_Direct(FRotator::ZeroRotator);

		// Clear out the attachment info after having copied the properties from the source actor
		Component->SetupAttachment(nullptr);
		FDirectAttachChildrenAccessor::Get(Component).Empty();

		// Ensure the light mass information is cleaned up
		Component->InvalidateLightingCache();
	}

	friend class FKismetEditorUtilities;
};

UBlueprint* FKismetEditorUtilities::CreateBlueprintFromActor(const FName BlueprintName, UObject* Outer, AActor* Actor, const FKismetEditorUtilities::FCreateBlueprintFromActorParams& Params)
{
	UBlueprint* NewBlueprint = nullptr;
	UClass* ParentClassOverride = Params.ParentClassOverride;

	if (Actor != nullptr)
	{
		if (Outer != nullptr)
		{
			if (ParentClassOverride)
			{
				if (!ParentClassOverride->IsChildOf(Actor->GetClass()))
				{
					// Invalid input, use ActorClass instead?
					return nullptr;
				}
			}
			else
			{
				ParentClassOverride = Actor->GetClass();
			}

			// We don't have a factory, but we can still try to create a blueprint for this actor class
			NewBlueprint = FKismetEditorUtilities::CreateBlueprint(ParentClassOverride, Outer, BlueprintName, EBlueprintType::BPTYPE_Normal, UBlueprint::StaticClass(), UBlueprintGeneratedClass::StaticClass(), FName("CreateFromActor") );
		}

		if (NewBlueprint != nullptr)
		{
			// Notify the asset registry
			FAssetRegistryModule::AssetCreated(NewBlueprint);

			// Mark the package dirty
			Outer->MarkPackageDirty();

			// If the source Actor has Instance Components we need to translate these in to SCS Nodes
			if (Actor->GetInstanceComponents().Num() > 0)
			{
				bool bUsedDefaultSceneRoot = false;
				if (USceneComponent* RootComponent = Actor->GetRootComponent())
				{
					// If the instance root component of the actor being converted is a scene component named the same as the default scene root node
					// then we'll use that from the SimpleConstructionScript rather than creating a new root node
					if (   RootComponent->GetClass() == USceneComponent::StaticClass()
					    && RootComponent->GetFName() == NewBlueprint->SimpleConstructionScript->GetDefaultSceneRootNode()->GetVariableName()
					    && Actor->GetInstanceComponents().Contains(RootComponent))
					{
						bUsedDefaultSceneRoot = true;
						if (Actor->GetInstanceComponents().Num() > 1)
						{
							TArray<UActorComponent*> InstanceComponents = Actor->GetInstanceComponents();
							InstanceComponents.Remove(RootComponent);

							FKismetEditorUtilities::FAddComponentsToBlueprintParams AddCompParams;
							AddCompParams.OptionalNewRootNode = NewBlueprint->SimpleConstructionScript->GetDefaultSceneRootNode();
							AddCompParams.bKeepMobility = Params.bKeepMobility;
							AddComponentsToBlueprint(NewBlueprint, InstanceComponents, AddCompParams);
						}
					}
				}

				if (!bUsedDefaultSceneRoot)
				{
					FKismetEditorUtilities::FAddComponentsToBlueprintParams AddCompParams;
					AddCompParams.bKeepMobility = Params.bKeepMobility;
					AddComponentsToBlueprint(NewBlueprint, Actor->GetInstanceComponents(), AddCompParams);
				}
			}

			if (NewBlueprint->GeneratedClass != nullptr)
			{
				AActor* CDO = CastChecked<AActor>(NewBlueprint->GeneratedClass->GetDefaultObject());
				const EditorUtilities::ECopyOptions::Type CopyOptionFlags = (EditorUtilities::ECopyOptions::Type)(EditorUtilities::ECopyOptions::OnlyCopyEditOrInterpProperties | EditorUtilities::ECopyOptions::PropagateChangesToArchetypeInstances | EditorUtilities::ECopyOptions::SkipInstanceOnlyProperties);
				const EditorUtilities::FCopyOptions CopyOptions(CopyOptionFlags, UE::Private::ExternalDataLayerAssetCopyPropertyFilter); // Don't allow to copy ExternalDataLayerAsset property to NewBlueprint
				EditorUtilities::CopyActorProperties(Actor, CDO, CopyOptions);

				if (USceneComponent* DstSceneRoot = CDO->GetRootComponent())
				{
					FResetSceneComponentAfterCopy::Reset(DstSceneRoot);

					// Copy relative scale from source to target.
					if (USceneComponent* SrcSceneRoot = Actor->GetRootComponent())
					{
						DstSceneRoot->SetRelativeScale3D_Direct(SrcSceneRoot->GetRelativeScale3D());
					}
				}
			}

			if (!Params.bDeferCompilation)
			{
				FKismetEditorUtilities::CompileBlueprint(NewBlueprint);
			}

			if (Params.bReplaceActor)
			{
				TArray<AActor*> Actors;
				Actors.Add(Actor);

				FVector Location = Actor->GetActorLocation();
				FRotator Rotator = Actor->GetActorRotation();

				AActor* NewActor = CreateBlueprintInstanceFromSelection(NewBlueprint, Actors, Location, Rotator, Actor->GetAttachParentActor());
				if (NewActor)
				{
					NewActor->SetActorScale3D(Actor->GetActorScale3D());
				}
			}
		}
	}

	if (NewBlueprint && Params.bOpenBlueprint)
	{
		// Open the editor for the new blueprint
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(NewBlueprint);
	}
	return NewBlueprint;
}

struct FBlueprintAssemblyProps
{
	FBlueprintAssemblyProps(UBlueprint* InBlueprint, const TArray<AActor*>& InActors)
		: Blueprint(InBlueprint)
		, Actors(InActors)
	{
	}

	UBlueprint* Blueprint; 
	const TArray<AActor*>& Actors;
	TArray<AActor*> RootActors;
	TArray<USCS_Node*>* OutNodes = nullptr;
	TMap<AActor*, TArray<AActor*>> AttachmentMap;
	USCS_Node* RootNodeOverride = nullptr;
};

void FKismetEditorUtilities::IdentifyRootActors(const TArray<AActor*>& Actors, TArray<AActor*>& RootActors, TMap<AActor*, TArray<AActor*>>* AttachmentMap)
{
	RootActors = Actors;

	int32 ConsideringActorIndex = 0;
	while (ConsideringActorIndex < RootActors.Num())
	{
		AActor* ConsideringActor = RootActors[ConsideringActorIndex];
		bool bIsRoot = true;
		for (int32 CheckIndex = RootActors.Num()-1; CheckIndex > ConsideringActorIndex; --CheckIndex)
		{
			AActor* ActorToCheck = RootActors[CheckIndex];
			if (ActorToCheck->IsAttachedTo(ConsideringActor))
			{
				if (AttachmentMap)
				{
					AttachmentMap->FindOrAdd(ConsideringActor).Add(ActorToCheck);
				}
				RootActors.RemoveAtSwap(CheckIndex);
			}
			else if (ConsideringActor->IsAttachedTo(ActorToCheck))
			{
				if (AttachmentMap)
				{
					AttachmentMap->FindOrAdd(ActorToCheck).Add(ConsideringActor);
				}
				bIsRoot = false;
				break;
			}
		}

		if (bIsRoot)
		{
			++ConsideringActorIndex;
		}
		else
		{
			RootActors.RemoveAtSwap(ConsideringActorIndex);
		}
	}
}

// Reposition nodes to recenter them around the new pivot
void RepositionNodes(const TArray<USCS_Node*>& Nodes, const FTransform& Pivot)
{
	for (USCS_Node* Node : Nodes)
	{
		if (USceneComponent* NodeTemplate = Cast<USceneComponent>(Node->ComponentTemplate))
		{
			//The relative transform for those component was converted into the world space
			const FTransform NewRelativeTransform = NodeTemplate->GetRelativeTransform().GetRelativeTransform(Pivot);
			NodeTemplate->SetRelativeTransform_Direct(NewRelativeTransform);
		}
	}
};

void CreateBlueprintFromActors_Internal(UBlueprint* Blueprint, const TArray<AActor*>& Actors, const bool bReplaceInWorld, TFunctionRef<void(const FBlueprintAssemblyProps&)> AssembleBlueprintFunc)
{
	check(Actors.Num());
	check(Blueprint);
	check(Blueprint->SimpleConstructionScript);
	USimpleConstructionScript* SCS = Blueprint->SimpleConstructionScript;

	FBlueprintAssemblyProps AssemblyProps(Blueprint, Actors);

	TMap<AActor*, TArray<AActor*>>& AttachmentMap = AssemblyProps.AttachmentMap;
	TArray<AActor*>& RootActors = AssemblyProps.RootActors;
	
	FKismetEditorUtilities::IdentifyRootActors(Actors, RootActors, &AttachmentMap);

	FTransform NewActorTransform = FTransform::Identity;
	bool bRepositionTopLevelNodes = false;

	// If the new blueprint already has a root node, then we will use it, but that will require adjusting
	// the positions of the SCS's root nodes as an inherited root is not in this blueprint's SCS
	if (SCS->GetSceneRootComponentTemplate() == nullptr)
	{
		// If there is not one unique root actor then create a new scene component to serve as the shared root node
		if (RootActors.Num() != 1)
		{
			AssemblyProps.RootNodeOverride = SCS->CreateNode(USceneComponent::StaticClass(), TEXT("SharedRoot"));
			SCS->AddNode(AssemblyProps.RootNodeOverride);
		}
	}
	else
	{
		bRepositionTopLevelNodes = true;
	}

	AssembleBlueprintFunc(AssemblyProps);

	if (RootActors.Num() == 1)
	{
		NewActorTransform = RootActors[0]->GetTransform();
		if (bRepositionTopLevelNodes)
		{
			RepositionNodes(SCS->GetRootNodes(), NewActorTransform);
		}
	}
	else if (RootActors.Num() > 1)
	{
		// Compute the average origin for all the actors, so it can be backed out when saving them in the blueprint
		{
			// Find average location of all selected actors
			FVector AverageLocation = FVector::ZeroVector;
			for (const AActor* Actor : RootActors)
			{
				if (USceneComponent* RootComponent = Actor->GetRootComponent())
				{
					AverageLocation += Actor->GetActorLocation();
				}
			}
			AverageLocation /= (float)RootActors.Num();

			// Spawn the new BP at that location
			NewActorTransform.SetTranslation(AverageLocation);
		}

		if (bRepositionTopLevelNodes)
		{
			RepositionNodes(SCS->GetRootNodes(), NewActorTransform);
		}
		else
		{
			for (USCS_Node* TopLevelNode : SCS->GetRootNodes())
			{
				if (USceneComponent* TestRoot = Cast<USceneComponent>(TopLevelNode->ComponentTemplate))
				{
					RepositionNodes(TopLevelNode->GetChildNodes(), NewActorTransform);
				}
			}
		}
	}

	FKismetEditorUtilities::CompileBlueprint(Blueprint);

	// Notify the asset registry
	FAssetRegistryModule::AssetCreated(Blueprint);

	// Mark the package dirty
	Blueprint->GetOutermost()->MarkPackageDirty();

	// Delete the old actors and create a new instance in the map
	if (bReplaceInWorld)
	{
		FVector Location = NewActorTransform.GetLocation();
		FRotator Rotator = NewActorTransform.Rotator();

		AActor* NewAttachParent = nullptr;
		if (RootActors.Num() == 1)
		{
			NewAttachParent = RootActors[0]->GetAttachParentActor();
		}
		else if (RootActors.Num() > 1)
		{
			TArray<AActor*> PossibleCommonAttachParents;

			// Using the first actor, gather the hierarchy of actors it is attached to starting with the direct attachment
			AActor* AttachParent = RootActors[0]->GetAttachParentActor();
			while (AttachParent)
			{
				PossibleCommonAttachParents.Add(AttachParent);
				AttachParent = AttachParent->GetAttachParentActor();
			}

			// For the remaining root actors, evaluate which, if any, of the first actor's parents they are also attached to
			// Whatever is left in the 0 index of the possible list is the first common ancestor
			for (int32 RootActorIndex = 1; PossibleCommonAttachParents.Num() > 0 && RootActorIndex < RootActors.Num(); ++RootActorIndex)
			{
				AActor* RootActor = RootActors[RootActorIndex];
				for (int32 PossibleIndex = PossibleCommonAttachParents.Num() - 1; PossibleIndex >= 0; --PossibleIndex)
				{
					if (!RootActor->IsAttachedTo(PossibleCommonAttachParents[PossibleIndex]))
					{
						// If we're not attached to a given actor, then we can't possibly also be attached to its children, so clear all of them out
						PossibleCommonAttachParents.RemoveAt(0, PossibleIndex + 1, EAllowShrinking::No);
						break;
					}
				}
			}

			if (PossibleCommonAttachParents.Num() > 0)
			{
				NewAttachParent = PossibleCommonAttachParents[0];
			}
		}

		// Get all the actors attached directly attached to actors we're about to destroy and replace and attach them to the new actor
		TArray<AActor*> AttachedActors;
		for (AActor* Actor : Actors)
		{
			Actor->GetAttachedActors(AttachedActors, false);
		}
		for (int32 Index = AttachedActors.Num() - 1; Index >= 0; --Index)
		{
			// Remove attached actors that are also in the set of actors being converted to blueprint
			if (Actors.Contains(AttachedActors[Index]))
			{
				AttachedActors.RemoveAtSwap(Index, EAllowShrinking::No);
			}
		}

		AActor* NewActor = FKismetEditorUtilities::CreateBlueprintInstanceFromSelection(Blueprint, Actors, Location, Rotator, NewAttachParent);
		if (NewActor)
		{
			NewActor->SetActorScale3D(NewActorTransform.GetScale3D());

			for (AActor* AttachedActor : AttachedActors)
			{
				AttachedActor->AttachToActor(NewActor, FAttachmentTransformRules::KeepWorldTransform);
			}
		}
	}
}

void CreateChildActorComponentsForActors(const FBlueprintAssemblyProps& AssemblyProps)
{
	USimpleConstructionScript* SCS = AssemblyProps.Blueprint->SimpleConstructionScript;

	TArray<UActorComponent*> ChildActorComponents;

	EComponentMobility::Type RootMobility = EComponentMobility::Static;
	if (USceneComponent* RootComp = SCS->GetSceneRootComponentTemplate(false))
	{
		RootMobility = RootComp->Mobility;
	}

	TFunction<void(AActor*, UChildActorComponent*)> RecursiveCreateChildActorTemplates = [SCS, RootMobility, &ChildActorComponents, &AssemblyProps, &RecursiveCreateChildActorTemplates](AActor* Actor, UChildActorComponent* ParentComponent)
	{
		const FName ComponentName = SCS->GenerateNewComponentName(UChildActorComponent::StaticClass(), Actor->GetFName());
		UChildActorComponent* CAC = NewObject<UChildActorComponent>(GetTransientPackage(), ComponentName, RF_ArchetypeObject);
		CAC->SetChildActorClass(Actor->GetClass(), Actor);

		if (USceneComponent* ActorRootComp = Actor->GetRootComponent())
		{
			CAC->SetMobility(FMath::Max(ActorRootComp->Mobility.GetValue(), RootMobility));
		}

		// Clear any properties that can't be on the template
		if (USceneComponent* RootComponent = CAC->GetChildActorTemplate()->GetRootComponent())
		{
			RootComponent->SetRelativeLocation_Direct(FVector::ZeroVector);
			RootComponent->SetRelativeRotation_Direct(FRotator::ZeroRotator);
			RootComponent->SetRelativeScale3D_Direct(FVector::OneVector);
		}

		CAC->SetWorldTransform(Actor->GetTransform());
		if (ParentComponent)
		{
			CAC->AttachToComponent(ParentComponent, FAttachmentTransformRules::KeepWorldTransform);
		}

		ChildActorComponents.Add(CAC);

		if (const TArray<AActor*>* ChildActors = AssemblyProps.AttachmentMap.Find(Actor))
		{
			for (AActor* ChildActor : *ChildActors)
			{
				RecursiveCreateChildActorTemplates(ChildActor, CAC);
			}
		}
	};

	for (AActor* Actor : AssemblyProps.RootActors)
	{
		RecursiveCreateChildActorTemplates(Actor, nullptr);
	}

	FKismetEditorUtilities::FAddComponentsToBlueprintParams Params;
	Params.HarvestMode = FKismetEditorUtilities::EAddComponentToBPHarvestMode::Harvest_UseComponentName;
	Params.OptionalNewRootNode = AssemblyProps.RootNodeOverride;
	Params.bKeepMobility = true;
	Params.OutNodes = AssemblyProps.OutNodes;
	FKismetEditorUtilities::AddComponentsToBlueprint(AssemblyProps.Blueprint, ChildActorComponents, Params);

	// Since the names we create are well defined relative to the SCS but created in the transient package, we could end up reusing objects
	// unless we rename these temporary components out of the way
	for (UActorComponent* CAC : ChildActorComponents)
	{
		CAC->Rename(nullptr, nullptr, REN_DoNotDirty | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
	}
}

UBlueprint* FKismetEditorUtilities::CreateBlueprintFromActors(const FName BlueprintName, UPackage* Package, const FKismetEditorUtilities::FCreateBlueprintFromActorsParams& Params)
{
	UBlueprint* NewBlueprint = nullptr;

	if (AActor* RootActor = Params.RootActor)
	{
		FCreateBlueprintFromActorParams CreateActorParams;
		CreateActorParams.bReplaceActor = false;
		CreateActorParams.bDeferCompilation = true;
		CreateActorParams.bOpenBlueprint = false;

		NewBlueprint = FKismetEditorUtilities::CreateBlueprintFromActor(BlueprintName, Package, RootActor, CreateActorParams);

		if (NewBlueprint)
		{
			FKismetEditorUtilities::FAddActorsToBlueprintParams AddActorsParams;
			AddActorsParams.bReplaceActors = false;
			AddActorsParams.RelativeToInstance = RootActor;

			FKismetEditorUtilities::AddActorsToBlueprint(NewBlueprint, Params.AdditionalActors, AddActorsParams);

			if (Params.bReplaceActors)
			{
				FVector Location = RootActor->GetActorLocation();
				FRotator Rotator = RootActor->GetActorRotation();

				TArray<AActor*> Actors;
				Actors.Reserve(Params.AdditionalActors.Num() + 1);
				Actors.Add(RootActor);
				Actors.Append(Params.AdditionalActors);

				AActor* NewActor = FKismetEditorUtilities::CreateBlueprintInstanceFromSelection(NewBlueprint, Actors, Location, Rotator, RootActor->GetAttachParentActor());
				if (NewActor)
				{
					NewActor->SetActorScale3D(RootActor->GetActorScale3D());
				}
			}
		}
	}
	else if (Params.AdditionalActors.Num() > 0)
	{
		if (Package)
		{
			// We don't have a factory, but we can still try to create a blueprint for this actor class
			NewBlueprint = FKismetEditorUtilities::CreateBlueprint(Params.ParentClass, Package, BlueprintName, EBlueprintType::BPTYPE_Normal, UBlueprint::StaticClass(), UBlueprintGeneratedClass::StaticClass(), FName("CreateFromActors"));
		}

		if (NewBlueprint)
		{
			CreateBlueprintFromActors_Internal(NewBlueprint, Params.AdditionalActors, Params.bReplaceActors, &CreateChildActorComponentsForActors);
		}
	}

	if (Params.bOpenBlueprint && NewBlueprint)
	{
		// Open the editor for the new blueprint
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(NewBlueprint);
	}

	return NewBlueprint;
}

void FKismetEditorUtilities::AddActorsToBlueprint(UBlueprint* Blueprint, const TArray<AActor*>& Actors, const FKismetEditorUtilities::FAddActorsToBlueprintParams& Params)
{
	// Create the ChildActorComponents
	FBlueprintAssemblyProps AssemblyProps(Blueprint, Actors);
	if (Params.AttachNode && ensureMsgf(Params.AttachNode->GetSCS() == Blueprint->SimpleConstructionScript, TEXT("Invalid AttachNode supplied, attaching to root")))
	{
		AssemblyProps.RootNodeOverride = Params.AttachNode;
	}
	else
	{
		Blueprint->SimpleConstructionScript->GetSceneRootComponentTemplate(false, &AssemblyProps.RootNodeOverride);
	}
	FKismetEditorUtilities::IdentifyRootActors(Actors, AssemblyProps.RootActors, &AssemblyProps.AttachmentMap);

	TArray<USCS_Node*> ComponentNodes;
	AssemblyProps.OutNodes = &ComponentNodes;
	CreateChildActorComponentsForActors(AssemblyProps);

	FTransform Pivot;
	if (Params.RelativeToInstance)
	{
		Pivot = Params.RelativeToInstance->GetTransform();
	}
	Pivot *= Params.RelativeToTransform;

	RepositionNodes(ComponentNodes, Pivot);

	if (!Params.bDeferCompilation)
	{
		FKismetEditorUtilities::CompileBlueprint(Blueprint);
	}

	if (Params.bReplaceActors)
	{
		bool bModifiedSelectedActors = false;

		ULayersSubsystem* Layers = GEditor->GetEditorSubsystem<ULayersSubsystem>();
		for (AActor* Actor : Actors)
		{
			if (Actor)
			{
				if (Actor->IsSelectedInEditor())
				{
					if (!bModifiedSelectedActors)
					{
						GEditor->GetSelectedActors()->Modify();
						bModifiedSelectedActors = true;
					}

					// Remove from active selection in editor
					GEditor->SelectActor(Actor, /*bSelected=*/ false, /*bNotify=*/ false);
				}

				Layers->DisassociateActorFromLayers(Actor);
				Actor->GetWorld()->EditorDestroyActor(Actor, false);
			}
		}
	}
}

UBlueprint* FKismetEditorUtilities::HarvestBlueprintFromActors(const FString& Path, const TArray<AActor*>& Actors, const FKismetEditorUtilities::FHarvestBlueprintFromActorsParams& Params)
{
	UBlueprint* NewBlueprint = nullptr;
	FString AssetName;

	if (UPackage* Package = CreateBlueprintPackage(Path, AssetName))
	{
		NewBlueprint = HarvestBlueprintFromActors(FName(*AssetName), Package, Actors, Params);
	}

	return NewBlueprint;
}

UBlueprint* FKismetEditorUtilities::HarvestBlueprintFromActors(const FName BlueprintName, UPackage* Package, const TArray<AActor*>& Actors, const FKismetEditorUtilities::FHarvestBlueprintFromActorsParams& Params)
{
	auto AssemblyFunction = [](const FBlueprintAssemblyProps& AssemblyProps)
	{
		// Harvest the components from each actor and clone them into the SCS
		TArray<UActorComponent*> AllSelectedComponents;
		for (const AActor* Actor : AssemblyProps.Actors)
		{
			for (UActorComponent* ComponentToConsider : Actor->GetComponents())
			{
				if (ComponentToConsider && !ComponentToConsider->IsVisualizationComponent())
				{
					AllSelectedComponents.Add(ComponentToConsider);
				}
			}
		}

		TArray<TPair<USceneComponent*, FTransform>> SceneComponentOldRelativeTransforms;
		if (AssemblyProps.RootActors.Num() > 1)
		{
			SceneComponentOldRelativeTransforms.Reserve(AssemblyProps.RootActors.Num());
			// Convert the components relative transform to world 
			for (AActor* Actor : AssemblyProps.RootActors)
			{
				USceneComponent* SceneComponent = Actor->GetRootComponent();
				SceneComponentOldRelativeTransforms.Emplace(SceneComponent, SceneComponent->GetRelativeTransform());
				SceneComponent->SetRelativeTransform_Direct(SceneComponent->GetComponentTransform());
			}
		}

		FKismetEditorUtilities::FAddComponentsToBlueprintParams AddComponentsParams;
		AddComponentsParams.HarvestMode = (AssemblyProps.Actors.Num() > 1 ? FKismetEditorUtilities::EAddComponentToBPHarvestMode::Havest_AppendOwnerName : FKismetEditorUtilities::EAddComponentToBPHarvestMode::Harvest_UseComponentName);
		AddComponentsParams.OptionalNewRootNode = AssemblyProps.RootNodeOverride;

		FKismetEditorUtilities::AddComponentsToBlueprint(AssemblyProps.Blueprint, AllSelectedComponents, AddComponentsParams);

		// Replace the modified components to their relative transform
		for (const TPair<USceneComponent*, FTransform >& Pair : SceneComponentOldRelativeTransforms)
		{
			Pair.Key->SetRelativeTransform_Direct(Pair.Value);
		}
	};

	UBlueprint* Blueprint = nullptr;

	if (Actors.Num() > 0)
	{
		if (Package != nullptr)
		{
			// We don't have a factory, but we can still try to create a blueprint for this actor class
			Blueprint = FKismetEditorUtilities::CreateBlueprint(Params.ParentClass, Package, BlueprintName, EBlueprintType::BPTYPE_Normal, UBlueprint::StaticClass(), UBlueprintGeneratedClass::StaticClass(), FName("HarvestFromActors"));
		}

		if (Blueprint != nullptr)
		{
			CreateBlueprintFromActors_Internal(Blueprint, Actors, Params.bReplaceActors, AssemblyFunction);

			// Open the editor for the new blueprint
			if (Params.bOpenBlueprint)
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Blueprint);
			}
		}
	}

	return Blueprint;
}


/**
This struct saves and deselects all selected instanced components (from given actor), then finds them (in recreated actor instance, after compilation) and selects them again.
*/
struct FRestoreSelectedInstanceComponent
{
	TWeakObjectPtr<UClass> ActorClass;
	FName ActorName;
	TWeakObjectPtr<UObject> ActorOuter;

	struct FComponentKey
	{
		FName Name;
		TWeakObjectPtr<UClass> Class;

		FComponentKey(FName InName, UClass* InClass) : Name(InName), Class(InClass) {}
	};
	TArray<FComponentKey> ComponentKeys;

	FRestoreSelectedInstanceComponent()
		: ActorClass(nullptr)
		, ActorOuter(nullptr)
	{ }

	void Save(AActor* InActor)
	{
		check(InActor);
		ActorClass = InActor->GetClass();
		ActorName = InActor->GetFName();
		ActorOuter = InActor->GetOuter();

		check(GEditor);
		TArray<UActorComponent*> ComponentsToSaveAndDelesect;
		for (auto Iter = GEditor->GetSelectedComponentIterator(); Iter; ++Iter)
		{
			UActorComponent* Component = CastChecked<UActorComponent>(*Iter, ECastCheckedType::NullAllowed);
			if (Component && InActor->GetInstanceComponents().Contains(Component))
			{
				ComponentsToSaveAndDelesect.Add(Component);
			}
		}

		for (UActorComponent* Component : ComponentsToSaveAndDelesect)
		{
			USelection* SelectedComponents = GEditor->GetSelectedComponents();
			if (ensure(SelectedComponents))
			{
				ComponentKeys.Add(FComponentKey(Component->GetFName(), Component->GetClass()));
				SelectedComponents->Deselect(Component);
			}
		}
	}

	void Restore()
	{
		AActor* Actor = (ActorClass.IsValid() && ActorOuter.IsValid())
			? Cast<AActor>((UObject*)FindObjectWithOuter(ActorOuter.Get(), ActorClass.Get(), ActorName))
			: nullptr;
		if (Actor)
		{
			for (const FComponentKey& IterKey : ComponentKeys)
			{
				UActorComponent* const* ComponentPtr = Algo::FindByPredicate(Actor->GetComponents(), [&](UActorComponent* InComp)
				{
					return InComp && (InComp->GetFName() == IterKey.Name) && (InComp->GetClass() == IterKey.Class.Get());
				});
				if (ComponentPtr && *ComponentPtr)
				{
					check(GEditor);
					GEditor->SelectComponent(*ComponentPtr, true, false);
				}
			}
		}
	}
};

int32 FKismetEditorUtilities::ApplyInstanceChangesToBlueprint(AActor* Actor)
{
	int32 NumChangedProperties = 0;

	UBlueprint* const Blueprint = (Actor != nullptr) ? Cast<UBlueprint>(Actor->GetClass()->ClassGeneratedBy) : nullptr;
	if (Actor != nullptr && Blueprint != nullptr)
	{
		// Cache the actor label as by the time we need it, it may be invalid
		const FString ActorLabel = Actor->GetActorLabel();
		FRestoreSelectedInstanceComponent RestoreSelectedInstanceComponent;
		{
			const FScopedTransaction Transaction(LOCTEXT("PushToBlueprintDefaults_Transaction", "Apply Changes to Blueprint"));

			// The component selection state should be maintained
			GEditor->GetSelectedActors()->Modify();
			GEditor->GetSelectedComponents()->Modify();

			Actor->Modify();

			// Mark components that are either native or from the SCS as modified so they will be restored
			for (UActorComponent* ActorComponent : Actor->GetComponents())
			{
				if (ActorComponent && (ActorComponent->CreationMethod == EComponentCreationMethod::SimpleConstructionScript || ActorComponent->CreationMethod == EComponentCreationMethod::Native))
				{
					ActorComponent->Modify();
				}
			}

			// Perform the actual copy
			{
				AActor* BlueprintCDO = Actor->GetClass()->GetDefaultObject<AActor>();
				if (BlueprintCDO != NULL)
				{
					const EditorUtilities::ECopyOptions::Type CopyOptionFlags = (EditorUtilities::ECopyOptions::Type)(EditorUtilities::ECopyOptions::OnlyCopyEditOrInterpProperties | EditorUtilities::ECopyOptions::PropagateChangesToArchetypeInstances | EditorUtilities::ECopyOptions::SkipInstanceOnlyProperties);
					const EditorUtilities::FCopyOptions CopyOptions(CopyOptionFlags, UE::Private::ExternalDataLayerAssetCopyPropertyFilter); // Don't allow to copy ExternalDataLayerAsset property to BlueprintCDO
					NumChangedProperties = EditorUtilities::CopyActorProperties(Actor, BlueprintCDO, CopyOptions);
					if (Actor->GetInstanceComponents().Num() > 0)
					{
						RestoreSelectedInstanceComponent.Save(Actor);
						FKismetEditorUtilities::AddComponentsToBlueprint(Blueprint, Actor->GetInstanceComponents());
						NumChangedProperties += Actor->GetInstanceComponents().Num();
						Actor->ClearInstanceComponents(true);
					}
					if (NumChangedProperties > 0)
					{
						TArray<AActor*> Actors;
						Actors.Add(Actor);

						FVector Location = Actor->GetActorLocation();
						FRotator Rotator = Actor->GetActorRotation();

						AActor* NewActor = CreateBlueprintInstanceFromSelection(Blueprint, Actors, Location, Rotator, Actor->GetAttachParentActor());
						if (NewActor)
						{
							NewActor->SetActorScale3D(Actor->GetActorScale3D());
						}
					}
				}
			}
			
			if (NumChangedProperties > 0)
			{
				FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
				FKismetEditorUtilities::CompileBlueprint(Blueprint);
				RestoreSelectedInstanceComponent.Restore();
			}
		}
	}

	return NumChangedProperties;
}


AActor* FKismetEditorUtilities::CreateBlueprintInstanceFromSelection(UBlueprint* Blueprint, const TArray<AActor*>& SelectedActors, const FVector& Location, const FRotator& Rotator, AActor* AttachParent)
{
	check (SelectedActors.Num() > 0 );

	// Create transaction to cover conversion
	const FScopedTransaction Transaction( NSLOCTEXT("EditorEngine", "ConvertActorToBlueprint", "Replace Actor(s) with blueprint") );

	// Assume all selected actors are in the same world
	UWorld* World = SelectedActors[0]->GetWorld();

	GEditor->GetSelectedActors()->Modify();

	ULayersSubsystem* Layers = GEditor->GetEditorSubsystem<ULayersSubsystem>();
	for (AActor* Actor : SelectedActors)
	{
		if (Actor)
		{
			// Remove from active selection in editor
			GEditor->SelectActor(Actor, /*bSelected=*/ false, /*bNotify=*/ false);

			Layers->DisassociateActorFromLayers(Actor);
			World->EditorDestroyActor(Actor, false);
		}
	}

	AActor* NewActor = World->SpawnActor(Blueprint->GeneratedClass, &Location, &Rotator);
	Layers->InitializeNewActorLayers(NewActor);
	FActorLabelUtilities::SetActorLabelUnique(NewActor, Blueprint->GetName());

	if (AttachParent)
	{
		NewActor->AttachToActor(AttachParent, FAttachmentTransformRules::KeepWorldTransform);
	}

	// Quietly ensure that no components are selected
	USelection* ComponentSelection = GEditor->GetSelectedComponents();
	ComponentSelection->BeginBatchSelectOperation();
	ComponentSelection->DeselectAll();
	ComponentSelection->EndBatchSelectOperation(false);

	// Update selection to new actor
	GEditor->SelectActor( NewActor, /*bSelected=*/ true, /*bNotify=*/ true );

	return NewActor;
}

UBlueprint* FKismetEditorUtilities::CreateBlueprintFromClass(FText InWindowTitle, UClass* InParentClass, FString NewNameSuggestion)
{
	check(FKismetEditorUtilities::CanCreateBlueprintOfClass(InParentClass));

	// Pre-generate a unique asset name to fill out the path picker dialog with.
	if (NewNameSuggestion.Len() == 0)
	{
		NewNameSuggestion = TEXT("NewBlueprint");
	}

	UClass* BlueprintClass = nullptr;
	UClass* BlueprintGeneratedClass = nullptr;

	IKismetCompilerInterface& KismetCompilerModule = FModuleManager::LoadModuleChecked<IKismetCompilerInterface>("KismetCompiler");
	KismetCompilerModule.GetBlueprintTypesForClass(InParentClass, BlueprintClass, BlueprintGeneratedClass);

	FString PackageName = FString(TEXT("/Game/Blueprints/")) + NewNameSuggestion;
	FString Name;
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	AssetToolsModule.Get().CreateUniqueAssetName(PackageName, TEXT(""), PackageName, Name);

	TSharedPtr<SDlgPickAssetPath> PickAssetPathWidget =
		SNew(SDlgPickAssetPath)
		.Title(InWindowTitle)
		.DefaultAssetPath(FText::FromString(PackageName));

	if (EAppReturnType::Ok == PickAssetPathWidget->ShowModal())
	{
		// Get the full name of where we want to create the physics asset.
		FString UserPackageName = PickAssetPathWidget->GetFullAssetPath().ToString();
		FName BPName(*FPackageName::GetLongPackageAssetName(UserPackageName));

		// Check if the user inputed a valid asset name, if they did not, give it the generated default name
		if (BPName == NAME_None)
		{
			// Use the defaults that were already generated.
			UserPackageName = PackageName;
			BPName = *Name;
		}

		// Then find/create it.
		UPackage* Package = CreatePackage( *UserPackageName);
		check(Package);

		// Create and init a new Blueprint
		UBlueprint* Blueprint = FKismetEditorUtilities::CreateBlueprint(InParentClass, Package, BPName, BPTYPE_Normal, BlueprintClass, BlueprintGeneratedClass, FName("LevelEditorActions"));
		if (Blueprint)
		{
			// Notify the asset registry
			FAssetRegistryModule::AssetCreated(Blueprint);

			// Mark the package dirty...
			Package->MarkPackageDirty();

			return Blueprint;
		}
	}
	return NULL;
}

UBlueprint* FKismetEditorUtilities::CreateBlueprintUsingAsset(UObject* Asset, bool bOpenInEditor)
{
	// Check we have an asset.
	if(Asset == NULL)
	{
		return NULL;
	}

	// Check we can create a component from this asset
	TSubclassOf<UActorComponent> ComponentClass = FComponentAssetBrokerage::GetPrimaryComponentForAsset(Asset->GetClass());
	if(ComponentClass != NULL)
	{
		// Create a new empty Actor BP
		UBlueprint* NewBP = CreateBlueprintFromClass(LOCTEXT("CreateBlueprint", "Create Blueprint"), AActor::StaticClass(), Asset->GetName());
		if(NewBP != NULL)
		{
			// Create a new SCS node
			check(NewBP->SimpleConstructionScript != NULL);
			USCS_Node* NewNode = NewBP->SimpleConstructionScript->CreateNode(ComponentClass);

			// Assign the asset to the template
			FComponentAssetBrokerage::AssignAssetToComponent(NewNode->ComponentTemplate, Asset);

			// Add node to the SCS
			NewBP->SimpleConstructionScript->AddNode(NewNode);

			// Recompile skeleton because of the new component we added (and 
			// broadcast the change to those that care, like the BP node database)
			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(NewBP);

			// Open in BP editor if desired
			if(bOpenInEditor)
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(NewBP);
			}
		}

		return NewBP;
	}

	return NULL;
}

void FKismetEditorUtilities::AddToSelection(const class UEdGraph* Graph, UEdGraphNode* InNode)
{
	TSharedPtr<class IBlueprintEditor> BlueprintEditor = GetIBlueprintEditorForObject(Graph, false);
	if (BlueprintEditor.IsValid())
	{
		BlueprintEditor->AddToSelection(InNode);
	}
}

TSharedPtr<class IBlueprintEditor> FKismetEditorUtilities::GetIBlueprintEditorForObject( const UObject* ObjectToFocusOn, bool bOpenEditor )
{
	check(ObjectToFocusOn);

	// Find the associated blueprint
	UBlueprint* TargetBP = nullptr;
	for (UObject* TestObject = const_cast<UObject*>(ObjectToFocusOn); (TestObject != nullptr) && (TargetBP == nullptr); TestObject = TestObject->GetOuter())
	{
		if (UBlueprintGeneratedClass* BPGeneratedClass = Cast<UBlueprintGeneratedClass>(TestObject))
		{
			TargetBP = Cast<UBlueprint>(BPGeneratedClass->ClassGeneratedBy);
		}
		else
		{
			TargetBP = Cast<UBlueprint>(TestObject);
		}
	}

	TSharedPtr<IBlueprintEditor> BlueprintEditor;
	if (TargetBP != nullptr)
	{
		if (bOpenEditor)
		{
			// @todo toolkit major: Needs world-centric support
			 GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(TargetBP);
		}

		TSharedPtr< IToolkit > FoundAssetEditor = FToolkitManager::Get().FindEditorForAsset(TargetBP);
		// If we found a BlueprintEditor
		if (FoundAssetEditor.IsValid() && FoundAssetEditor->IsBlueprintEditor())
		{
			BlueprintEditor = StaticCastSharedPtr<IBlueprintEditor>(FoundAssetEditor);
		}
	}
	return BlueprintEditor;
}

void FKismetEditorUtilities::PasteNodesHere( class UEdGraph* Graph, const UE::Slate::FDeprecateVector2DParameter& Location )
{
	TSharedPtr<class IBlueprintEditor> Kismet = GetIBlueprintEditorForObject(Graph,false);
	if(Kismet.IsValid())
	{
		Kismet->PasteNodesHere(Graph,Location);
	}
}

bool FKismetEditorUtilities::CanPasteNodes( const class UEdGraph* Graph )
{
	bool bCanPaste = false;
	TSharedPtr<class IBlueprintEditor> Kismet = GetIBlueprintEditorForObject(Graph,false);
	if (Kismet.IsValid())
	{
		bCanPaste = Kismet->CanPasteNodes();
	}
	return bCanPaste;
}

bool FKismetEditorUtilities::GetBoundsForSelectedNodes(const class UBlueprint* Blueprint,  class FSlateRect& Rect, float Padding)
{
	bool bCanPaste = false;
	TSharedPtr<class IBlueprintEditor> Kismet = GetIBlueprintEditorForObject(Blueprint, false);
	if (Kismet.IsValid())
	{
		bCanPaste = Kismet->GetBoundsForSelectedNodes(Rect, Padding);
	}
	return bCanPaste;
}

int32 FKismetEditorUtilities::GetNumberOfSelectedNodes(const class UBlueprint* Blueprint)
{
	int32 NumberNodesSelected = 0;
	TSharedPtr<class IBlueprintEditor> Kismet = GetIBlueprintEditorForObject(Blueprint, false);
	if (Kismet.IsValid())
	{
		NumberNodesSelected = Kismet->GetNumberOfSelectedNodes();
	}
	return NumberNodesSelected;
}

/** Open a Kismet window, focusing on the specified object (either a pin, a node, or a graph).  Prefers existing windows, but will open a new application if required. */
void FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(const UObject* ObjectToFocusOn, bool bRequestRename)
{
	TSharedPtr<IBlueprintEditor> BlueprintEditor = GetIBlueprintEditorForObject(ObjectToFocusOn, true);
	if (BlueprintEditor.IsValid())
	{
		BlueprintEditor->FocusWindow();
		BlueprintEditor->JumpToHyperlink(ObjectToFocusOn, bRequestRename);
	}
}

void FKismetEditorUtilities::BringKismetToFocusAttentionOnPin(const UEdGraphPin* PinToFocusOn )
{
	TSharedPtr<IBlueprintEditor> BlueprintEditor = GetIBlueprintEditorForObject(PinToFocusOn->GetOwningNode(), true);
	if (BlueprintEditor.IsValid())
	{
		BlueprintEditor->FocusWindow();
		BlueprintEditor->JumpToPin(PinToFocusOn);
	}
}

void FKismetEditorUtilities::ShowActorReferencesInLevelScript(const AActor* Actor)
{
	if (Actor != NULL)
	{
		ULevelScriptBlueprint* LSB = Actor->GetLevel()->GetLevelScriptBlueprint();
		if (LSB != NULL)
		{
			// @todo toolkit major: Needs world-centric support.  Other spots, too?
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(LSB);
			TSharedPtr<IToolkit> FoundAssetEditor = FToolkitManager::Get().FindEditorForAsset(LSB);
			if (FoundAssetEditor.IsValid())
			{
				TSharedRef<IBlueprintEditor> BlueprintEditor = StaticCastSharedRef<IBlueprintEditor>(FoundAssetEditor.ToSharedRef());
				BlueprintEditor->FocusWindow();

				const bool bSetFindWithinBlueprint = true;
				const bool bSelectFirstResult = true;
				BlueprintEditor->SummonSearchUI(bSetFindWithinBlueprint, Actor->GetActorLabel(), bSelectFirstResult);
			}
		}

	}
}

// Upgrade any cosmetically stale information in a blueprint (done when edited instead of PostLoad to make certain operations easier)
void FKismetEditorUtilities::UpgradeCosmeticallyStaleBlueprint(UBlueprint* Blueprint)
{
	// Rename the ubergraph page 'StateGraph' to be named 'EventGraph' if possible
	if (FBlueprintEditorUtils::DoesSupportEventGraphs(Blueprint))
	{
		UEdGraph* OldStateGraph = FindObject<UEdGraph>(Blueprint, TEXT("StateGraph"));
		UObject* CollidingObject = FindObject<UObject>(Blueprint, *(UEdGraphSchema_K2::GN_EventGraph.ToString()));

		if ((OldStateGraph != NULL) && (CollidingObject == NULL))
		{
			check(!OldStateGraph->HasAnyFlags(RF_Public));
			OldStateGraph->Rename(*(UEdGraphSchema_K2::GN_EventGraph.ToString()), OldStateGraph->GetOuter(), REN_DoNotDirty | REN_AllowPackageLinkerMismatch);
			Blueprint->Status = BS_Dirty;
		}
	}
}

void FKismetEditorUtilities::CreateNewBoundEventForActor(AActor* Actor, FName EventName, bool bShouldJumpToNode)
{
	if ((Actor != nullptr) && (EventName != NAME_None))
	{
		// First, find the property we want to bind to
		if (FMulticastDelegateProperty* DelegateProperty = FindFProperty<FMulticastDelegateProperty>(Actor->GetClass(), EventName))
		{
			// Get the correct level script blueprint
			if (ULevelScriptBlueprint* LSB = Actor->GetLevel()->GetLevelScriptBlueprint())
			{
				if (UEdGraph* TargetGraph = LSB->GetLastEditedUberGraph())
				{
					// Figure out a decent place to stick the node
					const FVector2D NewNodePos = TargetGraph->GetGoodPlaceForNewNode();

					// Create a new event node
					UK2Node_ActorBoundEvent* EventNode = FEdGraphSchemaAction_K2NewNode::SpawnNode<UK2Node_ActorBoundEvent>(
						TargetGraph,
						NewNodePos,
						EK2NewNodeFlags::SelectNewNode,
						[Actor, DelegateProperty](UK2Node_ActorBoundEvent* NewInstance)
						{
							NewInstance->InitializeActorBoundEventParams(Actor, DelegateProperty);
						}
					);
					// Finally, bring up kismet and jump to the new node
					if (EventNode && bShouldJumpToNode)
					{
						BringKismetToFocusAttentionOnObject(EventNode);
					}
				}
			}
		}
	}
}

void FKismetEditorUtilities::CreateNewBoundEventForComponent(UObject* Component, FName EventName, UBlueprint* Blueprint, FObjectProperty* ComponentProperty, bool bShouldJumpToNode)
{
	if ( Component != nullptr )
	{
		CreateNewBoundEventForClass(Component->GetClass(), EventName, Blueprint, ComponentProperty, bShouldJumpToNode);
	}
}

void FKismetEditorUtilities::CreateNewBoundEventForClass(UClass* Class, FName EventName, UBlueprint* Blueprint, FObjectProperty* ComponentProperty, bool bShouldJumpToNode)
{
	if ( ( Class != nullptr ) && ( EventName != NAME_None ) && ( Blueprint != nullptr ) && ( ComponentProperty != nullptr ) )
	{
		// First, find the property we want to bind to
		FMulticastDelegateProperty* DelegateProperty = FindFProperty<FMulticastDelegateProperty>(Class, EventName);
		if ( DelegateProperty != nullptr )
		{
			UEdGraph* TargetGraph = Blueprint->GetLastEditedUberGraph();

			if ( TargetGraph != nullptr )
			{
				// Figure out a decent place to stick the node
				const FVector2D NewNodePos = TargetGraph->GetGoodPlaceForNewNode();

				// Create a new event node
				UK2Node_ComponentBoundEvent* EventNode = FEdGraphSchemaAction_K2NewNode::SpawnNode<UK2Node_ComponentBoundEvent>(
					TargetGraph,
					NewNodePos,
					EK2NewNodeFlags::SelectNewNode,
					[ComponentProperty, DelegateProperty](UK2Node_ComponentBoundEvent* NewInstance)
					{
						NewInstance->InitializeComponentBoundEventParams(ComponentProperty, DelegateProperty);
					}
				);

				// Finally, bring up kismet and jump to the new node
				if ( EventNode != nullptr && bShouldJumpToNode)
				{
					BringKismetToFocusAttentionOnObject(EventNode);
				}
			}
		}
	}
}

const UK2Node_ActorBoundEvent* FKismetEditorUtilities::FindBoundEventForActor(AActor const* Actor, FName EventName)
{
	const UK2Node_ActorBoundEvent* Node = NULL;
	if(Actor != NULL && EventName != NAME_None)
	{
		ULevelScriptBlueprint* LSB = Actor->GetLevel()->GetLevelScriptBlueprint(true);
		if(LSB != NULL)
		{
			TArray<UK2Node_ActorBoundEvent*> EventNodes;
			FBlueprintEditorUtils::GetAllNodesOfClass(LSB, EventNodes);
			for(int32 i=0; i<EventNodes.Num(); i++)
			{
				UK2Node_ActorBoundEvent* BoundEvent = EventNodes[i];
				if(BoundEvent->EventOwner == Actor && BoundEvent->DelegatePropertyName == EventName)
				{
					Node = BoundEvent;
					break;
				}
			}
		}
	}
	return Node;
}

const UK2Node_ComponentBoundEvent* FKismetEditorUtilities::FindBoundEventForComponent(const UBlueprint* Blueprint, FName EventName, FName PropertyName)
{
	const UK2Node_ComponentBoundEvent* Node = NULL;
	if ( Blueprint && EventName != NAME_None && PropertyName != NAME_None )
	{
		TArray<UK2Node_ComponentBoundEvent*> EventNodes;
		FBlueprintEditorUtils::GetAllNodesOfClass(Blueprint, EventNodes);
		for ( auto NodeIter = EventNodes.CreateIterator(); NodeIter; ++NodeIter )
		{
			UK2Node_ComponentBoundEvent* BoundEvent = *NodeIter;
			if ( ( BoundEvent->ComponentPropertyName == PropertyName ) && ( BoundEvent->DelegatePropertyName == EventName ) )
			{
				Node = *NodeIter;
				break;
			}
		}
	}
	return Node;
}

void FKismetEditorUtilities::FindAllBoundEventsForComponent(const UBlueprint* Blueprint, FName PropertyName, TArray<UK2Node_ComponentBoundEvent*>& OutNodes)
{
	if (Blueprint && PropertyName != NAME_None)
	{
		TArray<UK2Node_ComponentBoundEvent*> EventNodes;
		FBlueprintEditorUtils::GetAllNodesOfClass(Blueprint, EventNodes);
		for (UK2Node_ComponentBoundEvent* CurNode : EventNodes)
		{
			if (CurNode && CurNode->ComponentPropertyName == PropertyName)
			{
				OutNodes.Add(CurNode);
			}
		}
	}
}

bool FKismetEditorUtilities::PropertyHasBoundEvents(const UBlueprint* Blueprint, FName PropertyName)
{
	TArray<UK2Node_ComponentBoundEvent*> EventNodes;
	FKismetEditorUtilities::FindAllBoundEventsForComponent(Blueprint, PropertyName, EventNodes);
	return EventNodes.Num() > 0;
}

bool FKismetEditorUtilities::IsClassABlueprintInterface(const UClass* Class)
{
	if (Class->HasAnyClassFlags(CLASS_Interface) && !Class->HasAnyClassFlags(CLASS_NewerVersionExists))
	{
		return true;
	}
	return false;
}

bool FKismetEditorUtilities::IsClassABlueprintImplementableInterface(const UClass* Class)
{
	if (!IsClassABlueprintInterface(Class))
	{
		return false;
	}
	
	// First check explicit tags
	if (Class->HasMetaData(FBlueprintMetadata::MD_CannotImplementInterfaceInBlueprint))
	{
		return false;
	}
	if (Class->HasMetaDataHierarchical(FBlueprintMetadata::MD_IsBlueprintBase))
	{
		if (Class->GetBoolMetaDataHierarchical(FBlueprintMetadata::MD_IsBlueprintBase))
		{
			return true;
		}
		else
		{
			return false;
		}
	}
	
	// Unclear, treat it as blueprintable if it has any events as the header parser would complain if they were the wrong type
	for (TFieldIterator<UFunction> FuncIt(Class, EFieldIteratorFlags::IncludeSuper); FuncIt; ++FuncIt)
	{
		UFunction* Function = *FuncIt;
		if (Function->HasAnyFunctionFlags(FUNC_BlueprintEvent))
		{
			return true;
		}
	}

	return false;
}

bool FKismetEditorUtilities::CanBlueprintImplementInterface(UBlueprint const* Blueprint, UClass const* Class)
{
	bool bCanImplementInterface = false;

	// if the class is an actual implementable interface
	if (IsClassABlueprintImplementableInterface(Class))
	{
		bCanImplementInterface = true;

		UClass const* const ParentClass = Blueprint->ParentClass;
		// see if the parent class has any prohibited interfaces
		if ((ParentClass != NULL) && ParentClass->HasMetaData(FBlueprintMetadata::MD_ProhibitedInterfaces))
		{
			FString const& ProhibitedList = Blueprint->ParentClass->GetMetaData(FBlueprintMetadata::MD_ProhibitedInterfaces);
			
			TArray<FString> ProhibitedInterfaceNames;
			ProhibitedList.ParseIntoArray(ProhibitedInterfaceNames, TEXT(","), true);

			FString const& InterfaceName = Class->GetName();
			// loop over all the prohibited interfaces
			for (int32 ExclusionIndex = 0; ExclusionIndex < ProhibitedInterfaceNames.Num(); ++ExclusionIndex)
			{
				ProhibitedInterfaceNames[ExclusionIndex].TrimStartInline();
				FString const& Exclusion = ProhibitedInterfaceNames[ExclusionIndex];
				// if this interface matches one of the prohibited ones
				if (InterfaceName == Exclusion) 
				{
					bCanImplementInterface = false;
					break;
				}
			}
		}
	}

	return bCanImplementInterface;
}

bool FKismetEditorUtilities::IsClassABlueprintSpawnableComponent(const UClass* Class)
{
	// @fixme: Cooked packages don't have any metadata (yet; they might become available via the sidecar editor data)
	// However, all uncooked BPs that derive from ActorComponent have the BlueprintSpawnableComponent metadata set on them
	// (see FBlueprintEditorUtils::RecreateClassMetaData), so include any ActorComponent BP that comes from a cooked package
	return (!Class->HasAnyClassFlags(CLASS_Abstract) &&
			Class->IsChildOf<UActorComponent>() &&
			(Class->HasMetaData(FBlueprintMetadata::MD_BlueprintSpawnableComponent) || Class->GetPackage()->bIsCookedForEditor));
}

bool FKismetEditorUtilities::IsClassABlueprintSkeleton(const UClass* Class)
{
	// Find generating blueprint for a class
	UBlueprint* GeneratingBP = Cast<UBlueprint>(Class->ClassGeneratedBy);
	if( GeneratingBP && GeneratingBP->SkeletonGeneratedClass )
	{
		return (Class == GeneratingBP->SkeletonGeneratedClass) && (GeneratingBP->SkeletonGeneratedClass != GeneratingBP->GeneratedClass);
	}
	return Class->HasAnyFlags(RF_Transient) && Class->HasAnyClassFlags(CLASS_CompiledFromBlueprint);
}

bool FKismetEditorUtilities::IsClassABlueprintMacroLibrary(const UClass* Class)
{
	// Find generating blueprint for a class
	UBlueprint* GeneratingBP = Cast<UBlueprint>(Class->ClassGeneratedBy);
	return (GeneratingBP && GeneratingBP->BlueprintType == BPTYPE_MacroLibrary);
}

/** Run over the components references, and then NULL any that fall outside this blueprint's scope (e.g. components brought over after reparenting from another class, which are now in the transient package) */
void FKismetEditorUtilities::StripExternalComponents(class UBlueprint* Blueprint)
{
	FArchiveInvalidateTransientRefs InvalidateRefsAr;
	
	UClass* SkeletonGeneratedClass = Blueprint->SkeletonGeneratedClass;
	if (SkeletonGeneratedClass)
	{
		UObject* SkeletonCDO = SkeletonGeneratedClass->GetDefaultObject();

		SkeletonCDO->Serialize(InvalidateRefsAr);
	}

	UClass* GeneratedClass = Blueprint->GeneratedClass;
	UObject* GeneratedCDO = GeneratedClass->GetDefaultObject();

	GeneratedCDO->Serialize(InvalidateRefsAr);
}

bool FKismetEditorUtilities::IsTrackedBlueprintParent(const UClass* ParentClass)
{
	if (ParentClass->ClassGeneratedBy == NULL)
	{
		// Always track native parent classes
		return true;
	}

	UBlueprint* ParentBlueprint = Cast<UBlueprint>(ParentClass->ClassGeneratedBy);

	// Cache the list of allowed blueprint names the first time it is requested
	if (TrackedBlueprintParentList.Num() == 0)
	{
		GConfig->GetArray(TEXT("Kismet"), TEXT("TrackedBlueprintParents"), /*out*/ TrackedBlueprintParentList, GEngineIni);
	}

	for (auto TrackedBlueprintIter = TrackedBlueprintParentList.CreateConstIterator(); TrackedBlueprintIter; ++TrackedBlueprintIter)
	{
		if (ParentBlueprint->GetName().EndsWith(*TrackedBlueprintIter))
		{
			return true;
		}
	}
	return false;
}

bool FKismetEditorUtilities::IsActorValidForLevelScript(const AActor* Actor)
{
	return Actor && !FActorEditorUtils::IsABuilderBrush(Actor);
}

bool FKismetEditorUtilities::AnyBoundLevelScriptEventForActor(AActor* Actor, bool bCouldAddAny)
{
	if (IsActorValidForLevelScript(Actor))
	{
		for (TFieldIterator<FMulticastDelegateProperty> PropertyIt(Actor->GetClass(), EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
		{
			FProperty* Property = *PropertyIt;
			// Check for multicast delegates that we can safely assign
			if (!Property->HasAnyPropertyFlags(CPF_Parm) && Property->HasAllPropertyFlags(CPF_BlueprintAssignable))
			{
				const FName EventName = Property->GetFName();
				const UK2Node_ActorBoundEvent* ExistingNode = FKismetEditorUtilities::FindBoundEventForActor(Actor, EventName);
				if ((NULL != ExistingNode) != bCouldAddAny)
				{
					return true;
				}
			}
		}
	}
	return false;
}

void FKismetEditorUtilities::AddLevelScriptEventOptionsForActor(UToolMenu* Menu, TWeakObjectPtr<AActor> ActorPtr, bool bExistingEvents, bool bNewEvents, bool bOnlyEventName)
{
	struct FCreateEventForActorHelper
	{
		static void CreateEventForActor(TWeakObjectPtr<AActor> InActorPtr, FName EventName)
		{
			if (!GEditor->bIsSimulatingInEditor && GEditor->PlayWorld == NULL)
			{
				AActor* Actor = InActorPtr.Get();
				if (Actor != NULL && EventName != NAME_None)
				{
					const UK2Node_ActorBoundEvent* ExistingNode = FKismetEditorUtilities::FindBoundEventForActor(Actor, EventName);
					if (ExistingNode != NULL)
					{
						FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(ExistingNode);
					}
					else
					{
						FKismetEditorUtilities::CreateNewBoundEventForActor(Actor, EventName);
					}
				}
			}
		}
	};

	AActor* Actor = ActorPtr.Get();
	if (IsActorValidForLevelScript(Actor))
	{
		// Struct to store event properties by category
		struct FEventCategory
		{
			FString CategoryName;
			TArray<FProperty*> EventProperties;
		};
		// ARray of event properties by category
		TArray<FEventCategory> CategorizedEvents;

		// Find all events we can assign
		for (TFieldIterator<FMulticastDelegateProperty> PropertyIt(Actor->GetClass(), EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
		{
			FProperty* Property = *PropertyIt;
			// Check for multicast delegates that we can safely assign
			if (!Property->HasAnyPropertyFlags(CPF_Parm) && Property->HasAllPropertyFlags(CPF_BlueprintAssignable))
			{
				// Get category for this property
				FString PropertyCategory = FObjectEditorUtils::GetCategory(Property);
				// See if we already have a list for this
				bool bFound = false;
				for (FEventCategory& Category : CategorizedEvents)
				{
					if(Category.CategoryName == PropertyCategory)
					{
						Category.EventProperties.Add(Property);
						bFound = true;
					}
				}
				// If not, create one
				if(!bFound)
				{
					FEventCategory NewCategory;
					NewCategory.CategoryName = PropertyCategory;
					NewCategory.EventProperties.Add(Property);
					CategorizedEvents.Add(NewCategory);
				}
			}
		}

		// Now build the menu
		for(FEventCategory& Category : CategorizedEvents)
		{
			FToolMenuSection& Section = Menu->AddSection(NAME_None, FText::FromString(Category.CategoryName));

			for(FProperty* Property : Category.EventProperties)
			{
				const FName EventName = Property->GetFName();
				const UK2Node_ActorBoundEvent* ExistingNode = FKismetEditorUtilities::FindBoundEventForActor(Actor, EventName);

				if ((!ExistingNode && !bNewEvents) || (ExistingNode && !bExistingEvents))
				{
					continue;
				}

				FText EntryText;
				if (bOnlyEventName)
				{
					EntryText = FText::FromName(EventName);
				}
				else
				{
					FFormatNamedArguments Args;
					Args.Add(TEXT("EventName"), FText::FromName(EventName));

					if (NULL == ExistingNode)
					{
						EntryText = FText::Format(LOCTEXT("AddEvent_ToolTip", "Add {EventName}"), Args);
					}
					else
					{
						EntryText = FText::Format(LOCTEXT("ViewEvent_ToolTip", "View {EventName}"), Args);
					}
				}

				// create menu entry
				Section.AddMenuEntry(
					NAME_None,
					EntryText,
					Property->GetToolTipText(),
					FSlateIcon(),
					FExecuteAction::CreateStatic(&FCreateEventForActorHelper::CreateEventForActor, ActorPtr, EventName)
					);
			}
		}
	}
}

void FKismetEditorUtilities::GetInformationOnMacro(UEdGraph* MacroGraph, /*out*/ UK2Node_Tunnel*& EntryNode, /*out*/ UK2Node_Tunnel*& ExitNode, bool& bIsMacroPure)
{
	check(MacroGraph);

	// Look at the graph for the entry & exit nodes
	TArray<UK2Node_Tunnel*> TunnelNodes;
	MacroGraph->GetNodesOfClass(TunnelNodes);

	for (int32 i = 0; i < TunnelNodes.Num(); i++)
	{
		UK2Node_Tunnel* Node = TunnelNodes[i];

		// Composite nodes should never be considered for function entry / exit, since we're searching for a graph's terminals
		if (Node->IsEditable() && !Node->IsA(UK2Node_Composite::StaticClass()))
		{
			if (Node->bCanHaveOutputs)
			{
				check(!EntryNode);
				EntryNode = Node;
			}
			else if (Node->bCanHaveInputs)
			{
				check(!ExitNode);
				ExitNode = Node;
			}
		}
	}

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// Determine the macro's purity
	//@TODO: May want to check what is *inside* a macro too, to determine it's relative purity
	bIsMacroPure = true;

	if (EntryNode != NULL)
	{
		for (int32 PinIndex = 0; PinIndex < EntryNode->Pins.Num(); ++PinIndex)
		{
			if (K2Schema->IsExecPin(*(EntryNode->Pins[PinIndex])))
			{
				bIsMacroPure = false;
				break;
			}
		}
	}

	if (bIsMacroPure && (ExitNode != NULL))
	{
		for (int32 PinIndex = 0; PinIndex < ExitNode->Pins.Num(); ++PinIndex)
		{
			if (K2Schema->IsExecPin(*(ExitNode->Pins[PinIndex])))
			{
				bIsMacroPure = false;
				break;
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE 
