// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet2/Kismet2NameValidators.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"
#include "K2Node_FunctionEntry.h"
#include "AnimStateTransitionNode.h"
#include "AnimationStateMachineGraph.h"
#include "Kismet2/BlueprintEditorUtils.h"

#define LOCTEXT_NAMESPACE "KismetNameValidators"

//////////////////////////////////////////////////
// FNameValidatorFactory

TSharedPtr<INameValidatorInterface> FNameValidatorFactory::MakeValidator(UEdGraphNode* Node)
{
	TSharedPtr<INameValidatorInterface> Validator;
	 
	// create a name validator for the node
	Validator = Node->MakeNameValidator();

	check(Validator.IsValid());
	return Validator;
}


FText INameValidatorInterface::GetErrorText(const FString& Name, EValidatorResult ErrorCode)
{
	FText ErrorText;
	switch (ErrorCode)
	{
	case EValidatorResult::EmptyName:
		ErrorText = LOCTEXT("EmptyName_Error", "Name cannot be empty.");
		break;
	case EValidatorResult::AlreadyInUse:
		ErrorText = LOCTEXT("AlreadyInUse_Error", "Name is already in use.");
		break;
	case EValidatorResult::ExistingName:
		ErrorText = LOCTEXT("ExistingName_Error", "Name cannot be the same as the existing name.");
		break;
	case EValidatorResult::ContainsInvalidCharacters:
		ErrorText = LOCTEXT("ContainsInvalidCharacters_Error", "Name cannot contain invalid characters.");
		FName::IsValidXName(Name, UE_BLUEPRINT_INVALID_NAME_CHARACTERS, /*out*/ &ErrorText);
		break;
	case EValidatorResult::TooLong:
		ErrorText = LOCTEXT("NameTooLong_Error", "Names must have fewer than 100 characters!");
		break;
	case EValidatorResult::LocallyInUse:
		ErrorText  = LOCTEXT("LocallyInUse_Error", "Conflicts with another object in the same scope!");
		break;
	case EValidatorResult::Ok:
		break;
	default:
		ErrorText = LOCTEXT("UnknownError", "Unknown error");
		check(false);
		break;
	}

	return ErrorText;
}


EValidatorResult INameValidatorInterface::FindValidString(FString& InOutName)
{
	FString DesiredName = InOutName;
	FString NewName = DesiredName;
	int32 NameIndex = 1;

	while (true)
	{
		EValidatorResult VResult = IsValid(NewName, true);
		if (VResult == EValidatorResult::Ok)
		{
			InOutName = NewName;
			return NewName == DesiredName? EValidatorResult::Ok : EValidatorResult::AlreadyInUse;
		}

		NewName = FString::Printf(TEXT("%s_%d"), *DesiredName, NameIndex++);
	}
}


 bool INameValidatorInterface::BlueprintObjectNameIsUnique(class UBlueprint* Blueprint, const FName& Name)
 {
	 UObject* Obj = FindObject<UObject>(Blueprint, *Name.ToString());
	 
	 return (Obj == NULL)
		 ? true
		 : false;
 }

//////////////////////////////////////////////////
// FKismetNameValidator

namespace BlueprintNameConstants
{
	 int32 NameMaxLength = 100;
}

FKismetNameValidator::FKismetNameValidator(const class UBlueprint* Blueprint, FName InExistingName/* = NAME_None*/, const UStruct* InScope/* = nullptr */)
	: BlueprintObject(Blueprint)
	, ExistingName(InExistingName)
	, Scope(InScope)
{
	FBlueprintEditorUtils::GetClassVariableList(BlueprintObject, Names, true);
	FBlueprintEditorUtils::GetFunctionNameList(BlueprintObject, Names);
	FBlueprintEditorUtils::GetAllGraphNames(BlueprintObject, Names);
	FBlueprintEditorUtils::GetSCSVariableNameList(Blueprint, Names);
	FBlueprintEditorUtils::GetImplementingBlueprintsFunctionNameList(Blueprint, Names);
}

int32 FKismetNameValidator::GetMaximumNameLength()
{
	return BlueprintNameConstants::NameMaxLength;
}

EValidatorResult FKismetNameValidator::IsValid(const FString& Name, bool /*bOriginal*/)
{
	// Converting a string that is too large for an FName will cause an assert, so verify the length
	if(Name.Len() >= NAME_SIZE)
	{
		return EValidatorResult::TooLong;
	}
	else if (!FName::IsValidXName(Name, UE_BLUEPRINT_INVALID_NAME_CHARACTERS))
	{
		return EValidatorResult::ContainsInvalidCharacters;
	}

	// If not defined in name table, not current graph name
	return IsValid( FName(*Name) );
}

EValidatorResult FKismetNameValidator::IsValid(const FName& Name, bool /* bOriginal */)
{
	EValidatorResult ValidatorResult = EValidatorResult::AlreadyInUse;

	if(Name == NAME_None)
	{
		ValidatorResult = EValidatorResult::EmptyName;
	}
	else if(Name == ExistingName)
	{
		ValidatorResult = EValidatorResult::Ok;
	}
	else if(Name.ToString().Len() > BlueprintNameConstants::NameMaxLength)
	{
		ValidatorResult = EValidatorResult::TooLong;
	}
	else
	{
		// If it is in the names list then it is already in use.
		if(!Names.Contains(Name))
		{
			ValidatorResult = EValidatorResult::Ok;

			// Check for collision with an existing object.
			if (UObject* ExistingObject = StaticFindObject(/*Class=*/ nullptr, const_cast<UBlueprint*>(BlueprintObject), *Name.ToString(), EFindObjectFlags::ExactClass))
			{
				// To allow the linker to resolve imports when dependent Blueprints are loaded, macro libraries
				// will leave behind a redirector whenever one of its macro graph objects are renamed. These get
				// moved aside to allow their names to be recycled, so we consider a collision with an existing
				// redirector object in that case to be a false positive.
				// 
				// Note that while the linker will resolve references to either a graph or redirector by name on load,
				// macro instance nodes will instead resolve references to any external macro library's graph by GUID. 
				const bool bIgnoreRedirectors = BlueprintObject && BlueprintObject->BlueprintType == BPTYPE_MacroLibrary;
				if (!bIgnoreRedirectors || !ExistingObject->IsA<UObjectRedirector>())
				{
					ValidatorResult = EValidatorResult::AlreadyInUse;
				}
			}
		}

		if(ValidatorResult == EValidatorResult::Ok)
		{
			if(Scope == nullptr)
			{
				// Search through all functions for their local variables and prevent duplicate names
				TArray<UK2Node_FunctionEntry*> FunctionEntryNodes;
				FBlueprintEditorUtils::GetAllNodesOfClass(BlueprintObject, FunctionEntryNodes);
				for (UK2Node_FunctionEntry* const FunctionEntry : FunctionEntryNodes)
				{
					for( const FBPVariableDescription& Variable : FunctionEntry->LocalVariables )
					{
						if(Variable.VarName == Name)
						{
							ValidatorResult = EValidatorResult::AlreadyInUse;
							break;
						}
					}

					if(ValidatorResult != EValidatorResult::Ok)
					{
						break;
					}
				}
			}
			else
			{
				if(FindFProperty<const FProperty>(Scope, *Name.ToString()) != nullptr)
				{
					ValidatorResult = EValidatorResult::LocallyInUse;
				}
			}
		}
	}
	
	return ValidatorResult;
}

//////////////////////////////////////////////////////////////////
// FStringSetNameValidator

EValidatorResult FStringSetNameValidator::IsValid(const FString& Name, bool bOriginal)
{
	if (Name.IsEmpty())
	{
		return EValidatorResult::EmptyName;
	}
	else if (Name == ExistingName)
	{
		return EValidatorResult::ExistingName;
	}
	else
	{
		return (Names.Contains(Name)) ? EValidatorResult::AlreadyInUse : EValidatorResult::Ok;
	}
}

EValidatorResult FStringSetNameValidator::IsValid(const FName& Name, bool bOriginal)
{
	return IsValid(Name.ToString(), bOriginal);
}

//////////////////////////////////////////////////////////////////
// FAnimStateTransitionNodeSharedRulesNameValidator
// this doesn't go to MakeValidator in factory, as it is validator for internal name in AnimStateTransitionNode
FAnimStateTransitionNodeSharedRulesNameValidator::FAnimStateTransitionNodeSharedRulesNameValidator(class UAnimStateTransitionNode* InStateTransitionNode)
	: FStringSetNameValidator(FString())
{
	TArray<UAnimStateTransitionNode*> Nodes;
	UAnimationStateMachineGraph* StateMachine = CastChecked<UAnimationStateMachineGraph>(InStateTransitionNode->GetOuter());

	StateMachine->GetNodesOfClass<UAnimStateTransitionNode>(Nodes);
	for (auto NodeIt = Nodes.CreateIterator(); NodeIt; ++NodeIt)
	{
		auto Node = *NodeIt;
		if(Node != InStateTransitionNode &&
		   Node->bSharedRules &&
		   Node->SharedRulesGuid != InStateTransitionNode->SharedRulesGuid) // store only those shared rules who have different guid
		{
			Names.Add(Node->SharedRulesName);
		}
	}
}

//////////////////////////////////////////////////////////////////
// FAnimStateTransitionNodeSharedCrossfadeNameValidator
// this doesn't go to MakeValidator in factory, as it is validator for internal name in AnimStateTransitionNode
FAnimStateTransitionNodeSharedCrossfadeNameValidator::FAnimStateTransitionNodeSharedCrossfadeNameValidator(class UAnimStateTransitionNode* InStateTransitionNode)
	: FStringSetNameValidator(FString())
{
	TArray<UAnimStateTransitionNode*> Nodes;
	UAnimationStateMachineGraph* StateMachine = CastChecked<UAnimationStateMachineGraph>(InStateTransitionNode->GetOuter());

	StateMachine->GetNodesOfClass<UAnimStateTransitionNode>(Nodes);
	for (auto NodeIt = Nodes.CreateIterator(); NodeIt; ++NodeIt)
	{
		auto Node = *NodeIt;
		if(Node != InStateTransitionNode &&
		   Node->bSharedCrossfade &&
		   Node->SharedCrossfadeGuid != InStateTransitionNode->SharedCrossfadeGuid) // store only those shared crossfade who have different guid
		{
			Names.Add(Node->SharedCrossfadeName);
		}
	}
}

////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
