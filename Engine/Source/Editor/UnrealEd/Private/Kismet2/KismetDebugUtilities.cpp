// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet2/KismetDebugUtilities.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "GameFramework/Actor.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/TextProperty.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/SMultiLineEditableText.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Styling/AppStyle.h"
#include "Engine/Blueprint.h"
#include "EdGraph/EdGraph.h"
#include "Editor/UnrealEdEngine.h"
#include "Settings/EditorExperimentalSettings.h"
#include "CallStackViewer.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "UnrealEdGlobals.h"
#include "Kismet2/Breakpoint.h"
#include "Kismet2/WatchedPin.h"
#include "ActorEditorUtils.h"
#include "EdGraphSchema_K2.h"
#include "K2Node.h"
#include "K2Node_Tunnel.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_CallFunction.h"
#include "K2Node_Knot.h"
#include "K2Node_MacroInstance.h"
#include "K2Node_Composite.h"
#include "K2Node_Message.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Logging/TokenizedMessage.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "AnimGraphNode_Base.h"
#include "UObject/UnrealType.h"
#include "AnimationGraphSchema.h"
#include "BlueprintEditorSettings.h"
#include "Blueprint/BlueprintExceptionInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(KismetDebugUtilities)

#define LOCTEXT_NAMESPACE "BlueprintDebugging"

enum class EKismetDebuggingMode : uint8
{
	None,
	Engine, // for Blutility
	World,  // for PIE
};

/** Per-thread data for use by FKismetDebugUtilities functions */
class FKismetDebugUtilitiesData : public TThreadSingleton<FKismetDebugUtilitiesData>
{
public:
	FKismetDebugUtilitiesData()
		: TargetGraphNodes()
		, CurrentInstructionPointer(nullptr)
		, MostRecentBreakpointInstructionPointer(nullptr)
		, MostRecentStoppedNode(nullptr)
		, CurrentDebuggingWorld(nullptr)
		, TargetGraphStackDepth(INDEX_NONE)
		, MostRecentBreakpointGraphStackDepth(INDEX_NONE)
		, MostRecentBreakpointInstructionOffset(INDEX_NONE)
		, StackFrameAtIntraframeDebugging(nullptr)
		, TraceStackSamples(FKismetDebugUtilities::MAX_TRACE_STACK_SAMPLES)
		, CurrentDebuggingMode(EKismetDebuggingMode::None)
		, bIsSingleStepping(false)
		, bIsSteppingOut(false)
	{
	}

	void Reset()
	{
		TargetGraphNodes.Empty();
		CurrentInstructionPointer = nullptr;
		MostRecentStoppedNode = nullptr;
		CurrentDebuggingWorld = nullptr;

		TargetGraphStackDepth = INDEX_NONE;
		MostRecentBreakpointGraphStackDepth = INDEX_NONE;
		MostRecentBreakpointInstructionOffset = INDEX_NONE;
		StackFrameAtIntraframeDebugging = nullptr;
		CurrentDebuggingMode = EKismetDebuggingMode::None;

		bIsSingleStepping = false;
		bIsSteppingOut = false;
	}

	// List of graph nodes that the user wants to stop at, at the current TargetGraphStackDepth. Used for Step Over:
	TArray< TWeakObjectPtr< class UEdGraphNode> > TargetGraphNodes;

	// Current node:
	TWeakObjectPtr< class UEdGraphNode > CurrentInstructionPointer;

	// The current instruction encountered if we are stopped at a breakpoint; NULL otherwise
	TWeakObjectPtr< class UEdGraphNode > MostRecentBreakpointInstructionPointer;
	
	// The last node that we decided to break on for any reason (e.g. breakpoint, exception, or step operation):
	TWeakObjectPtr< class UEdGraphNode > MostRecentStoppedNode;

	// The PlayWorld that generated
	TWeakObjectPtr<UWorld> CurrentDebuggingWorld;

	// The target graph call stack depth. INDEX_NONE if not active
	int32 TargetGraphStackDepth;

	// The graph stack depth that a breakpoint was hit at, used to ensure that breakpoints
	// can be hit multiple times in the case of recursion
	int32 MostRecentBreakpointGraphStackDepth;

	// The instruction that we hit a breakpoint at, this is used to ensure that a given node
	// can be stepped over reliably (but still break multiple times in the case of recursion):
	int32 MostRecentBreakpointInstructionOffset;

	// The last message that an exception delivered
	FText LastExceptionMessage;

	// Only valid inside intraframe debugging
	const FFrame* StackFrameAtIntraframeDebugging;

	// This data is used for the 'marching ants' display in the blueprint editor
	TSimpleRingBuffer<FKismetTraceSample> TraceStackSamples;

	// The type of current debugging 
	EKismetDebuggingMode CurrentDebuggingMode;

	// This flag controls whether we're trying to 'step in' to a function
	bool bIsSingleStepping;

	// This flag controls whether we're trying to 'step out' of a graph
	bool bIsSteppingOut;
};

//////////////////////////////////////////////////////////////////////////
// FKismetDebugUtilities

void FKismetDebugUtilities::EndOfScriptExecution(const FBlueprintContextTracker& BlueprintContext)
{
	if(BlueprintContext.GetScriptEntryTag() == 1)
	{
		// if this is our last VM frame, then clear stepping data:
		FKismetDebugUtilitiesData& Data = FKismetDebugUtilitiesData::Get();

		Data.Reset();
	}
}

void FKismetDebugUtilities::RequestAbortingExecution()
{
	check(IsInGameThread());
	FKismetDebugUtilitiesData& Data = FKismetDebugUtilitiesData::Get();
	if (Data.StackFrameAtIntraframeDebugging)
	{
		const_cast<FFrame*>(Data.StackFrameAtIntraframeDebugging)->bAbortingExecution = true;
	}
}

void FKismetDebugUtilities::RequestSingleStepIn()
{
	FKismetDebugUtilitiesData& Data = FKismetDebugUtilitiesData::Get();

	Data.bIsSingleStepping = true;
}

void FKismetDebugUtilities::RequestStepOver()
{
	FKismetDebugUtilitiesData& Data = FKismetDebugUtilitiesData::Get();
	TArrayView<const FFrame* const> ScriptStack = FBlueprintContextTracker::Get().GetCurrentScriptStack();

	if (ScriptStack.Num() > 0)
	{
		Data.TargetGraphStackDepth = ScriptStack.Num();

		if (const UEdGraphNode* StoppedNode = Data.MostRecentStoppedNode.Get())
		{
			if (StoppedNode->IsA<UK2Node_MacroInstance>() || StoppedNode->IsA<UK2Node_Composite>() || StoppedNode->IsA<UK2Node_CallFunction>())
			{
				for (const UEdGraphPin* Pin : StoppedNode->Pins)
				{
					// add any nodes connected via execs as TargetGraphNodes:
					if (Pin->Direction == EGPD_Output && Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Exec && Pin->LinkedTo.Num() > 0)
					{
						for (UEdGraphPin* LinkedTo : Pin->LinkedTo)
						{
							UEdGraphNode* GraphNode = LinkedTo->GetOwningNode();
							if (UK2Node_Knot* Knot = Cast<UK2Node_Knot>(GraphNode))
							{
								// search the knot chain to find the actual node:
								GraphNode = Knot->GetExecTerminal();
							}

							if (GraphNode)
							{
								Data.TargetGraphNodes.AddUnique(GraphNode);
							}
						}
					}

				}

				return;
			}
		}

		Data.bIsSingleStepping = false;
		Data.bIsSteppingOut = true;
	}
}

void FKismetDebugUtilities::RequestStepOut()
{
	FKismetDebugUtilitiesData& Data = FKismetDebugUtilitiesData::Get();
	TArrayView<const FFrame* const> ScriptStack = FBlueprintContextTracker::Get().GetCurrentScriptStack();

	Data.bIsSingleStepping = false;
	if (ScriptStack.Num() > 1)
	{
		Data.bIsSteppingOut = true;
		Data.TargetGraphStackDepth = ScriptStack.Num() - 1;
	}
}

void FKismetDebugUtilities::OnScriptException(const UObject* ActiveObject, const FFrame& StackFrame, const FBlueprintExceptionInfo& Info)
{
	FKismetDebugUtilitiesData& Data = FKismetDebugUtilitiesData::Get();

	struct Local
	{
		static void OnMessageLogLinkActivated(const class TSharedRef<IMessageToken>& Token)
		{
			if( Token->GetType() == EMessageToken::Object )
			{
				const TSharedRef<FUObjectToken> UObjectToken = StaticCastSharedRef<FUObjectToken>(Token);
				if(UObjectToken->GetObject().IsValid())
				{
					FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(UObjectToken->GetObject().Get());
				}	
			}
		}
	};

	checkSlow(ActiveObject != nullptr);

	const UClass* ClassContainingCode = FindClassForNode(ActiveObject, StackFrame.Node);
	UBlueprint* BlueprintObj = (ClassContainingCode ? Cast<UBlueprint>(ClassContainingCode->ClassGeneratedBy) : nullptr);
	if (BlueprintObj)
	{
		const FBlueprintExceptionInfo* ExceptionInfo = &Info;
		bool bResetObjectBeingDebuggedWhenFinished = false;
		UObject* ObjectBeingDebugged = BlueprintObj->GetObjectBeingDebugged();
		if (UClass* GeneratedClass = BlueprintObj->GeneratedClass; 
			ObjectBeingDebugged == nullptr && BPTYPE_FunctionLibrary == BlueprintObj->BlueprintType && GeneratedClass)
		{
			ObjectBeingDebugged = GeneratedClass->GetDefaultObject(false);
		}

		auto IsAPreviewOrInactiveObject = [](const UObject* InObject)
		{
			UWorld* World = InObject ? InObject->GetWorld() : nullptr;
			return World && (World->WorldType == EWorldType::EditorPreview || World->WorldType == EWorldType::Inactive);
		};

		// Ignore script exceptions for preview objects that are not already being debugged
		if (IsAPreviewOrInactiveObject(ActiveObject) && ObjectBeingDebugged != ActiveObject)
		{
			return;
		}

		UObject* SavedObjectBeingDebugged = ObjectBeingDebugged;
		UWorld* WorldBeingDebugged = BlueprintObj->GetWorldBeingDebugged();
		const FString& PathToDebug = BlueprintObj->GetObjectPathToDebug();
		
		if (ObjectBeingDebugged == nullptr && !PathToDebug.IsEmpty())
		{
			// Check if we need to update the object being debugged
			UObject* ObjectToDebug = FindObjectSafe<UObject>(nullptr, *PathToDebug);
			if (IsValid(ObjectToDebug))
			{
				// If the path to debug matches a newly-spawned object, set the hard reference now
				ObjectBeingDebugged = ObjectToDebug;
				BlueprintObj->SetObjectBeingDebugged(ObjectBeingDebugged);
			}
		}

		const int32 BreakpointOffset = StackFrame.Code - StackFrame.Node->Script.GetData() - 1;

		bool bShouldBreakExecution = false;
		bool bForceToCurrentObject = false;
		bool bIsStepping = Data.bIsSingleStepping || Data.TargetGraphStackDepth != INDEX_NONE;

		switch (Info.GetType())
		{
		case EBlueprintExceptionType::Breakpoint:
			bShouldBreakExecution = true;
			break;
		case EBlueprintExceptionType::Tracepoint:
			bShouldBreakExecution = bIsStepping && TracepointBreakAllowedOnOwningWorld(ActiveObject);
			break;
		case EBlueprintExceptionType::WireTracepoint:
			break;
		case EBlueprintExceptionType::AccessViolation:
		case EBlueprintExceptionType::UserRaisedError:
			if ( GIsEditor && GIsPlayInEditorWorld )
			{
				// declared as its own variable since it's flushed (logs pushed to std output) on destruction
				// we want the full message constructed before it's logged
				TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error);
				Message->AddToken(FTextToken::Create(FText::Format(LOCTEXT("RuntimeErrorMessageFmt", "Blueprint Runtime Error: \"{0}\"."), Info.GetDescription())));

#if WITH_EDITORONLY_DATA // to protect access to GeneratedClass->DebugData
				const UBlueprintGeneratedClass* GeneratedClass = Cast<UBlueprintGeneratedClass>(ClassContainingCode);
				if ((GeneratedClass != nullptr) && GeneratedClass->DebugData.IsValid())
				{
					UEdGraphNode* BlueprintNode = GeneratedClass->DebugData.FindSourceNodeFromCodeLocation(StackFrame.Node, BreakpointOffset, true);
					// if instead, there is a node we can point to...
					if (BlueprintNode != nullptr)
					{
						Message->AddToken(FTextToken::Create(LOCTEXT("RuntimeErrorBlueprintNodeLabel", "Node: ")));
						Message->AddToken(FUObjectToken::Create(BlueprintNode, BlueprintNode->GetNodeTitle(ENodeTitleType::ListView))
							->OnMessageTokenActivated(FOnMessageTokenActivated::CreateStatic(&Local::OnMessageLogLinkActivated))
						);

						Message->AddToken(FTextToken::Create(LOCTEXT("RuntimeErrorBlueprintGraphLabel", "Graph: ")));
						Message->AddToken(FUObjectToken::Create(BlueprintNode->GetGraph(), FText::FromString(GetNameSafe(BlueprintNode->GetGraph())))
							->OnMessageTokenActivated(FOnMessageTokenActivated::CreateStatic(&Local::OnMessageLogLinkActivated))
						);
					}
				}
#endif // WITH_EDITORONLY_DATA

				// NOTE: StackFrame.Node is not a blueprint node like you may think ("Node" has some legacy meaning)
				Message->AddToken(FTextToken::Create(LOCTEXT("RuntimeErrorBlueprintFunctionLabel", "Function: ")));
				Message->AddToken(FUObjectToken::Create(StackFrame.Node, StackFrame.Node->GetDisplayNameText())
					->OnMessageTokenActivated(FOnMessageTokenActivated::CreateStatic(&Local::OnMessageLogLinkActivated))
				);

				Message->AddToken(FTextToken::Create(LOCTEXT("RuntimeErrorBlueprintObjectLabel", "Blueprint: ")));
				Message->AddToken(FUObjectToken::Create(BlueprintObj, FText::FromString(BlueprintObj->GetName()))
					->OnMessageTokenActivated(FOnMessageTokenActivated::CreateStatic(&Local::OnMessageLogLinkActivated))
				);

				FMessageLog("PIE").AddMessage(Message);
			}
			bForceToCurrentObject = true;
			bShouldBreakExecution = GetDefault<UEditorExperimentalSettings>()->bBreakOnExceptions;
			break;
		case EBlueprintExceptionType::InfiniteLoop:
			bForceToCurrentObject = true;
			bShouldBreakExecution = GetDefault<UEditorExperimentalSettings>()->bBreakOnExceptions;
			break;
		default:
			bForceToCurrentObject = true;
			bShouldBreakExecution = GetDefault<UEditorExperimentalSettings>()->bBreakOnExceptions;
			break;
		}

		if (!bForceToCurrentObject && bIsStepping)
		{
			// If we're stepping, temporarily override the selected debug object so step into always works)
			bForceToCurrentObject = true;
		}

		// If we are debugging a specific world, the object needs to be in it
		if (WorldBeingDebugged != nullptr && !ActiveObject->IsIn(WorldBeingDebugged))
		{
			// Might be a streaming level case, so find the real world to see
			const UObject *ObjOuter = ActiveObject;
			const UWorld *ObjWorld = nullptr;
			bool FailedWorldCheck = true;
			while(ObjWorld == nullptr && ObjOuter != nullptr)
			{
				ObjOuter = ObjOuter->GetOuter();
				ObjWorld = Cast<const UWorld>(ObjOuter);
			}
			if (ObjWorld && ObjWorld->PersistentLevel)
			{
				if (ObjWorld->PersistentLevel->OwningWorld == WorldBeingDebugged)
				{
					// Its ok, the owning world is the world being debugged
					FailedWorldCheck = false;
				}				
			}

			if (FailedWorldCheck)
			{
				bForceToCurrentObject = false;
				bShouldBreakExecution = false;
			}
		}

		if (bShouldBreakExecution)
		{
			if ((PathToDebug.IsEmpty()) || (bForceToCurrentObject))
			{
				// If there was nothing being debugged, treat this as a one-shot, temporarily set this object as being debugged,
				// and continue allowing any breakpoint to hit later on
				bResetObjectBeingDebuggedWhenFinished = true;
				ObjectBeingDebugged = const_cast<UObject*>(ActiveObject);
				BlueprintObj->SetObjectBeingDebugged(ObjectBeingDebugged);
			}
		}

		if (ObjectBeingDebugged == ActiveObject)
		{
			// Record into the trace log
			FKismetTraceSample& Tracer = Data.TraceStackSamples.WriteNewElementUninitialized();
			Tracer.Context = MakeWeakObjectPtr(const_cast<UObject*>(ActiveObject));
			Tracer.Function = StackFrame.Node;
			Tracer.Offset = BreakpointOffset; //@TODO: Might want to make this a parameter of Info
			Tracer.ObservationTime = FPlatformTime::Seconds();

			// Find the node that generated the code which we hit
			UEdGraphNode* NodeStoppedAt = FindSourceNodeForCodeLocation(ActiveObject, StackFrame.Node, BreakpointOffset, /*bAllowImpreciseHit=*/ true);
			if (NodeStoppedAt && (Info.GetType() == EBlueprintExceptionType::Tracepoint || Info.GetType() == EBlueprintExceptionType::Breakpoint))
			{
				// Handle Node stepping and update the stack
				CheckBreakConditions(NodeStoppedAt, Info.GetType() == EBlueprintExceptionType::Breakpoint, BreakpointOffset, bShouldBreakExecution);
			}

			// Can't do intraframe debugging when the editor is actively stopping
			if (GEditor->ShouldEndPlayMap())
			{
				bShouldBreakExecution = false;
			}

			// Handle a breakpoint or single-step
			if (bShouldBreakExecution)
			{
				AttemptToBreakExecution(BlueprintObj, ActiveObject, StackFrame, *ExceptionInfo, NodeStoppedAt, BreakpointOffset);
			}
		}

		// Reset the object being debugged if we forced it to be something different
		if (bResetObjectBeingDebuggedWhenFinished)
		{
			if (BlueprintObj->GetObjectBeingDebugged() == ObjectBeingDebugged)
			{
				// Only reset if it's still what we expected, if the user picked a new object from the UI we want to respect that
				BlueprintObj->SetObjectBeingDebugged(SavedObjectBeingDebugged);
			}
		}

		const auto ShowScriptExceptionError = [&](const FText& InExceptionErrorMsg)
		{
			if (GUnrealEd->PlayWorld != nullptr)
			{
				GEditor->RequestEndPlayMap();
				FSlateApplication::Get().LeaveDebuggingMode();
			}

			// Launch a message box notifying the user why they have been booted
			{
				// Callback to display a pop-up showing the Callstack, the user can highlight and copy this if needed
				auto DisplayCallStackLambda = [](const FText CallStack)
				{
					TSharedPtr<SMultiLineEditableText> TextBlock;
					TSharedRef<SWidget> DisplayWidget =
						SNew(SBox)
						.MaxDesiredHeight(512.0f)
						.MaxDesiredWidth(512.0f)
						.Content()
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
							[
								SNew(SScrollBox)
								+ SScrollBox::Slot()
								[
									SAssignNew(TextBlock, SMultiLineEditableText)
									.AutoWrapText(true)
									.IsReadOnly(true)
									.Text(CallStack)
								]
							]
						];

					FSlateApplication::Get().PushMenu(
						FSlateApplication::Get().GetActiveTopLevelWindow().ToSharedRef(),
						FWidgetPath(),
						DisplayWidget,
						FSlateApplication::Get().GetCursorPos(),
						FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup)
						);

					FSlateApplication::Get().SetKeyboardFocus(TextBlock);
				};

				TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error);

				// Display the main error message
				Message->AddToken(FTextToken::Create(InExceptionErrorMsg));

				// Display a link to the UObject and the UFunction that is crashing
				{
					// Get the name of the Blueprint
					FString BlueprintName;
					BlueprintObj->GetName(BlueprintName);

					Message->AddToken(FTextToken::Create(LOCTEXT("ShowScriptExceptionError_BlueprintLabel", "Blueprint: ")));
					Message->AddToken(FUObjectToken::Create(BlueprintObj, FText::FromString(BlueprintName)));
				}
				{
					// If a source node is found, that's the token we want to link, otherwise settle with the UFunction
					const int32 BreakpointOpCodeOffset = StackFrame.Code - StackFrame.Node->Script.GetData() - 1; //@TODO: Might want to make this a parameter of Info
					UEdGraphNode* SourceNode = FindSourceNodeForCodeLocation(ActiveObject, StackFrame.Node, BreakpointOpCodeOffset, /*bAllowImpreciseHit=*/ true);

					Message->AddToken(FTextToken::Create(LOCTEXT("ShowScriptExceptionError_FunctionLabel", "Function: ")));
					if (SourceNode)
					{
						Message->AddToken(FUObjectToken::Create(SourceNode, SourceNode->GetNodeTitle(ENodeTitleType::ListView)));
					}
					else
					{
						Message->AddToken(FUObjectToken::Create(StackFrame.Node, StackFrame.Node->GetDisplayNameText()));
					}
				}

				// Display a pop-up that will display the complete script callstack
				Message->AddToken(FTextToken::Create(LOCTEXT("ShowScriptExceptionError_CallStackLabel", "Call Stack: ")));
				Message->AddToken(FActionToken::Create(LOCTEXT("ShowScriptExceptionError_ShowCallStack", "Show"), LOCTEXT("ShowScriptExceptionError_ShowCallStackDesc", "Displays the underlying callstack, tracing what function calls led to the assert occuring."), FOnActionTokenExecuted::CreateStatic(DisplayCallStackLambda, FText::FromString(StackFrame.GetStackTrace()))));
				FMessageLog("PIE").AddMessage(Message);
			}
		};

		// Extra cleanup after potential interactive handling
		switch (Info.GetType())
		{
		case EBlueprintExceptionType::FatalError:
			ShowScriptExceptionError(FText::Format(LOCTEXT("ShowScriptExceptionError_FatalErrorFmt", "Fatal error detected: \"{0}\"."), Info.GetDescription()));
			break;
		case EBlueprintExceptionType::InfiniteLoop:
			ShowScriptExceptionError(LOCTEXT("ShowScriptExceptionError_InfiniteLoop", "Infinite loop detected."));
			break;
		default:
			// Left empty intentionally
			break;
		}
	}
}

bool FKismetDebugUtilities::TracepointBreakAllowedOnOwningWorld(const UObject* ObjOuter)
{
	bool bAllowTracepointBreak = true;

	const UWorld* ObjWorld = ObjOuter->GetWorld();

	// Disable tracepoints on EditorPreviews or Inactive worlds
	if (ObjWorld && (ObjWorld->WorldType == EWorldType::EditorPreview || ObjWorld->WorldType == EWorldType::Inactive))
	{
		bAllowTracepointBreak = false;
	}

	return bAllowTracepointBreak;
}

const UClass* FKismetDebugUtilities::FindClassForNode(const UObject* Object, const UFunction* Function)
{
	if (NULL != Function)
	{
		UClass* FunctionOwner = Function->GetOwnerClass();
		return FunctionOwner;
	}
	if(NULL != Object)
	{
		UClass* ObjClass = Object->GetClass();
		return ObjClass;
	}
	return NULL;
}	

const TSimpleRingBuffer<FKismetTraceSample>& FKismetDebugUtilities::GetTraceStack()
{
	return FKismetDebugUtilitiesData::Get().TraceStackSamples; 
}

UEdGraphNode* FKismetDebugUtilities::FindSourceNodeForCodeLocation(const UObject* Object, const UFunction* Function, int32 DebugOpcodeOffset, bool bAllowImpreciseHit)
{
	if (Object != NULL)
	{
		// Find the blueprint that corresponds to the object
		if (const UBlueprintGeneratedClass* Class = Cast<UBlueprintGeneratedClass>(FindClassForNode(Object, Function)))
		{
			return Class->GetDebugData().FindSourceNodeFromCodeLocation(Function, DebugOpcodeOffset, bAllowImpreciseHit);
		}
	}

	return NULL;
}

void FKismetDebugUtilities::CheckBreakConditions(UEdGraphNode* NodeStoppedAt, bool bHitBreakpoint, int32 BreakpointOffset, bool& InOutBreakExecution)
{
	FKismetDebugUtilitiesData& Data = FKismetDebugUtilitiesData::Get();
	TArrayView<const FFrame* const> ScriptStack = FBlueprintContextTracker::Get().GetCurrentScriptStack();

	if (NodeStoppedAt)
	{
		const bool bIsTryingToBreak = bHitBreakpoint ||
			Data.TargetGraphStackDepth != INDEX_NONE ||
			Data.bIsSingleStepping;

		if(bIsTryingToBreak)
		{
			// Update the TargetGraphStackDepth if we're on the same node - this handles things like
			// event nodes in the Event Graph, which will push another frame on to the stack:
			if(NodeStoppedAt == Data.MostRecentStoppedNode &&
				Data.MostRecentBreakpointGraphStackDepth < ScriptStack.Num() &&
				Data.TargetGraphStackDepth != INDEX_NONE)
			{
				// when we recurse, when a node increases stack depth itself we want to increase our 
				// target depth to compensate:
				Data.TargetGraphStackDepth += 1;
			}
			else if(NodeStoppedAt != Data.MostRecentStoppedNode)
			{
				Data.MostRecentStoppedNode = nullptr;
			}

			// We should only actually break execution when we're on a new node or we've recursed to the same
			// node. We detect recursion by checking for a deeper stack and an earlier instruction:
			InOutBreakExecution = 
				NodeStoppedAt != Data.MostRecentStoppedNode ||
				(
					Data.MostRecentBreakpointGraphStackDepth < ScriptStack.Num() &&
					Data.MostRecentBreakpointInstructionOffset >= BreakpointOffset
				);

			// If we have a TargetGraphStackDepth, don't break if we haven't reached that stack depth, or if we've stepped
			// in to a collapsed graph/macro instance:
			if(InOutBreakExecution && Data.TargetGraphStackDepth != INDEX_NONE && !bHitBreakpoint)
			{
				InOutBreakExecution = Data.TargetGraphStackDepth >= ScriptStack.Num();
				if(InOutBreakExecution && Data.TargetGraphStackDepth == ScriptStack.Num())
				{
					// If we have Data.TargetGraphNodes.Num() > 0, see if we can find a BlueprintNode matching a TargetGraphNode by iterating
					// up the Blueprint class hierarchy of our CurrentFrame->Object calling FindSourceNodeFromCodeLocation at each level.
					if (Data.TargetGraphNodes.Num() > 0)
					{
						// we're at the same stack depth, don't break if we've entered a different graph, but do break if we left the 
						// graph that we were trying to step over..
						const FFrame* CurrentFrame = ScriptStack.Last();
						if (CurrentFrame->Object)
						{
							UClass* BPClass = nullptr;
							while (true)
							{
								BPClass = (BPClass == nullptr) ? CurrentFrame->Object->GetClass() : BPClass->GetSuperClass();
								if (BPClass == nullptr)
								{
									InOutBreakExecution = false;
									break;
								}

								const UBlueprintGeneratedClass* BPGC = Cast<UBlueprintGeneratedClass>(BPClass);
								if (BPGC == nullptr)
								{
									InOutBreakExecution = false;
									break;
								}

								const UEdGraphNode* BlueprintNode = BPGC->DebugData.FindSourceNodeFromCodeLocation(CurrentFrame->Node, BreakpointOffset, true);

								if (Data.TargetGraphNodes.Contains(BlueprintNode))
								{
									break;
								}
							}
						}
					}
				}
			}
		}
		else if (NodeStoppedAt != Data.MostRecentStoppedNode)
		{
			Data.MostRecentStoppedNode = nullptr;
		}
	}
	
	if (InOutBreakExecution)
	{
		Data.MostRecentStoppedNode = NodeStoppedAt;
		Data.MostRecentBreakpointGraphStackDepth = ScriptStack.Num();
		Data.MostRecentBreakpointInstructionOffset = BreakpointOffset;
		Data.TargetGraphStackDepth = INDEX_NONE;
		Data.TargetGraphNodes.Empty();
		Data.bIsSteppingOut = false;
	}
	else if(Data.TargetGraphStackDepth != INDEX_NONE && Data.bIsSteppingOut)
	{
		UK2Node_Tunnel* AsTunnel = Cast<UK2Node_Tunnel>(NodeStoppedAt);
		if(AsTunnel)
		{
			// if we go through a tunnel entry/exit node update the target stack depth...
			if(AsTunnel->bCanHaveInputs)
			{
				Data.TargetGraphStackDepth += 1;
			}
			else if(AsTunnel->bCanHaveOutputs)
			{
				Data.TargetGraphStackDepth -= 1;
			}
		}
	}
}

void FKismetDebugUtilities::AttemptToBreakExecution(UBlueprint* BlueprintObj, const UObject* ActiveObject, const FFrame& StackFrame, const FBlueprintExceptionInfo& Info, UEdGraphNode* NodeStoppedAt, int32 DebugOpcodeOffset)
{
	checkSlow(BlueprintObj->GetObjectBeingDebugged() == ActiveObject);
	check(IsInGameThread());

	FKismetDebugUtilitiesData& Data = FKismetDebugUtilitiesData::Get();

	// Cannot have re-entrancy while processing a breakpoint; return from this call stack before resuming execution!
	check(!GIntraFrameDebuggingGameThread);
	check(Data.CurrentDebuggingMode == EKismetDebuggingMode::None);
	
	TGuardValue<bool> SignalGameThreadBeingDebugged(GIntraFrameDebuggingGameThread, true);
	TGuardValue<const FFrame*> ResetStackFramePointer(Data.StackFrameAtIntraframeDebugging, &StackFrame);

	// Should we pump Slate messages from this callstack, allowing intra-frame debugging?
	bool bShouldInStackDebug = false;

	if (NodeStoppedAt != NULL)
	{
		bShouldInStackDebug = true;

		Data.CurrentInstructionPointer = NodeStoppedAt;

		Data.MostRecentBreakpointInstructionPointer = NULL;

		// Find the breakpoint object for the node, assuming we hit one
		if (Info.GetType() == EBlueprintExceptionType::Breakpoint)
		{
			FBlueprintBreakpoint* Breakpoint = FindBreakpointForNode(NodeStoppedAt, BlueprintObj);

			if (Breakpoint != NULL)
			{
				Data.MostRecentBreakpointInstructionPointer = NodeStoppedAt;
				UpdateBreakpointStateWhenHit(NodeStoppedAt, BlueprintObj);
					
				//@TODO: K2: DEBUGGING: Debug print text can go eventually
				UE_LOGF(LogBlueprintDebug, Warning, "Hit breakpoint on node '%ls', from offset %d", *(NodeStoppedAt->GetDescriptiveCompiledName()), DebugOpcodeOffset);
				UE_LOGF(LogBlueprintDebug, Log, "\n%ls", *StackFrame.GetStackTrace());
			}
			else
			{
				UE_LOGF(LogBlueprintDebug, Warning, "Unknown breakpoint hit at node %ls in object %ls:%04X", *NodeStoppedAt->GetDescriptiveCompiledName(), *StackFrame.Node->GetFullName(), DebugOpcodeOffset);
			}
		}

		// Turn off single stepping; we've hit a node
		if (Data.bIsSingleStepping)
		{
			Data.bIsSingleStepping = false;
		}
	}
	else if(UEdGraphNode* PreviousNode = FKismetDebugUtilities::GetCurrentInstruction())
	{
		if (UK2Node_Message* MessageNode = Cast<UK2Node_Message>(PreviousNode))
		{
			//Looks like object not implement one of their interfaces
			UE_LOGF(LogBlueprintDebug, Warning, "Can't break execution on function '%ls'. Possibly interface '%ls' in class '%ls' was not fully implemented.",
				*(PreviousNode->GetDocumentationExcerptName()),					  //Function name
				*(MessageNode->GetTargetFunction()->GetOuterUClass()->GetName()), //Interface name
				*(ActiveObject->GetClass()->GetName()));						  //Current object class name
		}
		else
		{
			UE_LOGF(LogBlueprintDebug, Warning, "Can't break execution on function '%ls'. Possibly it was not implemented in class '%ls'.",
				*(PreviousNode->GetDocumentationExcerptName()),					  //Function name
				*(ActiveObject->GetClass()->GetName()));						  //Current object class name
		}
	}
	else
	{
		UE_LOGF(LogBlueprintDebug, Warning, "Tried to break execution in an unknown spot at object %ls:%04llX", *StackFrame.Node->GetFullName(), StackFrame.Code - StackFrame.Node->Script.GetData());
	}

	// A check to !GIsAutomationTesting was removed from here as it seemed redundant.
	// Breakpoints have to be explicitly enabled by the user which shouldn't happen 
	// under automation and this was preventing debugging on automation test bp's.
	if ((GUnrealEd->PlayWorld != NULL) && NodeStoppedAt)
	{
		Data.CurrentDebuggingMode = EKismetDebuggingMode::World;
		Data.CurrentDebuggingWorld = GUnrealEd->PlayWorld;

		// Pause the simulation
		GUnrealEd->PlayWorld->bDebugPauseExecution = true;
		GUnrealEd->PlayWorld->bDebugFrameStepExecution = false;
		bShouldInStackDebug = true;
	}
	else if (NodeStoppedAt)
	{
		Data.CurrentDebuggingMode = EKismetDebuggingMode::Engine;
	}
	else
	{
		Data.CurrentDebuggingMode = EKismetDebuggingMode::None;
		bShouldInStackDebug = false;
	}

	// Now enter within-the-frame debugging mode
	if (bShouldInStackDebug)
	{
		FTemporaryPlayInEditorIDOverride GuardDisablePIE(INDEX_NONE);
		TArrayView<const FFrame* const> ScriptStack = FBlueprintContextTracker::Get().GetCurrentScriptStack();
		Data.LastExceptionMessage = Info.GetDescription();
		FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(NodeStoppedAt);
		CallStackViewer::UpdateDisplayedCallstack(ScriptStack);
		FSlateApplication::Get().EnterDebuggingMode();
	}

	Data.CurrentDebuggingMode = EKismetDebuggingMode::None;
	Data.CurrentDebuggingWorld.Reset();
}

UEdGraphNode* FKismetDebugUtilities::GetCurrentInstruction()
{
	// If paused at the end of the frame, or while not paused, there is no 'current instruction' to speak of
	// It only has meaning during intraframe debugging.
	if (GIntraFrameDebuggingGameThread)
	{
		return FKismetDebugUtilitiesData::Get().CurrentInstructionPointer.Get();
	}
	else
	{
		return nullptr;
	}
}

UEdGraphNode* FKismetDebugUtilities::GetMostRecentBreakpointHit()
{
	// If paused at the end of the frame, or while not paused, there is no 'current instruction' to speak of
	// It only has meaning during intraframe debugging.
	if (GIntraFrameDebuggingGameThread)
	{
		return FKismetDebugUtilitiesData::Get().MostRecentBreakpointInstructionPointer.Get();
	}
	else
	{
		return nullptr;
	}
}

UWorld* FKismetDebugUtilities::GetCurrentDebuggingWorld()
{
	// If paused at the end of the frame, or while not paused, there is no 'current instruction' to speak of
	// It only has meaning during intraframe debugging.
	if (GIntraFrameDebuggingGameThread)
	{
		const FKismetDebugUtilitiesData& Data = FKismetDebugUtilitiesData::Get();
		return Data.CurrentDebuggingMode == EKismetDebuggingMode::World ? Data.CurrentDebuggingWorld.Get() : nullptr;
	}
	else
	{
		return nullptr;
	}
}

// Notify the debugger of the start of the game frame
void FKismetDebugUtilities::NotifyDebuggerOfStartOfGameFrame(UWorld* CurrentWorld)
{
}

// Notify the debugger of the end of the game frame
void FKismetDebugUtilities::NotifyDebuggerOfEndOfGameFrame(UWorld* CurrentWorld)
{
	FKismetDebugUtilitiesData::Get().bIsSingleStepping = false;
}

bool FKismetDebugUtilities::IsSingleStepping()
{
	const FKismetDebugUtilitiesData& Data = FKismetDebugUtilitiesData::Get();
	return Data.bIsSingleStepping
		|| Data.bIsSteppingOut
		|| Data.TargetGraphStackDepth != INDEX_NONE; 
}

FPerBlueprintSettings* FKismetDebugUtilities::GetPerBlueprintSettings(const UBlueprint* Blueprint)
{
	UBlueprintEditorSettings* Settings = GetMutableDefault<UBlueprintEditorSettings>();
	check(Settings);
	return Settings->PerBlueprintSettings.Find(Blueprint->GetPathName());
}

TArray<FBlueprintBreakpoint>* FKismetDebugUtilities::GetBreakpoints(const UBlueprint* Blueprint)
{
	FPerBlueprintSettings* Settings = GetPerBlueprintSettings(Blueprint);

	// return nullptr if there's no breakpoints associated w/ this blueprint
	return (!Settings || Settings->Breakpoints.IsEmpty()) ?
		nullptr :
		&Settings->Breakpoints;
}

TArray<FBlueprintWatchedPin>* FKismetDebugUtilities::GetWatchedPins(const UBlueprint* Blueprint)
{
	FPerBlueprintSettings* Settings = GetPerBlueprintSettings(Blueprint);

	// return nullptr if there's no breakpoints associated w/ this blueprint
	return (!Settings || Settings->WatchedPins.IsEmpty()) ?
		nullptr :
		&Settings->WatchedPins;
}

void FKismetDebugUtilities::SaveBlueprintEditorSettings()
{
	UBlueprintEditorSettings* Settings = GetMutableDefault<UBlueprintEditorSettings>();
	check(Settings);
	Settings->SaveConfig();
}

void FKismetDebugUtilities::CleanupBreakpoints(const UBlueprint* Blueprint)
{
	RemoveBreakpointsByPredicate(
		Blueprint,
		[Blueprint](const FBlueprintBreakpoint& Breakpoint)
		{
			if (Breakpoint.GetLocation() == nullptr)
			{
				UE_LOGF(LogBlueprintDebug, Display, "Encountered a blueprint breakpoint in %ls without an associated node. The blueprint breakpoint has been removed", *Blueprint->GetPathName());
				return true;
			}
			return false;
		}
	);
}

void FKismetDebugUtilities::CleanupWatches(const UBlueprint* Blueprint)
{
	RemovePinWatchesByPredicate(
		Blueprint,
		[Blueprint](const UEdGraphPin* Pin)->bool
		{
			if(Pin)
			{
				if(UEdGraphNode* Node = Pin->GetOwningNode())
				{
					if(UEdGraph* Graph = Node->GetGraph())
					{
						TArray<UEdGraph*> BPGraphs;
						Blueprint->GetAllGraphs(BPGraphs);
						if(BPGraphs.Contains(Graph))
						{
							return false;
						}
					}
				}
			}
			return true;
		}
	);
}

void FKismetDebugUtilities::RemoveEmptySettings(const FString& BlueprintPath)
{
	UBlueprintEditorSettings* Settings = GetMutableDefault<UBlueprintEditorSettings>();
	check(Settings);
	if(FPerBlueprintSettings *PerBlueprintSettings =
		Settings->PerBlueprintSettings.Find(BlueprintPath))
	{
		// if all settings data is default, we can remove it from the map
		if(*PerBlueprintSettings == FPerBlueprintSettings())
		{
			Settings->PerBlueprintSettings.Remove(BlueprintPath);
		}
		SaveBlueprintEditorSettings();
	}
}

//////////////////////////////////////////////////////////////////////////
// Breakpoint

// Is the node a valid breakpoint target? (i.e., the node is impure and ended up generating code)
bool FKismetDebugUtilities::IsBreakpointValid(const FBlueprintBreakpoint& Breakpoint)
{
	// Breakpoints on impure nodes in a macro graph are always considered valid
	UK2Node* K2Node = Cast<UK2Node>(Breakpoint.GetLocation());
	if (K2Node)
	{
		UBlueprint* Blueprint = Cast<UBlueprint>(K2Node->GetOuter()->GetOuter());
		if (Blueprint && Blueprint->BlueprintType == BPTYPE_MacroLibrary)
		{
			return K2Node->IsA<UK2Node_MacroInstance>()
				|| (!K2Node->IsNodePure() && !K2Node->IsA<UK2Node_Tunnel>());
		}
	}

	TArray<uint8*> InstallSites;
	GetBreakpointInstallationSites(Breakpoint, InstallSites);
	return InstallSites.Num() > 0;
}

// Set the node that the breakpoint should focus on
void FKismetDebugUtilities::SetBreakpointLocation(FBlueprintBreakpoint& Breakpoint, UEdGraphNode* NewNode)
{
	if (NewNode != Breakpoint.Node)
	{
		// Uninstall it from the old site if needed
		SetBreakpointInternal(Breakpoint, false);

		// Make the new site accurate
		Breakpoint.Node = NewNode;
		SetBreakpointInternal(Breakpoint, Breakpoint.bEnabled);
	}
}

void FKismetDebugUtilities::SetBreakpointEnabled(FBlueprintBreakpoint& Breakpoint, bool bIsEnabled)
{
	if (Breakpoint.bStepOnce && !bIsEnabled)
	{
		// Want to be disabled, but the single-stepping is keeping it enabled
		bIsEnabled = true;
		Breakpoint.bStepOnce_WasPreviouslyDisabled = true;
	}

	Breakpoint.bEnabled = bIsEnabled;
	SetBreakpointInternal(Breakpoint, Breakpoint.bEnabled);
	SaveBlueprintEditorSettings();
}

// Set or clear the enabled flag for the breakpoint
void FKismetDebugUtilities::SetBreakpointEnabled(const UEdGraphNode* OwnerNode, const UBlueprint* OwnerBlueprint, bool bIsEnabled)
{
	if(FBlueprintBreakpoint* Breakpoint = FindBreakpointForNode(OwnerNode, OwnerBlueprint))
	{
		SetBreakpointEnabled(*Breakpoint, bIsEnabled);
	}
}

// Sets this breakpoint up as a single-step breakpoint (will disable or delete itself after one go if the breakpoint wasn't already enabled)
void FKismetDebugUtilities::SetBreakpointEnabledForSingleStep(FBlueprintBreakpoint& Breakpoint, bool bDeleteAfterStep)
{
	Breakpoint.bStepOnce = true;
	Breakpoint.bStepOnce_RemoveAfterHit = bDeleteAfterStep;
	Breakpoint.bStepOnce_WasPreviouslyDisabled = !Breakpoint.bEnabled;

	SetBreakpointEnabled(Breakpoint, true);
}

void FKismetDebugUtilities::ReapplyBreakpoint(FBlueprintBreakpoint& Breakpoint)
{
	SetBreakpointInternal(Breakpoint, Breakpoint.IsEnabled());
}

void FKismetDebugUtilities::RemoveBreakpointFromNode(const UEdGraphNode* OwnerNode, const UBlueprint* OwnerBlueprint)
{
#if WITH_EDITORONLY_DATA
	RemoveBreakpointsByPredicate(
		OwnerBlueprint,
		[OwnerNode](const FBlueprintBreakpoint& Breakpoint)
		{
			return Breakpoint.GetLocation() == OwnerNode;
		}
	);
#endif	//#if WITH_EDITORONLY_DATA
}

// Update the internal state of the breakpoint when it got hit
void FKismetDebugUtilities::UpdateBreakpointStateWhenHit(const UEdGraphNode* OwnerNode, const UBlueprint* OwnerBlueprint)
{
	if (FBlueprintBreakpoint* Breakpoint = FindBreakpointForNode(OwnerNode, OwnerBlueprint))
	{
		// Handle single-step breakpoints
		if (Breakpoint->bStepOnce)
		{
			Breakpoint->bStepOnce = false;

			if (Breakpoint->bStepOnce_RemoveAfterHit)
			{
				RemoveBreakpointFromNode(Breakpoint->GetLocation(), OwnerBlueprint);
			}
			else if (Breakpoint->bStepOnce_WasPreviouslyDisabled)
			{
				SetBreakpointEnabled(*Breakpoint, false);
			}
		}
	}
}

// Install/uninstall the breakpoint into/from the script code for the generated class that contains the node
void FKismetDebugUtilities::SetBreakpointInternal(FBlueprintBreakpoint& Breakpoint, bool bShouldBeEnabled)
{
	TArray<uint8*> InstallSites;
	GetBreakpointInstallationSites(Breakpoint, InstallSites);

	for (int i = 0; i < InstallSites.Num(); ++i)
	{
		if (uint8* InstallSite = InstallSites[i])
		{
			*InstallSite = bShouldBeEnabled ? EX_Breakpoint : EX_Tracepoint;
		}
	}
}

// Returns the installation site(s); don't cache these pointers!
void FKismetDebugUtilities::GetBreakpointInstallationSites(const FBlueprintBreakpoint& Breakpoint, TArray<uint8*>& InstallSites)
{
	InstallSites.Empty();

#if WITH_EDITORONLY_DATA
	if (Breakpoint.Node != NULL)
	{
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(Breakpoint.GetLocation());

		if ((Blueprint != NULL) && (Blueprint->GeneratedClass != NULL))
		{
			if (UBlueprintGeneratedClass* Class = Cast<UBlueprintGeneratedClass>(*Blueprint->GeneratedClass))
			{
				// Find the insertion point from the debugging data
				Class->GetDebugData().FindBreakpointInjectionSites(Breakpoint.GetLocation(), InstallSites);
			}
		}
	}
#endif	//#if WITH_EDITORONLY_DATA
}

// Returns the set of valid breakpoint locations for the given macro instance node
void FKismetDebugUtilities::GetValidBreakpointLocations(const UK2Node_MacroInstance* MacroInstanceNode, TArray<const UEdGraphNode*>& BreakpointLocations)
{
	check(MacroInstanceNode);
	BreakpointLocations.Empty();

	// Gather information from the macro graph associated with the macro instance node
	bool bIsMacroPure = false;
	UK2Node_Tunnel* MacroEntryNode = NULL;
	UK2Node_Tunnel* MacroResultNode = NULL;
	UEdGraph* InstanceNodeMacroGraph = MacroInstanceNode->GetMacroGraph();
	if (ensure(InstanceNodeMacroGraph != nullptr))
	{
		FKismetEditorUtilities::GetInformationOnMacro(InstanceNodeMacroGraph, MacroEntryNode, MacroResultNode, bIsMacroPure);
	}
	if (!bIsMacroPure && MacroEntryNode)
	{
		// Get the execute pin outputs on the entry node
		for (const UEdGraphPin* ExecPin : MacroEntryNode->Pins)
		{
			if (ExecPin && ExecPin->Direction == EGPD_Output
				&& ExecPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Exec)
			{
				// For each pin linked to each execute pin, collect the node that owns it
				for (const UEdGraphPin* LinkedToPin : ExecPin->LinkedTo)
				{
					check(LinkedToPin);

					const UEdGraphNode* LinkedToNode = LinkedToPin->GetOwningNode();
					check(LinkedToNode);

					if (LinkedToNode->IsA<UK2Node_MacroInstance>())
					{
						// Recursively descend into macro instance nodes encountered in a macro graph
						TArray<const UEdGraphNode*> SubLocations;
						GetValidBreakpointLocations(Cast<const UK2Node_MacroInstance>(LinkedToNode), SubLocations);
						BreakpointLocations.Append(SubLocations);
					}
					else
					{
						BreakpointLocations.AddUnique(LinkedToNode);
					}
				}
			}
		}
	}
}

void FKismetDebugUtilities::CreateBreakpoint(const UBlueprint* Blueprint, UEdGraphNode* Node, bool bIsEnabled)
{
	UBlueprintEditorSettings* Settings = GetMutableDefault<UBlueprintEditorSettings>();
	check(Settings);
	FPerBlueprintSettings &BlueprintSettings = Settings->PerBlueprintSettings.FindOrAdd(Blueprint->GetPathName());

	// ensure that this node doesn't already contain a breakpoint
	checkSlow(!BlueprintSettings.Breakpoints.ContainsByPredicate(
		[Node](const FBlueprintBreakpoint& Other)
		{
			return Other.Node == Node;
		}
	));
	
	BlueprintSettings.Breakpoints.Emplace();
	SetBreakpointEnabled(BlueprintSettings.Breakpoints.Top(), bIsEnabled);
	SetBreakpointLocation(BlueprintSettings.Breakpoints.Top(), Node);
	SaveBlueprintEditorSettings();
}

void FKismetDebugUtilities::ForeachBreakpoint(const UBlueprint* Blueprint,
	const TFunctionRef<void(FBlueprintBreakpoint&)> Task)
{
	if(TArray<FBlueprintBreakpoint>* Breakpoints = GetBreakpoints(Blueprint))
	{
		for(FBlueprintBreakpoint& Breakpoint : *Breakpoints)
		{
			Task(Breakpoint);
		}
	}
}

void FKismetDebugUtilities::RemoveBreakpointsByPredicate(const UBlueprint* Blueprint,
                                                         const TFunctionRef<bool(const FBlueprintBreakpoint&)> Predicate)
{
	if(TArray<FBlueprintBreakpoint>* Breakpoints = GetBreakpoints(Blueprint))
	{
		// notify the debugger of the breakpoints being removed
		for(FBlueprintBreakpoint& Breakpoint : *Breakpoints)
		{
			if(Predicate(Breakpoint))
			{
				SetBreakpointLocation(Breakpoint, nullptr);
			}
		}

		// remove the breakpoints from the data
		if(Breakpoints->RemoveAllSwap(Predicate, EAllowShrinking::No))
		{
			if(Breakpoints->IsEmpty())
			{
				// keeps the ini file clean by removing empty arrays
				ClearBreakpoints(Blueprint);
			}
			SaveBlueprintEditorSettings();
		}
	}
}

FBlueprintBreakpoint* FKismetDebugUtilities::FindBreakpointByPredicate(const UBlueprint* Blueprint,
                                                              const TFunctionRef<bool(const FBlueprintBreakpoint&)> Predicate)
{
	if(TArray<FBlueprintBreakpoint>* Breakpoints = GetBreakpoints(Blueprint))
	{
		for(FBlueprintBreakpoint& Breakpoint : *Breakpoints)
		{
			if(Predicate(Breakpoint))
			{
				return &Breakpoint;
			}
		}
	}
	return nullptr;
}

// Finds a breakpoint for a given node if it exists, or returns NULL
FBlueprintBreakpoint* FKismetDebugUtilities::FindBreakpointForNode(const UEdGraphNode* OwnerNode, const UBlueprint* OwnerBlueprint,
                                                          bool bCheckSubLocations)
{
	// remove expired data from deleted nodes and such
	CleanupBreakpoints(OwnerBlueprint);

	// find breakpoint
	return FindBreakpointByPredicate(
		OwnerBlueprint,
		[OwnerNode, bCheckSubLocations](const FBlueprintBreakpoint &Breakpoint)
		{
			UEdGraphNode* BreakpointLoaction = Breakpoint.GetLocation();
			// Return this breakpoint if the location matches the given node
			if (BreakpointLoaction == OwnerNode)
			{
				return true;
			}
			if (bCheckSubLocations)
			{
				// If this breakpoint is set on a macro instance node, check the set of valid breakpoint locations. If we find a
				// match in the returned set, return the breakpoint that's set on the macro instance node. This allows breakpoints
				// to be set and hit on macro instance nodes contained in a macro graph that will be expanded during compile.
				const UK2Node_MacroInstance* MacroInstanceNode = Cast<UK2Node_MacroInstance>(BreakpointLoaction);
				if (MacroInstanceNode)
				{
					TArray<const UEdGraphNode*> ValidBreakpointLocations;
					GetValidBreakpointLocations(MacroInstanceNode, ValidBreakpointLocations);
					if (ValidBreakpointLocations.Contains(OwnerNode))
					{
						return true;
					}
				}
			}
			return false;
		}
	);
}

void FKismetDebugUtilities::RestoreBreakpointsOnLoad(const UBlueprint* Blueprint)
{
	if (!Blueprint)
	{
		return;
	}

#if WITH_EDITORONLY_DATA
	// Remove stale breakpoints
	RemoveBreakpointsByPredicate(
		Blueprint,
		[Blueprint](const FBlueprintBreakpoint& Breakpoint)
		{
			const UEdGraphNode* const Location = Breakpoint.GetLocation();
			return !Location || Location->GetPackage()->GetPersistentGuid() != Blueprint->GetPackage()->GetPersistentGuid();
		}
	);
#endif

	// Restore breakpoints based on preferred method
	const UBlueprintEditorSettings* BlueprintEditorSettings = GetDefault<UBlueprintEditorSettings>();
	switch (BlueprintEditorSettings->BreakpointReloadMethod)
	{
	case EBlueprintBreakpointReloadMethod::RestoreAllAndDisable:
		ForeachBreakpoint(
			Blueprint,
			[](FBlueprintBreakpoint& Breakpoint)
			{
				SetBreakpointEnabled(Breakpoint, false);
			}
		);
		break;

	case EBlueprintBreakpointReloadMethod::DiscardAll:
		ClearBreakpoints(Blueprint);
		break;

	case EBlueprintBreakpointReloadMethod::RestoreAll:
	default:
		break;
	}
}

bool FKismetDebugUtilities::HasDebuggingData(const UBlueprint* Blueprint)
{
	return Cast<UBlueprintGeneratedClass>(*Blueprint->GeneratedClass)->GetDebugData().IsValid();
}

//////////////////////////////////////////////////////////////////////////
// Blueprint utils

void FKismetDebugUtilities::PostDuplicateBlueprint(UBlueprint* SrcBlueprint, UBlueprint* DupBlueprint, const TArray<UEdGraphNode*>& DupNodes)
{
	// Duplicate Breakpoints from the source blueprint
	if (TArray<FBlueprintBreakpoint>* Breakpoints = GetBreakpoints(SrcBlueprint))
	{
		for (FBlueprintBreakpoint& Breakpoint : *Breakpoints)
		{
			if (UEdGraphNode* Location = Breakpoint.GetLocation())
			{
				UEdGraphNode* const* NewLocation = DupNodes.FindByPredicate(
					[Location](const UEdGraphNode* Node) -> bool
					{
						return Node->NodeGuid == Location->NodeGuid;
					}
				);
				if (NewLocation)
				{
					CreateBreakpoint(DupBlueprint, *NewLocation, Breakpoint.IsEnabled());
				}
			}
		}
	}

	// Duplicate Watched Pins
	if (TArray<FBlueprintWatchedPin>* WatchedPins = GetWatchedPins(SrcBlueprint))
	{
		for (FBlueprintWatchedPin& WatchedPin : *WatchedPins)
		{
			if (UEdGraphPin* Pin = WatchedPin.Get())
			{
				UEdGraphNode* const* NewOwningNode = DupNodes.FindByPredicate(
					[Pin](const UEdGraphNode* Node) -> bool
					{
						return Node->NodeGuid == Pin->GetOwningNode()->NodeGuid;
					}
				);

				if (NewOwningNode)
				{
					if (const UEdGraphPin* NewPin = (*NewOwningNode)->FindPin(Pin->PinName))
					{
						AddPinWatch(DupBlueprint, NewPin);
					}
				}
			}
		}
	}
}

bool FKismetDebugUtilities::BlueprintHasBreakpoints(const UBlueprint* Blueprint)
{
	return GetBreakpoints(Blueprint) != nullptr;
}

// Looks thru the debugging data for any class variables associated with the node
FProperty* FKismetDebugUtilities::FindClassPropertyForPin(UBlueprint* Blueprint, const UEdGraphPin* Pin)
{
	FProperty* FoundProperty = nullptr;

	if (Pin)
	{
		// Input Pins linked to a reroute node should use debug property from the reroute node's input pin
		if (Pin->Direction == EGPD_Input && Pin->LinkedTo.Num() == 1)
		{
			if (const UK2Node_Knot* LinkedKnot = Cast<UK2Node_Knot>(Pin->LinkedTo[0]->GetOwningNode()))
			{
				return FindClassPropertyForPin(Blueprint, LinkedKnot->GetInputPin());
			}
		}

		// Reroute nodes should always use the input pin, not the output pin
		if (const UK2Node_Knot* OwningKnot = Cast<UK2Node_Knot>(Pin->GetOwningNode()))
		{
			if (Pin->Direction == EGPD_Output)
			{
				return FindClassPropertyForPin(Blueprint, OwningKnot->GetInputPin());
			}
		}
	}

	UClass* Class = Blueprint->GeneratedClass;
	while (UBlueprintGeneratedClass* BlueprintClass = Cast<UBlueprintGeneratedClass>(Class))
	{
		FoundProperty = BlueprintClass->GetDebugData().FindClassPropertyForPin(Pin);
		if (FoundProperty != nullptr)
		{
			break;
		}

		Class = BlueprintClass->GetSuperClass();
	}

	return FoundProperty;
}

// Looks thru the debugging data for any class variables associated with the node (e.g., temporary variables or timelines)
FProperty* FKismetDebugUtilities::FindClassPropertyForNode(UBlueprint* Blueprint, const UEdGraphNode* Node)
{
	if (UBlueprintGeneratedClass* Class = Cast<UBlueprintGeneratedClass>(*Blueprint->GeneratedClass))
	{
		return Class->GetDebugData().FindClassPropertyForNode(Node);
	}

	return NULL;
}


void FKismetDebugUtilities::ClearBreakpoints(const UBlueprint* Blueprint)
{
	ClearBreakpointsForPath(Blueprint->GetPathName());
}


void FKismetDebugUtilities::ClearBreakpointsForPath(const FString& BlueprintPath)
{	
	UBlueprintEditorSettings* Settings = GetMutableDefault<UBlueprintEditorSettings>();
	check(Settings);
	if(FPerBlueprintSettings *PerBlueprintSettings =
		Settings->PerBlueprintSettings.Find(BlueprintPath))
	{
		for(FBlueprintBreakpoint& Breakpoint : PerBlueprintSettings->Breakpoints)
		{
			// notify debugger that the breakpont has been removed
			SetBreakpointLocation(Breakpoint, nullptr);	
		}
		PerBlueprintSettings->Breakpoints.Empty();
		
		RemoveEmptySettings(BlueprintPath);
        
		SaveBlueprintEditorSettings();
	}
}

FKismetDebugUtilities::FOnWatchedPinsListChanged FKismetDebugUtilities::WatchedPinsListChangedEvent;

bool FKismetDebugUtilities::CanWatchPin(const UBlueprint* Blueprint, const UEdGraphPin* Pin, const TArray<FName>& InPathToProperty)
{
	// Forward to schema
	if (const UEdGraphNode* Node = Pin->GetOwningNode())
	{
		if (const UAnimationGraphSchema* AnimationGraphSchema = Cast<UAnimationGraphSchema>(Node->GetSchema()))
		{
			// Anim graphs need to respect whether they have a binding as they are effectively unlinked
			bool bHasBinding = false;

			if (UAnimGraphNode_Base* AnimGraphNode = Cast<UAnimGraphNode_Base>(Pin->GetOwningNode()))
			{
				if(AnimGraphNode->HasBinding(Pin->GetFName()))
				{
					bHasBinding = true;
				}
			}

			UEdGraph* Graph = Pin->GetOwningNode()->GetGraph();

			// We allow input pins to be watched only if they have bindings, otherwise we need to follow to output pins
			const bool bNotAnInputOrBound = (Pin->Direction != EGPD_Input) || bHasBinding;

			return !AnimationGraphSchema->IsMetaPin(*Pin) && bNotAnInputOrBound && !IsPinBeingWatched(Blueprint, Pin, InPathToProperty);
		}
		else if (const UEdGraphSchema_K2* K2Schema = Cast<UEdGraphSchema_K2>(Node->GetSchema()))
		{
			UEdGraph* Graph = Pin->GetOwningNode()->GetGraph();

			// Inputs should always be followed to their corresponding output in the world above
			const bool bNotAnInput = (Pin->Direction != EGPD_Input);

			//@TODO: Make watching a schema-allowable/denyable thing
			const bool bCanWatchThisGraph = true;

			return bCanWatchThisGraph && !K2Schema->IsMetaPin(*Pin) && bNotAnInput && !IsPinBeingWatched(Blueprint, Pin, InPathToProperty);
		}
	}

	return false;
}

bool FKismetDebugUtilities::IsPinBeingWatched(const UBlueprint* Blueprint, const UEdGraphPin* Pin, const TArray<FName>& InPathToProperty)
{
	if (TArray<FBlueprintWatchedPin>* WatchedPins = GetWatchedPins(Blueprint))
	{
		for (const FBlueprintWatchedPin& WatchedPin : *WatchedPins)
		{
			if (WatchedPin.Get() == Pin && WatchedPin.GetPathToProperty() == InPathToProperty)
			{
				return true;
			}
		}
	}
	return false;
}

bool FKismetDebugUtilities::DoesPinHaveWatches(const UBlueprint* Blueprint, const UEdGraphPin* Pin)
{
	if (TArray<FBlueprintWatchedPin>* WatchedPins = GetWatchedPins(Blueprint))
	{
		for (const FBlueprintWatchedPin& WatchedPin : *WatchedPins)
		{
			if (WatchedPin.Get() == Pin)
			{
				return true;
			}
		}
	}
	return false;
}

bool FKismetDebugUtilities::RemovePinWatch(const UBlueprint* Blueprint, const UEdGraphPin* Pin, const TArray<FName>& InPathToProperty)
{
	return RemovePinPropertyWatchesByPredicate(
		Blueprint,
		[Pin, &InPathToProperty](const FBlueprintWatchedPin& Other)
		{
			return Other.Get()->PinId == Pin->PinId && Other.GetPathToProperty() == InPathToProperty;
		}
	);
}

void FKismetDebugUtilities::AddPinWatch(const UBlueprint* Blueprint, FBlueprintWatchedPin&& WatchedPin)
{
	UBlueprintEditorSettings* Settings = GetMutableDefault<UBlueprintEditorSettings>();
	check(Settings);
	FPerBlueprintSettings& BlueprintSettings = Settings->PerBlueprintSettings.FindOrAdd(Blueprint->GetPathName());

	BlueprintSettings.WatchedPins.Emplace(MoveTemp(WatchedPin));

	SaveBlueprintEditorSettings();
	WatchedPinsListChangedEvent.Broadcast(const_cast<UBlueprint*>(Blueprint));
}

void FKismetDebugUtilities::TogglePinWatch(const UBlueprint* Blueprint, const UEdGraphPin* Pin)
{
	if (IsPinBeingWatched(Blueprint, Pin))
	{
		RemovePinWatch(Blueprint, Pin);
	}
	else
	{
		AddPinWatch(Blueprint, Pin);
	}
}

void FKismetDebugUtilities::ClearPinWatches(const UBlueprint* Blueprint)
{
	UBlueprintEditorSettings* Settings = GetMutableDefault<UBlueprintEditorSettings>();
	check(Settings);
	if(FPerBlueprintSettings *PerBlueprintSettings =
		Settings->PerBlueprintSettings.Find(Blueprint->GetPathName()))
	{
		PerBlueprintSettings->WatchedPins.Empty();
		
		RemoveEmptySettings(Blueprint->GetPathName());
        		
        SaveBlueprintEditorSettings();
        WatchedPinsListChangedEvent.Broadcast(const_cast<UBlueprint*>(Blueprint));
	}
}

bool FKismetDebugUtilities::BlueprintHasPinWatches(const UBlueprint* Blueprint)
{
	return GetWatchedPins(Blueprint) != nullptr;
}

void FKismetDebugUtilities::ForeachPinWatch(const UBlueprint* Blueprint, TFunctionRef<void(UEdGraphPin*)> Task)
{
	if(TArray<FBlueprintWatchedPin>* WatchedPins = GetWatchedPins(Blueprint))
	{
		for(FBlueprintWatchedPin& WatchedPin : *WatchedPins)
		{
			if (UEdGraphPin* Pin = WatchedPin.Get())
			{
				Task(Pin);
			}
		}
	}
}

void FKismetDebugUtilities::ForeachPinPropertyWatch(const UBlueprint* Blueprint, TFunctionRef<void(FBlueprintWatchedPin&)> Task)
{
	if (TArray<FBlueprintWatchedPin>* WatchedPins = GetWatchedPins(Blueprint))
	{
		for (FBlueprintWatchedPin& WatchedPin : *WatchedPins)
		{
			Task(WatchedPin);
		}
	}
}

bool FKismetDebugUtilities::RemovePinWatchesByPredicate(const UBlueprint* Blueprint,
	const TFunctionRef<bool(const UEdGraphPin*)> Predicate)
{
	auto ModifiedPedicate = [Predicate](FBlueprintWatchedPin& WatchedPin)
	{
		const UEdGraphPin* Pin = WatchedPin.Get();
		return Pin && Predicate(Pin);
	};
	
	if(TArray<FBlueprintWatchedPin>* WatchedPins = GetWatchedPins(Blueprint))
	{
		if(WatchedPins->RemoveAllSwap(ModifiedPedicate, EAllowShrinking::No))
		{
			if(WatchedPins->IsEmpty())
			{
				// keeps the ini file clean by removing empty arrays
				ClearPinWatches(Blueprint);
			}
			SaveBlueprintEditorSettings();
			WatchedPinsListChangedEvent.Broadcast(const_cast<UBlueprint*>(Blueprint));
			return true;
		}
	}
	return false;
}

bool FKismetDebugUtilities::RemovePinPropertyWatchesByPredicate(const UBlueprint* Blueprint, const TFunctionRef<bool(const FBlueprintWatchedPin&)> Predicate)
{
	auto ModifiedPedicate = [Predicate](FBlueprintWatchedPin& WatchedPin)
	{
		const UEdGraphPin* Pin = WatchedPin.Get();
		return Pin && Predicate(WatchedPin);
	};

	if (TArray<FBlueprintWatchedPin>* WatchedPins = GetWatchedPins(Blueprint))
	{
		if (WatchedPins->RemoveAllSwap(ModifiedPedicate, EAllowShrinking::No))
		{
			if (WatchedPins->IsEmpty())
			{
				// keeps the ini file clean by removing empty arrays
				ClearPinWatches(Blueprint);
			}
			SaveBlueprintEditorSettings();
			WatchedPinsListChangedEvent.Broadcast(const_cast<UBlueprint*>(Blueprint));
			return true;
		}
	}
	return false;
}

UEdGraphPin* FKismetDebugUtilities::FindPinWatchByPredicate(const UBlueprint* Blueprint,
	const TFunctionRef<bool(const UEdGraphPin*)> Predicate)
{
	if(TArray<FBlueprintWatchedPin>* WatchedPins = GetWatchedPins(Blueprint))
	{
		for(FBlueprintWatchedPin& WatchedPin : *WatchedPins)
		{
			UEdGraphPin* Pin = WatchedPin.Get();
			if(Pin && Predicate(Pin))
			{
				return Pin;
			}
		}
	}
	return nullptr;
}

// Gets the watched tooltip for a specified site
FKismetDebugUtilities::EWatchTextResult FKismetDebugUtilities::GetWatchText(FString& OutWatchText, UBlueprint* Blueprint, UObject* ActiveObject, const UEdGraphPin* WatchPin)
{
	const FProperty* PropertyToDebug = nullptr;
	const void* DataPtr = nullptr;
	const void* DeltaPtr = nullptr;
	UObject* ParentObj = nullptr;
	TArray<UObject*> SeenObjects;
	bool bIsDirectPtr = false;
	FKismetDebugUtilities::EWatchTextResult Result = FindDebuggingData(Blueprint, ActiveObject, WatchPin, PropertyToDebug, DataPtr, DeltaPtr, ParentObj, SeenObjects, &bIsDirectPtr);

	if (Result == FKismetDebugUtilities::EWatchTextResult::EWTR_Valid)
	{
		// If this came from an out parameter it isn't in a property container, so must be accessed directly
		if (bIsDirectPtr)
		{
			PropertyToDebug->ExportText_Direct(/*inout*/ OutWatchText, DataPtr, DeltaPtr, ParentObj, PPF_PropertyWindow | PPF_BlueprintDebugView);
		}
		else
		{
			PropertyToDebug->ExportText_InContainer(/*ArrayElement=*/ 0, /*inout*/ OutWatchText, DataPtr, DeltaPtr, /*Parent=*/ ParentObj, PPF_PropertyWindow | PPF_BlueprintDebugView);
		}
	}

	return Result;
}

bool FKismetDebugUtilities::CanInspectPinValue(const UEdGraphPin* Pin)
{
	const UBlueprintEditorSettings* BlueprintEditorSettings = GetDefault<UBlueprintEditorSettings>();
	if (!BlueprintEditorSettings->bEnablePinValueInspectionTooltips)
	{
		return false;
	}

	// Can't inspect the value on an invalid pin object.
	if (!Pin || Pin->IsPendingKill())
	{
		return false;
	}

	// Can't inspect the value on an orphaned pin object.
	if (Pin->bOrphanedPin)
	{
		return false;
	}

	// Can't inspect the value on an unknown pin object or if the owning node is disabled.
	const UEdGraphNode* OwningNode = Pin->GetOwningNodeUnchecked();
	if (!OwningNode || !OwningNode->IsNodeEnabled())
	{
		return false;
	}

	// Can't inspect exec pins or delegate pins; their values are not defined.
	// Disallow non-K2 Schemas (like ControlRig)
	const UEdGraphSchema_K2* K2Schema = Cast<UEdGraphSchema_K2>(OwningNode->GetSchema());
	if (!K2Schema || !K2Schema->CanShowDataTooltipForPin(*Pin))
	{
		return false;
	}

	// Can't inspect the value if there is no active debug context.
	const UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(OwningNode);
	if (!Blueprint || !Blueprint->GetObjectBeingDebugged())
	{
		return false;
	}

	// Can't inspect if a debug object isn't selected
	const UObject* Object = Blueprint->GetObjectBeingDebugged();
	if (!Object)
	{
		return false;
	}

	// Can't inspect if not in PIE
	const UWorld* OwningWorld = Object->GetWorld();
	if (!OwningWorld || !(OwningWorld->IsPlayInEditor() || OwningWorld->IsPreviewWorld()))
	{
		return false;
	}

	return true;
}

FKismetDebugUtilities::EWatchTextResult FKismetDebugUtilities::GetDebugInfo(TSharedPtr<FPropertyInstanceInfo> &OutDebugInfo, UBlueprint* Blueprint, UObject* ActiveObject, const UEdGraphPin* WatchPin)
{
	const void* DataPtr = nullptr;
	const void* DeltaPtr = nullptr;
	UObject* ParentObj = nullptr;
	TArray<UObject*> SeenObjects;
	bool bIsDirectPtr = false;
	const FProperty* Property = nullptr;
	FKismetDebugUtilities::EWatchTextResult Result = FindDebuggingData(Blueprint, ActiveObject, WatchPin, Property, DataPtr, DeltaPtr, ParentObj, SeenObjects, &bIsDirectPtr);

	if (Result == FKismetDebugUtilities::EWatchTextResult::EWTR_Valid)
	{
		// If this came from an out parameter it isn't in a property container, so must be accessed directly
		if (bIsDirectPtr)
		{
			GetDebugInfoInternal(OutDebugInfo, Property, DataPtr);
		}
		else
		{
			GetDebugInfo_InContainer(0, OutDebugInfo, Property, DataPtr);
		}
	}

	return Result;
}

FKismetDebugUtilities::EWatchTextResult FKismetDebugUtilities::FindDebuggingData(UBlueprint* Blueprint, UObject* ActiveObject, const UEdGraphPin* WatchPin, const FProperty*& OutProperty, const void*& OutData, const void*& OutDelta, UObject*& OutParent, TArray<UObject*>& SeenObjects, bool* bOutIsDirectPtr /* = nullptr */)
{
	FKismetDebugUtilitiesData& Data = FKismetDebugUtilitiesData::Get();

	if (FProperty* Property = FKismetDebugUtilities::FindClassPropertyForPin(Blueprint, WatchPin))
	{
		if (!Property->IsValidLowLevel())
		{
			//@TODO: Temporary checks to attempt to determine intermittent unreproducable crashes in this function
			static bool bErrorOnce = true;
			if (bErrorOnce)
			{
				ensureMsgf(false, TEXT("Error: Invalid (but non-null) property associated with pin; cannot get variable value"));
				bErrorOnce = false;
			}
			return EWTR_NoProperty;
		}

		if (ActiveObject != nullptr)
		{
			if (!ActiveObject->IsValidLowLevel())
			{
				//@TODO: Temporary checks to attempt to determine intermittent unreproducable crashes in this function
				static bool bErrorOnce = true;
				if (bErrorOnce)
				{
					ensureMsgf(false, TEXT("Error: Invalid (but non-null) active object being debugged; cannot get variable value for property %s"), *Property->GetPathName());
					bErrorOnce = false;
				}
				return EWTR_NoDebugObject;
			}

			void* PropertyBase = nullptr;

			// Walk up the stack frame to see if we can find a function scope that contains the property as a local
			for (const FFrame* TestFrame = Data.StackFrameAtIntraframeDebugging; TestFrame != NULL; TestFrame = TestFrame->PreviousFrame)
			{
				if (Property->IsIn(TestFrame->Node))
				{
					// output parameters need special handling
					for (FOutParmRec* OutParmRec = TestFrame->OutParms; OutParmRec != nullptr; OutParmRec = OutParmRec->NextOutParm)
					{
						if (OutParmRec->Property == Property)
						{
							if (WatchPin->Direction == EEdGraphPinDirection::EGPD_Input)
							{
								// try to use the output pin we're linked to
								// otherwise the output param won't show any data since the return node hasn't executed when we stop here
								if (WatchPin->LinkedTo.Num() == 1)
								{
									return FindDebuggingData(Blueprint, ActiveObject, WatchPin->LinkedTo[0], OutProperty, OutData, OutDelta, OutParent, SeenObjects, bOutIsDirectPtr);
								}
								else if (!WatchPin->LinkedTo.Num())
								{
									// If this is an output pin with no links, then we have no debug data
									// so fallback to the local stack frame
									PropertyBase = TestFrame->Locals;
								}
							}

							if (PropertyBase == nullptr && bOutIsDirectPtr)
							{
								// Flag to caller that PropertyBase points directly at the data, not at the
								// base of a property container (so don't apply property's Offset_Internal)
								*bOutIsDirectPtr = true;
								PropertyBase = OutParmRec->PropAddr;
							}
							break;
						}
					}

					// Fallback to the local variables if we couldn't find one
					if (PropertyBase == nullptr)
					{
						PropertyBase = TestFrame->Locals;
					}
					break;
				}
			}

			// Try at member scope if it wasn't part of a current function scope
			UClass* PropertyClass = Property->GetOwner<UClass>();
			if (!PropertyBase && PropertyClass)
			{
				if (ActiveObject->GetClass()->IsChildOf(PropertyClass))
				{
					PropertyBase = ActiveObject;
				}
				else if (AActor* Actor = Cast<AActor>(ActiveObject))
				{
					// Try and locate the propertybase in the actor components
					for (UActorComponent* ComponentIter : Actor->GetComponents())
					{
						if (ComponentIter->GetClass()->IsChildOf(PropertyClass))
						{
							PropertyBase = ComponentIter;
							break;
						}
					}
				}
			}

			// Try find the propertybase in the persistent ubergraph frame
			UFunction* OuterFunction = Property->GetOwner<UFunction>();
			if (!PropertyBase && OuterFunction)
			{
				UBlueprintGeneratedClass* BPGC = Cast<UBlueprintGeneratedClass>(Blueprint->GeneratedClass);
				if (BPGC && ActiveObject->IsA(BPGC))
				{
					PropertyBase = BPGC->GetPersistentUberGraphFrame(ActiveObject, OuterFunction);
				}
			}

			// see if our WatchPin is on a animation node & if so try to get its property info
			const UAnimBlueprintGeneratedClass* AnimBlueprintGeneratedClass = Cast<UAnimBlueprintGeneratedClass>(Blueprint->GeneratedClass);

			// Use the root anim BP's debug data - derived anim BPs have empty debug data
			if(AnimBlueprintGeneratedClass)
			{
				if (UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(AnimBlueprintGeneratedClass->ClassGeneratedBy))
				{
					if(UAnimBlueprint* RootAnimBP = UAnimBlueprint::FindRootAnimBlueprint(AnimBlueprint))
					{
						AnimBlueprintGeneratedClass = RootAnimBP->GetAnimBlueprintGeneratedClass();
					}
				}
			}

			if (!PropertyBase && AnimBlueprintGeneratedClass)
			{
				// are we linked to an anim graph node?
				UEdGraphPin* LinkedPin = nullptr;
				const FProperty* LinkedProperty = Property;
				const UAnimGraphNode_Base* Node = Cast<UAnimGraphNode_Base>(WatchPin->GetOuter());
				if (Node == nullptr && WatchPin->LinkedTo.Num() > 0)
				{
					LinkedPin = WatchPin->LinkedTo[0];
					// When we change Node we *must* change Property, so it's still a sub-element of that.
					LinkedProperty = FKismetDebugUtilities::FindClassPropertyForPin(Blueprint, LinkedPin);
					Node = Cast<UAnimGraphNode_Base>(LinkedPin->GetOuter());
				}

				if (Node && LinkedProperty)
				{
					// In case the property was folded its value has to be retrieved from the Mutable data struct on the instance rather than from the Anim Node itself
					if (FProperty* const* FoldedPropertyPathPtr = AnimBlueprintGeneratedClass->AnimBlueprintDebugData.GraphPinToFoldedPropertyMap.Find(LinkedPin))
					{
						const FProperty* FoldedProperty = *FoldedPropertyPathPtr;
						const UStruct* FoldedPropertyStruct = FoldedProperty ? FoldedProperty->GetOwnerStruct() : nullptr;

						if(FoldedPropertyStruct && FoldedPropertyStruct->IsChildOf(FAnimBlueprintMutableData::StaticStruct()))
						{
							const FAnimBlueprintMutableData* MutableData = AnimBlueprintGeneratedClass->GetMutableNodeData(Cast<const UObject>(ActiveObject));
						
							if(bOutIsDirectPtr && MutableData)
							{
								const void* ValuePtr = FoldedProperty->ContainerPtrToValuePtr<void>(MutableData);
								OutProperty = FoldedProperty;
								OutData = ValuePtr;
								OutDelta = ValuePtr;
								OutParent = nullptr;
										
								// Flag to caller that OutData points directly at the data, not at the
								// base of a property container (so don't apply property's Offset_Internal)
								*bOutIsDirectPtr = true;
								
								return EWTR_Valid;
							}
						}
					}
					else if (FStructProperty* NodeStructProperty = CastField<FStructProperty>(FKismetDebugUtilities::FindClassPropertyForNode(Blueprint, Node)))
					{						
						for (const FStructPropertyPath& NodeProperty : AnimBlueprintGeneratedClass->GetAnimNodeProperties())
						{
							if (NodeProperty.Get() == NodeStructProperty)
							{
								const void* NodePtr = NodeProperty->ContainerPtrToValuePtr<void>(ActiveObject);
								OutProperty = LinkedProperty;
								OutData = NodePtr;
								OutDelta = NodePtr;
								OutParent = ActiveObject;
								return EWTR_Valid;
							}
						}
					}
				}
			}

			// If we still haven't found a result, try changing the active object to whatever is passed into the self pin.
			if (!PropertyBase)
			{
				UEdGraphNode* WatchNode = WatchPin->GetOwningNode();

				if (WatchNode)
				{
					UEdGraphPin* SelfPin = WatchNode->FindPin(TEXT("self"));
					if (SelfPin && SelfPin != WatchPin)
					{
						const FProperty* SelfPinProperty = nullptr;
						const void* SelfPinData = nullptr;
						const void* SelfPinDelta = nullptr;
						UObject* SelfPinParent = nullptr;
						SeenObjects.AddUnique(ActiveObject);
						FKismetDebugUtilities::EWatchTextResult Result = FindDebuggingData(Blueprint, ActiveObject, SelfPin, SelfPinProperty, SelfPinData, SelfPinDelta, SelfPinParent, SeenObjects);
						const FObjectPropertyBase* SelfPinPropertyBase = CastField<const FObjectPropertyBase>(SelfPinProperty);
						if (Result == EWTR_Valid && SelfPinPropertyBase != nullptr)
						{
							const void* PropertyValue = SelfPinProperty->ContainerPtrToValuePtr<void>(SelfPinData);
							UObject* TempActiveObject = SelfPinPropertyBase->GetObjectPropertyValue(PropertyValue);
							if (TempActiveObject && TempActiveObject != ActiveObject)
							{
								if (!SeenObjects.Contains(TempActiveObject))
								{
									return FindDebuggingData(Blueprint, TempActiveObject, WatchPin, OutProperty, OutData, OutDelta, OutParent, SeenObjects);
								}
							}
						}
					}
				}
			}

			// Now either print out the variable value, or that it was out-of-scope
			if (PropertyBase != nullptr)
			{
				OutProperty = Property;
				OutData = PropertyBase;
				OutDelta = PropertyBase;
				OutParent = ActiveObject;
				return EWTR_Valid;
			}
			else
			{
				return EWTR_NotInScope;
			}
		}
		else
		{
			return EWTR_NoDebugObject;
		}
	}
	else
	{
		return EWTR_NoProperty;
	}
}

void FKismetDebugUtilities::GetDebugInfo_InContainer(int32 Index, TSharedPtr<FPropertyInstanceInfo> &DebugInfo, const FProperty* Property, const void* Data)
{
	GetDebugInfoInternal(DebugInfo, Property, Property->ContainerPtrToValuePtr<void>(Data, Index));
}

void FKismetDebugUtilities::GetDebugInfoInternal(TSharedPtr<FPropertyInstanceInfo> &DebugInfo, const FProperty* Property, const void* PropertyValue)
{
	DebugInfo = FPropertyInstanceInfo::Make({Property, PropertyValue}, nullptr);
}

FPropertyInstanceInfo::FPropertyInstanceInfo(FPropertyInstance PropertyInstance, const TSharedPtr<FPropertyInstanceInfo>& InParent) :
	Name(FText::FromString(PropertyInstance.Property->GetName())),
	DisplayName(PropertyInstance.Property->GetDisplayNameText()),
	Type(UEdGraphSchema_K2::TypeToText(PropertyInstance.Property)),
	Property(PropertyInstance.Property),
	ValueAddress(PropertyInstance.Value),
	Parent(InParent)
{
	const FProperty* ResolvedProperty = Property.Get();
	check(ResolvedProperty);
	if (PropertyInstance.Value == nullptr)
	{
		return;
	}

	if (const FByteProperty* ByteProperty = CastField<FByteProperty>(ResolvedProperty))
	{
		UEnum* Enum = ByteProperty->GetIntPropertyEnum();
		if (Enum)
		{
			if (Enum->IsValidEnumValue(*(const uint8*)PropertyInstance.Value))
			{
				Value = Enum->GetDisplayNameTextByValue(*(const uint8*)PropertyInstance.Value);
			}
			else
			{
				Value = FText::FromString(TEXT("(INVALID)"));
			}

			return;
		}

		// if there is no Enum we need to fall through and treat this as a FNumericProperty
	}

	if (const FNumericProperty* NumericProperty = CastField<FNumericProperty>(ResolvedProperty))
	{
		Value = FText::FromString(NumericProperty->GetNumericPropertyValueToString(PropertyInstance.Value));
		return;
	}
	else if (const FBoolProperty* BoolProperty = CastField<FBoolProperty>(ResolvedProperty))
	{
		const FCoreTexts& CoreTexts = FCoreTexts::Get();

		Value = BoolProperty->GetPropertyValue(PropertyInstance.Value) ? CoreTexts.True : CoreTexts.False;
		return;
	}
	else if (const FNameProperty* NameProperty = CastField<FNameProperty>(ResolvedProperty))
	{
		Value = FText::FromName(*(FName*)PropertyInstance.Value);
		return;
	}
	else if (const FTextProperty* TextProperty = CastField<FTextProperty>(ResolvedProperty))
	{
		Value = TextProperty->GetPropertyValue(PropertyInstance.Value);
		return;
	}
	else if (const FStrProperty* StringProperty = CastField<FStrProperty>(ResolvedProperty))
	{
		Value = FText::FromString(StringProperty->GetPropertyValue(PropertyInstance.Value));
		return;
	}
	else if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(ResolvedProperty))
	{
		checkSlow(ArrayProperty->Inner);

		FScriptArrayHelper ArrayHelper(ArrayProperty, PropertyInstance.Value);

		Value = FText::Format(LOCTEXT("ArraySize", "Num={0}"), FText::AsNumber(ArrayHelper.Num()));
		return;
	}
	else if (const FStructProperty* StructProperty = CastField<FStructProperty>(ResolvedProperty))
	{
		FString WatchText;
		StructProperty->ExportTextItem_Direct(WatchText, PropertyInstance.Value, PropertyInstance.Value, nullptr, PPF_PropertyWindow | PPF_BlueprintDebugView, nullptr);
		Value = FText::FromString(WatchText);
		return;
	}
	else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(ResolvedProperty))
	{
		FNumericProperty* LocalUnderlyingProp = EnumProperty->GetUnderlyingProperty();
		UEnum* Enum = EnumProperty->GetEnum();

		int64 NumValue = LocalUnderlyingProp->GetSignedIntPropertyValue(PropertyInstance.Value);

		// if the value is the max value (the autogenerated *_MAX value), export as "INVALID", unless we're exporting text for copy/paste (for copy/paste,
		// the property text value must actually match an entry in the enum's names array)
		if (Enum)
		{
			if (Enum->IsValidEnumValue(NumValue))
			{
				Value = Enum->GetDisplayNameTextByValue(NumValue);
			}
			else
			{
				Value = LOCTEXT("Invalid", "(INVALID)");
			}
		}
		else
		{
			Value = FText::AsNumber(NumValue);
		}

		return;
	}
	else if (const FMapProperty* MapProperty = CastField<FMapProperty>(ResolvedProperty))
	{
		FScriptMapHelper MapHelper(MapProperty, PropertyInstance.Value);
		Value = FText::Format(LOCTEXT("MapSize", "Num={0}"), FText::AsNumber(MapHelper.Num()));
		return;
	}
	else if (const FSetProperty* SetProperty = CastField<FSetProperty>(ResolvedProperty))
	{
		FScriptSetHelper SetHelper(SetProperty, PropertyInstance.Value);
		Value = FText::Format(LOCTEXT("SetSize", "Num={0}"), FText::AsNumber(SetHelper.Num()));
		return;
	}
	else if (const FObjectPropertyBase* ObjectPropertyBase = CastField<FObjectPropertyBase>(ResolvedProperty))
	{
		Object = ObjectPropertyBase->GetObjectPropertyValue(PropertyInstance.Value);
		if (Object.IsValid())
		{
			Value = FText::FromString(Object->GetFullName());
		}
		else
		{
			Value = FText::FromString(TEXT("None"));
		}

		return;
	}
	else if (const FDelegateProperty* DelegateProperty = CastField<FDelegateProperty>(ResolvedProperty))
	{
		if (DelegateProperty->SignatureFunction)
		{
			Value = DelegateProperty->SignatureFunction->GetDisplayNameText();
		}
		else
		{
			Value = LOCTEXT("NoFunc", "(No bound function)");
		}

		return;
	}
	else if (const FMulticastDelegateProperty* MulticastDelegateProperty = CastField<FMulticastDelegateProperty>(ResolvedProperty))
	{
		if (MulticastDelegateProperty->SignatureFunction)
		{
			Value = MulticastDelegateProperty->SignatureFunction->GetDisplayNameText();
		}
		else
		{
			Value = LOCTEXT("NoFunc", "(No bound function)");
		}

		return;
	}
	else if (const FInterfaceProperty* InterfaceProperty = CastField<FInterfaceProperty>(ResolvedProperty))
	{
		const FScriptInterface* InterfaceData = StaticCast<const FScriptInterface*>(PropertyInstance.Value);
		Object = InterfaceData->GetObject();
		
		if (Object.IsValid())
		{
			Value = FText::FromString(Object->GetFullName());
		}
		else
		{
			Value = FText::FromString(TEXT("None"));
		}

		return;
	}

	ensureMsgf(false, TEXT("Failed to identify property type. This function may need to be expanded to include it: %s"), *ResolvedProperty->GetClass()->GetName());
}

TSharedPtr<FPropertyInstanceInfo> FPropertyInstanceInfo::Make(FPropertyInstance PropertyInstance, const TSharedPtr<FPropertyInstanceInfo>& Parent)
{
	return MakeShared<FPropertyInstanceInfo>(PropertyInstance, Parent);
}

void FPropertyInstanceInfo::PopulateChildren(FPropertyInstance PropertyInstance)
{
	check(PropertyInstance.Property);
	if (PropertyInstance.Value == nullptr)
	{
		return;
	}

	if (UObject* ResolvedObject = Object.Get())
	{
		if (ResolvedObject->GetClass()->HasMetaDataHierarchical("DebugTreeLeaf"))
		{
			return;
		}
	}

	if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property.Get()))
	{
		checkSlow(ArrayProperty->Inner);

		FScriptArrayHelper ArrayHelper(ArrayProperty, PropertyInstance.Value);
		for (int32 i = 0; i < ArrayHelper.Num(); i++)
		{
			FPropertyInstance ChildProperty = {
				ArrayProperty->Inner,
				ArrayHelper.GetRawPtr(i)
			};
			const TSharedPtr<FPropertyInstanceInfo> ChildInfo = Make(ChildProperty, AsShared());
			
			// overwrite the display name with the array index for the current element
			ChildInfo->DisplayName = FText::Format(LOCTEXT("ArrayIndexName", "[{0}]"), FText::AsNumber(i));
			ChildInfo->bIsInContainer = true;
			ChildInfo->ContainerIndex = i;
			Children.Add(ChildInfo);
		}
	}
	else if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property.Get()))
	{
		for (TFieldIterator<FProperty> It(StructProperty->Struct); It; ++It)
		{
			FPropertyInstance ChildProperty = {
				*It,
				It->ContainerPtrToValuePtr<void>(PropertyInstance.Value, 0)
			};
			const TSharedPtr<FPropertyInstanceInfo> ChildInfo = Make(ChildProperty, AsShared());
			
			Children.Add(ChildInfo);
		}
	}
	else if (const FMapProperty* MapProperty = CastField<FMapProperty>(Property.Get()))
	{
		FScriptMapHelper MapHelper(MapProperty, PropertyInstance.Value);

		for (FScriptMapHelper::FIterator Itr = MapHelper.CreateIterator(); Itr; ++Itr)
		{
			uint8* KeyData = MapHelper.GetKeyPtr(Itr);
			uint8* ValData = MapHelper.GetValuePtr(Itr);
			FPropertyInstance ChildProperty = {
				MapProperty->ValueProp,
				ValData
			};
			const TSharedPtr<FPropertyInstanceInfo> ChildInfo = Make(ChildProperty, AsShared());
				
			FString NameStr = TEXT("[");
			MapProperty->KeyProp->ExportTextItem_Direct(
				NameStr,
				KeyData,
				nullptr,
				nullptr,
				PPF_PropertyWindow | PPF_BlueprintDebugView | PPF_Delimited,
				nullptr
			);
			NameStr += TEXT("] ");
			ChildInfo->DisplayName = FText::FromString(NameStr);
			ChildInfo->bIsInContainer = true;
			ChildInfo->ContainerIndex = Itr.GetLogicalIndex();
				
			Children.Add(ChildInfo);
		}
	}
	else if (const FSetProperty* SetProperty = CastField<FSetProperty>(Property.Get()))
	{
		FScriptSetHelper SetHelper(SetProperty, PropertyInstance.Value);
		for (FScriptSetHelper::FIterator Itr = SetHelper.CreateIterator(); Itr; ++Itr)
		{
			uint8* PropData = SetHelper.GetElementPtr(Itr);
			FPropertyInstance ChildProperty = {
				SetProperty->ElementProp,
				PropData
			};
			const TSharedPtr<FPropertyInstanceInfo> ChildInfo = Make(ChildProperty, AsShared());
				
			ChildInfo->DisplayName = FText::Format(LOCTEXT("SetIndexName", "[{0}]"), FText::AsNumber(Itr.GetLogicalIndex()));
			ChildInfo->bIsInContainer = true;
			ChildInfo->ContainerIndex = Itr.GetLogicalIndex();

			Children.Add(ChildInfo);
		}
	}
	else if (const FObjectPropertyBase* ObjectPropertyBase = CastField<FObjectPropertyBase>(Property.Get()))
	{
		if (UObject* Obj = ObjectPropertyBase->GetObjectPropertyValue(PropertyInstance.Value))
		{
			for (TFieldIterator<FProperty> It(Obj->GetClass()); It; ++It)
			{
				if(It->HasAllPropertyFlags(CPF_BlueprintVisible))
				{
					FPropertyInstance ChildProperty = {
						*It,
						It->ContainerPtrToValuePtr<void*>(Obj)
					};
					const TSharedPtr<FPropertyInstanceInfo> ChildInfo = Make(ChildProperty, AsShared());
					Children.Add(ChildInfo);
				}
			}
		}
	}
}


TSharedPtr<FPropertyInstanceInfo> FPropertyInstanceInfo::ResolvePathToProperty(const TArray<FName>& InPathToProperty)
{
	const auto FindChildInPropertyInfo = [](FPropertyInstanceInfo* InPropertyInfo, const FName& InChildName) -> TSharedPtr<FPropertyInstanceInfo>*
	{
		if (InPropertyInfo)
		{
			const FString ChildNameStr = InChildName.ToString();
			const FProperty* Prop = InPropertyInfo->Property.Get();

			if (Prop->IsA<FSetProperty>() || Prop->IsA<FArrayProperty>() || Prop->IsA<FMapProperty>())
			{
				return InPropertyInfo->Children.FindByPredicate(
					[&ChildNameStr](const TSharedPtr<FPropertyInstanceInfo>& Child)
					{
						// Display name for Container Element Properties is just their index
						return Child->DisplayName.ToString() == ChildNameStr;
					}
				);
			}
			else
			{
				return InPropertyInfo->Children.FindByPredicate(
					[&ChildNameStr](const TSharedPtr<FPropertyInstanceInfo>& Child)
					{
						return Child->Property->GetAuthoredName() == ChildNameStr;
					}
				);
			}
		}
		return nullptr;
	};

	TSharedPtr<FPropertyInstanceInfo> ThisDebugInfo = nullptr;
	for (const FName& ChildName : InPathToProperty)
	{
		TSharedPtr<FPropertyInstanceInfo>* FoundChild = ThisDebugInfo.IsValid() ? FindChildInPropertyInfo(ThisDebugInfo.Get(), ChildName) : FindChildInPropertyInfo(this, ChildName);

		if (FoundChild)
		{
			ThisDebugInfo = *FoundChild;
		}
		else
		{
			return nullptr;
		}
	}

	return ThisDebugInfo;
}

FString FPropertyInstanceInfo::GetWatchText() const
{
	const FProperty* Prop = Property.Get();
	if (Prop)
	{
		if (Prop->IsA<FSetProperty>() || Prop->IsA<FArrayProperty>() || Prop->IsA<FMapProperty>())
		{
			FString WatchText;
			for (const TSharedPtr<FPropertyInstanceInfo>& ContainerElement : Children)
			{
				WatchText.Append(FText::Format(LOCTEXT("WatchTextFmt", "{0} {1}\n"), ContainerElement->DisplayName, ContainerElement->Value).ToString());
			}

			return WatchText;
		}


	}

	return Value.ToString();
}

static bool IsAContainer(const FProperty* Property)
{
	return Property->IsA<FArrayProperty>() || Property->IsA<FSetProperty>() || Property->IsA<FMapProperty>() ||
		Property->IsA<FStructProperty>() || Property->IsA<FObjectPropertyBase>();
}

const TArray<TSharedPtr<FPropertyInstanceInfo>>& FPropertyInstanceInfo::GetChildren()
{
	if (Children.IsEmpty() && IsAContainer(Property.Get()))
	{
		// Children haven't been generated yet. Generate them.
		const FPropertyInstance PropertyInstance = GetPropertyInstance();
		if (PropertyInstance.Value != nullptr)
		{
			TMap<FPropertyInstance, TSharedPtr<FPropertyInstanceInfo>> VisitedNodes;
			PopulateChildren(PropertyInstance);
		}
	}

	return Children;
}

FPropertyInstanceInfo::FPropertyInstance FPropertyInstanceInfo::GetPropertyInstance() const
{
	// reiterate the property tree to rebuild data pointer in case it was invalidated
	
	TArray<const FPropertyInstanceInfo*> Path;
	const FPropertyInstanceInfo* Itr = this;
	while (const FPropertyInstanceInfo* ItrParent = Itr->Parent.Pin().Get())
	{
		Path.Add(Itr);
		Itr = ItrParent;
	}
	Path.Add(Itr);
	Algo::Reverse(Path);
	const void* Data = Itr->ValueAddress;
	for (int I = 1; I < Path.Num() && Data != nullptr; ++I)
	{
		const FPropertyInstanceInfo* ParentInfo = Path[I - 1];
		const FPropertyInstanceInfo* Info = Path[I];
		if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(ParentInfo->Property.Get()))
		{
			FScriptArrayHelper ArrayHelper(ArrayProperty, Data);
			const int32 Index = Info->ContainerIndex;
			if (ArrayHelper.IsValidIndex(Index))
			{
				Data = ArrayHelper.GetRawPtr(Index);
			}
			else
			{
				Data = nullptr;
				break;
			}
		}
		else if (const FSetProperty* SetProperty = CastField<FSetProperty>(ParentInfo->Property.Get()))
		{
			FScriptSetHelper SetHelper(SetProperty, Data);
			const int32 Index = Info->ContainerIndex;
			Data = SetHelper.FindNthElementPtr(Index);
		}
		else if (const FMapProperty* MapProperty = CastField<FMapProperty>(ParentInfo->Property.Get()))
		{
			FScriptMapHelper MapHelper(MapProperty, Data);
			const int32 Index = Info->ContainerIndex;
			Data = MapHelper.FindNthValuePtr(Index);
		}
		else if (const FStructProperty* StructProperty = CastField<FStructProperty>(ParentInfo->Property.Get()))
		{
			Data = Info->Property->ContainerPtrToValuePtr<void>(Data);
		}
		else if (const FObjectPropertyBase* ObjectPropertyBase = CastField<FObjectPropertyBase>(ParentInfo->Property.Get()))
		{
			if (UObject* Obj = ObjectPropertyBase->GetObjectPropertyValue(Data))
			{
				if (!Obj->IsA(Info->Property->GetOwner<UClass>()))
				{
					// This node is invalid. (this can happen if it was removed from an array, map, or set)
					Data = nullptr;
					break;
				}
				Data = Info->Property->ContainerPtrToValuePtr<void>(Obj);
			}
		}
		if (Data != Info->ValueAddress)
		{
			// This node is invalid. (this can happen if it was removed from an array, map, or set)
			Data = nullptr;
			break;
		}
	}
	
	return Data ? FPropertyInstance{Property.Get(), Data} : FPropertyInstance{nullptr, nullptr};
}

FText FKismetDebugUtilities::GetAndClearLastExceptionMessage()
{
	FKismetDebugUtilitiesData& Data = FKismetDebugUtilitiesData::Get();
	const FText Result = Data.LastExceptionMessage;
	Data.LastExceptionMessage = FText();
	return Result;
}

#undef LOCTEXT_NAMESPACE
