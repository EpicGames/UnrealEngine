// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet2/KismetReinstanceUtilities.h"
#include "Algo/ForEach.h"
#include "Algo/RemoveIf.h"
#include "BlueprintCompilationManager.h"
#include "ComponentInstanceDataCache.h"
#include "Engine/Blueprint.h"
#include "JsonObjectGraph/Stringify.h"
#include "Stats/StatsMisc.h"
#include "UObject/ObjectInstancingGraph.h"
#include "UObject/Package.h"
#include "Components/SceneComponent.h"
#include "GameFramework/Actor.h"
#include "ActorTransactionAnnotation.h"
#include "Engine/World.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/ChildActorComponent.h"
#include "Animation/AnimInstance.h"
#include "Engine/Engine.h"
#include "Editor/EditorEngine.h"
#include "Animation/AnimBlueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/SimpleConstructionScript.h"
#include "FileHelpers.h"
#include "Misc/ScopedSlowTask.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Layers/LayersSubsystem.h"
#include "Editor.h"
#include "UObject/ReferencerFinder.h"
#include "UObject/UObjectThreadContext.h"

#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "UObject/VerseValueProperty.h"
#include "UObject/InstanceDataObjectUtils.h"
#include "VerseVM/VBPVMDynamicProperty.h"
#include "Serialization/FindObjectReferencers.h"
#include "Serialization/ArchiveReplaceObjectAndStructPropertyRef.h"
#include "Serialization/ObjectReader.h"
#include "Serialization/ObjectWriter.h"
#include "BlueprintEditor.h"
#include "Engine/Selection.h"
#include "BlueprintEditorSettings.h"
#include "Engine/NetDriver.h"
#include "Engine/ActorChannel.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Engine/ScopedMovementUpdate.h"
#include "Algo/TopologicalSort.h"
#include "UObject/OverridableManager.h"
#include "UObject/PropertyOptional.h"
#include "UObject/PropertyBagRepository.h"
#include "UObject/UObjectArchetypeHelper.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "Reinstancing/ReinstancingInternal.h"

DECLARE_CYCLE_STAT(TEXT("Replace Instances"), EKismetReinstancerStats_ReplaceInstancesOfClass, STATGROUP_KismetReinstancer );
DECLARE_CYCLE_STAT(TEXT("Find Referencers"), EKismetReinstancerStats_FindReferencers, STATGROUP_KismetReinstancer );
DECLARE_CYCLE_STAT(TEXT("Replace References"), EKismetReinstancerStats_ReplaceReferences, STATGROUP_KismetReinstancer );
DECLARE_CYCLE_STAT(TEXT("Construct Replacements"), EKismetReinstancerStats_ReplacementConstruction, STATGROUP_KismetReinstancer );
DECLARE_CYCLE_STAT(TEXT("Update Bytecode References"), EKismetReinstancerStats_UpdateBytecodeReferences, STATGROUP_KismetReinstancer );
DECLARE_CYCLE_STAT(TEXT("Recompile Child Classes"), EKismetReinstancerStats_RecompileChildClasses, STATGROUP_KismetReinstancer );
DECLARE_CYCLE_STAT(TEXT("Replace Classes Without Reinstancing"), EKismetReinstancerStats_ReplaceClassNoReinsancing, STATGROUP_KismetReinstancer );
DECLARE_CYCLE_STAT(TEXT("Reinstance Objects"), EKismetCompilerStats_ReinstanceObjects, STATGROUP_KismetCompiler);

bool GUseLegacyAnimInstanceReinstancingBehavior = false;
static FAutoConsoleVariableRef CVarUseLegacyAnimInstanceReinstancingBehavior(
	TEXT("bp.UseLegacyAnimInstanceReinstancingBehavior"),
	GUseLegacyAnimInstanceReinstancingBehavior,
	TEXT("Use the legacy re-instancing behavior for anim instances where the instance is destroyed and re-created.")
);

bool GUseLegacyRootComponentPreservation = false;
static FAutoConsoleVariableRef CVarUseLegacyRootComponentPreservation(
	TEXT("bp.UseLegacyRootComponentPreservation"),
	GUseLegacyRootComponentPreservation,
	TEXT("Use the legacy re-instancing behavior to preserve root components, ignoring data in previous instances")
);

bool GUseReinstancingGraphCache = true;
static FAutoConsoleVariableRef CVarUseReinstancingGraphCache(
	TEXT("bp.UseReinstancingGraphCache"),
	GUseReinstancingGraphCache,
	TEXT("Use a subobject graph cache to improve performance of reinstancing")
);

namespace UE::Private
{

// The class for finding and replacing weak references.
// We can't relay on "standard" weak references replacement as
// it depends on FSoftObjectPath::ResolveObject, which
// tries to find the object with the stored path. It is
// impossible, cause above we deleted old actors (after
// spawning new ones), so during objects traverse we have to
// find FSoftObjectPath with the raw given path taken
// before deletion of old actors and fix them.
class FReferenceReplace : public FArchiveReplaceObjectAndStructPropertyRef<UObject>
{
public:
	FReferenceReplace(UObject* InSearchObject, const TMap<UObject*, UObject*>& InReplacementMap, const TMap<FSoftObjectPath, UObject*>& InWeakReferencesMap)
		: FArchiveReplaceObjectAndStructPropertyRef<UObject>(InSearchObject, InReplacementMap, EArchiveReplaceObjectFlags::DelayStart), WeakReferencesMap(InWeakReferencesMap)
	{
		SerializeSearchObject();
	}

	FReferenceReplace(UObject* InSearchObject, const TMap<UObject*, UObject*>& InReplacementMap, const TMap<FSoftObjectPath, UObject*>& InWeakReferencesMap, EArchiveReplaceObjectFlags Flags)
		: FArchiveReplaceObjectAndStructPropertyRef<UObject>(InSearchObject, InReplacementMap, EArchiveReplaceObjectFlags::DelayStart), WeakReferencesMap(InWeakReferencesMap)
	{
		if (!(Flags & EArchiveReplaceObjectFlags::DelayStart))
		{
			SerializeSearchObject();
		}
	}

	FArchive& operator<<(FSoftObjectPath& Ref) override
	{
		const UObject*const* PtrToObjPtr = WeakReferencesMap.Find(Ref);

		if (PtrToObjPtr != nullptr)
		{
			Ref = *PtrToObjPtr;
		}

		return *this;
	}

	FArchive& operator<<(FSoftObjectPtr& Ref) override
	{
		return operator<<(Ref.GetUniqueID());
	}

private:
	const TMap<FSoftObjectPath, UObject*>& WeakReferencesMap;
};

struct FReplaceReferenceHelper
{
	static void ValidateReplacementMappings(const TMap<UObject*, UObject*>& OldToNewInstanceMap)
	{
		// Test long unstated assumption - alternatively we could 'flatten' the chains
		// but would then have to guard against cycles:
		bool bFoundChains = false;
		for (const TPair<UObject*, UObject*>& OldToNew : OldToNewInstanceMap)
		{
			bFoundChains = OldToNewInstanceMap.Find(OldToNew.Value) != nullptr;
			if (bFoundChains)
			{
				break;
			}
		}

		if (!bFoundChains)
		{
			return;
		}

		TSet<UObject*> ObjectsInvolved;
		Algo::ForEach(OldToNewInstanceMap, [&ObjectsInvolved, &OldToNewInstanceMap](const TPair<UObject*, UObject*>& OldToNew)
			{
				UObject* const* NewMappedToNew = OldToNewInstanceMap.Find(OldToNew.Value);
				if (NewMappedToNew != nullptr)
				{
					if(OldToNew.Key)
					{
						ObjectsInvolved.Add(OldToNew.Key);
					}
					if (OldToNew.Value)
					{
						ObjectsInvolved.Add(OldToNew.Value);
					}
					if(*NewMappedToNew)
					{
						ObjectsInvolved.Add(*NewMappedToNew);
					}
				}
			});

		TSet<UClass*> ClassesInvolved;
		Algo::ForEach(ObjectsInvolved, [&ClassesInvolved](UObject* Object)
			{
				ClassesInvolved.Add(Object->GetClass());
			});

		TStringBuilder<256> NamesOfClasses;
		Algo::ForEach(ClassesInvolved, [&NamesOfClasses](UClass* Class) { NamesOfClasses.Append(Class->GetName() + TEXT("\n")); });
		TStringBuilder<256> NamesOfObjects;
		Algo::ForEach(ObjectsInvolved, [&NamesOfObjects](UObject* Obj) { NamesOfObjects.Append(Obj->GetName() + TEXT("\n")); });

		ensureMsgf(false, TEXT("Found chains of replacement objects while updating class layouts, please report a bug involving Classes:\n%sAnd Objects:\n%s"), 
			*NamesOfClasses, *NamesOfObjects);
	}

	static void IncludeDSOs(UObject* OldOuter, UObject* NewOuter, TMap<UObject*, UObject*>& OldToNewInstanceMap, TArray<UObject*>& SourceObjects)
	{
		TArray<UObject*> OldSubObjArray;
		GetObjectsWithOuter(OldOuter, OldSubObjArray, EGetObjectsFlags::None);
		for (UObject* OldSubObj : OldSubObjArray)
		{
			if (UObject* NewSubObj = NewOuter->GetDefaultSubobjectByName(OldSubObj->GetFName()))
			{
				ensure(!OldToNewInstanceMap.Contains(OldSubObj));
				OldToNewInstanceMap.Add(OldSubObj, NewSubObj);
				SourceObjects.Add(OldSubObj);

				// Recursively include any nested DSOs
				IncludeDSOs(OldSubObj, NewSubObj, OldToNewInstanceMap, SourceObjects);
			}
		}
	}

	static void IncludeCDO(UClass* OldClass, UClass* NewClass, TMap<UObject*, UObject*>& OldToNewInstanceMap, TArray<UObject*>& SourceObjects, UObject* OriginalCDO, TMap<UClass*, TMap<UObject*, UObject*>>* OldToNewTemplates = nullptr)
	{
		UObject* OldCDO = OldClass->GetDefaultObject();
		UObject* NewCDO = NewClass->GetDefaultObject();

		if (const TMap<UObject*, UObject*>* OldToNewTemplateMapping = OldToNewTemplates ? OldToNewTemplates->Find(OldClass) : nullptr)
		{
			OldToNewInstanceMap.Append(*OldToNewTemplateMapping);

			TArray<UObject*> SourceTemplateObjects;
			OldToNewTemplateMapping->GenerateKeyArray(SourceTemplateObjects);
			SourceObjects.Append(SourceTemplateObjects);
		}
		else
		{
			// Add the old->new CDO mapping into the fixup map
			OldToNewInstanceMap.Add(OldCDO, NewCDO);
			// Add in the old CDO to this pass, so CDO references are fixed up
			SourceObjects.Add(OldCDO);
			// Add any old->new CDO default subobject mappings
			IncludeDSOs(OldCDO, NewCDO, OldToNewInstanceMap, SourceObjects);
		}

		if (OriginalCDO && OriginalCDO != OldCDO)
		{
			OldToNewInstanceMap.Add(OriginalCDO, NewCDO);
			SourceObjects.Add(OriginalCDO);
			IncludeDSOs(OriginalCDO, NewCDO, OldToNewInstanceMap, SourceObjects);
		}
	}

	static void IncludeClass(UClass* OldClass, UClass* NewClass, TMap<UObject*, UObject*> &OldToNewInstanceMap, TArray<UObject*> &SourceObjects, TArray<UObject*> &ObjectsToReplace)
	{
		OldToNewInstanceMap.Add(OldClass, NewClass);
		SourceObjects.Add(OldClass);

		if (UObject* OldCDO = OldClass->GetDefaultObject(false))
		{
			ObjectsToReplace.Add(OldCDO);
		}
	}

	static void FindAndReplaceReferences(const TArray<UObject*>& SourceObjects, const TSet<UObject*>* ObjectsThatShouldUseOldStuff, const TArray<UObject*>& ObjectsToReplace, const TMap<UObject*, UObject*>& OldToNewInstanceMap, const TMap<FSoftObjectPath, UObject*>& ReinstancedObjectsWeakReferenceMap)
	{
		if(SourceObjects.Num() == 0 && ObjectsToReplace.Num() == 0 )
		{
			return;
		}

		// Remember what values were in UActorChannel::Actor so we can restore them later (this should only affect reinstancing during PIE)
		// We need the old actor channel to tear down cleanly without affecting the new actor
		TMap<UActorChannel*, AActor*> ActorChannelActorRestorationMap;
		for (UActorChannel* ActorChannel : TObjectRange<UActorChannel>())
		{
			if (OldToNewInstanceMap.Contains(ActorChannel->Actor))
			{
				ActorChannelActorRestorationMap.Add(ActorChannel, ActorChannel->Actor);
			}
		}

		// Find everything that references these objects
		TArray<UObject *> Targets;
		{
			BP_SCOPED_COMPILER_EVENT_STAT(EKismetReinstancerStats_FindReferencers);

			Targets = FReferencerFinder::GetAllReferencers(SourceObjects, ObjectsThatShouldUseOldStuff);
		}

		if (Targets.Num())
		{
			BP_SCOPED_COMPILER_EVENT_STAT(EKismetReinstancerStats_ReplaceReferences);

			FScopedSlowTask SlowTask(static_cast<float>(Targets.Num()), NSLOCTEXT("Kismet", "PerformingReplaceReferences", "Performing replace references..."));
			SlowTask.MakeDialogDelayed(1.0f);

			for (UObject* Obj : Targets)
			{
				SlowTask.EnterProgressFrame(1);

				// Make sure we don't update properties in old objects, as they
				// may take ownership of objects referenced in new objects (e.g.
				// delete components owned by new actors)
				if (!ObjectsToReplace.Contains(Obj))
				{
					FReferenceReplace ReplaceAr(Obj, OldToNewInstanceMap, ReinstancedObjectsWeakReferenceMap);
				}
			}
		}
	
		// Restore the old UActorChannel::Actor values (undoing what the replace references archiver did above to them)
		for (const auto& KVP : ActorChannelActorRestorationMap)
		{
			KVP.Key->Actor = KVP.Value;
		}
	}
	
	// Others may want this simple iteration function, but hiding it here for now:
	static void ForEachSubObject(const FProperty* TargetProp, const UObject* Outer, const void* ContainerAddress, TFunctionRef<void(const UObject*)> ObjRefFunc)
	{
		check(ContainerAddress && Outer);
		if (TargetProp->HasAnyPropertyFlags(CPF_Transient))
		{
			return;
		}

		if (const FArrayProperty* ArrayProperty = CastField<const FArrayProperty>(TargetProp))
		{
			FScriptArrayHelper ArrayHelper(ArrayProperty, ContainerAddress);
			for (int32 ElementIndex = 0; ElementIndex < ArrayHelper.Num(); ++ElementIndex)
			{
				const void* ValueAddress = ArrayHelper.GetRawPtr(ElementIndex);

				ForEachSubObject(ArrayProperty->Inner, Outer, ValueAddress, ObjRefFunc);
			}
		}
		else if (const FMapProperty* MapProperty = CastField<const FMapProperty>(TargetProp))
		{
			// Exit now if the map doesn't contain any instanced references.
			int32 LogicalIndex = 0;
			FScriptMapHelper MapHelper(MapProperty, ContainerAddress);
			for (int32 ElementIndex = 0; ElementIndex < MapHelper.GetMaxIndex(); ++ElementIndex)
			{
				if (MapHelper.IsValidIndex(ElementIndex))
				{
					const void* KeyAddress = MapHelper.GetKeyPtr(ElementIndex);
					const void* ValueAddress = MapHelper.GetValuePtr(ElementIndex);

					// Note: Keep these as the logical (Nth) index in case the map changes internally after we construct the path or in case we resolve using a different object.
					ForEachSubObject(MapProperty->KeyProp, Outer, KeyAddress, ObjRefFunc);
					ForEachSubObject(MapProperty->ValueProp, Outer, ValueAddress, ObjRefFunc);

					++LogicalIndex;
				}
			}
		}
		else if (const FSetProperty* SetProperty = CastField<const FSetProperty>(TargetProp))
		{
			int32 LogicalIndex = 0;
			FScriptSetHelper SetHelper(SetProperty, ContainerAddress);
			for (int32 ElementIndex = 0; ElementIndex < SetHelper.GetMaxIndex(); ++ElementIndex)
			{
				if (SetHelper.IsValidIndex(ElementIndex))
				{
					const void* ValueAddress = SetHelper.GetElementPtr(ElementIndex);

					// Note: Keep this as the logical (Nth) index in case the set changes internally after we construct the path or in case we resolve using a different object.
					ForEachSubObject(SetProperty->ElementProp, Outer, ValueAddress, ObjRefFunc);

					++LogicalIndex;
				}
			}
		}
		else if (const FOptionalProperty* OptionalProperty = CastField<FOptionalProperty>(TargetProp))
		{
			if (const void* ValueAddress = static_cast<const void*>(OptionalProperty->GetValuePointerForReadOrReplaceIfSet(ContainerAddress)))
			{
				ForEachSubObject(OptionalProperty->GetValueProperty(), Outer, ValueAddress, ObjRefFunc);
			}
		}
		else if (const FStructProperty* StructProperty = CastField<const FStructProperty>(TargetProp))
		{
			for (FProperty* StructProp = StructProperty->Struct->RefLink; StructProp; StructProp = StructProp->NextRef)
			{
				for (int32 ArrayIdx = 0; ArrayIdx < StructProp->ArrayDim; ++ArrayIdx)
				{
					const void* ValueAddress = StructProp->ContainerPtrToValuePtr<uint8>(ContainerAddress, ArrayIdx);

					ForEachSubObject(StructProp, Outer, ValueAddress, ObjRefFunc);
				}
			}
		}
		else if (const FObjectProperty* ObjectProperty = CastField<const FObjectProperty>(TargetProp))
		{
			if (UObject* ObjectValue = ObjectProperty->GetObjectPropertyValue(ContainerAddress))
			{
				// don't need to push to PropertyPath, since this property is already at its head
				ObjRefFunc(ObjectValue);
			}
		}
		else if (const FVerseDynamicProperty* DynamicProperty = CastField<const FVerseDynamicProperty>(TargetProp))
		{
			if (const UE::FDynamicallyTypedValue* DynamicValue = DynamicProperty->GetPropertyValuePtr(ContainerAddress))
			{
				DynamicValue->ForEachSubobject(ObjRefFunc);
			}
		}
#if WITH_VERSE_VM
		else if (const FVRestValueProperty* RestValueProperty = CastField<const FVRestValueProperty>(TargetProp))
		{
			// TODO: deal with reinstancing in the new VM.
			ensureMsgf(false && "NotImplemented", TEXT("NotImplemented"));
		}
#endif
	}

	static void GetOwnedSubobjectsRecursive(const UObject* Container, TSet<UObject*>& OutObjects, const UObject* Root = nullptr)
	{
		if (Root == nullptr)
		{
			Root = Container;
		}

		// loose subobjects need to be reinstanced too so iterate the IDO instead if it exists so we can include loose subobjects references
		if (UObject* FoundIdo = UE::FPropertyBagRepository::Get().FindInstanceDataObject(Container))
		{
			Container = FoundIdo;
		}

		const UClass* ContainerClass = Container->GetClass();
		for (FProperty* Prop = ContainerClass->RefLink; Prop; Prop = Prop->NextRef)
		{
			for (int32 ArrayIdx = 0; ArrayIdx < Prop->ArrayDim; ++ArrayIdx)
			{
				const uint8* ValuePtr = Prop->ContainerPtrToValuePtr<uint8>(Container, ArrayIdx);
				ForEachSubObject(Prop, Container, ValuePtr, [&OutObjects, Root](const UObject* Ref)
					{
						if (Ref && !OutObjects.Contains(Ref) && Ref->IsIn(Root))
						{
							OutObjects.Add(const_cast<UObject*>(Ref)); // consumer is not const correct
							GetOwnedSubobjectsRecursive(Ref, OutObjects, Root);
						}
					});
			}
		}
	}
};

struct FArchetypeReinstanceHelper
{
	/** Returns the full set of archetypes rooted at a single archetype object, with additional object flags (optional) */
	static void GetArchetypeObjects(UObject* InObject, TArray<UObject*>& OutArchetypeObjects, EObjectFlags SubArchetypeFlags = RF_NoFlags)
	{
		OutArchetypeObjects.Empty();

		if (InObject != nullptr && InObject->HasAllFlags(RF_ArchetypeObject))
		{
			OutArchetypeObjects.Add(InObject);

			TArray<UObject*> ArchetypeInstances;
			InObject->GetArchetypeInstances(ArchetypeInstances);

			for (int32 Idx = 0; Idx < ArchetypeInstances.Num(); ++Idx)
			{
				UObject* ArchetypeInstance = ArchetypeInstances[Idx];
				if (IsValid(ArchetypeInstance) && ArchetypeInstance->HasAllFlags(RF_ArchetypeObject | SubArchetypeFlags))
				{
					OutArchetypeObjects.Add(ArchetypeInstance);

					TArray<UObject*> SubArchetypeInstances;
					ArchetypeInstance->GetArchetypeInstances(SubArchetypeInstances);

					if (SubArchetypeInstances.Num() > 0)
					{
						ArchetypeInstances.Append(SubArchetypeInstances);
					}
				}
			}
		}
	}

	/** Returns an object name that's found to be unique within the given set of archetype objects */
	static FName FindUniqueArchetypeObjectName(TArray<UObject*>& InArchetypeObjects)
	{
		FName OutName = NAME_None;

		if (InArchetypeObjects.Num() > 0)
		{
			while (OutName == NAME_None)
			{
				UObject* ArchetypeObject = InArchetypeObjects[0];
				OutName = MakeUniqueObjectName(ArchetypeObject->GetOuter(), ArchetypeObject->GetClass());
				for (int32 ObjIdx = 1; ObjIdx < InArchetypeObjects.Num(); ++ObjIdx)
				{
					ArchetypeObject = InArchetypeObjects[ObjIdx];
					if (StaticFindObjectFast(ArchetypeObject->GetClass(), ArchetypeObject->GetOuter(), OutName))
					{
						OutName = NAME_None;
						break;
					}
				}
			}
		}

		return OutName;
	}
};
}

/////////////////////////////////////////////////////////////////////////////////
// FBlueprintCompileReinstancer

TSet<TWeakObjectPtr<UBlueprint>> FBlueprintCompileReinstancer::CompiledBlueprintsToSave = TSet<TWeakObjectPtr<UBlueprint>>();

UClass* FBlueprintCompileReinstancer::HotReloadedOldClass = nullptr;
UClass* FBlueprintCompileReinstancer::HotReloadedNewClass = nullptr;

FBlueprintCompileReinstancer::FBlueprintCompileReinstancer(UClass* InClassToReinstance, EBlueprintCompileReinstancerFlags Flags)
	: FGCObject(EFlags::RegisterLater) 
	, ClassToReinstance(InClassToReinstance)
	, DuplicatedClass(nullptr)
	, OriginalCDO(nullptr)
	, OriginalSCD(nullptr)
	, OriginalSCDStruct(nullptr)
	, bHasReinstanced(false)
	, ReinstClassType(RCT_Unknown)
	, ClassToReinstanceDefaultValuesCRC(0)
	, bIsRootReinstancer(false)
	, bAllowResaveAtTheEndIfRequested(false)
{
	// This is not needed during async loading since GC cannot run and we don't want
	// to expose any objects we're currently working on to other threads if they try to use
	// the GC interface to collect objects.
	if (!IsInActualLoadingThread() && !IsInParallelLoadingThread())
	{
		// Call this in the most derived class to avoid pure calls when registering from the base class
		// and the vtbl isn't yet ready.
		RegisterGCObject();
	}

	if( InClassToReinstance != nullptr && InClassToReinstance->GetDefaultObject(false) )
	{
		bool bAutoInferSaveOnCompile = !!(Flags & EBlueprintCompileReinstancerFlags::AutoInferSaveOnCompile);
		bool bIsBytecodeOnly = !!(Flags & EBlueprintCompileReinstancerFlags::BytecodeOnly);
		bool bAvoidCDODuplication = !!(Flags & EBlueprintCompileReinstancerFlags::AvoidCDODuplication);

		if (FKismetEditorUtilities::IsClassABlueprintSkeleton(ClassToReinstance))
		{
			ReinstClassType = RCT_BpSkeleton;
		}
		else if (ClassToReinstance->HasAnyClassFlags(CLASS_CompiledFromBlueprint))
		{
			ReinstClassType = RCT_BpGenerated;
		}
		else if (ClassToReinstance->HasAnyClassFlags(CLASS_Native))
		{
			ReinstClassType = RCT_Native;
		}
		bAllowResaveAtTheEndIfRequested = bAutoInferSaveOnCompile && !bIsBytecodeOnly && (ReinstClassType != RCT_BpSkeleton);
		bUseDeltaSerializationToCopyProperties = !!(Flags & EBlueprintCompileReinstancerFlags::UseDeltaSerialization);

		SaveClassFieldMapping(InClassToReinstance);

		// Remember the initial CDO for the class being resinstanced
		OriginalCDO = ClassToReinstance->GetDefaultObject();

		DuplicatedClass = MoveCDOToNewClass(ClassToReinstance, TMap<UClass*, UClass*>(), bAvoidCDODuplication);

		if(!bAvoidCDODuplication)
		{
			ensure( ClassToReinstance->GetDefaultObject(false)->GetClass() == DuplicatedClass );
			ClassToReinstance->GetDefaultObject(false)->Rename(nullptr, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
		}
		
		// Note that we can't clear ClassToReinstance->GetDefaultObject(false) even though
		// we have moved it aside CleanAndSanitizeClass will want to grab the old CDO 
		// so it can propagate values to the new one note that until that happens we are 
		// in an extraordinary state: this class has a CDO of a different type

		ObjectsThatShouldUseOldStuff.Add(DuplicatedClass); //CDO of REINST_ class can be used as archetype

		if (!bIsBytecodeOnly)
		{
			TArray<UObject*> ObjectsToChange;
			const bool bIncludeDerivedClasses = false;
			GetObjectsOfClass(ClassToReinstance, ObjectsToChange, bIncludeDerivedClasses);
			for (UObject* ObjectToChange : ObjectsToChange)
			{
				ObjectToChange->SetClass(DuplicatedClass);
			}

			TArray<UClass*> ChildrenOfClass;
			GetDerivedClasses(ClassToReinstance, ChildrenOfClass);
			for (UClass* ChildClass : ChildrenOfClass)
			{
				UBlueprint* ChildBP = Cast<UBlueprint>(ChildClass->ClassGeneratedBy);
				if (ChildBP)
				{
					const bool bClassIsDirectlyGeneratedByTheBlueprint = (ChildBP->GeneratedClass == ChildClass)
						|| (ChildBP->SkeletonGeneratedClass == ChildClass);

					if (ChildBP->HasAnyFlags(RF_BeingRegenerated) || !bClassIsDirectlyGeneratedByTheBlueprint)
					{
						if (ChildClass->GetSuperClass() == ClassToReinstance)
						{
							ReparentChild(ChildClass);
						}
						else
						{
							ChildClass->AssembleReferenceTokenStream();
							ChildClass->Bind();
							ChildClass->StaticLink(true);
						}

						//TODO: some stronger condition would be nice
						if (!bClassIsDirectlyGeneratedByTheBlueprint)
						{
							ObjectsThatShouldUseOldStuff.Add(ChildClass);
						}
					}
					// If this is a direct child, change the parent and relink so the property chain is valid for reinstancing
					else if (!ChildBP->HasAnyFlags(RF_NeedLoad))
					{
						if (ChildClass->GetSuperClass() == ClassToReinstance)
						{
							ReparentChild(ChildBP);
						}

						Children.AddUnique(ChildBP);
					}
					else
					{
						// If this is a child that caused the load of their parent, relink to the REINST class so that we can still serialize in the CDO, but do not add to later processing
						ReparentChild(ChildClass);
					}
				}
			}
		}

		// Pull the blueprint that generated this reinstance target, and gather the blueprints that are dependent on it
		UBlueprint* GeneratingBP = Cast<UBlueprint>(ClassToReinstance->ClassGeneratedBy);
		if(!IsReinstancingSkeleton() && GeneratingBP)
		{
			ClassToReinstanceDefaultValuesCRC = GeneratingBP->CrcLastCompiledCDO;

			// Never queue for saving when regenerating on load
			if (!GeneratingBP->bIsRegeneratingOnLoad && !IsReinstancingSkeleton())
			{
				bool const bIsLevelPackage = (UWorld::FindWorldInPackage(GeneratingBP->GetOutermost()) != nullptr);
				// we don't want to save the entire level (especially if this 
				// compile was already kicked off as a result of a level save, as it
				// could cause a recursive save)... let the "SaveOnCompile" setting 
				// only save blueprint assets
				if (!bIsLevelPackage)
				{
					CompiledBlueprintsToSave.Add(GeneratingBP);
				}
			}
		}
	}
}

void FBlueprintCompileReinstancer::SaveClassFieldMapping(UClass* InClassToReinstance)
{
	check(InClassToReinstance);

	for (FProperty* Prop = InClassToReinstance->PropertyLink; Prop && (Prop->GetOwner<UObject>() == InClassToReinstance); Prop = Prop->PropertyLinkNext)
	{
		PropertyMap.Add(Prop->GetFName(), Prop);
	}

	for (UFunction* Function : TFieldRange<UFunction>(InClassToReinstance, EFieldIteratorFlags::ExcludeSuper))
	{
		FunctionMap.Add(Function->GetFName(),Function);
	}
}

void FBlueprintCompileReinstancer::GenerateFieldMappings(TMap<FFieldVariant, FFieldVariant>& FieldMapping)
{
	check(ClassToReinstance);

	FieldMapping.Empty();

	for (TPair<FName, FProperty*>& Prop : PropertyMap)
	{
		FieldMapping.Add(Prop.Value, FindFProperty<FProperty>(ClassToReinstance, *Prop.Key.ToString()));
	}

	for (auto& Func : FunctionMap)
	{
		UFunction* NewFunction = ClassToReinstance->FindFunctionByName(Func.Key, EIncludeSuperFlag::ExcludeSuper);
		FieldMapping.Add(Func.Value, NewFunction);
	}

	if(!ClassToReinstance->bLayoutChanging)
	{
		UObject* NewCDO = ClassToReinstance->GetDefaultObject();
		FieldMapping.Add(OriginalCDO, NewCDO);
	}
}

void FBlueprintCompileReinstancer::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AllowEliminatingReferences(false);
	Collector.AddReferencedObject(OriginalCDO);
	Collector.AddReferencedObject(DuplicatedClass);
	Collector.AllowEliminatingReferences(true);

	// it's ok for these to get GC'd, but it is not ok for the memory to be reused (after a GC), 
	// for that reason we cannot allow these to be freed during the life of this reinstancer
	// 
	// for example, we saw this as a problem in UpdateBytecodeReferences() - if the GC'd function 
	// memory was used for a new (unrelated) function, then we were replacing references to the 
	// new function (bad), as well as any old stale references (both were using the same memory address)
	Collector.AddReferencedObjects(FunctionMap);
	for (TPair<FName, FProperty*>& PropertyNamePair : PropertyMap)
	{
		if (PropertyNamePair.Value)
		{
			PropertyNamePair.Value->AddReferencedObjects(Collector);
		}
	}
}

void FBlueprintCompileReinstancer::OptionallyRefreshNodes(UBlueprint* CurrentBP)
{
	if (HotReloadedNewClass)
	{
		UPackage* const Package = CurrentBP->GetOutermost();
		const bool bStartedWithUnsavedChanges = Package != nullptr ? Package->IsDirty() : true;

		FBlueprintEditorUtils::RefreshExternalBlueprintDependencyNodes(CurrentBP, HotReloadedNewClass);

		if (Package != nullptr && Package->IsDirty() && !bStartedWithUnsavedChanges)
		{
			Package->SetDirtyFlag(false);
		}
	}
}

FBlueprintCompileReinstancer::~FBlueprintCompileReinstancer()
{
	if (bIsRootReinstancer && bAllowResaveAtTheEndIfRequested)
	{
		if (CompiledBlueprintsToSave.Num() > 0)
		{
			if ( !IsRunningCommandlet() && !GIsAutomationTesting )
			{
				TArray<UPackage*> PackagesToSave;
				for (TWeakObjectPtr<UBlueprint> BPPtr : CompiledBlueprintsToSave)
				{
					if (BPPtr.IsValid())
					{
						UBlueprint* BP = BPPtr.Get();

						UBlueprintEditorSettings* Settings = GetMutableDefault<UBlueprintEditorSettings>();
						const bool bShouldSaveOnCompile = ((Settings->SaveOnCompile == SoC_Always) || ((Settings->SaveOnCompile == SoC_SuccessOnly) && (BP->Status == BS_UpToDate)));

						if (bShouldSaveOnCompile)
						{
							PackagesToSave.Add(BP->GetOutermost());
						}
					}
				}

				FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, /*bCheckDirty =*/true, /*bPromptToSave =*/false);
			}
			CompiledBlueprintsToSave.Empty();		
		}
	}

	UnregisterGCObject();
}

class FReinstanceFinalizer : public TSharedFromThis<FReinstanceFinalizer>
{
public:
	TSharedPtr<FBlueprintCompileReinstancer> Reinstancer;
	TArray<UObject*> ObjectsToReplace;
	TArray<UObject*> ObjectsToFinalize;
	TSet<UObject*> SelectedObjecs;
	UClass* ClassToReinstance;

	FReinstanceFinalizer(UClass* InClassToReinstance) : ClassToReinstance(InClassToReinstance)
	{
		check(ClassToReinstance);
	}

	void Finalize()
	{
		if (!ensure(Reinstancer.IsValid()))
		{
			return;
		}
		check(ClassToReinstance);

		const bool bIsActor = ClassToReinstance->IsChildOf<AActor>();
		if (bIsActor)
		{
			for (UObject* Obj : ObjectsToFinalize)
			{
				AActor* Actor = CastChecked<AActor>(Obj);

				UWorld* World = Actor->GetWorld();
				if (World)
				{
					// NOTE: This function does not handle gameplay edge cases correctly!
					// FActorReplacementHelper has a better implementation of this code

					// Remove any pending latent actions, as the compiled script code may have changed, and thus the
					// cached LinkInfo data may now be invalid. This could happen in the fast path, since the original
					// Actor instance will not be replaced in that case, and thus might still have latent actions pending.
					World->GetLatentActionManager().RemoveActionsForObject(Actor);

					// Drop any references to anim script components for skeletal mesh components, depending on how
					// the blueprints have changed during compile this could contain invalid data so we need to do
					// a full initialisation to ensure everything is set up correctly.
					TInlineComponentArray<USkeletalMeshComponent*> SkelComponents(Actor);
					for(USkeletalMeshComponent* SkelComponent : SkelComponents)
					{
						SkelComponent->AnimScriptInstance = nullptr;
					}

					Actor->ReregisterAllComponents();
					Actor->RerunConstructionScripts();

					// The reinstancing case doesn't ever explicitly call Actor->FinishSpawning, we've handled the construction script
					// portion above but still need the PostActorConstruction() case so BeginPlay gets routed correctly while in a BegunPlay world
					if (World->HasBegunPlay())
					{
						Actor->PostActorConstruction();
					}

					if (SelectedObjecs.Contains(Obj) && GEditor)
					{
						GEditor->SelectActor(Actor, /*bInSelected =*/true, /*bNotify =*/true, false, true);
					}
				}
			}
		}

		const bool bIsAnimInstance = ClassToReinstance->IsChildOf<UAnimInstance>();
		//UAnimBlueprintGeneratedClass* AnimClass = Cast<UAnimBlueprintGeneratedClass>(ClassToReinstance);
		if(bIsAnimInstance)
		{
			for (UObject* Obj : ObjectsToFinalize)
			{
				if(USkeletalMeshComponent* SkelComponent = Cast<USkeletalMeshComponent>(Obj->GetOuter()))
				{
					// This snippet catches all of the exposed value handlers that will have invalid UFunctions
					// and clears the init flag so they will be reinitialized on the next call to InitAnim.
					// Unknown whether there are other unreachable properties so currently clearing the anim
					// instance below
					// #TODO investigate reinstancing anim blueprints to correctly catch all deep references

					//UAnimInstance* ActiveInstance = SkelComponent->GetAnimInstance();
					//if(AnimClass && ActiveInstance)
					//{
					//	for(FStructProperty* NodeProp : AnimClass->AnimNodeProperties)
					//	{
					//		// Guaranteed to have only FAnimNode_Base pointers added during compilation
					//		FAnimNode_Base* AnimNode = NodeProp->ContainerPtrToValuePtr<FAnimNode_Base>(ActiveInstance);
					//
					//		AnimNode->EvaluateGraphExposedInputs.bInitialized = false;
					//	}
					//}

					// Clear out the script instance on the component to force a rebuild during initialization.
					// This is necessary to correctly reinitialize certain properties that still reference the 
					// old class as they are unreachable during reinstancing.
					SkelComponent->AnimScriptInstance = nullptr;
					SkelComponent->InitAnim(true);
				}
			}
		}

		Reinstancer->FinalizeFastReinstancing(ObjectsToReplace);
	}
};

TSharedPtr<FReinstanceFinalizer> FBlueprintCompileReinstancer::ReinstanceFast()
{
	UE_LOGF(LogBlueprint, Log, "BlueprintCompileReinstancer: Doing a fast path refresh on class '%ls'.", *GetPathNameSafe(ClassToReinstance));

	TSharedPtr<FReinstanceFinalizer> Finalizer = MakeShareable(new FReinstanceFinalizer(ClassToReinstance));
	Finalizer->Reinstancer = SharedThis(this);

	GetObjectsOfClass(DuplicatedClass, Finalizer->ObjectsToReplace, /*bIncludeDerivedClasses=*/ false);

	const bool bIsActor = ClassToReinstance->IsChildOf<AActor>();
	const bool bIsComponent = ClassToReinstance->IsChildOf<UActorComponent>();
	for (UObject* Obj : Finalizer->ObjectsToReplace)
	{
		UE_LOGF(LogBlueprint, Log, "  Fast path is refreshing (not replacing) %ls", *Obj->GetFullName());

		const bool bIsChildActorTemplate = (bIsActor ? CastChecked<AActor>(Obj)->GetOuter()->IsA<UChildActorComponent>() : false);
		if ((!Obj->IsTemplate() || bIsComponent || bIsChildActorTemplate) && IsValid(Obj))
		{
			if (bIsActor && Obj->IsSelected())
			{
				Finalizer->SelectedObjecs.Add(Obj);
			}

			Obj->SetClass(ClassToReinstance);

			Finalizer->ObjectsToFinalize.Push(Obj);
		}
	}

	return Finalizer;
}

void FBlueprintCompileReinstancer::FinalizeFastReinstancing(TArray<UObject*>& ObjectsToReplace)
{
	TArray<UObject*> SourceObjects;
	TMap<UObject*, UObject*> OldToNewInstanceMap;
	TMap<FSoftObjectPath, UObject*> ReinstancedObjectsWeakReferenceMap;
	UE::Private::FReplaceReferenceHelper::IncludeCDO(DuplicatedClass, ClassToReinstance, OldToNewInstanceMap, SourceObjects, OriginalCDO);

	if (IsClassObjectReplaced())
	{
		UE::Private::FReplaceReferenceHelper::IncludeClass(DuplicatedClass, ClassToReinstance, OldToNewInstanceMap, SourceObjects, ObjectsToReplace);
	}

	UE::Private::FReplaceReferenceHelper::FindAndReplaceReferences(SourceObjects, &ObjectsThatShouldUseOldStuff, ObjectsToReplace, OldToNewInstanceMap, ReinstancedObjectsWeakReferenceMap);

	if (ClassToReinstance->IsChildOf<UActorComponent>())
	{
		// ReplaceInstancesOfClass() handles this itself, if we had to re-instance
		ReconstructOwnerInstances(ClassToReinstance);
	}
}

void FBlueprintCompileReinstancer::CompileChildren()
{
	BP_SCOPED_COMPILER_EVENT_STAT(EKismetReinstancerStats_RecompileChildClasses);

	// Reparent all dependent blueprints, and recompile to ensure that they get reinstanced with the new memory layout
	for (UBlueprint* BP : Children)
	{
		if (BP->ParentClass == ClassToReinstance || BP->ParentClass == DuplicatedClass)
		{
			ReparentChild(BP);

			// avoid the skeleton compile if we don't need it - if the class 
			// we're reinstancing is a Blueprint class, then we assume sub-class
			// skeletons were kept in-sync (updated/reinstanced when the parent 
			// was updated); however, if this is a native class (like when hot-
			// reloading), then we want to make sure to update the skel as well
			EBlueprintCompileOptions Options = EBlueprintCompileOptions::SkipGarbageCollection;
			if(!ClassToReinstance->HasAnyClassFlags(CLASS_Native))
			{
				Options |= EBlueprintCompileOptions::SkeletonUpToDate;
			}
			FKismetEditorUtilities::CompileBlueprint(BP, Options);
		}
		else if (IsReinstancingSkeleton())
		{
			const bool bForceRegeneration = true;
			FKismetEditorUtilities::GenerateBlueprintSkeleton(BP, bForceRegeneration);
		}
	}
}

TSharedPtr<FReinstanceFinalizer> FBlueprintCompileReinstancer::ReinstanceInner(bool bForceAlwaysReinstance)
{
	TSharedPtr<FReinstanceFinalizer> Finalizer;
	if (ClassToReinstance && DuplicatedClass)
	{
		static const FBoolConfigValueHelper ReinstanceOnlyWhenNecessary(TEXT("Kismet"), TEXT("bReinstanceOnlyWhenNecessary"), GEngineIni);
		bool bShouldReinstance = true;
		// See if we need to do a full reinstance or can do the faster refresh path (when enabled or no values were modified, and the structures match)
		if (ReinstanceOnlyWhenNecessary && !bForceAlwaysReinstance)
		{
			BP_SCOPED_COMPILER_EVENT_STAT(EKismetReinstancerStats_ReplaceClassNoReinsancing);

			const UBlueprintGeneratedClass* BPClassA = Cast<const UBlueprintGeneratedClass>(DuplicatedClass);
			const UBlueprintGeneratedClass* BPClassB = Cast<const UBlueprintGeneratedClass>(ClassToReinstance);
			const UBlueprint* BP = Cast<const UBlueprint>(ClassToReinstance->ClassGeneratedBy);

			const bool bTheSameDefaultValues = (BP != nullptr) && (ClassToReinstanceDefaultValuesCRC != 0) && (BP->CrcLastCompiledCDO == ClassToReinstanceDefaultValuesCRC);
			const bool bTheSameLayout = (BPClassA != nullptr) && (BPClassB != nullptr) && FStructUtils::TheSameLayout(BPClassA, BPClassB, true);
			const bool bAllowedToDoFastPath = bTheSameDefaultValues && bTheSameLayout;
			if (bAllowedToDoFastPath)
			{
				Finalizer = ReinstanceFast();
				bShouldReinstance = false;
			}
		}

		if (bShouldReinstance)
		{
			UE_LOGF(LogBlueprint, Log, "BlueprintCompileReinstancer: Doing a full reinstance on class '%ls'", *GetPathNameSafe(ClassToReinstance));

			FReplaceInstancesOfClassParameters Params;
			Params.OriginalCDO = OriginalCDO;
			Params.ObjectsThatShouldUseOldStuff = &ObjectsThatShouldUseOldStuff;
			Params.bClassObjectReplaced = IsClassObjectReplaced();
			Params.bPreserveRootComponent = ShouldPreserveRootComponentOfReinstancedActor();
			ReplaceInstancesOfClass(DuplicatedClass, ClassToReinstance, Params);
		}
	}
	return Finalizer;
}

void FBlueprintCompileReinstancer::BlueprintWasRecompiled(UBlueprint* BP, bool bBytecodeOnly)
{
}

void FBlueprintCompileReinstancer::ReinstanceObjects(bool bForceAlwaysReinstance)
{
	BP_SCOPED_COMPILER_EVENT_STAT(EKismetCompilerStats_ReinstanceObjects);
	
	// Make sure we only reinstance classes once!
	static TArray<TSharedRef<FBlueprintCompileReinstancer>> QueueToReinstance;

	if (!bHasReinstanced)
	{
		TSharedRef<FBlueprintCompileReinstancer> SharedThis = AsShared();
		bool bAlreadyQueued = QueueToReinstance.Contains(SharedThis);

		// We may already be reinstancing this class, this happens when a dependent blueprint has a compile error and we try to reinstance the stub:
		if (!bAlreadyQueued)
		{
			for (const TSharedRef<FBlueprintCompileReinstancer>& Entry : QueueToReinstance)
			{
				if (Entry->ClassToReinstance == SharedThis->ClassToReinstance)
				{
					bAlreadyQueued = true;
					break;
				}
			}
		}

		if (!bAlreadyQueued)
		{
			QueueToReinstance.Push(SharedThis);

			if (ClassToReinstance && DuplicatedClass)
			{
				CompileChildren();
			}

			if (QueueToReinstance.Num() && (QueueToReinstance[0] == SharedThis))
			{
				// Mark it as the source reinstancer, no other reinstancer can get here until this Blueprint finishes compiling
				bIsRootReinstancer = true;

				if (!IsReinstancingSkeleton())
				{
					TGuardValue<std::atomic<bool>, bool> ReinstancingGuard(GIsReinstancing, true);

					TArray<TSharedPtr<FReinstanceFinalizer>> Finalizers;

					// All children were recompiled. It's safe to reinstance.
					for (int32 Idx = 0; Idx < QueueToReinstance.Num(); ++Idx)
					{
						TSharedPtr<FReinstanceFinalizer> Finalizer = QueueToReinstance[Idx]->ReinstanceInner(bForceAlwaysReinstance);
						if (Finalizer.IsValid())
						{
							Finalizers.Push(Finalizer);
						}
						QueueToReinstance[Idx]->bHasReinstanced = true;
					}
					QueueToReinstance.Empty();

					for (TSharedPtr<FReinstanceFinalizer>& Finalizer : Finalizers)
					{
						if (Finalizer.IsValid())
						{
							Finalizer->Finalize();
						}
					}

					if (GEditor)
					{
						GEditor->BroadcastBlueprintCompiled();
					}
				}
				else
				{
					QueueToReinstance.Empty();
				}
			}
		}
	}
}


class FArchiveReplaceFieldReferences : public FArchiveReplaceObjectRefBase
{
public:
	/**
	 * Initializes variables and starts the serialization search
	 *
	 * @param InSearchObject		The object to start the search on
	 * @param ReplacementMap		Map of objects to find -> objects to replace them with (null zeros them)
	 */
	FArchiveReplaceFieldReferences(UObject* InSearchObject, const TMap<FFieldVariant, FFieldVariant>& InReplacementMap)
		: ReplacementMap(InReplacementMap)
	{
		SearchObject = InSearchObject;
		Count = 0;
		bNullPrivateReferences = false;

		ArIsObjectReferenceCollector = true;
		ArIsModifyingWeakAndStrongReferences = true;		// Also replace weak references too!
		ArIgnoreArchetypeRef = true;
		ArIgnoreOuterRef = true;
		ArIgnoreClassGeneratedByRef = true;

		SerializeSearchObject();
	}

	/**
	 * Starts the serialization of the root object
	 */
	void SerializeSearchObject()
	{
		ReplacedReferences.Reset();

		if (SearchObject != NULL && !SerializedObjects.Find(SearchObject)
			&& (ReplacementMap.Num() > 0 || bNullPrivateReferences))
		{
			// start the initial serialization
			SerializedObjects.Add(SearchObject);
			SerializingObject = SearchObject;
			SerializeObject(SearchObject);
			for (int32 Iter = 0; Iter < PendingSerializationObjects.Num(); Iter++)
			{
				SerializingObject = PendingSerializationObjects[Iter];
				SerializeObject(SerializingObject);
			}
			PendingSerializationObjects.Reset();
		}
	}

	/**
	 * Serializes the reference to the object
	 */
	virtual FArchive& operator << (UObject*& Obj) override
	{
		if (Obj != nullptr)
		{
			// If these match, replace the reference
			const FFieldVariant* ReplaceWith = ReplacementMap.Find(Obj);
			if (ReplaceWith != nullptr)
			{
				Obj = ReplaceWith->ToUObject();
				if (bTrackReplacedReferences)
				{
					ReplacedReferences.FindOrAdd(SerializingObject).AddUnique(GetSerializedProperty());
				}
				Count++;
			}
			// A->IsIn(A) returns false, but we don't want to NULL that reference out, so extra check here.
			else if (Obj == SearchObject || Obj->IsIn(SearchObject))
			{
#if 0
				// DEBUG: Log when we are using the A->IsIn(A) path here.
				if (Obj == SearchObject)
				{
					FString ObjName = Obj->GetPathName();
					UE_LOGF(LogSerialization, Log, "FArchiveReplaceObjectRef: Obj == SearchObject : '%ls'", *ObjName);
				}
#endif
				bool bAlreadyAdded = false;
				SerializedObjects.Add(Obj, &bAlreadyAdded);
				if (!bAlreadyAdded)
				{
					// No recursion
					PendingSerializationObjects.Add(Obj);
				}
			}
			else if (bNullPrivateReferences && !Obj->HasAnyFlags(RF_Public))
			{
				Obj = nullptr;
			}
		}
		return *this;
	}


	/**
	 * Serializes the reference to a field
	 */
	virtual FArchive& operator << (FField*& Field) override
	{
		if (Field != nullptr)
		{
			// If these match, replace the reference
			const FFieldVariant* ReplaceWith = ReplacementMap.Find(Field);
			if (ReplaceWith != nullptr)
			{
				Field = ReplaceWith->ToField();
				Count++;
			}
		}
		return *this;
	}

	/**
	 * Serializes a resolved or unresolved object reference
	 */
	FArchive& operator<<( FObjectPtr& Obj )
	{
		if (ShouldSkipReplacementCheckForObjectPtr(Obj, ReplacementMap, [] (const TPair<FFieldVariant, FFieldVariant>& ReplacementPair) -> const UObject*
			{
				if (ReplacementPair.Key.IsValid() && ReplacementPair.Key.IsUObject())
				{
					return ReplacementPair.Key.ToUObject();
				}
				return nullptr;
			}))
		{
			return *this;
		}

		// Allow object references to go through the normal code path of resolving and running the raw pointer code path
		return FArchiveReplaceObjectRefBase::operator<<(Obj);
	}

protected:
	/** Map of objects to find references to -> object to replace references with */
	const TMap<FFieldVariant, FFieldVariant>& ReplacementMap;
};

void FBlueprintCompileReinstancer::UpdateBytecodeReferences(
	TSet<UBlueprint*>& OutDependentBlueprints,
	TMap<FFieldVariant, FFieldVariant>& OutFieldMapping)
{
	BP_SCOPED_COMPILER_EVENT_STAT(EKismetReinstancerStats_UpdateBytecodeReferences);

	if (!ClassToReinstance)
	{
		return;
	}

	if(UBlueprint* CompiledBlueprint = UBlueprint::GetBlueprintFromClass(ClassToReinstance))
	{
		TMap<FFieldVariant, FFieldVariant> FieldMappings;
		GenerateFieldMappings(FieldMappings);
		OutFieldMapping.Append(FieldMappings);

		// Note: This API returns a cached set of blueprints that's updated at compile time.
		TArray<UBlueprint*> CachedDependentBPs;
		FBlueprintEditorUtils::GetDependentBlueprints(CompiledBlueprint, CachedDependentBPs);

		// Determine whether or not we will be updating references for an Animation Blueprint class.
		const bool bIsAnimBlueprintClass = !!Cast<UAnimBlueprint>(ClassToReinstance->ClassGeneratedBy);

		for (auto BpIt = CachedDependentBPs.CreateIterator(); BpIt; ++BpIt)
		{
			UBlueprint* DependentBP = *BpIt;
			UClass* BPClass = DependentBP->GeneratedClass;

			// Skip cases where the class is junk, or haven't finished serializing in yet
			// Note that BPClass can be null for blueprints that can no longer be compiled:
			if (!BPClass
				|| (BPClass == ClassToReinstance)
				|| (BPClass->GetOutermost() == GetTransientPackage()) 
				|| BPClass->HasAnyClassFlags(CLASS_NewerVersionExists)
				|| (BPClass->ClassGeneratedBy && BPClass->ClassGeneratedBy->HasAnyFlags(RF_NeedLoad|RF_BeingRegenerated)) )
			{
				continue;
			}

			BPClass->ClearFunctionMapsCaches();

			// Ensure that Animation Blueprint child class dependencies are always re-linked, as the child may reference properties generated during
			// compilation of the parent class, which will have shifted to a TRASHCLASS Outer at this point (see UAnimBlueprintGeneratedClass::Link()).
			if(bIsAnimBlueprintClass && BPClass->IsChildOf(ClassToReinstance))
			{
				BPClass->StaticLink(true);
			}

			// For each function defined in this blueprint, run through the bytecode, and update any refs from the old properties to the new
			for( TFieldIterator<UFunction> FuncIter(BPClass, EFieldIteratorFlags::ExcludeSuper); FuncIter; ++FuncIter )
			{
				UFunction* CurrentFunction = *FuncIter;
				FArchiveReplaceFieldReferences ReplaceAr(CurrentFunction, FieldMappings);
			}

			// Update any refs in called functions array, as the bytecode was just similarly updated:
			if(UBlueprintGeneratedClass* AsBPGC = Cast<UBlueprintGeneratedClass>(BPClass))
			{
				for(int32 Idx = 0; Idx < AsBPGC->CalledFunctions.Num(); ++Idx)
				{
					FFieldVariant* Val = FieldMappings.Find(AsBPGC->CalledFunctions[Idx]);
					if(Val && Val->IsValid())
					{
						// This ::Cast should always succeed, but I'm uncomfortable making 
						// rigid assumptions about the FieldMappings array:
						if(UFunction* NewFn = Val->Get<UFunction>())
						{
							AsBPGC->CalledFunctions[Idx] = NewFn;
						}
					}
				}
			}

			OutDependentBlueprints.Add(DependentBP);
		}
	}
}

void FBlueprintCompileReinstancer::SaveSparseClassData(const UClass* ForClass)
{
	check(ForClass);
	UClass* SuperClass = ForClass->GetSuperClass();
	const void* SCD = const_cast<UClass*>(ForClass)->GetSparseClassData(EGetSparseClassDataMethod::ReturnIfNull);
	if (!SuperClass || !SCD)
	{
		return; // null SuperClass should only be possible for UObject, but good to be complete
	}

	FObjectWriter Writer(SCDSnapshot);
	UScriptStruct* SuperSCDType = SuperClass->GetSparseClassDataStruct();
	const void* SuperSCD = SuperClass->GetSparseClassData(EGetSparseClassDataMethod::ReturnIfNull);

	ForClass->GetSparseClassDataStruct()->SerializeTaggedProperties(
		Writer,
		(uint8*)SCD,
		(UStruct*)SuperSCDType,
		(uint8*)SuperSCD);
}

void FBlueprintCompileReinstancer::TakeOwnershipOfSparseClassData(UClass* ForClass)
{
	check(ForClass);
	OriginalSCDStruct = ForClass->GetSparseClassDataStruct();
	if (!OriginalSCDStruct)
	{
		return;
	}

	OriginalSCD = const_cast<void*>(ForClass->GetSparseClassData(EGetSparseClassDataMethod::ReturnIfNull));
	if (OriginalSCDStruct->GetOuter() == ForClass)
	{
		OriginalSCDStruct->Rename(nullptr, DuplicatedClass, REN_DoNotDirty | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch | REN_NonTransactional);
	}
	// We own these now, remove ForClass's knowledge of the sparse class data - they
	// will be freed when reinstancing is complete:
	ForClass->SparseClassData = nullptr;
	ForClass->SparseClassDataStruct = nullptr;
}

void FBlueprintCompileReinstancer::PropagateSparseClassDataToNewClass(UClass* NewClass)
{
	if (!OriginalSCD || 
		NewClass->GetSparseClassData(EGetSparseClassDataMethod::ReturnIfNull))
	{
		return;
	}

	UScriptStruct* SparseClassDataStruct = OriginalSCDStruct;
	if (UScriptStruct* NewSCD = NewClass->GetSparseClassDataStruct())
	{
		SparseClassDataStruct = NewSCD;
	}

	if (!IsValid(SparseClassDataStruct) ||
		SparseClassDataStruct->GetOutermost() == GetTransientPackage())
	{
		return;
	}

	if (SparseClassDataStruct == OriginalSCDStruct && SparseClassDataStruct->GetOuter() == DuplicatedClass)
	{
		SparseClassDataStruct->Rename(nullptr, NewClass, REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional | REN_AllowPackageLinkerMismatch);
	}
	NewClass->SparseClassDataStruct = SparseClassDataStruct;
	NewClass->CreateSparseClassData();

	FObjectReader Reader(SCDSnapshot);
	SparseClassDataStruct->SerializeTaggedProperties(
		Reader,
		(uint8*)NewClass->SparseClassData,
		nullptr,
		nullptr);

	if (OriginalSCD && OriginalSCDStruct)
	{
		OriginalSCDStruct->DestroyStruct(OriginalSCD);
		FMemory::Free(OriginalSCD);
		OriginalSCD = nullptr;
		OriginalSCDStruct = nullptr;
	}
}

void FBlueprintCompileReinstancer::FinishUpdateBytecodeReferences(
	const TSet<UBlueprint*>& DependentBPs,
	const TMap<FFieldVariant, FFieldVariant>& FieldMappings)
{
	for (UBlueprint* DependentBP : DependentBPs)
	{
		FArchiveReplaceFieldReferences ReplaceInBPAr(DependentBP, FieldMappings);

		if (ReplaceInBPAr.GetCount())
		{
			UE_LOGF(LogBlueprint, Log, 
				"UpdateBytecodeReferences: %lld references were replaced in BP %ls", 
				ReplaceInBPAr.GetCount(), *GetPathNameSafe(DependentBP));
		}
	}
}

/** Lots of redundancy with ReattachActorsHelper */
struct FAttachedActorInfo
{
	FAttachedActorInfo()
		: AttachedActor(nullptr)
		, AttachedToSocket()
	{
	}

	AActor* AttachedActor;
	FName   AttachedToSocket;
};

struct FActorAttachmentData
{
	FActorAttachmentData();
	FActorAttachmentData(AActor* OldActor);
	FActorAttachmentData(const FActorAttachmentData&) = default;
	FActorAttachmentData& operator=(const FActorAttachmentData&) = default;
	FActorAttachmentData(FActorAttachmentData&&) = default;
	FActorAttachmentData& operator=(FActorAttachmentData&&) = default;
	~FActorAttachmentData() = default;

	AActor*          TargetAttachParent;
	USceneComponent* TargetParentComponent;
	FName            TargetAttachSocket;

	TArray<FAttachedActorInfo> PendingChildAttachments;
};

FActorAttachmentData::FActorAttachmentData()
	: TargetAttachParent(nullptr)
	, TargetParentComponent(nullptr)
	, TargetAttachSocket()
	, PendingChildAttachments()
{
}

FActorAttachmentData::FActorAttachmentData(AActor* OldActor)
{
	TargetAttachParent = nullptr;
	TargetParentComponent = nullptr;

	TArray<AActor*> AttachedActors;
	OldActor->GetAttachedActors(AttachedActors);

	// if there are attached objects detach them and store the socket names
	for (AActor* AttachedActor : AttachedActors)
	{
		USceneComponent* AttachedActorRoot = AttachedActor->GetRootComponent();
		if (AttachedActorRoot && AttachedActorRoot->GetAttachParent())
		{
			// Save info about actor to reattach
			FAttachedActorInfo Info;
			Info.AttachedActor = AttachedActor;
			Info.AttachedToSocket = AttachedActorRoot->GetAttachSocketName();
			PendingChildAttachments.Add(Info);
		}
	}

	if (USceneComponent* OldRootComponent = OldActor->GetRootComponent())
	{
		if (OldRootComponent->GetAttachParent() != nullptr)
		{
			TargetAttachParent = OldRootComponent->GetAttachParent()->GetOwner();
			// Root component should never be attached to another component in the same actor!
			if (TargetAttachParent == OldActor)
			{
				UE_LOGF(LogBlueprint, Warning, "ReplaceInstancesOfClass: RootComponent (%ls) attached to another component in this Actor (%ls).", *OldRootComponent->GetPathName(), *TargetAttachParent->GetPathName());
				TargetAttachParent = nullptr;
			}

			TargetAttachSocket = OldRootComponent->GetAttachSocketName();
			TargetParentComponent = OldRootComponent->GetAttachParent();
		}
	}
}

/** 
 * Utility struct that represents a single replacement actor. Used to cache off
 * attachment info for the old actor (the one being replaced), that will be
 * used later for the new actor (after all instances have been replaced).
 */
struct FActorReplacementHelper
{
	/** NOTE: this detaches OldActor from all child/parent attachments. */
	FActorReplacementHelper(AActor* InNewActor, AActor* OldActor, FActorAttachmentData&& InAttachmentData)
		: NewActor(InNewActor)
		, TargetWorldTransform(FTransform::Identity)
		, AttachmentData( MoveTemp(InAttachmentData) )
	{
		CachedActorData = StaticCastSharedPtr<FActorTransactionAnnotation>(OldActor->FindOrCreateTransactionAnnotation());
		TArray<AActor*> AttachedActors;
		OldActor->GetAttachedActors(AttachedActors);

		// Cache the actor initialization status
		bHasRegisteredAllComponents = OldActor->HasActorRegisteredAllComponents();
		bHasInitialized = OldActor->IsActorInitialized();
		bHasBegunPlay = OldActor->HasActorBegunPlay();
		bWasHiddenEdLevel = OldActor->bHiddenEdLevel;

		// if there are attached objects detach them and store the socket names
		for (AActor* AttachedActor : AttachedActors)
		{
			USceneComponent* AttachedActorRoot = AttachedActor->GetRootComponent();
			if (AttachedActorRoot && AttachedActorRoot->GetAttachParent())
			{
				AttachedActorRoot->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform);
			}
		}

		if (USceneComponent* OldRootComponent = OldActor->GetRootComponent())
		{
			if (OldRootComponent->GetAttachParent() != nullptr)
			{
				// detach it to remove any scaling
				OldRootComponent->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform);
			}

			// Save off transform
			TargetWorldTransform = OldRootComponent->GetComponentTransform();
			TargetWorldTransform.SetTranslation(OldRootComponent->GetComponentLocation()); // take into account any custom location
		}

		for (UActorComponent* OldActorComponent : OldActor->GetComponents())
		{
			if (OldActorComponent)
			{
				OldActorComponentNameMap.Add(OldActorComponent->GetFName(), OldActorComponent);
			}
		}
	}

	/**
	 * Runs construction scripts on the new actor and then finishes it off by
	 * attaching it to the same attachments that its predecessor was set with. 
	 */
	void Finalize(const TMap<UObject*, UObject*>& OldToNewInstanceMap, const TSet<UObject*>* ObjectsThatShouldUseOldStuff, const TArray<UObject*>& ObjectsToReplace, const TMap<FSoftObjectPath, UObject*>& ReinstancedObjectsWeakReferenceMap);

	/**
	* Takes the cached child actors, as well as the old AttachParent, and sets
	* up the new actor so that its attachment hierarchy reflects the old actor
	* that it is replacing. Must be called after *all* instances have been Finalized.
	*
	* @param OldToNewInstanceMap Mapping of reinstanced objects.
	*/
	void ApplyAttachments(const TMap<UObject*, UObject*>& OldToNewInstanceMap, const TSet<UObject*>* ObjectsThatShouldUseOldStuff, const TArray<UObject*>& ObjectsToReplace, const TMap<FSoftObjectPath, UObject*>& ReinstancedObjectsWeakReferenceMap);

private:
	/**
	 * Takes the cached child actors, and attaches them under the new actor.
	 *
	 * @param  RootComponent	The new actor's root, which the child actors should attach to.
	 * @param  OldToNewInstanceMap	Mapping of reinstanced objects. Used for when child and parent actor are of the same type (and thus parent may have been reinstanced, so we can't reattach to the old instance).
	 */
	void AttachChildActors(USceneComponent* RootComponent, const TMap<UObject*, UObject*>& OldToNewInstanceMap);

	AActor*          NewActor;
	FTransform       TargetWorldTransform;
	FActorAttachmentData AttachmentData;
	bool bHasRegisteredAllComponents = false;
	bool bHasInitialized = false;
	bool bHasBegunPlay = false;
	bool bWasHiddenEdLevel = false;

	/** Holds actor component data, etc. that we use to apply */
	TSharedPtr<FActorTransactionAnnotation> CachedActorData;

	TMap<FName, UActorComponent*> OldActorComponentNameMap;
};

void FActorReplacementHelper::Finalize(const TMap<UObject*, UObject*>& OldToNewInstanceMap, const TSet<UObject*>* ObjectsThatShouldUseOldStuff, const TArray<UObject*>& ObjectsToReplace, const TMap<FSoftObjectPath, UObject*>& ReinstancedObjectsWeakReferenceMap)
{
	if (!IsValid(NewActor))
	{
		return;
	}

	// because this is an editor context it's important to use this execution guard
	FEditorScriptExecutionGuard ScriptGuard;

	// Only run construction if this world was constructed in the first place, it could be halfway through loading
	UWorld* World = NewActor->GetWorld();
	if (World && World->IsInitialized())
	{
		// run the construction script, which will use the properties we just copied over
		// @TODO: This code is similar to AActor::RerunConstructionScripts and ideally could use shared code for restoring state

		bool bCanReRun = UBlueprint::IsBlueprintHierarchyErrorFree(NewActor->GetClass());
		if (NewActor->CurrentTransactionAnnotation.IsValid() && bCanReRun)
		{
			NewActor->CurrentTransactionAnnotation->ActorTransactionAnnotationData.ComponentInstanceData.FindAndReplaceInstances(OldToNewInstanceMap);
			NewActor->RerunConstructionScripts();
		}
		else if (CachedActorData.IsValid())
		{
			CachedActorData->ActorTransactionAnnotationData.ComponentInstanceData.FindAndReplaceInstances(OldToNewInstanceMap);
			const bool bErrorFree = NewActor->ExecuteConstruction(TargetWorldTransform, nullptr, &CachedActorData->ActorTransactionAnnotationData.ComponentInstanceData);
			if (!bErrorFree)
			{
				// Save off the cached actor data for once the blueprint has been fixed so we can reapply it
				NewActor->CurrentTransactionAnnotation = CachedActorData;
			}
		}
		else
		{
			FComponentInstanceDataCache DummyComponentData;
			NewActor->ExecuteConstruction(TargetWorldTransform, nullptr, &DummyComponentData);
		}

		// Try to restore gameplay initialization state
		// This is unsafe to call from a loading stack but that should never happen for an actor that was fully initialized
		// @TODO: If there is a need for this case, it must be deferred until later in the frame
		if (World->IsGameWorld() && bHasInitialized && ensure(!FUObjectThreadContext::Get().IsRoutingPostLoad))
		{
			// GAllowActorScriptExecutionInEditor must be false when we call events from initialization
			TGuardValue AutoRestore(GAllowActorScriptExecutionInEditor, false);

			// Restore initialization state
			NewActor->PreInitializeComponents();
			NewActor->InitializeComponents();
			NewActor->PostInitializeComponents();

			// Also call begin play if necessary
			if (bHasBegunPlay)
			{
				NewActor->DispatchBeginPlay(false);
			}
		}

		// Restore editor visibility
		if (bWasHiddenEdLevel)
		{
			NewActor->bHiddenEdLevel = true;
			NewActor->MarkComponentsRenderStateDirty();
		}
	}

	TMap<UObject*, UObject*> ConstructedComponentReplacementMap;
	for (UActorComponent* NewActorComponent : NewActor->GetComponents())
	{
		if (NewActorComponent)
		{
			if (UActorComponent** OldActorComponent = OldActorComponentNameMap.Find(NewActorComponent->GetFName()))
			{
				ConstructedComponentReplacementMap.Add(*OldActorComponent, NewActorComponent);
			}
		}
	}
	if (GEditor)
	{
		GEditor->NotifyToolsOfObjectReplacement(ConstructedComponentReplacementMap);
	}

	NewActor->Modify();
	if (GEditor)
	{
		ULayersSubsystem* Layers = GEditor->GetEditorSubsystem<ULayersSubsystem>();
		if (Layers)
		{
			Layers->InitializeNewActorLayers(NewActor);
		}
	}
}

void FActorReplacementHelper::ApplyAttachments(const TMap<UObject*, UObject*>& OldToNewInstanceMap, const TSet<UObject*>* ObjectsThatShouldUseOldStuff, const TArray<UObject*>& ObjectsToReplace, const TMap<FSoftObjectPath, UObject*>& ReinstancedObjectsWeakReferenceMap)
{
	USceneComponent* NewRootComponent = NewActor->GetRootComponent();
	if (NewRootComponent == nullptr)
	{
		return;
	}

	if (AttachmentData.TargetAttachParent)
	{
		UObject* const* NewTargetAttachParent = OldToNewInstanceMap.Find(AttachmentData.TargetAttachParent);
		if (NewTargetAttachParent)
		{
			AttachmentData.TargetAttachParent = CastChecked<AActor>(*NewTargetAttachParent);
		}
	}
	if (AttachmentData.TargetParentComponent)
	{
		UObject* const* NewTargetParentComponent = OldToNewInstanceMap.Find(AttachmentData.TargetParentComponent);
		if (NewTargetParentComponent && *NewTargetParentComponent)
		{
			AttachmentData.TargetParentComponent = CastChecked<USceneComponent>(*NewTargetParentComponent);
		}
	}

	// attach the new instance to original parent
	if (AttachmentData.TargetAttachParent != nullptr)
	{
		if (AttachmentData.TargetParentComponent == nullptr)
		{
			AttachmentData.TargetParentComponent = AttachmentData.TargetAttachParent->GetRootComponent();
		}
		else if(IsValid(AttachmentData.TargetParentComponent))
		{
			NewRootComponent->AttachToComponent(AttachmentData.TargetParentComponent, FAttachmentTransformRules::KeepWorldTransform, AttachmentData.TargetAttachSocket);
		}
	}

	AttachChildActors(NewRootComponent, OldToNewInstanceMap);
}

void FActorReplacementHelper::AttachChildActors(USceneComponent* RootComponent, const TMap<UObject*, UObject*>& OldToNewInstanceMap)
{
	// if we had attached children reattach them now - unless they are already attached
	for (FAttachedActorInfo& Info : AttachmentData.PendingChildAttachments)
	{
		// Check for a reinstanced attachment, and redirect to the new instance if found
		AActor* NewAttachedActor = Cast<AActor>(OldToNewInstanceMap.FindRef(Info.AttachedActor));
		if (NewAttachedActor)
		{
			Info.AttachedActor = NewAttachedActor;
		}

		// If this actor is no longer attached to anything, reattach
		check(Info.AttachedActor);
		if (IsValid(Info.AttachedActor) && Info.AttachedActor->GetAttachParentActor() == nullptr)
		{
			USceneComponent* ChildRoot = Info.AttachedActor->GetRootComponent();
			if (ChildRoot && ChildRoot->GetAttachParent() != RootComponent)
			{
				ChildRoot->AttachToComponent(RootComponent, FAttachmentTransformRules::KeepWorldTransform, Info.AttachedToSocket);
				ChildRoot->UpdateComponentToWorld();
			}
		}
	}
}

// 
namespace InstancedPropertyUtils
{
	typedef TMap<FName, UObject*> FInstancedPropertyMap;

	/** 
	 * Aids in finding instanced property values that will not be duplicated nor
	 * copied in CopyPropertiesForUnRelatedObjects().
	 */
	class FArchiveInstancedSubObjCollector : public FArchiveUObject
	{
	public:
		//----------------------------------------------------------------------
		FArchiveInstancedSubObjCollector(UObject* TargetObj, FInstancedPropertyMap& PropertyMapOut, bool bAutoSerialize = true)
			: Target(TargetObj)
			, InstancedPropertyMap(PropertyMapOut)
		{
			ArIsObjectReferenceCollector = true;
			this->SetIsPersistent(false);
			ArIgnoreArchetypeRef = false;

			if (bAutoSerialize)
			{
				RunSerialization();
			}
		}

		//----------------------------------------------------------------------
		FArchive& operator<<(FObjectPtr& Obj)
		{
			// Avoid resolving an FObjectPtr if it is not an instanced property
			FProperty* SerializingProperty = GetSerializedProperty();
			const bool bHasInstancedValue = SerializingProperty && SerializingProperty->HasAnyPropertyFlags(CPF_PersistentInstance);
			if (!bHasInstancedValue)
			{
				return *this;
			}

			return FArchiveUObject::operator<<(Obj);
		}

		FArchive& operator<<(UObject*& Obj)
		{
			if (Obj != nullptr)
			{
				FProperty* SerializingProperty = GetSerializedProperty();
				const bool bHasInstancedValue = SerializingProperty && SerializingProperty->HasAnyPropertyFlags(CPF_PersistentInstance);

				// default sub-objects are handled by CopyPropertiesForUnrelatedObjects()
				if (bHasInstancedValue && !Obj->IsDefaultSubobject())
				{
					
					UObject* ObjOuter = Obj->GetOuter();
					bool bIsSubObject = (ObjOuter == Target);
					// @TODO: handle nested sub-objects when we're more clear on 
					//        how this'll affect the makeup of the reinstanced object
// 					while (!bIsSubObject && (ObjOuter != nullptr))
// 					{
// 						ObjOuter = ObjOuter->GetOuter();
// 						bIsSubObject |= (ObjOuter == Target);
// 					}

					if (bIsSubObject)
					{
						InstancedPropertyMap.Add(SerializingProperty->GetFName(), Obj);
					}
				}
			}
			return *this;
		}

		//----------------------------------------------------------------------
		void RunSerialization()
		{
			InstancedPropertyMap.Empty();
			if (Target != nullptr)
			{
				Target->Serialize(*this);
			}
		}

	private:
		UObject* Target;
		FInstancedPropertyMap& InstancedPropertyMap;
	};

	/** 
	 * Duplicates and assigns instanced property values that may have been 
	 * missed by CopyPropertiesForUnRelatedObjects().
	 */
	class FArchiveInsertInstancedSubObjects : public FArchiveUObject
	{
	public:
		//----------------------------------------------------------------------
		FArchiveInsertInstancedSubObjects(UObject* TargetObj, const FInstancedPropertyMap& OldInstancedSubObjs, bool bAutoSerialize = true)
			: TargetCDO(TargetObj->GetClass()->GetDefaultObject())
			, Target(TargetObj)
			, OldInstancedSubObjects(OldInstancedSubObjs)
		{
			ArIsObjectReferenceCollector = true;
			ArIsModifyingWeakAndStrongReferences = true;

			if (bAutoSerialize)
			{
				RunSerialization();
			}
		}

		//----------------------------------------------------------------------
		FArchive& operator<<(FObjectPtr& Obj)
		{
			// Avoid resolving an FObjectPtr if it is not an instanced property
			FProperty* SerializingProperty = GetSerializedProperty();
			const bool bHasInstancedValue = SerializingProperty && SerializingProperty->HasAnyPropertyFlags(CPF_PersistentInstance);
			if (!bHasInstancedValue)
			{
				return *this;
			}

			return FArchiveUObject::operator<<(Obj);
		}

		FArchive& operator<<(UObject*& Obj)
		{
			if (Obj == nullptr)
			{
				if (FProperty* SerializingProperty = GetSerializedProperty())
				{
					if (UObject* const* OldInstancedObjPtr = OldInstancedSubObjects.Find(SerializingProperty->GetFName()))
					{
						const UObject* OldInstancedObj = *OldInstancedObjPtr;
						check(SerializingProperty->HasAnyPropertyFlags(CPF_PersistentInstance));

						UClass* TargetClass = TargetCDO->GetClass();
						// @TODO: Handle nested instances when we have more time to flush this all out  
						if (TargetClass->IsChildOf(SerializingProperty->GetOwnerClass()))
						{
							FObjectPropertyBase* SerializingObjProperty = CastFieldChecked<FObjectPropertyBase>(SerializingProperty);
							// being extra careful, not to create our own instanced version when we expect one from the CDO
							if (SerializingObjProperty->GetObjectPropertyValue_InContainer(TargetCDO) == nullptr)
							{
								// @TODO: What if the instanced object is of the same type 
								//        that we're currently reinstancing
								Obj = StaticDuplicateObject(OldInstancedObj, Target);// NewObject<UObject>(Target, OldInstancedObj->GetClass()->GetAuthoritativeClass(), OldInstancedObj->GetFName());
							}
						}
					}
				}
			}
			return *this;
		}

		//----------------------------------------------------------------------
		void RunSerialization()
		{
			if ((Target != nullptr) && (OldInstancedSubObjects.Num() != 0))
			{
				Target->Serialize(*this);
			}
		}

	private:
		UObject* TargetCDO;
		UObject* Target;
		const FInstancedPropertyMap& OldInstancedSubObjects;
	};
}

class FReplaceActorHelperSetActorInstanceGuid
{
public:
	FReplaceActorHelperSetActorInstanceGuid(AActor* InActor, const FGuid& InActorInstanceGuid)
	{
		FSetActorInstanceGuid SetActorInstanceGuid(InActor, InActorInstanceGuid);
	}
};

// @todo_deprecated - Remove in a future release.
void FBlueprintCompileReinstancer::ReplaceInstancesOfClass(UClass* OldClass, UClass* NewClass, UObject*	OriginalCDO, TSet<UObject*>* ObjectsThatShouldUseOldStuff, bool bClassObjectReplaced, bool bPreserveRootComponent)
{
	FReplaceInstancesOfClassParameters Options;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Options.OldClass = OldClass;
	Options.NewClass = NewClass;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	Options.OriginalCDO = OriginalCDO;
	Options.ObjectsThatShouldUseOldStuff = ObjectsThatShouldUseOldStuff;
	Options.bClassObjectReplaced = bClassObjectReplaced;
	Options.bPreserveRootComponent = bPreserveRootComponent;
	ReplaceInstancesOfClass(OldClass, NewClass, Options);
}

// @todo_deprecated - Remove in a future release.
void FBlueprintCompileReinstancer::ReplaceInstancesOfClassEx(const FReplaceInstancesOfClassParameters& Parameters )
{
	ReplaceInstancesOfClass(
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Parameters.OldClass,
		Parameters.NewClass,
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		Parameters
	);
}

// @todo_deprecated - Remove in a future release.
void FBlueprintCompileReinstancer::BatchReplaceInstancesOfClass(
	TMap<UClass*, UClass*>& InOldToNewClassMap,
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const FBatchReplaceInstancesOfClassParameters& BatchParams
PRAGMA_ENABLE_DEPRECATION_WARNINGS
)
{
	if (InOldToNewClassMap.Num() == 0)
	{
		return;
	}

	FReplaceInstancesOfClassParameters Params;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Params.bArchetypesAreUpToDate = BatchParams.bArchetypesAreUpToDate;
	Params.bReplaceReferencesToOldClasses = BatchParams.bReplaceReferencesToOldClasses;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	ReplaceInstancesOfClass_Inner(InOldToNewClassMap, Params);
}

void FBlueprintCompileReinstancer::ReplaceInstancesOfClass(UClass* OldClass, UClass* NewClass, const FReplaceInstancesOfClassParameters& Params)
{
	ensureMsgf(!Params.bClassObjectReplaced || Params.OriginalCDO != nullptr, TEXT("bClassObjectReplaced is not expected to be set without OriginalCDO"));

	TMap<UClass*, UClass*> OldToNewClassMap;
	OldToNewClassMap.Add(OldClass, NewClass);
	ReplaceInstancesOfClass_Inner(OldToNewClassMap, Params);
}

void FBlueprintCompileReinstancer::BatchReplaceInstancesOfClass(const TMap<UClass*, UClass*>& InOldToNewClassMap, const FReplaceInstancesOfClassParameters& Params)
{
	if (InOldToNewClassMap.Num() == 0)
	{
		return;
	}

	checkf(Params.OriginalCDO == nullptr, TEXT("This path requires OriginalCDO to be NULL - use ReplaceInstancesOfClass() if you need to set it"));
	ensureMsgf(!Params.bClassObjectReplaced, TEXT("bClassObjectReplaced is not expected to be set in this path - use ReplaceInstancesOfClass() instead"));

	ReplaceInstancesOfClass_Inner(InOldToNewClassMap, Params);
}

bool FBlueprintCompileReinstancer::ReinstancerOrderingFunction(UClass* A, UClass* B)
{
	int32 DepthA = 0;
	int32 DepthB = 0;
	UStruct* Iter = A ? A->GetSuperStruct() : nullptr;
	while (Iter)
	{
		++DepthA;
		Iter = Iter->GetSuperStruct();
	}

	Iter = B ? B->GetSuperStruct() : nullptr;
	while (Iter)
	{
		++DepthB;
		Iter = Iter->GetSuperStruct();
	}

	if (DepthA == DepthB && A && B)
	{
		return A->GetFName().LexicalLess(B->GetFName());
	}
	return DepthA < DepthB;
}

void FBlueprintCompileReinstancer::GetSortedClassHierarchy(UClass* ClassToSearch, TArray<UClass*>& OutHierarchy, UClass** OutNativeParent)
{
	GetDerivedClasses(ClassToSearch, OutHierarchy);

	UClass* Iter = ClassToSearch;
	while (Iter)
	{
		OutHierarchy.Add(Iter);

		// Store the latest native super struct that we know of
		if (Iter->IsNative() && OutNativeParent && *OutNativeParent == nullptr)
		{
			*OutNativeParent = Iter;
		}

		Iter = Iter->GetSuperClass();
	}

	// Sort the hierarchy to get a deterministic result
	OutHierarchy.Sort([](UClass& A, UClass& B)->bool { return FBlueprintCompileReinstancer::ReinstancerOrderingFunction(&A, &B); });
}

void FBlueprintCompileReinstancer::MoveDependentSkelToReinst(UClass* const OwnerClass, TMap<UClass*, UClass*>& NewSkeletonToOldSkeleton)
{
	// Gather the whole class hierarchy up the native class so that we can correctly create the REINST class parented to native
	TArray<UClass*> ClassHierarchy;
	UClass* NativeParentClass = nullptr;
	FBlueprintCompileReinstancer::GetSortedClassHierarchy(OwnerClass, ClassHierarchy, &NativeParentClass);
	check(NativeParentClass);

	// Traverse the class Hierarchy, and determine if the given class needs to be REINST and have its parent set to the one we created
	const int32 NewParentIndex = ClassHierarchy.Find(OwnerClass);

	for (int32 i = NewParentIndex; i < ClassHierarchy.Num(); ++i)
	{
		UClass* CurClass = ClassHierarchy[i];
		check(CurClass);
		const int32 PrevStructSize = CurClass->GetStructureSize();

		GIsDuplicatingClassForReinstancing = true;
		// Create a REINST version of the given class
		UObject* OldCDO = CurClass->GetDefaultObject(false);
		const FName ReinstanceName = MakeUniqueObjectName(GetTransientPackage(), CurClass->GetClass(), *(FString(TEXT("REINST_")) + *CurClass->GetName()));

		if (!IsValid(CurClass) || CurClass->HasAnyClassFlags(CLASS_NewerVersionExists))
		{
			if (UClass* const* NewSuper = NewSkeletonToOldSkeleton.Find(CurClass->GetSuperClass()))
			{
				CurClass->SetSuperStruct(*NewSuper);
				CurClass->Bind();
				CurClass->StaticLink(true);
			}
			continue;
		}

		UClass* ReinstClass = CastChecked<UClass>(StaticDuplicateObject(CurClass, GetTransientPackage(), ReinstanceName, ~RF_Transactional));
		
		ReinstClass->RemoveFromRoot();
		ReinstClass->ClassFlags |= CLASS_NewerVersionExists;

		GIsDuplicatingClassForReinstancing = false;

		UClass** OverridenParent = NewSkeletonToOldSkeleton.Find(ReinstClass->GetSuperClass());
		if (OverridenParent && *OverridenParent)
		{
			ReinstClass->SetSuperStruct(*OverridenParent);
		}

		ReinstClass->Bind();
		ReinstClass->StaticLink(true);

		// Mark this class as being an internal class to ease filtering when enumerating all BP classes. 
		if (UBlueprintGeneratedClass* BPReinstClass = Cast<UBlueprintGeneratedClass>(ReinstClass))
		{
			BPReinstClass->bIsInternalClass = true;
		}
		
		if (!ReinstClass->GetSparseClassDataStruct())
		{
			if (UScriptStruct* SparseClassDataStructArchetype = ReinstClass->GetSparseClassDataArchetypeStruct())
			{
				ReinstClass->SetSparseClassDataStruct(SparseClassDataStructArchetype);
			}
		}

		// Map the old class to the new one
		NewSkeletonToOldSkeleton.Add(CurClass, ReinstClass);

		// Actually move the old CDO reference out of the way
		if (OldCDO)
		{
			CurClass->SetDefaultObject(nullptr);
			OldCDO->Rename(nullptr, ReinstClass->GetOuter(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional | REN_AllowPackageLinkerMismatch);
			ReinstClass->SetDefaultObject(OldCDO);
			OldCDO->SetClass(ReinstClass);
		}

		// Ensure that we are not changing the class layout by setting a new super struct, 
		// if they do not match we may see crashes because instances of the structs do match the 
		// correct layout size
		const int32 NewStructSize = ReinstClass->GetStructureSize();
		ensure(PrevStructSize == NewStructSize);
	}
}

UClass* FBlueprintCompileReinstancer::MoveCDOToNewClass(UClass* OwnerClass, const TMap<UClass*, UClass*>& OldToNewMap, bool bAvoidCDODuplication)
{
	GIsDuplicatingClassForReinstancing = true;
	OwnerClass->ClassFlags |= CLASS_NewerVersionExists;
	
	ensureMsgf(!FBlueprintCompileReinstancer::IsReinstClass(OwnerClass), TEXT("OwnerClass should not be 'REINST_'! This means that a REINST class was parented to another REINST class, causing unwanted recursion!"));

	// For consistency I'm moving archetypes that are outered to the UClass aside. The current implementation
	// of IsDefaultSubobject (used by StaticDuplicateObject) will not duplicate these instances if they 
	// are based on the CDO, but if they are based on another archetype (ie, they are inherited) then
	// they will be considered sub objects and they will be duplicated. There is no reason to duplicate
	// these archetypes here, so we move them aside and restore them after the uclass has been duplicated:
	TArray<UObject*> OwnedObjects;
	GetObjectsWithOuter(OwnerClass, OwnedObjects, EGetObjectsFlags::None);
	// record original names:
	TArray<FName> OriginalNames;
	for(UObject* OwnedObject : OwnedObjects)
	{
		OriginalNames.Add(OwnedObject->GetFName());
		if(OwnedObject->HasAnyFlags(RF_ArchetypeObject))
		{
			OwnedObject->Rename(nullptr, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch | REN_NonTransactional);
		}
	}
	const int32 PrevStructSize = OwnerClass->GetStructureSize();

	UObject* OldCDO = OwnerClass->GetDefaultObject(false);
	const FName ReinstanceName = MakeUniqueObjectName(GetTransientPackage(), OwnerClass->GetClass(), *(FString(TEXT("REINST_")) + *OwnerClass->GetName()));

	checkf(IsValid(OwnerClass), TEXT("%s is invalid - will not duplicate successfully"), *(OwnerClass->GetName()));
	UClass* CopyOfOwnerClass = CastChecked<UClass>(StaticDuplicateObject(OwnerClass, GetTransientPackage(), ReinstanceName, ~RF_Transactional));

	CopyOfOwnerClass->RemoveFromRoot();
	OwnerClass->ClassFlags &= ~CLASS_NewerVersionExists;
	GIsDuplicatingClassForReinstancing = false;

	UClass * const* OverridenParent = OldToNewMap.Find(CopyOfOwnerClass->GetSuperClass());
	if(OverridenParent && *OverridenParent)
	{
		CopyOfOwnerClass->SetSuperStruct(*OverridenParent);
	}

	UBlueprintGeneratedClass* BPClassToReinstance = Cast<UBlueprintGeneratedClass>(OwnerClass);
	UBlueprintGeneratedClass* BPGDuplicatedClass = Cast<UBlueprintGeneratedClass>(CopyOfOwnerClass);
	if (BPGDuplicatedClass && BPClassToReinstance && BPClassToReinstance->OverridenArchetypeForCDO)
	{
		BPGDuplicatedClass->OverridenArchetypeForCDO = BPClassToReinstance->OverridenArchetypeForCDO;
	}

#if VALIDATE_UBER_GRAPH_PERSISTENT_FRAME
	if (BPGDuplicatedClass && BPClassToReinstance)
	{
		BPGDuplicatedClass->UberGraphFunctionKey = BPClassToReinstance->UberGraphFunctionKey;
	}
#endif

	UFunction* DuplicatedClassUberGraphFunction = BPGDuplicatedClass ? ToRawPtr(BPGDuplicatedClass->UberGraphFunction) : nullptr;
	if (DuplicatedClassUberGraphFunction)
	{
		DuplicatedClassUberGraphFunction->Bind();
		DuplicatedClassUberGraphFunction->StaticLink(true);
	}

	for( int32 I = 0; I < OwnedObjects.Num(); ++I )
	{
		UObject* OwnedArchetype = OwnedObjects[I];
		if(OwnedArchetype->HasAnyFlags(RF_ArchetypeObject))
		{
			OwnedArchetype->Rename(*OriginalNames[I].ToString(), OwnerClass, REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional | REN_AllowPackageLinkerMismatch);
		}
	}

	CopyOfOwnerClass->Bind();
	CopyOfOwnerClass->StaticLink(true);
	
	const int32 NewStructSize = CopyOfOwnerClass->GetStructureSize();
	ensure(PrevStructSize == NewStructSize);

	// Mark this class as being an internal class to ease filtering when enumerating all BP classes. 
	if (BPGDuplicatedClass)
	{
		BPGDuplicatedClass->bIsInternalClass = true;
	}
	
	// make sure we've bound to our native sparse data - this should have been done in
	// link but don't want to destabilize early adopters:
	if (!CopyOfOwnerClass->GetSparseClassDataStruct())
	{
		if (UScriptStruct* SparseClassDataStructArchetype = CopyOfOwnerClass->GetSparseClassDataArchetypeStruct())
		{
			CopyOfOwnerClass->SetSparseClassDataStruct(SparseClassDataStructArchetype);
		}
	}

	if(OldCDO)
	{
		// @todo: #dano, rename bAvoidCDODuplication because it's really a flag to move the CDO aside not 'prevent duplication':
		if(bAvoidCDODuplication)
		{
			OwnerClass->SetDefaultObject(nullptr);
			OldCDO->Rename(nullptr, CopyOfOwnerClass->GetOuter(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch | REN_NonTransactional);
			CopyOfOwnerClass->SetDefaultObject(OldCDO);
		}
		OldCDO->SetClass(CopyOfOwnerClass);
	}
	return CopyOfOwnerClass;
}

bool FBlueprintCompileReinstancer::IsReinstClass(const UClass* Class)
{
	static const FString ReinstPrefix = TEXT("REINST");
	return Class && Class->GetFName().ToString().StartsWith(ReinstPrefix);
}

void FBlueprintCompileReinstancer::ReplaceObjectHelper(const FReinstancingState& State, UObject*& OldObject, UClass* OldClass, UObject*& NewUObject, UClass* NewClass, int32 OldObjIndex, TArray<UObject*>& ObjectsToReplace, TFunctionRef<TArray<TObjectPtr<USceneComponent>>& (USceneComponent*)> GetAttachChildrenArray, bool bIsComponent, bool bArchetypesAreUpToDate, const TSet<UObject*>* InstancesThatShouldUseOldClass)
{
	SCOPED_LOADTIMER_ASSET_TEXT(*WriteToString<256>(TEXT("ReplaceObjectHelper "), *GetPathNameSafe(OldObject)));
	// If the old object was spawned from an archetype (i.e. not the CDO), we must use the new version of that archetype as the template object when constructing the new instance.
	UObject* NewArchetype = nullptr;
	if(bArchetypesAreUpToDate)
	{
		FName NewName = State.OldToNewNameMap.FindRef(OldObject);
		if (NewName == NAME_None)
		{
			// Otherwise, just use the old object's current name.
			NewName = OldObject->GetFName();
		}
		NewArchetype = UObject::GetArchetypeFromRequiredInfo(NewClass, OldObject->GetOuter(), NewName, OldObject->GetFlags() & RF_Reinst);
	}
	else
	{
		UObject* OldArchetype = OldObject->GetArchetype();
		NewArchetype = State.OldToNewInstanceMap.FindRef(OldArchetype);

		bool bArchetypeReinstanced = (OldArchetype == OldClass->GetDefaultObject()) || (NewArchetype != nullptr);
		// if we don't have a updated archetype to spawn from, we need to update/reinstance it
		while (!bArchetypeReinstanced)
		{
			int32 ArchetypeIndex = ObjectsToReplace.Find(OldArchetype);
			if (ArchetypeIndex != INDEX_NONE)
			{
				if (!ensure(ArchetypeIndex < OldObjIndex))
				{
					// if this object has an archetype, but it hasn't been 
					// reinstanced yet (but is queued to) then we need to swap out 
					// the two, and reinstance the archetype first
					ObjectsToReplace.Swap(ArchetypeIndex, OldObjIndex);
					OldObject = ObjectsToReplace[OldObjIndex];
					check(OldObject == OldArchetype);

					OldArchetype = OldObject->GetArchetype();
					NewArchetype = State.OldToNewInstanceMap.FindRef(OldArchetype);
					bArchetypeReinstanced = (OldArchetype == OldClass->GetDefaultObject()) || (NewArchetype != nullptr);
				}
				else
				{
					break;
				}
			}
			else
			{
				break;
			}
		}
		// Check that either this was an instance of the class directly, or we found a new archetype for it
		ensureMsgf(bArchetypeReinstanced, TEXT("Reinstancing non-actor (%s); failed to resolve archetype object - property values may be lost."), *OldObject->GetPathName());
	}

	// Prepares OldObject for cleanup and adds an entry for NewUObject to the replacement map.
	auto ReplaceObjectHelper_Finalize = [&State](UObject* OldObject, UObject* NewUObject)
	{
		OldObject->ClearFlags(RF_Standalone);
		OldObject->RemoveFromRoot();
		OldObject->MarkAsGarbage();
		ForEachObjectWithOuter(OldObject,
			[](UObject* ObjectInOuter)
			{
				ObjectInOuter->ClearFlags(RF_Standalone);
				ObjectInOuter->RemoveFromRoot();
				ObjectInOuter->MarkAsGarbage();
			}
		, EGetObjectsFlags::IncludeNestedObjects, RF_NoFlags, EInternalObjectFlags::Garbage);

		State.OldToNewInstanceMap.Add(OldObject, NewUObject);
	};

	EObjectFlags OldFlags = OldObject->GetFlags();
	UPackage* OldExternalPackage = OldObject->GetExternalPackage();

	FName OldName(OldObject->GetFName());

	// If the old object is in this table, we've already renamed it away in a previous iteration. Don't rename it again!
	if (!State.OldToNewNameMap.Contains(OldObject))
	{
		// If we're reinstancing a component template, we also need to rename any inherited templates that are found to be based on it, in order to preserve archetype paths.
		if (bIsComponent && OldObject->HasAllFlags(RF_ArchetypeObject) && OldObject->GetOuter()->IsA<UBlueprintGeneratedClass>())
		{
			// Gather all component templates from the current archetype to the farthest antecedent inherited template(s).
			TArray<UObject*> OldArchetypeObjects;
			UE::Private::FArchetypeReinstanceHelper::GetArchetypeObjects(OldObject, OldArchetypeObjects, RF_InheritableComponentTemplate);

			// Find a unique object name that does not conflict with anything in the scope of all outers in the template chain.
			const FString OldArchetypeName = UE::Private::FArchetypeReinstanceHelper::FindUniqueArchetypeObjectName(OldArchetypeObjects).ToString();

			for (UObject* OldArchetypeObject : OldArchetypeObjects)
			{
				State.OldToNewNameMap.Add(OldArchetypeObject, OldName);
				OldArchetypeObject->Rename(*OldArchetypeName, OldArchetypeObject->GetOuter(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
			}
		}
		else
		{
			OldObject->Rename(nullptr, OldObject->GetOuter(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
		}
	}
						
	{
		// We may have already renamed this object to temp space if it was an inherited archetype in a previous iteration; check for that here.
		FName NewName = State.OldToNewNameMap.FindRef(OldObject);
		if (NewName == NAME_None)
		{
			// Otherwise, just use the old object's current name.
			NewName = OldName;
		}

		UObject* DestinationOuter = OldObject->GetOuter();
		// Check to make sure our original outer hasn't already been reinstanced:
		if (UObject* const* ReinstancedOuter = State.OldToNewInstanceMap.Find(DestinationOuter))
		{
			if (*ReinstancedOuter != nullptr)
			{
				// Our outer has been replaced, use the newer object:
				DestinationOuter = *ReinstancedOuter;
			}
			else
			{	
				// Our outer has been replaced with NULL, which implies that it was not reinstanced, but we may have added it to the map to ensure that references to it get replaced with NULL (see ReplaceInstancesOfClass_Inner).
				if (UNLIKELY(IsValid(DestinationOuter)))
				{
					// The outer is still valid, so we'll reuse it as the scope for reinstancing the old object below. This would generally not be expected, so emit a warning, but allow it to pass just in case, to avoid losing data.
					UE_LOGF(LogBlueprint, Warning, "ReplaceObjectHelper(): OldObject (%ls) has an outer that was replaced with NULL, but is still valid for instancing.", *GetFullNameSafe(OldObject));
				}
				else
				{
					// The outer is either missing or pending garbage collection, so there is no need to replace inner objects at this scope. But we do need to update the replacement map so any references to it are replaced with NULL.
					UE_LOGF(LogBlueprint, Log, "ReplaceObjectHelper(): OldObject (%ls) will not be replaced because its outer is no longer valid and was not reinstanced.", *GetFullNameSafe(OldObject));
					ReplaceObjectHelper_Finalize(OldObject, nullptr);
					return;
				}
			}

			// since we're changing the destination outer, make sure that there's no chance of collision with
			// another object:
			UObject* ExistingObject = StaticFindObjectFast(UObject::StaticClass(), DestinationOuter, NewName);
			if (ExistingObject)
			{
				// Potential bug: if the conflict is an actor (e.g. actor template, we may need to use
				// UObject::Rename explicitly to prevent side effects)
				ExistingObject->Rename(
					nullptr,
					GetTransientPackage(),
					REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
			}
		}

		// Expected to be non-NULL at this point.
		checkf(DestinationOuter != nullptr, TEXT("Attempting to reinstance OldObject (%s) with a NULL DestinationOuter."), *GetFullNameSafe(OldObject));

		FObjectInstancingGraph PrePopulatedInstancingGraph;
		const bool bUsePrePopulatedInstancingGraph = PrePopulateInstancingGraphForArchetype(State.GraphCache, PrePopulatedInstancingGraph, OldObject, NewArchetype, &State.OldToNewInstanceMap);
		
		FMakeClassSpawnableOnScope TemporarilySpawnable(NewClass);
		NewUObject = NewObject<UObject>(DestinationOuter, NewClass, NewName, RF_NoFlags, NewArchetype, false, bUsePrePopulatedInstancingGraph ? &PrePopulatedInstancingGraph : nullptr, OldExternalPackage);
	}

	check(NewUObject != nullptr);

	NewUObject->SetFlags(OldFlags & RF_Reinst);

	TMap<UObject*, UObject*> CreatedInstanceMap;
	TArray< TTuple<UObject*, UObject*>> OrderedListOfObjectToCopy;
	PreCreateSubObjectsForReinstantiation(State.InOldToNewClassMap, OldObject, NewUObject, CreatedInstanceMap, &State.OldToNewInstanceMap, &OrderedListOfObjectToCopy, InstancesThatShouldUseOldClass);
	State.OldToNewInstanceMap.Append(CreatedInstanceMap);

	// Copy property values
	UEngine::FCopyPropertiesForUnrelatedObjectsParams Options;
	Options.bNotifyObjectReplacement = true;
	Options.bSkipCompilerGeneratedDefaults = true;
	Options.bOnlyHandleDirectSubObjects = true;
	Options.OptionalReplacementMappings = &State.OldToNewInstanceMap;
	if (FOverridableManager::Get().IsEnabled(OldObject))
	{
		Options.bReplaceInternalReferenceUponRead = true;
		Options.OptionalOldToNewClassMappings = &State.InOldToNewClassMap;
	}
	// this currently happens because of some misguided logic in UBlueprintGeneratedClass::FindArchetype that
	// points us to a mismatched archetype, in which case delta serialization becomes unsafe.. without
	// that logic we could lose data, so for now i'm disabling delta serialization when we detect that situation
	if(Options.SourceObjectArchetype && !OldObject->IsA(Options.SourceObjectArchetype->GetClass()))
	{
		Options.bDoDelta = false;
	}
	// We only need to copy properties of the pre-created instances, the rest of the default sub object is done inside the UEditorEngine::CopyPropertiesForUnrelatedObjects
	for (const auto& Pair : OrderedListOfObjectToCopy)	
	{
		UEditorEngine::CopyPropertiesForUnrelatedObjects(Pair.Key, Pair.Value, Options);
		FOverridableManager::Get().RestoreOverrideState(Pair.Key, Pair.Value);
	}

	UWorld* RegisteredWorld = nullptr;
	bool bWasRegistered = false;
	if (bIsComponent)
	{
		UActorComponent* OldComponent = CastChecked<UActorComponent>(OldObject);
		if (OldComponent->IsRegistered())
		{
			bWasRegistered = true;
			RegisteredWorld = OldComponent->GetWorld();
			OldComponent->UnregisterComponent();
		}
	}

	ReplaceObjectHelper_Finalize(OldObject, NewUObject);

	if (bIsComponent)
	{
		UActorComponent* Component = CastChecked<UActorComponent>(NewUObject);
		AActor* OwningActor = Component->GetOwner();
		if (OwningActor)
		{
			OwningActor->ResetOwnedComponents();

			// Check to see if they have an editor that potentially needs to be refreshed
			if (OwningActor->GetClass()->ClassGeneratedBy)
			{
				State.PotentialEditorsForRefreshing.AddUnique(OwningActor->GetClass()->ClassGeneratedBy);
			}

			// we need to keep track of actor instances that need 
			// their construction scripts re-ran (since we've just 
			// replaced a component they own)
			State.OwnersToRerunConstructionScript.Add(OwningActor);
		}

		if (bWasRegistered)
		{
			if (RegisteredWorld && OwningActor == nullptr)
			{
				// Thumbnail components are added to a World without an actor, so we must special case their
				// REINST to register them with the world again.
				// The old thumbnail component is GC'd and will ensure if all it's attachments are not released
				// @TODO: This special case can breakdown if the nature of thumbnail components changes and could
				// use a cleanup later.
				if (OldObject->GetOutermost() == GetTransientPackage())
				{
					if (USceneComponent* SceneComponent = Cast<USceneComponent>(OldObject))
					{
						GetAttachChildrenArray(SceneComponent).Empty();
						SceneComponent->SetupAttachment(nullptr);
					}
				}

				Component->RegisterComponentWithWorld(RegisteredWorld);
			}
			else
			{
				Component->RegisterComponent();
			}
		}
	}
}

static void ReplaceActorHelper(AActor* OldActor, UClass* OldClass, UObject*& NewUObject, UClass* NewClass, TMap<UObject*, UObject*>& OldToNewInstanceMap, const TMap<UClass*, UClass*>& InOldToNewClassMap, TMap<FSoftObjectPath, UObject*>& ReinstancedObjectsWeakReferenceMap, TMap<UObject*, FActorAttachmentData>& ActorAttachmentData, TArray<FActorReplacementHelper>& ReplacementActors, bool bPreserveRootComponent, bool& bSelectionChanged)
{
	FVector  Location = FVector::ZeroVector;
	FRotator Rotation = FRotator::ZeroRotator;
	if (USceneComponent* OldRootComponent = OldActor->GetRootComponent())
	{
		// We need to make sure that the GetComponentTransform() transform is up to date, but we don't want to run any initialization logic
		// so we silence the update, cache it off, revert the change (so no events are raised), and then directly update the transform
		// with the value calculated in ConditionalUpdateComponentToWorld:
		FScopedMovementUpdate SilenceMovement(OldRootComponent);

		OldRootComponent->ConditionalUpdateComponentToWorld();
		FTransform OldComponentToWorld = OldRootComponent->GetComponentTransform();
		SilenceMovement.RevertMove();

		OldRootComponent->SetComponentToWorld(OldComponentToWorld);
		Location = OldActor->GetActorLocation();
		Rotation = OldActor->GetActorRotation();
	}

	// If this actor was spawned from an Archetype, we spawn the new actor from the new version of that archetype
	UObject* OldArchetype = OldActor->GetArchetype();
	UWorld*  World = OldActor->GetWorld();
	AActor*  NewArchetype = Cast<AActor>(OldToNewInstanceMap.FindRef(OldArchetype));
	// Check that either this was an instance of the class directly, or we found a new archetype for it
	check(OldArchetype == OldClass->GetDefaultObject() || NewArchetype);

	// Spawn the new actor instance, in the same level as the original, but deferring running the construction script until we have transferred modified properties
	ULevel*  ActorLevel = OldActor->GetLevel();
	UClass* const* MappedClass = InOldToNewClassMap.Find(OldActor->GetClass());
	UClass*  SpawnClass = MappedClass ? *MappedClass : NewClass;

	FActorSpawnParameters SpawnInfo;
	SpawnInfo.OverrideLevel = ActorLevel;
	SpawnInfo.Owner = OldActor->GetOwner();
	SpawnInfo.Instigator = OldActor->GetInstigator();
	SpawnInfo.Template = NewArchetype;
	SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnInfo.bDeferConstruction = true;
	SpawnInfo.Name = OldActor->GetFName();
	SpawnInfo.ObjectFlags |= OldActor->GetFlags() & RF_Reinst;

	if (!OldActor->IsListedInSceneOutliner())
	{
		SpawnInfo.bHideFromSceneOutliner = true;
	}

	// Make sure to reuse the same external package if any
	SpawnInfo.bCreateActorPackage = false;
	SpawnInfo.OverridePackage = OldActor->GetExternalPackage();

	SpawnInfo.OverrideActorGuid = OldActor->GetActorGuid();

	// Don't go through AActor::Rename here because we aren't changing outers (the actor's level) and we also don't want to reset loaders
	// if the actor is using an external package. We really just want to rename that actor out of the way so we can spawn the new one in
	// the exact same package, keeping the package name intact.
	OldActor->UObject::Rename(nullptr, OldActor->GetOuter(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);

	const bool bPackageNewlyCreated = OldActor->GetExternalPackage() && OldActor->GetExternalPackage()->HasAnyPackageFlags(PKG_NewlyCreated);

	// Unregister native components so we don't copy any sub-components they generate for themselves (like UCameraComponent does)
	// Perform this before spawning the new one to avoid collisions for components registering to external systems (e.g., actor will reuse the same guid)
	bool bHadRegisteredComponents = OldActor->HasActorRegisteredAllComponents();
	OldActor->UnregisterAllComponents();

	AActor* NewActor = nullptr;
	{
		FMakeClassSpawnableOnScope TemporarilySpawnable(SpawnClass);
		NewActor = World->SpawnActor(SpawnClass, &Location, &Rotation, SpawnInfo);
	}

	if (OldActor->CurrentTransactionAnnotation.IsValid())
	{
		NewActor->CurrentTransactionAnnotation = OldActor->CurrentTransactionAnnotation;
	}

	check(NewActor != nullptr);

	// Set the actor instance guid before registering components
	FReplaceActorHelperSetActorInstanceGuid SetActorInstanceGuid(NewActor, OldActor->GetActorInstanceGuid());

	// When Spawning an actor that has an external package the package can be PKG_NewlyCreated. 
	// We need to remove this flag if the package didn't have that flag prior to the SpawnActor. 
	// This means we are reinstancing an actor that was already saved on disk.
	if (UPackage* ExternalPackage = NewActor->GetExternalPackage())
	{
		if (!bPackageNewlyCreated && ExternalPackage->HasAnyPackageFlags(PKG_NewlyCreated))
		{
			ExternalPackage->ClearPackageFlags(PKG_NewlyCreated);
		}
	}

	// store the new actor for the second pass (NOTE: this detaches 
	// OldActor from all child/parent attachments)
	//
	// running the NewActor's construction-script is saved for that 
	// second pass (because the construction-script may reference 
	// another instance that hasn't been replaced yet).
	NewUObject = NewActor;

	FActorAttachmentData& CurrentAttachmentData = ActorAttachmentData.FindChecked(OldActor);
	ReplacementActors.Add(FActorReplacementHelper(NewActor, OldActor, MoveTemp(CurrentAttachmentData)));
	ActorAttachmentData.Remove(OldActor);

	ReinstancedObjectsWeakReferenceMap.Add(OldActor, NewUObject);

	OldActor->DestroyConstructedComponents(); // don't want to serialize components from the old actor

	// Unregister any native components, might have cached state based on properties we are going to overwrite
	NewActor->UnregisterAllComponents();

	UEngine::FCopyPropertiesForUnrelatedObjectsParams Params;
	Params.bPreserveRootComponent = bPreserveRootComponent;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Leaving this enabled for now for the purposes of the aggressive replacement auditing
	Params.bAggressiveDefaultSubobjectReplacement = true;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	Params.bNotifyObjectReplacement = true;
	Params.OptionalReplacementMappings = &OldToNewInstanceMap;
	// This shouldn't be possible, but if GetArchetype has a bug we could crash in delta serialization
	// attempting to use it:
	if (OldArchetype &&!ensure(OldActor->IsA(OldArchetype->GetClass())))
	{
		Params.bDoDelta = false;
	}
	UEngine::CopyPropertiesForUnrelatedObjects(OldActor, NewActor, Params);

	// reset properties/streams
	NewActor->ResetPropertiesForConstruction();

	// Only register the native components if the actor had already registered them
	if (bHadRegisteredComponents)
	{
		NewActor->RegisterAllComponents();
	}
	
	// 
	// clean up the old actor (unselect it, remove it from the world, etc.)...

	if (OldActor->IsSelected())
	{
		if(GEditor)
		{
			GEditor->SelectActor(OldActor, /*bInSelected =*/false, /*bNotify =*/false);
		}
		bSelectionChanged = true;
	}
	if (GEditor)
	{
		ULayersSubsystem* Layers = GEditor->GetEditorSubsystem<ULayersSubsystem>();
		if (Layers)
		{
			Layers->DisassociateActorFromLayers(OldActor);
		}
	}

	OldToNewInstanceMap.Add(OldActor, NewActor);
}

void FBlueprintCompileReinstancer::ReplaceInstancesOfClass_Inner(const TMap<UClass*, UClass*>& InOldToNewClassMap, const FReplaceInstancesOfClassParameters& Params)
{
	// If there is an original CDO, make sure we are only reinstancing a single class (legacy path, non-batch)
	UObject* InOriginalCDO = Params.OriginalCDO;
	check((InOriginalCDO != nullptr && InOldToNewClassMap.Num() == 1) || InOriginalCDO == nullptr);

	// This flag only applies to the legacy (i.e. non-batch) path.
	const bool bClassObjectReplaced = InOriginalCDO != nullptr && Params.bClassObjectReplaced;

	// If we're in the legacy path, always replace references to the CDO. Otherwise, it must be enabled.
	const bool bReplaceReferencesToOldCDOs = InOriginalCDO != nullptr || Params.bReplaceReferencesToOldCDOs;
	
	TSet<UObject*>* ObjectsThatShouldUseOldStuff = Params.ObjectsThatShouldUseOldStuff;
	const TSet<UObject*>* InstancesThatShouldUseOldClass = Params.InstancesThatShouldUseOldClass;
	const bool bPreserveRootComponent = Params.bPreserveRootComponent;
	const bool bArchetypesAreUpToDate = Params.bArchetypesAreUpToDate;
	const bool bReplaceReferencesToOldClasses = Params.bReplaceReferencesToOldClasses;

	USelection* SelectedActors = nullptr;
	TArray<UObject*> ObjectsReplaced;
	bool bSelectionChanged = false;
	bool bFixupSCS = false;
	const bool bLogConversions = false; // for debugging

	// Map of old objects to new objects
	TMap<UObject*, UObject*> OldToNewInstanceMap;

	// Map of old objects to new name (used to assist with reinstancing archetypes)
	TMap<UObject*, FName> OldToNewNameMap;

	TMap<FSoftObjectPath, UObject*> ReinstancedObjectsWeakReferenceMap;

	TMap<UObject*, FObjectInstancingGraph> GraphCache;

	// actors being replace
	TArray<FActorReplacementHelper> ReplacementActors;

	// A list of objects (e.g. Blueprints) that potentially have editors open that we need to refresh
	TArray<UObject*> PotentialEditorsForRefreshing;

	// A list of component owners that need their construction scripts re-ran (because a component of theirs has been reinstanced)
	TSet<AActor*> OwnersToRerunConstructionScript;

	// Set global flag to let system know we are reconstructing blueprint instances
	TGuardValue<bool> GuardTemplateNameFlag(GIsReconstructingBlueprintInstances, true);

	// Keep track of non-dirty packages for objects about to be reinstanced, so we can clear the dirty state after reinstancing them
	TSet<UPackage*> CleanPackageList;

	struct FObjectRemappingHelper
	{
		void OnObjectsReplaced(const TMap<UObject*, UObject*>& InReplacedObjects)
		{
			for (const TPair<UObject*, UObject*>& Pair : InReplacedObjects)
			{
				// CPFUO is going to tell us that the old class
				// has been replaced with the new class, but we created
				// the old class and we don't want to blindly replace
				// references to the old class. This could cause, for example,
				// the compilation manager to replace its references to the
				// old class with references to the new class:
				if (Pair.Key == nullptr || 
					Pair.Value == nullptr ||
					(	!Pair.Key->IsA<UClass>() &&
						!Pair.Value->IsA<UClass>()) )
				{
					ReplacedObjects.Add(Pair);
				}
			}
		}

		TMap<UObject*, UObject*> ReplacedObjects;
	} ObjectRemappingHelper;

	FDelegateHandle OnObjectsReplacedHandle = FCoreUObjectDelegates::OnObjectsReplaced.AddRaw(&ObjectRemappingHelper, &FObjectRemappingHelper::OnObjectsReplaced);

	{
		TArray<UObject*> ObjectsToReplace;
		TSet<UObject*> CachedArchetypeObjects;
		FEditorCacheArchetypeManager& CacheManager = FEditorCacheArchetypeManager::Get();

		BP_SCOPED_COMPILER_EVENT_STAT(EKismetReinstancerStats_ReplaceInstancesOfClass);

		// Reinstantiation can happen on the asyncloading thread and should not interact with GEditor in this case.
		if (IsInGameThread())
		{
			if(GEditor && GEditor->GetSelectedActors())
			{
				SelectedActors = GEditor->GetSelectedActors();

				// Note: For OFPA, each instance may be stored in its own external package.
				for (int32 SelectionIdx = 0; SelectionIdx < SelectedActors->Num(); ++SelectionIdx)
				{
					if (const UObject* SelectedActor = SelectedActors->GetSelectedObject(SelectionIdx))
					{
						FBlueprintCompileReinstancer::CheckAndSaveOuterPackageToCleanList(CleanPackageList, SelectedActor);
					}
				}
			
				SelectedActors->BeginBatchSelectOperation();
				SelectedActors->Modify();
			}
		}
		
		FDelegateHandle OnLevelActorDeletedHandle = GEngine ? GEngine->OnLevelActorDeleted().AddLambda([&OldToNewInstanceMap](AActor* DestroyedActor)
		{
			if (UObject** ReplacementObject = OldToNewInstanceMap.Find(DestroyedActor))
			{
				AActor* ReplacementActor = CastChecked<AActor>(*ReplacementObject);
				ReplacementActor->GetWorld()->EditorDestroyActor(ReplacementActor, /*bShouldModifyLevel =*/true);
			}
		}) : FDelegateHandle();

		FReinstancingState ReinstancingState
		{
			.ObjectsToReplaceScratch = ObjectsToReplace,
			.CachedArchetypeObjects = CachedArchetypeObjects,
			.OldToNewInstanceMap = OldToNewInstanceMap,
			.InOldToNewClassMap = InOldToNewClassMap,
			.OldToNewNameMap = OldToNewNameMap,
			.OldToNewTemplates = Params.OldToNewTemplates,
			.PotentialEditorsForRefreshing = PotentialEditorsForRefreshing,
			.OwnersToRerunConstructionScript = OwnersToRerunConstructionScript,
			.ReplacementActors = ReplacementActors,
			.ReinstancedObjectsWeakReferenceMap = ReinstancedObjectsWeakReferenceMap,
			.GraphCache = GraphCache,
			.ConstructionVersioningData = Params.ConstructionVersioningData,
			.ObjectsReplaced = ObjectsReplaced,
			.CleanPackageList = CleanPackageList,
			.bSelectionChanged = bSelectionChanged,
			.bFixupSCS = bFixupSCS
		};

		if(Params.ConstructionVersioningData)
		{
			check(bArchetypesAreUpToDate); // not supported, can't be
			check(Params.InstancesThatShouldUseOldClass == nullptr); // not supported, could be.. but why?

			ReinstanceWithVersioningData(ReinstancingState, InstancesThatShouldUseOldClass);
		}
		else
		{
			// Pre gather all the replacement map so that while doing reintanciation, we have all the information needed
			for (TPair<UClass*, UClass*> OldToNewClass : InOldToNewClassMap)
			{
				if (TMap<UObject*, UObject*>* ReplaceTemplateMapping = Params.OldToNewTemplates ? Params.OldToNewTemplates->Find(OldToNewClass.Key) : nullptr)
				{
					OldToNewInstanceMap.Append(*ReplaceTemplateMapping);
				}
			}

		// WARNING: for (TPair<UClass*, UClass*> OldToNewClass : InOldToNewClassMap) duplicated below 
		// to handle reconstructing actors which need to be reinstanced after their owned components 
		// have been updated:
		for (TPair<UClass*, UClass*> OldToNewClass : InOldToNewClassMap)
		{
			UClass* OldClass = OldToNewClass.Key;
			UClass* NewClass = OldToNewClass.Value;
			check(OldClass && NewClass);
			check(OldClass != NewClass || IsReloadActive());
			{
				const bool bIsComponent = NewClass->IsChildOf<UActorComponent>();

				// If any of the class changes are of an actor component to scene component or reverse then we will fixup SCS of all actors affected
				if (bIsComponent && !bFixupSCS)
				{
					bFixupSCS = (NewClass->IsChildOf<USceneComponent>() != OldClass->IsChildOf<USceneComponent>());
				}

				const bool bIncludeDerivedClasses = false;
				ObjectsToReplace.Reset();
				GetObjectsOfClass(OldClass, ObjectsToReplace, bIncludeDerivedClasses);

				if (InstancesThatShouldUseOldClass)
				{
					// Pre-remove instance that will not need to be replaced. That way ReplaceObjectHelper will not barf on the archetype not being replaced before its instances.
					for (auto It = ObjectsToReplace.CreateIterator(); It; ++It)
					{
						if (InstancesThatShouldUseOldClass->Contains(*It))
						{
							It.RemoveCurrentSwap();
						}
					}
				}

				if(!bArchetypesAreUpToDate)
				{
					Algo::TopologicalSort(ObjectsToReplace, [&ObjectsToReplace](const UObject* OldObject)
					{
						TArray<UObject*> Dependencies;
						UObject* Archetype = OldObject->GetArchetype();
						if (Archetype && ObjectsToReplace.Contains(Archetype) && !Archetype->HasAnyFlags(RF_ClassDefaultObject))
						{
							Dependencies.Add(Archetype);
						}
						return Dependencies;
					});
				}

				CacheArchetypes(ObjectsToReplace, CacheManager, CachedArchetypeObjects);
				
				// Then fix 'real' (non archetype) instances of the class
				for (int32 OldObjIndex = 0; OldObjIndex < ObjectsToReplace.Num(); ++OldObjIndex)
				{
					UObject* OldObject = ObjectsToReplace[OldObjIndex];

					// Skipping any default sub object that outer is going to be replaced
					// This isn't needed for the actor loop as the only outer for actor is a level
					if ((OldObject->IsDefaultSubobject() || OldObject->HasAnyFlags(RF_DefaultSubObject)) && InOldToNewClassMap.Contains(OldObject->GetOuter()->GetClass()))
					{
						continue;
					}

					// with IDOs, it's possible for a loose subobject to become non-loose again which the above condition would've missed.
					// that object will be part of the new archetype however so we want to skip reinstancing it here because the outer will handle that.
					if (UE::IsInstanceDataObjectSupportEnabled(OldObject))
					{
						if (UClass*const* NewOuterClass = InOldToNewClassMap.Find(OldObject->GetOuter()->GetClass()))
						{
							// if this subobject is in the outer's CDO, we can assume that it will be handled when the outer is reinstanced
							UObject* NewOuterCDO = (*NewOuterClass)->GetDefaultObject();
							if (StaticFindObjectFast(UObject::StaticClass(), NewOuterCDO, OldObject->GetFName()))
							{
								continue;
							}
						}
					}

					AActor* OldActor = Cast<AActor>(OldObject);
					bool bIsValid = IsValid(OldObject);

					// Skip archetype instances, EXCEPT for component templates and child actor templates
					const bool bIsChildActorTemplate = OldActor && OldActor->GetOuter()->IsA<UChildActorComponent>();
					if (!bIsValid)
					{
						// the object is invalid, we cannot replace it but we can ensure any external references
						// to it are nulled out, this is necessary because we've given it a new (invalid) REINST
						// class.
						if(!OldToNewInstanceMap.Contains(OldObject))
						{
							OldToNewInstanceMap.Add(OldObject, nullptr);
						}
						continue;
					}
					else if (!bIsComponent && !bIsChildActorTemplate &&
						    ( (bArchetypesAreUpToDate && OldObject->IsTemplate()) ||
						      (!bArchetypesAreUpToDate && OldObject->HasAnyFlags(RF_ClassDefaultObject))
						    )
						  )
					{
						continue;
					}

					// WARNING: This loop only handles non-actor objects, actor objects are handled below:
					if (OldActor == nullptr)
					{
						CheckAndSaveOuterPackageToCleanList(CleanPackageList, OldObject);

						UObject* NewUObject = nullptr;
						ReplaceObjectHelper(ReinstancingState, OldObject, OldClass, NewUObject, NewClass, OldObjIndex, ObjectsToReplace, &FDirectAttachChildrenAccessor::Get, bIsComponent, bArchetypesAreUpToDate, InstancesThatShouldUseOldClass);
						UpdateObjectBeingDebugged(OldObject, NewUObject);
						ObjectsReplaced.Add(OldObject);

						if (bLogConversions)
						{
							UE_LOGF(LogBlueprint, Log, "Converted instance '%ls' to '%ls'", *GetPathNameSafe(OldObject), *GetPathNameSafe(NewUObject));
						}
					}
				}
			}
		}

		// WARNING: for (TPair<UClass*, UClass*> OldToNewClass : InOldToNewClassMap) duplicated above 
		// this loop only handles actors - which need to be reconstructed *after* their owned components 
		// have been reinstanced:
		for (TPair<UClass*, UClass*> OldToNewClass : InOldToNewClassMap)
		{
			UClass* OldClass = OldToNewClass.Key;
			UClass* NewClass = OldToNewClass.Value;
			check(OldClass && NewClass);

			{
				const bool bIncludeDerivedClasses = false;
				ObjectsToReplace.Reset();
				GetObjectsOfClass(OldClass, ObjectsToReplace, bIncludeDerivedClasses);

				// store old attachment data before we mess with components, etc:
				TMap<UObject*, FActorAttachmentData> ActorAttachmentData;
				for (int32 OldObjIndex = 0; OldObjIndex < ObjectsToReplace.Num(); ++OldObjIndex)
				{
					UObject* OldObject = ObjectsToReplace[OldObjIndex];
					if(!IsValid(OldObject) || 
						(InstancesThatShouldUseOldClass && InstancesThatShouldUseOldClass->Contains(OldObject)))
					{
						continue;
					}

					if (AActor* OldActor = Cast<AActor>(OldObject))
					{
						ActorAttachmentData.Add(OldObject, FActorAttachmentData(OldActor));
					}
				}

				if (TMap<UObject*, UObject*>* ReplaceTemplateMapping = Params.OldToNewTemplates ? Params.OldToNewTemplates->Find(OldClass) : nullptr)
				{
					OldToNewInstanceMap.Append(*ReplaceTemplateMapping);
				}

				// Then fix 'real' (non archetype) instances of the class
				for (int32 OldObjIndex = 0; OldObjIndex < ObjectsToReplace.Num(); ++OldObjIndex)
				{
					UObject* OldObject = ObjectsToReplace[OldObjIndex];
					AActor* OldActor = Cast<AActor>(OldObject);

					// Skip archetype instances, EXCEPT for child actor templates
					const bool bIsChildActorTemplate = OldActor && OldActor->GetOuter()->IsA<UChildActorComponent>();
					if (!IsValid(OldObject) || 
						(!bIsChildActorTemplate && OldObject->IsTemplate()) ||
						(InstancesThatShouldUseOldClass && InstancesThatShouldUseOldClass->Contains(OldObject)))
					{
						continue;
					}
					
					// WARNING: This loop only handles actor objects that are in a level, all other objects are
					// handled above
					if (OldActor != nullptr)
					{
						// Note: For OFPA, each instance may be stored in its own external package.
						CheckAndSaveOuterPackageToCleanList(CleanPackageList, OldActor);

						UObject* NewUObject = nullptr;
						if (OldActor->GetLevel() && OldActor->GetWorld() && !OldActor->GetWorld()->IsCleanedUp())
						{
							CheckAndSaveAttachedActorsOuterPackageToCleanList(CleanPackageList, OldActor);
							ReplaceActorHelper(OldActor, OldClass, NewUObject, NewClass, OldToNewInstanceMap, InOldToNewClassMap, ReinstancedObjectsWeakReferenceMap, ActorAttachmentData, ReplacementActors, bPreserveRootComponent, bSelectionChanged);
						}
						else
						{
							// Actors that are not in a level cannot be reconstructed, sequencer team decided to reinstance these as normal objects:
							ReplaceObjectHelper(ReinstancingState, OldObject, OldClass, NewUObject, NewClass, OldObjIndex, ObjectsToReplace, &FDirectAttachChildrenAccessor::Get, false, bArchetypesAreUpToDate, InstancesThatShouldUseOldClass);
						}
						UpdateObjectBeingDebugged(OldObject, NewUObject);
						ObjectsReplaced.Add(OldObject);

						if (bLogConversions)
						{
							UE_LOGF(LogBlueprint, Log, "Converted instance '%ls' to '%ls'", *GetPathNameSafe(OldObject), *GetPathNameSafe(NewUObject));
						}
					}
				}
			}
		}
		}

		// Reset any cached archetypes
		for (UObject* CachedArchetypeObject : CachedArchetypeObjects)
		{
			CacheManager.ResetCacheArchetype(CachedArchetypeObject);
		}

		if (GEngine)
		{
			GEngine->OnLevelActorDeleted().Remove(OnLevelActorDeletedHandle);
		}

		for (TPair<UObject*, UObject*> ReinstancedPair : OldToNewInstanceMap)
		{
			if (AActor* OldActor = Cast<AActor>(ReinstancedPair.Key))
			{
				if (UWorld* World = OldActor->GetWorld())
				{
					World->EditorDestroyActor(OldActor, /*bShouldModifyLevel =*/true);
				}
			}
		}
	}

	FCoreUObjectDelegates::OnObjectsReplaced.Remove(OnObjectsReplacedHandle);

	// Now replace any pointers to the old archetypes/instances with pointers to the new one
	TArray<UObject*> SourceObjects;
	OldToNewInstanceMap.GenerateKeyArray(SourceObjects);
	
	TArray<UObject*> OldCDOSourceObjects;
	for (TPair<UClass*, UClass*> OldToNewClass : InOldToNewClassMap)
	{
		UClass* OldClass = OldToNewClass.Key;
		UClass* NewClass = OldToNewClass.Value;
		check(OldClass && NewClass);
		check(OldClass != NewClass || IsReloadActive());

		// Always map old to new instances of CDOs along with any owned subobject(s). This allows delegates to be
		// notified that these instances have been replaced. However, we don't proactively find and replace those
		// references ourselves unless input parameters have explicitly configured this path to do so (see below).
		OldCDOSourceObjects.Reset();
		UE::Private::FReplaceReferenceHelper::IncludeCDO(OldClass, NewClass, OldToNewInstanceMap, OldCDOSourceObjects, InOriginalCDO, Params.OldToNewTemplates);
		if (bReplaceReferencesToOldCDOs)
		{
			// This means we'll proactively find and replace references to old CDOs and any owned subobject(s). It
			// has an additional cost and is not enabled by default, since most systems don't store these references;
			// those that do (e.g. the editor's transaction buffer) may do their own reference replacement pass instead.
			SourceObjects.Append(OldCDOSourceObjects);

			// This is part of the legacy reload path; it is only enabled if we're also replacing references to old CDOs.
			if (bClassObjectReplaced)
			{
				UE::Private::FReplaceReferenceHelper::IncludeClass(OldClass, NewClass, OldToNewInstanceMap, SourceObjects, ObjectsReplaced);
			}
		}
	}

	{ BP_SCOPED_COMPILER_EVENT_STAT(EKismetReinstancerStats_ReplacementConstruction);

		// the process of setting up new replacement actors is split into two 
		// steps (this here, is the second)...
		// 
		// the "finalization" here runs the replacement actor's construction-
		// script and is left until late to account for a scenario where the 
		// construction-script attempts to modify another instance of the 
		// same class... if this were to happen above, in the ObjectsToReplace 
		// loop, then accessing that other instance would cause an assert in 
		// FProperty::ContainerPtrToValuePtrInternal() (which appropriatly 
		// complains that the other instance's type doesn't match because it 
		// hasn't been replaced yet... that's why we wait until after 
		// FArchiveReplaceObjectRef to run construction-scripts).
		for (FActorReplacementHelper& ReplacementActor : ReplacementActors)
		{
			ReplacementActor.Finalize(ObjectRemappingHelper.ReplacedObjects, ObjectsThatShouldUseOldStuff, ObjectsReplaced, ReinstancedObjectsWeakReferenceMap);
		}

		for (FActorReplacementHelper& ReplacementActor : ReplacementActors)
		{
			ReplacementActor.ApplyAttachments(ObjectRemappingHelper.ReplacedObjects, ObjectsThatShouldUseOldStuff, ObjectsReplaced, ReinstancedObjectsWeakReferenceMap);
		}

		OldToNewInstanceMap.Append(ObjectRemappingHelper.ReplacedObjects);
	}

	if(bReplaceReferencesToOldClasses)
	{
		check(ObjectsThatShouldUseOldStuff);

		for (TPair<UClass*, UClass*> OldToNew : InOldToNewClassMap)
		{
			ObjectsThatShouldUseOldStuff->Add(OldToNew.Key);

			TArray<UObject*> OldFunctions;
			GetObjectsWithOuter(OldToNew.Key, OldFunctions);
			ObjectsThatShouldUseOldStuff->Append(OldFunctions);
			
			OldToNewInstanceMap.Add(OldToNew.Key, OldToNew.Value);
			SourceObjects.Add(OldToNew.Key);
		}
	}

	//FReplaceReferenceHelper::ValidateReplacementMappings(OldToNewInstanceMap);
	// Don't run expensive FindAndReplaceReferences when no objects were replaced if we have versioning data.
	// SourceObjects will always be non zero when using this code path but we can't afford to do global
	// reference replacement at all times:
	if(!Params.ConstructionVersioningData || ObjectsReplaced.Num() > 0 ) 
	{
		UE::Private::FReplaceReferenceHelper::FindAndReplaceReferences(SourceObjects, ObjectsThatShouldUseOldStuff, ObjectsReplaced, OldToNewInstanceMap, ReinstancedObjectsWeakReferenceMap);
	}
	
	for (UObject* Obj : ObjectsReplaced)
	{
		UObject** NewObject = OldToNewInstanceMap.Find(Obj);
		if (NewObject && *NewObject)
		{
			if (Obj)
			{
				// Patch the new object into the old object linker's export map; subsequent loads may import
				// this entry and we need to make sure that it returns the new object instead of the old one.
				FLinkerLoad::PRIVATE_PatchNewObjectIntoExport(Obj, *NewObject);

				// In some cases (e.g. reparenting across a hierarchy), the new object may contain subobjects
				// (e.g. components) which are no longer binary-compatible with the old object's instance, which
				// may have been delta-serialized to the outermost package. In that case, we need to ensure that
				// the package (e.g. level/actor) remains dirty, so that the user sees that it requires a re-save.
				const UPackage* NewObjectPackage = (*NewObject)->GetPackage();
				if (CleanPackageList.Contains(NewObjectPackage) && NewObjectPackage->IsDirty())
				{
					bool bShouldPreservePackageDirtyState = false;
					ForEachObjectWithOuterBreakable(Obj, [&OldToNewInstanceMap, &bShouldPreservePackageDirtyState](UObject* OldSubobject)
					{
						if (UObject* NewSubobject = OldToNewInstanceMap.FindRef(OldSubobject))
						{
							// If the new subobject type is not of the old subobject type, then the old subobject's
							// data (if serialized) is no longer binary-compatible, and can no longer be imported on
							// load (i.e. it will fail), resulting in data loss. In that case, we need the dirty state.
							if (!NewSubobject->GetClass()->IsChildOf(OldSubobject->GetClass()))
							{
								bShouldPreservePackageDirtyState = true;
							}
						}

						// No need to continue the iteration once we've found a discrepancy.
						return !bShouldPreservePackageDirtyState;
					});

					if (bShouldPreservePackageDirtyState)
					{
						CleanPackageList.Remove(NewObjectPackage);
					}
				}
			}

			if (UAnimInstance* AnimTree = Cast<UAnimInstance>(*NewObject))
			{
				if (USkeletalMeshComponent* SkelComponent = Cast<USkeletalMeshComponent>(AnimTree->GetOuter()))
				{
					if(GUseLegacyAnimInstanceReinstancingBehavior)
					{
						// Legacy behavior - destroy and re-create the anim instance
						SkelComponent->ClearAnimScriptInstance();
						SkelComponent->InitAnim(true);

						// compile change ignores motion vector, so ignore this. 
						SkelComponent->ClearMotionVector();
					}
				}
			}
		}
	}

	// Reassociate relevant property bags 
	UE::FPropertyBagRepository::Get().ReassociateObjects(OldToNewInstanceMap);

	// Inform listeners of object reinstancing
	FCoreUObjectDelegates::OnObjectsReinstanced.Broadcast(OldToNewInstanceMap);
	
	if(SelectedActors)
	{
		SelectedActors->EndBatchSelectOperation();
	}

	if (bSelectionChanged && GEditor)
	{
		GEditor->NoteSelectionChange();
	}

	// Clear the dirty flag on packages that didn't already have it set prior to objects being reinstanced.
	for (UPackage* PackageToMarkAsClean : CleanPackageList)
	{
		check(PackageToMarkAsClean);
		PackageToMarkAsClean->SetDirtyFlag(false);
	}

	TSet<UBlueprintGeneratedClass*> FixedSCS;

	// in the case where we're replacing component instances, we need to make 
	// sure to re-run their owner's construction scripts
	for (AActor* ActorInstance : OwnersToRerunConstructionScript)
	{
		// Before rerunning the construction script, first fix up the SCS if any component class has changed from actor to scene
		if (bFixupSCS)
		{
			UBlueprintGeneratedClass* BPGC = Cast<UBlueprintGeneratedClass>(ActorInstance->GetClass());
			while (BPGC && !FixedSCS.Contains(BPGC))
			{
				if (BPGC->SimpleConstructionScript)
				{
					BPGC->SimpleConstructionScript->FixupRootNodeParentReferences();
					BPGC->SimpleConstructionScript->ValidateSceneRootNodes();
				}
				FixedSCS.Add(BPGC);
				BPGC = Cast<UBlueprintGeneratedClass>(BPGC->GetSuperClass());
			}
		}

		// Skipping CDOs as CSs are not allowed for them.
		if (!ActorInstance->HasAnyFlags(RF_ClassDefaultObject) && ActorInstance->GetLevel())
		{
			ActorInstance->RerunConstructionScripts();
		}
	}

	if (GEditor)
	{
		// Refresh any editors for objects that we've updated components for
		for (UObject* BlueprintAsset : PotentialEditorsForRefreshing)
		{
			FBlueprintEditor* BlueprintEditor = static_cast<FBlueprintEditor*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(BlueprintAsset, /*bFocusIfOpen =*/false));
			if (BlueprintEditor)
			{
				BlueprintEditor->RefreshEditors();
			}
		}
	}
}

void FBlueprintCompileReinstancer::ReconstructOwnerInstances(TSubclassOf<UActorComponent> ComponentClass)
{
	if (ComponentClass == nullptr)
	{
		return;
	}

	TArray<UObject*> ComponentInstances;
	GetObjectsOfClass(ComponentClass, ComponentInstances, /*bIncludeDerivedClasses =*/false);

	TSet<AActor*> OwnerInstances;
	for (UObject* ComponentObj : ComponentInstances)
	{
	
		UActorComponent* Component = CastChecked<UActorComponent>(ComponentObj);
			
		if (AActor* OwningActor = Component->GetOwner())
		{
			// we don't just rerun construction here, because we could end up 
			// doing it twice for the same actor (if it had multiple components 
			// of this kind), so we put that off as a secondary pass
			OwnerInstances.Add(OwningActor);
		}
	}

	for (AActor* ComponentOwner : OwnerInstances)
	{
		ComponentOwner->RerunConstructionScripts();
	}
}

void FBlueprintCompileReinstancer::VerifyReplacement()
{
	TArray<UObject*> SourceObjects;

	// Find all instances of the old class
	for( TObjectIterator<UObject> it; it; ++it )
	{
		UObject* CurrentObj = *it;

		if( (CurrentObj->GetClass() == DuplicatedClass) )
		{
			SourceObjects.Add(CurrentObj);
		}
	}

	// For each instance, track down references
	if( SourceObjects.Num() > 0 )
	{
		TFindObjectReferencers<UObject> Referencers(SourceObjects, nullptr, false);
		for (TFindObjectReferencers<UObject>::TIterator It(Referencers); It; ++It)
		{
			UObject* CurrentObject = It.Key();
			UObject* ReferencedObj = It.Value();
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("- Object %s is referencing %s ---"), *CurrentObject->GetName(), *ReferencedObj->GetName());
		}
	}
}
void FBlueprintCompileReinstancer::ReparentChild(UBlueprint* ChildBP)
{
	check(ChildBP);

	UClass* SkeletonClass = ChildBP->SkeletonGeneratedClass;
	UClass* GeneratedClass = ChildBP->GeneratedClass;

	const bool ReparentGeneratedOnly = (ReinstClassType == RCT_BpGenerated);
	if( !ReparentGeneratedOnly && SkeletonClass )
	{
		ReparentChild(SkeletonClass);
	}

	const bool ReparentSkelOnly = (ReinstClassType == RCT_BpSkeleton);
	if( !ReparentSkelOnly && GeneratedClass )
	{
		ReparentChild(GeneratedClass);
	}
}

void FBlueprintCompileReinstancer::ReparentChild(UClass* ChildClass)
{
	check(ChildClass && ClassToReinstance && DuplicatedClass && ChildClass->GetSuperClass());
	bool bIsReallyAChild = ChildClass->GetSuperClass() == ClassToReinstance || ChildClass->GetSuperClass() == DuplicatedClass;
	const UBlueprint* SuperClassBP = Cast<UBlueprint>(ChildClass->GetSuperClass()->ClassGeneratedBy);
	if (SuperClassBP && !bIsReallyAChild)
	{
		bIsReallyAChild |= (SuperClassBP->SkeletonGeneratedClass == ClassToReinstance) || (SuperClassBP->SkeletonGeneratedClass == DuplicatedClass);
		bIsReallyAChild |= (SuperClassBP->GeneratedClass == ClassToReinstance) || (SuperClassBP->GeneratedClass == DuplicatedClass);
	}
	check(bIsReallyAChild);

	ChildClass->AssembleReferenceTokenStream();
	ChildClass->SetSuperStruct(DuplicatedClass);
	ChildClass->Bind();
	ChildClass->StaticLink(true);
}

void FBlueprintCompileReinstancer::CopyPropertiesForUnrelatedObjects(UObject* OldObject, UObject* NewObject, bool bClearExternalReferences, bool bForceDeltaSerialization /* = false */, bool bOnlyHandleDirectSubObjects/* = false */, TMap<UObject*, UObject*>* OldToNewInstanceMap /*=nullptr*/, const TMap<UClass*,UClass*>* OldToNewClassMap /*=nullptr*/)
{
	SCOPED_LOADTIMER_ASSET_TEXT(*WriteToString<256>(TEXT("CopyPropertiesForUnrelatedObjects "), *GetPathNameSafe(NewObject)));
	UEngine::FCopyPropertiesForUnrelatedObjectsParams Params;
	// During a blueprint reparent, delta serialization must be enabled to correctly copy all properties
	Params.bDoDelta = bForceDeltaSerialization || !OldObject->HasAnyFlags(RF_ClassDefaultObject);
	Params.bCopyDeprecatedProperties = true;
	Params.bSkipCompilerGeneratedDefaults = true;
	Params.bClearReferences = bClearExternalReferences;
	Params.bNotifyObjectReplacement = true;
	Params.OptionalReplacementMappings = OldToNewInstanceMap;
	Params.bOnlyHandleDirectSubObjects = bOnlyHandleDirectSubObjects;
	// Overridable serialization needs this to be able to merge back containers of subobjects.
	if (FOverridableManager::Get().IsEnabled(OldObject))
	{
		Params.bReplaceInternalReferenceUponRead = true;
		Params.OptionalOldToNewClassMappings = OldToNewClassMap;
	}
	Params.bPreserveRootComponent = GUseLegacyRootComponentPreservation || 
		// If the native parent class has changed, we may need to conform the root component
		// this behavior may not be exactly correct but it's better than preserving 
		// the root component that was likely set by our old native parent:
		(FBlueprintEditorUtils::FindFirstNativeClass(OldObject->GetClass()) !=
		FBlueprintEditorUtils::FindFirstNativeClass(NewObject->GetClass()));
	if(!Params.bPreserveRootComponent)
	{
		// If the old actor has a bad root component then don't copy it
		// and just preserve the one created by the native constructor:
		if(const AActor* OldActor = Cast<AActor>(OldObject))
		{
			if(const USceneComponent* OldComponent = OldActor->GetRootComponent())
			{
				// this flag's name is somewhat confusing in this context, setting it to true
				// will preserve the root component created by the NewActor's constructor,
				// rather than the one on the old actor:
				Params.bPreserveRootComponent = (OldComponent->GetOwner() != OldActor);
			}
		}
	}

	UEngine::CopyPropertiesForUnrelatedObjects(OldObject, NewObject, Params);
}

void FBlueprintCompileReinstancer::PreCreateSubObjectsForReinstantiation(
	const TMap<UClass*, UClass*>& OldToNewClassMap, 
	UObject* OldObject, UObject* NewUObject, 
	TMap<UObject*, UObject*>& CreatedInstanceMap, 
	const TMap<UObject*, UObject*>* OldToNewInstanceMap/* = nullptr*/, 
	TArray< TTuple<UObject*, UObject*>>* OrderedListOfObjectToCopy /*= nullptr*/,
	const TSet<UObject*>* InstancesThatShouldUseOldClass)
{
	// When handling only referenced owned subobjects, we absolutely need to have done the postload so that we are pointing 
	// to all the correct loaded subobjects, otherwise we are skip important objects. Ex: Added subobjects in containers during the load.
	// Scoped only for OS object for now as it is the only case so far that has problem with this filtering.
	TSet<UObject*> OldInstancedSubObjects;
	TSet<UObject*>* OldInstancedSubObjectsPtr = nullptr;
	if (!FOverridableManager::Get().IsEnabled(OldObject) || !OldObject->HasAnyFlags(RF_NeedPostLoad))
	{
		OldInstancedSubObjectsPtr = &OldInstancedSubObjects;
		UE::Private::FReplaceReferenceHelper::GetOwnedSubobjectsRecursive(OldObject, OldInstancedSubObjects);
	}

	// Add the mapping from the old to the new object exists...
	CreatedInstanceMap.Add(OldObject, NewUObject);
	PreCreateSubObjectsForReinstantiation_Inner(OldInstancedSubObjectsPtr, OldToNewClassMap, OldObject, NewUObject, CreatedInstanceMap, OldToNewInstanceMap, OrderedListOfObjectToCopy, InstancesThatShouldUseOldClass);
	if (OrderedListOfObjectToCopy)
	{
		// Post add for deep first order
		OrderedListOfObjectToCopy->Add({OldObject, NewUObject});
	}
}

bool FBlueprintCompileReinstancer::PrePopulateInstancingGraphForArchetype(TMap<UObject*, FObjectInstancingGraph>& GraphCache, FObjectInstancingGraph& InstancingGraph, TNotNull<UObject*> OldInstance, UObject* InArchetype, TMap<UObject*, UObject*>* OldToNewInstanceMap)
{
	checkf(!InstancingGraph.HasDestinationRoot(), TEXT("Can only pre-populate an empty InstancingGraph"));

	// Only needed for objects that are using OS for now
	if (!FOverridableManager::Get().IsEnabled(OldInstance))
	{
		return false;
	}

	// Find the root of the instantiation
	UObject* InstancingGraphRoot = OldInstance;
	do
	{
		if (UObject** NewInstancingGraphRoot = OldToNewInstanceMap ? OldToNewInstanceMap->Find(InstancingGraphRoot) : nullptr)
		{
			InstancingGraphRoot = *NewInstancingGraphRoot;
		}
		InstancingGraphRoot = InstancingGraphRoot->GetOuter();
	} while (InstancingGraphRoot && !InstancingGraphRoot->GetArchetype()->HasAnyFlags(RF_ClassDefaultObject));

	// Sanitization of the found instancing root
	if (!InstancingGraphRoot || Cast<UPackage>(InstancingGraphRoot) != nullptr)
	{
		UE_LOGF(LogClass, Warning, "Object %ls was created from an archetype we could not find the root of the instancing graph.", *GetPathNameSafe(OldInstance));
		return false;
	}

	if (GUseReinstancingGraphCache)
	{
		// Look for an existing graph
		FObjectInstancingGraph* SourceGraph = GraphCache.Find(InstancingGraphRoot);

		if (!SourceGraph)
		{
			SourceGraph = &GraphCache.Add(InstancingGraphRoot);
		
			// Compute the graph for a specific top level object
			auto GatherMatchingSubObjects = [SourceGraph, InstancingGraphRoot, OldToNewInstanceMap](UObject* Instance, UObject* Template, auto GatherMatchingSubObjectsRecurse) -> void
			{
				SourceGraph->PrepopulateWithInstance(Instance, Template);

				ForEachObjectWithOuter(Instance, [Template, InstancingGraphRoot, GatherMatchingSubObjectsRecurse, OldToNewInstanceMap](UObject* SubObject)
				{
					// Just handle subobjects that are created from archetype, skip the others
					UObject* SubObjectArchetype = SubObject->GetArchetype();
					if (SubObjectArchetype->HasAnyFlags(RF_ClassDefaultObject))
					{
						return;
					}

					// Sanitization of the archetype
					if (SubObjectArchetype->GetOuter() != Template)
					{
						// Handle the fact that subobject will be re-instantiated later by pointing its archetype to the new one.
						if (UObject** NewSubObjectArchetype = OldToNewInstanceMap ? OldToNewInstanceMap->Find(SubObjectArchetype) : nullptr)
						{
							SubObjectArchetype = *NewSubObjectArchetype;
						}
						else
						{
							UE_LOGF(LogClass, Warning, "Missmatched while trying to rebuild instancing graph for '%ls', expecting the outer of archetype %ls to be %ls but was %ls", 
								*GetPathNameSafe(InstancingGraphRoot), *GetPathNameSafe(SubObjectArchetype), *GetPathNameSafe(Template), *GetPathNameSafe(SubObjectArchetype->GetOuter()));
							return;
						}
					}

					GatherMatchingSubObjectsRecurse(SubObject, SubObjectArchetype, GatherMatchingSubObjectsRecurse);

				}, EGetObjectsFlags::None);
			};

			GatherMatchingSubObjects(InstancingGraphRoot, InstancingGraphRoot->GetArchetype(), GatherMatchingSubObjects);
		}

		// Copy from existing graph
		InstancingGraph = *SourceGraph;

		// Remove this instance and subobjects from graph to make sure they get properly reinstanced
		InstancingGraph.RemoveInstanceFromPrepopulate(OldInstance);

		ForEachObjectWithOuter(OldInstance, [&InstancingGraph](UObject* SubObject)
		{
			InstancingGraph.RemoveInstanceFromPrepopulate(SubObject);
		}, EGetObjectsFlags::IncludeNestedObjects);
	}
	else
	{
		// Old path without caching between calls, remove after verification
		auto GatherMatchingSubObjects = [&InstancingGraph, OldInstance, OldToNewInstanceMap](UObject* Instance, UObject* Template, auto GatherMatchingSubObjectsRecurse) -> void
		{
			InstancingGraph.PrepopulateWithInstance(Instance, Template);

			ForEachObjectWithOuter(Instance, [OldInstance, Template, GatherMatchingSubObjectsRecurse, OldToNewInstanceMap](UObject* SubObject)
			{
				// Skip the object that is about to be re-instantiated
				if (SubObject == OldInstance)
				{
					return;
				}

				// Just handle subobject that are created from archetype, skip the others
				UObject* SubObjectArchetype = SubObject->GetArchetype();
				if (SubObjectArchetype->HasAnyFlags(RF_ClassDefaultObject))
				{
					return;
				}

				// Sanitization of the archetype
				if (SubObjectArchetype->GetOuter() != Template)
				{
					// Handle the fact that subobject will be re-instantiated later by pointing its archetype to the new one.
					if (UObject** NewSubObjectArchetype = OldToNewInstanceMap ? OldToNewInstanceMap->Find(SubObjectArchetype) : nullptr)
					{
						SubObjectArchetype = *NewSubObjectArchetype;
					}
					else
					{
						UE_LOGF(LogClass, VeryVerbose, "Missmatched while trying to rebuild instancing graph for '%ls', expecting the outer of archetype %ls to be %ls but was %ls", 
							*GetPathNameSafe(OldInstance), *GetPathNameSafe(SubObjectArchetype), *GetPathNameSafe(Template), *GetPathNameSafe(SubObjectArchetype->GetOuter()));
						return;
					}
				}

				GatherMatchingSubObjectsRecurse(SubObject, SubObjectArchetype, GatherMatchingSubObjectsRecurse);

			}, EGetObjectsFlags::None);
		};

		GatherMatchingSubObjects(InstancingGraphRoot, InstancingGraphRoot->GetArchetype(), GatherMatchingSubObjects );
	}

	return true;
}

void FBlueprintCompileReinstancer::PreCreateSubObjectsForReinstantiation_Inner(
	const TSet<UObject*>* OldInstancedSubObjects, 
	const TMap<UClass*, UClass*>& OldToNewClassMap, 
	UObject* OldObject, UObject* NewUObject, 
	TMap<UObject*, UObject*>& CreatedInstanceMap, 
	const TMap<UObject*, UObject*>* OldToNewInstanceMap,
	TArray< TTuple<UObject*, UObject*>>* OrderedListOfObjectToCopy,
	const TSet<UObject*>* InstancesThatShouldUseOldClass)
{
	// Gather subobjects on old object and pre-create them if needed
	TArray<UObject*> ContainedOldSubObjects;
	GetObjectsWithOuter(OldObject, ContainedOldSubObjects, EGetObjectsFlags::None);

	// Gather subobjects on old object and pre-create them if needed
	TArray<UObject*> ContainedNewSubObjects;
	GetObjectsWithOuter(NewUObject, ContainedNewSubObjects, EGetObjectsFlags::None);

	// Pre-create all non default subobjects to prevent re-instancing them as an old classes 
	for (int32 i = 0; i < ContainedOldSubObjects.Num(); ++i)
	{
		UObject* OldSubObject = ContainedOldSubObjects[i];

		// Filter out SubObjects that are not referenced as instanced
		if(OldInstancedSubObjects && !OldInstancedSubObjects->Contains(OldSubObject))
		{
			continue;
		}

		UClass* OldSubObjectClass = OldSubObject->GetClass();
		UObject* OldSubObjectOuter = OldSubObject->GetOuter();
		FName SubObjectName = OldSubObject->GetFName();
		EObjectFlags SubObjectFlags = OldSubObject->GetFlags() & RF_Reinst;
		UPackage* OldExternalPackage = OldSubObject->GetExternalPackage();

		// Only re-create objects that are created from a default object,
		// skip any default subobjects are they will be created by their parent object CDO.
		UObject* Archetype = UObject::GetArchetypeFromRequiredInfo(OldSubObjectClass, OldSubObjectOuter, SubObjectName, SubObjectFlags);
		if (Archetype->HasAnyFlags(RF_ClassDefaultObject))
		{
			// Was it already re-instantiated
			if (UObject* const* AlreadyCreatedSubObject = OldToNewInstanceMap ? OldToNewInstanceMap->Find(OldSubObject) : nullptr)
			{
				int32 AlreadyCreatedSubObjectIndex = ContainedOldSubObjects.Find(*AlreadyCreatedSubObject);
				if (AlreadyCreatedSubObjectIndex > i)
				{
					ContainedOldSubObjects.RemoveAt(AlreadyCreatedSubObjectIndex);
				}
				else
				{
					// This happens on object that were reinstantiated and were not yet outered to the right object because it was not yet created
					checkf(OldInstancedSubObjects && !OldInstancedSubObjects->Contains(*AlreadyCreatedSubObject), TEXT("The object must have already been created but should have been skipped by the fact that the old object was not referencing it"));
				}
				(*AlreadyCreatedSubObject)->Rename(nullptr, NewUObject, REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional | REN_AllowPackageLinkerMismatch);
			}
			// Could still be already re-instantiated but be also replaced
			else if (UObject*const* AlreadyReplacedSubObject = OldToNewInstanceMap && OldToNewInstanceMap->FindKey(OldSubObject) ? &OldSubObject : nullptr)
			{
				(*AlreadyReplacedSubObject)->Rename(nullptr, NewUObject, REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional | REN_AllowPackageLinkerMismatch);
			}
			else
			{
				// No need to handled invalid objects from this point on
				if (!IsValid(OldSubObject))
				{
					continue;
				}

				UClass* SubObjectClass = OldSubObjectClass;
				if (UClass* const* NewSubObjectClass = (!InstancesThatShouldUseOldClass || !InstancesThatShouldUseOldClass->Contains(OldSubObject)) ? OldToNewClassMap.Find(OldSubObjectClass) : nullptr)
				{
					SubObjectClass = *NewSubObjectClass;
				}
				else if (OldSubObjectClass->HasAnyClassFlags(CLASS_NewerVersionExists))
				{
					// The old subobject class is a compiler artifact, with no reinstanced counterpart. Its source may have been renamed (or deleted) and thus it could not be recompiled.
					// In this case, if property bag features are enabled, we attempt to redirect to a placeholder type so we can serialize to a property bag in order to preserve its data.
					if (UE::FPropertyBagRepository::IsPropertyBagPlaceholderObjectFeatureEnabled(UE::FPropertyBagRepository::EPlaceholderObjectFeature::ReplaceMissingReinstancedTypes))
					{
						FTopLevelAssetPath OldSubObjectClassPath = OldSubObjectClass->GetReinstancedClassPathName();
						if (OldSubObjectClassPath.IsValid())
						{
							if (UPackage* OldSubObjectClassPackage = FindPackage(nullptr, *OldSubObjectClassPath.GetPackageName().ToString()))
							{
								SubObjectClass = UE::FPropertyBagRepository::CreatePropertyBagPlaceholderClass(OldSubObjectClassPackage, OldSubObjectClass->GetClass(), OldSubObjectClassPath.GetAssetName(), OldSubObjectClass->GetFlags() & ~RF_Transient);
							}
						}
					}
				}

				// Only pre-create object where the class does not have newer version of the it
				if(!SubObjectClass->HasAnyClassFlags(CLASS_NewerVersionExists))
				{
					UObject* NewSubObject = nullptr;
					if (UObject** ExistingNewSubObject = ContainedNewSubObjects.FindByPredicate([SubObjectName](UObject* SubObject) { return SubObject && SubObject->GetFName() == SubObjectName; }))
					{
						NewSubObject = *ExistingNewSubObject;
					}
					else
					{
						// Making sure the template we are about to use had all its references patched up
						UObject* Template = SubObjectClass->GetDefaultObject();
						if (OldToNewInstanceMap)
						{
							FArchiveReplaceObjectRef<UObject> ReplaceInCDOAr(Template, *OldToNewInstanceMap);
						}

						NewSubObject = NewObject<UObject>(NewUObject, SubObjectClass, SubObjectName, SubObjectFlags, Template);
					}

					// Transfer external package state if necessary 
					if (OldExternalPackage)
					{
						NewSubObject->SetExternalPackage(OldExternalPackage); // place new subobject in the pre-existing external package 
						OldSubObject->SetExternalPackage(nullptr); // move old subobject outside of external package (it'll be in the transient package) so that it's doesn't cause the package to stick around
					}

					CreatedInstanceMap.Add(OldSubObject, NewSubObject);
					PreCreateSubObjectsForReinstantiation_Inner(OldInstancedSubObjects, OldToNewClassMap, OldSubObject, NewSubObject, CreatedInstanceMap, OldToNewInstanceMap, OrderedListOfObjectToCopy, InstancesThatShouldUseOldClass);
					if (OrderedListOfObjectToCopy)
					{
						// Post add for deep first order
						OrderedListOfObjectToCopy->Add({OldSubObject, NewSubObject});
					}
				}
			}
		}
		// There might be new subobjects attached to the sub object that are particular to this instance, let's traverse it to find them out.
		else if (UObject** NewSubObject = ContainedNewSubObjects.FindByPredicate([SubObjectName](UObject* SubObject) { return SubObject && SubObject->GetFName() == SubObjectName; }))
		{
			// Transfer external package state if necessary 
			if (OldExternalPackage)
			{
				(*NewSubObject)->SetExternalPackage(OldExternalPackage); // place new suboject in the pre-existing external package 
				OldSubObject->SetExternalPackage(nullptr); // move old subobject outside of external package (it'll be in the transient package) so that it's doesn't cause the package to stick around
			}

			PreCreateSubObjectsForReinstantiation_Inner(OldInstancedSubObjects, OldToNewClassMap, OldSubObject, *NewSubObject, CreatedInstanceMap, OldToNewInstanceMap, OrderedListOfObjectToCopy, InstancesThatShouldUseOldClass);
		}
	}
}

void FBlueprintCompileReinstancer::CacheArchetypes(
	const TArray<UObject*>& ObjectsToReplace, 
	FEditorCacheArchetypeManager& CacheManager, 
	TSet<UObject*>& CachedArchetypeObjects)
{
	// We need to cache the archetype of the objects about to be replaced 
	// as it will not be possible to get them during this process as it renames the objects
	// These cached archetypes will not be updated if they were set earlier
	for (UObject* OldObject : ObjectsToReplace)
	{
		if(!IsValid(OldObject))
		{
			continue;
		}

		CacheManager.CacheArchetype(OldObject);
		CachedArchetypeObjects.Add(OldObject);

		ForEachObjectWithOuter(OldObject, [&CacheManager, &CachedArchetypeObjects](UObject* SubObject)
		{
			CacheManager.CacheArchetype(SubObject);
			CachedArchetypeObjects.Add(SubObject);
		});
	}
}

FBlake3Hash FBlueprintCompileReinstancer::CalculateConstructionHashIfNecessary(const UClass* ForClass)
{
	TArray<UObject*> Count;
	GetObjectsToReinstance(ForClass, Count);
	if(Count.Num() == 0)
	{
		return FBlake3Hash();
	}

	return CalculateConstructionHash(ForClass);
}

FBlake3Hash FBlueprintCompileReinstancer::CalculateConstructionHash(const UClass* ForClass)
{
	check(ForClass && ForClass->GetDefaultObject(false));

	// must capture all template data - just grab the subobjects of the class and exclude the ufunctions.
	// UFunctions should be excluded as on load of an editor asset they will not include bytecode, but
	// after compile they will so if we include them in the class version we'll always have to reinstance. 
	// This can be impactful as we're frequently 'noop' reinstnacing level script actors.
	TArray<UObject*> Objects;
	GetObjectsWithOuter(ForClass, Objects, EGetObjectsFlags::None, RF_Transient, EInternalObjectFlags::Garbage );
	Objects.SetNum(Algo::RemoveIf(Objects, [](const UObject* Obj)
	{
		return Obj->IsA<UFunction>();
	}));
	Objects.Sort([](const UObject& A, const UObject& B) { return A.GetFName().LexicalLess(B.GetFName()); });
	Objects.Add(ForClass->GetDefaultObject(false));
	
	FUtf8String TemplateData = UE::JsonObjectGraph::Stringify(
		Objects,
		FJsonStringifyOptions(EJsonStringifyFlags::FilterEditorOnlyData));

	FBlake3 Hash;
	Hash.Update(TemplateData.GetCharArray().GetData(), TemplateData.GetCharArray().Num());
	return Hash.Finalize();
}

void FBlueprintCompileReinstancer::ReinstanceWithVersioningData(const FReinstancingState& ReinstancingState, const TSet<UObject*>* InstancesThatShouldUseOldClass)
{
	// if an OldToNewTemplates map has been provided, include all the instances in the replacement map
	// this is used to make reinstancing during PIE more robust:
	if (ReinstancingState.OldToNewTemplates)
	{
		for (const TPair<UClass*, TMap<UObject*, UObject*>>& Mapping : *ReinstancingState.OldToNewTemplates)
		{
			ReinstancingState.OldToNewInstanceMap.Append(Mapping.Value);
		}
	}
	// Include a mapping from the old class to the new class as well - this will
	// prevent mutation of the old classes by reference replacement, and aslo patch
	// leaks to REINST classes:
	for (const TPair<UClass*, UClass*>& Mapping : ReinstancingState.InOldToNewClassMap)
	{
		ReinstancingState.OldToNewInstanceMap.Add(Mapping.Key, Mapping.Value);
	}

	TMap<UClass*, UClass*> UnchangedClasses;
	TMap<UClass*, UClass*> NormalReinstancingClasses;
	TMap<UClass*, UClass*> ActorReinstancingClasses;
	CategorizeClassesAndInstances(
		ReinstancingState.InOldToNewClassMap, 
		UnchangedClasses, 
		NormalReinstancingClasses, 
		ActorReinstancingClasses, 
		ReinstancingState);
	RestoreClass(UnchangedClasses, ReinstancingState);
	ReinstanceNormalObjects(NormalReinstancingClasses, ReinstancingState, InstancesThatShouldUseOldClass);
	ReinstanceActors(ActorReinstancingClasses, ReinstancingState, InstancesThatShouldUseOldClass);
}

void FBlueprintCompileReinstancer::CategorizeClassesAndInstances(
	const TMap<UClass*, UClass*>& InOldToNewClassMap
	, TMap<UClass*, UClass*>& UnchangedClasses
	, TMap<UClass*, UClass*>& NormalReinstancingClasses
	, TMap<UClass*, UClass*>& ActorReinstancingClasses
	, const FBlueprintCompileReinstancer::FReinstancingState& ReinstancingState)
{
	for (TPair<UClass*, UClass*> OldToNewClass : InOldToNewClassMap)
	{
		const UClass* const OldClass = OldToNewClass.Key;
		const UClass* const NewClass = OldToNewClass.Value;

		const bool bIsComponent = NewClass->IsChildOf<UActorComponent>();
		// If any of the class changes are of an actor component to scene component or reverse then we will fixup SCS of all actors affected
		if (bIsComponent && !ReinstancingState.bFixupSCS)
		{
			ReinstancingState.bFixupSCS = (NewClass->IsChildOf<USceneComponent>() != OldClass->IsChildOf<USceneComponent>());
		}

		// if there are no instances, nothing to do:
		TArray<UObject*>& ObjectsToReplace = ReinstancingState.ObjectsToReplaceScratch;
		GetObjectsToReinstance(OldClass, ObjectsToReplace);
		if(ObjectsToReplace.Num() == 0)
		{
			continue; // unambiguously no reinstancing necessary
		}
		
		// if layout and default values match, we can reinstance fast:
		bool bClassMatchesOldClass = false;
		bool bUseActorReinstancing = false;
		// if we have a script constructor we would have to check it for equivalence here
		// but there are many obstacles to script constructors:
		const FBlake3Hash* ConstructionVersion = ReinstancingState.ConstructionVersioningData ? ReinstancingState.ConstructionVersioningData->Find(NewClass) : nullptr;
		// anim blueprints have uncontrolled backpointers to CDO data and therefore instances must
		// be reconstructed after compilation:
		if( ConstructionVersion && 
			FStructUtils::TheSameLayout(OldClass, NewClass) &&
			CalculateConstructionHash(NewClass) == *ConstructionVersion)
		{
			bClassMatchesOldClass = true;

			// we can only do fast reinstancing when ownership of objects is clear,
			// otherwise disable it - this avoids n^2 archetype lookups for each 
			// instance, which may be still profitable but not much.. we
			// still have to pay for deep hierarchies here, which is not ideal
			// but is only O(n) and hierarchy depth is never going to be >1000
			// at some point we'll need a flag to identify notionally free
			// objects (e.g. not created by a ctor, native or otherwise) - could
			// rely on some kind of 'is editable' flag but we're fighting with perspectives
			// there, where editability is associated with a property and has no
			// tracking on the object...:
			const auto OuteredToRegeneratedObject = [](const UObject* Obj)
			{
				UObject* OuterIter = Obj->GetOuter();
				while(OuterIter)
				{
					if(OuterIter->GetClass()->HasAnyClassFlags(CLASS_NewerVersionExists))
					{
						return true;
					}
					OuterIter = OuterIter->GetOuter();
				}
				return false;
			};
			for(UObject* Object : ObjectsToReplace)
			{
				if(OuteredToRegeneratedObject(Object))
				{
					bClassMatchesOldClass = false;
					break;
				}
			}
		}
		// actors require late reinstancing, but if reinstancing fast
		// they should be ok, as finalization happens at the same time
		if(NewClass->IsChildOf(AActor::StaticClass()))
		{
			bUseActorReinstancing = true;
		}

		if(bClassMatchesOldClass)
		{
			UnchangedClasses.Add(OldToNewClass.Key, OldToNewClass.Value);
		}
		else if(bUseActorReinstancing)
		{
			ActorReinstancingClasses.Add(OldToNewClass.Key, OldToNewClass.Value);
		}
		else
		{
			NormalReinstancingClasses.Add(OldToNewClass.Key, OldToNewClass.Value);
		}

	}
}

void FBlueprintCompileReinstancer::RestoreClass(
	TMap<UClass*, UClass*>& OldToNewClassMap, 
	const FBlueprintCompileReinstancer::FReinstancingState& ReinstancingState)
{
	// update classes
	for (TPair<UClass*, UClass*> OldToNewClass : OldToNewClassMap)
	{
		const UClass* const OldClass = OldToNewClass.Key;
		UClass* const NewClass = OldToNewClass.Value;
		TArray<UObject*>& ObjectsToReplace = ReinstancingState.ObjectsToReplaceScratch;
		GetObjectsToReinstance(OldClass, ObjectsToReplace);

		const UBlueprintGeneratedClass* BPGC = Cast<UBlueprintGeneratedClass>(NewClass);

		for (UObject* Object : ObjectsToReplace)
		{
			Object->SetClass(NewClass); // class layouts match, so we can just set the object back to its original class
			
			// run reference replacement on the object, in case it has stored references to regenerated archetype data:
			UE::Private::FReferenceReplace ReplaceAr(
				Object, 
				ReinstancingState.OldToNewInstanceMap,
				ReinstancingState.ReinstancedObjectsWeakReferenceMap);
			BPGC->ReinitializeObjectAfterCompile(Object);
		}
	}
}

void FBlueprintCompileReinstancer::ReinstanceNormalObjects(
	TMap<UClass*, UClass*>& OldToNewClassMap,
	const FBlueprintCompileReinstancer::FReinstancingState& ReinstancingState,
	const TSet<UObject*>* InstancesThatShouldUseOldClass)
{
	FEditorCacheArchetypeManager& CacheManager = FEditorCacheArchetypeManager::Get();
	TArray<UObject*> EmptyList;
	const int32 OldObjIndex = INDEX_NONE; // not needed as archetypes are up to date in this codepath
	const bool bArchetypesAreUpToDate = true;

	for (TPair<UClass*, UClass*> OldToNewClass : OldToNewClassMap)
	{
		UClass* const OldClass = OldToNewClass.Key;
		UClass* const NewClass = OldToNewClass.Value;
		
		const bool bIsComponent = NewClass->IsChildOf<UActorComponent>();

		TArray<UObject*>& ObjectsToReplace = ReinstancingState.ObjectsToReplaceScratch;
		GetObjectsToReinstance(OldClass, ObjectsToReplace);
		
		CacheArchetypes(ObjectsToReplace, CacheManager, ReinstancingState.CachedArchetypeObjects);

		for (UObject* OldObject : ObjectsToReplace)
		{
			check(IsValid(OldObject)); // is RF_MirroredGarbage in GetObjectsToReinstance not enough?
			if(!ShouldReinstanceObject(OldObject, ReinstancingState.InOldToNewClassMap, bIsComponent))
			{
				continue;
			}

			CheckAndSaveOuterPackageToCleanList(ReinstancingState.CleanPackageList, OldObject);
			UObject* NewUObject = nullptr;
			ReplaceObjectHelper(
				ReinstancingState,
				OldObject, 
				OldClass, 
				NewUObject, 
				NewClass, 
				OldObjIndex, 
				EmptyList, 
				&FDirectAttachChildrenAccessor::Get, 
				bIsComponent, 
				bArchetypesAreUpToDate,
				InstancesThatShouldUseOldClass);
			UpdateObjectBeingDebugged(OldObject, NewUObject);
			ReinstancingState.ObjectsReplaced.Add(OldObject);
		}
	}
}

void FBlueprintCompileReinstancer::ReinstanceActors(
	TMap<UClass*, UClass*>& OldToNewClassMap, 
	const FBlueprintCompileReinstancer::FReinstancingState& ReinstancingState,
	const TSet<UObject*>* InstancesThatShouldUseOldClass)
{
	TArray<UObject*> EmptyList;
	const int32 OldObjIndex = INDEX_NONE; // not needed as archetypes are up to date in this codepath
	const bool bArchetypesAreUpToDate = true;
	const bool bIsComponent = false;
	const bool bPreserveRootComponent = true;

	for (TPair<UClass*, UClass*> OldToNewClass : OldToNewClassMap)
	{
		UClass* OldClass = OldToNewClass.Key;
		UClass* NewClass = OldToNewClass.Value;
		check(OldClass && NewClass);
		TArray<UObject*>& ObjectsToReplace = ReinstancingState.ObjectsToReplaceScratch;
		GetObjectsToReinstance(OldClass, ObjectsToReplace);

		// store old attachment data before we mess with components, etc:
		TMap<UObject*, FActorAttachmentData> ActorAttachmentData;
		for (UObject* OldObject : ObjectsToReplace)
		{
			if (AActor* OldActor = Cast<AActor>(OldObject))
			{
				ActorAttachmentData.Add(OldObject, FActorAttachmentData(OldActor));
			}
		}
		
		for (UObject* OldObject : ObjectsToReplace)
		{
			check(IsValid(OldObject)); // is RF_MirroredGarbage in GetObjectsToReinstance not enough?
			AActor* OldActor = CastChecked<AActor>(OldObject);
			if(!ShouldReinstanceActor(OldActor))
			{
				continue;
			}

			CheckAndSaveOuterPackageToCleanList(ReinstancingState.CleanPackageList, OldObject);

			UObject* NewUObject = nullptr;
			if (OldActor->GetLevel() && OldActor->GetWorld() && !OldActor->GetWorld()->IsCleanedUp())
			{
				CheckAndSaveAttachedActorsOuterPackageToCleanList(ReinstancingState.CleanPackageList, OldActor);
				ReplaceActorHelper(
					OldActor, 
					OldClass, 
					NewUObject, 
					NewClass, 
					ReinstancingState.OldToNewInstanceMap, 
					ReinstancingState.InOldToNewClassMap, 
					ReinstancingState.ReinstancedObjectsWeakReferenceMap, 
					ActorAttachmentData, 
					ReinstancingState.ReplacementActors, 
					bPreserveRootComponent, 
					ReinstancingState.bSelectionChanged);
			}
			else
			{
				// Actors that are not in a level cannot be reconstructed, reinstance them as normal objects:
				ReplaceObjectHelper(
					ReinstancingState,
					OldObject, 
					OldClass,
					NewUObject,
					NewClass,
					OldObjIndex,
					EmptyList,
					&FDirectAttachChildrenAccessor::Get,
					bIsComponent,
					bArchetypesAreUpToDate,
					InstancesThatShouldUseOldClass);
			}
			UpdateObjectBeingDebugged(OldObject, NewUObject);
			ReinstancingState.ObjectsReplaced.Add(OldObject);
		}
	}
}

void FBlueprintCompileReinstancer::GetObjectsToReinstance(const UClass* OldClass, TArray<UObject*>& ObjectsToReplace)
{
	ObjectsToReplace.Reset();
	const bool bIncludeDerivedClasses = false;
	GetObjectsOfClass(OldClass, ObjectsToReplace, bIncludeDerivedClasses, RF_ClassDefaultObject | RF_MirroredGarbage);
}

void FBlueprintCompileReinstancer::UpdateObjectBeingDebugged(UObject* InOldObject, UObject* InNewObject)
{
	if (InOldObject == nullptr)
	{
		return;
	}

	if (UBlueprint* OldObjBlueprint = Cast<UBlueprint>(InOldObject->GetClass()->ClassGeneratedBy))
	{
		// For now, don't update the object if the outer BP assets don't match (e.g. after a reload). Otherwise, it will
		// trigger an ensure() in SetObjectBeginDebugged(). This will be replaced with a better solution in a future release.
		if (InNewObject == nullptr || OldObjBlueprint == Cast<UBlueprint>(InNewObject->GetClass()->ClassGeneratedBy))
		{
			// The old object may already be PendingKill, but we still want to check the current
			// ptr value for a match. Otherwise, the selection will get cleared after every compile.
			const UObject* DebugObj = OldObjBlueprint->GetObjectBeingDebugged(EGetObjectOrWorldBeingDebuggedFlags::IgnorePendingKill);
			if (DebugObj == InOldObject)
			{
				OldObjBlueprint->SetObjectBeingDebugged(InNewObject);
			}
		}
	}
}

void FBlueprintCompileReinstancer::CheckAndSaveOuterPackageToCleanList(TSet<UPackage*>& CleanPackageList, const UObject* InObject)
{
	check(InObject);
	UPackage* ObjectPackage = InObject->GetPackage();
	if (ObjectPackage && !ObjectPackage->IsDirty() && ObjectPackage != GetTransientPackage())
	{
		CleanPackageList.Add(ObjectPackage);
	}
}

void FBlueprintCompileReinstancer::CheckAndSaveAttachedActorsOuterPackageToCleanList(TSet<UPackage*>& CleanPackageList, const AActor* Actor)
{
	// Attached actors will be marked dirty as well when the old instance is destroyed, but
	// since these attachments will get restored as part of replacement, there's no need to
	// re-save after reinstancing. In general, this applies only to OFPA-enabled levels, as
	// in that case, attached actors reside in their own external packages and not the level.
	TArray<AActor*> AttachedActors;
	Actor->GetAttachedActors(AttachedActors);
	for (AActor* AttachedActor : AttachedActors)
	{
		if (AttachedActor)
		{
			CheckAndSaveOuterPackageToCleanList(CleanPackageList, AttachedActor);
		}
	}
}

bool FBlueprintCompileReinstancer::ShouldReinstanceObject(
	const UObject* OldObject, 
	const TMap<UClass*, UClass*>& InOldToNewClassMap, 
	const bool bIsComponent)
{
	// Skipping any default sub object that outer is going to be replaced
	// This isn't needed for the actor loop as the only outer for actor is a level
	if ((OldObject->IsDefaultSubobject() || OldObject->HasAnyFlags(RF_DefaultSubObject)) && 
		InOldToNewClassMap.Contains(OldObject->GetOuter()->GetClass()))
	{
		return false;
	}

	// Skip archetype instances, EXCEPT for component templates - this component
	// check is likely not needed, but leaving it for now:
	if ( !bIsComponent && OldObject->IsTemplate() )
	{
		return false;
	}

	return true;
}

bool FBlueprintCompileReinstancer::ShouldReinstanceActor(const AActor* OldActor)
{
	// Skip archetype instances, EXCEPT for child actor templates
	const bool bIsChildActorTemplate = OldActor->GetOuter()->IsA<UChildActorComponent>();
	if (!bIsChildActorTemplate && OldActor->IsTemplate())
	{
		return false;
	}
	return true;
}
