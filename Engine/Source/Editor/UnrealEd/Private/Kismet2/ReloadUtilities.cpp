// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ReloadUtilities.cpp: Helpers for reloading
=============================================================================*/

#include "Kismet2/ReloadUtilities.h"
#include "AssetCompilingManager.h"
#include "Async/AsyncWork.h"
#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/KismetReinstanceUtilities.h"
#include "Misc/QueuedThreadPool.h"
#include "Serialization/ArchiveReplaceObjectRef.h"
#include "UObject/Class.h"
#include "UObject/UObjectIterator.h"
#include "Misc/StringBuilder.h"

namespace UE::Reload::Private
{
	/** Holds a property and its offset in the serialized properties data array */
	struct FCDOProperty
	{
		FCDOProperty()
			: Property(nullptr)
			, SubobjectName(NAME_None)
			, SerializedValueOffset(0)
			, SerializedValueSize(0)
		{}

		FProperty* Property;
		FName SubobjectName;
		int64 SerializedValueOffset;
		int64 SerializedValueSize;
	};

	/** Contains all serialized CDO property data and the map of all serialized properties */
	struct FCDOPropertyData
	{
		TArray<uint8> Bytes;
		TMap<FName, FCDOProperty> Properties;
	};

	/**
	 * Helper class used for re-instancing native and blueprint classes after hot-reload
	 */
	class FReloadClassReinstancer : public FBlueprintCompileReinstancer
	{

		/** Hot-reloaded version of the old class */
		UClass* NewClass;

		/** Necessary for delta serialization */
		TObjectPtr<UObject> CopyOfPreviousCDO;

		/**
		 * Sets the re-instancer up for new class re-instancing
		 *
		 * @param InNewClass Class that has changed after hot-reload
		 * @param InOldClass Class before it was hot-reloaded
		 */
		void SetupNewClassReinstancing(UClass* InNewClass, UClass* InOldClass);

		/**
		* Sets the re-instancer up for old class re-instancing. Always re-creates the CDO.
		*
		* @param InOldClass Class that has NOT changed after hot-reload
		*/
		void RecreateCDOAndSetupOldClassReinstancing(UClass* InOldClass);

	public:

		/** Sets the re-instancer up to re-instance native classes */
		FReloadClassReinstancer(UClass* InNewClass, UClass* InOldClass, UObject* InOldClassCDO,
			const TSet<UObject*>& InReinstancingObjects, TSet<UBlueprint*>& InCompiledBlueprints);

		/** Destructor */
		virtual ~FReloadClassReinstancer();

		// FSerializableObject interface
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		// End of FSerializableObject interface

		virtual bool IsClassObjectReplaced() const override { return true; }

		virtual void BlueprintWasRecompiled(UBlueprint* BP, bool bBytecodeOnly) override;

	protected:

		// FBlueprintCompileReinstancer interface
		virtual bool ShouldPreserveRootComponentOfReinstancedActor() const override { return false; }
		// End of FBlueprintCompileReinstancer interface

	private:
		/** Collection of blueprints already recompiled */
		TSet<UBlueprint*>& CompiledBlueprints;
	};

	FReloadClassReinstancer::FReloadClassReinstancer(UClass* InNewClass, UClass* InOldClass, UObject* InOldClassCDO, const TSet<UObject*>& InReinstancingObjects, TSet<UBlueprint*>& InCompiledBlueprints)
		: NewClass(nullptr)
		, CopyOfPreviousCDO(nullptr)
		, CompiledBlueprints(InCompiledBlueprints)
	{
		ensure(InOldClass);
		ensure(!HotReloadedOldClass && !HotReloadedNewClass);
		HotReloadedOldClass = InOldClass;
		HotReloadedNewClass = InNewClass ? InNewClass : InOldClass;
		OriginalCDO = InOldClassCDO;

		for (UObject* Object : InReinstancingObjects)
		{
			ObjectsThatShouldUseOldStuff.Add(Object);
		}

		// If InNewClass is NULL, then the old class has not changed after hot-reload.
		// However, we still need to check for changes to its constructor code (CDO values).
		if (InNewClass)
		{
			SetupNewClassReinstancing(InNewClass, InOldClass);

			TMap<UObject*, UObject*> ClassRedirects;
			ClassRedirects.Add(InOldClass, InNewClass);

			for (TObjectIterator<UBlueprint> BlueprintIt; BlueprintIt; ++BlueprintIt)
			{
				constexpr EArchiveReplaceObjectFlags ReplaceObjectArchFlags = (EArchiveReplaceObjectFlags::IgnoreOuterRef | EArchiveReplaceObjectFlags::IgnoreArchetypeRef);
				FArchiveReplaceObjectRef<UObject> ReplaceObjectArch(*BlueprintIt, ClassRedirects, ReplaceObjectArchFlags);
			}
		}
		else
		{
			RecreateCDOAndSetupOldClassReinstancing(InOldClass);
		}
	}

	FReloadClassReinstancer::~FReloadClassReinstancer()
	{
		// Make sure the base class does not remove the DuplicatedClass from root, we not always want it.
		// For example when we're just reconstructing CDOs. Other cases are handled by HotReloadClassReinstancer.
		DuplicatedClass = nullptr;

		ensure(HotReloadedOldClass);
		HotReloadedOldClass = nullptr;
		HotReloadedNewClass = nullptr;
	}

	void FReloadClassReinstancer::SetupNewClassReinstancing(UClass* InNewClass, UClass* InOldClass)
	{
		// Set base class members to valid values
		ClassToReinstance = InNewClass;
		DuplicatedClass = InOldClass;
		bHasReinstanced = false;
		NewClass = InNewClass;

		SaveClassFieldMapping(InOldClass);

		ObjectsThatShouldUseOldStuff.Add(InOldClass); //CDO of REINST_ class can be used as archetype

		TArray<UClass*> ChildrenOfClass;
		GetDerivedClasses(InOldClass, ChildrenOfClass);
		for (auto ClassIt = ChildrenOfClass.CreateConstIterator(); ClassIt; ++ClassIt)
		{
			UClass* ChildClass = *ClassIt;
			UBlueprint* ChildBP = Cast<UBlueprint>(ChildClass->ClassGeneratedBy);
			if (ChildBP && !ChildBP->HasAnyFlags(RF_BeingRegenerated))
			{
				// If this is a direct child, change the parent and relink so the property chain is valid for reinstancing
				if (!ChildBP->HasAnyFlags(RF_NeedLoad))
				{
					if (ChildClass->GetSuperClass() == InOldClass)
					{
						ReparentChild(ChildBP);
					}

					Children.AddUnique(ChildBP);
					if (ChildBP->ParentClass == InOldClass)
					{
						ChildBP->ParentClass = NewClass;
					}
				}
				else
				{
					// If this is a child that caused the load of their parent, relink to the REINST class so that we can still serialize in the CDO, but do not add to later processing
					ReparentChild(ChildClass);
				}
			}
		}

		// Finally, remove the old class from Root so that it can get GC'd and mark it as CLASS_NewerVersionExists
		InOldClass->RemoveFromRoot();
		InOldClass->ClassFlags |= CLASS_NewerVersionExists;
	}

	void FReloadClassReinstancer::RecreateCDOAndSetupOldClassReinstancing(UClass* InOldClass)
	{
		// Set base class members to valid values
		ClassToReinstance = InOldClass;
		DuplicatedClass = InOldClass;
		bHasReinstanced = false;
		NewClass = InOldClass; // The class doesn't change in this case

		SaveClassFieldMapping(InOldClass);

		TArray<UClass*> ChildrenOfClass;
		GetDerivedClasses(InOldClass, ChildrenOfClass);
		for (auto ClassIt = ChildrenOfClass.CreateConstIterator(); ClassIt; ++ClassIt)
		{
			UClass* ChildClass = *ClassIt;
			UBlueprint* ChildBP = Cast<UBlueprint>(ChildClass->ClassGeneratedBy);
			if (ChildBP && !ChildBP->HasAnyFlags(RF_BeingRegenerated))
			{
				if (!ChildBP->HasAnyFlags(RF_NeedLoad))
				{
					Children.AddUnique(ChildBP);
					UBlueprintGeneratedClass* BPGC = Cast<UBlueprintGeneratedClass>(ChildBP->GeneratedClass);
					UObject* CurrentCDO = BPGC ? BPGC->GetDefaultObject(false) : nullptr;
					if (CurrentCDO && (OriginalCDO == CurrentCDO->GetArchetype()))
					{
						BPGC->OverridenArchetypeForCDO = OriginalCDO;
					}
				}
			}
		}
	}

	void FReloadClassReinstancer::AddReferencedObjects(FReferenceCollector& Collector)
	{
		FBlueprintCompileReinstancer::AddReferencedObjects(Collector);
		Collector.AllowEliminatingReferences(false);
		Collector.AddReferencedObject(CopyOfPreviousCDO);
		Collector.AllowEliminatingReferences(true);
	}

	void FReloadClassReinstancer::BlueprintWasRecompiled(UBlueprint* BP, bool bBytecodeOnly)
	{
		CompiledBlueprints.Add(BP);

		FBlueprintCompileReinstancer::BlueprintWasRecompiled(BP, bBytecodeOnly);
	}

	/**
	* Creates a mem-comparable array of data containing CDO property values.
	*
	* @param InObject CDO
	* @param OutData Data containing all of the CDO property values
	*/
	void SerializeCDOProperties(UObject* InObject, FCDOPropertyData& OutData)
	{
		// Creates a mem-comparable CDO data
		class FCDOWriter : public FMemoryWriter
		{
			/** Objects already visited by this archive */
			TSet<UObject*>& VisitedObjects;
			/** Output property data */
			FCDOPropertyData& PropertyData;
			/** Current subobject being serialized */
			FName SubobjectName;

		public:
			/** Serializes all script properties of the provided DefaultObject */
			FCDOWriter(FCDOPropertyData& InOutData, TSet<UObject*>& InVisitedObjects, FName InSubobjectName)
				: FMemoryWriter(InOutData.Bytes, /* bIsPersistent = */ false, /* bSetOffset = */ true)
				, VisitedObjects(InVisitedObjects)
				, PropertyData(InOutData)
				, SubobjectName(InSubobjectName)
			{
				// Disable delta serialization, we want to serialize everything
				ArNoDelta = true;
			}
			virtual void Serialize(void* Data, int64 Num) override
			{
				// Collect serialized properties so we can later update their values on instances if they change
				FProperty* SerializedProperty = GetSerializedProperty();
				if (SerializedProperty != nullptr)
				{
					FCDOProperty& PropertyInfo = PropertyData.Properties.FindOrAdd(SerializedProperty->GetFName());
					if (PropertyInfo.Property == nullptr)
					{
						PropertyInfo.Property = SerializedProperty;
						PropertyInfo.SubobjectName = SubobjectName;
						PropertyInfo.SerializedValueOffset = Tell();
						PropertyInfo.SerializedValueSize = Num;
					}
					else
					{
						PropertyInfo.SerializedValueSize += Num;
					}
				}
				FMemoryWriter::Serialize(Data, Num);
			}
			/** Serializes an object. Only name and class for normal references, deep serialization for DSOs */
			virtual FArchive& operator<<(class UObject*& InObj) override
			{
				FArchive& Ar = *this;
				if (InObj)
				{
					FName ClassName = InObj->GetClass()->GetFName();
					FName ObjectName = InObj->GetFName();
					Ar << ClassName;
					Ar << ObjectName;
					if (!VisitedObjects.Contains(InObj))
					{
						VisitedObjects.Add(InObj);
						if (Ar.GetSerializedProperty() && Ar.GetSerializedProperty()->ContainsInstancedObjectProperty())
						{
							// Serialize all DSO properties too
							FCDOWriter DefaultSubobjectWriter(PropertyData, VisitedObjects, InObj->GetFName());
							InObj->SerializeScriptProperties(DefaultSubobjectWriter);
							Seek(PropertyData.Bytes.Num());
						}
					}
				}
				else
				{
					FName UnusedName = NAME_None;
					Ar << UnusedName;
					Ar << UnusedName;
				}

				return *this;
			}
			virtual FArchive& operator<<(FObjectPtr& InObj) override
			{
				// Invoke the method above
				return FArchiveUObject::SerializeObjectPtr(*this, InObj);
			}
			/** Serializes an FName as its index and number */
			virtual FArchive& operator<<(FName& InName) override
			{
				FArchive& Ar = *this;
				FNameEntryId ComparisonIndex = InName.GetComparisonIndex();
				FNameEntryId DisplayIndex = InName.GetDisplayIndex();
				int32 Number = InName.GetNumber();
				Ar << ComparisonIndex;
				Ar << DisplayIndex;
				Ar << Number;
				return Ar;
			}
			virtual FArchive& operator<<(FLazyObjectPtr& LazyObjectPtr) override
			{
				FArchive& Ar = *this;
				FUniqueObjectGuid UniqueID = LazyObjectPtr.GetUniqueID();
				Ar << UniqueID;
				return *this;
			}
			virtual FArchive& operator<<(FSoftObjectPtr& Value) override
			{
				FArchive& Ar = *this;
				FSoftObjectPath UniqueID = Value.GetUniqueID();
				Ar << UniqueID;
				return Ar;
			}
			virtual FArchive& operator<<(FSoftObjectPath& Value) override
			{
				FArchive& Ar = *this;

				FString Path = Value.ToString();

				Ar << Path;

				if (IsLoading())
				{
					Value.SetPath(MoveTemp(Path));
				}

				return Ar;
			}
			FArchive& operator<<(FWeakObjectPtr& WeakObjectPtr) override
			{
				return FArchiveUObject::SerializeWeakObjectPtr(*this, WeakObjectPtr);
			}
			/** Archive name, for debugging */
			virtual FString GetArchiveName() const override { return TEXT("FCDOWriter"); }
		};
		TSet<UObject*> VisitedObjects;
		VisitedObjects.Add(InObject);
		FCDOWriter Ar(OutData, VisitedObjects, NAME_None);
		InObject->SerializeScriptProperties(Ar);
	}

	/** Returns true if the properties of the CDO have changed during hot-reload */
	bool HavePropertiesChanged(const FCDOPropertyData& lhs, const FCDOPropertyData& rhs)
	{
		return lhs.Bytes.Num() != rhs.Bytes.Num() || FMemory::Memcmp(lhs.Bytes.GetData(), rhs.Bytes.GetData(), lhs.Bytes.Num());
	}

	/** Helper for finding subobject in an array. Usually there's not that many subobjects on a class to justify a TMap */
	UObject* FindDefaultSubobject(TArray<UObject*>& InDefaultSubobjects, FName SubobjectName)
	{
		for (UObject* Subobject : InDefaultSubobjects)
		{
			if (Subobject->GetFName() == SubobjectName)
			{
				return Subobject;
			}
		}
		return nullptr;
	}

	/** Update the properties from the old class CDO to the new class CDO */
	void UpdateDefaultProperties(UClass* NewClass, const FCDOPropertyData& OldClassCDOProperties, const FCDOPropertyData& NewClassCDOProperties)
	{
		struct FPropertyToUpdate
		{
			FProperty* Property;
			FName SubobjectName;
			const uint8* OldSerializedValuePtr;
			uint8* NewValuePtr;
			int64 OldSerializedSize;
		};
		/** Memory writer archive that supports UObject values the same way as FCDOWriter. */
		class FPropertyValueMemoryWriter : public FMemoryWriter
		{
		public:
			FPropertyValueMemoryWriter(TArray<uint8>& OutData)
				: FMemoryWriter(OutData)
			{}
			virtual FArchive& operator<<(class UObject*& InObj) override
			{
				FArchive& Ar = *this;
				if (InObj)
				{
					FName ClassName = InObj->GetClass()->GetFName();
					FName ObjectName = InObj->GetFName();
					Ar << ClassName;
					Ar << ObjectName;
				}
				else
				{
					FName UnusedName = NAME_None;
					Ar << UnusedName;
					Ar << UnusedName;
				}
				return *this;
			}
			virtual FArchive& operator<<(FObjectPtr& InObj) override
			{
				// Invoke the method above
				return FArchiveUObject::SerializeObjectPtr(*this, InObj);
			}
			virtual FArchive& operator<<(FName& InName) override
			{
				FArchive& Ar = *this;
				FNameEntryId ComparisonIndex = InName.GetComparisonIndex();
				FNameEntryId DisplayIndex = InName.GetDisplayIndex();
				int32 Number = InName.GetNumber();
				Ar << ComparisonIndex;
				Ar << DisplayIndex;
				Ar << Number;
				return Ar;
			}
			virtual FArchive& operator<<(FLazyObjectPtr& LazyObjectPtr) override
			{
				FArchive& Ar = *this;
				FUniqueObjectGuid UniqueID = LazyObjectPtr.GetUniqueID();
				Ar << UniqueID;
				return *this;
			}
			virtual FArchive& operator<<(FSoftObjectPtr& Value) override
			{
				FArchive& Ar = *this;
				FSoftObjectPath UniqueID = Value.GetUniqueID();
				Ar << UniqueID;
				return Ar;
			}
			virtual FArchive& operator<<(FSoftObjectPath& Value) override
			{
				FArchive& Ar = *this;

				FString Path = Value.ToString();

				Ar << Path;

				if (IsLoading())
				{
					Value.SetPath(MoveTemp(Path));
				}

				return Ar;
			}
			FArchive& operator<<(FWeakObjectPtr& WeakObjectPtr) override
			{
				return FArchiveUObject::SerializeWeakObjectPtr(*this, WeakObjectPtr);
			}
		};

		// Collect default subobjects to update their properties too
		const int32 DefaultSubobjectArrayCapacity = 16;
		TArray<UObject*> DefaultSubobjectArray;
		DefaultSubobjectArray.Empty(DefaultSubobjectArrayCapacity);
		NewClass->GetDefaultObject()->GetDefaultSubobjects(DefaultSubobjectArray);

		TArray<FPropertyToUpdate> PropertiesToUpdate;
		// Collect all properties that have actually changed
		for (const TPair<FName, FCDOProperty>& Pair : NewClassCDOProperties.Properties)
		{
			const FCDOProperty* OldPropertyInfo = OldClassCDOProperties.Properties.Find(Pair.Key);
			if (OldPropertyInfo)
			{
				const FCDOProperty& NewPropertyInfo = Pair.Value;

				const uint8* OldSerializedValuePtr = OldClassCDOProperties.Bytes.GetData() + OldPropertyInfo->SerializedValueOffset;
				const uint8* NewSerializedValuePtr = NewClassCDOProperties.Bytes.GetData() + NewPropertyInfo.SerializedValueOffset;
				if (OldPropertyInfo->SerializedValueSize != NewPropertyInfo.SerializedValueSize ||
					FMemory::Memcmp(OldSerializedValuePtr, NewSerializedValuePtr, OldPropertyInfo->SerializedValueSize) != 0)
				{
					// Property value has changed so add it to the list of properties that need updating on instances
					FPropertyToUpdate PropertyToUpdate;
					PropertyToUpdate.Property = NewPropertyInfo.Property;
					PropertyToUpdate.NewValuePtr = nullptr;
					PropertyToUpdate.SubobjectName = NewPropertyInfo.SubobjectName;

					if (NewPropertyInfo.Property->GetOwner<UObject>() == NewClass)
					{
						PropertyToUpdate.NewValuePtr = PropertyToUpdate.Property->ContainerPtrToValuePtr<uint8>(NewClass->GetDefaultObject());
					}
					else if (NewPropertyInfo.SubobjectName != NAME_None)
					{
						UObject* DefaultSubobjectPtr = FindDefaultSubobject(DefaultSubobjectArray, NewPropertyInfo.SubobjectName);
						if (DefaultSubobjectPtr && NewPropertyInfo.Property->GetOwner<UObject>() == DefaultSubobjectPtr->GetClass())
						{
							PropertyToUpdate.NewValuePtr = PropertyToUpdate.Property->ContainerPtrToValuePtr<uint8>(DefaultSubobjectPtr);
						}
					}
					if (PropertyToUpdate.NewValuePtr)
					{
						PropertyToUpdate.OldSerializedValuePtr = OldSerializedValuePtr;
						PropertyToUpdate.OldSerializedSize = OldPropertyInfo->SerializedValueSize;

						PropertiesToUpdate.Add(PropertyToUpdate);
					}
				}
			}
		}
		if (PropertiesToUpdate.Num())
		{
			TArray<uint8> CurrentValueSerializedData;

			// Update properties on all existing instances of the class
			const UPackage* TransientPackage = GetTransientPackage();
			for (FThreadSafeObjectIterator It(NewClass); It; ++It)
			{
				UObject* ObjectPtr = *It;
				if (!IsValidChecked(ObjectPtr) || ObjectPtr->GetOutermost() == TransientPackage)
				{
					continue;
				}

				DefaultSubobjectArray.Empty(DefaultSubobjectArrayCapacity);
				ObjectPtr->GetDefaultSubobjects(DefaultSubobjectArray);

				for (auto& PropertyToUpdate : PropertiesToUpdate)
				{
					uint8* InstanceValuePtr = nullptr;
					if (PropertyToUpdate.SubobjectName == NAME_None)
					{
						InstanceValuePtr = PropertyToUpdate.Property->ContainerPtrToValuePtr<uint8>(ObjectPtr);
					}
					else
					{
						UObject* DefaultSubobjectPtr = FindDefaultSubobject(DefaultSubobjectArray, PropertyToUpdate.SubobjectName);
						if (DefaultSubobjectPtr && PropertyToUpdate.Property->GetOwner<UObject>() == DefaultSubobjectPtr->GetClass())
						{
							InstanceValuePtr = PropertyToUpdate.Property->ContainerPtrToValuePtr<uint8>(DefaultSubobjectPtr);
						}
					}

					if (InstanceValuePtr)
					{
						// Serialize current value to a byte array as we don't have the previous CDO to compare against, we only have its serialized property data
						CurrentValueSerializedData.Empty(CurrentValueSerializedData.Num() + CurrentValueSerializedData.GetSlack());
						FPropertyValueMemoryWriter CurrentValueWriter(CurrentValueSerializedData);
						PropertyToUpdate.Property->SerializeItem(FStructuredArchiveFromArchive(CurrentValueWriter).GetSlot(), InstanceValuePtr);

						// Update only when the current value on the instance is identical to the original CDO
						if (CurrentValueSerializedData.Num() == PropertyToUpdate.OldSerializedSize &&
							FMemory::Memcmp(CurrentValueSerializedData.GetData(), PropertyToUpdate.OldSerializedValuePtr, CurrentValueSerializedData.Num()) == 0)
						{
							// Update with the new value
							PropertyToUpdate.Property->CopyCompleteValue(InstanceValuePtr, PropertyToUpdate.NewValuePtr);
						}
					}
				}
			}
		}
	}

	/** Helper class that handles class reloading */

	class FReloadClassHelper
	{
	private:
		enum class TopologicalState
		{
			None,
			Visited,
			Finished,
		};

		struct ClassReinstanceState
		{
			UClass* OldClass;
			UObject* OldClassCDO;
			TopologicalState State = TopologicalState::None;
			FCDOPropertyData OldClassCDOProperties;
		};

	public:
		FReloadClassHelper(FOutputDevice& InAr, const TSet<UObject*>& InReinstancingObjects, TSet<UBlueprint*>& InCompiledBlueprints, TMap<UObject*, UObject*>& InReconstructedCDOsMap, TMap<UObject*, UObject*>& InReinstancedCDOsMap);

		/** 
		 * Re-instance the collection of classes.  
		 * The map is from old to new class object.
		 * If the old class is not changing structure, then the new class pointer is null.
		 */
		void ReinstanceClasses(const TMap<UClass*, UClass*>& ClassesToReinstance);

	private:

		/** Given a new/old class pair, re-instance the class */
		void ReinstanceClass(UClass* NewClass, ClassReinstanceState& State);

		/** Scan the properties looking for any referenced classes that need to be re-instanced first */
		void ReinstanceClassPropertyScan(FProperty* PropertyLink);

		/** Checked the property looking for any referenced classes that need to be re-instanced first */
		void ReinstanceClassProperty(FProperty* Property);

	private:
		FOutputDevice& Ar;
		const TSet<UObject*>& ReinstancingObjects;
		TSet<UBlueprint*>& CompiledBlueprints;
		TMap<UObject*, UObject*>& ReconstructedCDOsMap;
		TMap<UObject*, UObject*>& ReinstancedCDOsMap;


		TMap<UClass*, ClassReinstanceState> ReinstanceStates;
	};

	FReloadClassHelper::FReloadClassHelper(FOutputDevice& InAr, const TSet<UObject*>& InReinstancingObjects, TSet<UBlueprint*>& InCompiledBlueprints, TMap<UObject*, UObject*>& InReconstructedCDOsMap, TMap<UObject*, UObject*>& InReinstancedCDOsMap)
		: Ar(InAr)
		, ReinstancingObjects(InReinstancingObjects)
		, CompiledBlueprints(InCompiledBlueprints)
		, ReconstructedCDOsMap(InReconstructedCDOsMap)
		, ReinstancedCDOsMap(InReinstancedCDOsMap)
	{
	}

	void FReloadClassHelper::ReinstanceClasses(const TMap<UClass*, UClass*>& ClassesToReinstance)
	{
		// Create the collection of re-instance states so we can re-instance with topological sorting.
		for (const TPair<UClass*, UClass*>& Pair : ClassesToReinstance)
		{
			UClass* NewClass = Pair.Value != nullptr ? Pair.Value : Pair.Key;
			UClass* OldClass = Pair.Key;
			UObject* OldClassCDO = OldClass->GetDefaultObject();
			ClassReinstanceState State{ OldClass, OldClassCDO };
			UE::Reload::Private::SerializeCDOProperties(OldClassCDO, State.OldClassCDOProperties);
			ReinstanceStates.Add(NewClass, MoveTemp(State));
		}

		// Rename and clear the old defaults
		for (TPair<UClass*, ClassReinstanceState>& Pair : ReinstanceStates)
		{

			// If we are re-instancing a class that didn't change structure, then we need to replace
			// the existing default object
			if (Pair.Key == Pair.Value.OldClass)
			{
				Pair.Value.OldClassCDO->Rename(
					*MakeUniqueObjectName(
						GetTransientPackage(),
						Pair.Value.OldClassCDO->GetClass(),
						*FString::Printf(TEXT("BPGC_ARCH_FOR_CDO_%s"), *Pair.Value.OldClass->GetName())
					).ToString(),
					GetTransientPackage(),
					REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional | REN_SkipGeneratedClasses);

				// Clear the class default object so it gets recreated.
				Pair.Value.OldClass->SetDefaultObject(nullptr);
			}
		}

		// Re-instance the classes
		for (TPair<UClass*, ClassReinstanceState>& Pair : ReinstanceStates)
		{
			ReinstanceClass(Pair.Key, Pair.Value);
		}
	}

	void FReloadClassHelper::ReinstanceClass(UClass* NewClass, ClassReinstanceState& State)
	{
		// If this isn't a class we care about or we are already processing, then skip
		if (State.State != TopologicalState::None)
		{
			return;
		}

		// Mark that we are in the process of re-instancing this class
		State.State = TopologicalState::Visited;

		// Run the parents
		for (UClass* Super = NewClass->GetSuperClass(); Super != nullptr; Super = Super->GetSuperClass())
		{
			ClassReinstanceState* SuperState = ReinstanceStates.Find(Super);
			if (SuperState != nullptr)
			{
				ReinstanceClass(Super, *SuperState);
			}
		}

		// Look for any properties that reference other types needing re-instancing.  They will be re-instanced first
		ReinstanceClassPropertyScan(NewClass->PropertyLink);

		// Now we re-instance this class
		{
			UClass* OldClass = State.OldClass;

			// Collect the property values of the new CDO
			FCDOPropertyData NewClassCDOProperties;
			SerializeCDOProperties(NewClass->GetDefaultObject(), NewClassCDOProperties);

			// If the structure didn't change, always add to the list of reconstructed CDOs
			if (OldClass == NewClass)
			{
				ReconstructedCDOsMap.Add(State.OldClassCDO, NewClass->GetDefaultObject());
			}

			// We only need to do re-instancing when we have a new UClass.
			else
			{
				ReinstancedCDOsMap.Add(State.OldClassCDO, NewClass->GetDefaultObject());
				UClass* NullableNewClass = NewClass == OldClass ? nullptr : NewClass;
				TSharedPtr<FReloadClassReinstancer> ReinstanceHelper = MakeShareable(new FReloadClassReinstancer(
					NullableNewClass, OldClass, State.OldClassCDO, ReinstancingObjects, CompiledBlueprints));
				Ar.Logf(ELogVerbosity::Log, TEXT("Re-instancing %s after reload."), *NewClass->GetName());
				ReinstanceHelper->ReinstanceObjects(true);
			}

			// Update the default values
			UpdateDefaultProperties(NewClass, State.OldClassCDOProperties, NewClassCDOProperties);
		}

		State.State = TopologicalState::Finished;
	}

	void FReloadClassHelper::ReinstanceClassPropertyScan(FProperty* PropertyLink)
	{
		for (FProperty* Property = PropertyLink; Property; Property = Property->PropertyLinkNext)
		{
			ReinstanceClassProperty(Property);
		}
	}

	void FReloadClassHelper::ReinstanceClassProperty(FProperty* Property)
	{
		if (Property == nullptr)
		{
			return;
		}

		if (FObjectPropertyBase* objProp = CastField<FObjectPropertyBase>(Property))
		{
			if (objProp->PropertyClass != nullptr)
			{
				ClassReinstanceState* State = ReinstanceStates.Find(objProp->PropertyClass);
				if (State != nullptr)
				{
					ReinstanceClass(objProp->PropertyClass, *State);
				}
			}
			if (FClassProperty* classProp = CastField<FClassProperty>(Property))
			{
				if (classProp->MetaClass != nullptr)
				{
					ClassReinstanceState* State = ReinstanceStates.Find(classProp->MetaClass);
					if (State != nullptr)
					{
						ReinstanceClass(classProp->MetaClass, *State);
					}
				}
			}
		}
		else if (FStructProperty* structProp = CastField<FStructProperty>(Property))
		{
			ReinstanceClassPropertyScan(structProp->Struct->PropertyLink);
		}
		else if (FMapProperty* MapProp = CastField<FMapProperty>(Property))
		{
			ReinstanceClassProperty(MapProp->KeyProp);
			ReinstanceClassProperty(MapProp->ValueProp);
		}
		else if (FSetProperty* SetProp = CastField<FSetProperty>(Property))
		{
			ReinstanceClassProperty(SetProp->ElementProp);
		}
		else if (FArrayProperty* ArrayProp = CastField<FArrayProperty>(Property))
		{
			ReinstanceClassProperty(ArrayProp->Inner);
		}
	}
} // namespace UE::Reload::Private

FReload::FReload(EActiveReloadType InType, const TCHAR* InPrefix, const TArray<UPackage*>& InPackages, FOutputDevice& InAr)
	: Type(InType)
	, Prefix(InPrefix)
	, Packages(InPackages)
	, Ar(InAr)
	, bCollectPackages(false)
{
#if WITH_RELOAD
	BeginReload(Type, *this);
#endif
}

FReload::FReload(EActiveReloadType InType, const TCHAR* InPrefix, FOutputDevice& InAr)
	: Type(InType)
	, Prefix(InPrefix)
	, Ar(InAr)
	, bCollectPackages(true)
{
#if WITH_RELOAD
	BeginReload(Type, *this);
#endif
}

FReload::~FReload()
{
#if WITH_RELOAD
	EndReload();
#endif

	TStringBuilder<256> Builder;
	if (PackageStats.HasValues() || ClassStats.HasValues() || StructStats.HasValues() || EnumStats.HasValues() || NumFunctionsRemapped != 0 || NumScriptStructsRemapped != 0)
	{
		FormatStats(Builder, TEXT("package"), TEXT("packages"), PackageStats);
		FormatStats(Builder, TEXT("class"), TEXT("classes"), ClassStats);
		FormatStats(Builder, TEXT("enum"), TEXT("enums"), EnumStats);
		FormatStats(Builder, TEXT("scriptstruct"), TEXT("scriptstructs"), StructStats);
		FormatStat(Builder, TEXT("function"), TEXT("functions"), TEXT("remapped"), NumFunctionsRemapped);
		FormatStat(Builder, TEXT("scriptstruct"), TEXT("scriptstructs"), TEXT("remapped"), NumScriptStructsRemapped);
	}
	else
	{
		Builder << TEXT("No object changes detected");
	}
	Ar.Logf(ELogVerbosity::Display, TEXT("Reload/Re-instancing Complete: %s"), *Builder);

	if (bSendReloadComplete)
	{
		FCoreUObjectDelegates::ReloadCompleteDelegate.Broadcast(EReloadCompleteReason::None);
	}
}

bool FReload::GetEnableReinstancing(bool bHasChanged) const  
{ 
	if (bHasChanged && !bEnableReinstancing  && !bEnabledMessage)
	{
		bEnabledMessage = true;
		bHasReinstancingOccurred = true;
		Ar.Logf(ELogVerbosity::Display, TEXT("Re-instancing has been disabled.  Some changes will be ignored."));
	}
	return bEnableReinstancing;
}


void FReload::Reset()
{
	FunctionRemap.Empty();
	ReconstructedCDOsMap.Empty();
	ReinstancedClasses.Empty();
	NewClasses.Empty();
	ReinstancedEnums.Empty();
	ReinstancedStructs.Empty();
	Packages.Empty();
	bHasReinstancingOccurred = false;
}

void FReload::UpdateStats(FReinstanceStats& Stats, void* New, void* Old)
{
	if (Old == nullptr)
	{
		++Stats.New;
	}
	else if (Old != New)
	{
		++Stats.Changed;
	}
	else
	{
		++Stats.Unchanged;
	}
}

void FReload::FormatStats(FStringBuilderBase& Out, const TCHAR* Singular, const TCHAR* Plural, const FReinstanceStats& Stats)
{
	FormatStat(Out, Singular, Plural, TEXT("new"), Stats.New);
	FormatStat(Out, Singular, Plural, TEXT("changed"), Stats.Changed);
	FormatStat(Out, Singular, Plural, TEXT("unchanged"), Stats.Unchanged);
}

void FReload::FormatStat(FStringBuilderBase& Out, const TCHAR* Singular, const TCHAR* Plural, const TCHAR* What, int32 Value)
{
	if (Value == 0)
	{
		return;
	}

	if (Out.Len() != 0)
	{
		Out << TEXT(", ");
	}
	Out << Value << TEXT(" ") << (Value > 1 ? Plural : Singular) << TEXT(" ") << What;
}

void FReload::NotifyFunctionRemap(FNativeFuncPtr NewFunctionPointer, FNativeFuncPtr OldFunctionPointer)
{
	FNativeFuncPtr OtherNewFunction = FunctionRemap.FindRef(OldFunctionPointer);
	check(!OtherNewFunction || OtherNewFunction == NewFunctionPointer);
	check(NewFunctionPointer);
	check(OldFunctionPointer);
	FunctionRemap.Add(OldFunctionPointer, NewFunctionPointer);
}

void FReload::NotifyChange(UClass* New, UClass* Old)
{
	UpdateStats(ClassStats, New, Old);

	if (New != Old)
	{
		bHasReinstancingOccurred = true;
	}

	if (Old != nullptr)
	{
		// Don't allow re-instancing of UEngine classes
		if (!Old->IsChildOf(UEngine::StaticClass()))
		{
			UClass* NewIfChanged = Old != New ? New : nullptr; // supporting code detects unchanged based on null new pointer
			TMap<UClass*, UClass*>& ClassesToReinstance = GetClassesToReinstanceForHotReload();
			checkf(!ClassesToReinstance.Contains(Old) || ClassesToReinstance[Old] == NewIfChanged, TEXT("Attempting to reload a class which is already being reloaded as a different class"));
			ClassesToReinstance.Add(Old, NewIfChanged);
		}
		else if (Old != New) // This has changed
		{
			Ar.Logf(ELogVerbosity::Warning, TEXT("Engine class '%s' has changed but will be ignored for reload"), *New->GetName());
		}
	}
	else if (New != nullptr) // should always be the case, but protect against it
	{
		NewClasses.Add(New);
	}
}

void FReload::NotifyChange(UEnum* New, UEnum* Old)
{
	UpdateStats(EnumStats, New, Old);

	if (New != Old)
	{
		bHasReinstancingOccurred = true;
	}


	if (Old != nullptr)
	{
		UEnum* NewIfChanged = Old != New ? New : nullptr; // supporting code detects unchanged based on null new pointer
		checkf(!ReinstancedEnums.Contains(Old) || ReinstancedEnums[Old] == NewIfChanged, TEXT("Attempting to reload an enumeration which is already being reloaded as a different enumeration"));
		ReinstancedEnums.Add(Old, NewIfChanged);
	}
}

void FReload::NotifyChange(UScriptStruct* New, UScriptStruct* Old)
{
	UpdateStats(StructStats, New, Old);

	if (New != Old)
	{
		bHasReinstancingOccurred = true;
	}

	if (Old != nullptr)
	{
		UScriptStruct* NewIfChanged = Old != New ? New : nullptr; // supporting code detects unchanged based on null new pointer
		checkf(!ReinstancedStructs.Contains(Old) || ReinstancedStructs[Old] == NewIfChanged, TEXT("Attempting to reload a structure which is already being reloaded as a different structure"));
		ReinstancedStructs.Add(Old, NewIfChanged);
	}
}

void FReload::NotifyChange(UPackage* New, UPackage* Old)
{
	if (Old != nullptr)
	{
		++PackageStats.Changed;
	}
	else
	{
		++PackageStats.New;
	}

	Packages.Add(New);
}

namespace
{
	template<typename T>
	void CollectPackages(TSet<UPackage*>& Packages, const TMap<T*, T*>& Reinstances)
	{
		for (const TPair<T*, T*>& Pair : Reinstances)
		{
			T* Old = Pair.Key;
			T* New = Pair.Value;
			Packages.Add(New ? New->GetPackage() : Old->GetPackage());
		}
	}
}

void FReload::Reinstance()
{
	if (Type != EActiveReloadType::Reinstancing)
	{
		UClass::AssembleReferenceTokenStreams();
	}

	TMap<UClass*, UClass*>& ClassesToReinstance = GetClassesToReinstanceForHotReload();

	// If we have to collect the packages, gather them from the reinstanced objects
	if (bCollectPackages)
	{
		CollectPackages(Packages, ClassesToReinstance);
		CollectPackages(Packages, ReinstancedStructs);
		CollectPackages(Packages, ReinstancedEnums);
	}

	// Remap all native functions (and gather scriptstructs)
	UPackage* TransientPackage = GetTransientPackage();
	TArray<UScriptStruct*> ScriptStructs;
	for (FRawObjectIterator It; It; ++It)
	{
		if (UFunction* Function = Cast<UFunction>(static_cast<UObject*>(It->GetObject())))
		{
			if (FNativeFuncPtr NewFunction = FunctionRemap.FindRef(Function->GetNativeFunc()))
			{
				++NumFunctionsRemapped;
				Function->SetNativeFunc(NewFunction);
			}
		} 
		else if (UScriptStruct* ScriptStruct = Cast<UScriptStruct>(static_cast<UObject*>(It->GetObject())))
		{
			// IMPORTANT: The order here is very important.  There is a chance that GetCppStructOps will fail for 
			// types that are in a package we don't care about.  So we have to filter on the package first.
			UPackage* ScriptStructPackage = ScriptStruct->GetPackage();
			if (!ScriptStruct->HasAnyFlags(RF_ClassDefaultObject) &&
				ScriptStructPackage != TransientPackage &&
				Packages.Contains(ScriptStruct->GetPackage()) &&
				ScriptStruct->GetCppStructOps())
			{
				ScriptStructs.Add(ScriptStruct);
			}
		}
	}

	// now let's set up the script structs...this relies on super behavior, so null them all, then set them all up. Internally this sets them up hierarchically.
	for (UScriptStruct* Script : ScriptStructs)
	{
		Script->ClearCppStructOps();
	}
	for (UScriptStruct* Script : ScriptStructs)
	{
		Script->PrepareCppStructOps();
		check(Script->GetCppStructOps());
	}
	NumScriptStructsRemapped = ScriptStructs.Num();

	// Collect all the classes being re-instanced
	TSet<UObject*> ReinstancingObjects;
	ReinstancingObjects.Reserve(ClassesToReinstance.Num() + ReinstancedStructs.Num() + ReinstancedEnums.Num());
	for (const TPair<UClass*, UClass*>& Pair : ClassesToReinstance)
	{
		ReinstancingObjects.Add(Pair.Key);
	}

	// Collect all of the blueprint nodes that are getting updated due to enum/struct changes
	TMap<UBlueprint*, FBlueprintUpdateInfo> ModifiedBlueprints;
	FBlueprintEditorUtils::FOnNodeFoundOrUpdated OnNodeFoundOrUpdated = [&ModifiedBlueprints](UBlueprint* Blueprint, UK2Node* Node)
	{
		// Blueprint can be nullptr
		FBlueprintUpdateInfo& BlueprintUpdateInfo = ModifiedBlueprints.FindOrAdd(Blueprint);
		BlueprintUpdateInfo.Nodes.Add(Node);
	};

	// Update all the structures.  We add the unchanging structs to the list to make sure the defaults are updated
	TMap<UScriptStruct*, UScriptStruct*> ChangedStructs;
	for (const TPair<UScriptStruct*, UScriptStruct*>& Pair : ReinstancedStructs)
	{
		ReinstancingObjects.Add(Pair.Key);
		if (Pair.Value)
		{
			Pair.Key->StructFlags = EStructFlags(Pair.Key->StructFlags | STRUCT_NewerVersionExists);
			ChangedStructs.Emplace(Pair.Key, Pair.Value);
		}
		else
		{
			ChangedStructs.Emplace(Pair.Key, Pair.Key);
		}
	}
	FBlueprintEditorUtils::UpdateScriptStructsInNodes(ChangedStructs, OnNodeFoundOrUpdated);

	// Update all the enumeration nodes
	TMap<UEnum*, UEnum*> ChangedEnums;
	for (const TPair<UEnum*, UEnum*>& Pair : ReinstancedEnums)
	{
		ReinstancingObjects.Add(Pair.Key);
		if (Pair.Value)
		{
			Pair.Key->SetEnumFlags(EEnumFlags::NewerVersionExists);
			ChangedEnums.Emplace(Pair.Key, Pair.Value);
		}
	}
	FBlueprintEditorUtils::UpdateEnumsInNodes(ChangedEnums, OnNodeFoundOrUpdated);

	// Update all the nodes before we could possibly recompile
	for (TPair<UBlueprint*, FBlueprintUpdateInfo>& KVP : ModifiedBlueprints)
	{
		UBlueprint* Blueprint = KVP.Key;
		FBlueprintUpdateInfo& Info = KVP.Value;

		// We must reconstruct the node first other wise some pins might not be 
		// in a good state for the recompile
		for (UK2Node* Node : Info.Nodes)
		{
			Node->ReconstructNode();
		}
	}

	// Re-instance the classes
	TSet<UBlueprint*> CompiledBlueprints;
	UE::Reload::Private::FReloadClassHelper rch(Ar, ReinstancingObjects, CompiledBlueprints, ReconstructedCDOsMap, ReinstancedCDOsMap);
	rch.ReinstanceClasses(ClassesToReinstance);

	// Recompile blueprints if they haven't already been recompiled)
	for (TPair<UBlueprint*, FBlueprintUpdateInfo>& KVP : ModifiedBlueprints)
	{
		UBlueprint* Blueprint = KVP.Key;
		FBlueprintUpdateInfo& Info = KVP.Value;

		if (Blueprint && !CompiledBlueprints.Contains(Blueprint))
		{
			EBlueprintCompileOptions Options = EBlueprintCompileOptions::SkipGarbageCollection;
			FKismetEditorUtilities::CompileBlueprint(Blueprint, Options);
		}
	}

	ReinstancedClasses = MoveTemp(ClassesToReinstance);

	FCoreUObjectDelegates::ReloadReinstancingCompleteDelegate.Broadcast();
}

UObject* FReload::GetReinstancedCDO(UObject* CDO)
{
	return const_cast<UObject*>(GetReinstancedCDO(const_cast<const UObject*>(CDO)));
}

const UObject* FReload::GetReinstancedCDO(const UObject* CDO)
{
	UObject* const* NewCDO = ReconstructedCDOsMap.Find(CDO);
	if (NewCDO != nullptr)
	{
		return *NewCDO;
	}

	NewCDO = ReinstancedCDOsMap.Find(CDO);
	if (NewCDO != nullptr)
	{
		return *NewCDO;
	}

	return CDO;
}

void FReload::Finalize(bool bRunGC)
{
	FAssetCompilingManager::Get().FinishAllCompilation();

	// Make sure new classes have the token stream assembled
	UClass::AssembleReferenceTokenStreams();

	ReplaceReferencesToReconstructedCDOs();

	// Force GC to collect reinstanced objects
	if (bRunGC)
	{
		// Make sure the GIsInitialLoad flag is false.  Otherwise GC does nothing
		TGuardValue<bool> GuardIsInitialLoad(GIsInitialLoad, false);
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, true);
	}
}

void FReload::ReplaceReferencesToReconstructedCDOs()
{
	if (ReconstructedCDOsMap.Num() == 0)
	{
		return;
	}

	// Thread pool manager. We need new thread pool with increased
	// amount of stack size. Standard GThreadPool was encountering
	// stack overflow error during serialization.
	static struct FReplaceReferencesThreadPool
	{
		FReplaceReferencesThreadPool()
		{
			Pool = FQueuedThreadPool::Allocate();
			int32 NumThreadsInThreadPool = FPlatformMisc::NumberOfWorkerThreadsToSpawn();
			verify(Pool->Create(NumThreadsInThreadPool, 256 * 1024));
		}

		~FReplaceReferencesThreadPool()
		{
			Pool->Destroy();
		}

		FQueuedThreadPool* GetPool() { return Pool; }

	private:
		FQueuedThreadPool* Pool;
	} ThreadPoolManager;

	// Async task to enable multithreaded CDOs reference search.
	class FFindRefTask : public FNonAbandonableTask
	{
	public:
		explicit FFindRefTask(const TMap<UObject*, UObject*>& InReconstructedCDOsMap, int32 ReserveElements)
			: ReconstructedCDOsMap(InReconstructedCDOsMap)
		{
			ObjectsArray.Reserve(ReserveElements);
		}

		void DoWork()
		{
			for (UObject* Object : ObjectsArray)
			{
				class FReplaceCDOReferencesArchive : public FArchiveUObject
				{
				public:
					FReplaceCDOReferencesArchive(UObject* InPotentialReferencer, const TMap<UObject*, UObject*>& InReconstructedCDOsMap)
						: ReconstructedCDOsMap(InReconstructedCDOsMap)
						, PotentialReferencer(InPotentialReferencer)
					{
						ArIsObjectReferenceCollector = true;
						ArIgnoreOuterRef = true;
						ArShouldSkipBulkData = true;
					}

					virtual FString GetArchiveName() const override
					{
						return TEXT("FReplaceCDOReferencesArchive");
					}

					FArchive& operator<<(UObject*& ObjRef)
					{
						UObject* Obj = ObjRef;

						if (Obj && Obj != PotentialReferencer)
						{
							if (UObject* const* FoundObj = ReconstructedCDOsMap.Find(Obj))
							{
								ObjRef = *FoundObj;
							}
						}

						return *this;
					}

					virtual bool ShouldSkipProperty(const FProperty* InProperty) const
					{
						return InProperty->GetClass()->HasAnyCastFlags(CASTCLASS_FDelegateProperty | CASTCLASS_FMulticastDelegateProperty | CASTCLASS_FMulticastInlineDelegateProperty | CASTCLASS_FMulticastSparseDelegateProperty);
					}

					const TMap<UObject*, UObject*>& ReconstructedCDOsMap;
					UObject* PotentialReferencer;
				};

				FReplaceCDOReferencesArchive FindRefsArchive(Object, ReconstructedCDOsMap);
				Object->Serialize(FindRefsArchive);
			}
		}

		FORCEINLINE TStatId GetStatId() const
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(FFindRefTask, STATGROUP_ThreadPoolAsyncTasks);
		}

		TArray<UObject*> ObjectsArray;

	private:
		const TMap<UObject*, UObject*>& ReconstructedCDOsMap;
	};

	const int32 NumberOfThreads = FPlatformMisc::NumberOfWorkerThreadsToSpawn();
	const int32 NumObjects = GUObjectArray.GetObjectArrayNum();
	const int32 ObjectsPerTask = FMath::CeilToInt((float)NumObjects / NumberOfThreads);

	// Create tasks.
	TArray<FAsyncTask<FFindRefTask>> Tasks;
	Tasks.Reserve(NumberOfThreads);

	for (int32 TaskId = 0; TaskId < NumberOfThreads; ++TaskId)
	{
		Tasks.Emplace(ReconstructedCDOsMap, ObjectsPerTask);
	}

	// Distribute objects uniformly between tasks.
	int32 CurrentTaskId = 0;
	for (FThreadSafeObjectIterator ObjIter; ObjIter; ++ObjIter)
	{
		UObject* CurObject = *ObjIter;

		if (!IsValidChecked(CurObject))
		{
			continue;
		}

		Tasks[CurrentTaskId].GetTask().ObjectsArray.Add(CurObject);
		CurrentTaskId = (CurrentTaskId + 1) % NumberOfThreads;
	}

	// Run async tasks in worker threads.
	for (FAsyncTask<FFindRefTask>& Task : Tasks)
	{
		Task.StartBackgroundTask(ThreadPoolManager.GetPool());
	}

	// Wait until tasks are finished
	for (FAsyncTask<FFindRefTask>& AsyncTask : Tasks)
	{
		AsyncTask.EnsureCompletion();
	}
}
