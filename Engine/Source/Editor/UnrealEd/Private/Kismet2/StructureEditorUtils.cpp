// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet2/StructureEditorUtils.h"
#include "Misc/MessageDialog.h"
#include "Misc/CoreMisc.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectHash.h"
#include "UObject/UnrealType.h"
#include "UObject/TextProperty.h"
#include "Engine/Blueprint.h"
#include "Engine/DataTable.h"
#include "EdMode.h"
#include "ScopedTransaction.h"
#include "EdGraphSchema_K2.h"
#include "UserDefinedStructure/UserDefinedStructEditorData.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompilerModule.h"

#define LOCTEXT_NAMESPACE "Structure"

//////////////////////////////////////////////////////////////////////////
// FStructEditorManager
FStructureEditorUtils::FStructEditorManager::FStructEditorManager()
{
	FUserDefinedStructEditorUtils::OnUserDefinedStructChanged.BindLambda([](UUserDefinedStruct* Struct)
		{
			OnStructureChanged(Struct);
		});
}

FStructureEditorUtils::FStructEditorManager& FStructureEditorUtils::FStructEditorManager::Get()
{
	static TSharedRef< FStructEditorManager > EditorManager( new FStructEditorManager() );
	return *EditorManager;
}

FStructureEditorUtils::EStructureEditorChangeInfo FStructureEditorUtils::FStructEditorManager::ActiveChange = FStructureEditorUtils::EStructureEditorChangeInfo::Unknown;

//////////////////////////////////////////////////////////////////////////
// FStructureEditorUtils
UUserDefinedStruct* FStructureEditorUtils::CreateUserDefinedStruct(UObject* InParent, FName Name, EObjectFlags Flags)
{
	UUserDefinedStruct* Struct = NULL;
	
	if (UserDefinedStructEnabled())
	{
		Struct = NewObject<UUserDefinedStruct>(InParent, Name, Flags);
		UUserDefinedStructEditorData* EditorData = NewObject<UUserDefinedStructEditorData>(Struct, NAME_None, RF_Transactional);
		Struct->EditorData = EditorData;
		EditorData->MetaData.Add(FBlueprintTags::BlueprintType, TEXT("true"));

		Struct->Guid = FGuid::NewGuid();
		Struct->Bind();
		Struct->StaticLink(true);
		Struct->Status = UDSS_Error;

		{
			AddVariable(Struct, FEdGraphPinType(UEdGraphSchema_K2::PC_Boolean, NAME_None, nullptr, EPinContainerType::None, false, FEdGraphTerminalType()));
		}
	}

	return Struct;
}

namespace 
{
	static bool IsObjPropertyValid(const FProperty* Property)
	{
		if (const FInterfaceProperty* InterfaceProperty = CastField<const FInterfaceProperty>(Property))
		{
			return InterfaceProperty->InterfaceClass != nullptr;
		}
		else if (const FArrayProperty* ArrayProperty = CastField<const FArrayProperty>(Property))
		{
			return ArrayProperty->Inner && IsObjPropertyValid(ArrayProperty->Inner);
		}
		else if (const FObjectProperty* ObjectProperty = CastField<const FObjectProperty>(Property))
		{
			return ObjectProperty->PropertyClass != nullptr;
		}
		return true;
	}
}

FStructureEditorUtils::EStructureError FStructureEditorUtils::IsStructureValid(const UScriptStruct* Struct, const UStruct* RecursionParent, FString* OutMsg)
{
	check(Struct);
	if (Struct == RecursionParent)
	{
		if (OutMsg)
		{
			*OutMsg = FText::Format(LOCTEXT("StructureRecursionFmt", "Recursion: Recursion: Struct cannot have itself or a nested struct member referencing itself as a member variable. Struct '{0}', recursive parent '{1}'"), 
				 FText::FromString(Struct->GetFullName()), FText::FromString(RecursionParent->GetFullName())).ToString();
		}
		return EStructureError::Recursion;
	}

	const UScriptStruct* FallbackStruct = GetFallbackStruct();
	if (Struct == FallbackStruct)
	{
		if (OutMsg)
		{
			*OutMsg = LOCTEXT("StructureUnknown", "Struct unknown (deleted?)").ToString();
		}
		return EStructureError::FallbackStruct;
	}

	if (Struct->GetStructureSize() <= 0)
	{
		if (OutMsg)
		{
			*OutMsg = FText::Format(LOCTEXT("StructureSizeIsZeroFmt", "Struct '{0}' is empty"), FText::FromString(Struct->GetFullName())).ToString();
		}
		return EStructureError::EmptyStructure;
	}

	if (const UUserDefinedStruct* UDStruct = Cast<const UUserDefinedStruct>(Struct))
	{
		if (UDStruct->Status != EUserDefinedStructureStatus::UDSS_UpToDate && UDStruct->Status != EUserDefinedStructureStatus::UDSS_Error)
		{
			if (OutMsg)
			{
				*OutMsg = FText::Format(LOCTEXT("StructureNotCompiledFmt", "Struct '{0}' is not compiled"), FText::FromString(Struct->GetFullName())).ToString();
			}
			return EStructureError::NotCompiled;
		}

		for (const FProperty* P = Struct->PropertyLink; P; P = P->PropertyLinkNext)
		{
			const FStructProperty* StructProp = CastField<const FStructProperty>(P);
			if (NULL == StructProp)
			{
				if (const FArrayProperty* ArrayProp = CastField<const FArrayProperty>(P))
				{
					StructProp = CastField<const FStructProperty>(ArrayProp->Inner);
				}
			}

			if (StructProp)
			{
				if ((NULL == StructProp->Struct) || (FallbackStruct == StructProp->Struct))
				{
					if (OutMsg)
					{
						*OutMsg = FText::Format(
							LOCTEXT("StructureUnknownPropertyFmt", "Struct unknown (deleted?). Parent '{0}' Property: '{1}'"),
							FText::FromString(Struct->GetFullName()),
							FText::FromString(StructProp->GetName())
						).ToString();
					}
					return EStructureError::FallbackStruct;
				}

				FString OutMsgInner;
				const EStructureError Result = IsStructureValid(
					StructProp->Struct,
					RecursionParent ? RecursionParent : Struct,
					OutMsg ? &OutMsgInner : NULL);
				if (EStructureError::Ok != Result)
				{
					if (OutMsg)
					{
						*OutMsg = FText::Format(
							LOCTEXT("StructurePropertyErrorTemplateFmt", "Struct '{0}' Property '{1}' Error ( {2} )"),
							FText::FromString(Struct->GetFullName()),
							FText::FromString(StructProp->GetName()),
							FText::FromString(OutMsgInner)
						).ToString();
					}
					return Result;
				}
			}

			// The structure is loaded (from .uasset) without recompilation. All properties should be verified.
			if (!IsObjPropertyValid(P))
			{
				if (OutMsg)
				{
					*OutMsg = FText::Format(
						LOCTEXT("StructureUnknownObjectPropertyFmt", "Invalid object property. Structure '{0}' Property: '{1}'"),
						FText::FromString(Struct->GetFullName()),
						FText::FromString(P->GetName())
					).ToString();
				}
				return EStructureError::NotCompiled;
			}
		}
	}

	return EStructureError::Ok;
}

bool FStructureEditorUtils::CanHaveAMemberVariableOfType(const UUserDefinedStruct* Struct, const FEdGraphPinType& VarType, FString* OutMsg)
{
	if ((VarType.PinCategory == UEdGraphSchema_K2::PC_Struct) && Struct)
	{
		if (const UObject* TypeObject = VarType.PinSubCategoryObject.Get())
		{
			if (const UScriptStruct* SubCategoryStruct = Cast<const UScriptStruct>(TypeObject))
			{
				const EStructureError Result = IsStructureValid(SubCategoryStruct, Struct, OutMsg);
				if (EStructureError::Ok != Result)
				{
					return false;
				}
			}
			else
			{
				if (OutMsg)
				{
					*OutMsg = LOCTEXT("StructureIncorrectStructType", "Incorrect struct type in a structure member variable.").ToString();
				}
				return false;
			}
		}
	}
	else if ((VarType.PinCategory == UEdGraphSchema_K2::PC_Exec) 
		|| (VarType.PinCategory == UEdGraphSchema_K2::PC_Wildcard)
		|| (VarType.PinCategory == UEdGraphSchema_K2::PC_MCDelegate)
		|| (VarType.PinCategory == UEdGraphSchema_K2::PC_Delegate))
	{
		if (OutMsg)
		{
			*OutMsg = LOCTEXT("StructureIncorrectTypeCategory", "Incorrect type for a structure member variable.").ToString();
		}
		return false;
	}
	return true;
}

struct FMemberVariableNameHelper
{
	static FName Generate(UUserDefinedStruct* Struct, const FString& NameBase, const FGuid Guid, FString* OutFriendlyName = NULL)
	{
		check(Struct);

		FString Result;
		if (!NameBase.IsEmpty())
		{
			if (!FName::IsValidXName(NameBase, INVALID_OBJECTNAME_CHARACTERS))
			{
				Result = MakeObjectNameFromDisplayLabel(NameBase, NAME_None).GetPlainNameString();
			}
			else
			{
				Result = NameBase;
			}
		}

		if (Result.IsEmpty())
		{
			Result = TEXT("MemberVar");
		}

		const uint32 UniqueNameId = CastChecked<UUserDefinedStructEditorData>(Struct->EditorData)->GenerateUniqueNameIdForMemberVariable();
		const FString FriendlyName = FString::Printf(TEXT("%s_%u"), *Result, UniqueNameId);
		if (OutFriendlyName)
		{
			*OutFriendlyName = FriendlyName;
		}
		const FName NameResult = *FString::Printf(TEXT("%s_%s"), *FriendlyName, *Guid.ToString(EGuidFormats::Digits));
		check(NameResult.IsValidXName(INVALID_OBJECTNAME_CHARACTERS));
		return NameResult;
	}

	static FGuid GetGuidFromName(const FName Name)
	{
		const FString NameStr = Name.ToString();
		const int32 GuidStrLen = 32;
		if (NameStr.Len() > (GuidStrLen + 1))
		{
			const int32 UnderscoreIndex = NameStr.Len() - GuidStrLen - 1;
			if (TCHAR('_') == NameStr[UnderscoreIndex])
			{
				const FString GuidStr = NameStr.Right(GuidStrLen);
				FGuid Guid;
				if (FGuid::ParseExact(GuidStr, EGuidFormats::Digits, Guid))
				{
					return Guid;
				}
			}
		}
		return FGuid();
	}
};

bool FStructureEditorUtils::AddVariable(UUserDefinedStruct* Struct, const FEdGraphPinType& VarType)
{
	if (Struct)
	{
		const FScopedTransaction Transaction( LOCTEXT("AddVariable", "Add Variable") );
		ModifyStructData(Struct);

		FString ErrorMessage;
		if (!CanHaveAMemberVariableOfType(Struct, VarType, &ErrorMessage))
		{
			UE_LOGF(LogBlueprint, Warning, "%ls", *ErrorMessage);
			return false;
		}

		const FGuid Guid = FGuid::NewGuid();
		FString DisplayName;
		const FName VarName = FMemberVariableNameHelper::Generate(Struct, FString(), Guid, &DisplayName);
		check(NULL == GetVarDesc(Struct).FindByPredicate(FStructureEditorUtils::FFindByNameHelper<FStructVariableDescription>(VarName)));
		check(IsUniqueVariableFriendlyName(Struct, DisplayName));

		FStructVariableDescription NewVar;
		NewVar.VarName = VarName;
		NewVar.FriendlyName = DisplayName;
		NewVar.SetPinType(VarType);
		NewVar.VarGuid = Guid;
		GetVarDesc(Struct).Add(NewVar);

		OnStructureChanged(Struct, EStructureEditorChangeInfo::AddedVariable);
		return true;
	}
	return false;
}

bool FStructureEditorUtils::RemoveVariable(UUserDefinedStruct* Struct, FGuid VarGuid)
{
	if(Struct)
	{
		const int32 OldNum = GetVarDesc(Struct).Num();
		const bool bAllowToMakeEmpty = false;
		if (bAllowToMakeEmpty || (OldNum > 1))
		{
			const FScopedTransaction Transaction(LOCTEXT("RemoveVariable", "Remove Variable"));
			ModifyStructData(Struct);

			GetVarDesc(Struct).RemoveAll(FFindByGuidHelper<FStructVariableDescription>(VarGuid));
			if (OldNum != GetVarDesc(Struct).Num())
			{
				OnStructureChanged(Struct, EStructureEditorChangeInfo::RemovedVariable);
				return true;
			}
		}
		else
		{
			UE_LOGF(LogBlueprint, Log, "Member variable cannot be removed. User Defined Structure cannot be empty");
		}
	}
	return false;
}

bool FStructureEditorUtils::RenameVariable(UUserDefinedStruct* Struct, FGuid VarGuid, const FString& NewDisplayNameStr)
{
	if (Struct)
	{
		FStructVariableDescription* VarDesc = GetVarDescByGuid(Struct, VarGuid);
		if (VarDesc 
			&& !NewDisplayNameStr.IsEmpty()
			&& IsUniqueVariableFriendlyName(Struct, NewDisplayNameStr))
		{
			const FScopedTransaction Transaction(LOCTEXT("RenameVariable", "Rename Variable"));
			ModifyStructData(Struct);

			VarDesc->FriendlyName = NewDisplayNameStr;
			//>>> TEMPORARY it's more important to prevent changes in structs instances, than to have consistent names
			if (GetGuidFromPropertyName(VarDesc->VarName).IsValid())
			//<<< TEMPORARY
			{
				const FName NewName = FMemberVariableNameHelper::Generate(Struct, NewDisplayNameStr, VarGuid);
				check(NULL == GetVarDesc(Struct).FindByPredicate(FFindByNameHelper<FStructVariableDescription>(NewName)))
				VarDesc->VarName = NewName;
			}
			OnStructureChanged(Struct, EStructureEditorChangeInfo::RenamedVariable);
			return true;
		}
	}
	return false;
}


bool FStructureEditorUtils::RenameVariable(UUserDefinedStruct* Struct, const FString& OldDisplayNameStr, const FString& NewDisplayNameStr)
{
	if (Struct)
	{
		if (TArray<FStructVariableDescription>* VarDescArray = GetVarDescPtr(Struct))
		{
			FStructVariableDescription* VarDesc = VarDescArray->FindByPredicate(
				[&OldDisplayNameStr](const FStructVariableDescription& Var)
				{
					return Var.FriendlyName == OldDisplayNameStr;
				}
			);

			if (VarDesc)
			{
				return RenameVariable(Struct, VarDesc->VarGuid, NewDisplayNameStr);
			}
		}
	}

	return false;
}

bool FStructureEditorUtils::ChangeVariableType(UUserDefinedStruct* Struct, FGuid VarGuid, const FEdGraphPinType& NewType)
{
	if (Struct)
	{
		FString ErrorMessage;
		if(!CanHaveAMemberVariableOfType(Struct, NewType, &ErrorMessage))
		{
			UE_LOGF(LogBlueprint, Warning, "%ls", *ErrorMessage);
			return false;
		}

		FStructVariableDescription* VarDesc = GetVarDescByGuid(Struct, VarGuid);
		if(VarDesc)
		{
			const bool bChangedType = (VarDesc->ToPinType() != NewType);
			if (bChangedType)
			{
				const FScopedTransaction Transaction(LOCTEXT("ChangeVariableType", "Change Variable Type"));
				ModifyStructData(Struct);

				VarDesc->VarName = FMemberVariableNameHelper::Generate(Struct, VarDesc->FriendlyName, VarDesc->VarGuid);
				VarDesc->DefaultValue = FString();
				VarDesc->SetPinType(NewType);

				OnStructureChanged(Struct, EStructureEditorChangeInfo::VariableTypeChanged);
				return true;
			}
		}
	}
	return false;
}

bool FStructureEditorUtils::ChangeVariableDefaultValue(UUserDefinedStruct* Struct, FGuid VarGuid, const FString& NewDefaultValue)
{
	auto ValidateDefaultValue = [](const FStructVariableDescription& VarDesc, const FString& InNewDefaultValue) -> bool
	{
		const FEdGraphPinType PinType = VarDesc.ToPinType();

		bool bResult = false;
		//TODO: validation for values, that are not passed by string
		if (PinType.PinCategory == UEdGraphSchema_K2::PC_Text)
		{
			bResult = true;
		}
		else if ((PinType.PinCategory == UEdGraphSchema_K2::PC_Object) 
			|| (PinType.PinCategory == UEdGraphSchema_K2::PC_Interface) 
			|| (PinType.PinCategory == UEdGraphSchema_K2::PC_Class)
			|| (PinType.PinCategory == UEdGraphSchema_K2::PC_SoftClass)
			|| (PinType.PinCategory == UEdGraphSchema_K2::PC_SoftObject))
		{
			// K2Schema->DefaultValueSimpleValidation finds an object, passed by path, invalid
			bResult = true;
		}
		else
		{
			const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
			bResult = K2Schema->DefaultValueSimpleValidation(PinType, NAME_None, InNewDefaultValue, nullptr, FText::GetEmpty());
		}
		return bResult;
	};

	FStructVariableDescription* VarDesc = GetVarDescByGuid(Struct, VarGuid);
	if (VarDesc 
		&& (NewDefaultValue != VarDesc->DefaultValue)
		&& ValidateDefaultValue(*VarDesc, NewDefaultValue))
	{
		bool bAdvancedValidation = true;
		if (!NewDefaultValue.IsEmpty())
		{
			const FProperty* Property = FindFProperty<FProperty>(Struct, VarDesc->VarName);
			FStructOnScope StructDefaultMem(Struct);
			bAdvancedValidation = StructDefaultMem.IsValid() && Property &&
				FBlueprintEditorUtils::PropertyValueFromString(Property, NewDefaultValue, StructDefaultMem.GetStructMemory(), Struct);
		}

		if (bAdvancedValidation)
		{
			const FScopedTransaction Transaction(LOCTEXT("ChangeVariableDefaultValue", "Change Variable Default Value"));
			
			TGuardValue<FStructureEditorUtils::EStructureEditorChangeInfo> ActiveChangeGuard(FStructureEditorUtils::FStructEditorManager::ActiveChange, EStructureEditorChangeInfo::DefaultValueChanged);

			ModifyStructData(Struct);
			
			VarDesc->DefaultValue = NewDefaultValue;
			OnStructureChanged(Struct, EStructureEditorChangeInfo::DefaultValueChanged);
			return true;
		}
	}
	return false;
}

bool FStructureEditorUtils::IsUniqueVariableFriendlyName(const UUserDefinedStruct* Struct, const FString& DisplayName)
{
	if(Struct)
	{
		for (const FStructVariableDescription& VarDesc : GetVarDesc(Struct))
		{
			if (VarDesc.FriendlyName == DisplayName)
			{
				return false;
			}
		}
		return true;
	}
	return false;
}

FString FStructureEditorUtils::GetVariableFriendlyName(const UUserDefinedStruct* Struct, FGuid VarGuid)
{
	const FStructVariableDescription* VarDesc = GetVarDescByGuid(Struct, VarGuid);
	return VarDesc ? VarDesc->FriendlyName : FString();
}

FString FStructureEditorUtils::GetVariableFriendlyNameForProperty(const UUserDefinedStruct* Struct, const FProperty* Property)
{
	if (Struct && Property)
	{
		const TArray<FStructVariableDescription>* VarDescArray = GetVarDescPtr(Struct);
		if (VarDescArray)
		{
			const FStructVariableDescription* VarDesc = VarDescArray->FindByPredicate(FFindByNameHelper<FStructVariableDescription>(Property->GetFName()));
			if (VarDesc)
			{
				return VarDesc->FriendlyName;
			}
		}
	}
	return FString();
}

FProperty* FStructureEditorUtils::GetPropertyByFriendlyName(const UUserDefinedStruct* Struct, FString DisplayName)
{
	if (Struct)
	{
		for (const FStructVariableDescription& VarDesc : GetVarDesc(Struct))
		{
			if (VarDesc.FriendlyName == DisplayName)
			{
				return FindFProperty<FProperty>(Struct, VarDesc.VarName);
			}
		}
	}
	return nullptr;
}

bool FStructureEditorUtils::UserDefinedStructEnabled()
{
	static FBoolConfigValueHelper UseUserDefinedStructure(TEXT("UserDefinedStructure"), TEXT("bUseUserDefinedStructure"));
	return UseUserDefinedStructure;
}

void FStructureEditorUtils::RecreateDefaultInstanceInEditorData(UUserDefinedStruct* Struct)
{
	UUserDefinedStructEditorData* StructEditorData = Struct ? CastChecked<UUserDefinedStructEditorData>(Struct->EditorData) : nullptr;
	if (StructEditorData)
	{
		StructEditorData->RecreateDefaultInstance();
	}
}

void FStructureEditorUtils::CompileStructure(UUserDefinedStruct* Struct)
{
	if (Struct)
	{
		IKismetCompilerInterface& Compiler = FModuleManager::LoadModuleChecked<IKismetCompilerInterface>(KISMET_COMPILER_MODULENAME);
		FCompilerResultsLog Results;
		Compiler.CompileStructure(Struct, Results);
	}
}

void FStructureEditorUtils::OnStructureChanged(UUserDefinedStruct* Struct, EStructureEditorChangeInfo ChangeReason)
{
	if (Struct)
	{
		TGuardValue<FStructureEditorUtils::EStructureEditorChangeInfo> ActiveChangeGuard(FStructureEditorUtils::FStructEditorManager::ActiveChange, ChangeReason);

		Struct->Status = EUserDefinedStructureStatus::UDSS_Dirty;
		CompileStructure(Struct);
		(void)Struct->MarkPackageDirty();
		Struct->OnChanged();
	}
}

void FStructureEditorUtils::RemoveInvalidStructureMemberVariableFromBlueprint(UBlueprint* Blueprint)
{
	FBlueprintEditorUtils::RemoveInvalidStructureMemberVariableFromBlueprint(Blueprint);
}

TArray<FStructVariableDescription>& FStructureEditorUtils::GetVarDesc(UUserDefinedStruct* Struct)
{
	check(Struct);
	return CastChecked<UUserDefinedStructEditorData>(Struct->EditorData)->VariablesDescriptions;
}

const TArray<FStructVariableDescription>& FStructureEditorUtils::GetVarDesc(const UUserDefinedStruct* Struct)
{
	check(Struct);
	return CastChecked<const UUserDefinedStructEditorData>(Struct->EditorData)->VariablesDescriptions;
}

TArray<FStructVariableDescription>* FStructureEditorUtils::GetVarDescPtr(UUserDefinedStruct* Struct)
{
	check(Struct);
	return Struct->EditorData ? &CastChecked<UUserDefinedStructEditorData>(Struct->EditorData)->VariablesDescriptions : nullptr;
}

const TArray<FStructVariableDescription>* FStructureEditorUtils::GetVarDescPtr(const UUserDefinedStruct* Struct)
{
	check(Struct);
	return Struct->EditorData ? &CastChecked<const UUserDefinedStructEditorData>(Struct->EditorData)->VariablesDescriptions : nullptr;
}

FStructVariableDescription* FStructureEditorUtils::GetVarDescByGuid(UUserDefinedStruct* Struct, FGuid VarGuid)
{
	if (Struct)
	{
		TArray<FStructVariableDescription>* VarDescArray = GetVarDescPtr(Struct);
		return VarDescArray ? VarDescArray->FindByPredicate(FFindByGuidHelper<FStructVariableDescription>(VarGuid)) : nullptr;
	}
	return nullptr;
}

const FStructVariableDescription* FStructureEditorUtils::GetVarDescByGuid(const UUserDefinedStruct* Struct, FGuid VarGuid)
{
	if (Struct)
	{
		const TArray<FStructVariableDescription>* VarDescArray = GetVarDescPtr(Struct);
		return VarDescArray ? VarDescArray->FindByPredicate(FFindByGuidHelper<FStructVariableDescription>(VarGuid)) : nullptr;
	}
	return nullptr;
}

FString FStructureEditorUtils::GetTooltip(const UUserDefinedStruct* Struct)
{
	const UUserDefinedStructEditorData* StructEditorData = Struct ? Cast<const UUserDefinedStructEditorData>(Struct->EditorData) : nullptr;
	return StructEditorData ? StructEditorData->ToolTip : FString();
}

bool FStructureEditorUtils::ChangeTooltip(UUserDefinedStruct* Struct, const FString& InTooltip)
{
	UUserDefinedStructEditorData* StructEditorData = Struct ? Cast<UUserDefinedStructEditorData>(Struct->EditorData) : NULL;
	if (StructEditorData && StructEditorData->ToolTip.Compare(InTooltip) != 0)
	{
		const FScopedTransaction Transaction(LOCTEXT("ChangeTooltip", "Change UDS Tooltip"));
		StructEditorData->Modify();
		StructEditorData->ToolTip = InTooltip;

		Struct->SetMetaData(FBlueprintMetadata::MD_Tooltip, *StructEditorData->ToolTip);
		Struct->PostEditChange();

		return true;
	}
	return false;
}

FString FStructureEditorUtils::GetVariableTooltip(const UUserDefinedStruct* Struct, FGuid VarGuid)
{
	const FStructVariableDescription* VarDesc = GetVarDescByGuid(Struct, VarGuid);
	return VarDesc ? VarDesc->ToolTip : FString();
}

bool FStructureEditorUtils::ChangeVariableTooltip(UUserDefinedStruct* Struct, FGuid VarGuid, const FString& InTooltip)
{
	FStructVariableDescription* VarDesc = GetVarDescByGuid(Struct, VarGuid);
	if (VarDesc && VarDesc->ToolTip.Compare(InTooltip) != 0)
	{
		const FScopedTransaction Transaction(LOCTEXT("ChangeVariableTooltip", "Change UDS Variable Tooltip"));
		ModifyStructData(Struct);
		VarDesc->ToolTip = InTooltip;

		FProperty* Property = FindFProperty<FProperty>(Struct, VarDesc->VarName);
		if (Property)
		{
			Property->SetMetaData(FBlueprintMetadata::MD_Tooltip, *VarDesc->ToolTip);
		}

		return true;
	}
	return false;
}

bool FStructureEditorUtils::ChangeEditableOnBPInstance(UUserDefinedStruct* Struct, FGuid VarGuid, bool bInIsEditable)
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	FStructVariableDescription* VarDesc = GetVarDescByGuid(Struct, VarGuid);
	const bool bNewDontEditOnInstance = !bInIsEditable;
	if (VarDesc && (bNewDontEditOnInstance != VarDesc->bDontEditOnInstance))
	{
		const FScopedTransaction Transaction(LOCTEXT("ChangeVariableOnBPInstance", "Change variable editable on BP instance"));
		ModifyStructData(Struct);

		VarDesc->bDontEditOnInstance = bNewDontEditOnInstance;
		OnStructureChanged(Struct);
		return true;
	}
	return false;
}

bool FStructureEditorUtils::ChangeSaveGameEnabled(UUserDefinedStruct* Struct, FGuid VarGuid, bool bInSaveGame)
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	FStructVariableDescription* VarDesc = GetVarDescByGuid(Struct, VarGuid);
	if (VarDesc && (bInSaveGame != VarDesc->bEnableSaveGame))
	{
		const FScopedTransaction Transaction(LOCTEXT("ChangeSaveGameOnVariable", "Change variable SaveGame flag"));
		ModifyStructData(Struct);

		VarDesc->bEnableSaveGame = bInSaveGame;
		OnStructureChanged(Struct);
		return true;
	}
	return false;
}

/** Compute the initial and new indices to move the specified variable above/below another variable. */
static bool ComputeIndicesForMove(
	const TArray<FStructVariableDescription>& DescArray,
	const FGuid& MoveVarGuid,
	const FGuid& RelativeToGuid,
	FStructureEditorUtils::EMovePosition Position,
	int32& OutInitialIndex,
	int32& OutNewIndex)
{
	int32 InitialIndex = DescArray.IndexOfByPredicate(
		[MoveVarGuid](const FStructVariableDescription& Desc)
		{
			return Desc.VarGuid == MoveVarGuid;
		});
	int32 NewIndex = DescArray.IndexOfByPredicate(
		[RelativeToGuid](const FStructVariableDescription& Desc)
		{
			return Desc.VarGuid == RelativeToGuid;
		});
	if (InitialIndex == INDEX_NONE || NewIndex == INDEX_NONE)
	{
		return false;
	}

	if (Position == FStructureEditorUtils::PositionBelow)
	{
		// If moving below a variable, then we actually move it to the next variable's index
		NewIndex++;
	}

	if (InitialIndex < NewIndex)
	{
		// When the element is removed from the array, all the other elements below it are shifted by one,
		// so moving an element down the array causes its new index to shift by one
		NewIndex--;
	}

	if (InitialIndex == NewIndex)
	{
		// No move is happening because the index didn't change.
		return false;
	}

	if (!ensure(NewIndex >= 0 && NewIndex < DescArray.Num()))
	{
		// New index is out of bounds - this shouldn't happen!
		return false;
	}

	OutInitialIndex = InitialIndex;
	OutNewIndex = NewIndex;
	return true;
}

bool FStructureEditorUtils::MoveVariable(UUserDefinedStruct* Struct, FGuid MoveVarGuid, FGuid RelativeToGuid, EMovePosition Position)
{
	if (Struct)
	{
		TArray<FStructVariableDescription>& DescArray = GetVarDesc(Struct);
		int32 InitialIndex, NewIndex;
		if (!ComputeIndicesForMove(DescArray, MoveVarGuid, RelativeToGuid, Position, InitialIndex, NewIndex))
		{
			return false;
		}

		const FScopedTransaction Transaction(LOCTEXT("ReorderVariables", "Variables reordered"));
		ModifyStructData(Struct);

		FStructVariableDescription MoveDesc = DescArray[InitialIndex];
		DescArray.RemoveAt(InitialIndex);
		DescArray.Insert(MoveDesc, NewIndex);

		OnStructureChanged(Struct, EStructureEditorChangeInfo::MovedVariable);
		return true;
	}
	return false;
}

bool FStructureEditorUtils::CanMoveVariable(UUserDefinedStruct* Struct, FGuid MoveVarGuid, FGuid RelativeToGuid, EMovePosition Position)
{
	if (Struct)
	{
		TArray<FStructVariableDescription>& DescArray = GetVarDesc(Struct);
		int32 OldIndex, NewIndex; // populated but unused
		return ComputeIndicesForMove(DescArray, MoveVarGuid, RelativeToGuid, Position, OldIndex, NewIndex);
	}
	return false;
}

void FStructureEditorUtils::ModifyStructData(UUserDefinedStruct* Struct)
{
	UUserDefinedStructEditorData* EditorData = Struct ? Cast<UUserDefinedStructEditorData>(Struct->EditorData) : NULL;
	ensure(EditorData);
	if (EditorData)
	{
		EditorData->Modify();
	}
}

bool FStructureEditorUtils::CanEnableMultiLineText(const UUserDefinedStruct* Struct, FGuid VarGuid)
{
	const FStructVariableDescription* VarDesc = GetVarDescByGuid(Struct, VarGuid);
	if (VarDesc)
	{
		FProperty* Property = FindFProperty<FProperty>(Struct, VarDesc->VarName);

		// If this is an array, we need to test its inner property as that's the real type
		if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
		{
			Property = ArrayProperty->Inner;
		}

		if (Property)
		{
			// Can only set multi-line text on string and text properties
			return Property->IsA(FStrProperty::StaticClass())
				|| Property->IsA(FTextProperty::StaticClass());
		}
	}
	return false;
}

bool FStructureEditorUtils::ChangeMultiLineTextEnabled(UUserDefinedStruct* Struct, FGuid VarGuid, bool bIsEnabled)
{
	FStructVariableDescription* VarDesc = GetVarDescByGuid(Struct, VarGuid);
	if (CanEnableMultiLineText(Struct, VarGuid) && VarDesc->bEnableMultiLineText != bIsEnabled)
	{
		const FScopedTransaction Transaction(LOCTEXT("ChangeMultiLineTextEnabled", "Change Multi-line Text Enabled"));
		ModifyStructData(Struct);

		VarDesc->bEnableMultiLineText = bIsEnabled;
		FProperty* Property = FindFProperty<FProperty>(Struct, VarDesc->VarName);
		if (Property)
		{
			if (VarDesc->bEnableMultiLineText)
			{
				Property->SetMetaData("MultiLine", TEXT("true"));
			}
			else
			{
				Property->RemoveMetaData("MultiLine");
			}
		}
		OnStructureChanged(Struct);
		return true;
	}
	return false;
}

bool FStructureEditorUtils::IsMultiLineTextEnabled(const UUserDefinedStruct* Struct, FGuid VarGuid)
{
	const FStructVariableDescription* VarDesc = GetVarDescByGuid(Struct, VarGuid);
	if (CanEnableMultiLineText(Struct, VarGuid))
	{
		return VarDesc->bEnableMultiLineText;
	}
	return false;
}

bool FStructureEditorUtils::CanEnable3dWidget(const UUserDefinedStruct* Struct, FGuid VarGuid)
{
	const FStructVariableDescription* VarDesc = GetVarDescByGuid(Struct, VarGuid);
	const UStruct* PropertyStruct = VarDesc ? Cast<const UStruct>(VarDesc->SubCategoryObject.Get()) : NULL;
	return FEdMode::CanCreateWidgetForStructure(PropertyStruct);
}

bool FStructureEditorUtils::Change3dWidgetEnabled(UUserDefinedStruct* Struct, FGuid VarGuid, bool bIsEnabled)
{
	FStructVariableDescription* VarDesc = GetVarDescByGuid(Struct, VarGuid);
	if (!VarDesc)
	{
		return false;
	}

	const UStruct* PropertyStruct = Cast<const UStruct>(VarDesc->SubCategoryObject.Get());
	if (FEdMode::CanCreateWidgetForStructure(PropertyStruct) && (VarDesc->bEnable3dWidget != bIsEnabled))
	{
		const FScopedTransaction Transaction(LOCTEXT("Change3dWidgetEnabled", "Change 3d Widget Enabled"));
		ModifyStructData(Struct);

		VarDesc->bEnable3dWidget = bIsEnabled;
		FProperty* Property = FindFProperty<FProperty>(Struct, VarDesc->VarName);
		if (Property)
		{
			if (VarDesc->bEnable3dWidget)
			{
				Property->SetMetaData(FEdMode::MD_MakeEditWidget, TEXT("true"));
			}
			else
			{
				Property->RemoveMetaData(FEdMode::MD_MakeEditWidget);
			}
		}
		return true;
	}
	return false;
}

bool FStructureEditorUtils::Is3dWidgetEnabled(const UUserDefinedStruct* Struct, FGuid VarGuid)
{
	const FStructVariableDescription* VarDesc = GetVarDescByGuid(Struct, VarGuid);
	const UStruct* PropertyStruct = VarDesc ? Cast<const UStruct>(VarDesc->SubCategoryObject.Get()) : nullptr;
	return VarDesc && VarDesc->bEnable3dWidget && FEdMode::CanCreateWidgetForStructure(PropertyStruct);
}

bool FStructureEditorUtils::CanEditValueRange(const UUserDefinedStruct* Struct, FGuid VarGuid)
{
	if (const FStructVariableDescription* VarDesc = GetVarDescByGuid(Struct, VarGuid))
	{
		if (const FProperty* Property = FindFProperty<FProperty>(Struct, VarDesc->VarName))
		{
			return Property->IsA(FNumericProperty::StaticClass());
		}
	}
	return false;
}

bool FStructureEditorUtils::SetMetaData(UUserDefinedStruct* Struct, FGuid VarGuid, FName Key, const FString& Value)
{
	FStructVariableDescription* VarDesc = GetVarDescByGuid(Struct, VarGuid);
	if (!VarDesc)
	{
		return false;
	}

	const FString* CurrentValue = VarDesc->MetaData.Find(Key);

	if (Value.IsEmpty())
	{
		if (!CurrentValue)
		{
			// No new or old value, nothing to do
			return false;
		}

		const FScopedTransaction Transaction(LOCTEXT("RemoveMetaData", "Unset Meta Data"));
		ModifyStructData(Struct);

		VarDesc->MetaData.Remove(Key);
		if (FProperty* Property = FindFProperty<FProperty>(Struct, VarDesc->VarName))
		{
			Property->RemoveMetaData(Key);
		}

		OnStructureChanged(Struct);
		return true;
	}

	if (CurrentValue && CurrentValue->Equals(Value, ESearchCase::CaseSensitive))
	{
		// There's both old and new value, and they are the same, so nothing to do
		return false;
	}

	const FScopedTransaction Transaction(LOCTEXT("ChangeMetaData", "Set Meta Data"));
	ModifyStructData(Struct);

	VarDesc->MetaData.Add(Key, Value);
	if (FProperty* Property = FindFProperty<FProperty>(Struct, VarDesc->VarName))
	{
		Property->SetMetaData(Key, *Value);
	}

	OnStructureChanged(Struct);
	return true;
}

const FString* FStructureEditorUtils::GetMetaData(const UUserDefinedStruct* Struct, FGuid VarGuid, FName Key)
{
	return GetVarDescByGuid(Struct, VarGuid)->MetaData.Find(Key);
}

FGuid FStructureEditorUtils::GetGuidForProperty(const FProperty* Property)
{
	const UUserDefinedStruct* UDStruct = Property ? Cast<const UUserDefinedStruct>(Property->GetOwnerStruct()) : nullptr;
	const FStructVariableDescription* VarDesc = UDStruct ? GetVarDesc(UDStruct).FindByPredicate(FFindByNameHelper<FStructVariableDescription>(Property->GetFName())) : nullptr;
	return VarDesc ? VarDesc->VarGuid : FGuid();
}

FProperty* FStructureEditorUtils::GetPropertyByGuid(const UUserDefinedStruct* Struct, const FGuid VarGuid)
{
	const FStructVariableDescription* VarDesc = GetVarDescByGuid(Struct, VarGuid);
	return VarDesc ? FindFProperty<FProperty>(Struct, VarDesc->VarName) : nullptr;
}

FGuid FStructureEditorUtils::GetGuidFromPropertyName(const FName Name)
{
	return FMemberVariableNameHelper::GetGuidFromName(Name);
}

struct FReinstanceDataTableHelper
{
	// TODO: shell we cache the dependency?
	static TArray<UDataTable*> GetTablesDependentOnStruct(UUserDefinedStruct* Struct)
	{
		TArray<UDataTable*> Result;
		if (Struct)
		{
			TArray<UObject*> DataTables;
			GetObjectsOfClass(UDataTable::StaticClass(), DataTables);
			for (UObject* DataTableObj : DataTables)
			{
				UDataTable* DataTable = Cast<UDataTable>(DataTableObj);
				if (DataTable && (Struct == DataTable->RowStruct))
				{
					Result.Add(DataTable);
				}
			}
		}
		return Result;
	}
};

void FStructureEditorUtils::BroadcastPreChange(UUserDefinedStruct* Struct)
{
	FStructureEditorUtils::FStructEditorManager::Get().PreChange(Struct, FStructureEditorUtils::FStructEditorManager::ActiveChange);
	TArray<UDataTable*> DataTables = FReinstanceDataTableHelper::GetTablesDependentOnStruct(Struct);
	for (UDataTable* DataTable : DataTables)
	{
		DataTable->CleanBeforeStructChange();
	}
}

void FStructureEditorUtils::BroadcastPostChange(UUserDefinedStruct* Struct)
{
	TArray<UDataTable*> DataTables = FReinstanceDataTableHelper::GetTablesDependentOnStruct(Struct);
	for (UDataTable* DataTable : DataTables)
	{
		DataTable->RestoreAfterStructChange();
	}
	FStructureEditorUtils::FStructEditorManager::Get().PostChange(Struct, FStructureEditorUtils::FStructEditorManager::ActiveChange);
}

#undef LOCTEXT_NAMESPACE
