// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet2/WatchedPin.h"

#include "EdGraph/EdGraphNode.h"
#include "Misc/AssertionMacros.h"
#include "Templates/UnrealTemplate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WatchedPin)

FBlueprintWatchedPin::FBlueprintWatchedPin()
{
}

FBlueprintWatchedPin::FBlueprintWatchedPin(const UEdGraphPin* Pin)
{
	SetFromPin(Pin);
}

FBlueprintWatchedPin::FBlueprintWatchedPin(const UEdGraphPin* Pin, TArray<FName>&& InPathToProperty)
	: PathToProperty(MoveTemp(InPathToProperty))
{
	SetFromPin(Pin);
}

UEdGraphPin* FBlueprintWatchedPin::Get() const
{
	UEdGraphPin* FoundPin = CachedPinRef.Get();

	if (!FoundPin && PinId.IsValid() && OwningNode.IsValid())
	{
		const UEdGraphNode* Node = OwningNode.Get();
		check(Node);

		UEdGraphPin* const* FoundPinPtr = Node->Pins.FindByPredicate([PinId = this->PinId](const UEdGraphPin* Pin)
		{
			return PinId == Pin->PinId;
		});

		if (FoundPinPtr)
		{
			FoundPin = *FoundPinPtr;
			CachedPinRef = FoundPin;
		}
	}

	return FoundPin;
}

void FBlueprintWatchedPin::SetFromPin(const UEdGraphPin* Pin)
{
	if (Pin)
	{
		PinId = Pin->PinId;
		OwningNode = Pin->GetOwningNodeUnchecked();
	}

	CachedPinRef = Pin;
}

void FBlueprintWatchedPin::SetFromWatchedPin(FBlueprintWatchedPin&& Other)
{
	PinId = Other.PinId;
	OwningNode = Other.OwningNode;
	CachedPinRef = Other.CachedPinRef;
	PathToProperty = MoveTemp(Other.PathToProperty);
}
