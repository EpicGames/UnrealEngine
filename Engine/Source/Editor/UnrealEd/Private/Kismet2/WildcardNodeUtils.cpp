// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet2/WildcardNodeUtils.h"

#include "Algo/Find.h"
#include "Containers/Array.h"
#include "EdGraphSchema_K2.h"
#include "Misc/AssertionMacros.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtrTemplates.h"

FEdGraphPinType FWildcardNodeUtils::GetDefaultWildcardPinType()
{
	static FEdGraphPinType WildcardPinType(
		/* InPinCategory */ UEdGraphSchema_K2::PC_Wildcard,
		/* InPinSubCategory */ NAME_None,
		/* InPinSubCategoryObject */ nullptr,
		/* InPinContainerType */ EPinContainerType::None,
		/* bInIsReference */ false,
		/* InValueTerminalType */{}
	);

	return WildcardPinType;
}

bool FWildcardNodeUtils::IsWildcardPin(const UEdGraphPin* const Pin)
{
	return Pin && Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard;
}

bool FWildcardNodeUtils::IsWildcardPin(const FEdGraphTerminalType& Terminal)
{
	return Terminal.TerminalCategory == UEdGraphSchema_K2::PC_Wildcard;
}

bool FWildcardNodeUtils::HasAnyWildcards(const UEdGraphPin* const Pin)
{
	check(Pin);
	return Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard ||
		Pin->PinType.PinValueType.TerminalCategory == UEdGraphSchema_K2::PC_Wildcard;
}

bool FWildcardNodeUtils::HasAnyNonWildcards(const UEdGraphPin* Pin)
{
	check(Pin);
	return Pin->PinType.PinCategory != UEdGraphSchema_K2::PC_Wildcard ||
		Pin->PinType.PinValueType.TerminalCategory != UEdGraphSchema_K2::PC_Wildcard;
}

const UEdGraphPin* FWildcardNodeUtils::FindInferrableLinkedPin( const UEdGraphPin* ForPin)
{
	// first, look for pins with no wildcards to infer from:
	UEdGraphPin *const* InferrablePin = Algo::FindByPredicate(ForPin->LinkedTo, 
		[](const UEdGraphPin* Link)
		{
			return !FWildcardNodeUtils::HasAnyWildcards(Link);
		});
	if(InferrablePin == nullptr)
	{
		// no pins with no wildcards - are there any pins that have
		// any non wildcards?
		InferrablePin = Algo::FindByPredicate(ForPin->LinkedTo, 
			[](const UEdGraphPin* Link)
			{
				return FWildcardNodeUtils::HasAnyNonWildcards(Link);
			});
	}

	return InferrablePin ? *InferrablePin : nullptr;
}

bool FWildcardNodeUtils::IsLinkedToWildcard(const UEdGraphPin* const Pin)
{
	if (Pin)
	{
		for (const UEdGraphPin* const LinkedPin : Pin->LinkedTo)
		{
			if (FWildcardNodeUtils::IsWildcardPin(LinkedPin))
			{
				return true;
			}
		}
	}
	return false;
}

UEdGraphPin* FWildcardNodeUtils::CreateWildcardPin(UEdGraphNode* Node, const FName PinName, const EEdGraphPinDirection Direction, const EPinContainerType ContainerType/* = EPinContainerType::None*/)
{
	check(Node);

	FEdGraphPinType PinType;
	PinType.PinCategory = UEdGraphSchema_K2::PC_Wildcard;
	PinType.PinSubCategory = NAME_None;
	PinType.PinSubCategoryObject = nullptr;
	PinType.bIsReference = false;
	PinType.ContainerType = ContainerType;

	return Node->CreatePin(Direction, PinType, PinName);
}

bool FWildcardNodeUtils::NodeHasAnyWildcards(const UEdGraphNode* const Node)
{
	check(Node);

	for (const UEdGraphPin* const Pin : Node->Pins)
	{
		if (FWildcardNodeUtils::IsWildcardPin(Pin))
		{
			return true;
		}
	}

	return false;
}

void FWildcardNodeUtils::InferType(UEdGraphPin* ToPin, const FEdGraphPinType& Type)
{
	InferType(ToPin->PinType, Type);
}

void FWildcardNodeUtils::InferType(FEdGraphPinType& ToType, const FEdGraphPinType& Type)
{
	bool bInferredSomething = false;
	if (ToType.PinCategory == UEdGraphSchema_K2::PC_Wildcard)
	{
		bInferredSomething = true;
		ToType.PinCategory = Type.PinCategory;
		ToType.PinSubCategory = Type.PinSubCategory;
		ToType.PinSubCategoryObject = Type.PinSubCategoryObject;
		ToType.bIsUObjectWrapper = Type.bIsUObjectWrapper;
	}
	if (ToType.PinValueType.TerminalCategory == UEdGraphSchema_K2::PC_Wildcard)
	{
		bInferredSomething = true;
		ToType.PinValueType = Type.PinValueType;
	}
	check(bInferredSomething);
}

void FWildcardNodeUtils::InferType(FEdGraphPinType& ToType, const FEdGraphTerminalType& Type)
{
	bool bInferredSomething = false;
	if (ToType.PinCategory == UEdGraphSchema_K2::PC_Wildcard)
	{
		bInferredSomething = true;
		ToType.PinCategory = Type.TerminalCategory;
		ToType.PinSubCategory = Type.TerminalSubCategory;
		ToType.PinSubCategoryObject = Type.TerminalSubCategoryObject;
		ToType.bIsUObjectWrapper = Type.bTerminalIsUObjectWrapper;
	}
	check(bInferredSomething);
}

void FWildcardNodeUtils::ResetToWildcard(UEdGraphPin* Pin)
{
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	check(Pin);
	if (Pin->SubPins.Num() > 0)
	{
		Schema->RecombinePin(Pin->SubPins[0]);
	}

	ResetToWildcard(Pin->PinType);
	Schema->ResetPinToAutogeneratedDefaultValue(Pin, false);
}

void FWildcardNodeUtils::ResetToWildcard(FEdGraphPinType& PinType)
{
	PinType.PinCategory = UEdGraphSchema_K2::PC_Wildcard;
	PinType.PinSubCategory = NAME_None;
	PinType.PinSubCategoryObject = nullptr;
	PinType.bIsUObjectWrapper = false;
	PinType.PinValueType.TerminalCategory = UEdGraphSchema_K2::PC_Wildcard;
	PinType.PinValueType.TerminalSubCategory = NAME_None;
	PinType.PinValueType.TerminalSubCategoryObject = nullptr;
	PinType.PinValueType.bTerminalIsUObjectWrapper = false;
}

