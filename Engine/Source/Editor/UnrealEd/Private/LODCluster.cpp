// Copyright Epic Games, Inc. All Rights Reserved.

#include "LODCluster.h"
#include "Modules/ModuleManager.h"
#include "Engine/World.h"
#include "Components/StaticMeshComponent.h"
#include "GameFramework/Volume.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Engine/HLODProxy.h"

#if WITH_EDITOR
#include "Engine/LODActor.h"
#include "GameFramework/WorldSettings.h"
#include "IHierarchicalLODUtilities.h"
#include "HierarchicalLODUtilitiesModule.h"
#endif // WITH_EDITOR


#define LOCTEXT_NAMESPACE "LODCluster"
#define CM_TO_METER		0.01f
#define METER_TO_CM		100.0f



/** Utility function to calculate overlap of two spheres */
double CalculateOverlap(const FSphere& ASphere, const double AFillingFactor, const FSphere& BSphere, const double BFillingFactor)
{
	// if it doesn't intersect, return zero 
	if (!ASphere.Intersects(BSphere))
	{
		return 0.f;
	}

	if (ASphere.IsInside(BSphere))
	{
		return ASphere.GetVolume();
	}

	if(BSphere.IsInside(ASphere))
	{
		return BSphere.GetVolume();
	}

	if (ASphere.Equals(BSphere))
	{
		return ASphere.GetVolume();
	}

	double Distance = (ASphere.Center - BSphere.Center).Size();
	check(!FMath::IsNearlyZero(Distance));

	double ARadius = ASphere.W;
	double BRadius = BSphere.W;

	double ACapHeight = (BRadius * BRadius - (ARadius - Distance) * (ARadius - Distance)) / (2 * Distance);
	double BCapHeight = (ARadius * ARadius - (BRadius - Distance) * (BRadius - Distance)) / (2 * Distance);

	if ((ACapHeight <= 0.f) || (BCapHeight <= 0.f))
	{
		// it's possible to get cap height to be less than 0 
		// since when we do check intersect, we do have regular tolerance
		return 0.f;		
	}

	double OverlapRadius1 = ((ARadius + BRadius) * (ARadius + BRadius) - Distance*Distance) * (Distance * Distance - (ARadius - BRadius) * (ARadius - BRadius));
	double OverlapRadius2 = 2 * Distance;
	double OverlapRadius = FMath::Sqrt(OverlapRadius1) / OverlapRadius2;
	double OverlapRadiusSq = FMath::Square(OverlapRadius);

	double ConstPI = UE_PI / 6.0f;
	double AVolume = ConstPI * (3 * OverlapRadiusSq + ACapHeight * ACapHeight) * ACapHeight;
	double BVolume = ConstPI * (3 * OverlapRadiusSq + BCapHeight * BCapHeight) * BCapHeight;

	double TotalVolume = AFillingFactor * AVolume + BFillingFactor * BVolume;
	return TotalVolume;
}

/** Utility function that calculates filling factor */
double CalculateFillingFactor(const FSphere& ASphere, const double AFillingFactor, const FSphere& BSphere, const double BFillingFactor)
{
	const double OverlapVolume = CalculateOverlap( ASphere, AFillingFactor, BSphere, BFillingFactor);
	FSphere UnionSphere = ASphere + BSphere;
	// it shouldn't be zero or it should be checked outside
	ensure(UnionSphere.W != 0.f);

	// http://deim.urv.cat/~rivi/pub/3d/icra04b.pdf
	// cost is calculated based on r^3 / filling factor
	// since it subtract by AFillingFactor * 1/2 overlap volume + BfillingFactor * 1/2 overlap volume
	return FMath::Max(0.0f, (AFillingFactor * ASphere.GetVolume() + BFillingFactor * BSphere.GetVolume() - OverlapVolume) / UnionSphere.GetVolume());
}

FLODCluster::FLODCluster(const FLODCluster& Other)
: Actors(Other.Actors)
, Bound(Other.Bound)
, FillingFactor(Other.FillingFactor)
, ClusterCost(Other.ClusterCost)
, bValid(Other.bValid)
{
	
}

FLODCluster::FLODCluster(FLODCluster&& Other)
	: Actors(Other.Actors)
	, Bound(Other.Bound)
	, FillingFactor(Other.FillingFactor)
	, ClusterCost(Other.ClusterCost)
	, bValid(Other.bValid)
{

}

FLODCluster::FLODCluster(AActor* Actor1)
: Bound(ForceInit)
, bValid(true)
{
	AddActor(Actor1);
	// calculate new filling factor
	FillingFactor = 1.f;	
	ClusterCost = (Bound.W * Bound.W * Bound.W);
}

FLODCluster::FLODCluster(AActor* Actor1, AActor* Actor2)
: Bound(ForceInit)
, bValid(true)
{
	FSphere Actor1Bound = AddActor(Actor1);
	FSphere Actor2Bound = AddActor(Actor2);
	
	// calculate new filling factor
	FillingFactor = CalculateFillingFactor(Actor1Bound, 1.f, Actor2Bound, 1.f);	
	ClusterCost = ( Bound.W * Bound.W * Bound.W ) / FillingFactor;
}

FLODCluster::FLODCluster()
: Bound(ForceInit)
, bValid(false)
{
	FillingFactor = 1.0f;
	ClusterCost = (Bound.W * Bound.W * Bound.W);
}

FSphere FLODCluster::AddActor(AActor* NewActor)
{
	bValid = true;

	Actors.Add(NewActor);
	FVector Origin, Extent;

	NewActor->GetActorBounds(false, Origin, Extent);

	// scale 0.01 (change to meter from centimeter)
	FSphere NewBound = FSphere(Origin*CM_TO_METER, Extent.Size()*CM_TO_METER);
	Bound += NewBound;

	return NewBound;
}

FLODCluster FLODCluster::operator+(const FLODCluster& Other) const
{
	FLODCluster UnionCluster(*this);
	UnionCluster.MergeClusters(Other);
	return UnionCluster;
}

FLODCluster& FLODCluster::operator+=(const FLODCluster& Other)
{
	MergeClusters(Other);
	return *this;
}

FLODCluster FLODCluster::operator-(const FLODCluster& Other) const
{
	FLODCluster Cluster(*this);
	Cluster.SubtractCluster(Other);
	return Cluster;
}

FLODCluster& FLODCluster::operator-=(const FLODCluster& Other)
{
	SubtractCluster(Other);
	return *this;
}

FLODCluster& FLODCluster::operator=(const FLODCluster& Other)
{
	this->bValid		= Other.bValid;
	this->Actors		= Other.Actors;
	this->Bound			= Other.Bound;
	this->FillingFactor = Other.FillingFactor;	
	this->ClusterCost = Other.ClusterCost;

	return *this;
}

FLODCluster& FLODCluster::operator=(FLODCluster&& Other)
{
	this->bValid = Other.bValid;
	this->Actors = Other.Actors;
	this->Bound = Other.Bound;
	this->FillingFactor = Other.FillingFactor;
	this->ClusterCost = Other.ClusterCost;

	return *this;
}

bool FLODCluster::operator==(const FLODCluster& Other) const
{
	return Actors.Num() == Other.Actors.Num() && Actors.Includes(Other.Actors);
}

double FLODCluster::GetMergedCost(const FLODCluster& Other) const
{
	double MergedFillingFactor = CalculateFillingFactor(Bound, FillingFactor, Other.Bound, Other.FillingFactor);
	FSphere MergedBound = Bound + Other.Bound;

	double MergedClusterCost = (MergedBound.W * MergedBound.W * MergedBound.W) / MergedFillingFactor;
	return MergedClusterCost;
}

void FLODCluster::MergeClusters(const FLODCluster& Other)
{
	// please note that when merge, we merge two boxes from each cluster, not exactly all actors' bound
	// have to recalculate filling factor and bound based on cluster data
	FillingFactor = CalculateFillingFactor(Bound, FillingFactor, Other.Bound, Other.FillingFactor);
	Bound += Other.Bound;	

	ClusterCost = ( Bound.W * Bound.W * Bound.W ) / FillingFactor;
	
	Actors.Append(Other.Actors);

	if (Actors.Num() > 0)
	{
		bValid = true;
	}
}

void FLODCluster::SubtractCluster(const FLODCluster& Other)
{
	Actors = Actors.Difference(Other.Actors);

	Invalidate();

	// We need to recalculate parameters
	if (Actors.Num() > 0)
	{
		bValid = true;
		FillingFactor = 1.f;
		Bound = FSphere(ForceInitToZero);

		for (AActor* Actor : Actors)
		{
			FVector Origin, Extent;
			Actor->GetActorBounds(false, Origin, Extent);

			// scale 0.01 (change to meter from centimeter)
			FSphere NewBound = FSphere(Origin * CM_TO_METER, Extent.Size() * CM_TO_METER);

			FillingFactor = CalculateFillingFactor(NewBound, 1.f, Bound, FillingFactor);
			Bound += NewBound;
		}

		ClusterCost = (Bound.W * Bound.W * Bound.W) / FillingFactor;
	}
}

bool FLODCluster::Contains(FLODCluster& Other) const
{
	if (IsValid() && Other.IsValid())
	{
		for(auto& Actor: Other.Actors)
		{
			if(Actors.Contains(Actor))
			{
				return true;
			}
		}
	}

	return false;
}

FString FLODCluster::ToString() const
{
	FString ActorList;
	for (auto& Actor: Actors)
	{
		ActorList += Actor->GetActorLabel();
		ActorList += ", ";
	}

	return FString::Printf(TEXT("ActorNum(%d), Actor List (%s)"), Actors.Num(), *ActorList);
}

#undef LOCTEXT_NAMESPACE 
