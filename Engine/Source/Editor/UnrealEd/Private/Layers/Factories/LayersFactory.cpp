// Copyright Epic Games, Inc. All Rights Reserved.

#include "LayersFactory.h"

#include "DataStorage/Features.h"
#include "Editor/UnrealEd/Private/Layers/Widgets/ActorLayersWidget.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Layers/Layer.h"
#include "Layers/Columns/LayersColumns.h"

namespace UE::Editor::Layers::Private
{
	static FName TableName(TEXT("LayersTable"));
}

void UEditorDataStorageLayersFactory::RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::Layers;
	using namespace UE::Editor::DataStorage;
	
	LayersTableHandle = DataStorage.RegisterTable<FLayerTag, FLayerNameColumn>(UE::Editor::Layers::Private::TableName);
	
	if (ICompatibilityProvider* TedsCompat = GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName))
	{
		TedsCompat->RegisterTypeTableAssociation(ULayer::StaticClass(), LayersTableHandle);
	}
}

void UEditorDataStorageLayersFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;
	
	DataStorage.RegisterQuery(
		Select(
			TEXT("Sync column to layer name"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
				.SetExecutionMode(EExecutionMode::GameThread),
			[](const FTypedElementUObjectColumn& Layer, FEditorDataStorageLayerNameColumn& NameColumn)
			{
				if (const ULayer* LayerInstance = Cast<ULayer>(Layer.Object))
				{
					NameColumn.LayerName = LayerInstance->GetLayerName();
				}
			}
		)
		.Where()
			.All<FEditorDataStorageLayerTag, FTypedElementSyncFromWorldTag>()
		.Compile());
	
	DataStorage.RegisterQuery(
		Select(
			TEXT("Sync layer name to column"),
			FProcessor(EQueryTickPhase::PostPhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncDataStorageToExternal))
				.SetExecutionMode(EExecutionMode::GameThread),
			[](FTypedElementUObjectColumn& Layer, const FEditorDataStorageLayerNameColumn& NameColumn)
			{
				if (ULayer* LayerInstance = Cast<ULayer>(Layer.Object))
				{
					LayerInstance->SetLayerName(NameColumn.LayerName);
				}
			}
		)
		.Where()
			.All<FEditorDataStorageLayerTag, FTypedElementSyncBackToWorldTag>()
		.Compile());
}

void UEditorDataStorageLayersFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	using namespace UE::Editor::Layers;

	DataStorageUi.RegisterWidgetFactory<FActorLayersWidgetConstructor>(DataStorageUi.FindPurpose(DataStorageUi.GetGeneralWidgetPurposeID()), TColumn<FActorLayersColumn>());
}

UE::Editor::DataStorage::TableHandle UEditorDataStorageLayersFactory::GetLayersTableHandle() const
{
	return LayersTableHandle;
}