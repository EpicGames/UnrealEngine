// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementHandle.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"

#include "LayersFactory.generated.h"

/**
 */
UCLASS(Transient)
class UEditorDataStorageLayersFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()
public:

	virtual void RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
	
	UE::Editor::DataStorage::TableHandle GetLayersTableHandle() const;
	
private:
	UE::Editor::DataStorage::TableHandle LayersTableHandle;
};


