// Copyright Epic Games, Inc. All Rights Reserved.

#include "Layers/LayersSubsystem.h"
#include "Engine/Brush.h"
#include "Components/PrimitiveComponent.h"
#include "Layers/Layer.h"
#include "LevelEditorViewport.h"
#include "Misc/IFilter.h"
#include "Engine/Selection.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "Editor.h"

#include "ActorEditorUtils.h"
#include "DataStorage/Features.h"
#include "Layers/Columns/LayersColumns.h"
#include "Model.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LayersSubsystem)

class FLayersBroadcast
{
public:

	/**
	 *	Constructor
	 */
	FLayersBroadcast(ULayersSubsystem* InLayersSubsystem);

	/**
	 *	Destructor
	 */
	~FLayersBroadcast();

	void Deinitialize();

private:
	void Initialize();

	/**
	 * Delegate handler for FEditorDelegates::MapChange. It internally calls LayersSubsystem->EditorMapChange().
	 **/
	void OnEditorMapChange(uint32 MapChangeFlags = 0);
	/**
	 * Delegate handler for FEditorDelegates::RefreshLayerBrowser. It internally calls LayersSubsystem->EditorRefreshLayerBrowser() to refresh the actors of each layer.
	 **/
	void OnEditorRefreshLayerBrowser();
	/**
	 * Delegate handler for FEditorDelegates::PostUndoRedo. It internally calls LayersSubsystem->PostUndoRedo() to refresh the actors of each layer.
	 **/
	void OnPostUndoRedo();

	ULayersSubsystem* LayersSubsystem;

	bool bIsInitialized;
};

namespace UE::Editor::Layers::Private
{
	static bool bRegisterLayersInTeds = true;
	
	FAutoConsoleVariableRef CVarRegisterLayersInTeds(
		TEXT("TEDS.Feature.Layers"),
		bRegisterLayersInTeds,
		TEXT("Registers Layers in TEDS"),
		ECVF_ReadOnly);
}



FLayersBroadcast::FLayersBroadcast(ULayersSubsystem* InLayersSubsystem)
	: LayersSubsystem(InLayersSubsystem)
	, bIsInitialized(false)
{
	Initialize();
}

FLayersBroadcast::~FLayersBroadcast()
{
	Deinitialize();
}

void FLayersBroadcast::Deinitialize()
{
	if (bIsInitialized)
	{
		bIsInitialized = false;
		// Remove all callback functions from FEditorDelegates::MapChange.Broadcast() and FEditorDelegates::RefreshLayerBrowser.Broadcast()
		FEditorDelegates::MapChange.RemoveAll(this);
		FEditorDelegates::RefreshLayerBrowser.RemoveAll(this);
		FEditorDelegates::PostUndoRedo.RemoveAll(this);
	}
}

void FLayersBroadcast::Initialize()
{
	if (!bIsInitialized)
	{
		bIsInitialized = true;
		// Add callback function to FEditorDelegates::MapChange.Broadcast() and FEditorDelegates::RefreshLayerBrowser.Broadcast()
		FEditorDelegates::MapChange.AddRaw(this, &FLayersBroadcast::OnEditorMapChange);
		FEditorDelegates::RefreshLayerBrowser.AddRaw(this, &FLayersBroadcast::OnEditorRefreshLayerBrowser);
		FEditorDelegates::PostUndoRedo.AddRaw(this, &FLayersBroadcast::OnPostUndoRedo);
	}
}

void FLayersBroadcast::OnEditorMapChange(uint32 MapChangeFlags)
{
	LayersSubsystem->EditorMapChange();
}

void FLayersBroadcast::OnEditorRefreshLayerBrowser()
{
	LayersSubsystem->EditorRefreshLayerBrowser();
}

void FLayersBroadcast::OnPostUndoRedo()
{
	LayersSubsystem->PostUndoRedo();
}

void ULayersSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	// Subsystems are loaded, so we can set up the broadcast functions for GetEditorSubsystem<ULayersSubsystem>()
	LayersBroadcast = MakeShareable(new FLayersBroadcast(this));
	
	if (UE::Editor::Layers::Private::bRegisterLayersInTeds)
	{
		using namespace UE::Editor::DataStorage;

		auto OnDataStorage = [this]
			{
				DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
				DataStorageCompat = GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);
			};

		if (AreEditorDataStorageFeaturesEnabled())
		{
			OnDataStorage();
		}
		else
		{
			OnDataStorageInitialized = OnEditorDataStorageFeaturesEnabled().AddLambda(OnDataStorage);
		}
	}
}

void ULayersSubsystem::Deinitialize()
{
	Super::Deinitialize();
	
	if (UE::Editor::Layers::Private::bRegisterLayersInTeds)
	{
		UE::Editor::DataStorage::OnEditorDataStorageFeaturesEnabled().Remove(OnDataStorageInitialized);
	}
	
	LayersBroadcast->Deinitialize();
}

ULayersSubsystem::~ULayersSubsystem()
{
}

void ULayersSubsystem::EditorMapChange()
{
	LayersChanged.Broadcast( ELayersAction::Reset, NULL, NAME_None );
}

void ULayersSubsystem::EditorRefreshLayerBrowser()
{
	// bNotifySelectionChange is false because the functions calling FEditorDelegates::RefreshLayerBrowser.Broadcast usually call GEditor->NoteSelectionChange
	const bool bNotifySelectionChange = false;
	const bool bRedrawViewports = false;
	UpdateAllActorsVisibility(bNotifySelectionChange, bRedrawViewports);
}

void ULayersSubsystem::PostUndoRedo()
{
	LayersChanged.Broadcast(ELayersAction::Reset, NULL, NAME_None);
	UpdateAllActorsVisibility_Internal(true, true, true);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// Operations on Levels
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void ULayersSubsystem::AddLevelLayerInformation(ULevel* Level)
{
	if (Level)
	{
		for (auto ActorIter = Level->Actors.CreateConstIterator(); ActorIter; ++ActorIter)
		{
			InitializeNewActorLayers(*ActorIter);
		}
	}
}

void ULayersSubsystem::RemoveLevelLayerInformation(ULevel* Level)
{
	if (Level)
	{
		for (auto ActorIter = Level->Actors.CreateConstIterator(); ActorIter; ++ActorIter)
		{
			DisassociateActorFromLayers(*ActorIter);
		}
	}
}


///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// Operations on an individual actor.
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

bool ULayersSubsystem::IsActorValidForLayer(AActor* Actor)
{
	return Actor && Actor->GetClass() && Actor->GetWorld() && Actor->SupportsLayers();
}

bool ULayersSubsystem::InitializeNewActorLayers(AActor* Actor)
{
	if(	!IsActorValidForLayer( Actor ) )
	{
		return false;
	}

	for( auto LayerNameIt = Actor->Layers.CreateConstIterator(); LayerNameIt; ++LayerNameIt )
	{
		const FName LayerName = *LayerNameIt;
		ULayer* Layer = EnsureLayerExists( LayerName );

		Layer->Modify();
		AddActorToStats( Layer, Actor);
		AddActorToLayerInTeds(Layer, Actor);
	}

	// update per-view visibility info
	UpdateActorAllViewsVisibility(Actor);

	// update general actor visibility
	bool bActorModified = false;
	bool bActorSelectionChanged = false;
	const bool bActorNotifySelectionChange = true;
	const bool bActorRedrawViewports = false;
	UpdateActorVisibility(Actor, bActorSelectionChanged, bActorModified, bActorNotifySelectionChange, bActorRedrawViewports);

	return Actor->Layers.Num() > 0;
}

bool ULayersSubsystem::DisassociateActorFromLayers(AActor* Actor)
{
	TArray< AActor* > Actors;
	Actors.Add(Actor);

	return DisassociateActorsFromLayers(Actors);
}


UWorld* ULayersSubsystem::GetWorld() const
{
	return GWorld;
}


bool ULayersSubsystem::AddActorToLayer(AActor* Actor, const FName& LayerName)
{
	TArray< AActor* > Actors;
	Actors.Add(Actor);

	TArray< FName > LayerNames;
	LayerNames.Add(LayerName);

	return AddActorsToLayers(Actors, LayerNames);
}

bool ULayersSubsystem::AddActorToLayers(AActor* Actor, const TArray< FName >& LayerNames)
{
	TArray< AActor* > Actors;
	Actors.Add(Actor);

	return AddActorsToLayers(Actors, LayerNames);
}

bool ULayersSubsystem::AddActorsToLayer(const TArray< AActor* >& Actors, const FName& LayerName)
{
	TArray< FName > LayerNames;
	LayerNames.Add(LayerName);

	return AddActorsToLayers(Actors, LayerNames);
}


bool ULayersSubsystem::AddActorsToLayer(const TArray< TWeakObjectPtr< AActor > >& Actors, const FName& LayerName)
{
	TArray< FName > LayerNames;
	LayerNames.Add(LayerName);

	return AddActorsToLayers(Actors, LayerNames);
}

bool ULayersSubsystem::AddActorsToLayers( const TArray< AActor* >& Actors, const TArray< FName >& LayerNames )
{
	bool bChangesOccurred = false;

	if ( LayerNames.Num() > 0 ) 
	{
		GEditor->GetSelectedActors()->BeginBatchSelectOperation();

		for( auto ActorIt = Actors.CreateConstIterator(); ActorIt; ++ActorIt )
		{
			AActor* Actor = *ActorIt;

			if ( !IsActorValidForLayer( Actor ) )
			{
				continue;
			}

			bool bActorWasModified = false;
			for( auto LayerNameIt = LayerNames.CreateConstIterator(); LayerNameIt; ++LayerNameIt )
			{
				const FName& LayerName = *LayerNameIt;

				if( !Actor->Layers.Contains( LayerName ) )
				{
					if( !bActorWasModified )
					{
						Actor->Modify();
						bActorWasModified = true;
					}

					ULayer* Layer = EnsureLayerExists( LayerName );
					Actor->Layers.Add( LayerName );

					Layer->Modify();
					AddActorToStats( Layer, Actor);
					AddActorToLayerInTeds(Layer, Actor);

					ActorsLayersChanged.Broadcast( Actor );
				}
			} //END Iteration over Layers

			if( bActorWasModified )
			{
				// update per-view visibility info
				UpdateActorAllViewsVisibility(Actor);

				// update general actor visibility
				bool bActorModified = false;
				bool bActorSelectionChanged = false;
				const bool bActorNotifySelectionChange = true;
				const bool bActorRedrawViewports = false;
				UpdateActorVisibility( Actor, bActorSelectionChanged, bActorModified, bActorNotifySelectionChange, bActorRedrawViewports );

				bChangesOccurred = true;
			}
		} //END Iteration over Actors

		GEditor->GetSelectedActors()->EndBatchSelectOperation();
	}

	return bChangesOccurred;
}

bool ULayersSubsystem::AddActorsToLayers(const TArray< TWeakObjectPtr< AActor > >& Actors, const TArray< FName >& LayerNames)
{
	TArray< AActor* > ActorsRawPtr;
	for (auto ActorIt = Actors.CreateConstIterator(); ActorIt; ++ActorIt)
	{
		AActor* Actor = (*ActorIt).Get();
		ActorsRawPtr.Add(Actor);
	}

	return AddActorsToLayers(ActorsRawPtr, LayerNames);
}

bool ULayersSubsystem::DisassociateActorsFromLayers(const TArray<AActor*>& Actors)
{
	bool bChangesOccurred = false;
	
	for(AActor* Actor : Actors)
	{
		if (!IsActorValidForLayer(Actor))
		{
			continue;
		}

		for(const FName& LayerName : Actor->Layers)
		{
			ULayer* Layer = EnsureLayerExists(LayerName);

			Layer->Modify();
			RemoveActorFromStats(Layer, Actor);
			RemoveActorFromLayerInTeds(Layer, Actor);
			bChangesOccurred = true;
		}
	}

	return bChangesOccurred;
}

bool ULayersSubsystem::RemoveActorFromLayer(AActor* Actor, const FName& LayerName, const bool bUpdateStats)
{
	TArray< AActor* > Actors;
	Actors.Add(Actor);

	TArray< FName > LayerNames;
	LayerNames.Add(LayerName);

	return RemoveActorsFromLayers(Actors, LayerNames, bUpdateStats);
}

bool ULayersSubsystem::RemoveActorFromLayers(AActor* Actor, const TArray< FName >& LayerNames, const bool bUpdateStats)
{
	TArray< AActor* > Actors;
	Actors.Add(Actor);

	return RemoveActorsFromLayers(Actors, LayerNames, bUpdateStats);
}

bool ULayersSubsystem::RemoveActorsFromLayer(const TArray< AActor* >& Actors, const FName& LayerName, const bool bUpdateStats)
{
	TArray< FName > LayerNames;
	LayerNames.Add(LayerName);

	return RemoveActorsFromLayers(Actors, LayerNames, bUpdateStats);
}

bool ULayersSubsystem::RemoveActorsFromLayer(const TArray< TWeakObjectPtr< AActor > >& Actors, const FName& LayerName, const bool bUpdateStats)
{
	TArray< FName > LayerNames;
	LayerNames.Add(LayerName);

	return RemoveActorsFromLayers(Actors, LayerNames, bUpdateStats);
}

bool ULayersSubsystem::RemoveActorsFromLayers( const TArray< AActor* >& Actors, const TArray< FName >& LayerNames, const bool bUpdateStats )
{
	GEditor->GetSelectedActors()->BeginBatchSelectOperation();

	bool bChangesOccurred = false;
	for( auto ActorIt = Actors.CreateConstIterator(); ActorIt; ++ActorIt )
	{
		AActor* Actor = *ActorIt;

		if ( !IsActorValidForLayer( Actor ) )
		{
			continue;
		}

		bool ActorWasModified = false;
		for( auto LayerNameIt = LayerNames.CreateConstIterator(); LayerNameIt; ++LayerNameIt )
		{
			const FName& LayerName = *LayerNameIt;
			if( Actor->Layers.Contains( LayerName ) )
			{
				if( !ActorWasModified )
				{
					Actor->Modify();
					ActorWasModified = true;
				}

				Actor->Layers.Remove( LayerName );

				ULayer* Layer;
				if (TryGetLayer( LayerName, Layer ))
				{
					if( bUpdateStats)
					{
						Layer->Modify();
						RemoveActorFromStats( Layer, Actor);
					}
					
					RemoveActorFromLayerInTeds(Layer, Actor);
				}
				
				ActorsLayersChanged.Broadcast( Actor );
			}
		} //END Iteration over Layers

		if( ActorWasModified )
		{
			// update per-view visibility info
			UpdateActorAllViewsVisibility(Actor);

			// update general actor visibility
			bool bActorModified = false;
			bool bActorSelectionChanged = false;
			const bool bActorNotifySelectionChange = true;
			const bool bActorRedrawViewports = false;
			UpdateActorVisibility( Actor, bActorSelectionChanged, bActorModified, bActorNotifySelectionChange, bActorRedrawViewports );

			bChangesOccurred = true;
		}
	} //END Iteration over Actors

	GEditor->GetSelectedActors()->EndBatchSelectOperation();

	return bChangesOccurred;
}

bool ULayersSubsystem::RemoveActorsFromLayers(const TArray< TWeakObjectPtr< AActor > >& Actors, const TArray< FName >& LayerNames, const bool bUpdateStats)
{
	TArray< AActor* > ActorsRawPtr;
	for (auto ActorIt = Actors.CreateConstIterator(); ActorIt; ++ActorIt)
	{
		AActor* Actor = (*ActorIt).Get();
		ActorsRawPtr.Add(Actor);
	}

	return RemoveActorsFromLayers(ActorsRawPtr, LayerNames, bUpdateStats);
}


///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// Operations on selected actors.
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
TArray< AActor* > ULayersSubsystem::GetSelectedActors() const
{
	// Unfortunately, the batch selection operation is not entirely effective
	// and the result can be that the iterator becomes invalid when adding an actor to a layer
	// due to unintended selection change notifications being fired.
	TArray< AActor* > CurrentlySelectedActors;
	for( FSelectionIterator It(GEditor->GetSelectedActorIterator() ); It; ++It )
	{
		AActor* Actor = static_cast<AActor*>(*It);
		checkSlow( Actor->IsA(AActor::StaticClass()) );

		CurrentlySelectedActors.Add( Actor );
	}

	return CurrentlySelectedActors;
}

bool ULayersSubsystem::AddSelectedActorsToLayer( const FName& LayerName )
{
	return AddActorsToLayer( GetSelectedActors(), LayerName );
}

bool ULayersSubsystem::RemoveSelectedActorsFromLayer( const FName& LayerName )
{
	return RemoveActorsFromLayer( GetSelectedActors(), LayerName );
}

bool ULayersSubsystem::AddSelectedActorsToLayers( const TArray< FName >& LayerNames )
{
	return AddActorsToLayers( GetSelectedActors(), LayerNames );
}

bool ULayersSubsystem::RemoveSelectedActorsFromLayers( const TArray< FName >& LayerNames )
{
	return RemoveActorsFromLayers( GetSelectedActors(), LayerNames );
}


///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// Operations on actors in layers
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
bool ULayersSubsystem::SelectActorsInLayer(const FName& LayerName, const bool bSelect, const bool bNotify, const bool bSelectEvenIfHidden)
{
	return SelectActorsInLayer(LayerName, bSelect, bNotify, bSelectEvenIfHidden, nullptr);
}

bool ULayersSubsystem::SelectActorsInLayer(const FName& LayerName, const bool bSelect, const bool bNotify, const bool bSelectEvenIfHidden, const TSharedPtr< ActorFilter >& Filter)
{
	GEditor->GetSelectedActors()->BeginBatchSelectOperation();
	bool bChangesOccurred = false;
	// Iterate over all actors, looking for actors in the specified layers.
	for (FActorIterator It(GetWorld()); It; ++It)
	{
		AActor* Actor = *It;
		if (!IsActorValidForLayer(Actor))
		{
			continue;
		}

		if (Filter.IsValid() && !Filter->PassesFilter(Actor))
		{
			continue;
		}

		if (Actor->Layers.Contains(LayerName))
		{
			// The actor was found to be in a specified layer.
			// Set selection state and move on to the next actor.
			bool bNotifyForActor = false;
			GEditor->GetSelectedActors()->Modify();
			GEditor->SelectActor(Actor, bSelect, bNotifyForActor, bSelectEvenIfHidden);
			bChangesOccurred = true;
		}
	}

	GEditor->GetSelectedActors()->EndBatchSelectOperation();

	if (bNotify)
	{
		GEditor->NoteSelectionChange();
	}

	return bChangesOccurred;
}


bool ULayersSubsystem::SelectActorsInLayers(const TArray< FName >& LayerNames, const bool bSelect, const bool bNotify, const bool bSelectEvenIfHidden)
{
	return SelectActorsInLayers(LayerNames, bSelect, bNotify, bSelectEvenIfHidden, nullptr);
}

bool ULayersSubsystem::SelectActorsInLayers(const TArray< FName >& LayerNames, const bool bSelect, const bool bNotify, const bool bSelectEvenIfHidden, const TSharedPtr< ActorFilter >& Filter)
{
	if (LayerNames.Num() == 0)
	{
		return true;
	}

	GEditor->GetSelectedActors()->BeginBatchSelectOperation();
	bool bChangesOccurred = false;

	// Iterate over all actors, looking for actors in the specified layers.
	for (AActor* Actor : FActorRange(GetWorld()))
	{
		if (!IsActorValidForLayer(Actor))
		{
			continue;
		}

		if (Filter.IsValid() && !Filter->PassesFilter(TWeakObjectPtr< AActor >(Actor)))
		{
			continue;
		}

		for (auto LayerNameIt = LayerNames.CreateConstIterator(); LayerNameIt; ++LayerNameIt)
		{
			if (Actor->Layers.Contains(*LayerNameIt))
			{
				// The actor was found to be in a specified layer.
				// Set selection state and move on to the next actor.
				bool bNotifyForActor = false;
				GEditor->GetSelectedActors()->Modify();
				GEditor->SelectActor(Actor, bSelect, bNotifyForActor, bSelectEvenIfHidden);
				bChangesOccurred = true;
				break;
			}
		}
	}

	GEditor->GetSelectedActors()->EndBatchSelectOperation();

	if (bNotify)
	{
		GEditor->NoteSelectionChange();
	}

	return bChangesOccurred;
}


///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// Operations on actor viewport visibility regarding layers
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
void ULayersSubsystem::UpdatePerViewVisibility( FLevelEditorViewportClient* ViewportClient, const FName& LayerThatChanged )
{
	const int32 ViewIndex = ViewportClient->ViewIndex;
	// get the viewport client
	// Iterate over all actors, looking for actors in the specified layers.
	if( ViewportClient->GetWorld() == NULL )
	{
		return;
	}
	for( FActorIterator It(ViewportClient->GetWorld()) ; It ; ++It )
	{
		AActor* Actor = *It;
		if( !IsActorValidForLayer( Actor ) )
		{
			continue;
		}

		// if the view has nothing hidden, just quickly mark the actor as visible in this view 
		if ( ViewportClient->ViewHiddenLayers.Num() == 0)
		{
			// if the actor had this view hidden, then unhide it
			if ( Actor->HiddenEditorViews & ( (uint64)1 << ViewIndex ) )
			{
				// make sure this actor doesn't have the view set
				Actor->HiddenEditorViews &= ~( (uint64)1 << ViewIndex );
				Actor->MarkComponentsRenderStateDirty();
			}
		}
		// else if we were given a name that was changed, only update actors with that name in their layers,
		// otherwise update all actors
		else if ( LayerThatChanged == NAME_Skip || Actor->Layers.Contains( LayerThatChanged ) )
		{
			UpdateActorViewVisibility(ViewportClient, Actor);
		}
	}

	// make sure we redraw the viewport
	ViewportClient->Invalidate();
}


void ULayersSubsystem::UpdateAllViewVisibility( const FName& LayerThatChanged )
{
	// update all views's hidden layers if they had this one
	for (FLevelEditorViewportClient* ViewportClient : GEditor->GetLevelViewportClients())
	{
		UpdatePerViewVisibility( ViewportClient, LayerThatChanged );
	}
}


void ULayersSubsystem::UpdateActorViewVisibility(FLevelEditorViewportClient* ViewportClient, AActor* Actor, bool bReregisterIfDirty)
{
	// get the viewport client
	const int32 ViewIndex = ViewportClient->ViewIndex;

	int32 NumHiddenLayers = 0;
	// look for which of the actor layers are hidden
	for (int32 LayerIndex = 0; LayerIndex < Actor->Layers.Num(); LayerIndex++)
	{
		// if its in the view hidden list, this layer is hidden for this actor
		if (ViewportClient->ViewHiddenLayers.Find( Actor->Layers[ LayerIndex ] ) != -1)
		{
			NumHiddenLayers++;
			// right now, if one is hidden, the actor is hidden
			break;
		}
	}

	uint64 OriginalHiddenViews = Actor->HiddenEditorViews;

	// right now, if one is hidden, the actor is hidden
	if (NumHiddenLayers)
	{
		Actor->HiddenEditorViews |= ((uint64)1 << ViewIndex);
	}
	else
	{
		Actor->HiddenEditorViews &= ~((uint64)1 << ViewIndex);
	}

	// reregister if we changed the visibility bits, as the rendering thread needs them
	if (bReregisterIfDirty && OriginalHiddenViews != Actor->HiddenEditorViews)
	{
		Actor->MarkComponentsRenderStateDirty();

		// make sure we redraw the viewport
		ViewportClient->Invalidate();
	}
}


void ULayersSubsystem::UpdateActorAllViewsVisibility(AActor* Actor)
{
	uint64 OriginalHiddenViews = Actor->HiddenEditorViews;

	for (FLevelEditorViewportClient* ViewportClient : GEditor->GetLevelViewportClients())
	{
		// don't have this reattach, as we can do it once for all views
		UpdateActorViewVisibility(ViewportClient, Actor, false);
	}

	// reregister if we changed the visibility bits, as the rendering thread needs them
	if (OriginalHiddenViews != Actor->HiddenEditorViews)
	{
		return;
	}

	Actor->MarkComponentsRenderStateDirty();

	// redraw all viewports if the actor
	for (FLevelEditorViewportClient* ViewportClient : GEditor->GetLevelViewportClients())
	{
		// make sure we redraw all viewports
		ViewportClient->Invalidate();
	}
}


void ULayersSubsystem::RemoveViewFromActorViewVisibility( FLevelEditorViewportClient* ViewportClient )
{
	const int32 ViewIndex = ViewportClient->ViewIndex;

	// get the bit for the view index
	uint64 ViewBit = ((uint64)1 << ViewIndex);
	// get all bits under that that we want to keep
	uint64 KeepBits = ViewBit - 1;

	// Iterate over all actors, looking for actors in the specified layers.
	if (ViewportClient->GetWorld())
	{
		for( FActorIterator It(ViewportClient->GetWorld()) ; It ; ++It )
		{
			AActor* Actor = *It;

			if( !IsActorValidForLayer( Actor ) )
			{
				continue;
			}

			// remember original bits
			uint64 OriginalHiddenViews = Actor->HiddenEditorViews;

			uint64 Was = Actor->HiddenEditorViews;

			// slide all bits higher than ViewIndex down one since the view is being removed from Editor
			uint64 LowBits = Actor->HiddenEditorViews & KeepBits;

			// now slide the top bits down by ViewIndex + 1 (chopping off ViewBit)
			uint64 HighBits = Actor->HiddenEditorViews >> (ViewIndex + 1);
			// then slide back up by ViewIndex, which will now have erased ViewBit, as well as leaving 0 in the low bits
			HighBits = HighBits << ViewIndex;

			// put it all back together
			Actor->HiddenEditorViews = LowBits | HighBits;

			// reregister if we changed the visibility bits, as the rendering thread needs them
			if (OriginalHiddenViews == Actor->HiddenEditorViews)
			{
				continue;
			}

			// Find all registered primitive components and update the scene proxy with the actors updated visibility map
			TInlineComponentArray<UPrimitiveComponent*> Components;
			Actor->GetComponents(Components);

			for (UActorComponent* Component : Actor->GetComponents())
			{
				UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Component);
				if (PrimitiveComponent && PrimitiveComponent->IsRegistered())
				{
					// Push visibility to the render thread
					PrimitiveComponent->PushEditorVisibilityToProxy( Actor->HiddenEditorViews );
				}
			}
		}
	}
}

static void UpdateBrushLayerVisibility(ABrush* Brush, bool bIsHidden)
{
	ULevel* Level = Brush->GetLevel();
	if (!Level)
	{
		return;
	}

	UModel* Model = Level->Model;
	if (!Model)
	{
		return;
	}

	bool bAnySurfaceWasFound = false;
	for (FBspSurf& Surf : Model->Surfs)
	{
		if (Surf.Actor == Brush)
		{
			Surf.bHiddenEdLayer = bIsHidden;
			bAnySurfaceWasFound = true;
		}
	}

	if (bAnySurfaceWasFound)
	{
		Level->UpdateModelComponents();
		Model->InvalidSurfaces = true;
	}
}

bool ULayersSubsystem::UpdateActorVisibility(AActor* Actor, bool& bOutSelectionChanged, bool& bOutActorModified, const bool bNotifySelectionChange, const bool bRedrawViewports)
{
	bOutActorModified = false;
	bOutSelectionChanged = false;

	if(	!IsActorValidForLayer( Actor ) )
	{
		return false;
	}

	// If the actor doesn't belong to any layers
	if( Actor->Layers.Num() == 0)
	{
		bOutActorModified = Actor->SetIsHiddenEdLayer(false);
		return bOutActorModified;
	}

	bool bActorBelongsToVisibleLayer = false;
	for( int32 LayerIndex = 0 ; LayerIndex < GetWorld()->Layers.Num() ; ++LayerIndex )
	{
		ULayer* Layer =  GetWorld()->Layers[ LayerIndex ];

		if( !Layer->IsVisible() )
		{
			continue;
		}

		if( Actor->Layers.Contains( Layer->GetLayerName() ) )
		{
			if (Actor->SetIsHiddenEdLayer(false))
			{
				bOutActorModified = true;

				if (ABrush* Brush = Cast<ABrush>(Actor))
				{
					const bool bIsHidden = false;
					UpdateBrushLayerVisibility(Brush, bIsHidden);
				}
			}

			// Stop, because we found at least one visible layer the actor belongs to
			bActorBelongsToVisibleLayer = true;
			break;
		}
	}

	// If the actor isn't part of a visible layer, hide and de-select it.
	if( !bActorBelongsToVisibleLayer )
	{
		if (Actor->SetIsHiddenEdLayer(true))
		{
			bOutActorModified = true;

			if (ABrush* Brush = Cast<ABrush>(Actor))
			{
				const bool bIsHidden = true;
				UpdateBrushLayerVisibility(Brush, bIsHidden);
			}
		}

		//if the actor was selected, mark it as unselected
		if ( Actor->IsSelected() )
		{
			bool bSelect = false;
			bool bNotify = false;
			bool bIncludeHidden = true;
			GEditor->SelectActor( Actor, bSelect, bNotify, bIncludeHidden );

			bOutSelectionChanged = true;
			bOutActorModified = true;
		}
	}

	if ( bNotifySelectionChange && bOutSelectionChanged )
	{
		GEditor->NoteSelectionChange();
	}

	if( bRedrawViewports )
	{
		GEditor->RedrawLevelEditingViewports();
	}

	return bOutActorModified || bOutSelectionChanged;
}


bool ULayersSubsystem::UpdateAllActorsVisibility(const bool bNotifySelectionChange, const bool bRedrawViewports)
{
	return UpdateAllActorsVisibility_Internal(bNotifySelectionChange, bRedrawViewports);
}


///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// Operations on layers
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void ULayersSubsystem::AppendActorsFromLayer(const FName& LayerName, TArray< AActor* >& InOutActors) const
{
	AppendActorsFromLayer(LayerName, InOutActors, nullptr);
}

#define LAYERS_SUBSYSTEM_APPEND_ACTORS_FOR_LAYER_PART_1 \
	for (FActorIterator ActorIt(GetWorld()); ActorIt; ++ActorIt) \
	{
#define LAYERS_SUBSYSTEM_APPEND_ACTORS_FOR_LAYER_PART_2 \
		if (Filter.IsValid() && !Filter->PassesFilter(Actor)) \
		{ \
			continue; \
		} \
		 \
		if (Actor->Layers.Contains(LayerName)) \
		{ \
			InOutActors.Add(Actor); \
		} \
	}

void ULayersSubsystem::AppendActorsFromLayer(const FName& LayerName, TArray< AActor* >& InOutActors, const TSharedPtr< ActorFilter >& Filter) const
{
	LAYERS_SUBSYSTEM_APPEND_ACTORS_FOR_LAYER_PART_1
		AActor* Actor = *ActorIt;
	LAYERS_SUBSYSTEM_APPEND_ACTORS_FOR_LAYER_PART_2
}

void ULayersSubsystem::AppendActorsFromLayer(const FName& LayerName, TArray< TWeakObjectPtr< AActor > >& InOutActors, const TSharedPtr< ActorFilter >& Filter) const
{
	LAYERS_SUBSYSTEM_APPEND_ACTORS_FOR_LAYER_PART_1
		const TWeakObjectPtr< AActor > Actor = *ActorIt;
	LAYERS_SUBSYSTEM_APPEND_ACTORS_FOR_LAYER_PART_2
}


void ULayersSubsystem::AppendActorsFromLayers(const TArray< FName >& LayerNames, TArray< AActor* >& InOutActors) const
{
	AppendActorsFromLayers(LayerNames, InOutActors, nullptr);
}

#define LAYERS_SUBSYSTEM_APPEND_ACTORS_FOR_LAYERS_PART_1 \
	for (FActorIterator ActorIt(GetWorld()); ActorIt; ++ActorIt) \
	{
#define LAYERS_SUBSYSTEM_APPEND_ACTORS_FOR_LAYERS_PART_2 \
		if (Filter.IsValid() && !Filter->PassesFilter(Actor)) \
		{ \
			continue; \
		} \
		 \
		for (auto LayerNameIt = LayerNames.CreateConstIterator(); LayerNameIt; ++LayerNameIt) \
		{ \
			const FName& LayerName = *LayerNameIt; \
			 \
			if (Actor->Layers.Contains(LayerName)) \
			{ \
				InOutActors.Add(Actor); \
				break; \
			} \
		} \
	}

void ULayersSubsystem::AppendActorsFromLayers(const TArray< FName >& LayerNames, TArray< AActor* >& InOutActors, const TSharedPtr< ActorFilter >& Filter) const
{
	LAYERS_SUBSYSTEM_APPEND_ACTORS_FOR_LAYERS_PART_1
		AActor* Actor = *ActorIt;
	LAYERS_SUBSYSTEM_APPEND_ACTORS_FOR_LAYERS_PART_2
}

void ULayersSubsystem::AppendActorsFromLayers(const TArray< FName >& LayerNames, TArray< TWeakObjectPtr< AActor > >& InOutActors, const TSharedPtr< ActorFilter >& Filter) const
{
	LAYERS_SUBSYSTEM_APPEND_ACTORS_FOR_LAYERS_PART_1
		const TWeakObjectPtr< AActor > Actor = *ActorIt;
	LAYERS_SUBSYSTEM_APPEND_ACTORS_FOR_LAYERS_PART_2
}


TArray< AActor* > ULayersSubsystem::GetActorsFromLayer(const FName& LayerName) const
{
	TArray< AActor* > OutActors;
	AppendActorsFromLayer(LayerName, OutActors);
	return OutActors;
}

TArray< AActor* > ULayersSubsystem::GetActorsFromLayer(const FName& LayerName, const TSharedPtr< ActorFilter >& Filter) const
{
	TArray< AActor* > OutActors;
	AppendActorsFromLayer(LayerName, OutActors, Filter);
	return OutActors;
}

TArray< AActor* > ULayersSubsystem::GetActorsFromLayers(const TArray< FName >& LayerNames) const
{
	TArray< AActor* > OutActors;
	AppendActorsFromLayers(LayerNames, OutActors);
	return OutActors;
}

TArray< AActor* > ULayersSubsystem::GetActorsFromLayers(const TArray< FName >& LayerNames, const TSharedPtr< ActorFilter >& Filter) const
{
	TArray< AActor* > OutActors;
	AppendActorsFromLayers(LayerNames, OutActors, Filter);
	return OutActors;
}


void ULayersSubsystem::SetLayerVisibility( const FName& LayerName, const bool bIsVisible )
{
	SetLayersVisibility( { LayerName }, bIsVisible );
}


void ULayersSubsystem::SetLayersVisibility( const TArray< FName >& LayerNames, const bool bIsVisible )
{
	bool bChangeOccurred = false;
	for( const auto& LayerName : LayerNames )
	{
		ULayer* Layer = EnsureLayerExists( LayerName );
		check( Layer != NULL );

		if( Layer->IsVisible() != bIsVisible )
		{
			Layer->Modify();
			Layer->SetVisible(bIsVisible);
			LayersChanged.Broadcast( ELayersAction::Modify, Layer, "bIsVisible" );
			bChangeOccurred = true;
		}
	}

	if( bChangeOccurred )
	{
		UpdateAllActorsVisibility( true, true );
	}
}


void ULayersSubsystem::ToggleLayerVisibility( const FName& LayerName )
{
	ULayer* Layer = EnsureLayerExists( LayerName );
	check( Layer != NULL );

	Layer->Modify();
	Layer->SetVisible(!Layer->IsVisible());

	LayersChanged.Broadcast( ELayersAction::Modify, Layer, "bIsVisible" );
	UpdateAllActorsVisibility( true, true );
}


void ULayersSubsystem::ToggleLayersVisibility( const TArray< FName >& LayerNames )
{
	if( LayerNames.Num() == 0 )
	{
		return;
	}

	for( auto LayerNameIt = LayerNames.CreateConstIterator(); LayerNameIt; ++LayerNameIt )
	{
		ULayer* Layer = EnsureLayerExists( *LayerNameIt );
		check( Layer != NULL );

		Layer->Modify();
		Layer->SetVisible(!Layer->IsVisible());
		LayersChanged.Broadcast( ELayersAction::Modify, Layer, "bIsVisible" );
	}

	UpdateAllActorsVisibility( true, true );
}


void ULayersSubsystem::MakeAllLayersVisible()
{
	for (auto Layer : GetWorld()->Layers)
	{
		if( !Layer->IsVisible() )
		{
			Layer->Modify();
			Layer->SetVisible(true);
			LayersChanged.Broadcast( ELayersAction::Modify, TWeakObjectPtr< ULayer >(Layer), "bIsVisible" );
		}
	}

	UpdateAllActorsVisibility( true, true );
}


ULayer* ULayersSubsystem::GetLayer(const FName& LayerName) const
{
	for (auto LayerIt = GetWorld()->Layers.CreateConstIterator(); LayerIt; ++LayerIt)
	{
		ULayer* Layer = *LayerIt;
		if (Layer->GetLayerName() == LayerName)
		{
			return Layer;
		}
	}

	return nullptr;
}


bool ULayersSubsystem::IsLayer(const FName& LayerName)
{
	ULayer* OutLayer = GetLayer(LayerName);
	return (OutLayer != nullptr);
}

bool ULayersSubsystem::TryGetLayer(const FName& LayerName, ULayer*& OutLayer)
{
	OutLayer = GetLayer(LayerName);
	return (OutLayer != nullptr);
}


void ULayersSubsystem::AddAllLayerNamesTo( TArray< FName >& OutLayers ) const
{
	for( auto LayerIt = GetWorld()->Layers.CreateConstIterator(); LayerIt; ++LayerIt )
	{
		ULayer* Layer = *LayerIt;
		OutLayers.Add( Layer->GetLayerName() );
	}
}

#define LAYERS_SUBSYSTEM_ADD_ALL_LAYERS_TO \
	for (auto LayerIt = GetWorld()->Layers.CreateConstIterator(); LayerIt; ++LayerIt) \
	{ \
		OutLayers.Add(*LayerIt); \
	}

void ULayersSubsystem::AddAllLayersTo(TArray< ULayer* >& OutLayers) const
{
	LAYERS_SUBSYSTEM_ADD_ALL_LAYERS_TO
}

void ULayersSubsystem::AddAllLayersTo(TArray< TWeakObjectPtr< ULayer > >& OutLayers) const
{
	LAYERS_SUBSYSTEM_ADD_ALL_LAYERS_TO
}


ULayer* ULayersSubsystem::CreateLayer(const FName& LayerName)
{
	ULayer* NewLayer = NewObject<ULayer>(GetWorld(), NAME_None, RF_Transactional | RF_Transient);
	check(NewLayer != NULL);

	GetWorld()->Modify(false);
	GetWorld()->Layers.Add(NewLayer);

	NewLayer->SetLayerName(LayerName);
	NewLayer->SetVisible(true);
	
	if (UE::Editor::Layers::Private::bRegisterLayersInTeds)
	{
		if (DataStorage && DataStorageCompat)
		{
			UE::Editor::DataStorage::RowHandle LayerRow = DataStorageCompat->AddCompatibleObject(NewLayer);
		}
	}
	
	LayersChanged.Broadcast(ELayersAction::Add, NewLayer, NAME_None);

	return NewLayer;
}


void ULayersSubsystem::DeleteLayers( const TArray< FName >& LayersToDelete )
{
	TArray< FName > ValidLayersToDelete;
	for( auto LayerNameIt = LayersToDelete.CreateConstIterator(); LayerNameIt; ++LayerNameIt )
	{
		if( IsLayer( *LayerNameIt ) )
		{
			ValidLayersToDelete.Add( *LayerNameIt );
		}
	}

	// Iterate over all actors, looking for actors in the specified layers.
	for( FActorIterator It(GetWorld()); It ; ++It )
	{
		AActor* Actor = *It;

		//The Layer must exist in order to remove actors from it,
		//so we have to wait to delete the ULayer object till after
		//all the actors have been disassociated with it.
		RemoveActorFromLayers( Actor, ValidLayersToDelete, false );
	}

	bool bValidLayerExisted = false;
	for (int LayerIndex = GetWorld()->Layers.Num() - 1; LayerIndex >= 0 ; LayerIndex--)
	{
		if( LayersToDelete.Contains( GetWorld()->Layers[ LayerIndex]->GetLayerName() ) )
		{
			if (UE::Editor::Layers::Private::bRegisterLayersInTeds)
			{
				if (DataStorage && DataStorageCompat)
				{
					DataStorageCompat->RemoveCompatibleObject(GetWorld()->Layers[ LayerIndex]);
				}
			}
			
			GetWorld()->Modify(false);
			GetWorld()->Layers.RemoveAt( LayerIndex );
			bValidLayerExisted = true;
		}
	}

	LayersChanged.Broadcast( ELayersAction::Delete, NULL, NAME_None );
}


void ULayersSubsystem::DeleteLayer( const FName& LayerToDelete )
{
	if( !IsLayer( LayerToDelete ) )
	{
		return;
	}
	// Iterate over all actors, looking for actors in the specified layer.
	for( FActorIterator It(GetWorld()) ; It ; ++It )
	{
		AActor* Actor = *It;

		//The Layer must exist in order to remove actors from it,
		//so we have to wait to delete the ULayer object till after
		//all the actors have been disassociated with it.
		RemoveActorFromLayer( Actor, LayerToDelete, false );
	}

	bool bValidLayerExisted = false;
	for (int LayerIndex = GetWorld()->Layers.Num() - 1; LayerIndex >= 0 ; LayerIndex--)
	{
		if( LayerToDelete == GetWorld()->Layers[ LayerIndex]->GetLayerName() )
		{
			if (UE::Editor::Layers::Private::bRegisterLayersInTeds)
			{
				if (DataStorage && DataStorageCompat)
				{
					DataStorageCompat->RemoveCompatibleObject(GetWorld()->Layers[ LayerIndex]);
				}
			}
			
			GetWorld()->Modify(false);
			GetWorld()->Layers.RemoveAt( LayerIndex );
			bValidLayerExisted = true;
		}
	}

	LayersChanged.Broadcast( ELayersAction::Delete, NULL, NAME_None );
}


bool ULayersSubsystem::RenameLayer( const FName& OriginalLayerName, const FName& NewLayerName )
{
	// We specifically don't pass the original LayerName by reference to avoid it changing
	// it's original value, in case, it would be the reference of the Layer's actually FName
	if ( OriginalLayerName == NewLayerName )
	{
		return false;
	}

	ULayer* Layer;
	if( !TryGetLayer( OriginalLayerName, Layer ) )
	{
		return false;
	}

	Layer->Modify();
	const FName OriginalLayerNameCopy = OriginalLayerName; // Otherwise, bug if RenameLayer(Layer->LayerName, NewLayerName) after the next LayerName rename
	Layer->SetLayerName(NewLayerName);
	Layer->ClearActorStats();
	// Iterate over all actors, swapping layers.
	for( FActorIterator It(GetWorld()) ; It ; ++It )
	{
		AActor* Actor = *It;
		if( !IsActorValidForLayer( Actor ) )
		{
			continue;
		}

		if (ULayersSubsystem::RemoveActorFromLayer(Actor, OriginalLayerNameCopy))
		{
			// No need to mark the actor as modified these functions take care of that
			AddActorToLayer( Actor, NewLayerName );
		}
	}

	// update all views's hidden layers if they had this one
	for (FLevelEditorViewportClient* ViewportClient : GEditor->GetLevelViewportClients())
	{
		if (ViewportClient->ViewHiddenLayers.Remove(OriginalLayerNameCopy) > 0)
		{
			ViewportClient->ViewHiddenLayers.AddUnique( NewLayerName );
			ViewportClient->Invalidate();
		}
	}

	LayersChanged.Broadcast( ELayersAction::Rename, Layer, "LayerName" );

	return true;
}


///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// Helper functions.
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void ULayersSubsystem::AddActorToStats(ULayer* Layer, AActor* Actor)
{
	if (!Actor)
	{
		return;
	}

	Layer->AddToStats(Actor);
	LayersChanged.Broadcast(ELayersAction::Modify, Layer, TEXT("ActorStats"));
}

void ULayersSubsystem::RemoveActorFromStats(ULayer* Layer, AActor* Actor)
{
	if (!Actor)
	{
		return;
	}

	bool bFoundClassStats = Layer->RemoveFromStats(Actor);
	if (bFoundClassStats)
	{
		LayersChanged.Broadcast(ELayersAction::Modify, Layer, TEXT("ActorStats"));
	}
}

void ULayersSubsystem::AddActorToLayerInTeds(ULayer* Layer, AActor* Actor)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::Layers;
	
	if (UE::Editor::Layers::Private::bRegisterLayersInTeds)
	{
		if (DataStorage && DataStorageCompat)
		{
			RowHandle LayerRow = DataStorageCompat->FindRowWithCompatibleObject(Layer);
			RowHandle ActorRow = DataStorageCompat->FindRowWithCompatibleObject(Actor);
			
			if (DataStorage->IsRowAvailable(LayerRow))
			{
				if (FActorLayersColumn* LayersColumn = DataStorage->GetColumn<FActorLayersColumn>(ActorRow))
				{
					LayersColumn->Layers.Add(LayerRow);
				
					// This also sorts the array
					LayersColumn->Layers.MakeUnique();
				}
				else
				{
					FActorLayersColumn NewLayersColumn;
					NewLayersColumn.Layers.Add(LayerRow, FRowHandleArrayView::EFlags::IsSorted);
					DataStorage->AddColumn(ActorRow, MoveTemp(NewLayersColumn));
				}
			}
		}
	}
}

void ULayersSubsystem::RemoveActorFromLayerInTeds(ULayer* Layer, AActor* Actor)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::Layers;
	
	if (UE::Editor::Layers::Private::bRegisterLayersInTeds)
	{
		if (DataStorage && DataStorageCompat)
		{
			RowHandle LayerRow = DataStorageCompat->FindRowWithCompatibleObject(Layer);
			RowHandle ActorRow = DataStorageCompat->FindRowWithCompatibleObject(Actor);
			
			if (FActorLayersColumn* LayersColumn = DataStorage->GetColumn<FActorLayersColumn>(ActorRow))
			{
				LayersColumn->Layers.Remove(LayerRow);
				
				if (LayersColumn->Layers.IsEmpty())
				{
					DataStorage->RemoveColumn<FActorLayersColumn>(ActorRow);
				}
			}
		}
	}
}

bool ULayersSubsystem::UpdateAllActorsVisibility_Internal(const bool bNotifySelectionChange, const bool bRedrawViewports, bool bUpdateTedsState)
{
	bool bSelectionChanged = false;
	bool bChangesOccurred = false;
	for( FActorIterator It(GetWorld()) ; It ; ++It )
	{
		AActor* Actor = *It;

		bool bActorModified = false;
		bool bActorSelectionChanged = false;
		const bool bActorNotifySelectionChange = false;
		const bool bActorRedrawViewports = false;

		bChangesOccurred |= UpdateActorVisibility( Actor, bActorSelectionChanged /*OUT*/, bActorModified /*OUT*/, bActorNotifySelectionChange, bActorRedrawViewports );
		bSelectionChanged |= bActorSelectionChanged;
		
		// The state in TEDS for the layers an actor belongs to might not be in sync after an undo/redo, since we are already iterating over all actors here
		// we can use this opportunity to also make sure the layer column is updated
		if (bUpdateTedsState)
		{
			SyncActorLayersToTeds(Actor);
		}
	}

	if ( bNotifySelectionChange && bSelectionChanged )
	{
		GEditor->NoteSelectionChange();
	}

	if( bRedrawViewports )
	{
		GEditor->RedrawLevelEditingViewports();
	}

	return bChangesOccurred;
}

void ULayersSubsystem::SyncActorLayersToTeds(AActor* Actor)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::Layers;
	
	if (UE::Editor::Layers::Private::bRegisterLayersInTeds)
	{
		if (DataStorage && DataStorageCompat)
		{
			RowHandle ActorRow = DataStorageCompat->FindRowWithCompatibleObject(Actor);
			
			if (DataStorage->IsRowAvailable(ActorRow))
			{
				// Get the state of the layers on the Actor UObject
				FRowHandleArray NewLayers;
		
				for (const FName& LayerName : Actor->Layers)
				{
					ULayer* Layer = EnsureLayerExists(LayerName);
					RowHandle LayerRow = DataStorageCompat->FindRowWithCompatibleObject(Layer);
			
					if (DataStorage->IsRowAvailable(LayerRow))
					{
						NewLayers.Add(LayerRow);
					}
				}
		
				NewLayers.Sort();
		
				// Overwrite the state in TEDS
				if (NewLayers.Num())
				{
					FActorLayersColumn LayersColumn;
					LayersColumn.Layers = MoveTemp(NewLayers);
					DataStorage->AddColumn<FActorLayersColumn>(ActorRow, MoveTemp(LayersColumn));
				}
				else
				{
					DataStorage->RemoveColumn<FActorLayersColumn>(ActorRow);
				}
			}
		}
	}
}

ULayer* ULayersSubsystem::EnsureLayerExists(const FName& LayerName)
{
	ULayer* Layer;
	if (!TryGetLayer(LayerName, Layer))
	{
		Layer = CreateLayer(LayerName);
	}

	return Layer;
}
