// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorLayersWidget.h"

#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

namespace UE::Editor::DataStorage
{
	void FActorLayersWidgetSearcher::GetSearchableString(FString& SearchableString, const ICoreProvider& Storage, const Layers::FActorLayersColumn& Column) const
	{
		for (const RowHandle LayerRow : Column.Layers.GetRows())
		{
			if (const Layers::FLayerNameColumn* LayerNameColumn = Storage.GetColumn<Layers::FLayerNameColumn>(LayerRow))
			{
				if (SearchableString.Len())
				{
					SearchableString += TEXT(", ");
				}

				SearchableString += LayerNameColumn->LayerName.ToString();
			}
		}
	}
}

FActorLayersWidgetConstructor::FActorLayersWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FActorLayersWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
                                                                UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow,
                                                                const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::Layers;
	
	FAttributeBinder Binder(TargetRow, DataStorage);
	
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(8, 0, 0, 0)
		[
			SNew(STextBlock)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Text(Binder.BindData<FText, FRowHandleArray>(&FActorLayersColumn::Layers, [DataStorage](const FRowHandleArray& Layers) -> FText
				{
					FString Result;
						
					for (RowHandle LayerRow : Layers.GetRows())
					{
						if (FLayerNameColumn* LayerNameColumn = DataStorage->GetColumn<FLayerNameColumn>(LayerRow))
						{
							if (Result.Len())
							{
								Result += TEXT(", ");
							}

							Result += LayerNameColumn->LayerName.ToString();
						}
					}
					return FText::FromString(Result);
				}))
		];
}

TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSearcherInterface>> FActorLayersWidgetConstructor::ConstructColumnSearchers(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	return TArray<TSharedPtr<const FColumnSearcherInterface>>(
		{
			MakeShared<FActorLayersWidgetSearcher>()
		});
}