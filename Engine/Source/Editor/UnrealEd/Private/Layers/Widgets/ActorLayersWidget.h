// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Layers/Columns/LayersColumns.h"

#include "ActorLayersWidget.generated.h"

namespace UE::Editor::DataStorage
{
	class FActorLayersWidgetSearcher final : public TColumnSearcherInterface<Layers::FActorLayersColumn>
	{
	protected:
		virtual void GetSearchableString(FString& SearchableString, const ICoreProvider& Storage, const Layers::FActorLayersColumn& Column) const override;
	};
}

// Widget to show the layers an actor belongs to
USTRUCT()
struct FActorLayersWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FActorLayersWidgetConstructor();
	~FActorLayersWidgetConstructor() override = default;
	
	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
		
	virtual TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSearcherInterface>> ConstructColumnSearchers(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};