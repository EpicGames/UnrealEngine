// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelEditorDragDropHandler.h"
#include "Editor/EditorEngine.h"
#include "Engine/World.h"
#include "ObjectTools.h"
#include "AssetRegistry/AssetData.h"
#include "Engine/BrushBuilder.h"
#include "GameFramework/Actor.h"
#include "AssetSelection.h"
#include "Materials/MaterialInterface.h"
#include "UnrealEdGlobals.h"
#include "Editor/UnrealEdEngine.h"
#include "Editor.h"
#include "HitProxies.h"
#include "Misc/MessageDialog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelEditorDragDropHandler)

#define LOCTEXT_NAMESPACE "UnrealEd"

ULevelEditorDragDropHandler::ULevelEditorDragDropHandler()
{
}

bool ULevelEditorDragDropHandler::PassesFilter(UWorld* World, const FAssetData& AssetData, TUniquePtr<FLevelEditorDragDropWorldSurrogateReferencingObject>* OutWorldSurrogateReferencingObject)
{
	if (World && bRunAssetFilter)
	{
		auto AssetPassesFilter = [](const FAssetData& InAssetData, const UObject* InReferencingAsset, FText* OutFailureReason = nullptr)
		{
			FAssetReferenceFilterContext AssetReferenceFilterContext;
			AssetReferenceFilterContext.AddReferencingAsset(FAssetData(InReferencingAsset));

			TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext);
			return AssetReferenceFilter.IsValid() ? AssetReferenceFilter->PassesFilter(InAssetData, OutFailureReason) : true;
		};

		ULevel* CurrentLevel = World->GetCurrentLevel();
		UWorld* CurrentLevelOuterWorld = CurrentLevel ? Cast<UWorld>(CurrentLevel->GetOuter()) : nullptr;
		UWorld* ReferencingWorld = CurrentLevelOuterWorld ? CurrentLevelOuterWorld : World;

		FText FailureReason;
		bool bPassesFilter = AssetPassesFilter(AssetData, ReferencingWorld, &FailureReason);
		
		// Try to find with referencing asset matching the AssetData
		if (OnLevelEditorDragDropWorldSurrogateReferencingObjectDelegate.IsBound())
		{
			TUniquePtr<FLevelEditorDragDropWorldSurrogateReferencingObject> Object = OnLevelEditorDragDropWorldSurrogateReferencingObjectDelegate.Execute(ReferencingWorld, AssetData.GetSoftObjectPath());
			const UObject* ReferencingAsset = Object.IsValid() ? Object->GetValue() : nullptr;
			bool bPassesFilterUsingWorldSurrogate = ReferencingAsset ? AssetPassesFilter(AssetData, ReferencingAsset) : false;
			if (bPassesFilterUsingWorldSurrogate)
			{
				if (OutWorldSurrogateReferencingObject)
				{
					*OutWorldSurrogateReferencingObject = MoveTemp(Object);
				}
				bPassesFilter = true;
			}
		}

		if (!bPassesFilter)
		{
			bCanDrop = false;
			HintText = FailureReason;
			return false;
		}
	}

	return true;
}

bool ULevelEditorDragDropHandler::PreviewDropObjectsAtCoordinates(int32 MouseX, int32 MouseY, UWorld* World, FViewport* Viewport, const FAssetData& AssetData)
{
	bCanDrop = true;
	HintText = FText::GetEmpty();

	if (!AssetData.IsValid())
	{
		bCanDrop = false;
		return false;
	}

	return PassesFilter(World, AssetData);
}

bool ULevelEditorDragDropHandler::PreDropObjectsAtCoordinates(int32 MouseX, int32 MouseY, UWorld* World, FViewport* Viewport, const TArray<UObject*>& DroppedObjects, TArray<AActor*>& OutNewActors)
{
	WorldSurrogateReferencingObject.Reset();
	if (World && bRunAssetFilter)
	{
		if (DroppedObjects.IsEmpty() ||
			(!PassesFilter(World, FAssetData(DroppedObjects[0]), &WorldSurrogateReferencingObject)))
		{
			return false;
		}
	
		auto GetValue = [](const TUniquePtr<FLevelEditorDragDropWorldSurrogateReferencingObject>& InSurrogateObject)
		{
			const FLevelEditorDragDropWorldSurrogateReferencingObject* Object = InSurrogateObject.Get();
			return Object ? Object->GetValue() : nullptr;
		};

		for (int i=1; i<DroppedObjects.Num(); ++i)
		{
			TUniquePtr<FLevelEditorDragDropWorldSurrogateReferencingObject> DroppedSurrogateObject;
			if (!PassesFilter(World, FAssetData(DroppedObjects[i]), &DroppedSurrogateObject) || 
				(GetValue(DroppedSurrogateObject) != GetValue(WorldSurrogateReferencingObject)))
			{
				FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("DroppedObjectsDifferentSourcePluginPath", "Dropped objects are not part of the same source plugin path."));
				WorldSurrogateReferencingObject.Reset();
				return false;
			}
		}

		return !WorldSurrogateReferencingObject.IsValid() || WorldSurrogateReferencingObject->OnPreDropObjects(World, DroppedObjects);
	}
	return true;
}

bool ULevelEditorDragDropHandler::PostDropObjectsAtCoordinates(int32 MouseX, int32 MouseY, UWorld* World, FViewport* Viewport, const TArray<UObject*>& DroppedObjects)
{
	if (WorldSurrogateReferencingObject.IsValid())
	{
		bool bSuccess = WorldSurrogateReferencingObject->OnPostDropObjects(World, DroppedObjects);
		WorldSurrogateReferencingObject.Reset();
		return bSuccess;
	}
	return true;
}

#undef LOCTEXT_NAMESPACE
