// Copyright Epic Games, Inc. All Rights Reserved.


#include "LevelEditorViewport.h"
#include "Animation/Skeleton.h"
#include "Materials/MaterialInterface.h"
#include "Modules/ModuleManager.h"
#include "Misc/PackageName.h"
#include "Misc/ITransaction.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/AppStyle.h"
#include "Components/MeshComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Animation/AnimSequenceBase.h"
#include "CanvasItem.h"
#include "Engine/BrushBuilder.h"
#include "Engine/SkeletalMesh.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Engine/Brush.h"
#include "AI/NavigationSystemBase.h"
#include "AssetRegistry/AssetData.h"
#include "Editor/UnrealEdEngine.h"
#include "Animation/AnimBlueprint.h"
#include "Exporters/ExportTextContainer.h"
#include "Factories/MaterialFactoryNew.h"
#include "Factories/MaterialInstanceConstantFactoryNew.h"
#include "Editor/GroupActor.h"
#include "Components/DecalComponent.h"
#include "Components/DirectionalLightComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Components/ModelComponent.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "Engine/Selection.h"
#include "UObject/UObjectIterator.h"
#include "EngineUtils.h"
#include "Editor.h"
#include "EditorDirectories.h"
#include "EditorModes.h"
#include "UnrealEdGlobals.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "EditorSupportDelegates.h"
#include "EditorGizmos/EditorTransformGizmoUtil.h"
#include "DataStorage/Features.h"
#include "Elements/Actor/ActorElementData.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementCommonActions.h"
#include "Elements/Framework/TypedElementListObjectUtil.h"
#include "Elements/Framework/TypedElementViewportInteraction.h"
#include "Elements/Interfaces/TypedElementObjectInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Elements/Actor/ActorElementLevelEditorViewportInteractionCustomization.h"
#include "Elements/Component/ComponentElementLevelEditorViewportInteractionCustomization.h"
#include "AudioDevice.h"
#include "MouseDeltaTracker.h"
#include "ScopedTransaction.h"
#include "HModel.h"
#include "Layers/LayersSubsystem.h"
#include "StaticLightingSystem/StaticLightingPrivate.h"
#include "SEditorViewport.h"
#include "LevelEditor.h"
#include "LevelViewportActions.h"
#include "SLevelViewport.h"
#include "AssetSelection.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "IPlacementModeModule.h"
#include "Engine/Polys.h"
#include "ActorEditorUtils.h"
#include "ObjectTools.h"
#include "PackageTools.h"
#include "SnappingUtils.h"
#include "ViewportSelectionUtilities.h"
#include "DragTool_BoxSelect.h"
#include "DragTool_FrustumSelect.h"
#include "DragTool_Measure.h"
#include "DragTool_ViewportChange.h"
#include "DragAndDrop/BrushBuilderDragDropOp.h"
#include "DynamicMeshBuilder.h"
#include "Editor/ActorPositioning.h"
#include "Editor/ObjectPositioning.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Settings/EditorProjectSettings.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "ContentStreaming.h"
#include "IHeadMountedDisplay.h"
#include "IXRTrackingSystem.h"
#include "ActorGroupingUtils.h"
#include "EditorInteractiveGizmoManager.h"
#include "EditorWorldExtension.h"
#include "VREditorMode.h"
#include "ViewportClientNavigationHelper.h"
#include "ViewportWorldInteraction.h"
#include "Subsystems/BrushEditingSubsystem.h"
#include "Engine/VolumeTexture.h"
#include "Engine/TextureCube.h"
#include "Materials/MaterialExpressionDivide.h"
#include "Materials/MaterialExpressionSubtract.h"
#include "Materials/MaterialExpressionTransformPosition.h"
#include "Materials/MaterialExpressionWorldPosition.h"
#include "Materials/MaterialExpressionReflectionVectorWS.h"
#include "Materials/MaterialExpressionBounds.h"
#include "LevelEditorDragDropHandler.h"
#include "UnrealWidget.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"
#include "Rendering/StaticLightingSystemInterface.h"
#include "TextureResource.h"
#include "Subsystems/PlacementSubsystem.h"
#include "Subsystems/EditorElementSubsystem.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"
#include "ViewportInteractions/ViewportInteractionsUtilities.h"

DEFINE_LOG_CATEGORY(LogEditorViewport);

#define LOCTEXT_NAMESPACE "LevelEditorViewportClient"
const FLevelViewportActorLock FLevelViewportActorLock::None(nullptr);

TWeakPtr<FTypedElementList> FLevelEditorViewportClient::StaticDropPreviewElements;
TArray< TWeakObjectPtr< AActor > > FLevelEditorViewportClient::DropPreviewActors;
TMap< TObjectKey< AActor >, TWeakObjectPtr< UActorComponent > > FLevelEditorViewportClient::ViewComponentForActorCache;

bool FLevelEditorViewportClient::bIsDroppingPreviewActor;

/** Static: List of objects we're hovering over */
TSet<FViewportHoverTarget> FLevelEditorViewportClient::HoveredObjects;

IMPLEMENT_HIT_PROXY( HLevelSocketProxy, HHitProxy );

namespace LevelEditorViewportLocals
{
	const FText DropObjectsTransactionName = LOCTEXT("PlaceObjects", "Place Objects");

	/**
	 * Calls Func on each element in ElementArray that represents an actor.
	 */
	void ForEachActorInElementArray(TArray<FTypedElementHandle> ElementArray, TFunctionRef<void(AActor&)> Func)
	{
		UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();
		if (!ensure(Registry))
		{
			return;
		}

		for (const FTypedElementHandle& Element : ElementArray)
		{
			TTypedElement<ITypedElementObjectInterface> ObjectInterface = Registry->GetElement<ITypedElementObjectInterface>(Element);
			if (!ObjectInterface)
			{
				continue;
			}
			AActor* Actor = ObjectInterface.GetObjectAs<AActor>();
			if (!Actor)
			{
				continue;
			}

			Func(*Actor);
		}
	}
}

/** Helper function to compute a new location that is snapped to the origin plane given the users cursor location and camera angle */
static FVector4 AttemptToSnapLocationToOriginPlane( const FViewportCursorLocation& Cursor, FVector4 Location )
{
	ELevelViewportType ViewportType = Cursor.GetViewportType();
	if ( ViewportType == LVT_Perspective )
	{
		FVector CamPos = Cursor.GetViewportClient()->GetViewLocation();

		FVector NewLocFloor( Location.X, Location.Y, 0 );

		bool CamBelowOrigin = CamPos.Z < 0;

		FPlane CamPlane( CamPos, FVector::UpVector );
		// If the camera is looking at the floor place the brush on the floor
		if ( !CamBelowOrigin && CamPlane.PlaneDot( Location ) < 0 )
		{
			Location = NewLocFloor;
		}
		else if ( CamBelowOrigin && CamPlane.PlaneDot( Location ) > 0 )
		{
			Location = NewLocFloor;
		}
	}
	else if ( ViewportType == LVT_OrthoXY || ViewportType == LVT_OrthoNegativeXY )
	{
		// In ortho place the brush at the origin of the hidden axis
		Location.Z = 0;
	}
	else if ( ViewportType == LVT_OrthoXZ || ViewportType == LVT_OrthoNegativeXZ )
	{
		// In ortho place the brush at the origin of the hidden axis
		Location.Y = 0;
	}
	else if ( ViewportType == LVT_OrthoYZ || ViewportType == LVT_OrthoNegativeYZ )
	{
		// In ortho place the brush at the origin of the hidden axis
		Location.X = 0;
	}

	return Location;
}

/** Helper function to get an atmosphere light from an index*/
static UDirectionalLightComponent* GetAtmosphericLight(const uint8 DesiredLightIndex, UWorld* ViewportWorld)
{
	UDirectionalLightComponent* SelectedAtmosphericLight = nullptr;
	float SelectedLightLuminance = 0.0f;
	for (TObjectIterator<UDirectionalLightComponent> ComponentIt; ComponentIt; ++ComponentIt)
	{
		if (ComponentIt->GetWorld() == ViewportWorld)
		{
			UDirectionalLightComponent* AtmosphericLight = *ComponentIt;

			if (!AtmosphericLight->IsUsedAsAtmosphereSunLight() || AtmosphericLight->GetAtmosphereSunLightIndex() != DesiredLightIndex || !AtmosphericLight->GetVisibleFlag())
				continue;

			float LightLuminance = AtmosphericLight->GetColoredLightBrightness().GetLuminance();
			if (!SelectedAtmosphericLight ||					// Set it if null
				SelectedLightLuminance < LightLuminance)		// Or choose the brightest atmospheric light
			{
				SelectedAtmosphericLight = AtmosphericLight;
			}
		}
	}
	return SelectedAtmosphericLight;
}

static void NotifyAtmosphericLightHasMoved(UDirectionalLightComponent& SelectedAtmosphericLight, bool bFinished)
{
	AActor* LightOwner = SelectedAtmosphericLight.GetOwner();
	if (LightOwner)
	{
		// Now notify the owner about the transform update, e.g. construction script on instance.
		LightOwner->PostEditMove(bFinished);
		// No PostEditChangeProperty because not paired with a PreEditChange

		if (bFinished)
		{
			SelectedAtmosphericLight.InvalidateLightingCache();
		}
		
		FStaticLightingSystemInterface::OnSkyAtmosphereModified.Broadcast();
	}
}

TArray<FTypedElementHandle> FLevelEditorViewportClient::TryPlacingAssetObject(ULevel* InLevel, UObject* ObjToUse,
	const UE::AssetPlacementUtil::FExtraPlaceAssetOptions& PlacementOptions,
	const FViewportCursorLocation* CursorInformation)
{
	using namespace LevelEditorViewportLocals;

	// This is used for issuing a legacy notification.
	TArray<AActor*> PlacedActors;

	// Our actual output.
	TArray<FTypedElementHandle> PlacedItems;

	// Helper to update PlacedActors and PlacedItems. Can be called with null actor.
	auto AddActorToPlaced = [&PlacedItems, &PlacedActors](AActor* Actor)
	{
		if (!Actor)
		{
			return;
		}

		PlacedActors.Add(Actor);

		FTypedElementHandle Handle = UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor);
		if (ensure(Handle))
		{
			PlacedItems.Add(Handle);
		}
	};

	if (!ensure(ObjToUse))
	{
		return PlacedItems;
	}

	UClass* ObjectClass = Cast<UClass>(ObjToUse);

	if ( ObjectClass == NULL )
	{
		ObjectClass = ObjToUse->GetClass();
	}

	if (!ensure(ObjectClass))
	{
		return PlacedItems;
	}

	bool bPlaced = false;
	if ( ObjectClass->IsChildOf( AActor::StaticClass() ) )
	{
		//Attempting to drop a UClass object
		UActorFactory* ActorFactory = Cast<UActorFactory>(PlacementOptions.FactoryToUse.GetObject());
		if ( ActorFactory == NULL )
		{
			ActorFactory = GEditor->FindActorFactoryForActorClass( ObjectClass );
		}

		if (ActorFactory != NULL )
		{
			UE::AssetPlacementUtil::FExtraPlaceAssetOptions OverridenPlacementOptions = PlacementOptions;
			OverridenPlacementOptions.FactoryToUse = TScriptInterface<IAssetFactoryInterface>(ActorFactory);
			PlacedItems = UE::AssetPlacementUtil::PlaceAssetInCurrentLevel(ObjToUse, OverridenPlacementOptions);
			ForEachActorInElementArray(PlacedItems, [&PlacedActors](AActor& Actor)
			{
				PlacedActors.Add(&Actor);
			});
			bPlaced = PlacedItems.Num() > 0;
		}
		
		if (!bPlaced && !ObjectClass->HasAnyClassFlags(CLASS_NotPlaceable | CLASS_Abstract) )
		{
			// If no actor factory was found or failed, add the actor directly.
	
			// TODO: We might want to investigate using the above PlaceAssetInCurrentLevel path even when we
			// don't have an actor factory, and only use this legacy one if that fails for whatever reason.
			// Note that this path acts a bit differently. For instance it doesn't use IsDroppingPreviewActor()
			// to set bIsEditorPrieviewActor on the output if we are dropping a preview. We fix this in
			// DropObjectsAtCoordinates where we directly know whether we are adding to previews (instead of using
			// the static IsDroppingPreviewActor).
			const FTransform ActorTransform = FActorPositioning::GetCurrentViewportPlacementTransform(
				*ObjectClass->GetDefaultObject<AActor>(), /*bSnap=*/true, CursorInformation);
			AActor* Actor = GEditor->AddActor( InLevel, ObjectClass, ActorTransform, 
				/*bSilent=*/false, PlacementOptions.ObjectFlags, PlacementOptions.bSelectOutput);
			AddActorToPlaced(Actor);
			bPlaced = (Actor != nullptr);
		}
	}
	
	if (!bPlaced && ObjToUse->IsA( UExportTextContainer::StaticClass() ) )
	{
		// TODO: This path probably needs fixing for non-actor placement

		UExportTextContainer* ExportContainer = CastChecked<UExportTextContainer>(ObjToUse);
		const TArray<AActor*> NewActors = GEditor->AddExportTextActors( ExportContainer->ExportText, 
			/*bSilent*/false, PlacementOptions.ObjectFlags);
		for (AActor* Actor : NewActors)
		{
			AddActorToPlaced(Actor);
		}
	}
	else if (!bPlaced && ObjToUse->IsA( UBrushBuilder::StaticClass() ) )
	{
		UBrushBuilder* BrushBuilder = CastChecked<UBrushBuilder>(ObjToUse);
		UWorld* World = InLevel->OwningWorld;
		BrushBuilder->Build(World);

		ABrush* DefaultBrush = World->GetDefaultBrush();
		if (DefaultBrush != NULL)
		{
			FVector ActorLoc = GEditor->ClickLocation + GEditor->ClickPlane * (FVector::BoxPushOut(GEditor->ClickPlane, DefaultBrush->GetPlacementExtent()));
			FSnappingUtils::SnapPointToGrid(ActorLoc, FVector::ZeroVector);

			DefaultBrush->SetActorLocation(ActorLoc);
			AddActorToPlaced(DefaultBrush);
		}
	}
	else if (!bPlaced)
	{
		bool bShouldPlace = true;
		if (ObjectClass->IsChildOf(UBlueprint::StaticClass()))
		{
			UBlueprint* BlueprintObj = StaticCast<UBlueprint*>(ObjToUse);
			bShouldPlace = BlueprintObj->GeneratedClass != NULL;
			if (bShouldPlace)
			{
				check(BlueprintObj->ParentClass == BlueprintObj->GeneratedClass->GetSuperClass());
				if (BlueprintObj->GeneratedClass->HasAnyClassFlags(CLASS_NotPlaceable | CLASS_Abstract))
				{
					bShouldPlace = false;
				}
			}
		}

		if (bShouldPlace)
		{
			PlacedItems = UE::AssetPlacementUtil::PlaceAssetInCurrentLevel(ObjToUse, PlacementOptions);
			
			LevelEditorViewportLocals::ForEachActorInElementArray(PlacedItems, [&PlacedActors](AActor& Actor)
			{
				PlacedActors.Add(&Actor);
				// Not clear why this would be necessary, but kept for legacy behavior.
				Actor.PostEditMove(true);
			});
		}
	}

	if (PlacedActors.Num() > 0)
	{
		FEditorDelegates::OnNewActorsPlaced.Broadcast(ObjToUse, PlacedActors);
	}

	return PlacedItems;
}

TArray<AActor*> FLevelEditorViewportClient::TryPlacingActorFromObject(ULevel* InLevel, UObject* ObjToUse,
	bool bSelectActors, EObjectFlags ObjectFlags, UActorFactory* FactoryToUse,
	const FName Name, const FViewportCursorLocation* Cursor)
{
	TArray<AActor*> OutputActors;

	// Forward the call to TryPlacingAssetObject
	UE::AssetPlacementUtil::FExtraPlaceAssetOptions PlacementOptions;
	PlacementOptions.bSelectOutput = bSelectActors;
	PlacementOptions.ObjectFlags = ObjectFlags;
	PlacementOptions.FactoryToUse = FactoryToUse;
	PlacementOptions.Name = Name;

	TArray<FTypedElementHandle> PlacedItems = TryPlacingAssetObject(InLevel, ObjToUse, PlacementOptions, Cursor);

	// Filter out actors into the actors array
	LevelEditorViewportLocals::ForEachActorInElementArray(PlacedItems, [&OutputActors](AActor& Actor)
	{
		OutputActors.Add(&Actor);
	});

	return OutputActors;
}

static FString GetSharedTextureNameAndKind( FString TextureName, EMaterialKind& Kind)
{
	// Try and strip the suffix from the texture name, if we're successful it must be of that type.
	bool hasBaseSuffix = TextureName.RemoveFromEnd( "_D" ) || TextureName.RemoveFromEnd( "_Diff" ) || TextureName.RemoveFromEnd( "_Diffuse" ) || TextureName.RemoveFromEnd( "_Detail" ) || TextureName.RemoveFromEnd( "_Base" );
	if ( hasBaseSuffix )
	{
		Kind = EMaterialKind::Base;
		return TextureName;
	}

	bool hasNormalSuffix = TextureName.RemoveFromEnd( "_N" ) || TextureName.RemoveFromEnd( "_Norm" ) || TextureName.RemoveFromEnd( "_Normal" );
	if ( hasNormalSuffix )
	{
		Kind = EMaterialKind::Normal;
		return TextureName;
	}
	
	bool hasSpecularSuffix = TextureName.RemoveFromEnd( "_S" ) || TextureName.RemoveFromEnd( "_Spec" ) || TextureName.RemoveFromEnd( "_Specular" );
	if ( hasSpecularSuffix )
	{
		Kind = EMaterialKind::Specular;
		return TextureName;
	}

	bool hasEmissiveSuffix = TextureName.RemoveFromEnd( "_E" ) || TextureName.RemoveFromEnd( "_Emissive" );
	if ( hasEmissiveSuffix )
	{
		Kind = EMaterialKind::Emissive;
		return TextureName;
	}

	Kind = EMaterialKind::Unknown;
	return TextureName;
}

static UTexture* GetTextureWithNameVariations( const FString& BasePackageName, const TArray<FString>& Suffixes )
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>( TEXT( "AssetRegistry" ) );

	// Try all the variations of suffixes, if we find a package matching the suffix, return it.
	const FTopLevelAssetPath Texture2DClassPath(TEXT("/Script/Engine"), TEXT("Texture2D"));
	for ( int i = 0; i < Suffixes.Num(); i++ )
	{
		TArray<FAssetData> OutAssetData;
		if ( AssetRegistryModule.Get().GetAssetsByPackageName( *( BasePackageName + Suffixes[i] ), OutAssetData ) && OutAssetData.Num() > 0 )
		{
			if ( OutAssetData[0].AssetClassPath == Texture2DClassPath )
			{
				return Cast<UTexture>(OutAssetData[0].GetAsset());
			}
		}
	}

	return nullptr;
}

static bool TryAndCreateMaterialInput( UMaterial* UnrealMaterial, EMaterialKind TextureKind, UTexture* UnrealTexture, FExpressionInput& MaterialInput, int X, int Y )
{
	// Ignore null textures.
	if ( UnrealTexture == nullptr )
	{
		return false;
	}

	bool bTextureHasAlpha = !UnrealTexture->CompressionNoAlpha;
	if ( bTextureHasAlpha )
	{
		if ( const UTexture2D* Texture2D = Cast<UTexture2D>(UnrealTexture) )
		{
			const EPixelFormatChannelFlags ValidTextureChannels = GetPixelFormatValidChannels(Texture2D->GetPixelFormat());
			bTextureHasAlpha = EnumHasAnyFlags(ValidTextureChannels, EPixelFormatChannelFlags::A);
		}
	}

	const bool bSetupAsNormalMap = UnrealTexture->IsNormalMap();

	UMaterialEditorOnlyData* UnrealMaterialEditorOnly = UnrealMaterial->GetEditorOnlyData();
	
	UnrealMaterial->BlendMode = bTextureHasAlpha ? EBlendMode::BLEND_Masked : EBlendMode::BLEND_Opaque;

	// Create a new texture sample expression, this is our texture input node into the material output.
	UMaterialExpressionTextureSample* UnrealTextureExpression = NewObject<UMaterialExpressionTextureSample>(UnrealMaterial);
	UnrealMaterial->GetExpressionCollection().AddExpression( UnrealTextureExpression );
	MaterialInput.Expression = UnrealTextureExpression;
	UnrealTextureExpression->Texture = UnrealTexture;
	UnrealTextureExpression->AutoSetSampleType();
	UnrealTextureExpression->MaterialExpressionEditorX += X;
	UnrealTextureExpression->MaterialExpressionEditorY += Y;

	if ( UnrealTexture->IsA<UVolumeTexture>() )
	{
		// If it's a volume texture, build an expression which computes UVW coordinates from bounds-relative pixel position.
		UMaterialExpressionDivide* DivideExpression = NewObject<UMaterialExpressionDivide>(UnrealMaterial);
		UMaterialExpressionSubtract* BoundsRelativePosExpression = NewObject<UMaterialExpressionSubtract>(UnrealMaterial);
		UMaterialExpressionSubtract* BoundsSizeExpression = NewObject<UMaterialExpressionSubtract>(UnrealMaterial);
		UMaterialExpressionBounds* LocalBoundsExpression = NewObject<UMaterialExpressionBounds>(UnrealMaterial);
		UMaterialExpressionTransformPosition* TransformPositionExpression = NewObject<UMaterialExpressionTransformPosition>(UnrealMaterial);
		UMaterialExpressionWorldPosition* WorldPosExpression = NewObject<UMaterialExpressionWorldPosition>(UnrealMaterial);

		UnrealMaterial->GetExpressionCollection().AddExpression( DivideExpression );
		UnrealMaterial->GetExpressionCollection().AddExpression( BoundsRelativePosExpression );
		UnrealMaterial->GetExpressionCollection().AddExpression( BoundsSizeExpression );
		UnrealMaterial->GetExpressionCollection().AddExpression( LocalBoundsExpression );
		UnrealMaterial->GetExpressionCollection().AddExpression( TransformPositionExpression );
		UnrealMaterial->GetExpressionCollection().AddExpression( WorldPosExpression );

		int32 EditorPosX = UnrealTextureExpression->MaterialExpressionEditorX;
		int32 EditorPosY = UnrealTextureExpression->MaterialExpressionEditorY;

		UnrealTextureExpression->Coordinates.Expression = DivideExpression;

		EditorPosX -= 150;

		DivideExpression->A.Expression = BoundsRelativePosExpression;
		DivideExpression->B.Expression = BoundsSizeExpression;
		DivideExpression->MaterialExpressionEditorX = EditorPosX;
		DivideExpression->MaterialExpressionEditorY = EditorPosY;

		EditorPosX -= 150;

		BoundsRelativePosExpression->A.Expression = TransformPositionExpression;
		BoundsRelativePosExpression->B.Expression = LocalBoundsExpression;
		BoundsRelativePosExpression->B.OutputIndex = 2;

		BoundsRelativePosExpression->MaterialExpressionEditorX = EditorPosX;
		BoundsRelativePosExpression->MaterialExpressionEditorY = EditorPosY;

		BoundsSizeExpression->A.Expression = LocalBoundsExpression;
		BoundsSizeExpression->A.OutputIndex = UMaterialExpressionBounds::BoundsMaxOutputIndex;
		BoundsSizeExpression->B.Expression = LocalBoundsExpression;
		BoundsSizeExpression->B.OutputIndex = UMaterialExpressionBounds::BoundsMinOutputIndex;
		BoundsSizeExpression->MaterialExpressionEditorX = EditorPosX;
		BoundsSizeExpression->MaterialExpressionEditorY = EditorPosY + 100;

		EditorPosX -= 300;

		TransformPositionExpression->Input.Expression = WorldPosExpression;
		TransformPositionExpression->TransformSourceType = TRANSFORMPOSSOURCE_World;
		TransformPositionExpression->TransformType = TRANSFORMPOSSOURCE_Local;
		TransformPositionExpression->MaterialExpressionEditorX = EditorPosX;
		TransformPositionExpression->MaterialExpressionEditorY = EditorPosY;

		LocalBoundsExpression->Type = MEILB_ObjectLocal;
		LocalBoundsExpression->MaterialExpressionEditorX = EditorPosX;
		LocalBoundsExpression->MaterialExpressionEditorY = EditorPosY + 100;

		EditorPosX -= 250;

		WorldPosExpression->WorldPositionShaderOffset = WPT_Default;
		WorldPosExpression->MaterialExpressionEditorX = EditorPosX;
		WorldPosExpression->MaterialExpressionEditorY = EditorPosY;
	}
	else if (UnrealTexture->IsA<UTextureCube>())
	{
		// If it's a cube texture, add a reflection vector expression to the UVW input to satisfy the expression input requirement
		UMaterialExpressionReflectionVectorWS* ReflectionVectorExpression = NewObject<UMaterialExpressionReflectionVectorWS>(UnrealMaterial);

		UnrealMaterial->GetExpressionCollection().AddExpression(ReflectionVectorExpression);

		ReflectionVectorExpression->MaterialExpressionEditorX = UnrealTextureExpression->MaterialExpressionEditorX - 250; // Place node to the left to avoid overlap
		ReflectionVectorExpression->MaterialExpressionEditorY = UnrealTextureExpression->MaterialExpressionEditorY;

		UnrealTextureExpression->Coordinates.Expression = ReflectionVectorExpression;
	}

	// If we know for a fact this is a normal map, it can only legally be placed in the normal map slot.
	// Ignore the Material kind for, but for everything else try and match it to the right slot, fallback
	// to the BaseColor if we don't know.
	if ( !bSetupAsNormalMap )
	{
		if ( TextureKind == EMaterialKind::Base )
		{
			UnrealMaterialEditorOnly->BaseColor.Expression = UnrealTextureExpression;
			if ( bTextureHasAlpha && UnrealTextureExpression->Outputs.IsValidIndex(4) )
			{
				UnrealMaterialEditorOnly->Opacity.Connect(4, UnrealTextureExpression);
				UnrealMaterialEditorOnly->OpacityMask.Connect(4, UnrealTextureExpression);
			}
		}
		else if ( TextureKind == EMaterialKind::Specular )
		{
			UnrealMaterialEditorOnly->Specular.Expression = UnrealTextureExpression;
		}
		else if ( TextureKind == EMaterialKind::Emissive )
		{
			UnrealMaterialEditorOnly->EmissiveColor.Expression = UnrealTextureExpression;
		}
		else
		{
			UnrealMaterialEditorOnly->BaseColor.Expression = UnrealTextureExpression;
		}
	}
	else
	{
		UnrealMaterialEditorOnly->Normal.Expression = UnrealTextureExpression;
	}


	return true;
}

UObject* FLevelEditorViewportClient::GetOrCreateMaterialFromTexture(UTexture* UnrealTexture)
{
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

	// Check if a base material and corresponding params are set in the Settings to determine whether to create a Material or MIC
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
	UMaterialInterface* BaseMaterial = ViewportSettings->MaterialForDroppedTextures.LoadSynchronous();
	const bool bCreateMaterialInstance = BaseMaterial && ViewportSettings->MaterialParamsForDroppedTextures.Num() > 0;

	const FString TexturePackageName = UnrealTexture->GetPackage()->GetName();
	FString TextureShortName = FPackageName::GetShortName(TexturePackageName);

	// See if we can figure out what kind of material it is, based on a suffix, like _S for Specular, _D for Base/Detail/Diffuse.
	// if it can determine which type of texture it was, it will return the base name of the texture minus the suffix.
	EMaterialKind MaterialKind;
	TextureShortName = GetSharedTextureNameAndKind( TextureShortName, MaterialKind );

	FString NewPackageFolder;
	if (AssetTools.GetWritableFolderPermissionList()->PassesStartsWithFilter(TexturePackageName))
	{
		// Create the material in the source texture folder
		NewPackageFolder = FPackageName::GetLongPackagePath(TexturePackageName);
	}
	if (NewPackageFolder.IsEmpty())
	{
		// Create the material in the current world folder?
		if (UWorld* EditorWorld = GEditor->GetEditorWorldContext().World())
		{
			const FString EditorWorldPackageName = EditorWorld->GetPackage()->GetName();
			if (!FPackageName::IsTempPackage(EditorWorldPackageName))
			{
				NewPackageFolder = FPackageName::GetLongPackagePath(EditorWorldPackageName);
				if (!AssetTools.GetWritableFolderPermissionList()->PassesStartsWithFilter(NewPackageFolder))
				{
					NewPackageFolder.Reset();
				}
			}
		}
	}
	if (NewPackageFolder.IsEmpty())
	{
		// Create the material in the last directory an asset was created in?
		if (FPackageName::TryConvertFilenameToLongPackageName(FEditorDirectories::Get().GetLastDirectory(ELastDirectory::NEW_ASSET), NewPackageFolder))
		{
			if (!AssetTools.GetWritableFolderPermissionList()->PassesStartsWithFilter(NewPackageFolder))
			{
				NewPackageFolder.Reset();
			}
		}
		else
		{
			NewPackageFolder.Reset();
		}
	}
	if (NewPackageFolder.IsEmpty())
	{
		// Create the material in the game root folder
		NewPackageFolder = TEXT("/Game");
	}

	const FString MaterialFullName = bCreateMaterialInstance ? (TEXT("MI_") + TextureShortName) : (TextureShortName + TEXT("_Mat"));
	FString NewPackageName = FPaths::Combine(NewPackageFolder, MaterialFullName);
	NewPackageName = UPackageTools::SanitizePackageName(NewPackageName);	

	// See if the material asset already exists with the expected name, if it does, just return
	// an instance of it.
	TArray<FAssetData> OutAssetData;
	if (AssetRegistry.GetAssetsByPackageName(*NewPackageName, OutAssetData) && (OutAssetData.Num() > 0))
	{
		UObject* FoundAsset = OutAssetData[0].GetAsset();
		if (FoundAsset != nullptr && // Due to redirects we might have an asset by this name that actually doesn't exist under that name anymore.
			FoundAsset->IsA(UMaterialInterface::StaticClass()))
		{
			return FoundAsset;
		}
		UE_LOGFMT(LogEditorViewport, Warning, "Failed to create material {NewPackageName} from texture because a non-material asset already exists with that name", NewPackageName);
		return nullptr;
	}

	UPackage* Package = CreatePackage(*NewPackageName);

	// Variations for Base Maps.
	TArray<FString> BaseSuffixes;
	BaseSuffixes.Add("_D");
	BaseSuffixes.Add("_Diff");
	BaseSuffixes.Add("_Diffuse");
	BaseSuffixes.Add("_Detail");
	BaseSuffixes.Add("_Base");

	// Variations for Normal Maps.
	TArray<FString> NormalSuffixes;
	NormalSuffixes.Add("_N");
	NormalSuffixes.Add("_Norm");
	NormalSuffixes.Add("_Normal");

	// Variations for Specular Maps.
	TArray<FString> SpecularSuffixes;
	SpecularSuffixes.Add("_S");
	SpecularSuffixes.Add("_Spec");
	SpecularSuffixes.Add("_Specular");

	// Variations for Emissive Maps.
	TArray<FString> EmissiveSuffixes;
	EmissiveSuffixes.Add("_E");
	EmissiveSuffixes.Add("_Emissive");

	// The asset path for the base texture, we need this to try and append different suffixes to to find other textures we can use.
	const FString BaseTexturePackage = FPackageName::GetLongPackagePath(TexturePackageName) + TEXT("/") + TextureShortName;

	UMaterialInterface* CreatedMaterialInterface = nullptr;

	if (bCreateMaterialInstance)
	{
		UMaterialInstanceConstantFactoryNew* Factory = NewObject<UMaterialInstanceConstantFactoryNew>();
		Factory->InitialParent = BaseMaterial;
		UMaterialInstanceConstant* CreatedMIC = Cast<UMaterialInstanceConstant>(Factory->FactoryCreateNew(UMaterialInstanceConstant::StaticClass(), Package, *MaterialFullName, RF_Standalone | RF_Public, NULL, GWarn));
		if (!CreatedMIC)
		{
			return nullptr;
		}
		CreatedMaterialInterface = CreatedMIC;

		if (MaterialKind == EMaterialKind::Unknown)
		{
			// If the texture type cannot be determined, treat it as a base texture and don't look for any matching textures
			if (const FName* UnknownParam = ViewportSettings->MaterialParamsForDroppedTextures.Find(EMaterialKind::Unknown))
			{
				CreatedMIC->SetTextureParameterValueEditorOnly(*UnknownParam, UnrealTexture);
			}
			else if (const FName* BaseParam = ViewportSettings->MaterialParamsForDroppedTextures.Find(EMaterialKind::Base))
			{
				CreatedMIC->SetTextureParameterValueEditorOnly(*BaseParam, UnrealTexture);
			}
			else
			{
				UE_LOGFMT(LogEditorViewport, Warning, "Dropped texture not assigned to material instance {MaterialFullName} because no material parameter name was defined for Unknown or Base in LevelEditorViewportSettings", MaterialFullName);
			}
		}
		else
		{
			auto AssignParam = [ViewportSettings, CreatedMIC, BaseTexturePackage](EMaterialKind ParamKind, TArray<FString>& Suffixes)->bool
			{
				if (const FName* Param = ViewportSettings->MaterialParamsForDroppedTextures.Find(ParamKind))
				{
					if (UTexture* Texture = GetTextureWithNameVariations(BaseTexturePackage, Suffixes))
					{
						CreatedMIC->SetTextureParameterValueEditorOnly(*Param, Texture);
						return true;
					}
				}
				return false;
			};

			// The passed-in texture should be found and assigned in one of the below functions, however it's possible this fails because
			// the user hasn't set up their Settings properly. If another type of texture is found but not defined in the settings, then
			// those can just silently fail - only warn the user if the specific texture they chose failed to assign.
			bool bAssignedInputTexture = false;
			bAssignedInputTexture |= AssignParam(EMaterialKind::Base, BaseSuffixes) && MaterialKind == EMaterialKind::Base;
			bAssignedInputTexture |= AssignParam(EMaterialKind::Normal, NormalSuffixes) && MaterialKind == EMaterialKind::Normal;
			bAssignedInputTexture |= AssignParam(EMaterialKind::Specular, SpecularSuffixes) && MaterialKind == EMaterialKind::Specular;
			bAssignedInputTexture |= AssignParam(EMaterialKind::Emissive, EmissiveSuffixes) && MaterialKind == EMaterialKind::Emissive;
			if (!bAssignedInputTexture)
			{
				UE_LOGFMT(LogEditorViewport, Warning, "Dropped texture not assigned to material instance {MaterialFullName} because no material parameter name was defined in LevelEditorViewportSettings", MaterialFullName);
			}
		}
	}
	else
	{
		// create an unreal material asset
		UMaterialFactoryNew* MaterialFactory = NewObject<UMaterialFactoryNew>();

		UMaterial* UnrealMaterial = (UMaterial*)MaterialFactory->FactoryCreateNew(
			UMaterial::StaticClass(), Package, *MaterialFullName, RF_Standalone | RF_Public | RF_Transactional, NULL, GWarn);

		if (UnrealMaterial == nullptr)
		{
			return nullptr;
		}
		CreatedMaterialInterface = UnrealMaterial;

		UMaterialEditorOnlyData* UnrealMaterialEditorOnly = UnrealMaterial->GetEditorOnlyData();

		const int HSpace = -300;

		// If we were able to figure out the material kind, we need to try and build a complex material
		// involving multiple textures.  If not, just try and connect what we found to the base map.
		if (MaterialKind == EMaterialKind::Unknown)
		{
			TryAndCreateMaterialInput(UnrealMaterial, EMaterialKind::Base, UnrealTexture, UnrealMaterialEditorOnly->BaseColor, HSpace, 0);
		}
		else
		{
			// Try and find different variations
			UTexture* BaseTexture = GetTextureWithNameVariations(BaseTexturePackage, BaseSuffixes);
			UTexture* NormalTexture = GetTextureWithNameVariations(BaseTexturePackage, NormalSuffixes);
			UTexture* SpecularTexture = GetTextureWithNameVariations(BaseTexturePackage, SpecularSuffixes);
			UTexture* EmissiveTexture = GetTextureWithNameVariations(BaseTexturePackage, EmissiveSuffixes);

			// Connect and layout any textures we find into their respective inputs in the material.
			const int VSpace = 170;
			TryAndCreateMaterialInput(UnrealMaterial, EMaterialKind::Base, BaseTexture, UnrealMaterialEditorOnly->BaseColor, HSpace, VSpace * -1);
			TryAndCreateMaterialInput(UnrealMaterial, EMaterialKind::Specular, SpecularTexture, UnrealMaterialEditorOnly->Specular, HSpace, VSpace * 0);
			TryAndCreateMaterialInput(UnrealMaterial, EMaterialKind::Emissive, EmissiveTexture, UnrealMaterialEditorOnly->EmissiveColor, HSpace, VSpace * 1);
			TryAndCreateMaterialInput(UnrealMaterial, EMaterialKind::Normal, NormalTexture, UnrealMaterialEditorOnly->Normal, HSpace, VSpace * 2);
		}
	}

	CreatedMaterialInterface->PreEditChange(nullptr);
	CreatedMaterialInterface->PostEditChange();

	// Notify the asset registry
	FAssetRegistryModule::AssetCreated(CreatedMaterialInterface);

	// Set the dirty flag so this package will get saved later
	Package->SetDirtyFlag( true );

	CreatedMaterialInterface->ForceRecompileForRendering();

	// Warn users that a new material has been created
	FNotificationInfo Info( FText::Format( LOCTEXT( "DropTextureMaterialCreated", "{0} '{1}' Created" ), CreatedMaterialInterface->GetClass()->GetDisplayNameText(), FText::FromString(MaterialFullName)));
	Info.ExpireDuration = 4.0f;
	Info.bUseLargeFont = true;
	Info.bUseSuccessFailIcons = false;
	Info.Image = FAppStyle::GetBrush( "ClassThumbnail.Material" );
	FSlateNotificationManager::Get().AddNotification( Info );

	return CreatedMaterialInterface;
}

/**
* Helper function that attempts to apply the supplied object to the supplied actor.
*
* @param	ObjToUse				Object to attempt to apply as specific asset
* @param	ComponentToApplyTo		Component to whom the asset should be applied
* @param	TargetMaterialSlot      When dealing with submeshes this will represent the target section/slot to apply materials to.
* @param	bTest					Whether to test if the object would be successfully applied without actually doing it.
*
* @return	true if the provided object was successfully applied to the provided actor
*/
static bool AttemptApplyObjToComponent(UObject* ObjToUse, USceneComponent* ComponentToApplyTo, int32 TargetMaterialSlot = -1, bool bTest = false)
{
	bool bResult = false;

	if (ComponentToApplyTo && !ComponentToApplyTo->IsCreatedByConstructionScript())
	{
		// MESH/DECAL
		UMeshComponent* MeshComponent = Cast<UMeshComponent>(ComponentToApplyTo);
		UDecalComponent* DecalComponent = Cast<UDecalComponent>(ComponentToApplyTo);
		if (MeshComponent || DecalComponent)
		{
			// Dropping a texture?
			UTexture* DroppedObjAsTexture = Cast<UTexture>(ObjToUse);
			if (DroppedObjAsTexture != NULL)
			{
				if (bTest)
				{
					bResult = false;
				}
				else
				{
					// Turn dropped textures into materials
					ObjToUse = FLevelEditorViewportClient::GetOrCreateMaterialFromTexture(DroppedObjAsTexture);
				}
			}

			// Dropping a material?
			UMaterialInterface* DroppedObjAsMaterial = Cast<UMaterialInterface>(ObjToUse);
			if (DroppedObjAsMaterial)
			{
				if (bTest)
				{
					bResult = false;
				}
				else
				{
					bResult = FComponentEditorUtils::AttemptApplyMaterialToComponent(ComponentToApplyTo, DroppedObjAsMaterial, TargetMaterialSlot);

					if (bResult)
					{
						GEditor->OnSceneMaterialsModified();
					}
				}
			}
		}

		// SKELETAL MESH COMPONENT
		USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(ComponentToApplyTo);
		if (SkeletalMeshComponent)
		{
			// Dropping an Anim Blueprint?
			UAnimBlueprint* DroppedObjAsAnimBlueprint = Cast<UAnimBlueprint>(ObjToUse);
			if (DroppedObjAsAnimBlueprint)
			{
				USkeleton* AnimBPSkeleton = DroppedObjAsAnimBlueprint->TargetSkeleton;
				if (AnimBPSkeleton)
				{
					if (bTest)
					{
						bResult = true;
					}
					else
					{
						const FScopedTransaction Transaction(LOCTEXT("DropAnimBlueprintOnObject", "Drop Anim Blueprint On Object"));
						SkeletalMeshComponent->Modify();

						// If the component doesn't have a mesh, the mesh should change
						const bool bShouldChangeMesh = !SkeletalMeshComponent->GetSkeletalMeshAsset();

						if (bShouldChangeMesh)
						{
							SkeletalMeshComponent->SetSkeletalMesh(AnimBPSkeleton->GetPreviewMesh(true));
						}

						SkeletalMeshComponent->SetAnimInstanceClass(DroppedObjAsAnimBlueprint->GeneratedClass);
						bResult = true;
					}
				}
			}

			// Dropping an Anim Sequence or Vertex Animation?
			UAnimSequenceBase* DroppedObjAsAnimSequence = Cast<UAnimSequenceBase>(ObjToUse);
			if (DroppedObjAsAnimSequence)
			{
				USkeleton* AnimSkeleton = DroppedObjAsAnimSequence->GetSkeleton();

				if (AnimSkeleton)
				{
					if (bTest)
					{
						bResult = true;
					}
					else
					{
						const FScopedTransaction Transaction(LOCTEXT("DropAnimationOnObject", "Drop Animation On Object"));
						SkeletalMeshComponent->Modify();

						// If the component doesn't have a mesh, the mesh should change
						const bool bShouldChangeMesh = !SkeletalMeshComponent->GetSkeletalMeshAsset();

						if (bShouldChangeMesh)
						{
							SkeletalMeshComponent->SetSkeletalMesh(AnimSkeleton->GetAssetPreviewMesh(DroppedObjAsAnimSequence));
						}

						SkeletalMeshComponent->SetAnimationMode(EAnimationMode::Type::AnimationSingleNode);
						SkeletalMeshComponent->AnimationData.AnimToPlay = DroppedObjAsAnimSequence;

						// set runtime data
						SkeletalMeshComponent->SetAnimation(DroppedObjAsAnimSequence);

						if (SkeletalMeshComponent->GetSkeletalMeshAsset())
						{
							bResult = true;
							SkeletalMeshComponent->InitAnim(true);
						}
					}
				}
			}
		}
	}

	return bResult;
}

/**
 * Helper function that attempts to apply the supplied object to the supplied actor.
 *
 * @param	ObjToUse				Object to attempt to apply as specific asset
 * @param	ActorToApplyTo			Actor to whom the asset should be applied
 * @param   TargetMaterialSlot      When dealing with submeshes this will represent the target section/slot to apply materials to.
 * @param	bTest					Whether to test if the object would be successfully applied without actually doing it.
 *
 * @return	true if the provided object was successfully applied to the provided actor
 */
static bool AttemptApplyObjToActor( UObject* ObjToUse, AActor* ActorToApplyTo, int32 TargetMaterialSlot = -1, bool bTest = false )
{
	bool bResult = false;

	if ( ActorToApplyTo )
	{
		bResult = false;

		TInlineComponentArray<USceneComponent*> SceneComponents;
		ActorToApplyTo->GetComponents(SceneComponents);
		for (USceneComponent* SceneComp : SceneComponents)
		{
			bResult |= AttemptApplyObjToComponent(ObjToUse, SceneComp, TargetMaterialSlot, bTest);
		}

		// Notification hook for dropping asset onto actor
		if(!bTest)
		{
			FEditorDelegates::OnApplyObjectToActor.Broadcast(ObjToUse, ActorToApplyTo);
		}
	}

	return bResult;
}

/**
 * Helper function that attempts to apply the supplied object as a material to the BSP surface specified by the
 * provided model and index.
 *
 * @param	ObjToUse				Object to attempt to apply as a material
 * @param	ModelHitProxy			Hit proxy containing the relevant model
 * @param	SurfaceIndex			The index in the model's surface array of the relevant
 *
 * @return	true if the supplied object was successfully applied to the specified BSP surface
 */
bool FLevelEditorViewportClient::AttemptApplyObjAsMaterialToSurface( UObject* ObjToUse, HModel* ModelHitProxy, FViewportCursorLocation& Cursor )
{
	bool bResult = false;

	UTexture* DroppedObjAsTexture = Cast<UTexture>( ObjToUse );
	if ( DroppedObjAsTexture != NULL )
	{
		ObjToUse = GetOrCreateMaterialFromTexture( DroppedObjAsTexture );
	}

	// Ensure the dropped object is a material
	UMaterialInterface* DroppedObjAsMaterial = Cast<UMaterialInterface>( ObjToUse );

	if ( DroppedObjAsMaterial && ModelHitProxy )
	{
		FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
			Viewport, 
			GetScene(),
			EngineShowFlags)
			.SetRealtimeUpdate( IsRealtime() ));
		FSceneView* View = CalcSceneView( &ViewFamily );


		UModel* Model = ModelHitProxy->GetModel();
		
		// If our model doesn't exist
		if ( !Model )
		{
			return false;
		}
		// or is part of an outer that is being destroyed
		TWeakObjectPtr<UObject> ModelOuter(Model->GetOuter());
		if ( !ModelOuter.IsValid() )
		{
			return false;
		}

		TArray<uint32> SelectedSurfaces;

		bool bDropedOntoSelectedSurface = false;
		const int32 DropX = Cursor.GetCursorPos().X;
		const int32 DropY = Cursor.GetCursorPos().Y;

		{
			uint32 SurfaceIndex;
			ModelHitProxy->ResolveSurface(View, DropX, DropY, SurfaceIndex);
			if (SurfaceIndex != INDEX_NONE)
			{
				if ((Model->Surfs[SurfaceIndex].PolyFlags & PF_Selected) == 0)
				{
					// Surface was not selected so only apply to this surface
					SelectedSurfaces.Add(SurfaceIndex);
				}
				else
				{
					bDropedOntoSelectedSurface = true;
				}
			}
		}

		if( bDropedOntoSelectedSurface )
		{
			for (int32 SurfaceIndex = 0; SurfaceIndex < Model->Surfs.Num(); ++SurfaceIndex)
			{
				FBspSurf& Surf = Model->Surfs[SurfaceIndex];

				if (Surf.PolyFlags & PF_Selected)
				{
					SelectedSurfaces.Add(SurfaceIndex);
				}
			}
		}

		if( SelectedSurfaces.Num() )
		{
			
			// Apply the material to the specified surface
			FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "DragDrop_Transaction_ApplyMaterialToSurface", "Apply Material to Surface"));

			if (ModelHitProxy->GetModelComponent())
			{
				// Modify the component so that PostEditUndo can reregister the model after undo
				ModelHitProxy->GetModelComponent()->Modify();
			}

			for( int32 SurfListIndex = 0; SurfListIndex < SelectedSurfaces.Num(); ++SurfListIndex )
			{
				uint32 SelectedSurfIndex = SelectedSurfaces[SurfListIndex];

				check(Model->Surfs.IsValidIndex(SelectedSurfIndex));

				Model->ModifySurf(SelectedSurfIndex, true);
				Model->Surfs[SelectedSurfIndex].Material = DroppedObjAsMaterial;
				const bool bUpdateTexCoords = false;
				const bool bOnlyRefreshSurfaceMaterials = true;
				GEditor->polyUpdateBrush(Model, SelectedSurfIndex, bUpdateTexCoords, bOnlyRefreshSurfaceMaterials);
			}

			bResult = true;
		}
	}

	return bResult;
}


static bool AreAllDroppedObjectsBrushBuilders(const TArray<UObject*>& DroppedObjects)
{
	for (UObject* DroppedObject : DroppedObjects)
	{
		if (!DroppedObject->IsA(UBrushBuilder::StaticClass()))
		{
			return false;
		}
	}

	return true;
}

static bool AreAnyDroppedObjectsBrushBuilders(const TArray<UObject*>& DroppedObjects)
{
	for (UObject* DroppedObject : DroppedObjects)
	{
		if (DroppedObject->IsA(UBrushBuilder::StaticClass()))
		{
			return true;
		}
	}

	return false;
}

bool FLevelEditorViewportClient::DropObjectsOnBackground(FViewportCursorLocation& Cursor, 
	const TArray<UObject*>& DroppedObjects, EObjectFlags ObjectFlags, TArray<FTypedElementHandle>& OutNewItems, 
	const FDropObjectOptions& Options)
{
	if (DroppedObjects.Num() == 0)
	{
		return false;
	}

	bool bSuccess = false;

	const bool bTransacted = !Options.bCreateDropPreview && !AreAllDroppedObjectsBrushBuilders(DroppedObjects);

	// Create a transaction if not a preview drop
	if (bTransacted)
	{
		GEditor->BeginTransaction(LevelEditorViewportLocals::DropObjectsTransactionName);
	}

	for ( int32 DroppedObjectsIdx = 0; DroppedObjectsIdx < DroppedObjects.Num(); ++DroppedObjectsIdx )
	{
		UObject* AssetObj = DroppedObjects[DroppedObjectsIdx];
		ensure( AssetObj );

		// Attempt to create actors from the dropped object
		UE::AssetPlacementUtil::FExtraPlaceAssetOptions PlacementOptions;
		PlacementOptions.bSelectOutput = Options.bSelectOutput;
		PlacementOptions.ObjectFlags = ObjectFlags;
		PlacementOptions.FactoryToUse = Options.FactoryToUse;
		PlacementOptions.Name = NAME_None;
		TArray<FTypedElementHandle> PlacedItems = TryPlacingAssetObject(GetWorld()->GetCurrentLevel(), AssetObj,
			PlacementOptions, &Cursor);

		if (PlacedItems.Num() > 0)
		{
			OutNewItems.Append(PlacedItems);
			bSuccess = true;
		}
	}

	if (bTransacted)
	{
		GEditor->EndTransaction();
	}

	return bSuccess;
}

bool FLevelEditorViewportClient::DropObjectsOnBackground(FViewportCursorLocation& Cursor, const TArray<UObject*>& DroppedObjects, EObjectFlags ObjectFlags, 
	TArray<AActor*>& OutNewActors, bool bCreateDropPreview, bool bSelectOutput, UActorFactory* FactoryToUse)
{
	FDropObjectOptions Options;
	Options.bCreateDropPreview = bCreateDropPreview;
	Options.bSelectOutput = bSelectOutput;
	Options.FactoryToUse = FactoryToUse;

	TArray<FTypedElementHandle> NewItems;
	bool bSuccess = DropObjectsOnBackground(Cursor, DroppedObjects, ObjectFlags, NewItems, Options);

	LevelEditorViewportLocals::ForEachActorInElementArray(NewItems, [&OutNewActors](AActor& Actor)
	{
		OutNewActors.Add(&Actor);
	});
	return bSuccess;
}

bool FLevelEditorViewportClient::DropObjectsOnActor(FViewportCursorLocation& Cursor, 
	const TArray<UObject*>& DroppedObjects, AActor* DroppedUponActor, 
	int32 DroppedUponSlot, EObjectFlags ObjectFlags,
	TArray<FTypedElementHandle>& OutNewItems, const FDropObjectOptions& Options)
{
	if ( !DroppedUponActor || DroppedObjects.Num() == 0 )
	{
		return false;
	}

	bool bSuccess = false;

	const bool bTransacted = !Options.bCreateDropPreview && !AreAllDroppedObjectsBrushBuilders(DroppedObjects);

	// Create a transaction if not a preview drop
	if (bTransacted)
	{
		GEditor->BeginTransaction(LevelEditorViewportLocals::DropObjectsTransactionName);
	}

	for ( UObject* DroppedObject : DroppedObjects )
	{
		const bool bIsTestApplication = Options.bCreateDropPreview;
		const bool bAppliedToActor = (Options.FactoryToUse == nullptr ) ? AttemptApplyObjToActor( DroppedObject, DroppedUponActor, DroppedUponSlot, bIsTestApplication ) : false;

		if (!bAppliedToActor)
		{
			// Attempt to create actors from the dropped object
			UE::AssetPlacementUtil::FExtraPlaceAssetOptions PlacementOptions;
			PlacementOptions.bSelectOutput = Options.bSelectOutput;
			PlacementOptions.ObjectFlags = ObjectFlags;
			PlacementOptions.FactoryToUse = Options.FactoryToUse;
			PlacementOptions.Name = NAME_None;

			TArray<FTypedElementHandle> NewItems = TryPlacingAssetObject(GetWorld()->GetCurrentLevel(), 
				DroppedObject, PlacementOptions, &Cursor);

			if (NewItems.Num() > 0 )
			{
				OutNewItems.Append(NewItems);
				bSuccess = true;
			}
		}
		else
		{
			bSuccess = true;
		}
	}

	if (bTransacted)
	{
		GEditor->EndTransaction();
	}

	return bSuccess;
}

bool FLevelEditorViewportClient::DropObjectsOnActor(FViewportCursorLocation& Cursor, const TArray<UObject*>& DroppedObjects, 
	AActor* DroppedUponActor, int32 DroppedUponSlot, EObjectFlags ObjectFlags, 
	TArray<AActor*>& OutNewActors, bool bCreateDropPreview, bool bSelectOutput, UActorFactory* FactoryToUse)
{
	FDropObjectOptions Options;
	Options.bCreateDropPreview = bCreateDropPreview;
	Options.bSelectOutput = bSelectOutput;
	Options.FactoryToUse = FactoryToUse;

	TArray<FTypedElementHandle> NewItems;
	bool bSuccess = DropObjectsOnActor(Cursor, DroppedObjects, DroppedUponActor, DroppedUponSlot, ObjectFlags, NewItems, Options);

	LevelEditorViewportLocals::ForEachActorInElementArray(NewItems, [&OutNewActors](AActor& Actor)
	{
		OutNewActors.Add(&Actor);
	});
	return bSuccess;
}

bool FLevelEditorViewportClient::DropObjectsOnBSPSurface(FSceneView* View, FViewportCursorLocation& Cursor, 
	const TArray<UObject*>& DroppedObjects, HModel* TargetProxy, EObjectFlags ObjectFlags, 
	TArray<FTypedElementHandle>& OutNewItems, const FDropObjectOptions& Options)
{
	if (DroppedObjects.Num() == 0)
	{
		return false;
	}

	bool bSuccess = false;

	const bool bTransacted = !Options.bCreateDropPreview && !AreAllDroppedObjectsBrushBuilders(DroppedObjects);

	// Create a transaction if not a preview drop
	if (bTransacted)
	{
		GEditor->BeginTransaction(LevelEditorViewportLocals::DropObjectsTransactionName);
	}

	for (UObject* DroppedObject : DroppedObjects)
	{
		const bool bAppliedToActor = (!Options.bCreateDropPreview && Options.FactoryToUse == NULL) ? AttemptApplyObjAsMaterialToSurface(DroppedObject, TargetProxy, Cursor) : false;

		if (!bAppliedToActor)
		{
			// Attempt to create actors from the dropped object
			UE::AssetPlacementUtil::FExtraPlaceAssetOptions PlacementOptions;
			PlacementOptions.bSelectOutput = Options.bSelectOutput;
			PlacementOptions.ObjectFlags = ObjectFlags;
			PlacementOptions.FactoryToUse = Options.FactoryToUse;
			PlacementOptions.Name = NAME_None;

			TArray<FTypedElementHandle> NewItems = TryPlacingAssetObject(GetWorld()->GetCurrentLevel(), 
				DroppedObject, PlacementOptions, &Cursor);

			if (NewItems.Num() > 0)
			{
				OutNewItems.Append(NewItems);
				bSuccess = true;
			}
		}
		else
		{
			bSuccess = true;
		}
	}

	if (bTransacted)
	{
		GEditor->EndTransaction();
	}

	return bSuccess;
}

bool FLevelEditorViewportClient::DropObjectsOnBSPSurface(FSceneView* View, FViewportCursorLocation& Cursor,
	const TArray<UObject*>& DroppedObjects, HModel* TargetProxy, EObjectFlags ObjectFlags, 
	TArray<AActor*>& OutNewActors, bool bCreateDropPreview, bool bSelectOutput, UActorFactory* FactoryToUse)
{
	FDropObjectOptions Options;
	Options.bCreateDropPreview = bCreateDropPreview;
	Options.bSelectOutput = bSelectOutput;
	Options.FactoryToUse = FactoryToUse;

	TArray<FTypedElementHandle> NewItems;
	bool bSuccess = DropObjectsOnBSPSurface(View, Cursor, DroppedObjects, TargetProxy, ObjectFlags, NewItems, Options);
	
	LevelEditorViewportLocals::ForEachActorInElementArray(NewItems, [&OutNewActors](AActor& Actor)
	{
		OutNewActors.Add(&Actor);
	});
	return bSuccess;
}

/**
 * Called when an asset is dropped upon a manipulation widget.
 *
 * @param	View				The SceneView for the dropped-in viewport
 * @param	Cursor				Mouse cursor location
 * @param	DroppedObjects		Array of objects dropped into the viewport
 *
 * @return	true if the drop operation was successfully handled; false otherwise
 */
bool FLevelEditorViewportClient::DropObjectsOnWidget(FSceneView* View, FViewportCursorLocation& Cursor, const TArray<UObject*>& DroppedObjects, bool bCreateDropPreview)
{
	bool bResult = false;

	// Axis translation/rotation/scale widget - find out what's underneath the axis widget

	// Modify the ShowFlags for the scene so we can re-render the hit proxies without any axis widgets. 
	// Store original ShowFlags and assign them back when we're done
	const bool bOldModeWidgets1 = EngineShowFlags.ModeWidgets;
	const bool bOldModeWidgets2 = View->Family->EngineShowFlags.ModeWidgets;

	EngineShowFlags.SetModeWidgets(false);
	FSceneViewFamily* SceneViewFamily = const_cast< FSceneViewFamily* >( View->Family );
	SceneViewFamily->EngineShowFlags.SetModeWidgets(false);

	// Invalidate the hit proxy map so it will be rendered out again when GetHitProxy is called
	Viewport->InvalidateHitProxy();

	// This will actually re-render the viewport's hit proxies!
	FIntPoint DropPos = Cursor.GetCursorPos();

	HHitProxy* HitProxy = Viewport->GetHitProxy(DropPos.X, DropPos.Y);

	// We should never encounter a widget axis.  If we do, then something's wrong
	// with our ShowFlags (or the widget drawing code)
	check( !HitProxy || ( HitProxy && !HitProxy->IsA( HWidgetAxis::StaticGetType() ) ) );

	// Try this again, but without the widgets this time!
	TArray<FTypedElementHandle> NewObjects;
	const FIntPoint& CursorPos = Cursor.GetCursorPos();

	FDropObjectOptions DropOptions;
	DropOptions.bOnlyDropOnTarget = false;
	DropOptions.bCreateDropPreview = bCreateDropPreview;
	bResult = DropObjectsAtCoordinates(CursorPos.X, CursorPos.Y, DroppedObjects, NewObjects, DropOptions);

	// Restore the original flags
	EngineShowFlags.SetModeWidgets(bOldModeWidgets1);
	SceneViewFamily->EngineShowFlags.SetModeWidgets(bOldModeWidgets2);

	return bResult;
}

bool FLevelEditorViewportClient::HasDropPreviewActors() const
{
	return HasDropPreviewElements();
}

/* Helper functions to find a dropped position on a 2D layer */

static bool IsDroppingOn2DLayer()
{
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
	const ULevelEditor2DSettings* Settings2D = GetDefault<ULevelEditor2DSettings>();
	return ViewportSettings->bEnableLayerSnap && Settings2D->SnapLayers.IsValidIndex(ViewportSettings->ActiveSnapLayerIndex);
}

static UE::Positioning::FObjectPositioningTraceResult TraceForPositionOn2DLayer(const FViewportCursorLocation& Cursor)
{
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
	const ULevelEditor2DSettings* Settings2D = GetDefault<ULevelEditor2DSettings>();
	check(Settings2D->SnapLayers.IsValidIndex(ViewportSettings->ActiveSnapLayerIndex));

	const float Offset = Settings2D->SnapLayers[ViewportSettings->ActiveSnapLayerIndex].Depth;
	FVector PlaneCenter(0, 0, 0);
	FVector PlaneNormal(0, 0, 0);

	switch (Settings2D->SnapAxis)
	{
	case ELevelEditor2DAxis::X: PlaneCenter.X = Offset; PlaneNormal.X = -1; break;
	case ELevelEditor2DAxis::Y: PlaneCenter.Y = Offset; PlaneNormal.Y = -1; break;
	case ELevelEditor2DAxis::Z: PlaneCenter.Z = Offset; PlaneNormal.Z = -1; break;
	}

	UE::Positioning::FObjectPositioningTraceResult Result;
	const double Numerator = FVector::DotProduct(PlaneCenter - Cursor.GetOrigin(), PlaneNormal);
	const double Denominator = FVector::DotProduct(PlaneNormal, Cursor.GetDirection());
	if (FMath::Abs(Denominator) < SMALL_NUMBER)
	{
		Result.State = UE::Positioning::FObjectPositioningTraceResult::Failed;
	}
	else
	{
		Result.State = UE::Positioning::FObjectPositioningTraceResult::HitSuccess;
		Result.SurfaceNormal = PlaneNormal;
		double D = Numerator / Denominator;
		Result.Location = Cursor.GetOrigin() + D * Cursor.GetDirection();
	}

	return Result;
}

bool FLevelEditorViewportClient::UpdateDropPreviewActors(int32 MouseX, int32 MouseY, const TArray<UObject*>& DroppedObjects, bool& out_bDroppedObjectsVisible, UActorFactory* FactoryToUse)
{
	return UpdateDropPreviewElements(MouseX, MouseY, DroppedObjects, out_bDroppedObjectsVisible, FactoryToUse);
}

void FLevelEditorViewportClient::DestroyDropPreviewActors()
{
	DestroyDropPreviewElements();
}

bool FLevelEditorViewportClient::HasDropPreviewElements() const
{
	return ensure(DropPreviewElements) && DropPreviewElements->Num() > 0;
}

bool FLevelEditorViewportClient::UpdateDropPreviewElements(int32 MouseX, int32 MouseY, 
	const TArray<UObject*>& DroppedObjects, bool& out_bDroppedObjectsVisible, 
	TScriptInterface<IAssetFactoryInterface> Factory)
{
	out_bDroppedObjectsVisible = false;
	if (!HasDropPreviewElements())
	{
		return false;
	}

	// While dragging actors, allow viewport updates
	bNeedsRedraw = true;

	// If the mouse did not move, there is no need to update anything
	if (MouseX == DropPreviewMouseX && MouseY == DropPreviewMouseY)
	{
		return false;
	}

	// Update the cached mouse position
	DropPreviewMouseX = MouseX;
	DropPreviewMouseY = MouseY;

	// Get the center point between all the drop preview objects for use in calculations below
	// Also, build a list of valid AActor* pointers
	FVector CombinedOrigin = FVector::ZeroVector;
	int32 Count = 0;

	DropPreviewElements->ForEachElement<ITypedElementWorldInterface>([&CombinedOrigin, &Count](const TTypedElement<ITypedElementWorldInterface>& InElement)
	{
		FTransform Transform;
		if (InElement.GetWorldTransform(Transform))
		{
			CombinedOrigin += Transform.GetTranslation();
			++Count;
		}

		// true means don't stop
		return true;
	});

	if (Count == 0)
	{
		return false;
	}

	// Finish the calculation of the actors origin now that we know we are not dividing by zero
	CombinedOrigin /= Count;

	FCollisionQueryParams CollisionQueryParams;
	DropPreviewElements->ForEachElement<ITypedElementWorldInterface>(
	[&CollisionQueryParams](const TTypedElement<ITypedElementWorldInterface>& InElement)
	{
		InElement.AddIgnoredElementToCollisionQueryParams(CollisionQueryParams);

		// true means continue
		return true;
	});

	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		Viewport,
		GetScene(),
		EngineShowFlags)
		.SetRealtimeUpdate(IsRealtime()));
	FSceneView* View = CalcSceneView(&ViewFamily);
	FViewportCursorLocation Cursor(View, this, MouseX, MouseY);

	const UE::Positioning::FObjectPositioningTraceResult TraceResult = IsDroppingOn2DLayer() 
		? TraceForPositionOn2DLayer(Cursor) 
		: UE::Positioning::TraceWorldForPositionWithDefault(Cursor, *View, &CollisionQueryParams);

	GEditor->UnsnappedClickLocation = TraceResult.Location;
	GEditor->ClickLocation = TraceResult.Location;
	GEditor->ClickPlane = FPlane(TraceResult.Location, TraceResult.SurfaceNormal);

	// Snap the new location if snapping is enabled
	FSnappingUtils::SnapPointToGrid(GEditor->ClickLocation, FVector::ZeroVector);

	AActor* DroppedOnActor = Cast<AActor>(TraceResult.HitObject.Get());

	if (DroppedOnActor)
	{
		// We indicate that the dropped objects are visible if *any* of them are not applicable to other actors
		out_bDroppedObjectsVisible = DroppedObjects.ContainsByPredicate([&](UObject* AssetObj) {
			return !AttemptApplyObjToActor(AssetObj, DroppedOnActor, -1, true);
			});
	}
	else
	{
		// All dropped objects are visible if we're not dropping on an actor
		out_bDroppedObjectsVisible = true;
	}

	// Perform the actual updates
	DropPreviewElements->ForEachElementHandle(
		[this, &TraceResult, ActorFactory = Cast<UActorFactory>(Factory.GetObject())](const FTypedElementHandle& Element)
	{
		UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();
		if (!ensure(Registry))
		{
			return false;
		}

		// Used for the actual location update
		TTypedElement<ITypedElementWorldInterface> WorldInterface = Registry->GetElement<ITypedElementWorldInterface>(Element);

		// Used for some legacy updates if we're dealing with actors
		TTypedElement<ITypedElementObjectInterface> ObjectInterface = Registry->GetElement<ITypedElementObjectInterface>(Element);
		AActor* Actor = ObjectInterface ? ObjectInterface.GetObjectAs<AActor>() : nullptr;

		if (WorldInterface)
		{
			FTransform3d PreviousTransform = PreDragElementTransforms.FindRef(Element);
			
			UActorFactory* ActorFactoryToUse = (!ActorFactory && Actor) ? GEditor->FindActorFactoryForActorClass(Actor->GetClass())
				: ActorFactory;

			FSnappedPositioningData PositioningData = FSnappedPositioningData(this, TraceResult.Location, TraceResult.SurfaceNormal)
				.DrawSnapHelpers(true)
				.UseStartTransform(PreviousTransform)
				// TODO: Need to support non-actor factories here
				.UseFactory(ActorFactoryToUse)
				.UsePlacementExtent(Actor ? Actor->GetPlacementExtent() : FVector3d::Zero());

			FTransform DestinationTransform = FActorPositioning::GetSnappedSurfaceAlignedTransform(PositioningData);
			DestinationTransform.SetScale3D(PreviousTransform.GetScale3D());		// preserve scaling

			WorldInterface.SetWorldTransform(DestinationTransform);
		}

		if (Actor)
		{
			Actor->SetIsTemporarilyHiddenInEditor(false);
			Actor->PostEditMove(false);
		}

		// true means don't stop
		return true;
	});

	return true;
}

void FLevelEditorViewportClient::DestroyDropPreviewElements()
{
	using namespace LevelEditorViewportLocals;
	using namespace UE::Editor::DataStorage;

	if (!HasDropPreviewElements())
	{
		return;
	}

	// TODO: This code to remove the object from TEDS should not be necessary, because the element
	// deletion code further below should include TEDS deregistration. However, the code path for
	// deleting preview actors in UUnrealEdEngine::DeleteActors skips explicit handle deregistration,
	// and although it does still happen in the immediately triggered garbage cleanup, that feels
	// potentially brittle.
	if (ICompatibilityProvider* TedsCompat = GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName))
	{
		DropPreviewElements->ForEachElement<ITypedElementObjectInterface>(
			[this, TedsCompat](const TTypedElement<ITypedElementObjectInterface>& InElement)
			{
				UObject* PreviewObject = InElement.GetObject();
				if (PreviewObject && PreviewObject != GetWorld()->GetDefaultBrush())
				{
					TedsCompat->RemoveCompatibleObject(PreviewObject);
				}
				return true; // true means continue
			});
	}

	// Used for special casing BSP backwards compatibility: the builder brush is used as a preview object, and
	// we don't want to delete it.
	const UObject* DefaultBrush = GetWorld() ? GetWorld()->GetDefaultBrush() : nullptr;

	DropPreviewElements->ForEachElement<ITypedElementWorldInterface>([this, DefaultBrush](const TTypedElement<ITypedElementWorldInterface>& InElement)
	{
		// Make sure to remove any items keyed by this handle, otherwise we'll get complaints about
		// remaining references to the handle after destroying the item
		PreDragElementTransforms.Remove(InElement);

		// Don't delete the BSP builder brush, which legacy code uses for previews and expects to always exist.
		if (DefaultBrush)
		{
			TTypedElement<ITypedElementObjectInterface> ObjectInterface = UTypedElementRegistry::GetInstance()->GetElement<ITypedElementObjectInterface>(InElement);
			const UObject* Object = ObjectInterface ? ObjectInterface.GetObject() : nullptr;
			if (Object && Object == DefaultBrush)
			{
				// true means continue - don't fall through to the deletion.
				return true;
			}
		}
		
		FTypedElementDeletionOptions Options;
		Options
			.SetWarnAboutReferences(false)
			.SetWarnAboutSoftReferences(false);

		if (!ensure(InElement.DeleteElement(InElement.GetOwnerWorld(), GetMutableSelectionSet(), Options)))
		{
			// We don't expect to fail, but try a legacy actor deletion path if we do.
			// TODO: This might not help that much because for actors, the DeleteElements will frequently still 
			// return true even if deletion does not go through because it calls UUnrealEdEngine::DeleteActors,
			// and that function returns true for many failed deletions. It would be nice to make edits to make
			// the check more reliable.
			TTypedElement<ITypedElementObjectInterface> ObjectInterface = UTypedElementRegistry::GetInstance()->GetElement<ITypedElementObjectInterface>(InElement);
			AActor* Actor = ObjectInterface ? ObjectInterface.GetObjectAs<AActor>() : nullptr;
			if (Actor)
			{
				GetWorld()->DestroyActor(Actor);
			}
		}

		return true;  // true means continue
	});

	DropPreviewElements->Empty();
}


/**
* Checks the viewport to see if the given object can be dropped using the given mouse coordinates local to this viewport
*
* @param MouseX			The position of the mouse's X coordinate
* @param MouseY			The position of the mouse's Y coordinate
* @param AssetData			Asset in question to be dropped
*/
FDropQuery FLevelEditorViewportClient::CanDropObjectsAtCoordinates(int32 MouseX, int32 MouseY, const FAssetData& AssetData)
{
	FDropQuery Result;

	ULevelEditorDragDropHandler* DragDrop = GEditor->GetLevelEditorDragDropHandler();
	if (!DragDrop->PreviewDropObjectsAtCoordinates(MouseX, MouseY, GetWorld(), Viewport, AssetData))
	{
		Result.bCanDrop = DragDrop->GetCanDrop();
		Result.HintText = DragDrop->GetPreviewDropHintText();

		return Result;
	}

	UObject* AssetObj = AssetData.GetAsset();
	UClass* ClassObj = Cast<UClass>(AssetObj);

	UPlacementSubsystem* PlacementSubsystem = GEditor->GetEditorSubsystem<UPlacementSubsystem>();
	// Check if the asset has an actor factory
	bool bHasActorFactory = PlacementSubsystem && PlacementSubsystem->FindAssetFactoryFromAssetData(AssetData) != nullptr;
	bHasActorFactory = bHasActorFactory || FActorFactoryAssetProxy::GetFactoryForAsset(AssetData) != nullptr;

	if (ClassObj)
	{
		if (!bHasActorFactory && !ObjectTools::IsClassValidForPlacing(ClassObj))
		{
			Result.bCanDrop = false;
			Result.HintText = FText::Format(LOCTEXT("DragAndDrop_CannotDropAssetClassFmt", "The class '{0}' cannot be placed in a level"), FText::FromString(ClassObj->GetName()));
			return Result;
		}

		AssetObj = ClassObj->GetDefaultObject();
	}

	if (ensureMsgf(AssetObj != NULL, TEXT("AssetObj was null (%s)"), *AssetData.GetFullName()))
	{
		if (AssetObj->IsA(AActor::StaticClass()) || bHasActorFactory)
		{
			Result.bCanDrop = true;
			GUnrealEd->SetPivotMovedIndependently(false);
		}
		else if (AssetObj->IsA(UBrushBuilder::StaticClass()))
		{
			Result.bCanDrop = true;
			GUnrealEd->SetPivotMovedIndependently(false);
		}
		else
		{
			HHitProxy* HitProxy = Viewport->GetHitProxy(MouseX, MouseY);
			if (HitProxy != nullptr && CanApplyMaterialToHitProxy(HitProxy))
			{
				if (AssetObj->IsA(UMaterialInterface::StaticClass()) || AssetObj->IsA(UTexture::StaticClass()))
				{
					// If our asset is a material and the target is a valid recipient
					Result.bCanDrop = true;
					GUnrealEd->SetPivotMovedIndependently(false);
				}
			}
		}
	}

	return Result;
}

bool FLevelEditorViewportClient::DropObjectsAtCoordinates(int32 MouseX, int32 MouseY, const TArray<UObject*>& DroppedObjects, 
	TArray<FTypedElementHandle>& OutNewElements, const FDropObjectOptions& Options)
{
	bool bResult = false;

	// Allow the drag drop handler to do anything pre-drop.
	ULevelEditorDragDropHandler* DragDropHandler = GEditor->GetLevelEditorDragDropHandler();
	if (!Options.bCreateDropPreview)
	{
		// TODO: Deprecate PreDropObjectsAtCoordinates because it does not currently get used anywhere.
		TArray<AActor*> NewActors;
		if (!DragDropHandler->PreDropObjectsAtCoordinates(MouseX, MouseY, GetWorld(), Viewport, DroppedObjects, NewActors))
		{
			return bResult;
		}
		for (AActor* Actor : NewActors)
		{
			if (FTypedElementHandle Handle = UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor))
			{
				OutNewElements.Add(Handle);
			}
		}
	}

	// Make sure the placement dragging actor is cleaned up.
	DestroyDropPreviewActors();

	if(DroppedObjects.Num() > 0)
	{
		bIsDroppingPreviewActor = Options.bCreateDropPreview;
		Viewport->InvalidateHitProxy();

		FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
			Viewport, 
			GetScene(),
			EngineShowFlags)
			.SetRealtimeUpdate( IsRealtime() ));
		FSceneView* View = CalcSceneView( &ViewFamily );
		FViewportCursorLocation Cursor(View, this, MouseX, MouseY);

		HHitProxy* HitProxy = Viewport->GetHitProxy(Cursor.GetCursorPos().X, Cursor.GetCursorPos().Y);

		const UE::Positioning::FObjectPositioningTraceResult TraceResult = IsDroppingOn2DLayer() 
			? TraceForPositionOn2DLayer(Cursor) 
			: UE::Positioning::TraceWorldForPositionWithDefault(Cursor, *View, nullptr);
		
		GEditor->UnsnappedClickLocation = TraceResult.Location;
		GEditor->ClickLocation = TraceResult.Location;
		GEditor->ClickPlane = FPlane(TraceResult.Location, TraceResult.SurfaceNormal);

		// Snap the new location if snapping is enabled
		FSnappingUtils::SnapPointToGrid(GEditor->ClickLocation, FVector::ZeroVector);

		EObjectFlags ObjectFlags = Options.bCreateDropPreview ? RF_Transient : RF_Transactional;
		if (HitProxy == nullptr || HitProxy->IsA(HInstancedStaticMeshInstance::StaticGetType()))
		{
			bResult = DropObjectsOnBackground(Cursor, DroppedObjects, ObjectFlags, OutNewElements, Options);
		}
		else if (HitProxy->IsA(HActor::StaticGetType()) || HitProxy->IsA(HBSPBrushVert::StaticGetType()))
		{
			AActor* TargetActor = NULL;
			UPrimitiveComponent* TargetComponent = nullptr;
			int32 TargetMaterialSlot = -1;

			if (HitProxy->IsA(HActor::StaticGetType()))
			{
				HActor* TargetProxy = static_cast<HActor*>(HitProxy);
				TargetActor = TargetProxy->Actor;
				TargetComponent = ConstCast(TargetProxy->PrimComponent);
				TargetMaterialSlot = TargetProxy->MaterialIndex;
			}
			else if (HitProxy->IsA(HBSPBrushVert::StaticGetType()))
			{
				HBSPBrushVert* TargetProxy = static_cast<HBSPBrushVert*>(HitProxy);
				TargetActor = TargetProxy->Brush.Get();
			}

			// If shift is pressed set the material slot to -1, so that it's applied to every slot.
			// We have to request it from the platform application directly because IsShiftPressed gets 
			// the cached state, when the viewport had focus
			if ( FSlateApplication::Get().GetPlatformApplication()->GetModifierKeys().IsShiftDown() )
			{
				TargetMaterialSlot = -1;
			}

			if (TargetActor != nullptr && IsValidChecked(TargetActor->GetWorld()) && !TargetActor->GetWorld()->IsUnreachable())
			{
				FNavigationLockContext LockNavigationUpdates(TargetActor->GetWorld(), ENavigationLockReason::SpawnOnDragEnter, Options.bCreateDropPreview);

				// If the target actor is selected, we should drop onto all selected actors
				// otherwise, we should drop only onto the target object
				const bool bDropOntoSelectedActors = TargetActor->IsSelected();
				const bool bCanApplyToComponent = AttemptApplyObjToComponent(DroppedObjects[0], TargetComponent, TargetMaterialSlot, true);
				if (Options.bOnlyDropOnTarget || !bDropOntoSelectedActors || !bCanApplyToComponent)
				{
					if (bCanApplyToComponent)
					{
						const bool bIsTestAttempt = Options.bCreateDropPreview;
						bResult = AttemptApplyObjToComponent(DroppedObjects[0], TargetComponent, TargetMaterialSlot, bIsTestAttempt);
					}
					else
					{
						// Couldn't apply to a component, so try dropping the objects on the hit actor
						bResult = DropObjectsOnActor(Cursor, DroppedObjects, TargetActor, TargetMaterialSlot, 
							ObjectFlags, OutNewElements, Options);
					}
				}
				else
				{
					// Are any components selected?
					if (GEditor->GetSelectedComponentCount() > 0)
					{
						// Is the target component selected?
						USelection* ComponentSelection = GEditor->GetSelectedComponents();
						if (ComponentSelection->IsSelected(TargetComponent))
						{
							// The target component is selected, so try applying the object to every selected component
							for (FSelectedEditableComponentIterator It(GEditor->GetSelectedEditableComponentIterator()); It; ++It)
							{
								USceneComponent* SceneComponent = Cast<USceneComponent>(*It);
								AttemptApplyObjToComponent(DroppedObjects[0], SceneComponent, TargetMaterialSlot, Options.bCreateDropPreview);
								bResult = true;
							}
						}
						else
						{
							// The target component is not selected, so apply the object exclusively to it
							bResult = AttemptApplyObjToComponent(DroppedObjects[0], TargetComponent, TargetMaterialSlot, Options.bCreateDropPreview);
						}
					}
					
					if (!bResult)
					{
						const FScopedTransaction Transaction(LOCTEXT("DropObjectsOnSelectedActors", "Drop Objects on Selected Actors"));
						for (FSelectionIterator It(*GEditor->GetSelectedActors()); It; ++It)
						{
							TargetActor = static_cast<AActor*>( *It );
							if (TargetActor)
							{
								DropObjectsOnActor(Cursor, DroppedObjects, TargetActor, 
									TargetMaterialSlot, ObjectFlags, OutNewElements, Options);
								bResult = true;
							}
						}
					}
				}
			}
		}
		else if (HitProxy->IsA(HModel::StaticGetType()))
		{
			// BSP surface
			bResult = DropObjectsOnBSPSurface(View, Cursor, DroppedObjects, static_cast<HModel*>(HitProxy), ObjectFlags, OutNewElements, Options);
		}
		else if( HitProxy->IsA( HWidgetAxis::StaticGetType() ) )
		{
			// Axis translation/rotation/scale widget - find out what's underneath the axis widget
			bResult = DropObjectsOnWidget(View, Cursor, DroppedObjects);
		}
		// If the hit proxy was not one of the above types, we probably still don't want to cancel the drop.
		else
		{
			// Ideally we would probably use some interface to get the information we need to confirm the drop. However, the fallback
			// drop we use in the case of actors doesn't really need extra information. Instead, we'll just use the presence of a
			// ITypedElementWorldInterface interface to mean that the hitproxy corresponds to something real in the world.
			auto DoesElementHaveWorldInterface = [](const FTypedElementHandle& Handle) -> bool
			{
				UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();
				if (!Handle || !Registry)
				{
					return false;
				}
				return !!Registry->GetElementInterface<ITypedElementWorldInterface>(Handle);
			};

			if (DoesElementHaveWorldInterface(HitProxy->GetElementHandle()))
			{
				// This is the same thing that we do in other "drop objects" functions like FLevelEditorViewportClient::DropObjectsOnBackground
				// and FLevelEditorViewportClient::DropObjectsOnActor (when not doing some kind of special "apply to actor" operation)
				for (UObject* DroppedObject : DroppedObjects)
				{
					UE::AssetPlacementUtil::FExtraPlaceAssetOptions PlacementOptions;
					PlacementOptions.bSelectOutput = Options.bSelectOutput;
					PlacementOptions.ObjectFlags = ObjectFlags;
					PlacementOptions.FactoryToUse = Options.FactoryToUse;
					PlacementOptions.Name = NAME_None;

					TArray<FTypedElementHandle> NewElements = TryPlacingAssetObject(GetWorld()->GetCurrentLevel(), DroppedObject, PlacementOptions, &Cursor);
					if (NewElements.Num() > 0)
					{
						OutNewElements.Append(NewElements);
						bResult = true;
					}
				}
			}
		}

		if ( bResult )
		{
			// If we are creating a drop preview actor instead of a normal actor, we need to disable collision, selection, and make sure it is never saved.
			if (Options.bCreateDropPreview && OutNewElements.Num() > 0 )
			{
				// Theoretically we shouldn't have selected preview items to begin with, but we've had this safety deselection
				// code forever, and might as well keep it.
				UTypedElementSelectionSet* SelectionSet = GetMutableSelectionSet();
				if (ensure(SelectionSet))
				{
					FTypedElementSelectionOptions SelectionOptions;
					SelectionSet->DeselectElements(OutNewElements, SelectionOptions);
				}
				
				DestroyDropPreviewElements();

				// Helper to add elements to DropPreviewElements
				auto StorePreviewElement = [this](const FTypedElementHandle& Element)
				{
					UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();
					if (!ensure(Registry))
					{
						return;
					}

					TTypedElement<ITypedElementWorldInterface> WorldInterface = Registry->GetElement<ITypedElementWorldInterface>(Element);

					// Don't store preview element if it doesn't have a world interface, because we would need it 
					// to delete the preview element or to move it around
					if (!WorldInterface)
					{
						return;
					}

					DropPreviewElements->Add(Element);
				};

				// Collect the preview elements and save pre drag transforms.
				for (const FTypedElementHandle& Element : OutNewElements)
				{
					StorePreviewElement(Element);
				}

				// Store pre drag transforms
				DropPreviewElements->ForEachElement<ITypedElementWorldInterface>(
					[this](const TTypedElement<ITypedElementWorldInterface>& InElement)
				{
					FTransform Transform;
					if (ensure(InElement.GetWorldTransform(Transform)))
					{
						PreDragElementTransforms.Add(InElement, Transform);
					}

					return true; // true means continue
				});
				
				// Do some legacy actor handling
				DropPreviewElements->ForEachElement<ITypedElementObjectInterface>(
				[this](const TTypedElement<ITypedElementObjectInterface>& InElement)
				{
					AActor* NewActor = InElement.GetObjectAs<AActor>();
					if (!NewActor)
					{
						return true; // true means continue
					}

					NewActor->SetActorEnableCollision(false);

					// This boolean already gets set via UActorFactory::PlaceAsset in code paths that go through
					// PlaceAssetUsingFactory (PlacementOptions.bIsCreatingPreviewElements = FLevelEditorViewportClient::IsDroppingPreviewActor();)
					// But bIsEditorPrieviewActor does not get set if we go through GEditor->AddActor, which can
					// prevent preview actors from being properly destroyed later. If we fix/eliminate paths that
					// do not set this bool properly, we can do an ensure here.
					NewActor->bIsEditorPreviewActor = FLevelEditorViewportClient::IsDroppingPreviewActor();

					// Prevent future selection. This also prevents the hit proxy from interfering with placement logic.
					for (UActorComponent* Component : NewActor->GetComponents())
					{
						if (UPrimitiveComponent* PrimComp = Cast<UPrimitiveComponent>(Component))
						{
							PrimComp->bSelectable = false;
						}
					}

					// true means continue
					return true;
				});

				// Set the current MouseX/Y to prime the preview update
				DropPreviewMouseX = MouseX;
				DropPreviewMouseY = MouseY;

				// Invalidate the hit proxy now so the drop preview will be accurate.
				// We don't invalidate the hit proxy in the drop preview update itself because it is slow.
				//Viewport->InvalidateHitProxy();
			}
			// Dropping the actors rather than a preview? Probably want to select them all then. 
			else if(!Options.bCreateDropPreview && Options.bSelectOutput && DroppedObjects.Num() > 0

				// If we're dropping bsp brushes, that geometry actually gets created later, in 
				// FBrushBuilderDragDropOp::OnDrop(), so we don't want to select the (preview) builder brush.
				// (for reference, this function gets called via CurWidget.Widget->OnDrop() in FSlateApplication::RoutePointerUpEvent,
				// whereas the FBrushBuilderDragDropOp::OnDrop() gets called via SlateUser->NotifyPointerReleased).
				&& !AreAnyDroppedObjectsBrushBuilders(DroppedObjects))
			{
				UTypedElementSelectionSet* SelectionSet = GetMutableSelectionSet();
				if (ensure(SelectionSet))
				{
					FTypedElementSelectionOptions SelectionOptions;
					SelectionSet->SelectElements(OutNewElements, SelectionOptions);
				}
			}

			// Give the viewport focus
			//SetFocus( static_cast<HWND>( Viewport->GetWindow() ) );

			SetCurrentViewport();
		}
	}

	if (bResult)
	{
		if ( !Options.bCreateDropPreview && IPlacementModeModule::IsAvailable() )
		{
			IPlacementModeModule::Get().AddToRecentlyPlaced(DroppedObjects, Options.FactoryToUse);
		}

		if (!Options.bCreateDropPreview)
		{
			TArray<AActor*> Actors;
			LevelEditorViewportLocals::ForEachActorInElementArray(OutNewElements, [&Actors](AActor& Actor)
			{
				Actors.Add(&Actor);
			});

			FEditorDelegates::OnNewActorsDropped.Broadcast(DroppedObjects, Actors);
		}
	}

	if (!Options.bCreateDropPreview)
	{
		DragDropHandler->PostDropObjectsAtCoordinates(MouseX, MouseX, World, Viewport, DroppedObjects);
	}

	// Reset if creating a preview actor.
	bIsDroppingPreviewActor = false;

	return bResult;
}

bool FLevelEditorViewportClient::DropObjectsAtCoordinates(int32 MouseX, int32 MouseY, const TArray<UObject*>& DroppedObjects, TArray<AActor*>& OutNewActors, bool bOnlyDropOnTarget, bool bCreateDropPreview, bool bSelectOutput, UActorFactory* FactoryToUse)
{
	FDropObjectOptions Options;
	Options.bOnlyDropOnTarget = bOnlyDropOnTarget;
	Options.bCreateDropPreview = bCreateDropPreview;
	Options.bSelectOutput = bSelectOutput;
	Options.FactoryToUse = FactoryToUse;

	TArray<FTypedElementHandle> NewElements;
	bool bSuccess = DropObjectsAtCoordinates(MouseX, MouseY, DroppedObjects, NewElements, Options);
	
	LevelEditorViewportLocals::ForEachActorInElementArray(NewElements, [&OutNewActors](AActor& Actor)
	{
		OutNewActors.Add(&Actor);
	});
	return bSuccess;
}

/**
 *	Called to check if a material can be applied to an object, given the hit proxy
 */
bool FLevelEditorViewportClient::CanApplyMaterialToHitProxy( const HHitProxy* HitProxy ) const
{
	// The check for HWidgetAxis is made to prevent the transform widget from blocking an attempt at applying a material to a mesh.
	return ( HitProxy->IsA(HModel::StaticGetType()) || HitProxy->IsA(HActor::StaticGetType()) || HitProxy->IsA(HWidgetAxis::StaticGetType()) );
}

FTrackingTransaction::FTrackingTransaction()
{
}

FTrackingTransaction::~FTrackingTransaction()
{
	End();
	USelection::SelectionChangedEvent.RemoveAll(this);
}

void FTrackingTransaction::Begin(const FText& Description, AActor* AdditionalActor)
{
	End();
	
	ScopedTransaction = new FScopedTransaction( Description );

	TrackingTransactionState = ETransactionState::Active;

	if (UTypedElementSelectionSet* SelectionSet = GetMutableSelectionSet())
	{
		// TODO: Ideally "Begin" would *not* speculatively call Modify at all (nor need to track/restore dirty state), 
		//       as Modify should be called on-demand prior to changes within the transaction, however various existing 
		//       code makes assumptions that tracking transactions will call Modify on the active selection.

		// Call modify the currently selected objects, gathering up any groups for processing later
		TSet<AGroupActor*> GroupsPendingModify;
		auto ModifyObject = [this, &GroupsPendingModify](UObject* InObject)
		{
			// Track the dirty state of the packages so that it can be restored if the selection changes mid-transaction (eg, drag duplication)
			UPackage* Package = InObject->GetOutermost();
			if (!InitialPackageDirtyStates.Contains(Package))
			{
				InitialPackageDirtyStates.Add(Package, Package->IsDirty());
			}

			AActor* Actor = Cast<AActor>(InObject);
			if (Actor && UActorGroupingUtils::IsGroupingActive())
			{
				if (AGroupActor* GroupActor = AGroupActor::GetRootForActor(Actor, true))
				{
					GroupsPendingModify.Add(GroupActor);
				}
			}

			InObject->Modify();

			return true;
		};
		SelectionSet->ForEachSelectedObject(ModifyObject);
		if (AdditionalActor)
		{
			ModifyObject(AdditionalActor);
		}

		// Recursively call modify on any groups, as viewport manipulation of an actor within a group tends to affect the entire group
		for (AGroupActor* GroupPendingModify : GroupsPendingModify)
		{
			GroupPendingModify->ForEachActorInGroup([](AActor* InGroupedActor, AGroupActor* InGroupActor)
			{
				InGroupedActor->Modify();
			});
		}

		SelectionSet->OnChanged().AddRaw(this, &FTrackingTransaction::OnEditorSelectionChanged);
	}
}

void FTrackingTransaction::End()
{
	if( ScopedTransaction )
	{
		delete ScopedTransaction;
		ScopedTransaction = NULL;
	}
	TrackingTransactionState = ETransactionState::Inactive;
	InitialPackageDirtyStates.Empty();
	if (UTypedElementSelectionSet* SelectionSet = GetMutableSelectionSet())
	{
		SelectionSet->OnChanged().RemoveAll(this);
	}
}

void FTrackingTransaction::Cancel()
{
	if(ScopedTransaction)
	{
		ScopedTransaction->Cancel();
	}

	// If the transaction is cancelled, reset the package dirty states to reflect their original state
	for (const TTuple<UPackage*, bool>& InitialPackageDirtyState : InitialPackageDirtyStates)
	{
		UPackage* Package = InitialPackageDirtyState.Key;
		const bool bInitialDirtyState = InitialPackageDirtyState.Value;
		if (!bInitialDirtyState && Package->IsDirty())
		{
			Package->SetDirtyFlag(false);
		}
	}

	End();
}

void FTrackingTransaction::BeginPending(const FText& Description)
{
	End();

	PendingDescription = Description;
	TrackingTransactionState = ETransactionState::Pending;
}

void FTrackingTransaction::PromotePendingToActive()
{
	if(IsPending())
	{
		Begin(PendingDescription);
		PendingDescription = FText();
	}
}

// TODO: Ideally this should be given the selection set for the level editor, rather than rely on the global state
const UTypedElementSelectionSet* FTrackingTransaction::GetSelectionSet() const
{
	return GetMutableSelectionSet();
}
UTypedElementSelectionSet* FTrackingTransaction::GetMutableSelectionSet() const
{
	return (GEditor && GEditor->GetSelectedActors())
		? GEditor->GetSelectedActors()->GetElementSelectionSet()
		: nullptr;
}

void FTrackingTransaction::OnEditorSelectionChanged(const UTypedElementSelectionSet* InSelectionSet)
{
	if (!InitialPackageDirtyStates.Num())
	{
		return;
	}

	checkf(TrackingTransactionState == ETransactionState::Active, TEXT("Inactive tracking transaction contains package states"));

	// The selection state has changed mid-transaction (eg, drag duplication)
	// Restore the dirty state on any packages which no longer have selected objects
	TSet<UPackage*> SelectedPackages;
	InSelectionSet->ForEachSelectedObject([&SelectedPackages](UObject* InObject)
	{
		SelectedPackages.Add(InObject->GetOutermost());
		return true;
	});

	for (const TTuple<UPackage*, bool>& InitialPackageDirtyState : InitialPackageDirtyStates)
	{
		UPackage* Package = InitialPackageDirtyState.Key;
		const bool bInitialDirtyState = InitialPackageDirtyState.Value;
		if (!SelectedPackages.Contains(Package) && !bInitialDirtyState && Package->IsDirty())
		{
			Package->SetDirtyFlag(false);
		}
	}

	InitialPackageDirtyStates.Empty();
}


FLevelEditorViewportClient::FLevelEditorViewportClient(const TSharedPtr<SLevelViewport>& InLevelViewport)
	: FEditorViewportClient(&GLevelEditorModeTools(), nullptr, StaticCastSharedPtr<SEditorViewport>(InLevelViewport))
	, ViewHiddenLayers()
	, VolumeActorVisibility()
	, LastEditorViewLocation( FVector::ZeroVector )
	, LastEditorViewRotation( FRotator::ZeroRotator )
	, ColorScale( FVector(1,1,1) )
	, FadeColor( FColor(0,0,0) )
	, FadeAmount(0.f)	
	, bEnableFading(false)
	, bEnableColorScaling(false)
	, bDrawBaseInfo(false)
	, bDuplicateOnNextDrag( false )
	, bDuplicateActorsInProgress( false )
	, bIsTrackingBrushModification( false )
	, bOnlyMovedPivot(false)
	, bLockedCameraView(true)
	, bNeedToRestoreComponentBeingMovedFlag(false)
	, bHasBegunGizmoManipulation(false)
	, bReceivedFocusRecently(false)
	, bAlwaysShowModeWidgetAfterSelectionChanges(true)
	, CachedElementsToManipulate(UTypedElementRegistry::GetInstance()->CreateElementList())
	, SpriteCategoryVisibility()
	, World(nullptr)
	, TrackingTransaction()
	, CachedPilotTransform(TOptional<FTransform>())
	, DropPreviewMouseX(0)
	, DropPreviewMouseY(0)
	, bWasControlledByOtherViewport(false)
	, bCurrentlyEditingThroughMovementWidget(false)
	, bEditorCameraCut(false)
	, bWasEditorCameraCut(false)
	, bApplyCameraSpeedScaleByDistance(true)
{
	// By default a level editor viewport is pointed to the editor world
	SetReferenceToWorldContext(GEditor->GetEditorWorldContext());

	GEditor->AddLevelViewportClients(this);

	// The level editor fully supports mode tools and isn't doing any incompatible stuff with the Widget
	ModeTools->SetWidgetMode(UE::Widget::WM_Translate);
	Widget->SetUsesEditorModeTools(ModeTools.Get());

	ModeTools->OnWidgetModeChanged().AddRaw(this, &FLevelEditorViewportClient::OnWidgetModeChanged);

	// Register for editor cleanse events so we can release references to hovered actors
	FEditorSupportDelegates::CleanseEditor.AddRaw(this, &FLevelEditorViewportClient::OnEditorCleanse);

	// Register for editor PIE event that allows you to clean up states that might block PIE
	FEditorDelegates::PreBeginPIE.AddRaw(this, &FLevelEditorViewportClient::OnPreBeginPIE);

	// Add a delegate so we get informed when an actor has moved.
	GEngine->OnActorMoved().AddRaw(this, &FLevelEditorViewportClient::OnActorMoved);

	// Set up defaults for the draw helper.
	DrawHelper.bDrawGrid = true;
	DrawHelper.bDrawPivot = false;
	DrawHelper.bDrawBaseInfo = false;
	DrawHelper.bDrawWorldBox = false;
	DrawHelper.bDrawKillZ = false;

	InitializeVisibilityFlags();

	// Sign up for notifications about users changing settings.
	GetMutableDefault<ULevelEditorViewportSettings>()->OnSettingChanged().AddRaw(this, &FLevelEditorViewportClient::HandleViewportSettingChanged);

	FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor").OnMapChanged().AddRaw(this, &FLevelEditorViewportClient::OnMapChanged);

	DropPreviewElements = StaticDropPreviewElements.Pin();
	if (!StaticDropPreviewElements.IsValid() && UTypedElementRegistry::GetInstance())
	{
		DropPreviewElements = UTypedElementRegistry::GetInstance()->CreateElementList();
		StaticDropPreviewElements = DropPreviewElements;
	}
}

//
//	FLevelEditorViewportClient::~FLevelEditorViewportClient
//

FLevelEditorViewportClient::~FLevelEditorViewportClient()
{
	if (UTypedElementSelectionSet* SelectionSet = GetMutableSelectionSet())
	{
		SelectionSet->OnPreChange().RemoveAll(this);
	}
	if (UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance())
	{
		Registry->OnProcessingDeferredElementsToDestroy().RemoveAll(this);
	}
	ResetElementsToManipulate();

	ModeTools->OnWidgetModeChanged().RemoveAll(this);

	// Unregister for all global callbacks to this object
	FEditorSupportDelegates::CleanseEditor.RemoveAll(this);
	FEditorDelegates::PreBeginPIE.RemoveAll(this);

	FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor").OnMapChanged().RemoveAll(this);

	if(GEngine)
	{
		// Remove our move delegate
		GEngine->OnActorMoved().RemoveAll(this);

		// make sure all actors have this view removed from their visibility bits
		ULayersSubsystem* Layers = GEditor->GetEditorSubsystem<ULayersSubsystem>();
		Layers->RemoveViewFromActorViewVisibility(this);

		GEditor->RemoveLevelViewportClients(this);
		
		if (FocusTimerHandle.IsValid())
		{
			GEditor->GetTimerManager()->ClearTimer(FocusTimerHandle);
		}

		GetMutableDefault<ULevelEditorViewportSettings>()->OnSettingChanged().RemoveAll(this);

		RemoveReferenceToWorldContext(GEditor->GetEditorWorldContext());
	}

	//make to clean up the global "current" & "last" clients when we delete the active one.
	if (GCurrentLevelEditingViewportClient == this)
	{
		GCurrentLevelEditingViewportClient = NULL;
	}
	if (GLastKeyLevelEditingViewportClient == this)
	{
		GLastKeyLevelEditingViewportClient = NULL;
	}
}

void FLevelEditorViewportClient::InitializeVisibilityFlags()
{
	// make sure all actors know about this view for per-view layer vis
	ULayersSubsystem* Layers = GEditor->GetEditorSubsystem<ULayersSubsystem>();
	Layers->UpdatePerViewVisibility(this);

	// Get the number of volume classes so we can initialize our bit array
	TArray<UClass*> VolumeClasses;
	UUnrealEdEngine::GetSortedVolumeClasses(&VolumeClasses);
	VolumeActorVisibility.Init(true, VolumeClasses.Num());

	// Initialize all sprite categories to visible
	SpriteCategoryVisibility.Init(true, GUnrealEd->SpriteIDToIndexMap.Num());
}

void FLevelEditorViewportClient::InitializeViewportInteraction()
{
	{
		UTypedElementSelectionSet* SelectionSet = GetMutableSelectionSet();
		SelectionSet->OnPreChange().AddRaw(this, &FLevelEditorViewportClient::ResetElementsToManipulateFromSelectionChange);

		UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();
		Registry->OnProcessingDeferredElementsToDestroy().AddRaw(this, &FLevelEditorViewportClient::ResetElementsToManipulateFromProcessingDeferredElementsToDestroy);
	}

	{
		ViewportInteraction->GetDefaultInterfaceCustomization()->SetToolkitHost(ParentLevelEditor.Pin().Get());
	}

	{
		TUniquePtr<FActorElementLevelEditorViewportInteractionCustomization> ActorCustomization = MakeUnique<FActorElementLevelEditorViewportInteractionCustomization>();
		ActorCustomization->SetLevelEditorViewportClient(this);
		ActorCustomization->SetToolkitHost(ParentLevelEditor.Pin().Get());
		ViewportInteraction->RegisterInterfaceCustomizationByTypeName(NAME_Actor, MoveTemp(ActorCustomization));
	}
	{
		TUniquePtr<FComponentElementLevelEditorViewportInteractionCustomization> ComponentCustomization = MakeUnique<FComponentElementLevelEditorViewportInteractionCustomization>();
		ComponentCustomization->SetLevelEditorViewportClient(this);
		ComponentCustomization->SetToolkitHost(ParentLevelEditor.Pin().Get());
		ViewportInteraction->RegisterInterfaceCustomizationByTypeName(NAME_Components, MoveTemp(ComponentCustomization));
	}
}

FSceneView* FLevelEditorViewportClient::CalcSceneView(FSceneViewFamily* ViewFamily, const int32 StereoViewIndex)
{
	bWasControlledByOtherViewport = false;

	// set all other matching viewports to my location, if the LOD locking is enabled,
	// unless another viewport already set me this frame (otherwise they fight)
	if (GetDefault<ULevelEditorViewportSettings>()->bUseLODViewLocking)
	{
		for (FLevelEditorViewportClient* ViewportClient : GEditor->GetLevelViewportClients())
		{
			//only change camera for a viewport that is looking at the same scene
			if (ViewportClient == NULL || GetScene() != ViewportClient->GetScene())
			{
				continue;
			}

			// go over all other level viewports
			if (ViewportClient->Viewport && ViewportClient != this)
			{
				// force camera of same-typed viewports
				if (ViewportClient->GetViewportType() == GetViewportType())
				{
					ViewportClient->SetViewLocation( GetViewLocation() );
					ViewportClient->SetViewRotation( GetViewRotation() );
					ViewportClient->SetOrthoZoom( GetOrthoZoom() );

					// don't let this other viewport update itself in its own CalcSceneView
					ViewportClient->bWasControlledByOtherViewport = true;
				}
				// when we are LOD locking, ortho views get their camera position from this view, so make sure it redraws
				else if (IsPerspective() && !ViewportClient->IsPerspective())
				{
					// don't let this other viewport update itself in its own CalcSceneView
					ViewportClient->bWasControlledByOtherViewport = true;
				}
			}

			// if the above code determined that this viewport has changed, delay the update unless
			// an update is already in the pipe
			if (ViewportClient->bWasControlledByOtherViewport && ViewportClient->TimeForForceRedraw == 0.0)
			{
				ViewportClient->TimeForForceRedraw = FPlatformTime::Seconds() + 0.9 + FMath::FRand() * 0.2;
			}
		}
	}

	FSceneView* View = FEditorViewportClient::CalcSceneView(ViewFamily, StereoViewIndex);

	View->ViewActor = ActorLocks.GetLock().GetLockedActor();
	View->SpriteCategoryVisibility = SpriteCategoryVisibility;
	View->bCameraCut = bEditorCameraCut;
	View->bHasSelectedComponents = GEditor->GetSelectedComponentCount() > 0;

	return View;

}

ELevelViewportType FLevelEditorViewportClient::GetViewportType() const
{
	const UCameraComponent* ActiveCameraComponent = GetCameraComponentForView();
	
	if (ActiveCameraComponent != NULL)
	{
		return (ActiveCameraComponent->ProjectionMode == ECameraProjectionMode::Perspective) ? LVT_Perspective : LVT_OrthoFreelook;
	}
	else
	{
		return FEditorViewportClient::GetViewportType();
	}
}

void FLevelEditorViewportClient::SetViewportTypeFromTool( ELevelViewportType InViewportType )
{
	SetViewportType(InViewportType);
}

void FLevelEditorViewportClient::SetViewportType( ELevelViewportType InViewportType )
{
	if (InViewportType != LVT_Perspective)
	{
		SetActorLock(nullptr);
		UpdateViewForLockedActor();
	}

	FEditorViewportClient::SetViewportType(InViewportType);
}

void FLevelEditorViewportClient::RotateViewportType()
{
	SetActorLock(nullptr);
	UpdateViewForLockedActor();

	FEditorViewportClient::RotateViewportType();
}

void FLevelEditorViewportClient::OverridePostProcessSettings( FSceneView& View )
{
	const UCameraComponent* CameraComponent = GetCameraComponentForView();
	if (CameraComponent)
	{
		View.OverridePostProcessSettings(CameraComponent->PostProcessSettings, CameraComponent->PostProcessBlendWeight);
	}
}

bool FLevelEditorViewportClient::ShouldLockPitch() const 
{
	// If we have somehow gotten out of the locked rotation
	if ((GetViewRotation().Pitch < -90.f + KINDA_SMALL_NUMBER || GetViewRotation().Pitch > 90.f - KINDA_SMALL_NUMBER)
		&& FMath::Abs(GetViewRotation().Roll) > (90.f - KINDA_SMALL_NUMBER))
	{
		return false;
	}
	// Else use the standard rules
	return FEditorViewportClient::ShouldLockPitch();
}

void FLevelEditorViewportClient::BeginCameraMovement(bool bHasMovement)
{
	const bool bIsUsingLegacyMovementNotify = GetDefault<ULevelEditorViewportSettings>()->bUseLegacyCameraMovementNotifications;
	// If there's new movement broadcast it
	if (bHasMovement)
	{
		if (!bIsCameraMoving)
		{
			AActor* ActorLock = GetActiveActorLock().Get();
			if (!bIsCameraMovingOnTick && ActorLock)
			{
				GEditor->BroadcastBeginCameraMovement(*ActorLock);
				// consider modification from piloting as relative location changes
				FProperty* TransformProperty = FComponentElementLevelEditorViewportInteractionCustomization::GetEditTransformProperty(UE::Widget::WM_Translate);
				if (TransformProperty)
				{
					// Create edit property event
					FEditPropertyChain PropertyChain;
					PropertyChain.AddHead(TransformProperty);

					// Broadcast Pre Edit change notification, we can't call PreEditChange directly on Actor or ActorComponent from here since it will unregister the components until PostEditChange
					FCoreUObjectDelegates::OnPreObjectPropertyChanged.Broadcast(ActorLock, PropertyChain);
				}
			}

			if (!bIsUsingLegacyMovementNotify)
			{
				TrackingTransaction.Cancel();
			}
			bIsCameraMoving = true;
		}
	}
	else
	{
		const bool bDisableCameraMovement = [&]
		{
			if (bIsUsingLegacyMovementNotify)
			{
				return true;
			}
			if (ModeTools->UsesViewportITF())
			{
				if (UModeManagerInteractiveToolsContext* Context = ModeTools->GetInteractiveToolsContext())
				{
					if (const UViewportInteractionsBehaviorSource* Source = Context->GetViewportInteractionsBehaviorSource())
					{
						return !Source->IsMouseLooking();
					}
				}
				return true;
			}
			return !bIsTracking;
		}();
	
		if (bDisableCameraMovement)
		{
			bIsCameraMoving = false;
		}
	}
}

void FLevelEditorViewportClient::EndCameraMovement()
{
	// If there was movement and it has now stopped, broadcast it
	if (bIsCameraMovingOnTick && !bIsCameraMoving)
	{
		if (AActor* ActorLock = GetActiveActorLock().Get())
		{
			GEditor->BroadcastEndCameraMovement(*ActorLock);
			// Create post edit property change event, consider modification from piloting as relative location changes
			FProperty* TransformProperty = FComponentElementLevelEditorViewportInteractionCustomization::GetEditTransformProperty(UE::Widget::WM_Translate);
			FPropertyChangedEvent PropertyChangedEvent(TransformProperty, EPropertyChangeType::ValueSet);

			// Broadcast Post Edit change notification, we can't call PostEditChangeProperty directly on Actor or ActorComponent from here since it wasn't pair with a proper PreEditChange
			FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(ActorLock, PropertyChangedEvent);
		}
	}
}

void FLevelEditorViewportClient::PerspectiveCameraMoved()
{
	// Update the locked actor (if any) from the camera
	MoveLockedActorToCamera();

	// If any other viewports have this actor locked too, we need to update them
	if( GetActiveActorLock().IsValid() )
	{
		// Skip hit proxy invalidation during mouse-look rotation since the user is not clicking on anything
		constexpr bool bCheckRealtime = false;
		const bool bInvalidateHitProxies = !(ViewportNavigationHelper && ViewportNavigationHelper->bIsMouseLooking);
		UpdateLockedActorViewports(GetActiveActorLock().Get(), bCheckRealtime, bInvalidateHitProxies);
	}

	// Broadcast 'camera moved' delegate
	FEditorDelegates::OnEditorCameraMoved.Broadcast(GetViewLocation(), GetViewRotation(), ViewportType, ViewIndex);
	
	if (bHasAudioFocus)
	{
		UpdateAudioListener();
	}
}

/**
 * Reset the camera position and rotation.  Used when creating a new level.
 */
void FLevelEditorViewportClient::ResetCamera()
{
	// Initialize perspective view transform
	ViewTransformPerspective.SetLocation(EditorViewportDefs::DefaultPerspectiveViewLocation);
	ViewTransformPerspective.SetRotation(EditorViewportDefs::DefaultPerspectiveViewRotation);
	ViewTransformPerspective.SetLookAt(FVector::ZeroVector);

	FMatrix OrbitMatrix = ViewTransformPerspective.ComputeOrbitMatrix();
	OrbitMatrix = OrbitMatrix.InverseFast();

	ViewTransformPerspective.SetRotation(OrbitMatrix.Rotator());
	ViewTransformPerspective.SetLocation(OrbitMatrix.GetOrigin());

	ViewTransformPerspective.SetOrthoZoom(DEFAULT_ORTHOZOOM);

	// Initialize orthographic view transform
	ViewTransformOrthographic.SetLocation(FVector::ZeroVector);
	ViewTransformOrthographic.SetRotation(FRotator::ZeroRotator);
	ViewTransformOrthographic.SetOrthoZoom(DEFAULT_ORTHOZOOM);

	ViewFOV = FOVAngle;

	SetIsCameraCut();

	// Broadcast 'camera moved' delegate
	FEditorDelegates::OnEditorCameraMoved.Broadcast(GetViewLocation(), GetViewRotation(), ViewportType, ViewIndex);
}

void FLevelEditorViewportClient::ResetViewForNewMap()
{
	ResetCamera();
	bForcingUnlitForNewMap = true;
}

void FLevelEditorViewportClient::PrepareCameraForPIE()
{
	LastEditorViewLocation = GetViewLocation();
	LastEditorViewRotation = GetViewRotation();
}

void FLevelEditorViewportClient::RestoreCameraFromPIE()
{
	const bool bRestoreEditorCamera = GEditor != NULL && !GetDefault<ULevelEditorViewportSettings>()->bEnableViewportCameraToUpdateFromPIV;

	//restore the camera position if this is an ortho viewport OR if PIV camera dropping is undesired
	if ( IsOrtho() || bRestoreEditorCamera )
	{
		SetViewLocation( LastEditorViewLocation );
		SetViewRotation( LastEditorViewRotation );
	}

	if( IsPerspective() )
	{
		ViewFOV = FOVAngle;
		RemoveCameraRoll();
	}
}

void FLevelEditorViewportClient::ReceivedFocus(FViewport* InViewport)
{
	if (!bReceivedFocusRecently)
	{
		bReceivedFocusRecently = true;

		// A few frames can pass between receiving focus and processing a click, so we use a timer to track whether we have recently received focus.
		FTimerDelegate ResetFocusReceivedTimer;
		ResetFocusReceivedTimer.BindLambda([&] ()
		{
			bReceivedFocusRecently = false;
			FocusTimerHandle.Invalidate(); // The timer will only execute once, so we can invalidate now.
		});
		GEditor->GetTimerManager()->SetTimer(FocusTimerHandle, ResetFocusReceivedTimer, 0.1f, false);	
	}

	FEditorViewportClient::ReceivedFocus(InViewport);
	
	UpdateAudioListener();
	SetAudioFocus();
}

void FLevelEditorViewportClient::LostFocus(FViewport* InViewport)
{
	if (ModeTools)
	{
		if (const FEditorViewportClient* FocusedViewportClient = ModeTools->GetFocusedViewportClient())
		{
			// Make sure we are calling LostFocus on currently focused viewport
			// We don't need to call LostFocus multiple times on the same viewport if it's already un-focused.
			if (InViewport == FocusedViewportClient->Viewport)
            {
            	FEditorViewportClient::LostFocus(InViewport);
            }
		}
	}

	GEditor->SetPreviewMeshMode(false);
	ClearAudioFocus();
}


//
//	FLevelEditorViewportClient::ProcessClick
//
void FLevelEditorViewportClient::ProcessClick(FSceneView& View, HHitProxy* HitProxy, FKey Key, EInputEvent Event, uint32 HitX, uint32 HitY)
{
	UBrushEditingSubsystem* BrushSubsystem = GEditor->GetEditorSubsystem<UBrushEditingSubsystem>();

	// We may have started gizmo manipulation if hot-keys were pressed when we started this click
	// If so, we need to end that now before we potentially update the selection below, 
	// otherwise the usual call in TrackingStopped would include the newly selected element
	if (bHasBegunGizmoManipulation)
	{
		FTypedElementListConstRef ElementsToManipulate = GetElementsToManipulate();
		ViewportInteraction->EndGizmoManipulation(ElementsToManipulate, GetWidgetMode(), ETypedElementViewportInteractionGizmoManipulationType::Click);
		bHasBegunGizmoManipulation = false;
	}

	const FViewportClick Click(&View,this,Key,Event,HitX,HitY);
	if (Click.GetKey() == EKeys::MiddleMouseButton && !Click.IsAltDown() && !Click.IsShiftDown())
	{
		if (Click.IsControlDown())
		{
			if (TSharedPtr<ILevelEditor> ParentLevelEditorPinned = ParentLevelEditor.Pin())
			{
				ParentLevelEditorPinned->SummonLevelViewportViewOptionMenu(LVT_Perspective);
			}
		}
	
		return;
	}
	if (!ModeTools->HandleClick(this, HitProxy,Click))
	{
		const FTypedElementHandle HitElement = HitProxy ? HitProxy->GetElementHandle() : FTypedElementHandle();

		if (HitProxy == NULL)
		{
			UE::Editor::ViewportSelectionUtilities::ClickBackdrop(this,Click);
		}
		else if (HitProxy->IsA(HWidgetAxis::StaticGetType()))
		{
			// The user clicked on an axis translation/rotation hit proxy.  However, we want
			// to find out what's underneath the axis widget.  To do this, we'll need to render
			// the viewport's hit proxies again, this time *without* the axis widgets!

			// OK, we need to be a bit evil right here.  Basically we want to hijack the ShowFlags
			// for the scene so we can re-render the hit proxies without any axis widgets.  We'll
			// store the original ShowFlags and modify them appropriately
			const bool bOldModeWidgets1 = EngineShowFlags.ModeWidgets;
			const bool bOldModeWidgets2 = View.Family->EngineShowFlags.ModeWidgets;

			EngineShowFlags.SetModeWidgets(false);
			FSceneViewFamily* SceneViewFamily = const_cast<FSceneViewFamily*>(View.Family);
			SceneViewFamily->EngineShowFlags.SetModeWidgets(false);
			bool bWasWidgetDragging = Widget->IsDragging();
			Widget->SetDragging(false);

			// Invalidate the hit proxy map so it will be rendered out again when GetHitProxy
			// is called
			Viewport->InvalidateHitProxy();

			// This will actually re-render the viewport's hit proxies!
			HHitProxy* HitProxyWithoutAxisWidgets = Viewport->GetHitProxy(HitX, HitY);
			if (HitProxyWithoutAxisWidgets != NULL && !HitProxyWithoutAxisWidgets->IsA(HWidgetAxis::StaticGetType()))
			{
				// Try this again, but without the widget this time!
				ProcessClick(View, HitProxyWithoutAxisWidgets, Key, Event, HitX, HitY);
			}

			// Undo the evil
			EngineShowFlags.SetModeWidgets(bOldModeWidgets1);
			SceneViewFamily->EngineShowFlags.SetModeWidgets(bOldModeWidgets2);

			Widget->SetDragging(bWasWidgetDragging);

			// Invalidate the hit proxy map again so that it'll be refreshed with the original
			// scene contents if we need it again later.
			Viewport->InvalidateHitProxy();
		}
		else if (GUnrealEd->ComponentVisManager.HandleClick(this, HitProxy, Click))
		{
			// Component vis manager handled the click
		}
		else if (HitElement && UE::Editor::ViewportSelectionUtilities::ClickElement(this, HitElement, Click))
		{
			// Element handled the click
		}
		else if (HitProxy->IsA(HActor::StaticGetType()))
		{
			HActor* ActorHitProxy = (HActor*)HitProxy;
			AActor* ConsideredActor = ActorHitProxy->Actor;
			if (ConsideredActor) // It is possible to be clicking something during level transition if you spam click, and it might not be valid by this point
			{
				while (ConsideredActor->IsChildActor())
				{
					ConsideredActor = ConsideredActor->GetParentActor();
				}

				// We want to process the click on the component only if:
				// 1. The actor clicked is already selected
				// 2. The actor selected is the only actor selected
				// 3. The actor selected is blueprintable
				// 4. No components are already selected and the click was a double click
				// 5. OR, a component is already selected and the click was NOT a double click
				const bool bActorAlreadySelectedExclusively = GEditor->GetSelectedActors()->IsSelected(ConsideredActor) && (GEditor->GetSelectedActorCount() == 1);
				const bool bActorIsBlueprintable = FKismetEditorUtilities::CanCreateBlueprintOfClass(ConsideredActor->GetClass());
				const bool bComponentAlreadySelected = GEditor->GetSelectedComponentCount() > 0;
				const bool bWasDoubleClick = (Click.GetEvent() == IE_DoubleClick);

				const bool bSelectComponent = bActorAlreadySelectedExclusively && bActorIsBlueprintable && (bComponentAlreadySelected != bWasDoubleClick);
				bool bComponentSelected = false;

				if (bSelectComponent)
				{
					bComponentSelected = UE::Editor::ViewportSelectionUtilities::ClickComponent(this, ActorHitProxy, Click);
				}
				
				if (!bComponentSelected)
				{
					UE::Editor::ViewportSelectionUtilities::ClickActor(this, ConsideredActor, Click, true);
				}

				// We clicked an actor, allow the pivot to reposition itself.
				// GUnrealEd->SetPivotMovedIndependently(false);
			}
		}
		else if (HitProxy->IsA(HInstancedStaticMeshInstance::StaticGetType()))
		{
			UE::Editor::ViewportSelectionUtilities::ClickActor(this, ((HInstancedStaticMeshInstance*)HitProxy)->Component->GetOwner(), Click, true);
		}
		else if (HitProxy->IsA(HBSPBrushVert::StaticGetType()) && ((HBSPBrushVert*)HitProxy)->Brush.IsValid())
		{
			FVector Vertex = FVector(*((HBSPBrushVert*)HitProxy)->Vertex);
			UE::Editor::ViewportSelectionUtilities::ClickBrushVertex(this,((HBSPBrushVert*)HitProxy)->Brush.Get(),&Vertex,Click);
		}
		else if (HitProxy->IsA(HStaticMeshVert::StaticGetType()))
		{
			UE::Editor::ViewportSelectionUtilities::ClickStaticMeshVertex(this,((HStaticMeshVert*)HitProxy)->Actor,((HStaticMeshVert*)HitProxy)->Vertex,Click);
		}
		else if (BrushSubsystem && BrushSubsystem->ProcessClickOnBrushGeometry(static_cast<FEditorViewportClient*>(this), HitProxy, Click))
		{
			// Handled by the brush subsystem
		}
		else if (HitProxy->IsA(HModel::StaticGetType()))
		{
			HModel* ModelHit = (HModel*)HitProxy;

			// Compute the viewport's current view family.
			FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues( Viewport, GetScene(), EngineShowFlags ));
			FSceneView* SceneView = CalcSceneView( &ViewFamily );

			uint32 SurfaceIndex = INDEX_NONE;
			if(ModelHit->ResolveSurface(SceneView,HitX,HitY,SurfaceIndex))
			{
				UE::Editor::ViewportSelectionUtilities::ClickSurface(this,ModelHit->GetModel(),SurfaceIndex,Click);
			}
		}
		else if (HitProxy->IsA(HLevelSocketProxy::StaticGetType()))
		{
			UE::Editor::ViewportSelectionUtilities::ClickLevelSocket(this, HitProxy, Click);
		}
	}
}

void FLevelEditorViewportClient::SummonContextMenu(const FTypedElementHandle& HitProxyElement)
{
	if (TSharedPtr<ILevelEditor> ParentLevelEditorPinned = ParentLevelEditor.Pin())
	{
		ParentLevelEditorPinned->SummonLevelViewportContextMenu(HitProxyElement);
	}
}

// Frustum parameters for the perspective view.
static float GPerspFrustumAngle=90.f;
static float GPerspFrustumAspectRatio=1.77777f;
static float GPerspFrustumStartDist=GNearClippingPlane;
static float GPerspFrustumEndDist=UE_FLOAT_HUGE_DISTANCE;
static FMatrix GPerspViewMatrix;


void FLevelEditorViewportClient::Tick(float DeltaTime)
{
	if (bWasEditorCameraCut && bEditorCameraCut)
	{
		bEditorCameraCut = false;
	}
	bWasEditorCameraCut = bEditorCameraCut;
	bCurrentlyEditingThroughMovementWidget = false;

	// Gives FindViewComponentForActor a chance to refresh once every Tick.
	ViewComponentForActorCache.Reset();

	FEditorViewportClient::Tick(DeltaTime);

	// Update the preview mesh for the preview mesh mode. 
	GEditor->UpdatePreviewMesh();

	// Copy perspective views to the global if this viewport has streaming volume previs enabled
	if ( (IsPerspective() && GetDefault<ULevelEditorViewportSettings>()->bLevelStreamingVolumePrevis && Viewport->GetSizeXY().X > 0) )
	{
		GPerspFrustumAngle=ViewFOV;
		GPerspFrustumAspectRatio=AspectRatio;
		GPerspFrustumStartDist=GetNearClipPlane();

		GPerspFrustumEndDist= UE_FLOAT_HUGE_DISTANCE;

		FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
			Viewport,
			GetScene(),
			EngineShowFlags)
			.SetRealtimeUpdate( IsRealtime() ) );
		FSceneView* View = CalcSceneView(&ViewFamily);
		GPerspViewMatrix = View->ViewMatrices.GetWorldToView();
	}

	UpdateViewForLockedActor(DeltaTime);

	UserIsControllingAtmosphericLightTimer = FMath::Max(UserIsControllingAtmosphericLightTimer - DeltaTime, 0.0f);
}

void FLevelEditorViewportClient::UpdateViewForLockedActor(float DeltaTime)
{
	// We can't be locked to a cinematic actor if this viewport doesn't allow cinematic control
	if (!bAllowCinematicControl && ActorLocks.CinematicActorLock.HasValidLockedActor())
	{
		ActorLocks.CinematicActorLock = FLevelViewportActorLock::None;
	}

	bUseControllingActorViewInfo = false;
	ControllingActorViewInfo = FMinimalViewInfo();
	ControllingActorAspectRatioAxisConstraint.Reset();
	ControllingActorExtraPostProcessBlends.Empty();
	ControllingActorExtraPostProcessBlendWeights.Empty();

	const FLevelViewportActorLock& ActiveLock = ActorLocks.GetLock();
	AActor* Actor = ActiveLock.GetLockedActor();
	if( Actor != NULL )
	{
		// Check if the viewport is transitioning
		FViewportCameraTransform& ViewTransform = GetViewTransform();
		if (!ViewTransform.IsPlaying())
		{
			// Update transform
			bool bGotCameraView = false;
			if (bLockedCameraView)
			{
				// If this is a camera actor, then inherit some other settings
				UActorComponent* const ViewComponent = FindViewComponentForActor(Actor);
				if (ViewComponent != nullptr)
				{
					if ( ensure(ViewComponent->GetEditorPreviewInfo(DeltaTime, ControllingActorViewInfo)) )
					{
						bUseControllingActorViewInfo = true;
						if (UCameraComponent* CameraComponent = Cast<UCameraComponent>(ViewComponent))
						{
							CameraComponent->GetExtraPostProcessBlends(ControllingActorExtraPostProcessBlends, ControllingActorExtraPostProcessBlendWeights);
						}

						// Axis constraint for aspect ratio
						ControllingActorAspectRatioAxisConstraint = ActiveLock.AspectRatioAxisConstraint;
						
						// Post processing is handled by OverridePostProcessingSettings
						ViewFOV = ControllingActorViewInfo.FOV;
						AspectRatio = ControllingActorViewInfo.AspectRatio;
						SetViewLocation(ControllingActorViewInfo.Location);
						SetViewRotation(ControllingActorViewInfo.Rotation);
						bGotCameraView = true;
					}
				}
			}
			if (!bGotCameraView)
			{
				if (Actor->GetAttachParentActor() != NULL)
				{
					// Actor is parented, so use the actor to world matrix for translation and rotation information.
					SetViewLocation(Actor->GetActorLocation());
					SetViewRotation(Actor->GetActorRotation());
				}
				else if (Actor->GetRootComponent() != NULL)
				{
					// No attachment, so just use the relative location, so that we don't need to
					// convert from a quaternion, which loses winding information.
					SetViewLocation(Actor->GetRootComponent()->GetRelativeLocation());
					SetViewRotation(Actor->GetRootComponent()->GetRelativeRotation());
				}
			}

			const double DistanceToCurrentLookAt = FVector::Dist( GetViewLocation() , GetLookAtLocation() );

			const FQuat CameraOrientation = FQuat::MakeFromEuler( GetViewRotation().Euler() );
			FVector Direction = CameraOrientation.RotateVector( FVector(1,0,0) );

			SetLookAtLocation( GetViewLocation() + Direction * DistanceToCurrentLookAt );
		}
	}
}

/*namespace ViewportDeadZoneConstants
{
	enum
	{
		NO_DEAD_ZONE,
		STANDARD_DEAD_ZONE
	};
};*/

/** Trim the specified line to the planes of the frustum */
void TrimLineToFrustum(const FConvexVolume& Frustum, FVector& Start, FVector& End)
{
	FVector Intersection;
	for (const FPlane& Plane : Frustum.Planes)
	{
		if (FMath::SegmentPlaneIntersection(Start, End, Plane, Intersection))
		{
			// Chop the line inside the frustum
			if ((static_cast<const FVector&>(Plane) | (Intersection - End)) > 0.0f)
			{
				Start = Intersection;
			}
			else
			{
				End = Intersection;
			}
		}
	}
}

static void GetAttachedActorsRecursive(const AActor* InActor, TArray<AActor*>& OutActors)
{
	TArray<AActor*> AttachedActors;
	InActor->GetAttachedActors(AttachedActors);
	for (AActor* AttachedActor : AttachedActors)
	{
		GetAttachedActorsRecursive(AttachedActor, OutActors);
	}
	OutActors.Append(AttachedActors);
};

void FLevelEditorViewportClient::ProjectSelectedElementsIntoWorld(FViewport* InViewport, const FVector& Drag, const FRotator& Rot)
{
	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		InViewport, 
		GetScene(),
		EngineShowFlags)
		.SetRealtimeUpdate( IsRealtime() ));
	// SceneView is deleted with the ViewFamily
	FSceneView* SceneView = CalcSceneView( &ViewFamily );

	// Calculate the frustum so we can trim rays to it
	const FConvexVolume Frustum;
	GetViewFrustumBounds(const_cast<FConvexVolume&>(Frustum), SceneView->ViewMatrices.GetWorldToClip(), true);

	const FMatrix InputCoordSystem = GetWidgetCoordSystem();
	const EAxisList::Type CurrentAxis = GetCurrentWidgetAxis();
	
	const FVector DeltaTranslation = (ModeTools->PivotLocation - ModeTools->CachedLocation) + Drag;

	FTypedElementListConstRef ElementsToManipulate = GetElementsToManipulate();
	FCollisionQueryParams CollisionQueryParams;
	ElementsToManipulate->ForEachElement<ITypedElementWorldInterface>(
		[&CollisionQueryParams](const TTypedElement<ITypedElementWorldInterface>& InElement)
	{
		InElement.AddIgnoredElementToCollisionQueryParams(CollisionQueryParams, /*bAlsoIgnoreSubElements*/ true);
		return true; // continue iterating
	});

	// Loop over all the elements and attempt to snap them along the drag axis normal
	ElementsToManipulate->ForEachElement<ITypedElementWorldInterface>([&](const TTypedElement<ITypedElementWorldInterface>& InElement) -> bool
	{
		// Use the Delta of the Mode tool with the actor pre drag location to avoid accumulating snapping offsets
		FVector NewPosition;
		if (const FTransform* PreDragTransform = PreDragElementTransforms.Find(InElement))
		{
			NewPosition = PreDragTransform->GetLocation() + DeltaTranslation;
		}
		else
		{
			FTransform ElementTransform;
			ensure(InElement.GetWorldTransform(ElementTransform));
			PreDragElementTransforms.Add(InElement, ElementTransform);
			NewPosition = ElementTransform.GetLocation() + DeltaTranslation;
		}

		FViewportCursorLocation Cursor(SceneView, this, 0, 0);

		UE::Positioning::FObjectPositioningTraceResult TraceResult;
		bool bSnapped = false;

		bool bIsOnScreen = false;
		{
			// We only snap things that are on screen
			FVector2D ScreenPos;
			FIntPoint ViewportSize = InViewport->GetSizeXY();
			if (SceneView->WorldToPixel(NewPosition, ScreenPos) && FMath::IsWithin(ScreenPos.X, 0, ViewportSize.X) && FMath::IsWithin(ScreenPos.Y, 0, ViewportSize.Y))
			{
				bIsOnScreen = true;
				Cursor = FViewportCursorLocation(SceneView, this, static_cast<int32>(ScreenPos.X), static_cast<int32>(ScreenPos.Y));
			}
		}

		if (bIsOnScreen)
		{
			// Determine how we're going to attempt to project the object onto the world
			if (CurrentAxis == EAxisList::XY || CurrentAxis == EAxisList::XZ || CurrentAxis == EAxisList::YZ)
			{
				// Snap along the perpendicular axis
				const FVector PlaneNormal = CurrentAxis == EAxisList::XY ? FVector(0, 0, 1) : CurrentAxis == EAxisList::XZ ? FVector(0, 1, 0) : FVector(1, 0, 0);
				FVector TraceDirection = InputCoordSystem.TransformVector(PlaneNormal);

				// Make sure the trace normal points along the view direction
				if (FVector::DotProduct(SceneView->GetViewDirection(), TraceDirection) < 0.f)
				{
					TraceDirection = -TraceDirection;
				}

				FVector RayStart	= NewPosition - (TraceDirection * HALF_WORLD_MAX/2);
				FVector RayEnd		= NewPosition + (TraceDirection * HALF_WORLD_MAX/2);

				TrimLineToFrustum(Frustum, RayStart, RayEnd);

				TraceResult = UE::Positioning::TraceWorldForPosition(*GetWorld(), *SceneView, RayStart, RayEnd, &CollisionQueryParams);
			}
			else
			{
				TraceResult = UE::Positioning::TraceWorldForPosition(Cursor, *SceneView, &CollisionQueryParams);
			}
		}
				
		if (TraceResult.State == UE::Positioning::FObjectPositioningTraceResult::HitSuccess)
		{
			// Move the element to the position of the trace hit using the spawn offset rules
			// We only do this if we found a valid hit (we don't want to move the element in front of the camera by default)

			// Actors have a few snapping options set for them in their actor factory.
			const UActorFactory* ActorFactory = nullptr;
			AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElement);
			if (Actor)
			{
				ActorFactory = GEditor->FindActorFactoryForActorClass(Actor->GetClass());
			}
			
			const FTransform* PreDragTransform = PreDragElementTransforms.Find(InElement);
			if (!ensure(PreDragTransform))
			{
				return true; // continue to next element
			}

			// Compute the surface aligned transform. Note we do not use the snapped version here as our DragDelta is already snapped

			const FPositioningData PositioningData = FPositioningData(TraceResult.Location, TraceResult.SurfaceNormal)
				.UseStartTransform(*PreDragTransform)
				.UsePlacementExtent(Actor ? Actor->GetPlacementExtent() : FVector::ZeroVector)
				// It is ok for this to be null if we're not dealing with non-actors
				.UseFactory(ActorFactory);
			FTransform NewTransform = FActorPositioning::GetSurfaceAlignedTransform(PositioningData);

			FTransform CurrentTransform;
			InElement.GetWorldTransform(CurrentTransform);

			NewTransform.SetScale3D(CurrentTransform.GetScale3D());

			InElement.SetWorldTransform(NewTransform);
			if (Actor)
			{
				Actor->PostEditMove(/*bFinished*/ false);
			}
		}
		else
		{
			// Didn't find a valid surface snapping candidate, just apply the deltas directly
			FTransform CurrentTransform;
			InElement.GetWorldTransform(CurrentTransform);
			ApplyDeltaToElement(InElement, FTransform(Rot, NewPosition - CurrentTransform.GetLocation(), FVector(0.f, 0.f, 0.f)));
		}
		return true; // continue iterating
	});
}

bool FLevelEditorViewportClient::InputWidgetDelta(FViewport* InViewport, EAxisList::Type InCurrentAxis, FVector& Drag, FRotator& Rot, FVector& Scale)
{
	if (GUnrealEd->ComponentVisManager.HandleInputDelta(this, InViewport, Drag, Rot, Scale))
	{
		return true;
	}

	bool bHandled = false;

	// Give the current editor mode a chance to use the input first.  If it does, don't apply it to anything else.
	if (FEditorViewportClient::InputWidgetDelta(InViewport, InCurrentAxis, Drag, Rot, Scale))
	{
		bHandled = true;
	}
	else
	{
		//@TODO: MODETOOLS: Much of this needs to get pushed to Super, but not all of it can be...
		if( InCurrentAxis != EAxisList::None )
		{
			// Skip actors transformation routine in case if any of the selected actors locked
			// but still pretend that we have handled the input
			if (!GEditor->HasSelectedMovementLockedItems())
			{
				const bool LeftMouseButtonDown = InViewport->KeyState(EKeys::LeftMouseButton);
				const bool RightMouseButtonDown = InViewport->KeyState(EKeys::RightMouseButton);
				const bool MiddleMouseButtonDown = InViewport->KeyState(EKeys::MiddleMouseButton);

				const bool bUsesViewportITF = ModeTools->UsesViewportITF();

				// If duplicate dragging and using legacy viewport interaction behavior 
				if (IsAltPressed() && (LeftMouseButtonDown || RightMouseButtonDown) && !bUsesViewportITF)
				{
					// The widget has been offset, so check if we should duplicate the selection.
					if ( bDuplicateOnNextDrag )
					{
						// Only duplicate if we're translating or rotating.
						if ( !Drag.IsNearlyZero() || !Rot.IsZero() )
						{
							bDuplicateOnNextDrag = false;

							constexpr bool bSelectNewElements = true;
							DuplicateSelection(bSelectNewElements);
						}
					}
				}

				// We do not want actors updated if we are holding down the middle mouse button.
				// enable MMB for New TRS Gizmos
				const bool bModifyTransform = [this, MiddleMouseButtonDown]() -> bool
				{
					if (!UEditorInteractiveGizmoManager::AreEditorGizmosAllowed(ModeTools.Get()))
					{
						return !MiddleMouseButtonDown;
					}
					
					if (const UEditorTransformGizmoContextObject* GizmoContext = ModeTools->GetGizmoContext())
					{
						if (GizmoContext->bIsManipulatingPivot)
						{
							return false; 
						}
					}
					
					// Falling back to this value is here because that was the logic previously.
					// This value is only ever true when the legacy gizmos are dragged.
					return !bDraggingByHandle;
				}();
				
				if(bModifyTransform)
				{
					bool bSnapped = false;
					// Only perform snapping for legacy gizmos
					if (!bUsesViewportITF)
					{
						bSnapped = FSnappingUtils::SnapActorsToNearestActor( Drag, this );
					}

					bSnapped = bSnapped || FSnappingUtils::SnapDraggedActorsToNearestVertex( Drag, this );

					// If we are only changing position, project the actors onto the world
					const bool bOnlyTranslation = !Drag.IsZero() && Rot.IsZero() && Scale.IsZero();

					const EAxisList::Type CurrentAxis = GetCurrentWidgetAxis();
					const bool bSingleAxisDrag = CurrentAxis == EAxisList::X || CurrentAxis == EAxisList::Y || CurrentAxis == EAxisList::Z;
					if (!bSnapped && !bSingleAxisDrag && GetDefault<ULevelEditorViewportSettings>()->SnapToSurface.bEnabled && bOnlyTranslation)
					{
						ProjectSelectedElementsIntoWorld(InViewport, Drag, Rot);
					}
					else
					{
						ApplyDeltaToSelectedElements(FTransform(Rot, Drag, Scale));
					}

					ApplyDeltaToRotateWidget( Rot );

					bCurrentlyEditingThroughMovementWidget = true;
				}
				else
				{
					FSnappingUtils::SnapDragLocationToNearestVertex( ModeTools->PivotLocation, Drag, this, true );
					GUnrealEd->SetPivotMovedIndependently(true);
					bOnlyMovedPivot = true;
				}

				ModeTools->PivotLocation += Drag;
				ModeTools->SnappedLocation += Drag;

				if (IsShiftPressed() && !ModeTools->UsesViewportITF())
				{
					bApplyCameraSpeedScaleByDistance = false;
					FVector CameraDelta( Drag );
					MoveViewportCamera( CameraDelta, FRotator::ZeroRotator );
					bApplyCameraSpeedScaleByDistance = true;
				}

				ModeTools->UpdateInternalData();
			}

			bHandled = true;
		}

	}

	return bHandled;
}

bool FLevelEditorViewportClient::ShouldScaleCameraSpeedByDistance() const
{
	return bApplyCameraSpeedScaleByDistance && FEditorViewportClient::ShouldScaleCameraSpeedByDistance();
}

TSharedPtr<FDragTool> FLevelEditorViewportClient::MakeDragTool( EDragTool::Type DragToolType )
{
	// Let the drag tool handle the transaction
	TrackingTransaction.Cancel();

	TSharedPtr<FDragTool> DragTool;
	switch( DragToolType )
	{
	case EDragTool::BoxSelect:
		DragTool = MakeShareable( new FDragTool_ActorBoxSelect(this) );
		break;
	case EDragTool::FrustumSelect:
		DragTool = MakeShareable( new FDragTool_ActorFrustumSelect(this) );
		break;	
	case EDragTool::Measure:
		DragTool = MakeShareable( new FDragTool_Measure(this) );
		break;
	case EDragTool::ViewportChange:
		DragTool = MakeShareable( new FDragTool_ViewportChange(this) );
		break;
	};

	return DragTool;
}

static const FLevelViewportCommands& GetLevelViewportCommands()
{
	static FName LevelEditorName("LevelEditor");
	FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>( LevelEditorName );
	return LevelEditor.GetLevelViewportCommands();
}

void FLevelEditorViewportClient::SetCurrentViewport()
{
	// Set the current level editing viewport client to the dropped-in viewport client
	if (GCurrentLevelEditingViewportClient != this)
	{
		// Invalidate the old vp client to remove its special selection box
		if (GCurrentLevelEditingViewportClient)
		{
			GCurrentLevelEditingViewportClient->Invalidate();
		}
		GCurrentLevelEditingViewportClient = this;
	}
	Invalidate();
}

void FLevelEditorViewportClient::SetLastKeyViewport()
{
	// Store a reference to the last viewport that received a key press.
	GLastKeyLevelEditingViewportClient = this;

	if (GCurrentLevelEditingViewportClient != this)
	{
		if (GCurrentLevelEditingViewportClient)
		{
			//redraw without yellow selection box
			GCurrentLevelEditingViewportClient->Invalidate();
		}
		//cause this viewport to redraw WITH yellow selection box
		Invalidate();
		GCurrentLevelEditingViewportClient = this;
	}
}

bool FLevelEditorViewportClient::InputKey(const FInputKeyEventArgs& InEventArgs)
{
	if (bDisableInput)
	{
		return true;
	}


	const int32	HitX = InEventArgs.Viewport->GetMouseX();
	const int32	HitY = InEventArgs.Viewport->GetMouseY();

	FInputEventState InputState( InEventArgs.Viewport, InEventArgs.Key, InEventArgs.Event );

	SetLastKeyViewport();

	// Compute a view.
	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		InEventArgs.Viewport,
		GetScene(),
		EngineShowFlags )
		.SetRealtimeUpdate( IsRealtime() ) );
	FSceneView* View = CalcSceneView( &ViewFamily );

	// Compute the click location.
	if ( InputState.IsMouseButtonEvent() && InputState.IsAnyMouseButtonDown() )
	{
		// This Viewport Client received a click press, let's retrieve the currently focused viewport client.
		// If that's not This viewport client, let's make sure This is focused, instead of the currently focused one.
		FEditorViewportClient* CurrentlyFocusedViewport = ModeTools->GetFocusedViewportClient();
		if (CurrentlyFocusedViewport != this)
		{
			if (CurrentlyFocusedViewport)
			{
				// Call LostFocus on the Viewport Client losing focus
				LostFocus(CurrentlyFocusedViewport->Viewport);
			}

			// Focusing on This viewport
			ReceivedFocus(Viewport);
		}

		const FViewportCursorLocation Cursor(View, this, HitX, HitY);
		const FActorPositionTraceResult TraceResult = FActorPositioning::TraceWorldForPositionWithDefault(Cursor, *View);
		GEditor->UnsnappedClickLocation = TraceResult.Location;
		GEditor->ClickLocation = TraceResult.Location;
		GEditor->ClickPlane = FPlane(TraceResult.Location, TraceResult.SurfaceNormal);

		// Snap the new location if snapping is enabled
		FSnappingUtils::SnapPointToGrid(GEditor->ClickLocation, FVector::ZeroVector);
	}

	if (GUnrealEd->ComponentVisManager.HandleInputKey(this, InEventArgs.Viewport, InEventArgs.Key, InEventArgs.Event))
	{
		return true;
	}

	UWorld* ViewportWorld = GetWorld();
	auto ProcessAtmosphericLightShortcut = [&](const uint8 LightIndex, bool& bCurrentUserControl)
	{
		UDirectionalLightComponent* SelectedSunLight = GetAtmosphericLight(LightIndex, ViewportWorld);
		if (SelectedSunLight)
		{
			if (!bCurrentUserControl)
			{
				FText TrackingDescription = FText::Format(LOCTEXT("RotatationShortcut", "Rotate Atmosphere Light {0}"), LightIndex);
				TrackingTransaction.Begin(TrackingDescription, SelectedSunLight->GetOwner());
				AddRealtimeOverride(true, LOCTEXT("RealtimeOverrideMessage_AtmospherelLight", "Atmosphere Light Control"));// The first time, save that setting for RestoreRealtime
			}
			bCurrentUserControl = true;
			UserIsControllingAtmosphericLightTimer = 3.0f; // Keep the widget open for a few seconds even when not tweaking the sun light
		}
	};


	bool bCmdCtrlLPressed = (InputState.IsCommandButtonPressed() || InputState.IsCtrlButtonPressed()) && InEventArgs.Key == EKeys::L;
	if (bCmdCtrlLPressed && InputState.IsShiftButtonPressed())
	{
		ProcessAtmosphericLightShortcut(1, bUserIsControllingAtmosphericLight1);
		return true;
	}
	if (bCmdCtrlLPressed)
	{
		ProcessAtmosphericLightShortcut(0, bUserIsControllingAtmosphericLight0);
		return true;
	}
	if (bUserIsControllingAtmosphericLight0 || bUserIsControllingAtmosphericLight1)
	{
		UDirectionalLightComponent* SelectedSunLight = GetAtmosphericLight(bUserIsControllingAtmosphericLight0 ? 0 : 1, ViewportWorld);
		if (SelectedSunLight)
		{
			NotifyAtmosphericLightHasMoved(*SelectedSunLight, true);
		}
		TrackingTransaction.End();					// End undo/redo translation
		RemoveRealtimeOverride(LOCTEXT("RealtimeOverrideMessage_AtmospherelLight", "Atmosphere Light Control"));	// Restore previous real-time state
	}
	bUserIsControllingAtmosphericLight0 = false;	// Disable all atmospheric light controls
	bUserIsControllingAtmosphericLight1 = false;

	UEditorWorldExtensionCollection& EditorWorldExtensionCollection = *GEditor->GetEditorWorldExtensionsManager()->GetEditorWorldExtensions(GetWorld());
	if (EditorWorldExtensionCollection.InputKey(this, Viewport, InEventArgs.Key, InEventArgs.Event))
	{
		return true;
	}

	bool bHandled = FEditorViewportClient::InputKey(InEventArgs);

	// Handle input for the height preview mode. 
	bool bEnablePreviewMeshChordPressed = IsCommandChordPressed(GetLevelViewportCommands().EnablePreviewMesh);
	bool bCyclePreviewMeshChordPressed = IsCommandChordPressed(GetLevelViewportCommands().CyclePreviewMesh);
	if (!InputState.IsMouseButtonEvent() && (bEnablePreviewMeshChordPressed || bCyclePreviewMeshChordPressed))
	{
		GEditor->SetPreviewMeshMode(true);

		if (bCyclePreviewMeshChordPressed && (InEventArgs.Event == IE_Pressed))
		{
			GEditor->CyclePreviewMesh();
		}

		bHandled = true;
	}
	else
	{
		GEditor->SetPreviewMeshMode(false);
	}

	const bool bAltIsBeingPressed = (InEventArgs.Event == IE_Pressed) && (InEventArgs.Key == EKeys::LeftAlt || InEventArgs.Key == EKeys::RightAlt);

	// Clear Duplicate Actors mode when mouse buttons are released
	if ( !InputState.IsAnyMouseButtonDown() )
	{
		bDuplicateActorsInProgress = false;
	}
	// Stamp a duplicate at the current position during a drag when ALT is pressed and we're dragging with a transaction open, and there is no component visualizer
	else if ( bAltIsBeingPressed && bDraggingByHandle && TrackingTransaction.IsActive() && GUnrealEd->ComponentVisManager.GetActiveComponentVis() == nullptr )
	{
		constexpr bool bSelectNewElements = false;
		DuplicateSelection(bSelectNewElements);
	}

	
	return bHandled;
}

void SetActorBeingMovedByEditor(AActor* Actor, bool bIsBeingMoved)
{
	TInlineComponentArray<UPrimitiveComponent*> Components;
	Actor->GetComponents(Components);

	for (UPrimitiveComponent* PrimitiveComponent : Components)
	{
		PrimitiveComponent->SetIsBeingMovedByEditor(bIsBeingMoved);
	}
}

void FLevelEditorViewportClient::TrackingStarted( const FInputEventState& InInputState, bool bIsDraggingWidget, bool bNudge )
{
	// Begin transacting.  Give the current editor mode an opportunity to do the transacting.
	const bool bTrackingHandledExternally = ModeTools->StartTracking(this, Viewport);

	TrackingTransaction.End();

	// Re-initialize new tracking only if a new button was pressed, otherwise we continue the previous one.
	if ( InInputState.GetInputEvent() == IE_Pressed )
	{
		EInputEvent Event = InInputState.GetInputEvent();
		FKey Key = InInputState.GetKey();

		if ( InInputState.IsAltButtonPressed() && bDraggingByHandle )
		{
			if(Event == IE_Pressed && (Key == EKeys::LeftMouseButton || Key == EKeys::RightMouseButton) && !bDuplicateActorsInProgress)
			{
				// Set the flag so that the actors actors will be duplicated as soon as the widget is displaced.
				bDuplicateOnNextDrag = true;
				bDuplicateActorsInProgress = true;
			}
		}
		else
		{
			bDuplicateOnNextDrag = false;
		}
	}

	bOnlyMovedPivot = false;
	bNeedToRestoreComponentBeingMovedFlag = false;
	bHasBegunGizmoManipulation = false;

	PreDragElementTransforms.Empty();

	// Track BSP changes
	{
		bIsTrackingBrushModification = false;

		FTypedElementListConstRef ElementsToManipulate = GetElementsToManipulate();
		TypedElementListObjectUtil::ForEachObject<ABrush>(ElementsToManipulate, [this](ABrush* InBrush)
		{
			bIsTrackingBrushModification = InBrush && InBrush->CanEverAffectBSP();
			return !bIsTrackingBrushModification;
		});
	}

	if (bIsDraggingWidget)
	{
		Widget->SetSnapEnabled(true);

		FTypedElementListConstRef ElementsToManipulate = GetElementsToManipulate();
		ViewportInteraction->BeginGizmoManipulation(ElementsToManipulate, GetWidgetMode());
		bHasBegunGizmoManipulation = true;

		if (!bDuplicateActorsInProgress)
		{
			bNeedToRestoreComponentBeingMovedFlag = true;

			TypedElementListObjectUtil::ForEachObject<AActor>(ElementsToManipulate, [this](AActor* InActor)
			{
				SetActorBeingMovedByEditor(InActor, true);
				return true;
			});
		}
	}

	// Start a transformation transaction if required
	if( !bTrackingHandledExternally )
	{
		if( bIsDraggingWidget )
		{
			TrackingTransaction.TransCount++;

			FText TrackingDescription;
			switch( GetWidgetMode() )
			{
			case UE::Widget::WM_Translate:
				TrackingDescription = LOCTEXT("MoveTransaction", "Move Elements");
				break;
			case UE::Widget::WM_Rotate:
				TrackingDescription = LOCTEXT("RotateTransaction", "Rotate Elements");
				break;
			case UE::Widget::WM_Scale:
				TrackingDescription = LOCTEXT("ScaleTransaction", "Scale Elements");
				break;
			case UE::Widget::WM_TranslateRotateZ:
				TrackingDescription = LOCTEXT("TranslateRotateZTransaction", "Translate/RotateZ Elements");
				break;
			case UE::Widget::WM_2D:
				TrackingDescription = LOCTEXT("TranslateRotate2D", "Translate/Rotate2D Elements");
				break;
			default:
				if( bNudge )
				{
					TrackingDescription = LOCTEXT("NudgeTransaction", "Nudge Elements");
				}
			}

			if(!TrackingDescription.IsEmpty())
			{
				if(bNudge)
				{
					TrackingTransaction.Begin(TrackingDescription);
				}
				else
				{
					// If this hasn't begun due to a nudge, start it as a pending transaction so that it only really begins when the mouse is moved
					TrackingTransaction.BeginPending(TrackingDescription);
				}
			}
		}
		else
		{
			AActor* ActiveActorLock = GetActiveActorLock().Get();
			if (ActiveActorLock && !ActiveActorLock->IsLockLocation() && TrackingTransaction.TransCount == 0 && !CachedPilotTransform.IsSet())
			{
				// Cache Pilot Actor transform before piloting ends
				CachedPilotTransform = ActiveActorLock->GetTransform();
			}
		}

		if (TrackingTransaction.IsActive() || TrackingTransaction.IsPending())
		{
			// Suspend actor/component modification during each delta step to avoid recording unnecessary overhead into the transaction buffer
			GEditor->DisableDeltaModification(true);
		}

		GUnrealEd->ComponentVisManager.TrackingStarted(this);
	}
}

void FLevelEditorViewportClient::TrackingStopped()
{
	// Only disable the duplicate on next drag flag if we actually dragged the mouse.
	bDuplicateOnNextDrag = false;

	// here we check to see if anything of worth actually changed when ending our MouseMovement
	// If the TransCount > 0 (we changed something of value) so we need to call PostEditMove() on stuff
	// if we didn't change anything then don't call PostEditMove()
	bool bDidAnythingActuallyChange = false;

	// Stop transacting.  Give the current editor mode an opportunity to do the transacting.
	const bool bTransactingHandledByEditorMode = ModeTools->EndTracking(this, Viewport);
	if( !bTransactingHandledByEditorMode )
	{
		if( TrackingTransaction.TransCount > 0 )
		{
			bDidAnythingActuallyChange = true;
			TrackingTransaction.TransCount--;
		}
	}

	// Finish tracking a brush transform and update the Bsp
	if (bIsTrackingBrushModification)
	{
		bDidAnythingActuallyChange = HaveSelectedObjectsBeenChanged() && !bOnlyMovedPivot;

		bIsTrackingBrushModification = false;
		if ( bDidAnythingActuallyChange && bWidgetAxisControlledByDrag )
		{
			// Note that while the brush is now rebuilt in FActorElementLevelEditorViewportInteractionCustomization::GizmoManipulationStopped,
			// we still call it here as the call above has different conditions to this one. There's a re-entrant guard in RebuildAlteredBSP to prevent a second rebuild. 
			GEditor->RebuildAlteredBSP();
		}
	}

	// Notify the selected actors that they have been moved.
	// Don't do this if AddDelta was never called.
	{
		const bool bDidMove = bDidAnythingActuallyChange && MouseDeltaTracker->HasReceivedDelta();

		if (bHasBegunGizmoManipulation)
		{
			FTypedElementListConstRef ElementsToManipulate = GetElementsToManipulate();
			ViewportInteraction->EndGizmoManipulation(ElementsToManipulate, GetWidgetMode(), bDidMove ? ETypedElementViewportInteractionGizmoManipulationType::Drag : ETypedElementViewportInteractionGizmoManipulationType::Click);
			bHasBegunGizmoManipulation = false;
		}

		if (bDidMove && !GUnrealEd->IsPivotMovedIndependently())
		{
			GUnrealEd->UpdatePivotLocationForSelection();
		}

		GUnrealEd->ComponentVisManager.TrackingStopped(this, bDidMove);
	}

	if (bNeedToRestoreComponentBeingMovedFlag)
	{
		FTypedElementListConstRef ElementsToManipulate = GetElementsToManipulate();
		TypedElementListObjectUtil::ForEachObject<AActor>(ElementsToManipulate, [this](AActor* InActor)
		{
			SetActorBeingMovedByEditor(InActor, false);
			return true;
		});

		bNeedToRestoreComponentBeingMovedFlag = false;
	}

	// End the transaction here if one was started in StartTransaction()
	if( TrackingTransaction.IsActive() || TrackingTransaction.IsPending() )
	{
		if( !HaveSelectedObjectsBeenChanged())
		{
			TrackingTransaction.Cancel();
		}
		else
		{
			TrackingTransaction.End();
		}
		
		// Restore actor/component delta modification
		GEditor->DisableDeltaModification(false);
	}

	ModeTools->ActorMoveNotify();

	if( bDidAnythingActuallyChange )
	{
		FScopedLevelDirtied LevelDirtyCallback;
		LevelDirtyCallback.Request();

		RedrawAllViewportsIntoThisScene();
	}

	PreDragElementTransforms.Empty();
}

void FLevelEditorViewportClient::AbortTracking()
{
	if (TrackingTransaction.IsActive())
	{
		// Applying the global undo here will reset the drag operation
		if (GUndo)
		{
			GUndo->Apply();
		}
		TrackingTransaction.Cancel();
		StopTracking();
	}
}

void FLevelEditorViewportClient::HandleViewportSettingChanged(FName PropertyName)
{
	if (PropertyName == GET_MEMBER_NAME_CHECKED(ULevelEditorViewportSettings, bUseSelectionOutline))
	{
		EngineShowFlags.SetSelectionOutline(GetDefault<ULevelEditorViewportSettings>()->bUseSelectionOutline);
	}
}

void FLevelEditorViewportClient::OnMapChanged(UWorld* InWorld, EMapChangeType MapChangeType)
{
	if (InWorld != GetWorld())
	{
		return;
	}

	bDisableInput = (MapChangeType == EMapChangeType::TearDownWorld);

	if (MapChangeType == EMapChangeType::TearDownWorld)
	{
		// Clear pilot state before the world's actors are destroyed. Reset
		// CachedPilotTransform first so SetActorLock won't attempt transform
		// restoration or open an undo transaction against a world being torn down.
		CachedPilotTransform.Reset();
		SetCinematicActorLock(nullptr);
		SetActorLock(nullptr);
	}
}

void FLevelEditorViewportClient::OnActorMoved(AActor* InActor)
{
	// Update the cameras from their locked actor (if any)
	UpdateLockedActorViewport(InActor, false);
}

void FLevelEditorViewportClient::NudgeSelectedObjects( const struct FInputEventState& InputState )
{
	FViewport* InViewport = InputState.GetViewport();
	EInputEvent Event = InputState.GetInputEvent();
	FKey Key = InputState.GetKey();

	const int32 MouseX = InViewport->GetMouseX();
	const int32 MouseY = InViewport->GetMouseY();

	if( Event == IE_Pressed || Event == IE_Repeat )
	{
		// If this is a pressed event, start tracking.
		if ( !bIsTracking && Event == IE_Pressed )
		{
			// without the check for !bIsTracking, the following code would cause a new transaction to be created
			// for each "nudge" that occurred while the key was held down.  Disabling this code prevents the transaction
			// from being constantly recreated while as long as the key is held, so that the entire move is considered an atomic action (and
			// doing undo reverts the entire movement, as opposed to just the last nudge that occurred while the key was held down)
			MouseDeltaTracker->StartTracking( this, MouseX, MouseY, InputState, true );
			bIsTracking = true;
		}

		FIntPoint StartMousePos;
		InViewport->GetMousePos( StartMousePos );
		FKey VirtualKey = EKeys::MouseX;
		EAxisList::Type VirtualAxis = GetHorizAxis();
		float VirtualDelta = GEditor->GetGridSize() * (Key == EKeys::Left?-1:1);
		if( Key == EKeys::Up || Key == EKeys::Down )
		{
			VirtualKey = EKeys::MouseY;
			VirtualAxis = GetVertAxis();
			VirtualDelta = GEditor->GetGridSize() * (Key == EKeys::Up?1:-1);
		}

		bWidgetAxisControlledByDrag = false;
		Widget->SetCurrentAxis( VirtualAxis );
		MouseDeltaTracker->AddDelta( this, VirtualKey, static_cast<int32>(VirtualDelta), 1 );
		Widget->SetCurrentAxis( VirtualAxis );
		UpdateMouseDelta();
		InViewport->SetMouse( StartMousePos.X , StartMousePos.Y );
	}
	else if( bIsTracking && Event == IE_Released )
	{
		bWidgetAxisControlledByDrag = false;
		MouseDeltaTracker->EndTracking( this );
		bIsTracking = false;
		Widget->SetCurrentAxis( EAxisList::None );
	}

	RedrawAllViewportsIntoThisScene();
}


/**
 * Returns the horizontal axis for this viewport.
 */

EAxisList::Type FLevelEditorViewportClient::GetHorizAxis() const
{
	switch( GetViewportType() )
	{
	case LVT_OrthoXY:
	case LVT_OrthoNegativeXY:
		return EAxisList::X;
	case LVT_OrthoXZ:
	case LVT_OrthoNegativeXZ:
		return EAxisList::X;
	case LVT_OrthoYZ:
	case LVT_OrthoNegativeYZ:
		return EAxisList::Y;
	case LVT_OrthoFreelook:
	case LVT_Perspective:
		break;
	}

	return EAxisList::X;
}

/**
 * Returns the vertical axis for this viewport.
 */

EAxisList::Type FLevelEditorViewportClient::GetVertAxis() const
{
	switch( GetViewportType() )
	{
	case LVT_OrthoXY:
	case LVT_OrthoNegativeXY:
		return EAxisList::Y;
	case LVT_OrthoXZ:
	case LVT_OrthoNegativeXZ:
		return EAxisList::Z;
	case LVT_OrthoYZ:
	case LVT_OrthoNegativeYZ:
		return EAxisList::Z;
	case LVT_OrthoFreelook:
	case LVT_Perspective:
		break;
	}

	return EAxisList::Y;
}

//
//	FLevelEditorViewportClient::InputAxis
//

/**
 * Sets the current level editing viewport client when created and stores the previous one
 * When destroyed it sets the current viewport client back to the previous one.
 */
struct FScopedSetCurrentViewportClient
{
	FScopedSetCurrentViewportClient( FLevelEditorViewportClient* NewCurrentViewport )
	{
		PrevCurrentLevelEditingViewportClient = GCurrentLevelEditingViewportClient;
		GCurrentLevelEditingViewportClient = NewCurrentViewport;
	}
	~FScopedSetCurrentViewportClient()
	{
		GCurrentLevelEditingViewportClient = PrevCurrentLevelEditingViewportClient;
	}
private:
	FLevelEditorViewportClient* PrevCurrentLevelEditingViewportClient;
};

bool FLevelEditorViewportClient::InputAxis(const FInputKeyEventArgs& Args)
{
	if (bDisableInput)
	{
		return true;
	}

	// Forward input axis events to the Editor World extension collection and give extensions an opportunity to consume input.
	IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
	FPlatformUserId UserId = DeviceMapper.GetUserForInputDevice(Args.InputDevice);

	int32 ControllerId;
	if (DeviceMapper.RemapUserAndDeviceToControllerId(UserId, ControllerId, Args.InputDevice))
	{
		UEditorWorldExtensionCollection& ExtensionCollection = *GEditor->GetEditorWorldExtensionsManager()->GetEditorWorldExtensions(GetWorld());
		
		if (ExtensionCollection.InputAxis(this, Args.Viewport, ControllerId, Args.Key, Args.AmountDepressed, Args.DeltaTime))
		{
			return true;
		}
	}

	// @todo Slate: GCurrentLevelEditingViewportClient is switched multiple times per frame and since we draw the border in slate this effectively causes border to always draw on the last viewport

	FScopedSetCurrentViewportClient( this );

	return FEditorViewportClient::InputAxis(Args);
}



static uint32 GetVolumeActorVisibilityId( const AActor& InActor )
{
	UClass* Class = InActor.GetClass();

	static TMap<UClass*, uint32 > ActorToIdMap;
	if( ActorToIdMap.Num() == 0 )
	{
		// Build a mapping of volume classes to ID's.  Do this only once
		TArray< UClass *> VolumeClasses;
		UUnrealEdEngine::GetSortedVolumeClasses(&VolumeClasses);
		for( int32 VolumeIdx = 0; VolumeIdx < VolumeClasses.Num(); ++VolumeIdx )
		{
			// An actors flag is just the index of the actor in the stored volume array shifted left to represent a unique bit.
			ActorToIdMap.Add( VolumeClasses[VolumeIdx], VolumeIdx );
		}
	}

	uint32* ActorID =  ActorToIdMap.Find( Class );

	// return 0 if the actor flag was not found, otherwise return the actual flag.  
	return ActorID ? *ActorID : 0;
}


/** 
 * Returns true if the passed in volume is visible in the viewport (due to volume actor visibility flags)
 *
 * @param Volume	The volume to check
 */
bool FLevelEditorViewportClient::IsVolumeVisibleInViewport( const AActor& VolumeActor ) const
{
	// We pass in the actor class for compatibility but we should make sure 
	// the function is only given volume actors
	//check( VolumeActor.IsA(AVolume::StaticClass()) );

	uint32 VolumeId = GetVolumeActorVisibilityId( VolumeActor );
	return VolumeActorVisibility[ VolumeId ];
}

void FLevelEditorViewportClient::RedrawAllViewportsIntoThisScene()
{
	// Invalidate all viewports, so the new gizmo is rendered in each one
	GEditor->RedrawLevelEditingViewports();
}

void FLevelEditorViewportClient::SetVREditView(bool bGameViewEnable)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UEditorWorldExtensionCollection* ExtensionCollection = GEditor->GetEditorWorldExtensionsManager()->GetEditorWorldExtensions(GetWorld());
	check(ExtensionCollection != nullptr);
	UVREditorMode* VREditorMode = Cast<UVREditorMode>(ExtensionCollection->FindExtension(UVREditorMode::StaticClass()));
	if (VREditorMode && VREditorMode->IsFullyInitialized())
	{
		if (bGameViewEnable)
		{
			VREditorMode->OnPlacePreviewActor().AddStatic(&FLevelEditorViewportClient::SetIsDroppingPreviewActor);
		}
		else
		{
			VREditorMode->OnPlacePreviewActor().RemoveAll(this);
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FEditorViewportClient::SetVREditView(bGameViewEnable);
}

UE::Widget::EWidgetMode FLevelEditorViewportClient::GetWidgetMode() const
{
	if (GUnrealEd->ComponentVisManager.IsActive() && GUnrealEd->ComponentVisManager.IsVisualizingArchetype())
	{
		return UE::Widget::WM_None;
	}

	return FEditorViewportClient::GetWidgetMode();
}

FVector FLevelEditorViewportClient::GetWidgetLocation() const
{
	FVector ComponentVisWidgetLocation;
	if (GUnrealEd->ComponentVisManager.GetWidgetLocation(this, ComponentVisWidgetLocation))
	{
		return ComponentVisWidgetLocation;
	}

	return FEditorViewportClient::GetWidgetLocation();
}

FMatrix FLevelEditorViewportClient::GetWidgetCoordSystem() const 
{
	FMatrix ComponentVisWidgetCoordSystem;
	if (GUnrealEd->ComponentVisManager.GetCustomInputCoordinateSystem(this, ComponentVisWidgetCoordSystem))
	{
		return ComponentVisWidgetCoordSystem;
	}

	return FEditorViewportClient::GetWidgetCoordSystem();
}

void FLevelEditorViewportClient::MoveLockedActorToCamera()
{
	// If turned on, move any selected actors to the cameras location/rotation
	AActor* ActiveActorLock = GetActiveActorLock().Get();
	if (ActiveActorLock)
	{
		if (!ActiveActorLock->IsLockLocation())
		{
			if (TrackingTransaction.TransCount > 0)
			{
				SnapshotTransactionBuffer(ActiveActorLock);

				USceneComponent* ActiveActorLockComponent = ActiveActorLock->GetRootComponent();
				if (ActiveActorLockComponent && !ActiveActorLockComponent->IsCreatedByConstructionScript())
				{
					SnapshotTransactionBuffer(ActiveActorLockComponent);
				}
			}

			// Need to disable orbit camera before setting actor position so that the viewport camera location is converted back
			ToggleOrbitCamera(false);

			USceneComponent* ActiveActorLockComponent = ActiveActorLock->GetRootComponent();
			TOptional<FRotator> PreviousRotator;
			if (ActiveActorLockComponent)
			{
				PreviousRotator = ActiveActorLockComponent->GetRelativeRotation();
			}

			// If we're locked to a camera then we're reflecting the camera view and not the actor position. We need to reflect that delta when we reposition the piloted actor
			if (bUseControllingActorViewInfo)
			{
				const UActorComponent* ViewComponent = FindViewComponentForActor(ActiveActorLock);
				if (const UCameraComponent* CameraViewComponent = Cast<UCameraComponent>(ViewComponent))
				{
					FTransform AdditiveOffset;
					float AdditiveFOV;
					CameraViewComponent->GetAdditiveOffset(AdditiveOffset, AdditiveFOV);
					const FTransform RelativeTransform = (AdditiveOffset * CameraViewComponent->GetComponentTransform()).Inverse();
					const FTransform DesiredTransform = FTransform(GetViewRotation(), GetViewLocation());
					ActiveActorLock->SetActorTransform(ActiveActorLock->GetActorTransform() * RelativeTransform * DesiredTransform);
				}
				else if (const USceneComponent* SceneViewComponent = Cast<USceneComponent>(ViewComponent))
				{
					const FTransform RelativeTransform = SceneViewComponent->GetComponentTransform().Inverse();
					const FTransform DesiredTransform = FTransform(GetViewRotation(), GetViewLocation());
					ActiveActorLock->SetActorTransform(ActiveActorLock->GetActorTransform() * RelativeTransform * DesiredTransform);
				}
			}
			else
			{
				ActiveActorLock->SetActorLocation(GetViewLocation(), false);
				ActiveActorLock->SetActorRotation(GetViewRotation());
			}

			// If Selected actor is the piloted one, have the pivot follow it.
			if (USelection* const SelectedActors = GEditor->GetSelectedActors())
			{
				if (ActiveActorLock == SelectedActors->GetBottom<AActor>())
				{
					GEditor->SetPivot(ActiveActorLock->GetActorLocation(), false, true);
				}
			}

			if (ActiveActorLockComponent)
			{
				const FRotator Rot = PreviousRotator.GetValue();
				FRotator ActorRotWind, ActorRotRem;
				Rot.GetWindingAndRemainder(ActorRotWind, ActorRotRem);
				const FQuat ActorQ = ActorRotRem.Quaternion();
				const FQuat ResultQ = ActiveActorLockComponent->GetRelativeRotation().Quaternion();
				FRotator NewActorRotRem = FRotator(ResultQ);
				ActorRotRem.SetClosestToMe(NewActorRotRem);
				FRotator DeltaRot = NewActorRotRem - ActorRotRem;
				DeltaRot.Normalize();
				ActiveActorLockComponent->SetRelativeRotationExact(Rot + DeltaRot);
			}
		}

		if (ABrush* Brush = Cast<ABrush>(ActiveActorLock))
		{
			Brush->SetNeedRebuild(Brush->GetLevel());
		}

		FScopedLevelDirtied LevelDirtyCallback;
		LevelDirtyCallback.Request();

		// Skip hit proxy invalidation during mouse-look rotation since the user is not clicking on anything
		const bool bInvalidateHitProxies = !(ViewportNavigationHelper && ViewportNavigationHelper->bIsMouseLooking);
		GEditor->RedrawLevelEditingViewports(bInvalidateHitProxies);
	}
}

bool FLevelEditorViewportClient::HaveSelectedObjectsBeenChanged() const
{
	return (TrackingTransaction.TransCount > 0 || TrackingTransaction.IsActive()) && (MouseDeltaTracker->HasReceivedDelta() || MouseDeltaTracker->WasExternalMovement());
}

FTypedElementListConstRef FLevelEditorViewportClient::GetElementsToManipulate(const bool bForceRefresh)
{ 
	CacheElementsToManipulate(bForceRefresh);
	return CachedElementsToManipulate;
}

void FLevelEditorViewportClient::CacheElementsToManipulate(const bool bForceRefresh)
{
	if (bForceRefresh)
	{
		ResetElementsToManipulate();
	}

	if (!bHasCachedElementsToManipulate)
	{
		CachedElementsToManipulate = UEditorElementSubsystem::GetEditorNormalizedSelectionSet(*GetSelectionSet());
		UEditorElementSubsystem::GetEditorManipulableElements(CachedElementsToManipulate, GetWidgetMode(), bIsSimulateInEditorViewport ? GEditor->PlayWorld : GEditor->EditorWorld);

		bHasCachedElementsToManipulate = true;
	}
}

void FLevelEditorViewportClient::ResetElementsToManipulate(const bool bClearList)
{
	if (bClearList)
	{
		CachedElementsToManipulate->Reset();
	}
	bHasCachedElementsToManipulate = false;
}

void FLevelEditorViewportClient::ResetElementsToManipulateFromSelectionChange(const UTypedElementSelectionSet* InSelectionSet)
{
	check(InSelectionSet == GetSelectionSet());

	// Don't clear the list immediately, as the selection may change from a construction script running (while we're still iterating the list!)
	// We'll process the clear on the next cache request, or when the typed element registry actually processes its pending deletion
	ResetElementsToManipulate(/*bClearList*/false);
}

void FLevelEditorViewportClient::ResetElementsToManipulateFromProcessingDeferredElementsToDestroy()
{
	if (!bHasCachedElementsToManipulate)
	{
		// If we have no cache, make sure the cached list is definitely empty now to ensure it doesn't contain any lingering references to things that are about to be deleted
		CachedElementsToManipulate->Reset();
	}
}

const UTypedElementSelectionSet* FLevelEditorViewportClient::GetSelectionSet() const
{
	TSharedPtr<ILevelEditor> LevelEditor = ParentLevelEditor.Pin();
	return LevelEditor ? LevelEditor->GetElementSelectionSet() : GEditor->GetSelectedActors()->GetElementSelectionSet();
}

UTypedElementSelectionSet* FLevelEditorViewportClient::GetMutableSelectionSet() const
{
	TSharedPtr<ILevelEditor> LevelEditor = ParentLevelEditor.Pin();
	return LevelEditor ? LevelEditor->GetMutableElementSelectionSet() : GEditor->GetSelectedActors()->GetElementSelectionSet();
}

void FLevelEditorViewportClient::MoveCameraToLockedActor(const bool bInvalidateHitProxies)
{
	// Do a slightly shorter version of the code in UpdateViewForLockedActor where we don't care about
	// post-process settings.
	const FLevelViewportActorLock& ActiveLock = ActorLocks.GetLock();
	AActor* Actor = ActiveLock.GetLockedActor();
	if (Actor)
	{
		bool bGotCameraView = false;
		if (bLockedCameraView)
		{
			// If this is a camera actor, then inherit some other settings
			UActorComponent* const ViewComponent = FindViewComponentForActor(Actor);
			if (ViewComponent != nullptr)
			{
				FMinimalViewInfo ViewInfo;
				if (ensure(ViewComponent->GetEditorPreviewInfo(0.f, ViewInfo)))
				{
					ViewFOV = ViewInfo.FOV;
					AspectRatio = ViewInfo.AspectRatio;
					SetViewLocation(ViewInfo.Location);
					SetViewRotation(ViewInfo.Rotation);
					bGotCameraView = true;
				}
			}
		}
		if (!bGotCameraView)
		{
			if (Actor->GetAttachParentActor() != NULL)
			{
				// Actor is parented, so use the actor to world matrix for translation and rotation information.
				SetViewLocation(Actor->GetActorLocation());
				SetViewRotation(Actor->GetActorRotation());
			}
			else if (Actor->GetRootComponent() != NULL)
			{
				// No attachment, so just use the relative location, so that we don't need to
				// convert from a quaternion, which loses winding information.
				SetViewLocation(Actor->GetRootComponent()->GetRelativeLocation());
				SetViewRotation(Actor->GetRootComponent()->GetRelativeRotation());
			}
		}

		constexpr bool bInvalidateChildViews = true;
		Invalidate(bInvalidateChildViews, bInvalidateHitProxies);
	}
}

UActorComponent* FLevelEditorViewportClient::FindViewComponentForActor(AActor const* Actor)
{
	UActorComponent* PreviewComponent = nullptr;
	if (Actor)
	{
		const TWeakObjectPtr<UActorComponent> * CachedComponent = ViewComponentForActorCache.Find(Actor);
		if (CachedComponent != nullptr)
		{
			PreviewComponent = CachedComponent->Get();
		}
		else
		{
			TSet<AActor const*> CheckedActors;
			PreviewComponent = FindViewComponentForActor(Actor, CheckedActors);
			ViewComponentForActorCache.Add(Actor, PreviewComponent);
		}
	}

	return PreviewComponent;
}

UActorComponent* FLevelEditorViewportClient::FindViewComponentForActor(AActor const* Actor, TSet<AActor const*>& CheckedActors)
{
	UActorComponent* PreviewComponent = nullptr;
	if (Actor && !CheckedActors.Contains(Actor))
	{
		CheckedActors.Add(Actor);
		// see if actor has a component with preview capabilities (prioritize camera components)
		const TSet<UActorComponent*>& Comps = Actor->GetComponents();

		// We need to know if any child component with preview info is selected
		bool bFoundSelectedComp = false;

		for (UActorComponent* Comp : Comps)
		{
			FMinimalViewInfo DummyViewInfo;
			if (Comp && Comp->IsActive() && Comp->GetEditorPreviewInfo(/*DeltaTime =*/0.0f, DummyViewInfo))
			{
				if (Comp->IsSelected())
				{
					PreviewComponent = Comp;
					bFoundSelectedComp = true;
					break;
				}
				else if (PreviewComponent)
				{
					UCameraComponent* AsCamComp = Cast<UCameraComponent>(Comp);
					if (AsCamComp != nullptr)
					{
						PreviewComponent = AsCamComp;
					}
					continue;
				}
				PreviewComponent = Comp;
			}
		}

		// No preview if default preview is forbidden and no children selection found
		if (!Actor->IsDefaultPreviewEnabled() && !bFoundSelectedComp)
		{
			return nullptr;
		}

		// now see if any actors are attached to us, directly or indirectly, that have an active camera component we might want to use
		// we will just return the first one.
		if (PreviewComponent == nullptr)
		{
			Actor->ForEachAttachedActors(
				[&](AActor * AttachedActor) -> bool
				{
					UActorComponent* const Comp = FindViewComponentForActor(AttachedActor, CheckedActors);
					if (Comp)
					{
						PreviewComponent = Comp;
						return false; /* stops iteration */
					}

					return true; /* continue iteration */
				}
			);
		}
	}

	return PreviewComponent;
}

void FLevelEditorViewportClient::SetActorLock(AActor* Actor)
{
	// If we had an active lock and are clearing it, also end the transaction for that lock
	if (!Actor)
	{
		AActor* ActiveActorLock = GetActiveActorLock().Get();
		if (ActiveActorLock && !ActiveActorLock->IsLockLocation() && CachedPilotTransform.IsSet())
		{
			FTransform EndPosition = ActiveActorLock->GetTransform();
			ActiveActorLock->SetActorTransform(CachedPilotTransform.GetValue());
			{
				const FScopedTransaction Transaction(LOCTEXT("PilotTransaction", "Pilot Actor"));
				ActiveActorLock->Modify();
				ActiveActorLock->SetActorTransform(EndPosition);
			}

			CachedPilotTransform = TOptional<FTransform>();
		}
	}
	SetActorLock(FLevelViewportActorLock(Actor));
}

void FLevelEditorViewportClient::SetActorLock(const FLevelViewportActorLock& InActorLock)
{
	if (ActorLocks.ActorLock.LockedActor != InActorLock.LockedActor)
	{
		SetIsCameraCut();
	}
	if (ActorLocks.ActorLock.LockedActor.IsValid())
	{
		PreviousActorLocks.ActorLock = ActorLocks.ActorLock;
	}
	ActorLocks.ActorLock = InActorLock;
}

void FLevelEditorViewportClient::SetCinematicActorLock(AActor* Actor)
{
	SetCinematicActorLock(FLevelViewportActorLock(Actor));
}

void FLevelEditorViewportClient::SetCinematicActorLock(const FLevelViewportActorLock& InActorLock)
{
	if (ActorLocks.CinematicActorLock.LockedActor != InActorLock.LockedActor)
	{
		SetIsCameraCut();
	}
	if (ActorLocks.CinematicActorLock.LockedActor.IsValid())
	{
		PreviousActorLocks.CinematicActorLock = ActorLocks.CinematicActorLock;
	}
	ActorLocks.CinematicActorLock = InActorLock;
}

bool FLevelEditorViewportClient::IsActorLocked(const TWeakObjectPtr<const AActor> InActor) const
{
	return (InActor.IsValid() && GetActiveActorLock() == InActor);
}

bool FLevelEditorViewportClient::IsAnyActorLocked() const
{
	return GetActiveActorLock().IsValid();
}

void FLevelEditorViewportClient::UpdateLockedActorViewports(const AActor* InActor, const bool bCheckRealtime, const bool bInvalidateHitProxies)
{
	// Loop through all the other viewports, checking to see if the camera needs updating based on the locked actor
	for (FLevelEditorViewportClient* Client : GEditor->GetLevelViewportClients())
	{
		if( Client && Client != this )
		{
			Client->UpdateLockedActorViewport(InActor, bCheckRealtime, bInvalidateHitProxies);
		}
	}
}

void FLevelEditorViewportClient::UpdateLockedActorViewport(const AActor* InActor, const bool bCheckRealtime, const bool bInvalidateHitProxies)
{
	// If this viewport has the actor locked and we need to update the camera, then do so
	if( IsActorLocked(InActor) && ( !bCheckRealtime || IsRealtime() ) )
	{
		MoveCameraToLockedActor(bInvalidateHitProxies);
	}
}

void FLevelEditorViewportClient::ApplyDeltaToActors(const FVector& InDrag, const FRotator& InRot, const FVector& InScale)
{
	ApplyDeltaToSelectedElements(FTransform(InRot, InDrag, InScale));
}

void FLevelEditorViewportClient::ApplyDeltaToActor(AActor* InActor, const FVector& InDeltaDrag, const FRotator& InDeltaRot, const FVector& InDeltaScale)
{
	if (FTypedElementHandle ActorElementHandle = UEngineElementsLibrary::AcquireEditorActorElementHandle(InActor))
	{
		ApplyDeltaToElement(ActorElementHandle, FTransform(InDeltaRot, InDeltaDrag, InDeltaScale));
	}
}

void FLevelEditorViewportClient::ApplyDeltaToComponent(USceneComponent* InComponent, const FVector& InDeltaDrag, const FRotator& InDeltaRot, const FVector& InDeltaScale)
{
	if (FTypedElementHandle ComponentElementHandle = UEngineElementsLibrary::AcquireEditorComponentElementHandle(InComponent))
	{
		ApplyDeltaToElement(ComponentElementHandle, FTransform(InDeltaRot, InDeltaDrag, InDeltaScale));
	}
}

void FLevelEditorViewportClient::ApplyDeltaToSelectedElements(const FTransform& InDeltaTransform)
{
	if (InDeltaTransform.GetTranslation().IsZero() && InDeltaTransform.Rotator().IsZero() && InDeltaTransform.GetScale3D().IsZero())
	{
		return;
	}

	FTransform ModifiedDeltaTransform = InDeltaTransform;

	{
		FVector AdjustedScale = ModifiedDeltaTransform.GetScale3D();
		FSnappingUtils::AdjustPercentageBasedScale(AdjustedScale);

		ModifiedDeltaTransform.SetScale3D(AdjustedScale);
	}

	FInputDeviceState InputState;
	InputState.SetModifierKeyStates(IsShiftPressed(), IsAltPressed(), IsCtrlPressed(), IsCmdPressed());

	FTypedElementListConstRef ElementsToManipulate = GetElementsToManipulate();

	// Making sure manipulated elements have Modify called on them
	if (GUndo)
	{
		ElementsToManipulate->ForEachElement<ITypedElementObjectInterface>([](const TTypedElement<ITypedElementObjectInterface>& ObjectInterface)
		{
			if (UObject* const Object = ObjectInterface.GetObject())
			{
				if (!GUndo->ContainsObject(Object))
				{
					Object->Modify();
				}
			}

			return true;
		});
	}

	ViewportInteraction->UpdateGizmoManipulation(ElementsToManipulate, GetWidgetMode(), Widget ? Widget->GetCurrentAxis() : EAxisList::None, InputState, ModifiedDeltaTransform);
}

void FLevelEditorViewportClient::ApplyDeltaToElement(const FTypedElementHandle& InElementHandle, const FTransform& InDeltaTransform)
{
	FInputDeviceState InputState;
	InputState.SetModifierKeyStates(IsShiftPressed(), IsAltPressed(), IsCtrlPressed(), IsCmdPressed());

	ViewportInteraction->ApplyDeltaToElement(InElementHandle, GetWidgetMode(), Widget ? Widget->GetCurrentAxis() : EAxisList::None, InputState, InDeltaTransform);
}

void FLevelEditorViewportClient::MirrorSelectedActors(const FVector& InMirrorScale)
{
	FScopedLevelDirtied LevelDirtyCallback;

	const UTypedElementSelectionSet* SelectionSet = GetSelectionSet();
	SelectionSet->ForEachSelectedObject<AActor>([this, &InMirrorScale, &LevelDirtyCallback](AActor* InActor)
	{
		ViewportInteraction->MirrorElement(UEngineElementsLibrary::AcquireEditorActorElementHandle(InActor), InMirrorScale);
		LevelDirtyCallback.Request();
		return true;
	});

	if (UBrushEditingSubsystem* BrushSubsystem = GEditor->GetEditorSubsystem<UBrushEditingSubsystem>())
	{
		BrushSubsystem->UpdateGeometryFromSelectedBrushes();
	}

	RedrawAllViewportsIntoThisScene();
}

void FLevelEditorViewportClient::MirrorSelectedElements(const FVector& InMirrorScale)
{
	FScopedLevelDirtied LevelDirtyCallback;

	const UTypedElementSelectionSet* SelectionSet = GetSelectionSet();
	SelectionSet->ForEachSelectedElementHandle([this, &InMirrorScale, &LevelDirtyCallback](const FTypedElementHandle& InElementHandle)
	{
		ViewportInteraction->MirrorElement(InElementHandle, InMirrorScale);
		LevelDirtyCallback.Request();
		return true;
	});

	if (UBrushEditingSubsystem* BrushSubsystem = GEditor->GetEditorSubsystem<UBrushEditingSubsystem>())
	{
		BrushSubsystem->UpdateGeometryFromSelectedBrushes();
	}

	RedrawAllViewportsIntoThisScene();
}

bool FLevelEditorViewportClient::GetFocusBounds(FTypedElementListConstRef InElements, FBoxSphereBounds& OutBounds)
{
	return ViewportInteraction->GetFocusBounds(InElements, OutBounds);
}

FTransform FLevelEditorViewportClient::GetPreDragActorTransform(const AActor* InActor) const
{
	FTypedElementHandle Handle = UEngineElementsLibrary::AcquireEditorActorElementHandle(InActor, /*bAllowCreate*/ false);
	if (const FTransform* PreDragTransform = Handle ? PreDragElementTransforms.Find(Handle) : nullptr)
	{
		return *PreDragTransform;
	}
	
	FTransform Transform = InActor->GetTransform();
	// Scale manipulation and delta calculation are always based off of the relative scale.
	// Provide that information here.
	Transform.SetScale3D(InActor->GetActorRelativeScale3D());
	return Transform;
}

EMouseCursor::Type FLevelEditorViewportClient::GetCursor(FViewport* InViewport,int32 X,int32 Y)
{
	EMouseCursor::Type CursorType = FEditorViewportClient::GetCursor(InViewport,X,Y);

	UWorld* ViewportWorld = GetWorld();

	if (ViewportWorld != nullptr)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// Allow the viewport interaction to override any previously set mouse cursor
		UViewportWorldInteraction* WorldInteraction = Cast<UViewportWorldInteraction>(GEditor->GetEditorWorldExtensionsManager()->GetEditorWorldExtensions(GetWorld())->FindExtension(UViewportWorldInteraction::StaticClass()));
		if (WorldInteraction != nullptr)
		{
			if (WorldInteraction->ShouldForceCursor())
			{
				CursorType = EMouseCursor::Crosshairs;
				SetRequiredCursor(false, true);
				UpdateRequiredCursorVisibility();
			}
			else if (WorldInteraction->ShouldSuppressExistingCursor())
			{
				CursorType = EMouseCursor::None;
				SetRequiredCursor(false, false);
				UpdateRequiredCursorVisibility();
			}
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	// Don't select widget axes by mouse over while they're being controlled by a mouse drag.
	if( InViewport->IsCursorVisible() && !bWidgetAxisControlledByDrag && !ModeTools->HasOngoingTransform())
	{
		HHitProxy* HitProxy = InViewport->GetHitProxy(X, Y);
		if( !HitProxy && HoveredObjects.Num() > 0 )
		{
			ClearHoverFromObjects();
			Invalidate( false, false );
		}
	}

	return CursorType;

}

void FLevelEditorViewportClient::MouseMove(FViewport* InViewport, int32 x, int32 y)
{
	if (bUserIsControllingAtmosphericLight0 || bUserIsControllingAtmosphericLight1)
	{
		UWorld* ViewportWorld = GetWorld();

		const uint8 DesiredLightIndex = bUserIsControllingAtmosphericLight0 ? 0 : 1;
		UDirectionalLightComponent* SelectedAtmosphericLight = GetAtmosphericLight(DesiredLightIndex, ViewportWorld);

		if (SelectedAtmosphericLight)
		{
			int32 mouseDeltaX = x - CachedLastMouseX;
			int32 mouseDeltaY = y - CachedLastMouseY;

			FTransform ComponentTransform = SelectedAtmosphericLight->GetComponentTransform();
			FQuat LightRotation = ComponentTransform.GetRotation();
			// Rotate around up axis (yaw)
			FVector UpVector = FVector(0, 0, 1);
			LightRotation = FQuat(UpVector, float(mouseDeltaX)*0.01f) * LightRotation;
			// Light Zenith rotation (pitch)
			FVector PitchRotationAxis = FVector::CrossProduct(LightRotation.GetForwardVector(), UpVector);
			if (FMath::Abs(FVector::DotProduct(LightRotation.GetForwardVector(), UpVector)) > (1.0f - KINDA_SMALL_NUMBER))
			{
				PitchRotationAxis = FVector::CrossProduct(LightRotation.GetForwardVector(), FVector(1, 0, 0));
			}
			PitchRotationAxis.Normalize();
			LightRotation = FQuat(PitchRotationAxis, float(mouseDeltaY)*0.01f) * LightRotation;

			ComponentTransform.SetRotation(LightRotation);
			SelectedAtmosphericLight->SetWorldTransform(ComponentTransform);
			NotifyAtmosphericLightHasMoved(*SelectedAtmosphericLight, false);

			UserControlledAtmosphericLightMatrix = ComponentTransform;
			UserControlledAtmosphericLightMatrix.NormalizeRotation();
		}
	}

	FEditorViewportClient::MouseMove(InViewport, x, y);
}

/**
 * Called when the mouse is moved while a window input capture is in effect
 *
 * @param	InViewport	Viewport that captured the mouse input
 * @param	InMouseX	New mouse cursor X coordinate
 * @param	InMouseY	New mouse cursor Y coordinate
 */
void FLevelEditorViewportClient::CapturedMouseMove( FViewport* InViewport, int32 InMouseX, int32 InMouseY )
{
	// Commit to any pending transactions now
	TrackingTransaction.PromotePendingToActive();

	FEditorViewportClient::CapturedMouseMove(InViewport, InMouseX, InMouseY);
}


/**
 * Checks if the mouse is hovered over a hit proxy and decides what to do.
 */
void FLevelEditorViewportClient::CheckHoveredHitProxy( HHitProxy* HoveredHitProxy )
{
	FEditorViewportClient::CheckHoveredHitProxy(HoveredHitProxy);

	// We'll keep track of changes to hovered objects as the cursor moves
	const bool bUseHoverFeedback = GEditor != NULL && GetDefault<ULevelEditorViewportSettings>()->bEnableViewportHoverFeedback;
	TSet< FViewportHoverTarget > NewHoveredObjects;

	// If the cursor is visible over level viewports, then we'll check for new objects to be hovered over
	if( bUseHoverFeedback && HoveredHitProxy )
	{
		// Set mouse hover cue for objects under the cursor
		if (HoveredHitProxy->IsA(HActor::StaticGetType()) || HoveredHitProxy->IsA(HBSPBrushVert::StaticGetType()))
		{
			// Hovered over an actor
			AActor* ActorUnderCursor = NULL;
			if (HoveredHitProxy->IsA(HActor::StaticGetType()))
			{
				HActor* ActorHitProxy = static_cast<HActor*>(HoveredHitProxy);
				ActorUnderCursor = ActorHitProxy->Actor;
			}
			else if (HoveredHitProxy->IsA(HBSPBrushVert::StaticGetType()))
			{
				HBSPBrushVert* ActorHitProxy = static_cast<HBSPBrushVert*>(HoveredHitProxy);
				ActorUnderCursor = ActorHitProxy->Brush.Get();
			}

			if( ActorUnderCursor != NULL  )
			{
				// Check to see if the actor under the cursor is part of a group.  If so, we will how a hover cue the whole group
				AGroupActor* GroupActor = AGroupActor::GetRootForActor( ActorUnderCursor, true, false );

				if(GroupActor && UActorGroupingUtils::IsGroupingActive())
				{
					// Get all the actors in the group and add them to the list of objects to show a hover cue for.
					TArray<AActor*> ActorsInGroup;
					GroupActor->GetGroupActors( ActorsInGroup, true );
					for( int32 ActorIndex = 0; ActorIndex < ActorsInGroup.Num(); ++ActorIndex )
					{
						NewHoveredObjects.Add( FViewportHoverTarget( ActorsInGroup[ActorIndex] ) );
					}
				}
				else
				{
					NewHoveredObjects.Add( FViewportHoverTarget( ActorUnderCursor ) );
				}
			}
		}
		else if( HoveredHitProxy->IsA( HModel::StaticGetType() ) )
		{
			// Hovered over a model (BSP surface)
			HModel* ModelHitProxy = static_cast< HModel* >( HoveredHitProxy );
			UModel* ModelUnderCursor = ModelHitProxy->GetModel();
			if( ModelUnderCursor != NULL )
			{
				FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
					Viewport, 
					GetScene(),
					EngineShowFlags)
					.SetRealtimeUpdate( IsRealtime() ));
				FSceneView* SceneView = CalcSceneView( &ViewFamily );

				uint32 SurfaceIndex = INDEX_NONE;
				if( ModelHitProxy->ResolveSurface( SceneView, CachedMouseX, CachedMouseY, SurfaceIndex ) )
				{
					FBspSurf& Surf = ModelUnderCursor->Surfs[ SurfaceIndex ];
					Surf.PolyFlags |= PF_Hovered;

					NewHoveredObjects.Add( FViewportHoverTarget( ModelUnderCursor, SurfaceIndex ) );
				}
			}
		}
	}

	UpdateHoveredObjects( NewHoveredObjects );
}

void FLevelEditorViewportClient::UpdateHoveredObjects( const TSet<FViewportHoverTarget>& NewHoveredObjects )
{
	// Check to see if there are any hovered objects that need to be updated
	{
		bool bAnyHoverChanges = false;
		if( NewHoveredObjects.Num() > 0 )
		{
			for( TSet<FViewportHoverTarget>::TIterator It( HoveredObjects ); It; ++It )
			{
				FViewportHoverTarget& OldHoverTarget = *It;
				if( !NewHoveredObjects.Contains( OldHoverTarget ) )
				{
					// Remove hover effect from object that no longer needs it
					RemoveHoverEffect( OldHoverTarget );
					It.RemoveCurrent();

					bAnyHoverChanges = true;
				}
			}
		}

		for( TSet<FViewportHoverTarget>::TConstIterator It( NewHoveredObjects ); It; ++It )
		{
			const FViewportHoverTarget& NewHoverTarget = *It;
			if( !HoveredObjects.Contains( NewHoverTarget ) )
			{
				// Add hover effect to this object
				AddHoverEffect( NewHoverTarget );
				HoveredObjects.Add( NewHoverTarget );

				bAnyHoverChanges = true;
			}
		}


		// Redraw the viewport if we need to
		if( bAnyHoverChanges )
		{
			// NOTE: We're only redrawing the viewport that the mouse is over.  We *could* redraw all viewports
			//		 so the hover effect could be seen in all potential views, but it will be slower.
			RedrawRequested( Viewport );
		}
	}
}

bool FLevelEditorViewportClient::GetActiveSafeFrame(float& OutAspectRatio) const
{
	if (!IsOrtho())
	{
		const UCameraComponent* CameraComponent = GetCameraComponentForView();
		if (CameraComponent && CameraComponent->bConstrainAspectRatio)
		{
			OutAspectRatio = CameraComponent->AspectRatio;
			return true;
		}
	}

	return false;
}

/** 
 * Renders a view frustum specified by the provided frustum parameters
 *
 * @param	PDI					PrimitiveDrawInterface to use to draw the view frustum
 * @param	FrustumColor		Color to draw the view frustum in
 * @param	FrustumAngle		Angle of the frustum
 * @param	FrustumAspectRatio	Aspect ratio of the frustum
 * @param	FrustumStartDist	Start distance of the frustum
 * @param	FrustumEndDist		End distance of the frustum
 * @param	InViewMatrix		View matrix to use to draw the frustum
 */
static void RenderViewFrustum( FPrimitiveDrawInterface* PDI,
								const FLinearColor& FrustumColor,
								float FrustumAngle,
								float FrustumAspectRatio,
								float FrustumStartDist,
								float FrustumEndDist,
								const FMatrix& InViewMatrix)
{
	FVector Direction(0,0,1);
	FVector LeftVector(1,0,0);
	FVector UpVector(0,1,0);

	FVector Verts[8];

	// FOVAngle controls the horizontal angle.
	float HozHalfAngle = (FrustumAngle) * ((float)PI/360.f);
	float HozLength = FrustumStartDist * FMath::Tan(HozHalfAngle);
	float VertLength = HozLength/FrustumAspectRatio;

	// near plane verts
	Verts[0] = (Direction * FrustumStartDist) + (UpVector * VertLength) + (LeftVector * HozLength);
	Verts[1] = (Direction * FrustumStartDist) + (UpVector * VertLength) - (LeftVector * HozLength);
	Verts[2] = (Direction * FrustumStartDist) - (UpVector * VertLength) - (LeftVector * HozLength);
	Verts[3] = (Direction * FrustumStartDist) - (UpVector * VertLength) + (LeftVector * HozLength);

	HozLength = FrustumEndDist * FMath::Tan(HozHalfAngle);
	VertLength = HozLength/FrustumAspectRatio;

	// far plane verts
	Verts[4] = (Direction * FrustumEndDist) + (UpVector * VertLength) + (LeftVector * HozLength);
	Verts[5] = (Direction * FrustumEndDist) + (UpVector * VertLength) - (LeftVector * HozLength);
	Verts[6] = (Direction * FrustumEndDist) - (UpVector * VertLength) - (LeftVector * HozLength);
	Verts[7] = (Direction * FrustumEndDist) - (UpVector * VertLength) + (LeftVector * HozLength);

	for( int32 x = 0 ; x < 8 ; ++x )
	{
		Verts[x] = InViewMatrix.InverseFast().TransformPosition( Verts[x] );
	}

	const uint8 PrimitiveDPG = SDPG_Foreground;
	PDI->DrawLine( Verts[0], Verts[1], FrustumColor, PrimitiveDPG );
	PDI->DrawLine( Verts[1], Verts[2], FrustumColor, PrimitiveDPG );
	PDI->DrawLine( Verts[2], Verts[3], FrustumColor, PrimitiveDPG );
	PDI->DrawLine( Verts[3], Verts[0], FrustumColor, PrimitiveDPG );

	PDI->DrawLine( Verts[4], Verts[5], FrustumColor, PrimitiveDPG );
	PDI->DrawLine( Verts[5], Verts[6], FrustumColor, PrimitiveDPG );
	PDI->DrawLine( Verts[6], Verts[7], FrustumColor, PrimitiveDPG );
	PDI->DrawLine( Verts[7], Verts[4], FrustumColor, PrimitiveDPG );

	PDI->DrawLine( Verts[0], Verts[4], FrustumColor, PrimitiveDPG );
	PDI->DrawLine( Verts[1], Verts[5], FrustumColor, PrimitiveDPG );
	PDI->DrawLine( Verts[2], Verts[6], FrustumColor, PrimitiveDPG );
	PDI->DrawLine( Verts[3], Verts[7], FrustumColor, PrimitiveDPG );
}

void FLevelEditorViewportClient::Draw(const FSceneView* View,FPrimitiveDrawInterface* PDI)
{
	FMemMark Mark(FMemStack::Get());

	FEditorViewportClient::Draw(View,PDI);

	DrawBrushDetails(View, PDI);
	AGroupActor::DrawBracketsForGroups(PDI, Viewport);

	if (EngineShowFlags.StreamingBounds)
	{
		DrawTextureStreamingBounds(View, PDI);
	}

	// A frustum should be drawn if the viewport is ortho and level streaming volume previs is enabled in some viewport
	if ( IsOrtho() )
	{
		for (FLevelEditorViewportClient* CurViewportClient : GEditor->GetLevelViewportClients())
		{
			if ( CurViewportClient && IsPerspective() && GetDefault<ULevelEditorViewportSettings>()->bLevelStreamingVolumePrevis )
			{
				// Draw the view frustum of the level streaming volume previs viewport.
				RenderViewFrustum(PDI, FLinearColor(1.0, 0.0, 1.0, 1.0),
					GPerspFrustumAngle,
					GPerspFrustumAspectRatio,
					GPerspFrustumStartDist,
					GPerspFrustumEndDist,
					GPerspViewMatrix);

				break;
			}
		}
	}

	if (IsPerspective())
	{
		DrawStaticLightingDebugInfo( View, PDI );
	}

	if ( GEditor->bEnableSocketSnapping )
	{
		const bool bGameViewMode = View->Family->EngineShowFlags.Game && !GEditor->bDrawSocketsInGMode;

		for( FActorIterator It(GetWorld()); It; ++It )
		{
			AActor* Actor = *It;

			if (bGameViewMode || Actor->IsHiddenEd())
			{
				// Don't display sockets on hidden actors...
				continue;
			}

			for (UActorComponent* Component : Actor->GetComponents())
			{
				USceneComponent* SceneComponent = Cast<USceneComponent>(Component);
				if (SceneComponent && SceneComponent->HasAnySockets())
				{
					TArray<FComponentSocketDescription> Sockets;
					SceneComponent->QuerySupportedSockets(Sockets);

					for (int32 SocketIndex = 0; SocketIndex < Sockets.Num() ; ++SocketIndex)
					{
						FComponentSocketDescription& Socket = Sockets[SocketIndex];

						if (Socket.Type == EComponentSocketType::Socket)
						{
							const FTransform SocketTransform = SceneComponent->GetSocketTransform(Socket.Name);

							const float DiamondSize = 2.0f;
							const FColor DiamondColor(255,128,128);

							PDI->SetHitProxy( new HLevelSocketProxy( *It, SceneComponent, Socket.Name ) );
							DrawWireDiamond( PDI, SocketTransform.ToMatrixWithScale(), DiamondSize, DiamondColor, SDPG_Foreground );
							PDI->SetHitProxy( NULL );
						}
					}
				}
			}
		}
	}

	if( this == GCurrentLevelEditingViewportClient )
	{
		FSnappingUtils::DrawSnappingHelpers( View, PDI );
	}

	if(GUnrealEd != NULL && !IsInGameView())
	{
		GUnrealEd->DrawComponentVisualizers(View, PDI);
	}

	if (GEditor->bDrawParticleHelpers == true)
	{
		if (View->Family->EngineShowFlags.Game)
		{
			extern ENGINE_API void DrawParticleSystemHelpers(const FSceneView* View,FPrimitiveDrawInterface* PDI);
			DrawParticleSystemHelpers(View, PDI);
		}
	}

	if (UserIsControllingAtmosphericLightTimer > 0.0f)
	{
		// Draw a gizmo helping to figure out where is the light when moving it using a shortcut.
		FQuat ViewRotation = FQuat(GetViewRotation());
		FVector ViewPosition = GetViewLocation();
		const float GizmoDistance = 50.0f;
		const float GizmoSideOffset = 15.0f;
		const float GizmoRadius = 10.0f;
		const float ThicknessLight = 0.05f;
		const float ThicknessBold = 0.2f;

		// Always draw the gizmo right in in front of the camera with a little side shift.
		const FVector X(1.0f, 0.0f, 0.0f);
		const FVector Y(0.0f, 1.0f, 0.0f);
		const FVector Z(0.0f, 0.0f, 1.0f);
		const FVector Base = ViewPosition + GizmoDistance * ViewRotation.GetForwardVector() + GizmoSideOffset * (-ViewRotation.GetUpVector() + ViewRotation.GetRightVector());

		// Draw world main axis
		FRotator IdentityX(0.0f, 0.0f, 0.0f);
		FRotator IdentityY(0.0f, 90.0f, 0.0f);
		FRotator IdentityZ(90.0f, 0.0f, 0.0f);
		DrawDirectionalArrow(PDI, FQuatRotationTranslationMatrix(FQuat(IdentityX), Base), FColor(255, 0, 0, 127), GizmoRadius, 0.3f, SDPG_World, ThicknessBold);
		DrawDirectionalArrow(PDI, FQuatRotationTranslationMatrix(FQuat(IdentityY), Base), FColor(0, 255, 0, 127), GizmoRadius, 0.3f, SDPG_World, ThicknessBold);
		DrawDirectionalArrow(PDI, FQuatRotationTranslationMatrix(FQuat(IdentityZ), Base), FColor(0, 0, 255, 127), GizmoRadius, 0.3f, SDPG_World, ThicknessBold);

		// Render polar coordinate circles
		DrawCircle(PDI, Base, X, Y, FLinearColor(0.2f, 0.2f, 1.0f), GizmoRadius, 32, SDPG_World, ThicknessBold);
		DrawCircle(PDI, Base, X, Y, FLinearColor(0.2f, 0.2f, 0.75f), GizmoRadius*0.75f, 32, SDPG_World, ThicknessLight);
		DrawCircle(PDI, Base, X, Y, FLinearColor(0.2f, 0.2f, 0.50f), GizmoRadius*0.50f, 32, SDPG_World, ThicknessLight);
		DrawCircle(PDI, Base, X, Y, FLinearColor(0.2f, 0.2f, 0.25f), GizmoRadius*0.25f, 32, SDPG_World, ThicknessLight);
		DrawArc(PDI, Base, Z, Y, -90.0f, 90.0f, GizmoRadius, 32, FLinearColor(1.0f, 0.2f, 0.2f), SDPG_World);
		DrawArc(PDI, Base, Z, X, -90.0f, 90.0f, GizmoRadius, 32, FLinearColor(0.2f, 1.0f, 0.2f), SDPG_World);

		// Draw the light incoming light direction. The arrow is offset outward to help depth perception when it intersects with other gizmo elements.
		const FLinearColor ArrowColor = FLinearColor(-UserControlledAtmosphericLightMatrix.GetRotation().GetForwardVector() * 0.5f + 0.5f);
		const FVector ArrowOrigin = Base - UserControlledAtmosphericLightMatrix.GetRotation().GetForwardVector()*GizmoRadius*1.25;
		const FQuatRotationTranslationMatrix ArrowToWorld(UserControlledAtmosphericLightMatrix.GetRotation(), ArrowOrigin);
		DrawDirectionalArrow(PDI, ArrowToWorld, ArrowColor, GizmoRadius, 0.3f, SDPG_World, ThicknessBold);

		// Now draw x, y and z axis to help getting a sense of depth when look at the vectors on screen.
		FVector LightArrowTip = -UserControlledAtmosphericLightMatrix.GetRotation().GetForwardVector()*GizmoRadius;
		FVector P0 = Base + LightArrowTip * FVector(1.0f, 0.0f, 0.0f);
		FVector P1 = Base + LightArrowTip * FVector(1.0f, 1.0f, 0.0f);
		FVector P2 = Base + LightArrowTip * FVector(1.0f, 1.0f, 1.0f);
		PDI->DrawLine(Base, P0, FLinearColor(1.0f, 0.0f, 0.0f), SDPG_World, ThicknessLight);
		PDI->DrawLine(P0, P1, FLinearColor(0.0f, 1.0f, 0.0f), SDPG_World, ThicknessLight);
		PDI->DrawLine(P1, P2, FLinearColor(0.0f, 0.0f, 1.0f), SDPG_World, ThicknessLight);
	}

	Mark.Pop();
}

void FLevelEditorViewportClient::DrawBrushDetails(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	// Draw translucent polygons on brushes and volumes
	TRACE_CPUPROFILER_EVENT_SCOPE(FLevelEditorViewportClient::DrawBrushDetails);

	for (TActorIterator<ABrush> It(GetWorld()); It; ++It)
	{
		ABrush* Brush = *It;

		// Brush->Brush is checked to safe from brushes that were created without having their brush members attached.
		// Check whether it satisfies the ShowBrushMarkersPoly condition of being selected or whether bDisplayShadedVolume is true
		if (Brush->Brush && Brush->Brush->Polys && (FActorEditorUtils::IsABuilderBrush(Brush) || Brush->IsVolumeBrush()) && 
			(Brush->bDisplayShadedVolume || (GetDefault<ULevelEditorViewportSettings>()->bShowBrushMarkerPolys && ModeTools->GetSelectedActors()->IsSelected(Brush))))
		{
			// Build a mesh by basically drawing the triangles of each 
			FDynamicMeshBuilder MeshBuilder(View->GetFeatureLevel());
			int32 VertexOffset = 0;

			for (int32 PolyIdx = 0; PolyIdx < Brush->Brush->Polys->Element.Num(); ++PolyIdx)
			{
				const FPoly* Poly = &Brush->Brush->Polys->Element[PolyIdx];

				if (Poly->Vertices.Num() > 2)
				{
					const FVector3f Vertex0 = Poly->Vertices[0];
					FVector3f Vertex1 = Poly->Vertices[1];

					MeshBuilder.AddVertex(Vertex0, FVector2f::ZeroVector, FVector3f(1, 0, 0), FVector3f(0, 1, 0), FVector3f(0, 0, 1), FColor::White);
					MeshBuilder.AddVertex(Vertex1, FVector2f::ZeroVector, FVector3f(1, 0, 0), FVector3f(0, 1, 0), FVector3f(0, 0, 1), FColor::White);

					for (int32 VertexIdx = 2; VertexIdx < Poly->Vertices.Num(); ++VertexIdx)
					{
						const FVector3f Vertex2 = Poly->Vertices[VertexIdx];
						MeshBuilder.AddVertex(Vertex2, FVector2f::ZeroVector, FVector3f(1, 0, 0), FVector3f(0, 1, 0), FVector3f(0, 0, 1), FColor::White);
						MeshBuilder.AddTriangle(VertexOffset, VertexOffset + VertexIdx, VertexOffset + VertexIdx - 1);
						Vertex1 = Vertex2;
					}

					// Increment the vertex offset so the next polygon uses the correct vertex indices.
					VertexOffset += Poly->Vertices.Num();
				}
			}
			// Use the material below when bDisplayShadedVolume is true
			if (Brush->bDisplayShadedVolume)
			{
				// Allocate the material proxy and register it so it can be deleted properly once the rendering is done with it.
				FColor VolumeColor = Brush->GetWireColor();
				VolumeColor.A = static_cast<uint8>(Brush->ShadedVolumeOpacityValue * 255.0f);
				FDynamicColoredMaterialRenderProxy* MaterialProxy = new FDynamicColoredMaterialRenderProxy(GEngine->GeomMaterial->GetRenderProxy(), VolumeColor);
				PDI->RegisterDynamicResource(MaterialProxy);

				// Flush the mesh triangles.
				// This will get the default behavior of receiving decals but not having a hitproxy ID
				MeshBuilder.Draw(PDI, Brush->ActorToWorld().ToMatrixWithScale(), MaterialProxy, SDPG_World, false);
			}
			else
			{
				// Allocate the material proxy and register it so it can be deleted properly once the rendering is done with it.
				FDynamicColoredMaterialRenderProxy* MaterialProxy = new FDynamicColoredMaterialRenderProxy(GEngine->EditorBrushMaterial->GetRenderProxy(), Brush->GetWireColor());
				PDI->RegisterDynamicResource(MaterialProxy);

				// Flush the mesh triangles.
				MeshBuilder.Draw(PDI, Brush->ActorToWorld().ToMatrixWithScale(), MaterialProxy, SDPG_World, false, true, FHitProxyId::InvisibleHitProxyId);
			}
		}
	}
	
	if (ModeTools->ShouldDrawBrushVertices() && !IsInGameView())
	{
		UTexture2D* VertexTexture = GEngine->DefaultBSPVertexTexture;
		const float TextureSizeX = VertexTexture->GetSizeX() * 0.170f;
		const float TextureSizeY = VertexTexture->GetSizeY() * 0.170f;

		USelection* Selection = ModeTools->GetSelectedActors();
		if(Selection->IsClassSelected(ABrush::StaticClass()))
		{
			for (FSelectionIterator It(*ModeTools->GetSelectedActors()); It; ++It)
			{
				ABrush* Brush = Cast<ABrush>(*It);
				if(Brush && Brush->Brush && !FActorEditorUtils::IsABuilderBrush(Brush))
				{
					for(int32 p = 0; p < Brush->Brush->Polys->Element.Num(); ++p)
					{
						FTransform BrushTransform = Brush->ActorToWorld();

						FPoly* poly = &Brush->Brush->Polys->Element[p];
						for(int32 VertexIndex = 0; VertexIndex < poly->Vertices.Num(); ++VertexIndex)
						{
							const FVector3f& PolyVertex = poly->Vertices[VertexIndex];
							const FVector WorldLocation = FVector(BrushTransform.TransformPosition((FVector)PolyVertex));

							const float Scale =
								static_cast<float>(View->WorldToScreen(WorldLocation).W * (4.0f / View->UnscaledViewRect.Width() / View->ViewMatrices.GetViewToClip().M[0][0]));

							const FColor Color(Brush->GetWireColor());
							PDI->SetHitProxy(new HBSPBrushVert(Brush, &poly->Vertices[VertexIndex]));

							PDI->DrawSprite(WorldLocation, TextureSizeX * Scale, TextureSizeY * Scale, VertexTexture->GetResource(), Color, SDPG_World, 0.0f, 0.0f, 0.0f, 0.0f, SE_BLEND_Masked);

							PDI->SetHitProxy(NULL);
						}
					}
				}
			}
		}
	}
}

void FLevelEditorViewportClient::UpdateAudioListener(const FSceneView& View)
{
	UpdateAudioListener();
}

void FLevelEditorViewportClient::UpdateAudioListener()
{
	UWorld* ViewportWorld = GetWorld();

	if (ViewportWorld)
	{
		if (FAudioDevice* AudioDevice = ViewportWorld->GetAudioDeviceRaw())
		{
			FVector ViewLocation = GetViewLocation();
			FRotator ViewRotation = GetViewRotation();

			const bool bStereoRendering = GEngine->XRSystem.IsValid() && GEngine->IsStereoscopic3D( Viewport );
			if( bStereoRendering && GEngine->XRSystem->IsHeadTrackingAllowed() )
			{
				FQuat RoomSpaceHeadOrientation;
				FVector RoomSpaceHeadLocation;
				GEngine->XRSystem->GetCurrentPose( IXRTrackingSystem::HMDDeviceId, /* Out */ RoomSpaceHeadOrientation, /* Out */ RoomSpaceHeadLocation );

				// NOTE: The RoomSpaceHeadLocation has already been adjusted for WorldToMetersScale
				const FVector WorldSpaceHeadLocation = GetViewLocation() + GetViewRotation().RotateVector( RoomSpaceHeadLocation );
				ViewLocation = WorldSpaceHeadLocation;

				ViewRotation = (ViewRotation.Quaternion() * RoomSpaceHeadOrientation).Rotator();
			}

			FTransform ListenerTransform(ViewRotation);
			ListenerTransform.SetLocation(ViewLocation);

			AudioDevice->SetListener(ViewportWorld, 0, ListenerTransform, 0.f);
		}
	}
}

/** Determines if the new MoveCanvas movement should be used
 * @return - true if we should use the new drag canvas movement.  Returns false for combined object-camera movement and marquee selection
 */
bool FLevelEditorViewportClient::ShouldUseMoveCanvasMovement()
{
	const bool LeftMouseButtonDown = Viewport->KeyState(EKeys::LeftMouseButton) ? true : false;
	const bool MiddleMouseButtonDown = Viewport->KeyState(EKeys::MiddleMouseButton) ? true : false;
	const bool RightMouseButtonDown = Viewport->KeyState(EKeys::RightMouseButton) ? true : false;
	const bool bMouseButtonDown = (LeftMouseButtonDown || MiddleMouseButtonDown || RightMouseButtonDown );

	const bool AltDown = IsAltPressed();
	const bool ShiftDown = IsShiftPressed();
	const bool ControlDown = IsCtrlPressed();

	//if we're using the new move canvas mode, we're in an ortho viewport, and the mouse is down
	if (GetDefault<ULevelEditorViewportSettings>()->bPanMovesCanvas && IsOrtho() && bMouseButtonDown)
	{
		//MOVING CAMERA
		if ( !MouseDeltaTracker->UsingDragTool() && AltDown == false && ShiftDown == false && ControlDown == false && (Widget->GetCurrentAxis() == EAxisList::None) && (LeftMouseButtonDown ^ RightMouseButtonDown))
		{
			return true;
		}

		//OBJECT MOVEMENT CODE
		if ( ( AltDown == false && ShiftDown == false && ( LeftMouseButtonDown ^ RightMouseButtonDown ) ) &&
			( ( GetWidgetMode() == UE::Widget::WM_Translate && Widget->GetCurrentAxis() != EAxisList::None ) ||
			( GetWidgetMode() == UE::Widget::WM_TranslateRotateZ && Widget->GetCurrentAxis() != EAxisList::ZRotation &&  Widget->GetCurrentAxis() != EAxisList::None ) ||
			( GetWidgetMode() == UE::Widget::WM_2D && Widget->GetCurrentAxis() != EAxisList::Rotate2D &&  Widget->GetCurrentAxis() != EAxisList::None ) ) )
		{
			return true;
		}


		//ALL other cases hide the mouse
		return false;
	}
	else
	{
		//current system - do not show cursor when mouse is down
		return false;
	}
}

void FLevelEditorViewportClient::SetupViewForRendering( FSceneViewFamily& ViewFamily, FSceneView& View )
{
	FEditorViewportClient::SetupViewForRendering( ViewFamily, View );

	ViewFamily.bDrawBaseInfo = bDrawBaseInfo;
	ViewFamily.bCurrentlyBeingEdited = bCurrentlyEditingThroughMovementWidget;

	// Don't use fading or color scaling while we're in light complexity mode, since it may change the colors!
	if(!ViewFamily.EngineShowFlags.LightComplexity)
	{
		if(bEnableFading)
		{
			View.OverlayColor = FadeColor;
			View.OverlayColor.A = FMath::Clamp(FadeAmount, 0.f, 1.f);
		}

		if(bEnableColorScaling)
		{
			View.ColorScale = FLinearColor(FVector3f{ ColorScale });
		}
	}

	TSharedPtr<FDragDropOperation> DragOperation = FSlateApplication::Get().GetDragDroppingContent();
	if (!(DragOperation.IsValid() && DragOperation->IsOfType<FBrushBuilderDragDropOp>()))
	{
		// Hide the builder brush when not in geometry mode
		ViewFamily.EngineShowFlags.SetBuilderBrush(false);
	}
}

void FLevelEditorViewportClient::DrawCanvas( FViewport& InViewport, FSceneView& View, FCanvas& Canvas )
{
	// HUD for components visualizers
	if (GUnrealEd != NULL && !IsInGameView())
	{
		GUnrealEd->DrawComponentVisualizersHUD(&InViewport, &View, &Canvas);
	}

	FEditorViewportClient::DrawCanvas(InViewport, View, Canvas);

	// Testbed
	FCanvasItemTestbed TestBed;
	TestBed.Draw( Viewport, &Canvas );

	DrawStaticLightingDebugInfo(&View, &Canvas);
}

/**
 *	Draw the texture streaming bounds.
 */
void FLevelEditorViewportClient::DrawTextureStreamingBounds(const FSceneView* View,FPrimitiveDrawInterface* PDI)
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	TArray<FAssetData> SelectedAssetData;
	ContentBrowserModule.Get().GetSelectedAssets(SelectedAssetData);

	TArray<const UTexture2D*> SelectedTextures;
	for (auto AssetIt = SelectedAssetData.CreateConstIterator(); AssetIt; ++AssetIt)
	{
		if (AssetIt->IsAssetLoaded())
		{
			const UTexture2D* Texture = Cast<UTexture2D>(AssetIt->GetAsset());
			if (Texture)
	{
				SelectedTextures.Add(Texture);
			}
		}
	}

	TArray<FBox> AssetBoxes;
	if (IStreamingManager::Get().IsTextureStreamingEnabled())
		{
		for (const UTexture2D* Texture : SelectedTextures)
			{
			IStreamingManager::Get().GetTextureStreamingManager().GetObjectReferenceBounds(Texture, AssetBoxes);
			}
		}

	for (const FBox& Box : AssetBoxes)
	{
		DrawWireBox(PDI, Box, FColorList::Yellow, SDPG_World);
	}
}


/** Serialization. */
void FLevelEditorViewportClient::AddReferencedObjects( FReferenceCollector& Collector )
{
	FEditorViewportClient::AddReferencedObjects( Collector );

	for( TSet<FViewportHoverTarget>::TIterator It( FLevelEditorViewportClient::HoveredObjects ); It; ++It )
	{
		FViewportHoverTarget& CurHoverTarget = *It;
		Collector.AddReferencedObject( CurHoverTarget.HoveredActor );
		Collector.AddReferencedObject( CurHoverTarget.HoveredModel );
	}

	{
		FSceneViewStateInterface* Ref = ViewState.GetReference();

		if(Ref)
		{
			Ref->AddReferencedObjects(Collector);
		}
	}
}

/**
 * Copies layout and camera settings from the specified viewport
 *
 * @param InViewport The viewport to copy settings from
 */
void FLevelEditorViewportClient::CopyLayoutFromViewport( const FLevelEditorViewportClient& InViewport )
{
	SetViewLocation( InViewport.GetViewLocation() );
	SetViewRotation( InViewport.GetViewRotation() );
	ViewFOV = InViewport.ViewFOV;
	ViewportType = InViewport.ViewportType;
	SetOrthoZoom( InViewport.GetOrthoZoom() );
	ActorLocks = InViewport.ActorLocks;
	bAllowCinematicControl = InViewport.bAllowCinematicControl;
}


UWorld* FLevelEditorViewportClient::ConditionalSetWorld()
{
	// Should set GWorld to the play world if we are simulating in the editor and not already in the play world (reentrant calls to this would cause the world to be the same)
	if( bIsSimulateInEditorViewport && GEditor->PlayWorld && GEditor->PlayWorld != GWorld )
	{
		return SetPlayInEditorWorld( GEditor->PlayWorld );
	}

	// Returned world doesn't matter for this case
	return NULL;
}

void FLevelEditorViewportClient::ConditionalRestoreWorld( UWorld* InWorld )
{
	if( bIsSimulateInEditorViewport && InWorld )
	{
		// We should not already be in the world about to switch to an we should not be switching to the play world
		check( GWorld != InWorld && InWorld != GEditor->PlayWorld );
		RestoreEditorWorld( InWorld );
	}
}


/** Updates any orthographic viewport movement to use the same location as this viewport */
void FLevelEditorViewportClient::UpdateLinkedOrthoViewports( bool bInvalidate )
{
	// Only update if linked ortho movement is on, this viewport is orthographic, and is the current viewport being used.
	if (GetDefault<ULevelEditorViewportSettings>()->bUseLinkedOrthographicViewports && IsOrtho() && GCurrentLevelEditingViewportClient == this)
	{
		int32 MaxFrames = -1;
		int32 NextViewportIndexToDraw = INDEX_NONE;

		// Search through all viewports for orthographic ones
		for( int32 ViewportIndex = 0; ViewportIndex < GEditor->GetLevelViewportClients().Num(); ++ViewportIndex )
		{
			FLevelEditorViewportClient* Client = GEditor->GetLevelViewportClients()[ViewportIndex];
			check(Client);

			// Only update other orthographic viewports viewing the same scene
			if( (Client != this) && Client->IsOrtho() && (Client->GetScene() == this->GetScene()) )
			{
				int32 Frames = Client->FramesSinceLastDraw;
				Client->bNeedsLinkedRedraw = false;
				Client->SetOrthoZoom( GetOrthoZoom() );
				Client->SetViewLocation( GetViewLocation() );
				if( Client->IsVisible() )
				{
					// Find the viewport which has the most number of frames since it was last rendered.  We will render that next.
					if( Frames > MaxFrames )
					{
						MaxFrames = Frames;
						NextViewportIndexToDraw = ViewportIndex;
					}
					if( bInvalidate )
					{
						Client->Invalidate();
					}
				}
			}
		}

		if( bInvalidate )
		{
			Invalidate();
		}

		if( NextViewportIndexToDraw != INDEX_NONE )
		{
			// Force this viewport to redraw.
			GEditor->GetLevelViewportClients()[NextViewportIndexToDraw]->bNeedsLinkedRedraw = true;
		}
	}
}


//
//	FLevelEditorViewportClient::GetScene
//

FLinearColor FLevelEditorViewportClient::GetBackgroundColor() const
{
	return IsPerspective() ? GEditor->C_WireBackground : GEditor->C_OrthoBackground;
}

const FEditorViewportCameraSpeedSettings& FLevelEditorViewportClient::GetCameraSpeedSettings() const
{
	return GetDefault<ULevelEditorViewportSettings>()->CameraSpeedSettings;
}

void FLevelEditorViewportClient::SetCameraSpeedSettings(const FEditorViewportCameraSpeedSettings& InCameraSpeedSettings)
{
	GetMutableDefault<ULevelEditorViewportSettings>()->CameraSpeedSettings = InCameraSpeedSettings;
}

bool FLevelEditorViewportClient::OverrideHighResScreenshotCaptureRegion(FIntRect& OutCaptureRegion)
{
	FSlateRect Rect;
	if (CalculateEditorConstrainedViewRect(Rect, Viewport, GetDPIScale()))
	{
		FSlateRect InnerRect = Rect.InsetBy(FMargin(0.5f * SafePadding * Rect.GetSize().Size()));
		OutCaptureRegion = FIntRect((int32)InnerRect.Left, (int32)InnerRect.Top, (int32)(InnerRect.Left + InnerRect.GetSize().X), (int32)(InnerRect.Top + InnerRect.GetSize().Y));
		return true;
	}
	return false;
}

bool FLevelEditorViewportClient::BeginTransform(const FGizmoState& InState)
{
	TrackingTransaction.End();
	bDuplicateOnNextDrag = false;
	bOnlyMovedPivot = false;
	bNeedToRestoreComponentBeingMovedFlag = false;
	MouseDeltaTracker->SetExternalMovement(true);

	PreDragElementTransforms.Empty();

	Widget->SetSnapEnabled(true);

	const FTypedElementListConstRef ElementsToManipulate = GetElementsToManipulate();
	ViewportInteraction->BeginGizmoManipulation(ElementsToManipulate, GetWidgetMode());
	bHasBegunGizmoManipulation = true;

	if (!bDuplicateActorsInProgress)
	{
		bNeedToRestoreComponentBeingMovedFlag = true;
		TypedElementListObjectUtil::ForEachObject<AActor>(ElementsToManipulate, [this](AActor* InActor)
		{
			SetActorBeingMovedByEditor(InActor, true);
			return true;
		});
	}

	TrackingTransaction.TransCount++;
	const FText Description = LOCTEXT("TransformTransaction", "Transform Elements");
	TrackingTransaction.BeginPending(Description);

	if (TrackingTransaction.IsActive() || TrackingTransaction.IsPending())
	{
		// Suspend actor/component modification during each delta step to avoid recording unnecessary overhead into the transaction buffer
		GEditor->DisableDeltaModification(true);
	}

	GUnrealEd->ComponentVisManager.TrackingStarted(this);
	
	return true;
}

bool FLevelEditorViewportClient::EndTransform(const FGizmoState& InState)
{
	if (!bHasBegunGizmoManipulation)
	{
		return false;
	}
	
	bDuplicateOnNextDrag = false;

	// here we check to see if anything of worth actually changed when ending our MouseMovement
	// If the TransCount > 0 (we changed something of value) so we need to call PostEditMove() on stuff
	// if we didn't change anything then don't call PostEditMove()
	bool bDidAnythingActuallyChange = false;

	if( TrackingTransaction.TransCount > 0 )
	{
		bDidAnythingActuallyChange = true;
		TrackingTransaction.TransCount--;
	}

	// TODO ensure that the gizmo actually moved
	const bool bDidMove = bDidAnythingActuallyChange;
	const FTypedElementListConstRef ElementsToManipulate = GetElementsToManipulate();

	if (bHasBegunGizmoManipulation)
	{
		auto GetManipType = [bDidMove]()
		{
			if (bDidMove)
			{
				return ETypedElementViewportInteractionGizmoManipulationType::Drag; 
			}
		   return ETypedElementViewportInteractionGizmoManipulationType::Click;
		};
		ViewportInteraction->EndGizmoManipulation(ElementsToManipulate, GetWidgetMode(), GetManipType());
		bHasBegunGizmoManipulation = false;
	}

	if (bDidMove && !GUnrealEd->IsPivotMovedIndependently())
	{
		GUnrealEd->UpdatePivotLocationForSelection();
	}

	GUnrealEd->ComponentVisManager.TrackingStopped(this, bDidMove);

	if (bNeedToRestoreComponentBeingMovedFlag)
	{
		TypedElementListObjectUtil::ForEachObject<AActor>(ElementsToManipulate, [this](AActor* InActor)
		{
			SetActorBeingMovedByEditor(InActor, false);
			return true;
		});

		bNeedToRestoreComponentBeingMovedFlag = false;
	}

	// End the transaction here if one was started in StartTransaction()
	if( TrackingTransaction.IsActive() || TrackingTransaction.IsPending() )
	{
		if (!HaveSelectedObjectsBeenChanged())
		{
			TrackingTransaction.Cancel();
		}
		else
		{
			TrackingTransaction.End();
		}
	
		// Restore actor/component delta modification
		GEditor->DisableDeltaModification(false);
	}

	ModeTools->ActorMoveNotify();

	if (bDidAnythingActuallyChange)
	{
		FScopedLevelDirtied LevelDirtyCallback;
		LevelDirtyCallback.Request();

		RedrawAllViewportsIntoThisScene();
	}

	PreDragElementTransforms.Empty();
	MouseDeltaTracker->SetExternalMovement(false);

	return true;
}

bool FLevelEditorViewportClient::OnTransformCanceled(const FGizmoState& InState)
{
	GUnrealEd->UpdatePivotLocationForSelection();
	return true;
}

/**
 * Static: Adds a hover effect to the specified object
 *
 * @param	InHoverTarget	The hoverable object to add the effect to
 */
void FLevelEditorViewportClient::AddHoverEffect( const FViewportHoverTarget& InHoverTarget )
{
	AActor* ActorUnderCursor = InHoverTarget.HoveredActor;
	UModel* ModelUnderCursor = InHoverTarget.HoveredModel;

	if( ActorUnderCursor != nullptr )
	{
		for (UActorComponent* Component : ActorUnderCursor->GetComponents())
		{
			UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Component);
			if (PrimitiveComponent && PrimitiveComponent->IsRegistered())
			{
				PrimitiveComponent->PushHoveredToProxy( true );
			}
		}
	}
	else if (ModelUnderCursor != nullptr)
	{
		check( InHoverTarget.ModelSurfaceIndex != INDEX_NONE );
		check( InHoverTarget.ModelSurfaceIndex < (uint32)ModelUnderCursor->Surfs.Num() );
		FBspSurf& Surf = ModelUnderCursor->Surfs[ InHoverTarget.ModelSurfaceIndex ];
		Surf.PolyFlags |= PF_Hovered;
	}
}


/**
 * Static: Removes a hover effect from the specified object
 *
 * @param	InHoverTarget	The hoverable object to remove the effect from
 */
void FLevelEditorViewportClient::RemoveHoverEffect( const FViewportHoverTarget& InHoverTarget )
{
	AActor* CurHoveredActor = InHoverTarget.HoveredActor;
	if( CurHoveredActor != nullptr )
	{
		for (UActorComponent* Component : CurHoveredActor->GetComponents())
		{
			UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Component);
			if (PrimitiveComponent && PrimitiveComponent->IsRegistered())
			{
				check(PrimitiveComponent->IsRegistered());
				PrimitiveComponent->PushHoveredToProxy( false );
			}
		}
	}

	UModel* CurHoveredModel = InHoverTarget.HoveredModel;
	if( CurHoveredModel != nullptr )
	{
		if( InHoverTarget.ModelSurfaceIndex != INDEX_NONE &&
			(uint32)CurHoveredModel->Surfs.Num() >= InHoverTarget.ModelSurfaceIndex )
		{
			FBspSurf& Surf = CurHoveredModel->Surfs[ InHoverTarget.ModelSurfaceIndex ];
			Surf.PolyFlags &= ~PF_Hovered;
		}
	}
}


/**
 * Static: Clears viewport hover effects from any objects that currently have that
 */
void FLevelEditorViewportClient::ClearHoverFromObjects()
{
	// Clear hover feedback for any actor's that were previously drawing a hover cue
	if( HoveredObjects.Num() > 0 )
	{
		for( TSet<FViewportHoverTarget>::TIterator It( HoveredObjects ); It; ++It )
		{
			FViewportHoverTarget& CurHoverTarget = *It;
			RemoveHoverEffect( CurHoverTarget );
		}

		HoveredObjects.Empty();
	}
}

void FLevelEditorViewportClient::OnWidgetModeChanged(UE::Widget::EWidgetMode NewMode)
{
	// We need this in case the current selection allowed the previous widget mode but not the new one or vice versa.
	ResetElementsToManipulate();
}

void FLevelEditorViewportClient::OnEditorCleanse()
{
	ClearHoverFromObjects();

	FSceneViewStateInterface* ViewStateInterface = ViewState.GetReference();
	if (ViewStateInterface)
	{
		// The view state can reference materials from the world being cleaned up. (Example post process materials)
		ViewStateInterface->ClearMIDPool();
	}

}

void FLevelEditorViewportClient::OnPreBeginPIE(const bool bIsSimulating)
{
	// Called before PIE attempts to start, allowing the viewport to cancel processes, like dragging, that will block PIE from beginning
	AbortTracking();
}

bool FLevelEditorViewportClient::GetSpriteCategoryVisibility( const FName& InSpriteCategory ) const
{
	const int32 CategoryIndex = GEngine->GetSpriteCategoryIndex( InSpriteCategory );
	check( CategoryIndex != INDEX_NONE && CategoryIndex < SpriteCategoryVisibility.Num() );

	return SpriteCategoryVisibility[ CategoryIndex ];
}

bool FLevelEditorViewportClient::GetSpriteCategoryVisibility( int32 Index ) const
{
	check( Index >= 0 && Index < SpriteCategoryVisibility.Num() );
	return SpriteCategoryVisibility[ Index ];
}

void FLevelEditorViewportClient::SetSpriteCategoryVisibility( const FName& InSpriteCategory, bool bVisible )
{
	const int32 CategoryIndex = GEngine->GetSpriteCategoryIndex( InSpriteCategory );
	check( CategoryIndex != INDEX_NONE && CategoryIndex < SpriteCategoryVisibility.Num() );

	SpriteCategoryVisibility[ CategoryIndex ] = bVisible;
}

void FLevelEditorViewportClient::SetSpriteCategoryVisibility( int32 Index, bool bVisible )
{
	check( Index >= 0 && Index < SpriteCategoryVisibility.Num() );
	SpriteCategoryVisibility[ Index ] = bVisible;
}

void FLevelEditorViewportClient::SetAllSpriteCategoryVisibility( bool bVisible )
{
	SpriteCategoryVisibility.Init( bVisible, SpriteCategoryVisibility.Num() );
}

UWorld* FLevelEditorViewportClient::GetWorld() const
{
	if (bIsSimulateInEditorViewport)
	{
		return GEditor->PlayWorld;
	}
	else if (World)
	{
		return World;
	}
	return FEditorViewportClient::GetWorld();
}

void FLevelEditorViewportClient::SetReferenceToWorldContext(FWorldContext& WorldContext)
{
	WorldContext.AddRef(World);
}

void FLevelEditorViewportClient::RemoveReferenceToWorldContext(FWorldContext& WorldContext)
{
	WorldContext.RemoveRef(World);
}

void FLevelEditorViewportClient::SetIsSimulateInEditorViewport( bool bInIsSimulateInEditorViewport )
{ 
	bIsSimulateInEditorViewport = bInIsSimulateInEditorViewport; 
}

bool FLevelEditorViewportClient::GetPivotForOrbit(FVector& Pivot) const
{
	if (FEditorViewportClient::GetPivotForOrbit(Pivot))
	{
		return true;
	}

	FBox BoundingBox(ForceInit);
	int32 NumValidComponents = 0;
	for (FSelectionIterator It(GEditor->GetSelectedComponentIterator()); It; ++It)
	{
		// Allow orbiting on selected SceneComponents
		USceneComponent* Component = Cast<USceneComponent>(*It);
		if (Component && Component->IsRegistered())
		{
			TSharedPtr<FComponentVisualizer> Visualizer = GUnrealEd->FindComponentVisualizer(Component->GetClass());
			FBox FocusOnSelectionBBox;
			if (Visualizer && Visualizer->HasFocusOnSelectionBoundingBox(FocusOnSelectionBBox))
			{
				BoundingBox += FocusOnSelectionBBox;
			}
			else
			{
			// It's possible that it doesn't have a bounding box, so just take its position in that case
			FBox ComponentBBox = Component->Bounds.GetBox();
			if (ComponentBBox.GetVolume() != 0)
			{
				BoundingBox += ComponentBBox;
			}
			else
			{
				BoundingBox += Component->GetComponentLocation();
			}
			}
			++NumValidComponents;
		}
	}

	if (NumValidComponents > 0)
	{
		Pivot = BoundingBox.GetCenter();
		return true;
	}

	// Use the center of the bounding box of the current selected actors, if any, as the pivot point for orbiting the camera
	int32 NumSelectedActors = 0;
	for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
	{
		if (AActor* Actor = Cast<AActor>(*It))
		{
			TInlineComponentArray<UPrimitiveComponent*> PrimitiveComponents(Actor);

			for(int32 ComponentIndex = 0; ComponentIndex < PrimitiveComponents.Num(); ++ComponentIndex)
			{
				UPrimitiveComponent* PrimitiveComponent = PrimitiveComponents[ComponentIndex];

				if (PrimitiveComponent->IsRegistered() && !PrimitiveComponent->GetIgnoreBoundsForEditorFocus())
				{
					TSharedPtr<FComponentVisualizer> Visualizer = GUnrealEd->FindComponentVisualizer(PrimitiveComponent->GetClass());
					FBox FocusOnSelectionBBox;
					if (Visualizer && Visualizer->HasFocusOnSelectionBoundingBox(FocusOnSelectionBBox))
					{
						BoundingBox += FocusOnSelectionBBox;
					}
					else
					{
					BoundingBox += PrimitiveComponent->Bounds.GetBox();
					}
					++NumSelectedActors;
				}
			}
		}
	}

	if (NumSelectedActors > 0)
	{
		Pivot = BoundingBox.GetCenter();
		return true;
	}

	return false;
}

void FLevelEditorViewportClient::SetEditingThroughMovementWidget()
{
	bCurrentlyEditingThroughMovementWidget = true;
}

void FLevelEditorViewportClient::DuplicateSelection(bool bInSelectNewElements)
{
	TSharedPtr<ILevelEditor> LevelEditor = ParentLevelEditor.Pin();
	if (!LevelEditor)
	{
		return;
	}

	if (UTypedElementCommonActions* CommonActions = LevelEditor->GetCommonActions())
	{
		FToolBuilderState State;
		if (UTypedElementSelectionSet* MutableSelectionSet = GetMutableSelectionSet())
		{
			// Do not use the cached manipulation list here, as it will have removed attachments, and we do want to duplicate those
			const TArray<FTypedElementHandle> DuplicatedElements = CommonActions->DuplicateSelectedElements(
				GetSelectionSet(), LevelEditor->GetWorld(), FVector::ZeroVector
			);

			if (DuplicatedElements.Num() > 0)
			{
				// (assuming chosen modifier key is Alt)
				// Alt Duplicate has 2 modes:
				// 1 - with Alt pressed, dragging will create and select a copy, then drag it
				// 2 - while dragging, hitting Alt will crate a copy, NOT selecting it

				// Select newly created elements: Alt + Drag only
				if (bInSelectNewElements)
				{
					// We need to end the gizmo manipulation on the current selection, as they won't be moved again this drag...
					if (bHasBegunGizmoManipulation)
					{
						ViewportInteraction->EndGizmoManipulation(GetElementsToManipulate(), GetWidgetMode(), ETypedElementViewportInteractionGizmoManipulationType::Click);
						// Note: We don't reset bHasBegunGizmoManipulation here, as we test it again below to begin manipulating the duplicated selection
					}

					// Select the newly created elements
					const FTypedElementSelectionOptions SelectionOptions = FTypedElementSelectionOptions()
										.SetAllowLegacyNotifications(false); // Old drag duplicate code didn't used to notify about this selection change

					MutableSelectionSet->SetSelection(DuplicatedElements, SelectionOptions);
					MutableSelectionSet->NotifyPendingChanges();

					// Force the cached manipulation list to update, as we don't notify about the change above
					ResetElementsToManipulate();

					// We need to start gizmo manipulation on the duplicated selection, as they may continue to move as part of this drag duplicate
					if (bHasBegunGizmoManipulation)
					{
						ViewportInteraction->BeginGizmoManipulation(GetElementsToManipulate(), GetWidgetMode());
					}
				}

				// Notify the global mode tools, the selection set should be identical to the new actors at this point
				TArray<AActor*> SelectedActors = MutableSelectionSet->GetSelectedObjects<AActor>();
				constexpr bool bDidOffsetDuplicate = false;
				GLevelEditorModeTools().ActorsDuplicatedNotify(SelectedActors, SelectedActors, bDidOffsetDuplicate);
			}

			// Invalidate all viewports, so the new gizmo is rendered in each one
			RedrawAllViewportsIntoThisScene();

			// Prevent issues with dragged elements sometimes being set back to their starting position after stamping the duplicate
			PreDragElementTransforms.Empty();
		}
	}
}

#undef LOCTEXT_NAMESPACE
