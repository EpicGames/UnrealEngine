// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ImportVolumetricLightmap.h
=============================================================================*/

#include "Lightmass/Lightmass.h"
#include "EngineDefines.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "PrecomputedVolumetricLightmap.h"
#include "Engine/MapBuildDataRegistry.h"
#include "ImportExport.h"
#include "UnrealEngine.h"
#include "Logging/TokenizedMessage.h"
#include "Logging/MessageLog.h"
#include "RenderUtils.h"
#include "WorldPartition/StaticLightingData/VolumetricLightmapGrid.h"

#define LOCTEXT_NAMESPACE "ImportVolumetricLightmap"

DEFINE_LOG_CATEGORY_STATIC(LogVolumetricLightmapImport, Log, All);

FVector GDebugVoxelPosition = FVector::ZeroVector;

void CopyBrickToAtlasVolumeTexture(int32 FormatSize, FIntVector AtlasSize, FIntVector BrickMin, FIntVector BrickSize, const uint8* RESTRICT SourceData, uint8* RESTRICT DestData)
{
	const int32 SourcePitch = BrickSize.X * FormatSize;
	const int32 Pitch = AtlasSize.X * FormatSize;
	const int32 DepthPitch = AtlasSize.X * AtlasSize.Y * FormatSize;

	// Copy each row into the correct position in the global volume texture
	for (int32 ZIndex = 0; ZIndex < BrickSize.Z; ZIndex++)
	{
		const int32 DestZIndex = (BrickMin.Z + ZIndex) * DepthPitch + BrickMin.X * FormatSize;
		const int32 SourceZIndex = ZIndex * BrickSize.Y * SourcePitch;

		for (int32 YIndex = 0; YIndex < BrickSize.Y; YIndex++)
		{
			const int32 DestIndex = DestZIndex + (BrickMin.Y + YIndex) * Pitch;
			const int32 SourceIndex = SourceZIndex + YIndex * SourcePitch;
			FMemory::Memcpy((uint8*)&DestData[DestIndex], (const uint8*)&SourceData[SourceIndex], SourcePitch);
		}
	}
}

void CopyBetweenAtlasVolumeTextures(
	int32 FormatSize,
	FIntVector BrickSize,
	FIntVector SourceAtlasSize,
	FIntVector SourceBrickMin,
	const TArray<uint8>& SourceData,
	FIntVector DestAtlasSize,
	FIntVector DestBrickMin,
	TArray<uint8>& DestData)
{
	const int32 DestPitch = DestAtlasSize.X * FormatSize;
	const int32 DestDepthPitch = DestAtlasSize.X * DestAtlasSize.Y * FormatSize;
	const int32 SourcePitch = SourceAtlasSize.X * FormatSize;
	const int32 SourceDepthPitch = SourceAtlasSize.X * SourceAtlasSize.Y * FormatSize;

	// Copy each row into the correct position in the global volume texture
	for (int32 ZIndex = 0; ZIndex < BrickSize.Z; ZIndex++)
	{
		const int32 DestZIndex = (DestBrickMin.Z + ZIndex) * DestDepthPitch + DestBrickMin.X * FormatSize;
		const int32 SourceZIndex = (SourceBrickMin.Z + ZIndex) * SourceDepthPitch + SourceBrickMin.X * FormatSize;

		for (int32 YIndex = 0; YIndex < BrickSize.Y; YIndex++)
		{
			const int32 DestIndex = DestZIndex + (DestBrickMin.Y + YIndex) * DestPitch;
			const int32 SourceIndex = SourceZIndex + (SourceBrickMin.Y + YIndex) * SourcePitch;
			FMemory::Memcpy((uint8*)&DestData[DestIndex], (const uint8*)&SourceData[SourceIndex], BrickSize.X * FormatSize);
		}
	}
}

static int32 ComputeLinearVoxelIndex(FIntVector VoxelCoordinate, FIntVector VolumeDimensions)
{
	return (VoxelCoordinate.Z * VolumeDimensions.Y + VoxelCoordinate.Y) * VolumeDimensions.X + VoxelCoordinate.X;
}

struct FImportedVolumetricLightmapBrick
{
	FGuid IntersectingLevelGuid;
	FIntVector IndirectionTexturePosition;
	int32 TreeDepth;
	float AverageClosestGeometryDistance;
	TArray<FFloat3Packed> AmbientVector;
	TArray<FColor> SHCoefficients[6];
	TArray<FColor> SkyBentNormal;
	TArray<uint8> DirectionalLightShadowing;
	TArray<Lightmass::FIrradianceVoxelImportProcessingData> TaskVoxelImportProcessingData;
};

struct FImportedVolumetricLightmapTaskData
{
	TArray<FImportedVolumetricLightmapBrick> Bricks;
};

inline FIntVector ComputeBrickLayoutPosition(int32 BrickLayoutAllocation, FIntVector BrickLayoutDimensions)
{
	const FIntVector BrickPosition(
		BrickLayoutAllocation % BrickLayoutDimensions.X,
		(BrickLayoutAllocation / BrickLayoutDimensions.X) % BrickLayoutDimensions.Y,
		BrickLayoutAllocation / (BrickLayoutDimensions.X * BrickLayoutDimensions.Y));

	return BrickPosition;
}

bool CopyFromBrickmapTexel(
	FVector IndirectionDataSourceCoordinate,
	FIntVector LocalCellDestCoordinate, 
	int32 MinDestinationNumBottomLevelBricks,
	int32 BrickSize, 
	FIntVector BrickLayoutPosition,
	const FPrecomputedVolumetricLightmapData& CurrentLevelData,
	FVolumetricLightmapBrickData& BrickData)
{
	const FVector IndirectionCoordMax(FVector(CurrentLevelData.IndirectionTextureDimensions) * (1 - GPointFilteringThreshold));

	if (IndirectionDataSourceCoordinate.X < 0 || IndirectionDataSourceCoordinate.Y < 0 || IndirectionDataSourceCoordinate.Z < 0 || 
		IndirectionDataSourceCoordinate.X > IndirectionCoordMax.X || IndirectionDataSourceCoordinate.Y > IndirectionCoordMax.Y || IndirectionDataSourceCoordinate.Z > IndirectionCoordMax.Z)
	{
		return false;
	}

	FIntVector IndirectionBrickOffset;
	int32 IndirectionBrickSize;

	checkSlow(GPixelFormats[CurrentLevelData.IndirectionTexture.Format].BlockBytes == sizeof(uint8) * 4);
	SampleIndirectionTexture(IndirectionDataSourceCoordinate, CurrentLevelData.IndirectionTextureDimensions, CurrentLevelData.IndirectionTexture.Data.GetData(), IndirectionBrickOffset, IndirectionBrickSize);

	if (IndirectionBrickSize > MinDestinationNumBottomLevelBricks)
	{
		const FVector BrickTextureCoordinate = ComputeBrickTextureCoordinate(IndirectionDataSourceCoordinate, IndirectionBrickOffset, IndirectionBrickSize, BrickSize);

		const FIntVector DestCellPosition = BrickLayoutPosition + LocalCellDestCoordinate;
		const int32 LinearDestCellIndex = ComputeLinearVoxelIndex(DestCellPosition, CurrentLevelData.BrickDataDimensions);

		*(FFloat3Packed*)&BrickData.AmbientVector.Data[LinearDestCellIndex * sizeof(FFloat3Packed)] = FilteredVolumeLookupReconverted<FFloat3Packed>(BrickTextureCoordinate, CurrentLevelData.BrickDataDimensions, (const FFloat3Packed*)BrickData.AmbientVector.Data.GetData());
		
		for (int32 i = 0; i < UE_ARRAY_COUNT(BrickData.SHCoefficients); i++)
		{
			*(FColor*)&BrickData.SHCoefficients[i].Data[LinearDestCellIndex * sizeof(FColor)] = FilteredVolumeLookupReconverted<FColor>(BrickTextureCoordinate, CurrentLevelData.BrickDataDimensions, (const FColor*)BrickData.SHCoefficients[i].Data.GetData());
		}

		if (BrickData.SkyBentNormal.Data.Num() > 0)
		{
			*(FColor*)&BrickData.SkyBentNormal.Data[LinearDestCellIndex * sizeof(FColor)] = FilteredVolumeLookupReconverted<FColor>(BrickTextureCoordinate, CurrentLevelData.BrickDataDimensions, (const FColor*)BrickData.SkyBentNormal.Data.GetData());
		}
	
		*(uint8*)&BrickData.DirectionalLightShadowing.Data[LinearDestCellIndex * sizeof(uint8)] = FilteredVolumeLookupReconverted<uint8>(BrickTextureCoordinate, CurrentLevelData.BrickDataDimensions, (const uint8*)BrickData.DirectionalLightShadowing.Data.GetData());

		return true;
	}

	return false;
}

static NSwarm::TChannelFlags LM_VOLUMETRICLIGHTMAP_CHANNEL_FLAGS = NSwarm::SWARM_JOB_CHANNEL_READ;

void FLightmassProcessor::ImportIrradianceTasks(bool& bGenerateSkyShadowing, TArray<FImportedVolumetricLightmapTaskData>& TaskDataArray)
{
	TList<FGuid>* Element = CompletedVolumetricLightmapTasks.ExtractAll();

	while (Element)
	{
		// If this task has not already been imported, import it now
		TList<FGuid>* NextElement = Element->Next;

		const FString ChannelName = Lightmass::CreateChannelName(Element->Element, Lightmass::LM_VOLUMETRICLIGHTMAP_VERSION, Lightmass::LM_VOLUMETRICLIGHTMAP_EXTENSION);
		const int32 Channel = Swarm.OpenChannel( *ChannelName, LM_VOLUMETRICLIGHTMAP_CHANNEL_FLAGS );
		if (Channel >= 0)
		{
			TaskDataArray.AddDefaulted();
			FImportedVolumetricLightmapTaskData& NewTaskData = TaskDataArray.Last();

			int32 NumBricks;
			Swarm.ReadChannel(Channel, &NumBricks, sizeof(NumBricks));
			NewTaskData.Bricks.Empty(NumBricks);

			for (int32 BrickIndex = 0; BrickIndex < NumBricks; BrickIndex++)
			{
				NewTaskData.Bricks.AddDefaulted();
				FImportedVolumetricLightmapBrick& NewBrick = NewTaskData.Bricks.Last();
				Swarm.ReadChannel(Channel, &NewBrick.IntersectingLevelGuid, sizeof(NewBrick.IntersectingLevelGuid));
				Swarm.ReadChannel(Channel, &NewBrick.IndirectionTexturePosition, sizeof(NewBrick.IndirectionTexturePosition));
				Swarm.ReadChannel(Channel, &NewBrick.TreeDepth, sizeof(NewBrick.TreeDepth));
				Swarm.ReadChannel(Channel, &NewBrick.AverageClosestGeometryDistance, sizeof(NewBrick.AverageClosestGeometryDistance));
				ReadArray(Channel, NewBrick.AmbientVector);

				for (int32 i = 0; i < UE_ARRAY_COUNT(NewBrick.SHCoefficients); i++)
				{
					ReadArray(Channel, NewBrick.SHCoefficients[i]);
				}

				ReadArray(Channel, NewBrick.SkyBentNormal);
				ReadArray(Channel, NewBrick.DirectionalLightShadowing);

				bGenerateSkyShadowing = bGenerateSkyShadowing || NewBrick.SkyBentNormal.Num() > 0;

				ReadArray(Channel, NewBrick.TaskVoxelImportProcessingData);
			}

			ImportDebugOutputStruct(Channel);

			Swarm.CloseChannel(Channel);
		}
		else
		{
			UE_LOGF(LogVolumetricLightmapImport, Fatal,  "Error, failed to import volumetric lightmap %ls with error code %d", *ChannelName, Channel );
		}

		delete Element;
		Element = NextElement;
	}
}

// One pass needed to cover the trilinear filtering footprint, another pass needed to cover exterior voxels which see backfaces due to the large ray start bias.
int32 NumDilateOverEmbeddedVoxelsPasses = 2;

struct FFilteredBrickData
{
	FFloat3Packed AmbientVector;
	FColor SHCoefficients[UE_ARRAY_COUNT(FImportedVolumetricLightmapBrick::SHCoefficients)];
	uint8 DirectionalLightShadowing;
};

void FilterWithNeighbors(
	const TArray<const FImportedVolumetricLightmapBrick*>& BricksAtCurrentDepth,
	int32 BrickStartAllocation,
	int32 CurrentDepth,
	FIntVector BrickLayoutDimensions,
	const Lightmass::FVolumetricLightmapSettings& VolumetricLightmapSettings,
	FPrecomputedVolumetricLightmapData& CurrentLevelData,
	const TArray<Lightmass::FIrradianceVoxelImportProcessingData>& VoxelImportProcessingData,
	TArray<Lightmass::FIrradianceVoxelImportProcessingData>& NewVoxelImportProcessingData)
{
	int32 BrickSize = VolumetricLightmapSettings.BrickSize;
	int32 PaddedBrickSize = BrickSize + 1;
	const int32 BrickSizeLog2 = FMath::FloorLog2(BrickSize);
	const float InvBrickSize = 1.0f / BrickSize;

	const FBox VolumeBox(VolumetricLightmapSettings.VolumeMin, VolumetricLightmapSettings.VolumeMin + VolumetricLightmapSettings.VolumeSize);
	const FVector DebugPositionIndirectionCoordinate = ComputeIndirectionCoordinate(GDebugVoxelPosition, VolumeBox, CurrentLevelData.IndirectionTextureDimensions);

	TArray<FFilteredBrickData> FilteredBrickData;
	TArray<bool> FilteredBrickDataValid;

	FilteredBrickData.Empty(BrickSize * BrickSize * BrickSize);
	FilteredBrickData.AddZeroed(BrickSize * BrickSize * BrickSize);
	FilteredBrickDataValid.Empty(BrickSize * BrickSize * BrickSize);
	FilteredBrickDataValid.AddZeroed(BrickSize * BrickSize * BrickSize);

	// Fill in voxels which are inside geometry with their valid neighbors
	for (int32 BrickIndex = 0; BrickIndex < BricksAtCurrentDepth.Num(); BrickIndex++)
	{
		const FImportedVolumetricLightmapBrick& Brick = *BricksAtCurrentDepth[BrickIndex];

		// Initialize temporary brick data to invalid
		FPlatformMemory::Memzero(FilteredBrickDataValid.GetData(), FilteredBrickDataValid.Num() * FilteredBrickDataValid.GetTypeSize());

		checkSlow(Brick.TreeDepth == CurrentDepth);

		const int32 DetailCellsPerCurrentLevelBrick = 1 << ((VolumetricLightmapSettings.MaxRefinementLevels - Brick.TreeDepth) * BrickSizeLog2);
		const int32 NumBottomLevelBricks = DetailCellsPerCurrentLevelBrick / BrickSize;
		const FVector IndirectionTexturePosition = FVector(Brick.IndirectionTexturePosition);
		const FIntVector BrickLayoutPosition = ComputeBrickLayoutPosition(BrickStartAllocation + BrickIndex, BrickLayoutDimensions) * PaddedBrickSize;

		for (int32 Z = 0; Z < BrickSize; Z++)
		{
			for (int32 Y = 0; Y < BrickSize; Y++)
			{
				for (int32 X = 0; X < BrickSize; X++)
				{
					FIntVector VoxelCoordinate(X, Y, Z);

					const int32 LinearDestCellIndex = ComputeLinearVoxelIndex(VoxelCoordinate + BrickLayoutPosition, CurrentLevelData.BrickDataDimensions);
					Lightmass::FIrradianceVoxelImportProcessingData VoxelImportData = VoxelImportProcessingData[LinearDestCellIndex];

					if (GDebugVoxelPosition != FVector::ZeroVector)
					{
						const FVector CellIndirectionTexturePosition = IndirectionTexturePosition + FVector(X, Y, Z) * InvBrickSize * NumBottomLevelBricks;
						const FBox CellIndirectionTextureBounds(CellIndirectionTexturePosition, CellIndirectionTexturePosition + InvBrickSize * NumBottomLevelBricks);

						if (CellIndirectionTextureBounds.IsInside(DebugPositionIndirectionCoordinate))
						{
							int32 DebugBreakpoint = 0;
						}
					}

					if (VoxelImportData.bInsideGeometry
						// Don't modify border voxels
						&& !VoxelImportData.bBorderVoxel)
					{
						//@todo - filter SkyBentNormal from neighbors too
						FLinearColor AmbientVector = FLinearColor(0, 0, 0, 0);
						FLinearColor SHCoefficients[UE_ARRAY_COUNT(Brick.SHCoefficients)];
						FLinearColor DirectionalLightShadowing = FLinearColor(0, 0, 0, 0);

						for (int32 i = 0; i < UE_ARRAY_COUNT(SHCoefficients); i++)
						{
							SHCoefficients[i] = FLinearColor(0, 0, 0, 0);
						}
	
						float TotalWeight = 0.0f;

						for (int32 NeighborZ = -1; NeighborZ <= 1; NeighborZ++)
						{
							for (int32 NeighborY = -1; NeighborY <= 1; NeighborY++)
							{
								for (int32 NeighborX = -1; NeighborX <= 1; NeighborX++)
								{
									const FVector NeighborIndirectionDataSourceCoordinate = IndirectionTexturePosition + FVector(X + NeighborX, Y + NeighborY, Z + NeighborZ) * InvBrickSize * NumBottomLevelBricks;
									const FIntVector NeighborVoxelCoordinate(X + NeighborX, Y + NeighborY, Z + NeighborZ);

									if (NeighborVoxelCoordinate != VoxelCoordinate
										&& NeighborIndirectionDataSourceCoordinate.X >= 0.0f 
										&& NeighborIndirectionDataSourceCoordinate.Y >= 0.0f 
										&& NeighborIndirectionDataSourceCoordinate.Z >= 0.0f
										&& NeighborIndirectionDataSourceCoordinate.X < CurrentLevelData.IndirectionTextureDimensions.X 
										&& NeighborIndirectionDataSourceCoordinate.Y < CurrentLevelData.IndirectionTextureDimensions.Y 
										&& NeighborIndirectionDataSourceCoordinate.Z < CurrentLevelData.IndirectionTextureDimensions.Z)
									{
										FIntVector IndirectionBrickOffset;
										int32 IndirectionBrickSize;

										checkSlow(GPixelFormats[CurrentLevelData.IndirectionTexture.Format].BlockBytes == sizeof(uint8) * 4);
										SampleIndirectionTexture(NeighborIndirectionDataSourceCoordinate, CurrentLevelData.IndirectionTextureDimensions, CurrentLevelData.IndirectionTexture.Data.GetData(), IndirectionBrickOffset, IndirectionBrickSize);

										// Only filter from bricks with equal density, to avoid reading from uninitialized padding
										// This causes seams but they fall at density transitions so not noticeable
										if (IndirectionBrickSize == NumBottomLevelBricks)
										{
											const FVector BrickTextureCoordinate = ComputeBrickTextureCoordinate(NeighborIndirectionDataSourceCoordinate, IndirectionBrickOffset, IndirectionBrickSize, BrickSize);
											Lightmass::FIrradianceVoxelImportProcessingData NeighborVoxelImportData = NearestVolumeLookup<Lightmass::FIrradianceVoxelImportProcessingData>(BrickTextureCoordinate, CurrentLevelData.BrickDataDimensions, VoxelImportProcessingData.GetData());

											if (!NeighborVoxelImportData.bInsideGeometry && !NeighborVoxelImportData.bBorderVoxel)
											{
												const float NeighborTotal = static_cast<float>(FMath::Abs(NeighborX) + FMath::Abs(NeighborY) + FMath::Abs(NeighborZ));
												const float Weight = 1.0f / FMath::Max<float>(NeighborTotal, .5f);
												FLinearColor NeighborAmbientVector = FilteredVolumeLookup<FFloat3Packed>(BrickTextureCoordinate, CurrentLevelData.BrickDataDimensions, (const FFloat3Packed*)CurrentLevelData.BrickData.AmbientVector.Data.GetData());
												AmbientVector += NeighborAmbientVector * Weight;

												for (int32 i = 0; i < UE_ARRAY_COUNT(SHCoefficients); i++)
												{
													static_assert(UE_ARRAY_COUNT(SHCoefficients) == 6, "Assuming 2 SHCoefficient vectors per color channel");

													// Weight by ambient before filtering, normalized SH coefficients don't filter properly
													float AmbientCoefficient = NeighborAmbientVector.Component(i / 2);
													SHCoefficients[i] += AmbientCoefficient * Weight * FilteredVolumeLookup<FColor>(BrickTextureCoordinate, CurrentLevelData.BrickDataDimensions, (const FColor*)CurrentLevelData.BrickData.SHCoefficients[i].Data.GetData());
												}

												FLinearColor NeighborDirectionalLightShadowing = FilteredVolumeLookup<uint8>(BrickTextureCoordinate, CurrentLevelData.BrickDataDimensions, (const uint8*)CurrentLevelData.BrickData.DirectionalLightShadowing.Data.GetData());
												DirectionalLightShadowing += NeighborDirectionalLightShadowing * Weight;

												TotalWeight += Weight;
											}
										}
									}
								}
							}
						}

						if (TotalWeight > 0.0f)
						{
							const int32 LinearVoxelIndex = ComputeLinearVoxelIndex(VoxelCoordinate, FIntVector(BrickSize, BrickSize, BrickSize));

							// Store filtered output to temporary brick data to avoid order dependent results between voxels
							// This still produces order dependent filtering between neighboring bricks
							FilteredBrickDataValid[LinearVoxelIndex] = true;

							const float InvTotalWeight = 1.0f / TotalWeight;

							const FLinearColor FilteredAmbientColor = AmbientVector * InvTotalWeight;
							FilteredBrickData[LinearVoxelIndex].AmbientVector = ConvertFromLinearColor<FFloat3Packed>(FilteredAmbientColor);

							for (int32 i = 0; i < UE_ARRAY_COUNT(SHCoefficients); i++)
							{
								static_assert(UE_ARRAY_COUNT(SHCoefficients) == 6, "Assuming 2 SHCoefficient vectors per color channel");

								float AmbientCoefficient = FMath::Max(FilteredAmbientColor.Component(i / 2), KINDA_SMALL_NUMBER);

								FilteredBrickData[LinearVoxelIndex].SHCoefficients[i] = ConvertFromLinearColor<FColor>(SHCoefficients[i] * InvTotalWeight / AmbientCoefficient);
							}

							FilteredBrickData[LinearVoxelIndex].DirectionalLightShadowing = ConvertFromLinearColor<uint8>(DirectionalLightShadowing * InvTotalWeight);
						}
					}
				}
			}
		}

		for (int32 Z = 0; Z < BrickSize; Z++)
		{
			for (int32 Y = 0; Y < BrickSize; Y++)
			{
				for (int32 X = 0; X < BrickSize; X++)
				{
					FIntVector VoxelCoordinate(X, Y, Z);
					const int32 LinearVoxelIndex = ComputeLinearVoxelIndex(VoxelCoordinate, FIntVector(BrickSize, BrickSize, BrickSize));

					// Write filtered voxel data back to the original
					if (FilteredBrickDataValid[LinearVoxelIndex])
					{
						const int32 LinearBrickLayoutCellIndex = ComputeLinearVoxelIndex(VoxelCoordinate + BrickLayoutPosition, CurrentLevelData.BrickDataDimensions);

						// Mark as valid for future passes now that we've overwritten with filtered neighbors
						NewVoxelImportProcessingData[LinearBrickLayoutCellIndex].bInsideGeometry = false;

						FFloat3Packed* DestAmbientVector = (FFloat3Packed*)&CurrentLevelData.BrickData.AmbientVector.Data[LinearBrickLayoutCellIndex * sizeof(FFloat3Packed)];
						*DestAmbientVector = FilteredBrickData[LinearVoxelIndex].AmbientVector;

						for (int32 i = 0; i < UE_ARRAY_COUNT(FilteredBrickData[LinearVoxelIndex].SHCoefficients); i++)
						{
							FColor* DestCoefficients = (FColor*)&CurrentLevelData.BrickData.SHCoefficients[i].Data[LinearBrickLayoutCellIndex * sizeof(FColor)];
							*DestCoefficients = FilteredBrickData[LinearVoxelIndex].SHCoefficients[i];
						}

						uint8* DestDirectionalLightShadowing = (uint8*)&CurrentLevelData.BrickData.DirectionalLightShadowing.Data[LinearBrickLayoutCellIndex * sizeof(uint8)];
						*DestDirectionalLightShadowing = FilteredBrickData[LinearVoxelIndex].DirectionalLightShadowing;
					}
				}
			}
		}
	}
}

void StitchDetailBricksWithLowDensityNeighbors(
	const TArray<const FImportedVolumetricLightmapBrick*>& BricksAtCurrentDepth,
	int32 BrickStartAllocation,
	int32 CurrentDepth,
	FIntVector BrickLayoutDimensions,
	const Lightmass::FVolumetricLightmapSettings& VolumetricLightmapSettings,
	FPrecomputedVolumetricLightmapData& CurrentLevelData)
{
	int32 BrickSize = VolumetricLightmapSettings.BrickSize;
	int32 PaddedBrickSize = BrickSize + 1;
	const int32 BrickSizeLog2 = FMath::FloorLog2(BrickSize);
	const float InvBrickSize = 1.0f / BrickSize;

	// Stitch all higher density bricks with neighboring lower density bricks
	for (int32 BrickIndex = 0; BrickIndex < BricksAtCurrentDepth.Num(); BrickIndex++)
	{
		const FImportedVolumetricLightmapBrick& Brick = *BricksAtCurrentDepth[BrickIndex];
		const FIntVector BrickLayoutPosition = ComputeBrickLayoutPosition(BrickStartAllocation + BrickIndex, BrickLayoutDimensions) * PaddedBrickSize;
		const int32 DetailCellsPerCurrentLevelBrick = 1 << ((VolumetricLightmapSettings.MaxRefinementLevels - Brick.TreeDepth) * BrickSizeLog2);
		const int32 NumBottomLevelBricks = DetailCellsPerCurrentLevelBrick / BrickSize;
		const FVector IndirectionTexturePosition = FVector(Brick.IndirectionTexturePosition);

		int32 X, Y, Z = 0;

		// Iterate over unique data on the edge of the brick which needs to match padding on lower resolution bricks
		for (X = 0, Z = 0; Z < BrickSize; Z++)
		{
			for (Y = 0; Y < BrickSize; Y++)
			{
				FVector IndirectionDataSourceCoordinate = IndirectionTexturePosition + FVector(X, Y, Z) * InvBrickSize * NumBottomLevelBricks;

				for (int32 StitchDirection = 1; StitchDirection < 8; StitchDirection++)
				{
					FVector StitchSourceCoordinate = IndirectionDataSourceCoordinate;

					if ((StitchDirection & 1) && X == 0)
					{
						StitchSourceCoordinate.X -= GPointFilteringThreshold * 2;
					}

					if ((StitchDirection & 2) && Y == 0)
					{
						StitchSourceCoordinate.Y -= GPointFilteringThreshold * 2;
					}

					if ((StitchDirection & 4) && Z == 0)
					{
						StitchSourceCoordinate.Z -= GPointFilteringThreshold * 2;
					}

					if (StitchSourceCoordinate != IndirectionDataSourceCoordinate)
					{
						bool bStitched = CopyFromBrickmapTexel(
							StitchSourceCoordinate,
							FIntVector(X, Y, Z),
							// Restrict copies to only read from bricks that are lower effective resolution (higher NumBottomLevelBricks)
							NumBottomLevelBricks,
							BrickSize,
							BrickLayoutPosition,
							CurrentLevelData,
							CurrentLevelData.BrickData);

						if (bStitched)
						{
							break;
						}
					}
				}
			}
		}

		for (Z = 0, Y = 0; Y < BrickSize; Y++)
		{
			for (X = 1; X < BrickSize; X++)
			{
				FVector IndirectionDataSourceCoordinate = IndirectionTexturePosition + FVector(X, Y, Z) * InvBrickSize * NumBottomLevelBricks;
									
				for (int32 StitchDirection = 1; StitchDirection < 8; StitchDirection++)
				{
					FVector StitchSourceCoordinate = IndirectionDataSourceCoordinate;

					if ((StitchDirection & 1) && X == 0)
					{
						StitchSourceCoordinate.X -= GPointFilteringThreshold * 2;
					}

					if ((StitchDirection & 2) && Y == 0)
					{
						StitchSourceCoordinate.Y -= GPointFilteringThreshold * 2;
					}

					if ((StitchDirection & 4) && Z == 0)
					{
						StitchSourceCoordinate.Z -= GPointFilteringThreshold * 2;
					}

					if (StitchSourceCoordinate != IndirectionDataSourceCoordinate)
					{
						bool bStitched = CopyFromBrickmapTexel(
							StitchSourceCoordinate,
							FIntVector(X, Y, Z),
							// Restrict copies to only read from bricks that are lower effective resolution (higher NumBottomLevelBricks)
							NumBottomLevelBricks,
							BrickSize,
							BrickLayoutPosition,
							CurrentLevelData,
							CurrentLevelData.BrickData);

						if (bStitched)
						{
							break;
						}
					}
				}
			}
		}

		for (Y = 0, Z = 1; Z < BrickSize; Z++)
		{
			for (X = 1; X < BrickSize; X++)
			{
				FVector IndirectionDataSourceCoordinate = IndirectionTexturePosition + FVector(X, Y, Z) * InvBrickSize * NumBottomLevelBricks;
								
				for (int32 StitchDirection = 1; StitchDirection < 8; StitchDirection++)
				{
					FVector StitchSourceCoordinate = IndirectionDataSourceCoordinate;

					if ((StitchDirection & 1) && X == 0)
					{
						StitchSourceCoordinate.X -= GPointFilteringThreshold * 2;
					}

					if ((StitchDirection & 2) && Y == 0)
					{
						StitchSourceCoordinate.Y -= GPointFilteringThreshold * 2;
					}

					if ((StitchDirection & 4) && Z == 0)
					{
						StitchSourceCoordinate.Z -= GPointFilteringThreshold * 2;
					}

					if (StitchSourceCoordinate != IndirectionDataSourceCoordinate)
					{
						bool bStitched = CopyFromBrickmapTexel(
							StitchSourceCoordinate,
							FIntVector(X, Y, Z),
							// Restrict copies to only read from bricks that are lower effective resolution (higher NumBottomLevelBricks)
							NumBottomLevelBricks,
							BrickSize,
							BrickLayoutPosition,
							CurrentLevelData,
							CurrentLevelData.BrickData);

						if (bStitched)
						{
							break;
						}
					}
				}
			}
		}
	}
}

void CopyPaddingFromUniqueData(
	const TArray<const FImportedVolumetricLightmapBrick*>& BricksAtCurrentDepth,
	int32 BrickStartAllocation,
	int32 CurrentDepth,
	FIntVector BrickLayoutDimensions,
	const Lightmass::FVolumetricLightmapSettings& VolumetricLightmapSettings,
	const FPrecomputedVolumetricLightmapData& CurrentLevelData,
	FVolumetricLightmapBrickData& BrickData)
{
	int32 BrickSize = VolumetricLightmapSettings.BrickSize;
	int32 PaddedBrickSize = BrickSize + 1;
	const int32 BrickSizeLog2 = FMath::FloorLog2(BrickSize);
	const float InvBrickSize = 1.0f / BrickSize;

	for (int32 BrickIndex = 0; BrickIndex < BricksAtCurrentDepth.Num(); BrickIndex++)
	{
		const FImportedVolumetricLightmapBrick& Brick = *BricksAtCurrentDepth[BrickIndex];
		const FIntVector BrickLayoutPosition = ComputeBrickLayoutPosition(BrickStartAllocation + BrickIndex, BrickLayoutDimensions) * PaddedBrickSize;
		const int32 DetailCellsPerCurrentLevelBrick = 1 << ((VolumetricLightmapSettings.MaxRefinementLevels - Brick.TreeDepth) * BrickSizeLog2);
		const int32 NumBottomLevelBricks = DetailCellsPerCurrentLevelBrick / BrickSize;
		const FVector IndirectionTexturePosition = FVector(Brick.IndirectionTexturePosition);

		int32 X, Y, Z = 0;

		// Iterate over padding voxels
		for (X = PaddedBrickSize - 1, Z = 0; Z < PaddedBrickSize; Z++)
		{
			for (Y = 0; Y < PaddedBrickSize; Y++)
			{
				const FVector IndirectionDataSourceCoordinate = IndirectionTexturePosition + FVector(X, Y, Z) * InvBrickSize * NumBottomLevelBricks;

				// Overwrite padding with unique data from this same coordinate in the indirection texture
				CopyFromBrickmapTexel(
					IndirectionDataSourceCoordinate,
					FIntVector(X, Y, Z),
					0,
					BrickSize,
					BrickLayoutPosition,
					CurrentLevelData,
					BrickData);
			}
		}

		for (Z = PaddedBrickSize - 1, Y = 0; Y < PaddedBrickSize; Y++)
		{
			for (X = 0; X < PaddedBrickSize; X++)
			{
				const FVector IndirectionDataSourceCoordinate = IndirectionTexturePosition + FVector(X, Y, Z) * InvBrickSize * NumBottomLevelBricks;

				CopyFromBrickmapTexel(
					IndirectionDataSourceCoordinate,
					FIntVector(X, Y, Z),
					0,
					BrickSize,
					BrickLayoutPosition,
					CurrentLevelData,
					BrickData);
			}
		}

		for (Y = PaddedBrickSize - 1, Z = 0; Z < PaddedBrickSize; Z++)
		{
			for (X = 0; X < PaddedBrickSize; X++)
			{
				const FVector IndirectionDataSourceCoordinate = IndirectionTexturePosition + FVector(X, Y, Z) * InvBrickSize * NumBottomLevelBricks;

				CopyFromBrickmapTexel(
					IndirectionDataSourceCoordinate,
					FIntVector(X, Y, Z),
					0,
					BrickSize,
					BrickLayoutPosition,
					CurrentLevelData,
					BrickData);
			}
		}
	}
}

FVector GetLookupPositionAwayFromBorder(int32 PaddedBrickSize, FIntVector LocalCellCoordinate)
{
	FVector LookupCoordinate(LocalCellCoordinate);

	if (LocalCellCoordinate.X == PaddedBrickSize - 1)
	{
		LookupCoordinate.X -= 1;
	}

	if (LocalCellCoordinate.Y == PaddedBrickSize - 1)
	{
		LookupCoordinate.Y -= 1;
	}

	if (LocalCellCoordinate.Z == PaddedBrickSize - 1)
	{
		LookupCoordinate.Z -= 1;
	}

	return LookupCoordinate;
}

void CopyVolumeBorderFromInterior(
	const TArray<const FImportedVolumetricLightmapBrick*>& BricksAtCurrentDepth,
	int32 BrickStartAllocation,
	int32 CurrentDepth,
	FIntVector BrickLayoutDimensions,
	const Lightmass::FVolumetricLightmapSettings& VolumetricLightmapSettings,
	FPrecomputedVolumetricLightmapData& CurrentLevelData,
	FVolumetricLightmapBrickData& BrickData)
{
	int32 BrickSize = VolumetricLightmapSettings.BrickSize;
	int32 PaddedBrickSize = BrickSize + 1;
	const int32 BrickSizeLog2 = FMath::FloorLog2(BrickSize);
	const float InvBrickSize = 1.0f / BrickSize;

	for (int32 BrickIndex = 0; BrickIndex < BricksAtCurrentDepth.Num(); BrickIndex++)
	{
		const FImportedVolumetricLightmapBrick& Brick = *BricksAtCurrentDepth[BrickIndex];
		const FIntVector BrickLayoutPosition = ComputeBrickLayoutPosition(BrickStartAllocation + BrickIndex, BrickLayoutDimensions) * PaddedBrickSize;
		const int32 DetailCellsPerCurrentLevelBrick = 1 << ((VolumetricLightmapSettings.MaxRefinementLevels - Brick.TreeDepth) * BrickSizeLog2);
		const int32 NumBottomLevelBricks = DetailCellsPerCurrentLevelBrick / BrickSize;
		const FVector IndirectionTexturePosition = FVector(Brick.IndirectionTexturePosition);

		// Operate on bricks on the edge of the volume covered by the indirection texture
		if (Brick.IndirectionTexturePosition.X + NumBottomLevelBricks == CurrentLevelData.IndirectionTextureDimensions.X)
		{
			int32 X, Y, Z = 0;

			// Iterate over padding voxels
			for (X = PaddedBrickSize - 1, Z = 0; Z < PaddedBrickSize; Z++)
			{
				for (Y = 0; Y < PaddedBrickSize; Y++)
				{
					const FVector LookupPosition = GetLookupPositionAwayFromBorder(PaddedBrickSize, FIntVector(X, Y, Z));
					const FVector IndirectionDataSourceCoordinate = IndirectionTexturePosition + LookupPosition * InvBrickSize * NumBottomLevelBricks;

					// Overwrite padding on the edge of the volume with neighboring data inside the volume
					CopyFromBrickmapTexel(
						IndirectionDataSourceCoordinate,
						FIntVector(X, Y, Z),
						0,
						BrickSize,
						BrickLayoutPosition,
						CurrentLevelData,
						BrickData);
				}
			}
		}

		if (Brick.IndirectionTexturePosition.Y + NumBottomLevelBricks == CurrentLevelData.IndirectionTextureDimensions.Y)
		{
			int32 X, Y, Z = 0;

			// Iterate over padding voxels
			for (Y = PaddedBrickSize - 1, Z = 0; Z < PaddedBrickSize; Z++)
			{
				for (X = 0; X < PaddedBrickSize; X++)
				{
					const FVector LookupPosition = GetLookupPositionAwayFromBorder(PaddedBrickSize, FIntVector(X, Y, Z));
					const FVector IndirectionDataSourceCoordinate = IndirectionTexturePosition + LookupPosition * InvBrickSize * NumBottomLevelBricks;

					CopyFromBrickmapTexel(
						IndirectionDataSourceCoordinate,
						FIntVector(X, Y, Z),
						0,
						BrickSize,
						BrickLayoutPosition,
						CurrentLevelData,
						BrickData);
				}
			}
		}

		if (Brick.IndirectionTexturePosition.Z + NumBottomLevelBricks == CurrentLevelData.IndirectionTextureDimensions.Z)
		{
			int32 X, Y, Z = 0;

			// Iterate over padding voxels
			for (Z = PaddedBrickSize - 1, Y = 0; Y < PaddedBrickSize; Y++)
			{
				for (X = 0; X < PaddedBrickSize; X++)
				{
					const FVector LookupPosition = GetLookupPositionAwayFromBorder(PaddedBrickSize, FIntVector(X, Y, Z));
					const FVector IndirectionDataSourceCoordinate = IndirectionTexturePosition + LookupPosition * InvBrickSize * NumBottomLevelBricks;

					CopyFromBrickmapTexel(
						IndirectionDataSourceCoordinate,
						FIntVector(X, Y, Z),
						0,
						BrickSize,
						BrickLayoutPosition,
						CurrentLevelData,
						BrickData);
				}
			}
		}
	}
}

int32 TrimBricksByInterpolationError(
	TArray<TArray<const FImportedVolumetricLightmapBrick*>>& BricksByDepth,
	const Lightmass::FVolumetricLightmapSettings& VolumetricLightmapSettings)
{
	int32 NumBricksRemoved = 0;

	if (VolumetricLightmapSettings.MaxRefinementLevels > 1)
	{
		TArray<const FImportedVolumetricLightmapBrick*>& HighestDensityBricks = BricksByDepth[VolumetricLightmapSettings.MaxRefinementLevels - 1];
		const int32 ParentLevel = VolumetricLightmapSettings.MaxRefinementLevels - 2;
		TArray<const FImportedVolumetricLightmapBrick*>& ParentLevelBricks = BricksByDepth[ParentLevel];

		const int32 BrickSize = VolumetricLightmapSettings.BrickSize;
		const float InvTotalBrickSize = 1.0f / (BrickSize * BrickSize * BrickSize);
		const int32 BrickSizeLog2 = FMath::FloorLog2(BrickSize);
		const int32 DetailCellsPerParentLevelBrick = 1 << ((VolumetricLightmapSettings.MaxRefinementLevels - ParentLevel) * BrickSizeLog2);
		const int32 NumParentBottomLevelBricks = DetailCellsPerParentLevelBrick / BrickSize;

		for (int32 BrickIndex = 0; BrickIndex < HighestDensityBricks.Num(); BrickIndex++)
		{
			const FImportedVolumetricLightmapBrick& Brick = *HighestDensityBricks[BrickIndex];
			const FImportedVolumetricLightmapBrick* ParentBrick = NULL;

			for (int32 ParentBrickIndex = 0; ParentBrickIndex < ParentLevelBricks.Num(); ParentBrickIndex++)
			{
				const FImportedVolumetricLightmapBrick& ParentLevelBrick = *ParentLevelBricks[ParentBrickIndex];

				if (Brick.IndirectionTexturePosition.X >= ParentLevelBrick.IndirectionTexturePosition.X
					&& Brick.IndirectionTexturePosition.Y >= ParentLevelBrick.IndirectionTexturePosition.Y
					&& Brick.IndirectionTexturePosition.Z >= ParentLevelBrick.IndirectionTexturePosition.Z
					&& Brick.IndirectionTexturePosition.X < ParentLevelBrick.IndirectionTexturePosition.X + NumParentBottomLevelBricks
					&& Brick.IndirectionTexturePosition.Y < ParentLevelBrick.IndirectionTexturePosition.Y + NumParentBottomLevelBricks
					&& Brick.IndirectionTexturePosition.Z < ParentLevelBrick.IndirectionTexturePosition.Z + NumParentBottomLevelBricks)
				{
					ParentBrick = &ParentLevelBrick;
					break;
				}
			}

			check(ParentBrick);

			FVector ErrorSquared(0, 0, 0);
			const FVector ChildOffset = FVector(Brick.IndirectionTexturePosition - ParentBrick->IndirectionTexturePosition);

			for (int32 Z = 0; Z < BrickSize; Z++)
			{
				for (int32 Y = 0; Y < BrickSize; Y++)
				{
					for (int32 X = 0; X < BrickSize; X++)
					{
						const FIntVector VoxelCoordinate(X, Y, Z);
						const int32 LinearVoxelIndex = ComputeLinearVoxelIndex(VoxelCoordinate, FIntVector(BrickSize, BrickSize, BrickSize));
						const FVector AmbientVector = FVector(Brick.AmbientVector[LinearVoxelIndex].ToLinearColor());

						const FVector ParentCoordinate = FVector(VoxelCoordinate) / (float)BrickSize + ChildOffset;
						const FVector ParentAmbientVector = FVector(FilteredVolumeLookup(ParentCoordinate, FIntVector(BrickSize, BrickSize, BrickSize), ParentBrick->AmbientVector.GetData()));
						ErrorSquared += (AmbientVector - ParentAmbientVector) * (AmbientVector - ParentAmbientVector);
					}
				}
			}

			const double RMSE = FMath::Sqrt((ErrorSquared * InvTotalBrickSize).GetMax());
			const bool bCullBrick = RMSE < VolumetricLightmapSettings.MinBrickError;

			if (bCullBrick)
			{
				HighestDensityBricks.RemoveAt(BrickIndex);
				BrickIndex--;
				NumBricksRemoved++;
			}
		}
	}

	return NumBricksRemoved;
}

int32 TrimBricksForMemoryLimit(
	TArray<TArray<const FImportedVolumetricLightmapBrick*>>& BricksByDepth,
	const Lightmass::FVolumetricLightmapSettings& VolumetricLightmapSettings,
	int32 VoxelSizeBytes,
	const float MaximumBrickMemoryMb)
{
	int32 NumBricksBeforeTrimming = 0;

	for (int32 CurrentDepth = 0; CurrentDepth < VolumetricLightmapSettings.MaxRefinementLevels; CurrentDepth++)
	{
		const TArray<const FImportedVolumetricLightmapBrick*>& BricksAtCurrentDepth = BricksByDepth[CurrentDepth];
		NumBricksBeforeTrimming += BricksAtCurrentDepth.Num();
	}

	const int32 PaddedBrickSize = VolumetricLightmapSettings.BrickSize + 1;
	TArray<const FImportedVolumetricLightmapBrick*>& HighestDensityBricks = BricksByDepth[VolumetricLightmapSettings.MaxRefinementLevels - 1];

	const uint64 BrickSizeBytes = VoxelSizeBytes * PaddedBrickSize * PaddedBrickSize * PaddedBrickSize;
	const uint64 MaxBrickBytes = static_cast<uint64>(MaximumBrickMemoryMb * 1024 * 1024);
	check(FMath::DivideAndRoundUp(MaxBrickBytes, BrickSizeBytes) <= 0x7FFFFFFFull);
	const int32 NumBricksBudgeted = (int32)FMath::DivideAndRoundUp(MaxBrickBytes, BrickSizeBytes);
	const int32 NumBricksToRemove = FMath::Clamp<int32>(NumBricksBeforeTrimming - NumBricksBudgeted, 0, HighestDensityBricks.Num());

	if (NumBricksToRemove > 0)
	{
		struct FCompareBricksByVisualImpact
		{
			FORCEINLINE bool operator()(const FImportedVolumetricLightmapBrick& A, const FImportedVolumetricLightmapBrick& B) const
			{
				// Sort smallest to largest
				return A.AverageClosestGeometryDistance < B.AverageClosestGeometryDistance;
			}
		};

		HighestDensityBricks.Sort(FCompareBricksByVisualImpact());
		HighestDensityBricks.RemoveAt(HighestDensityBricks.Num() - NumBricksToRemove, NumBricksToRemove);
	}

	return NumBricksToRemove;
}

void BuildIndirectionTexture(
	const TArray<TArray<const FImportedVolumetricLightmapBrick*>>& BricksByDepth,
	const Lightmass::FVolumetricLightmapSettings& VolumetricLightmapSettings,
	int32 MaxBricksInLayoutOneDim,
	FIntVector BrickLayoutDimensions,
	int32 IndirectionTextureDataStride,
	FPrecomputedVolumetricLightmapData& CurrentLevelData,
	bool bOnlyBuildForPersistentLevel)
{
	const int32 BrickSizeLog2 = FMath::FloorLog2(VolumetricLightmapSettings.BrickSize);

	int32 BrickAllocation = 0;

	for (int32 CurrentDepth = 0; CurrentDepth < VolumetricLightmapSettings.MaxRefinementLevels; CurrentDepth++)
	{
		const TArray<const FImportedVolumetricLightmapBrick*>& BricksAtCurrentDepth = BricksByDepth[CurrentDepth];

		for (int32 BrickIndex = 0; BrickIndex < BricksAtCurrentDepth.Num(); BrickIndex++)
		{
			const FImportedVolumetricLightmapBrick& Brick = *BricksAtCurrentDepth[BrickIndex];

			if (CurrentDepth == VolumetricLightmapSettings.MaxRefinementLevels - 1)
			{
				const FGuid PersistentLevelGuid = FGuid(0, 0, 0, 0);

				// Skip non-intersecting, bottom detailed bricks for persistent level
				if (bOnlyBuildForPersistentLevel && Brick.IntersectingLevelGuid != PersistentLevelGuid)
				{
					continue;
				}
			}

			const FIntVector BrickLayoutPosition = ComputeBrickLayoutPosition(BrickAllocation, BrickLayoutDimensions);
			check(BrickLayoutPosition.X < MaxBricksInLayoutOneDim && BrickLayoutPosition.Y < MaxBricksInLayoutOneDim && BrickLayoutPosition.Z < MaxBricksInLayoutOneDim);
			checkSlow(IndirectionTextureDataStride == sizeof(uint8) * 4);

			const int32 DetailCellsPerCurrentLevelBrick = 1 << ((VolumetricLightmapSettings.MaxRefinementLevels - Brick.TreeDepth) * BrickSizeLog2);
			const int32 NumBottomLevelBricks = DetailCellsPerCurrentLevelBrick / VolumetricLightmapSettings.BrickSize;
			check(NumBottomLevelBricks < MaxBricksInLayoutOneDim);

			for (int32 Z = 0; Z < NumBottomLevelBricks; Z++)
			{
				for (int32 Y = 0; Y < NumBottomLevelBricks; Y++)
				{
					for (int32 X = 0; X < NumBottomLevelBricks; X++)
					{
						const FIntVector IndirectionDestDataCoordinate = Brick.IndirectionTexturePosition + FIntVector(X, Y, Z);
						const int32 IndirectionDestDataIndex = ((IndirectionDestDataCoordinate.Z * CurrentLevelData.IndirectionTextureDimensions.Y) + IndirectionDestDataCoordinate.Y) * CurrentLevelData.IndirectionTextureDimensions.X + IndirectionDestDataCoordinate.X;
						uint8* IndirectionVoxelPtr = (uint8*)&CurrentLevelData.IndirectionTexture.Data[IndirectionDestDataIndex * IndirectionTextureDataStride];
						*(IndirectionVoxelPtr + 0) = static_cast<uint8>(BrickLayoutPosition.X);
						*(IndirectionVoxelPtr + 1) = static_cast<uint8>(BrickLayoutPosition.Y);
						*(IndirectionVoxelPtr + 2) = static_cast<uint8>(BrickLayoutPosition.Z);
						*(IndirectionVoxelPtr + 3) = static_cast<uint8>(NumBottomLevelBricks);
					}
				}
			}

			BrickAllocation++;
		}
	}
}

inline void ConvertBGRA8ToRGBA8ForLayer(FVolumetricLightmapDataLayer& Layer)
{
	if (Layer.Format == PF_B8G8R8A8)
	{
		for (int32 PixelIndex = 0; PixelIndex < Layer.Data.Num() / GPixelFormats[PF_B8G8R8A8].BlockBytes; PixelIndex++)
		{
			FColor Color;

			Color.B = Layer.Data[PixelIndex * 4 + 0];
			Color.G = Layer.Data[PixelIndex * 4 + 1];
			Color.R = Layer.Data[PixelIndex * 4 + 2];
			Color.A = Layer.Data[PixelIndex * 4 + 3];

			Layer.Data[PixelIndex * 4 + 0] = Color.R;
			Layer.Data[PixelIndex * 4 + 1] = Color.G;
			Layer.Data[PixelIndex * 4 + 2] = Color.B;
			Layer.Data[PixelIndex * 4 + 3] = Color.A;
		}

		Layer.Format = PF_R8G8B8A8;
	}
}

// For debugging
bool bStitchDetailBricksWithLowDensityNeighbors = true;
bool bCopyPaddingFromUniqueData = true;
bool bCopyVolumeBorderFromInterior = true;

void FLightmassProcessor::SetVolumetricLightMapImportMode(bool bUseVLMCellGrid)
{
	bSplitToVLMCellGrid = bUseVLMCellGrid;
}

void FLightmassProcessor::ImportVolumetricLightmap()
{
	const double StartTime = FPlatformTime::Seconds();

	Lightmass::FVolumetricLightmapSettings VolumetricLightmapSettings;
	GetLightmassExporter()->SetVolumetricLightmapSettings(VolumetricLightmapSettings);

	const int32 BrickSize = VolumetricLightmapSettings.BrickSize;
	const int32 PaddedBrickSize = VolumetricLightmapSettings.BrickSize + 1;
	const int32 MaxBricksInLayoutOneDim = 1 << 8;
	const int32 MaxBrickTextureLayoutSize = PaddedBrickSize * MaxBricksInLayoutOneDim;
	const FGuid PersistentLevelGuid = FGuid(0, 0, 0, 0);

	TArray<FImportedVolumetricLightmapTaskData> TaskDataArray;
	TaskDataArray.Empty(Exporter->VolumetricLightmapTaskGuids.Num());

	bool bGenerateSkyShadowing = false;

	int32 LocalNumCompletedVolumetricLightmapTasks = FPlatformAtomics::AtomicRead(&NumCompletedVolumetricLightmapTasks);
	if (Exporter->VolumetricLightmapTaskGuids.Num() != LocalNumCompletedVolumetricLightmapTasks)
	{
		FMessageLog("LightingResults").Error(
			FText::Format(
				LOCTEXT("LightmassError_VolumetricLightmapImportFailed_NotAllCompleted", "Import Volumetric Lightmap failed: Expected {0} tasks, only {1} were reported as completed from Swarm"), 
				FText::AsNumber(Exporter->VolumetricLightmapTaskGuids.Num()), 
				FText::AsNumber(LocalNumCompletedVolumetricLightmapTasks)
			)
		);

		return;
	}

	ImportIrradianceTasks(bGenerateSkyShadowing, TaskDataArray);

	if (TaskDataArray.Num() != Exporter->VolumetricLightmapTaskGuids.Num())
	{
		FMessageLog("LightingResults").Error(
			FText::Format(
				LOCTEXT("LightmassError_VolumetricLightmapImportFailed_FewerThanExpectedImported", "Import Volumetric Lightmap failed: Expected {0} tasks, only found {1}"),
				FText::AsNumber(Exporter->VolumetricLightmapTaskGuids.Num()),
				FText::AsNumber(TaskDataArray.Num())
			)
		);

		return;
	}

	TArray<TArray<const FImportedVolumetricLightmapBrick*>> BricksByDepth;
	BricksByDepth.Empty(VolumetricLightmapSettings.MaxRefinementLevels);
	BricksByDepth.AddDefaulted(VolumetricLightmapSettings.MaxRefinementLevels);

	for (int32 TaskDataIndex = 0; TaskDataIndex < TaskDataArray.Num(); TaskDataIndex++)
	{
		const FImportedVolumetricLightmapTaskData& TaskData = TaskDataArray[TaskDataIndex];

		for (int32 BrickIndex = 0; BrickIndex < TaskData.Bricks.Num(); BrickIndex++)
		{
			const FImportedVolumetricLightmapBrick& Brick = TaskData.Bricks[BrickIndex];
			BricksByDepth[Brick.TreeDepth].Add(&Brick);
		}
	}

	UMapBuildDataRegistry* Registry = System.LightingContext.GetOrCreateGlobalRegistry();
	FPrecomputedVolumetricLightmapData CurrentLevelData;

	{
		CurrentLevelData.InitializeOnImport(FBox(VolumetricLightmapSettings.VolumeMin, VolumetricLightmapSettings.VolumeMin + VolumetricLightmapSettings.VolumeSize), BrickSize);

		// Temporarily assign format as PF_B8G8R8A8 since that matches FColor and makes our operations easier
		// Will be converted to PF_R8G8B8A8 by ConvertBGRA8ToRGBA8ForLayer() later
		CurrentLevelData.BrickData.AmbientVector.Format = PF_FloatR11G11B10;
		CurrentLevelData.BrickData.SkyBentNormal.Format = PF_B8G8R8A8;
		CurrentLevelData.BrickData.DirectionalLightShadowing.Format = PF_G8;

		for (int32 i = 0; i < UE_ARRAY_COUNT(CurrentLevelData.BrickData.SHCoefficients); i++)
		{
			CurrentLevelData.BrickData.SHCoefficients[i].Format = PF_B8G8R8A8;
		}
	}

	const int32 NumBottomLevelBricksTrimmedByInterpolationError = TrimBricksByInterpolationError(BricksByDepth, VolumetricLightmapSettings);

	const float MaximumBrickMemoryMb = System.GetWorld()->GetWorldSettings()->LightmassSettings.VolumetricLightmapMaximumBrickMemoryMb;

	// avoid trimming for memory if we're going to output the bricks to streaming grid
	// @todo_ow: Have a trim limit for base data + one limit per cell for the highest level bricks
	const int32 NumBottomLevelBricksTrimmedForMemoryLimit = bSplitToVLMCellGrid ? 0 : TrimBricksForMemoryLimit(BricksByDepth, VolumetricLightmapSettings, CurrentLevelData.BrickData.GetMinimumVoxelSize(), MaximumBrickMemoryMb);

	int32 BrickTextureLinearAllocator = 0;

	for (int32 CurrentDepth = 0; CurrentDepth < VolumetricLightmapSettings.MaxRefinementLevels; CurrentDepth++)
	{
		const TArray<const FImportedVolumetricLightmapBrick*>& BricksAtCurrentDepth = BricksByDepth[CurrentDepth];
		BrickTextureLinearAllocator += BricksAtCurrentDepth.Num();
	}

	FIntVector BrickLayoutDimensions;
	BrickLayoutDimensions.X = FMath::Min(BrickTextureLinearAllocator, MaxBricksInLayoutOneDim);
	BrickTextureLinearAllocator = FMath::DivideAndRoundUp(BrickTextureLinearAllocator, BrickLayoutDimensions.X);
	BrickLayoutDimensions.Y = FMath::Min(BrickTextureLinearAllocator, MaxBricksInLayoutOneDim);
	BrickTextureLinearAllocator = FMath::DivideAndRoundUp(BrickTextureLinearAllocator, BrickLayoutDimensions.Y);
	BrickLayoutDimensions.Z = FMath::Min(BrickTextureLinearAllocator, MaxBricksInLayoutOneDim);

	const int32 BrickSizeLog2 = FMath::FloorLog2(BrickSize);
	const int32 DetailCellsPerTopLevelBrick = 1 << (VolumetricLightmapSettings.MaxRefinementLevels * BrickSizeLog2);
	const int32 IndirectionCellsPerTopLevelCell = DetailCellsPerTopLevelBrick / BrickSize;

	int32 IndirectionTextureDataStride;
	{
		CurrentLevelData.IndirectionTextureDimensions = VolumetricLightmapSettings.TopLevelGridSize * IndirectionCellsPerTopLevelCell;
		CurrentLevelData.IndirectionTexture.Format = PF_R8G8B8A8_UINT;
		IndirectionTextureDataStride = GPixelFormats[CurrentLevelData.IndirectionTexture.Format].BlockBytes;

		const int32 TotalIndirectionTextureSize = CurrentLevelData.IndirectionTextureDimensions.X * CurrentLevelData.IndirectionTextureDimensions.Y * CurrentLevelData.IndirectionTextureDimensions.Z;
		CurrentLevelData.IndirectionTexture.Resize(TotalIndirectionTextureSize * IndirectionTextureDataStride);
	}

	BuildIndirectionTexture(BricksByDepth, VolumetricLightmapSettings, MaxBricksInLayoutOneDim, BrickLayoutDimensions, IndirectionTextureDataStride, CurrentLevelData, false);

	TArray<Lightmass::FIrradianceVoxelImportProcessingData> VoxelImportProcessingData;

	{
		CurrentLevelData.BrickDataDimensions = BrickLayoutDimensions * PaddedBrickSize;
		const int32 TotalBrickDataSize = CurrentLevelData.BrickDataDimensions.X * CurrentLevelData.BrickDataDimensions.Y * CurrentLevelData.BrickDataDimensions.Z;

		CurrentLevelData.BrickData.AmbientVector.Resize(TotalBrickDataSize * GPixelFormats[CurrentLevelData.BrickData.AmbientVector.Format].BlockBytes);

		if (bGenerateSkyShadowing)
		{
			CurrentLevelData.BrickData.SkyBentNormal.Resize(TotalBrickDataSize * GPixelFormats[CurrentLevelData.BrickData.SkyBentNormal.Format].BlockBytes);
		}

		CurrentLevelData.BrickData.DirectionalLightShadowing.Resize(TotalBrickDataSize * GPixelFormats[CurrentLevelData.BrickData.DirectionalLightShadowing.Format].BlockBytes);

		for (int32 i = 0; i < UE_ARRAY_COUNT(CurrentLevelData.BrickData.SHCoefficients); i++)
		{
			const int32 Stride = GPixelFormats[CurrentLevelData.BrickData.SHCoefficients[i].Format].BlockBytes;
			CurrentLevelData.BrickData.SHCoefficients[i].Resize(TotalBrickDataSize * Stride);
		}

		VoxelImportProcessingData.Empty(TotalBrickDataSize);
		VoxelImportProcessingData.AddZeroed(TotalBrickDataSize);
	}

	int32 BrickStartAllocation = 0;

	for (int32 CurrentDepth = 0; CurrentDepth < VolumetricLightmapSettings.MaxRefinementLevels; CurrentDepth++)
	{
		const TArray<const FImportedVolumetricLightmapBrick*>& BricksAtCurrentDepth = BricksByDepth[CurrentDepth];

		for (int32 BrickIndex = 0; BrickIndex < BricksAtCurrentDepth.Num(); BrickIndex++)
		{
			const FImportedVolumetricLightmapBrick& Brick = *BricksAtCurrentDepth[BrickIndex];
			const FIntVector BrickLayoutPosition = ComputeBrickLayoutPosition(BrickStartAllocation + BrickIndex, BrickLayoutDimensions) * PaddedBrickSize;

			CopyBrickToAtlasVolumeTexture(
				GPixelFormats[CurrentLevelData.BrickData.AmbientVector.Format].BlockBytes,
				CurrentLevelData.BrickDataDimensions,
				BrickLayoutPosition,
				FIntVector(BrickSize),
				(const uint8*)Brick.AmbientVector.GetData(),
				CurrentLevelData.BrickData.AmbientVector.Data.GetData());

			for (int32 i = 0; i < UE_ARRAY_COUNT(Brick.SHCoefficients); i++)
			{
				CopyBrickToAtlasVolumeTexture(
					GPixelFormats[CurrentLevelData.BrickData.SHCoefficients[i].Format].BlockBytes,
					CurrentLevelData.BrickDataDimensions,
					BrickLayoutPosition,
					FIntVector(BrickSize),
					(const uint8*)Brick.SHCoefficients[i].GetData(),
					CurrentLevelData.BrickData.SHCoefficients[i].Data.GetData());
			}

			if (bGenerateSkyShadowing)
			{
				CopyBrickToAtlasVolumeTexture(
					GPixelFormats[CurrentLevelData.BrickData.SkyBentNormal.Format].BlockBytes,
					CurrentLevelData.BrickDataDimensions,
					BrickLayoutPosition,
					FIntVector(BrickSize),
					(const uint8*)Brick.SkyBentNormal.GetData(),
					CurrentLevelData.BrickData.SkyBentNormal.Data.GetData());
			}

			CopyBrickToAtlasVolumeTexture(
				GPixelFormats[CurrentLevelData.BrickData.DirectionalLightShadowing.Format].BlockBytes,
				CurrentLevelData.BrickDataDimensions,
				BrickLayoutPosition,
				FIntVector(BrickSize),
				(const uint8*)Brick.DirectionalLightShadowing.GetData(),
				CurrentLevelData.BrickData.DirectionalLightShadowing.Data.GetData());

			CopyBrickToAtlasVolumeTexture(
				VoxelImportProcessingData.GetTypeSize(),
				CurrentLevelData.BrickDataDimensions,
				BrickLayoutPosition,
				FIntVector(BrickSize),
				(const uint8*)Brick.TaskVoxelImportProcessingData.GetData(),
				(uint8*)VoxelImportProcessingData.GetData());
		}

		BrickStartAllocation += BricksAtCurrentDepth.Num();
	}

	const float InvBrickSize = 1.0f / BrickSize;
	const FVector DetailCellSize = FVector(VolumetricLightmapSettings.VolumeSize) / FVector(VolumetricLightmapSettings.TopLevelGridSize * DetailCellsPerTopLevelBrick);

	for (int32 DilatePassIndex = 0; DilatePassIndex < NumDilateOverEmbeddedVoxelsPasses; DilatePassIndex++)
	{
		BrickStartAllocation = 0;

		// Compute the allocation start for the highest density level bricks
		for (int32 CurrentDepth = 0; CurrentDepth < VolumetricLightmapSettings.MaxRefinementLevels - 1; CurrentDepth++)
		{
			BrickStartAllocation += BricksByDepth[CurrentDepth].Num();
		}

		TArray<const FImportedVolumetricLightmapBrick*>& HighestDensityBricks = BricksByDepth[VolumetricLightmapSettings.MaxRefinementLevels - 1];

		// Need to double buffer bInsideGeometry as it is both read and written
		TArray<Lightmass::FIrradianceVoxelImportProcessingData> NewVoxelImportProcessingData = VoxelImportProcessingData;

		// Reads from unique data of any density bricks, writes to unique data
		// This is doing a filter in-place, which is only reliable because source and dest voxels are mutually exclusive
		FilterWithNeighbors(
			HighestDensityBricks,
			BrickStartAllocation,
			VolumetricLightmapSettings.MaxRefinementLevels - 1,
			BrickLayoutDimensions,
			VolumetricLightmapSettings,
			CurrentLevelData,
			VoxelImportProcessingData,
			NewVoxelImportProcessingData);

		VoxelImportProcessingData = MoveTemp(NewVoxelImportProcessingData);
	}

	BrickStartAllocation = 0;

	for (int32 CurrentDepth = 0; CurrentDepth < VolumetricLightmapSettings.MaxRefinementLevels; CurrentDepth++)
	{
		const TArray<const FImportedVolumetricLightmapBrick*>& BricksAtCurrentDepth = BricksByDepth[CurrentDepth];

		if (bStitchDetailBricksWithLowDensityNeighbors && CurrentDepth > 0)
		{
			// Reads from both unique and padding data of lower density bricks, writes to unique data
			StitchDetailBricksWithLowDensityNeighbors(
				BricksAtCurrentDepth,
				BrickStartAllocation,
				CurrentDepth,
				BrickLayoutDimensions,
				VolumetricLightmapSettings,
				CurrentLevelData);
		}

		if (bCopyPaddingFromUniqueData)
		{
			// Compute padding for all the bricks
			// Reads from unique data, writes to padding data of bricks
			// Padding must be computed after all operations that might modify the unique data
			CopyPaddingFromUniqueData(
				BricksAtCurrentDepth,
				BrickStartAllocation,
				CurrentDepth,
				BrickLayoutDimensions,
				VolumetricLightmapSettings,
				CurrentLevelData,
				CurrentLevelData.BrickData);
		}

		if (bCopyVolumeBorderFromInterior)
		{
			// The volume border padding had no unique data to copy from, replicate the neighboring interior value
			CopyVolumeBorderFromInterior(
				BricksAtCurrentDepth,
				BrickStartAllocation,
				CurrentDepth,
				BrickLayoutDimensions,
				VolumetricLightmapSettings,
				CurrentLevelData,
				CurrentLevelData.BrickData);
		}

		BrickStartAllocation += BricksAtCurrentDepth.Num();
	}

	TMap<FGuid, int32> SubLevelTotalBricks;
	TMap<FGuid, int32> SubLevelBrickAllocator;
	TMap<FGuid, FIntVector> SubLevelBrickLayoutDimensions;

	{
		for (int32 CurrentDepth = 0; CurrentDepth < VolumetricLightmapSettings.MaxRefinementLevels; CurrentDepth++)
		{
			const TArray<const FImportedVolumetricLightmapBrick*>& BricksAtCurrentDepth = BricksByDepth[CurrentDepth];

			for (int32 BrickIndex = 0; BrickIndex < BricksAtCurrentDepth.Num(); BrickIndex++)
			{
				const FImportedVolumetricLightmapBrick& Brick = *BricksAtCurrentDepth[BrickIndex];

				FGuid LevelGuid;

				// assign to proper data container
				if (CurrentDepth == VolumetricLightmapSettings.MaxRefinementLevels - 1)
				{
					if (bSplitToVLMCellGrid)
					{
						LevelGuid = System.LightingContext.GetLevelGuidForVLMBrick(Brick.IndirectionTexturePosition);
						const_cast<FImportedVolumetricLightmapBrick&>(Brick).IntersectingLevelGuid = LevelGuid;	// change the level guid for this brick, I use a const cast since it preserves the intent everywhere else
					}
                    else
                    {
						LevelGuid = Brick.IntersectingLevelGuid;
                    }
				}
				else
				{
					// Top level bricks are put into persistent level
					LevelGuid = PersistentLevelGuid;
				}

				if (!SubLevelTotalBricks.Contains(LevelGuid))
				{
					SubLevelTotalBricks.Add(LevelGuid, 0);
					SubLevelBrickAllocator.Add(LevelGuid, 0);
					SubLevelBrickLayoutDimensions.Add(LevelGuid, FIntVector(0));
				}

				SubLevelTotalBricks[LevelGuid]++;
			}
		}

		for (auto Pair : SubLevelTotalBricks)
		{
			bool bPersistentLevel = Pair.Key == FGuid();

			FIntVector& SubLevelBrickLayoutDimension = SubLevelBrickLayoutDimensions[Pair.Key];
			int32 SubLevelBrickTextureLinearAllocator = SubLevelTotalBricks[Pair.Key];
			SubLevelBrickLayoutDimension.X = FMath::Min(SubLevelBrickTextureLinearAllocator, MaxBricksInLayoutOneDim);
			SubLevelBrickTextureLinearAllocator = FMath::DivideAndRoundUp(SubLevelBrickTextureLinearAllocator, SubLevelBrickLayoutDimension.X);
			SubLevelBrickLayoutDimension.Y = FMath::Min(SubLevelBrickTextureLinearAllocator, MaxBricksInLayoutOneDim);
			SubLevelBrickTextureLinearAllocator = FMath::DivideAndRoundUp(SubLevelBrickTextureLinearAllocator, SubLevelBrickLayoutDimension.Y);
			SubLevelBrickLayoutDimension.Z = FMath::Min(SubLevelBrickTextureLinearAllocator, MaxBricksInLayoutOneDim);

			{
				FPrecomputedVolumetricLightmapData& SubLevelData =  System.LightingContext.GetOrCreateLevelPrecomputedVolumetricLightmapBuildData(Pair.Key);

				SubLevelData.InitializeOnImport(FBox(VolumetricLightmapSettings.VolumeMin, VolumetricLightmapSettings.VolumeMin + VolumetricLightmapSettings.VolumeSize), BrickSize);
				{
					// Temporarily assign format as PF_B8G8R8A8 since that matches FColor and makes our operations easier
					// Will be converted to PF_R8G8B8A8 by ConvertBGRA8ToRGBA8ForLayer() later
					SubLevelData.BrickData.AmbientVector.Format = PF_FloatR11G11B10;
					SubLevelData.BrickData.SkyBentNormal.Format = PF_B8G8R8A8;
					SubLevelData.BrickData.DirectionalLightShadowing.Format = PF_G8;

					for (int32 i = 0; i < UE_ARRAY_COUNT(SubLevelData.BrickData.SHCoefficients); i++)
					{
						SubLevelData.BrickData.SHCoefficients[i].Format = PF_B8G8R8A8;
					}
				}

				SubLevelData.BrickDataDimensions = SubLevelBrickLayoutDimension * PaddedBrickSize;
				const int32 TotalBrickDataSize = SubLevelData.BrickDataDimensions.X * SubLevelData.BrickDataDimensions.Y * SubLevelData.BrickDataDimensions.Z;

				SubLevelData.BrickData.AmbientVector.Resize(TotalBrickDataSize * GPixelFormats[SubLevelData.BrickData.AmbientVector.Format].BlockBytes);

				if (bGenerateSkyShadowing)
				{
					SubLevelData.BrickData.SkyBentNormal.Resize(TotalBrickDataSize * GPixelFormats[SubLevelData.BrickData.SkyBentNormal.Format].BlockBytes);
				}

				SubLevelData.BrickData.DirectionalLightShadowing.Resize(TotalBrickDataSize * GPixelFormats[SubLevelData.BrickData.DirectionalLightShadowing.Format].BlockBytes);

				for (int32 i = 0; i < UE_ARRAY_COUNT(SubLevelData.BrickData.SHCoefficients); i++)
				{
					const int32 Stride = GPixelFormats[SubLevelData.BrickData.SHCoefficients[i].Format].BlockBytes;
					SubLevelData.BrickData.SHCoefficients[i].Resize(TotalBrickDataSize * Stride);
				}
			}
		}

		BrickStartAllocation = 0;

		for (int32 CurrentDepth = 0; CurrentDepth < VolumetricLightmapSettings.MaxRefinementLevels; CurrentDepth++)
		{
			const TArray<const FImportedVolumetricLightmapBrick*>& BricksAtCurrentDepth = BricksByDepth[CurrentDepth];

			for (int32 BrickIndex = 0; BrickIndex < BricksAtCurrentDepth.Num(); BrickIndex++)
			{
				const FImportedVolumetricLightmapBrick& Brick = *BricksAtCurrentDepth[BrickIndex];

				FGuid LevelGuid;

				if (CurrentDepth == VolumetricLightmapSettings.MaxRefinementLevels - 1)
				{
					LevelGuid = Brick.IntersectingLevelGuid;
				}
				else
				{
					// Top level bricks are put into persistent level
					LevelGuid = PersistentLevelGuid;
				}

				FPrecomputedVolumetricLightmapData& SubLevelData =  System.LightingContext.GetOrCreateLevelPrecomputedVolumetricLightmapBuildData(LevelGuid);

				const FIntVector BrickLayoutPosition = ComputeBrickLayoutPosition(BrickStartAllocation + BrickIndex, BrickLayoutDimensions) * PaddedBrickSize;
				const FIntVector SubLevelBrickLayoutPosition = ComputeBrickLayoutPosition(SubLevelBrickAllocator[LevelGuid], SubLevelBrickLayoutDimensions[LevelGuid]) * PaddedBrickSize;

				CopyBetweenAtlasVolumeTextures(
					GPixelFormats[CurrentLevelData.BrickData.AmbientVector.Format].BlockBytes,
					FIntVector(PaddedBrickSize),
					CurrentLevelData.BrickDataDimensions,
					BrickLayoutPosition,
					CurrentLevelData.BrickData.AmbientVector.Data,
					SubLevelBrickLayoutDimensions[LevelGuid] * PaddedBrickSize,
					SubLevelBrickLayoutPosition,
					SubLevelData.BrickData.AmbientVector.Data);

				for (int32 i = 0; i < UE_ARRAY_COUNT(Brick.SHCoefficients); i++)
				{
					CopyBetweenAtlasVolumeTextures(
						GPixelFormats[CurrentLevelData.BrickData.SHCoefficients[i].Format].BlockBytes,
						FIntVector(PaddedBrickSize),
						CurrentLevelData.BrickDataDimensions,
						BrickLayoutPosition,
						CurrentLevelData.BrickData.SHCoefficients[i].Data,
						SubLevelBrickLayoutDimensions[LevelGuid] * PaddedBrickSize,
						SubLevelBrickLayoutPosition,
						SubLevelData.BrickData.SHCoefficients[i].Data);
				}

				if (bGenerateSkyShadowing)
				{
					CopyBetweenAtlasVolumeTextures(
						GPixelFormats[CurrentLevelData.BrickData.SkyBentNormal.Format].BlockBytes,
						FIntVector(PaddedBrickSize),
						CurrentLevelData.BrickDataDimensions,
						BrickLayoutPosition,
						CurrentLevelData.BrickData.SkyBentNormal.Data,
						SubLevelBrickLayoutDimensions[LevelGuid] * PaddedBrickSize,
						SubLevelBrickLayoutPosition,
						SubLevelData.BrickData.SkyBentNormal.Data);
				}

				CopyBetweenAtlasVolumeTextures(
					GPixelFormats[CurrentLevelData.BrickData.DirectionalLightShadowing.Format].BlockBytes,
					FIntVector(PaddedBrickSize),
					CurrentLevelData.BrickDataDimensions,
					BrickLayoutPosition,
					CurrentLevelData.BrickData.DirectionalLightShadowing.Data,
					SubLevelBrickLayoutDimensions[LevelGuid] * PaddedBrickSize,
					SubLevelBrickLayoutPosition,
					SubLevelData.BrickData.DirectionalLightShadowing.Data);

				SubLevelBrickAllocator[LevelGuid]++;

				if (LevelGuid != PersistentLevelGuid)
				{
					SubLevelData.SubLevelBrickPositions.Add(Brick.IndirectionTexturePosition);
					check(SubLevelData.SubLevelBrickPositions.Num() == SubLevelBrickAllocator[LevelGuid]);
				}
			}

			BrickStartAllocation += BricksAtCurrentDepth.Num();
		}

		{
			const int32 TotalIndirectionTextureSize = CurrentLevelData.IndirectionTextureDimensions.X * CurrentLevelData.IndirectionTextureDimensions.Y * CurrentLevelData.IndirectionTextureDimensions.Z;
			CurrentLevelData.IndirectionTexture.Resize(TotalIndirectionTextureSize * IndirectionTextureDataStride);
			BuildIndirectionTexture(BricksByDepth, VolumetricLightmapSettings, MaxBricksInLayoutOneDim, SubLevelBrickLayoutDimensions[FGuid()], IndirectionTextureDataStride, CurrentLevelData, true);

			FPrecomputedVolumetricLightmapData& PersistentLevelData =  System.LightingContext.GetOrCreateLevelPrecomputedVolumetricLightmapBuildData(PersistentLevelGuid);

			PersistentLevelData.InitializeOnImport(FBox(VolumetricLightmapSettings.VolumeMin, VolumetricLightmapSettings.VolumeMin + VolumetricLightmapSettings.VolumeSize), BrickSize);
			PersistentLevelData.IndirectionTexture = CurrentLevelData.IndirectionTexture;
			PersistentLevelData.IndirectionTextureDimensions = CurrentLevelData.IndirectionTextureDimensions;
		}

		{
			for (int32 CurrentDepth = 0; CurrentDepth < VolumetricLightmapSettings.MaxRefinementLevels; CurrentDepth++)
			{
				const TArray<const FImportedVolumetricLightmapBrick*>& BricksAtCurrentDepth = BricksByDepth[CurrentDepth];

				for (int32 BrickIndex = 0; BrickIndex < BricksAtCurrentDepth.Num(); BrickIndex++)
				{
					const FImportedVolumetricLightmapBrick& Brick = *BricksAtCurrentDepth[BrickIndex];

					FGuid LevelGuid;

					if (CurrentDepth == VolumetricLightmapSettings.MaxRefinementLevels - 1)
					{
						LevelGuid = Brick.IntersectingLevelGuid;
					}
					else
					{
						// Top level bricks are put into persistent level
						LevelGuid = PersistentLevelGuid;
					}

					if (LevelGuid != PersistentLevelGuid)
					{
						FPrecomputedVolumetricLightmapData& SubLevelData =  System.LightingContext.GetOrCreateLevelPrecomputedVolumetricLightmapBuildData(LevelGuid);

						FIntVector IndirectionBrickOffset;
						int32 IndirectionBrickSize;
						const FVector IndirectionDataSourceCoordinate = FVector(Brick.IndirectionTexturePosition) + FVector(0.5f, 0.5f, 0.5f);
						SampleIndirectionTexture(IndirectionDataSourceCoordinate, CurrentLevelData.IndirectionTextureDimensions, CurrentLevelData.IndirectionTexture.Data.GetData(), IndirectionBrickOffset, IndirectionBrickSize);

						SubLevelData.IndirectionTextureOriginalValues.Add(FColor((uint8)IndirectionBrickOffset.X, (uint8)IndirectionBrickOffset.Y, (uint8)IndirectionBrickOffset.Z, (uint8)IndirectionBrickSize));
					}
				}
			}

		}

		{
			for (auto Pair : SubLevelTotalBricks)
			{
				FGuid LevelGuid = Pair.Key;

				FPrecomputedVolumetricLightmapData& SubLevelData =  System.LightingContext.GetOrCreateLevelPrecomputedVolumetricLightmapBuildData(LevelGuid);

				ConvertBGRA8ToRGBA8ForLayer(SubLevelData.BrickData.SkyBentNormal);

				for (int32 i = 0; i < UE_ARRAY_COUNT(SubLevelData.BrickData.SHCoefficients); i++)
				{
					ConvertBGRA8ToRGBA8ForLayer(SubLevelData.BrickData.SHCoefficients[i]);
				}
			}
		}
	}

	{
		/**
		 * Statistics
		 */

		double ImportTime = FPlatformTime::Seconds() - StartTime;
		UE_LOGF(LogVolumetricLightmapImport, Log, "Imported Volumetric Lightmap in %.3fs", ImportTime);
		UE_LOGF(LogVolumetricLightmapImport, Log, "     Indirection Texture %ux%ux%u = %.1fMb",
			CurrentLevelData.IndirectionTextureDimensions.X,
			CurrentLevelData.IndirectionTextureDimensions.Y,
			CurrentLevelData.IndirectionTextureDimensions.Z,
			CurrentLevelData.IndirectionTexture.Data.Num() / 1024.0f / 1024.0f);

		uint64 BrickDataSize = 0;

		for (auto Pair : SubLevelTotalBricks)
		{
			FPrecomputedVolumetricLightmapData& SubLevelData =  System.LightingContext.GetOrCreateLevelPrecomputedVolumetricLightmapBuildData(Pair.Key);

			SubLevelData.FinalizeImport();

			check(SubLevelData.IndirectionTextureOriginalValues.Num() == SubLevelData.SubLevelBrickPositions.Num());

			BrickDataSize += 
				SubLevelData.BrickData.AmbientVector.Data.Num() + 
				SubLevelData.BrickData.SHCoefficients[0].Data.Num() + 
				SubLevelData.BrickData.SHCoefficients[1].Data.Num() + 
				SubLevelData.BrickData.SHCoefficients[2].Data.Num() + 
				SubLevelData.BrickData.SHCoefficients[3].Data.Num() + 
				SubLevelData.BrickData.SHCoefficients[4].Data.Num() + 
				SubLevelData.BrickData.SHCoefficients[5].Data.Num() + 
				SubLevelData.BrickData.SkyBentNormal.Data.Num() + 
				SubLevelData.BrickData.DirectionalLightShadowing.Data.Num() +
				SubLevelData.SubLevelBrickPositions.Num() * SubLevelData.SubLevelBrickPositions.GetTypeSize() +
				SubLevelData.IndirectionTextureOriginalValues.Num() * SubLevelData.IndirectionTextureOriginalValues.GetTypeSize();
		}

		if (bSplitToVLMCellGrid)
		{
			// Export the data			
			FVolumetricLightMapGridDesc& GridDesc = *System.LightingContext.GetVolumetricLightMapGridDesc();
			GridDesc.InitializeBulkData();
		}

		int32 TotalNumBricks = 0;

		for (int32 CurrentDepth = 0; CurrentDepth < VolumetricLightmapSettings.MaxRefinementLevels; CurrentDepth++)
		{
			const TArray<const FImportedVolumetricLightmapBrick*>& BricksAtCurrentDepth = BricksByDepth[CurrentDepth];
			TotalNumBricks += BricksAtCurrentDepth.Num();
		}

		const uint64 ActualBrickSizeBytes = BrickDataSize / TotalNumBricks;

		FString TrimmedString;

		if (NumBottomLevelBricksTrimmedByInterpolationError > 0)
		{
			TrimmedString += FString::Printf(TEXT(" (trimmed %.1fMb due to %f MinBrickError)"),
				NumBottomLevelBricksTrimmedByInterpolationError * ActualBrickSizeBytes / 1024.0f / 1024.0f,
				VolumetricLightmapSettings.MinBrickError);
		}

		if (NumBottomLevelBricksTrimmedForMemoryLimit > 0)
		{
			TrimmedString += FString::Printf(TEXT(" (trimmed %.1fMb due to %.1fMb MaximumBrickMemoryMb)"),
				NumBottomLevelBricksTrimmedForMemoryLimit * ActualBrickSizeBytes / 1024.0f / 1024.0f,
				MaximumBrickMemoryMb);
		}

		UE_LOGF(LogVolumetricLightmapImport, Log, "     BrickData (all levels) %.1fMb%ls",
			BrickDataSize / 1024.0f / 1024.0f,
			*TrimmedString);

		UE_LOGF(LogVolumetricLightmapImport, Log, "     Bricks at depth");

		const float TotalVolume = VolumetricLightmapSettings.VolumeSize.X * VolumetricLightmapSettings.VolumeSize.Y * VolumetricLightmapSettings.VolumeSize.Z;

		for (int32 CurrentDepth = 0; CurrentDepth < VolumetricLightmapSettings.MaxRefinementLevels; CurrentDepth++)
		{
			const int32 DetailCellsPerCurrentLevelBrick = 1 << ((VolumetricLightmapSettings.MaxRefinementLevels - CurrentDepth) * BrickSizeLog2);
			const FVector CurrentDepthBrickSize = DetailCellSize * DetailCellsPerCurrentLevelBrick;
			const TArray<const FImportedVolumetricLightmapBrick*>& BricksAtCurrentDepth = BricksByDepth[CurrentDepth];
			const double CurrentDepthBrickVolume = CurrentDepthBrickSize.X * CurrentDepthBrickSize.Y * CurrentDepthBrickSize.Z;

			UE_LOGF(LogVolumetricLightmapImport, Log, "         %u: %.1f%% covering %.1f%% of volume",
				CurrentDepth,
				100.0f * BricksAtCurrentDepth.Num() / TotalNumBricks,
				100.0f * BricksAtCurrentDepth.Num() * CurrentDepthBrickVolume / TotalVolume);
		}

		UE_LOGF(LogVolumetricLightmapImport, Log, "     Bricks in each level");

		for (auto Pair : SubLevelTotalBricks)
		{
			ULevel* Level = FindLevel(Pair.Key);
			FString LevelName;
			if (Level)
			{
				LevelName = Level->GetOuter()->GetName();
			}
			else
			{				
				LevelName = System.LightingContext.GetVolumetricLightMapGridDesc()->GetCellDesc(Pair.Key);
			}

			if (Level && Level->IsPersistentLevel())
			{
				UE_LOGF(LogVolumetricLightmapImport, Log, "         %ls \t\t\t %d bricks \t %.1f%% (Persistent Level)", *LevelName, Pair.Value, 100.0f * Pair.Value / TotalNumBricks);
			}
			else
			{
				UE_LOGF(LogVolumetricLightmapImport, Log, "         %ls \t\t\t %d bricks \t %.1f%%", *LevelName, Pair.Value, 100.0f * Pair.Value / TotalNumBricks);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
