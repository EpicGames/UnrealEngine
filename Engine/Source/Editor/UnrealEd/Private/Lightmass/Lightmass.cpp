// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Lightmass.h: lightmass import/export implementation.
=============================================================================*/

#include "Lightmass/Lightmass.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FeedbackContext.h"
#include "Misc/App.h"
#include "UObject/UObjectIterator.h"
#include "EngineDefines.h"
#include "Engine/World.h"
#include "StaticMeshLight.h"
#include "PrecomputedLightVolume.h"
#include "PrecomputedVolumetricLightmap.h"
#include "Engine/MapBuildDataRegistry.h"
#include "Engine/TextureLightProfile.h"
#include "ModelLight.h"
#include "LandscapeLight.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Camera/CameraActor.h"
#include "Components/PointLightComponent.h"
#include "Components/SpotLightComponent.h"
#include "Components/RectLightComponent.h"
#include "Engine/GeneratedMeshAreaLight.h"
#include "Components/DirectionalLightComponent.h"
#include "Components/SkyLightComponent.h"
#include "Rendering/SkyAtmosphereCommonData.h"
#include "Components/ModelComponent.h"
#include "Materials/MaterialInstanceConstant.h"
#include "EngineUtils.h"
#include "Editor.h"
#include "LevelEditorViewport.h"
#include "StaticMeshResources.h"
#include "LightMap.h"
#include "ShadowMap.h"
#include "LandscapeProxy.h"
#include "LandscapeComponent.h"
#include "Components/SplineMeshComponent.h"
#include "Lightmass/PrecomputedVisibilityVolume.h"
#include "Lightmass/PrecomputedVisibilityOverrideVolume.h"
#include "ComponentReregisterContext.h"
#include "ShaderCompiler.h"
#include "Engine/LevelStreaming.h"
#include "UnrealEngine.h"
#include "ComponentRecreateRenderStateContext.h"
#include "EditorLevelUtils.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "ImageCoreUtils.h"
#include "Misc/FileHelper.h"
#include "StaticLightingBuildContext.h"
#include "Engine/InstancedStaticMesh.h"
#include "JsonObjectConverter.h"

extern FSwarmDebugOptions GSwarmDebugOptions;

//@todo_ow: Potentially move to core for sharing ?
class FObjectMemoryWriter : public  FMemoryWriter 
{
public:
	FObjectMemoryWriter(TArray<uint8>& Payload) : 
		FMemoryWriter(Payload)
	{
	}

	virtual FArchive& operator<<(struct FSoftObjectPath& Value)
	{
		FString FullPath = Value.ToString();
		*this << FullPath;

		return *this;
	}
};

class FObjectMemoryReader : public  FMemoryReader
{
public:
	FObjectMemoryReader(TArray<uint8>& Payload) :
		FMemoryReader(Payload)
	{
	}

	virtual FArchive& operator<<(struct FSoftObjectPath& Value)
	{
		FString FullPath;
		*this << FullPath;
		Value = FSoftObjectPath(FullPath);

		return *this;
	}
};

struct FDeferredMappingsBundle
{
	// Map of all external GUIDs to internal mapping GUID in this bundle 
	// This is used to map the owners actors to the mappings
	TMap<FGuid, FGuid> OwnerToMapping;
	TMap<FGuid, const FLightmassProcessor::FTextureMappingImportHelper*> MappingGuidToHelper;
	TArray<FLightmassProcessor::FTextureMappingImportHelper> Mappings;
	
	void Serialize(FArchive& Ar, bool bManifestOnly)
	{
		Ar << OwnerToMapping;
		
		if (!bManifestOnly)
		{
			Ar << Mappings;
		}

	}

	void MapHelpers()
	{
		MappingGuidToHelper.Reset();
		OwnerToMapping.Reset();

		for (const FLightmassProcessor::FTextureMappingImportHelper& Helper : Mappings)
		{
			MappingGuidToHelper.Add(Helper.MappingGuid, &Helper);
			OwnerToMapping.Add(Helper.OwnerGuid, Helper.MappingGuid);
		}
	}
};




DEFINE_LOG_CATEGORY_STATIC(LogLightmassSolver, Warning, All);
/**
 * If false (default behavior), Lightmass is launched automatically when a lighting build starts.
 * If true, it must be launched manually (e.g. through a debugger).
 */
UNREALED_API bool GLightmassDebugMode = false; 

/** If true, all participating Lightmass agents will report back detailed stats to the log. */
UNREALED_API bool GLightmassStatsMode = false;

/*-----------------------------------------------------------------------------
	FSwarmDebugOptions
-----------------------------------------------------------------------------*/
// Doxygen cannot parse this since FSwarmDebugOptions is declared in an engine header
#if !UE_BUILD_DOCS
void FSwarmDebugOptions::Touch()
{
	//@todo For some reason, the global instance is not initializing to the default settings...
	if (bInitialized == false)
	{
		bDistributionEnabled = true;
		bForceContentExport = false;

		bInitialized = true;
	}
}
#endif

#include "ImportExport.h"
#include "MaterialExport.h"
#include "StatsViewerModule.h"
#include "LightingBuildInfo.h"
#include "Logging/TokenizedMessage.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"

#define LOCTEXT_NAMESPACE "Lightmass"

/** The number of available mappings to process before yielding back to the importing. */
int32 FLightmassProcessor::MaxProcessAvailableCount = 8;

/** We don't want any amortization steps to take longer than this amount every tick */
static const float AllowedAmortizationTimePerTick = 0.01f; // in seconds

volatile int32 FLightmassProcessor::VolumeSampleTaskCompleted = 0;
volatile int32 FLightmassProcessor::MeshAreaLightDataTaskCompleted = 0;
volatile int32 FLightmassProcessor::VolumeDistanceFieldTaskCompleted = 0;

/** Flags to use when opening the different kinds of input channels */
/** MUST PAIR APPROPRIATELY WITH THE SAME FLAGS IN LIGHTMASS */
static NSwarm::TChannelFlags LM_TEXTUREMAPPING_CHANNEL_FLAGS	= NSwarm::SWARM_JOB_CHANNEL_READ;
static NSwarm::TChannelFlags LM_VERTEXMAPPING_CHANNEL_FLAGS		= NSwarm::SWARM_JOB_CHANNEL_READ;
static NSwarm::TChannelFlags LM_VOLUMESAMPLES_CHANNEL_FLAGS		= NSwarm::SWARM_JOB_CHANNEL_READ;
static NSwarm::TChannelFlags LM_PRECOMPUTEDVISIBILITY_CHANNEL_FLAGS	= NSwarm::SWARM_JOB_CHANNEL_READ;
static NSwarm::TChannelFlags LM_VOLUMEDEBUGOUTPUT_CHANNEL_FLAGS	= NSwarm::SWARM_JOB_CHANNEL_READ;
static NSwarm::TChannelFlags LM_DOMINANTSHADOW_CHANNEL_FLAGS	= NSwarm::SWARM_JOB_CHANNEL_READ;
static NSwarm::TChannelFlags LM_MESHAREALIGHT_CHANNEL_FLAGS		= NSwarm::SWARM_JOB_CHANNEL_READ;
static NSwarm::TChannelFlags LM_DEBUGOUTPUT_CHANNEL_FLAGS		= NSwarm::SWARM_JOB_CHANNEL_READ;

/** Flags to use when opening the different kinds of output channels */
/** MUST PAIR APPROPRIATELY WITH THE SAME FLAGS IN LIGHTMASS */
#if LM_COMPRESS_INPUT_DATA
	static NSwarm::TChannelFlags LM_SCENE_CHANNEL_FLAGS			= (NSwarm::TChannelFlags)(NSwarm::SWARM_JOB_CHANNEL_WRITE | NSwarm::SWARM_CHANNEL_MISC_ENABLE_COMPRESSION);
	static NSwarm::TChannelFlags LM_STATICMESH_CHANNEL_FLAGS	= (NSwarm::TChannelFlags)(NSwarm::SWARM_CHANNEL_WRITE | NSwarm::SWARM_CHANNEL_MISC_ENABLE_COMPRESSION);
	static NSwarm::TChannelFlags LM_TERRAIN_CHANNEL_FLAGS		= (NSwarm::TChannelFlags)(NSwarm::SWARM_CHANNEL_WRITE | NSwarm::SWARM_CHANNEL_MISC_ENABLE_COMPRESSION);
	static NSwarm::TChannelFlags LM_MATERIAL_CHANNEL_FLAGS		= (NSwarm::TChannelFlags)(NSwarm::SWARM_CHANNEL_WRITE | NSwarm::SWARM_CHANNEL_MISC_ENABLE_COMPRESSION);
#else
	static NSwarm::TChannelFlags LM_SCENE_CHANNEL_FLAGS			= NSwarm::SWARM_JOB_CHANNEL_WRITE;
	static NSwarm::TChannelFlags LM_STATICMESH_CHANNEL_FLAGS	= NSwarm::SWARM_CHANNEL_WRITE;
	static NSwarm::TChannelFlags LM_TERRAIN_CHANNEL_FLAGS		= NSwarm::SWARM_CHANNEL_WRITE;
	static NSwarm::TChannelFlags LM_MATERIAL_CHANNEL_FLAGS		= NSwarm::SWARM_CHANNEL_WRITE;
#endif
	
#define VERIFYLIGHTMASSINI(x)			{bool bSucceeded = x; if (!bSucceeded) { VerifyLightmassIni(#x,__FILE__,__LINE__); }}

void VerifyLightmassIni(const ANSICHAR* Code,const ANSICHAR* Filename,uint32 Line)
{
	if (FApp::IsUnattended())
	{
		UE_LOGF(LogLightmassSolver, Fatal,"%ls failed \n at %ls:%u",ANSI_TO_TCHAR(Code),ANSI_TO_TCHAR(Filename),Line);
	}
	else
	{
		FString Error = FString::Printf(
			TEXT("Fatal error: A required key was missing from BaseLightmass.ini.  This can happen if BaseLightmass.ini is overwritten with an old version.\n")
			TEXT("Create a DefaultLightmass.ini in your project and override just the values you need, then the overrides will continue to work on version upgrades.\n")
			TEXT("https://docs.unrealengine.com/latest/INT/Programming/Basics/ConfigurationFiles\n\n")
			TEXT("%s failed \n at %s:%u"),
			ANSI_TO_TCHAR(Code),ANSI_TO_TCHAR(Filename),Line);
		FMessageDialog::Open(EAppMsgType::Ok, FText::FromString(Error));
		FPlatformMisc::RequestExit(1);
	}
}

/*-----------------------------------------------------------------------------
	FLightmassExporter
-----------------------------------------------------------------------------*/


void Copy( const ULightComponentBase* In, Lightmass::FLightData& Out )
{	
	FMemory::Memzero(Out);

	Out.LightFlags = 0;
	if (In->CastShadows)
	{
		Out.LightFlags |= Lightmass::GI_LIGHT_CASTSHADOWS;
	}

	if (In->HasStaticLighting())
	{
		Out.LightFlags |= Lightmass::GI_LIGHT_HASSTATICSHADOWING;
		Out.LightFlags |= Lightmass::GI_LIGHT_HASSTATICLIGHTING;
	}
	else if (In->HasStaticShadowing())
	{
		Out.LightFlags |= Lightmass::GI_LIGHT_STORE_SEPARATE_SHADOW_FACTOR;
		Out.LightFlags |= Lightmass::GI_LIGHT_HASSTATICSHADOWING;
	}

	if (In->CastStaticShadows)
	{
		Out.LightFlags |= Lightmass::GI_LIGHT_CASTSTATICSHADOWS;
	}

	Out.Color = In->LightColor;

	// Set brightness here for light types that only derive from ULightComponentBase and not from ULightComponent
	Out.Brightness = In->Intensity;
	Out.Guid = In->LightGuid;
	Out.IndirectLightingScale = In->IndirectLightingIntensity;
}

void Copy( const ULightComponent* In, Lightmass::FLightData& Out )
{	
	Copy((const ULightComponentBase*)In, Out);

	const ULocalLightComponent* LocalLight = Cast<const ULocalLightComponent>(In);
	const UPointLightComponent* PointLight = Cast<const UPointLightComponent>(In);

	if( ( LocalLight && LocalLight->GetLightType() == LightType_Rect ) ||
		( PointLight && PointLight->bUseInverseSquaredFalloff ) )
	{
		Out.LightFlags |= Lightmass::GI_LIGHT_INVERSE_SQUARED;
	}

	if (In->GetLightmassSettings().bUseAreaShadowsForStationaryLight)
	{
		Out.LightFlags |= Lightmass::GI_LIGHT_USE_AREA_SHADOWS_FOR_SEPARATE_SHADOW_FACTOR;
	}

	Out.Brightness = In->ComputeLightBrightness();
	Out.Position = (FVector4f)In->GetLightPosition();
	Out.Direction = (FVector3f)In->GetDirection();

	if( In->bUseTemperature )
	{
		Out.Color *= In->GetColorTemperature();
	}
}

void CopyLightProfile( const ULightComponentBase* In, Lightmass::FLightData& Out, TArray< uint8 >& OutLightProfileTextureData )
{
	OutLightProfileTextureData.Empty( Lightmass::FLightData::LightProfileTextureDataSize );
	OutLightProfileTextureData.AddUninitialized( Lightmass::FLightData::LightProfileTextureDataSize );

	FMemory::Memset(OutLightProfileTextureData.GetData(), 0xff, OutLightProfileTextureData.Num() * OutLightProfileTextureData.GetTypeSize());
}

void CopyLightProfile( const ULightComponent* In, Lightmass::FLightData& Out, TArray< uint8 >& OutLightProfileTextureData )
{
	CopyLightProfile( (const ULightComponentBase*)In, Out, OutLightProfileTextureData );

	if(In->IESTexture)
	{
		FTextureSource& Source = In->IESTexture->Source;

		// The current IES importer only uses this input format
		// even if we change the actual texture format this shouldn't change
		if( Source.GetFormat() == TSF_RGBA16F &&
			Source.GetSizeX() * Source.GetSizeY() <= Lightmass::FLightData::LightProfileTextureDataSize )
		{
			Out.LightFlags |= Lightmass::GI_LIGHT_USE_LIGHTPROFILE;

			TArray64<uint8> MipData;

			verify( Source.GetMipData(MipData, 0) );

			const uint32 Width = FMath::Sqrt( static_cast<float>(Lightmass::FLightData::LightProfileTextureDataSize) );
			const uint32 Height = Lightmass::FLightData::LightProfileTextureDataSize / Width;

			for(uint32 y = 0; y < Height; ++y)
			{
				uint32 SourceY = FMath::Min( y, (uint32)Source.GetSizeY() - 1 ); // We'll repeat the data if the source is smaller than the destination (we used to have 1D textures for IES files)

				for(uint32 x = 0; x < Width; ++x)
				{
					uint32 SourceX = FMath::Min( x, (uint32)Source.GetSizeX() - 1 );

					FFloat16 HalfValue = *(FFloat16*)&MipData[ (SourceY * (uint32)Source.GetSizeX() * 8) + (SourceX * 8) ];
					float Value = HalfValue;
					OutLightProfileTextureData[y * Width + x] = (uint8)(Value * 255.f + 0.5f);
				}
			}
		}
	}
}

FORCEINLINE void Copy( const FSplineMeshParams& In, Lightmass::FSplineMeshParams& Out )
{
	Out.StartPos = FVector3f(In.StartPos);	//LWC_TODO: Precision loss
	Out.StartTangent = FVector3f(In.StartTangent);
	Out.StartScale = FVector2f(In.StartScale);	// LWC_TODO: Precision loss
	Out.StartRoll = In.StartRoll;
	Out.StartOffset = FVector2f(In.StartOffset);	// LWC_TODO: Precision loss
	Out.EndPos = FVector3f(In.EndPos);	//LWC_TODO: Precision loss
	Out.EndTangent = FVector3f(In.EndTangent);
	Out.EndScale = FVector2f(In.EndScale);	// LWC_TODO: Precision loss
	Out.EndOffset = FVector2f(In.EndOffset);	// LWC_TODO: Precision loss
	Out.EndRoll = In.EndRoll;
}

void FLightmassProcessor::ProcessAlertMessages()
{
	FScopeLock Lock(&SwarmCallbackMessagesSection);

	for (int32 MessageIndex = 0; MessageIndex < SwarmCallbackMessages.Num(); MessageIndex++)
	{
		FLightmassAlertMessage& AlertMessage = SwarmCallbackMessages[MessageIndex];

		UObject* Object = NULL;

		switch (AlertMessage.Type)
		{
		case Lightmass::SOURCEOBJECTTYPE_StaticMesh:
			Object = FindStaticMesh(AlertMessage.ObjectId);
			break;
		case Lightmass::SOURCEOBJECTTYPE_Mapping:
			const FStaticLightingMapping* FoundMapping = GetLightmassExporter()->FindMappingByGuid(AlertMessage.ObjectId);
			if (FoundMapping)
			{
				Object = FoundMapping->GetMappedObject();
			}
			break;
		}

		FText LocalizedMessage;
		const FText* LocalizedMessagePtr = Messages.Find( FString( AlertMessage.MessageText ) );
		if ( LocalizedMessagePtr == NULL )
		{
			LocalizedMessage = FText::FromString( FString( AlertMessage.MessageText ) );
			LocalizedMessagePtr = &LocalizedMessage;
		}

		FMessageLog("LightingResults").Message((EMessageSeverity::Type)AlertMessage.Severity)
			->AddToken(FUObjectToken::Create(Object))
			->AddToken(FTextToken::Create( *LocalizedMessagePtr ));
	}

	SwarmCallbackMessages.Empty();
}

/*-----------------------------------------------------------------------------
	SwarmCallback function
-----------------------------------------------------------------------------*/
void FLightmassProcessor::SwarmCallback( NSwarm::FMessage* CallbackMessage, void* CallbackData )
{
	FLightmassProcessor* Processor = (FLightmassProcessor*) CallbackData;
	double SwarmCallbackStartTime = FPlatformTime::Seconds();

	switch ( CallbackMessage->Type )
	{
		case NSwarm::MESSAGE_JOB_STATE:
		{
			NSwarm::FJobState* JobStateMessage = (NSwarm::FJobState*)CallbackMessage;
			switch (JobStateMessage->JobState)
			{
				case NSwarm::JOB_STATE_INVALID:
					Processor->bProcessingFailed = true;
					break;
				case NSwarm::JOB_STATE_RUNNING:
					break;
				case NSwarm::JOB_STATE_COMPLETE_SUCCESS:
					Processor->bProcessingSuccessful = true;
					break;
				case NSwarm::JOB_STATE_COMPLETE_FAILURE:
					Processor->bProcessingFailed = true;
					break;
				case NSwarm::JOB_STATE_KILLED:
					Processor->bProcessingFailed = true;
					break;
				default:
					break;
			}
		}
		break;

		case NSwarm::MESSAGE_TASK_STATE:
		{
			NSwarm::FTaskState* TaskStateMessage = (NSwarm::FTaskState*)CallbackMessage;
			switch (TaskStateMessage->TaskState)
			{
				case NSwarm::JOB_TASK_STATE_INVALID:
					// Consider this cause for failing the entire Job
					Processor->bProcessingFailed = true;
					break;
				case NSwarm::JOB_TASK_STATE_ACCEPTED:
					break;
				case NSwarm::JOB_TASK_STATE_REJECTED:
					// Consider this cause for failing the entire Job
					Processor->bProcessingFailed = true;
					break;
				case NSwarm::JOB_TASK_STATE_RUNNING:
					break;
				case NSwarm::JOB_TASK_STATE_COMPLETE_SUCCESS:
				{
					FGuid PrecomputedVolumeLightingGuid = Lightmass::PrecomputedVolumeLightingGuid;
					FGuid MeshAreaLightDataGuid = Lightmass::MeshAreaLightDataGuid;
					FGuid VolumeDistanceFieldGuid = Lightmass::VolumeDistanceFieldGuid;
					if (TaskStateMessage->TaskGuid == PrecomputedVolumeLightingGuid)
					{
						FPlatformAtomics::InterlockedIncrement( &VolumeSampleTaskCompleted );
						FPlatformAtomics::InterlockedIncrement( &Processor->NumCompletedTasks );
					}
					else if (Processor->Exporter->VisibilityBucketGuids.Contains(TaskStateMessage->TaskGuid))
					{
						TList<FGuid>* NewElement = new TList<FGuid>(TaskStateMessage->TaskGuid, NULL);
						Processor->CompletedVisibilityTasks.AddElement( NewElement );
						FPlatformAtomics::InterlockedIncrement( &Processor->NumCompletedTasks );
					}
					else if (Processor->Exporter->VolumetricLightmapTaskGuids.Contains(TaskStateMessage->TaskGuid))
					{
						TList<FGuid>* NewElement = new TList<FGuid>(TaskStateMessage->TaskGuid, NULL);
						Processor->CompletedVolumetricLightmapTasks.AddElement( NewElement );
						FPlatformAtomics::InterlockedIncrement( &Processor->NumCompletedTasks );
						FPlatformAtomics::InterlockedIncrement( &Processor->NumCompletedVolumetricLightmapTasks );
					}
					else if (TaskStateMessage->TaskGuid == MeshAreaLightDataGuid)
					{
						FPlatformAtomics::InterlockedIncrement( &MeshAreaLightDataTaskCompleted );
						FPlatformAtomics::InterlockedIncrement( &Processor->NumCompletedTasks );
					}
					else if (TaskStateMessage->TaskGuid == VolumeDistanceFieldGuid)
					{
						FPlatformAtomics::InterlockedIncrement( &VolumeDistanceFieldTaskCompleted );
						FPlatformAtomics::InterlockedIncrement( &Processor->NumCompletedTasks );
					}
					else
					{
						// Add a mapping to the list of mapping GUIDs that have been completed
						TList<FGuid>* NewElement = new TList<FGuid>(TaskStateMessage->TaskGuid, NULL);
						Processor->CompletedMappingTasks.AddElement( NewElement );
						FPlatformAtomics::InterlockedIncrement( &Processor->NumCompletedTasks );
					}
					break;
				}
				case NSwarm::JOB_TASK_STATE_COMPLETE_FAILURE:
				{
					// Add a mapping to the list of mapping GUIDs that have been completed
					TList<FGuid>* NewElement = new TList<FGuid>(TaskStateMessage->TaskGuid, NULL);
					Processor->CompletedMappingTasks.AddElement( NewElement );
					FPlatformAtomics::InterlockedIncrement( &Processor->NumCompletedTasks );

					// Consider this cause for failing the entire Job
					Processor->bProcessingFailed = true;

					break;
				}
				case NSwarm::JOB_TASK_STATE_KILLED:
					break;
				default:
					break;
			}
		}
		break;

		case NSwarm::MESSAGE_INFO:
		{
#if !NO_LOGGING && !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			NSwarm::FInfoMessage* InfoMessage = (NSwarm::FInfoMessage*)CallbackMessage;
			GLog->Log( InfoMessage->TextMessage );
#endif
		}
		break;

		case NSwarm::MESSAGE_ALERT:
		{
			NSwarm::FAlertMessage* AlertMessage = (NSwarm::FAlertMessage*)CallbackMessage;
			EMessageSeverity::Type CheckType = EMessageSeverity::Info;

			switch (AlertMessage->AlertLevel)
			{
			case NSwarm::ALERT_LEVEL_INFO:	break;
			case NSwarm::ALERT_LEVEL_WARNING:			CheckType = EMessageSeverity::Warning;			break;
			case NSwarm::ALERT_LEVEL_ERROR:				CheckType = EMessageSeverity::Error;			break;
			}

			FGuid ObjectGuid = FGuid(	AlertMessage->ObjectGuid.A, 
										AlertMessage->ObjectGuid.B,
										AlertMessage->ObjectGuid.C,
										AlertMessage->ObjectGuid.D);
			
			{
				// Enqueue the message for the main thread to process, because FMessageLog isn't thread safe
				FScopeLock Lock(&Processor->SwarmCallbackMessagesSection);
				FLightmassAlertMessage NewMessage;
				NewMessage.ObjectId = ObjectGuid;
				NewMessage.MessageText = AlertMessage->TextMessage;
				NewMessage.Type = AlertMessage->TypeId;
				NewMessage.Severity = CheckType;
				Processor->SwarmCallbackMessages.Push(NewMessage);
			}
		}
		break;

		case NSwarm::MESSAGE_QUIT:
		{
			Processor->bQuitReceived = true;
		}
		break;
	}

	Processor->Statistics.SwarmCallbackTime += FPlatformTime::Seconds() - SwarmCallbackStartTime;
}

/*-----------------------------------------------------------------------------
	FLightmassExporter
-----------------------------------------------------------------------------*/
FLightmassExporter::FLightmassExporter(const FStaticLightingBuildContext& Context)
	: Swarm( NSwarm::FSwarmInterface::Get() ) 
	, SkyAtmosphereComponent(nullptr)
	, ExportStage(NotRunning)
	, CurrentAmortizationIndex(0)
	, OpenedMaterialExportChannels()
	, World( Context.World )
	, LightingContext(Context)
{
	// We must have a valid world
	check( World );

	if (GLightmassDebugOptions.bDebugMode)
	{
		SceneGuid = FGuid(0x0123, 0x4567, 0x89AB, 0xCDEF);
	}
	else
	{
		SceneGuid = FGuid::NewGuid();
	}

	// Compute the hash of UnrealLightmass executable and store it into LightmassExecutableHash
	// Guaranteed success as the executable has been checked by CheckLightmassExecutableVersion()
	TArray<uint8> Bytes;
	FFileHelper::LoadFileToArray(Bytes, *FPlatformProcess::GenerateApplicationPath(TEXT("UnrealLightmass"), EBuildConfiguration::Development));
	FSHA1::HashBuffer(&Bytes[0], Bytes.Num(), LightmassExecutableHash.Hash);
	
	ChannelName = Lightmass::CreateChannelName(SceneGuid, Lightmass::LM_SCENE_VERSION, Lightmass::LM_SCENE_EXTENSION);
}

FLightmassExporter::~FLightmassExporter()
{
	// clean up any opened channels that are opened during export
	if (ExportStage == ExportMaterials)
	{
		for (int32 i = 0; i < OpenedMaterialExportChannels.Num(); ++i)
		{
			int32 Result = Swarm.CloseChannel(OpenedMaterialExportChannels[i]);
		}
	}
	else if (ExportStage == CleanupMaterialExport)
	{
		for (int32 i = CurrentAmortizationIndex; i < OpenedMaterialExportChannels.Num(); ++i)
		{
			int32 Result = Swarm.CloseChannel(OpenedMaterialExportChannels[i]);
		}
	}
}

void FLightmassExporter::AddMaterial(UMaterialInterface* InMaterialInterface, const FStaticLightingMesh* InStaticLightingMesh /*= nullptr*/)
{
	if (InMaterialInterface)
	{
		FLightmassMaterialExportSettings ExportSettings = { InStaticLightingMesh };

		if (auto* ExistingExportSettings = MaterialExportSettings.Find(InMaterialInterface))
		{
			ensureMsgf(ExportSettings == *ExistingExportSettings, TEXT("Attempting to add the same material twice with different export settings, this is not (currently) supported"));
			return;
		}

		// Check for material texture changes...
		//@TODO: Add package to warning list if it needs to be resaved (perf warning)
		InMaterialInterface->UpdateLightmassTextureTracking();

		Materials.Add(InMaterialInterface);
		MaterialExportSettings.Add(InMaterialInterface, ExportSettings);
	}
}

const FStaticLightingMapping* FLightmassExporter::FindMappingByGuid(FGuid FindGuid) const
{
	for( int32 MappingIdx=0; MappingIdx < BSPSurfaceMappings.Num(); MappingIdx++ )
	{
		const FStaticLightingMapping* CurrentMapping = BSPSurfaceMappings[MappingIdx];
		if (CurrentMapping->GetLightingGuid() == FindGuid)
		{
			return CurrentMapping;
		}
	}

	for( int32 MappingIdx=0; MappingIdx < StaticMeshTextureMappings.Num(); MappingIdx++ )
	{
		const FStaticLightingMapping* CurrentMapping = StaticMeshTextureMappings[MappingIdx];
		if (CurrentMapping->GetLightingGuid() == FindGuid)
		{
			return CurrentMapping;
		}
	}

	for( int32 MappingIdx=0; MappingIdx < LandscapeTextureMappings.Num(); MappingIdx++ )
	{
		const FStaticLightingMapping* CurrentMapping = LandscapeTextureMappings[MappingIdx];
		if (CurrentMapping->GetLightingGuid() == FindGuid)
		{
			return CurrentMapping;
		}
	}

	for( int32 MappingIdx=0; MappingIdx < VolumeMappings.Num(); MappingIdx++ )
	{
		const FStaticLightingMapping* CurrentMapping = VolumeMappings[MappingIdx];
		if (CurrentMapping->GetLightingGuid() == FindGuid)
		{
			return CurrentMapping;
		}
	}

	for( int32 MappingIdx=0; MappingIdx < LandscapeVolumeMappings.Num(); MappingIdx++ )
	{
		const FStaticLightingMapping* CurrentMapping = LandscapeVolumeMappings[MappingIdx];
		if (CurrentMapping->GetLightingGuid() == FindGuid)
		{
			return CurrentMapping;
		}
	}

	return NULL;
}

void FLightmassExporter::WriteToChannel( FLightmassStatistics& Stats, FGuid& DebugMappingGuid )
{
	// Initialize the debug mapping Guid to something not in the scene.
	DebugMappingGuid = FGuid(0x96dc6516, 0xa616421d, 0x82f0ef5b, 0x299152b5);
	if( bSwarmConnectionIsValid )
	{
		int32 Channel = Swarm.OpenChannel( *ChannelName, LM_SCENE_CHANNEL_FLAGS );
		if( Channel >= 0 )
		{
			// Ensure the default material is present...
			AddMaterial(UMaterial::GetDefaultMaterial(MD_Surface));

			TotalProgress = 
				DirectionalLights.Num() + PointLights.Num() + SpotLights.Num() + RectLights.Num() + SkyLights.Num() + 
				StaticMeshes.Num() + StaticMeshLightingMeshes.Num() + StaticMeshTextureMappings.Num() + 
				BSPSurfaceMappings.Num() + VolumeMappings.Num() + Materials.Num() + 
				+ LandscapeLightingMeshes.Num() + LandscapeTextureMappings.Num() + LandscapeVolumeMappings.Num();

			CurrentProgress = 0;

			// Export scene header.
			Lightmass::FSceneFileHeader Scene;
			Scene.Cookie = 'SCEN';
			Scene.FormatVersion = FGuid( 0, 0, 0, 1 );
			Scene.Guid = FGuid( 0, 0, 0, 1 );

			WriteSceneSettings(Scene);
			WriteDebugInput(Scene.DebugInput, DebugMappingGuid);

			/** If true, pad the mappings (shrink the requested size and then pad) */
			Scene.bPadMappings = GLightmassDebugOptions.bPadMappings;
			Scene.bDebugPadding = GLightmassDebugOptions.bDebugPaddings;
			Scene.ExecutionTimeDivisor = GLightmassDebugOptions.ExecutionTimeDivisor;
			Scene.bColorByExecutionTime = GLightmassDebugOptions.bColorByExecutionTime;
			Scene.bUseRandomColors = GLightmassDebugOptions.bUseRandomColors;
			Scene.bColorBordersGreen = GLightmassDebugOptions.bColorBordersGreen;
			Scene.bOnlyCalcDebugTexelMappings = GLightmassDebugOptions.bOnlyCalcDebugTexelMappings;

			Scene.NumImportanceVolumes = ImportanceVolumes.Num();
			Scene.NumCharacterIndirectDetailVolumes = CharacterIndirectDetailVolumes.Num();
			Scene.NumVolumetricLightmapDensityVolumes = VolumetricLightmapDensityVolumes.Num();
			Scene.NumPortals = Portals.Num();
			Scene.NumDirectionalLights = DirectionalLights.Num();
			Scene.NumPointLights = PointLights.Num();
			Scene.NumSpotLights = SpotLights.Num();
			Scene.NumRectLights = RectLights.Num();
			Scene.NumSkyLights = SkyLights.Num();
			Scene.NumStaticMeshes = StaticMeshes.Num();
			Scene.NumStaticMeshInstances = StaticMeshLightingMeshes.Num();
			Scene.NumFluidSurfaceInstances = 0;
			Scene.NumLandscapeInstances = LandscapeLightingMeshes.Num();
			Scene.NumBSPMappings = BSPSurfaceMappings.Num();
			Scene.NumStaticMeshTextureMappings = StaticMeshTextureMappings.Num();
			Scene.NumFluidSurfaceTextureMappings = 0;
			Scene.NumLandscapeTextureMappings = LandscapeTextureMappings.Num();
			Scene.NumSpeedTreeMappings = 0;
			Scene.NumVolumeMappings = VolumeMappings.Num();
			Scene.NumLandscapeVolumeMappings = LandscapeVolumeMappings.Num();
			Scene.NumPrecomputedVisibilityBuckets = VisibilityBucketGuids.Num();
			Scene.NumVolumetricLightmapTasks = VolumetricLightmapTaskGuids.Num();
			Swarm.WriteChannel( Channel, &Scene, sizeof(Scene) );

			const int32 UserNameLength = FCString::Strlen(FPlatformProcess::UserName());
			Swarm.WriteChannel( Channel, &UserNameLength, sizeof(UserNameLength) );
			Swarm.WriteChannel( Channel, FPlatformProcess::UserName(), UserNameLength * sizeof(TCHAR) );

			const int32 LevelNameLength = LevelName.Len();
			Swarm.WriteChannel( Channel, &LevelNameLength, sizeof(LevelNameLength) );
			Swarm.WriteChannel( Channel, *LevelName, LevelName.Len() * sizeof(TCHAR) );

			for (int32 VolumeIndex = 0; VolumeIndex < ImportanceVolumes.Num(); VolumeIndex++)
			{
				FBox3f LMBox(ImportanceVolumes[VolumeIndex]);
				Swarm.WriteChannel(Channel, &LMBox, sizeof(LMBox));
			}

			for (int32 VolumeIndex = 0; VolumeIndex < CharacterIndirectDetailVolumes.Num(); VolumeIndex++)
			{
				FBox3f LMBox(CharacterIndirectDetailVolumes[VolumeIndex]);
				Swarm.WriteChannel(Channel, &LMBox, sizeof(LMBox));
			}

			for (int32 PortalIndex = 0; PortalIndex < Portals.Num(); PortalIndex++)
			{
				FMatrix44f Matrix(Portals[PortalIndex]);
				Swarm.WriteChannel(Channel, &Matrix, sizeof(Matrix));
			}

			{
				FLightmassStatistics::FScopedGather VisibilityStat(Stats.ExportVisibilityDataTime);
				WriteVisibilityData(Channel);
			}
			{
				FLightmassStatistics::FScopedGather VisibilityStat(Stats.ExportVolumetricLightmapDataTime);
				WriteVolumetricLightmapData(Channel);
			}
			{
				FLightmassStatistics::FScopedGather LightStat(Stats.ExportLightsTime);
				WriteLights( Channel );
			}
			{
				FLightmassStatistics::FScopedGather ModelStat(Stats.ExportModelsTime);
				WriteModels();
			}
			{
				FLightmassStatistics::FScopedGather StaticMeshStat(Stats.ExportStaticMeshesTime);
				WriteStaticMeshes();
			}
			{
				FLightmassStatistics::FScopedGather MeshInstanceStat(Stats.ExportMeshInstancesTime);
				WriteMeshInstances( Channel );
			}
			{
				FLightmassStatistics::FScopedGather LandscapeInstanceStat(Stats.ExportLandscapeInstancesTime);
				WriteLandscapeInstances( Channel );
			}
			{
				FLightmassStatistics::FScopedGather MappingStat(Stats.ExportMappingsTime);
				WriteMappings( Channel );
			}

			Swarm.CloseChannel( Channel );
		}
		else
		{
			UE_LOGF(LogLightmassSolver, Log,  "Error, OpenChannel failed to open %ls with error code %d", *ChannelName, Channel );
		}
	}
}

bool FLightmassExporter::WriteToMaterialChannel(FLightmassStatistics& Stats)
{
	if (bSwarmConnectionIsValid && !GEditor->GetMapBuildCancelled())
	{
		if (ExportStage == NotRunning) {ExportStage = BuildMaterials;}

		double ExportTime = 0.0;

		while (ExportTime < AllowedAmortizationTimePerTick && ExportStage != Complete)
		{
			FLightmassStatistics::FScopedGather ExportStat(ExportTime);
			switch (ExportStage)
			{
			case BuildMaterials:
				{
					if (CurrentAmortizationIndex >= Materials.Num()) 
					{
						ExportStage = ShaderCompilation; 
						CurrentAmortizationIndex = 0;
					}
					else
					{
						BuildMaterialMap(Materials[CurrentAmortizationIndex]);
						++CurrentAmortizationIndex;
					}
				}
				break;
			case ShaderCompilation:
				{
					BlockOnShaderCompilation();
					ExportStage = ExportMaterials;
					CurrentAmortizationIndex = 0;
				}
				break;
			case ExportMaterials:
				{
					if (CurrentAmortizationIndex >= Materials.Num()) 
					{
						ExportStage = CleanupMaterialExport; 
						CurrentAmortizationIndex = 0;
					}
					else
					{
						auto* CurrentMaterial = Materials[CurrentAmortizationIndex];
						ExportMaterial(CurrentMaterial, MaterialExportSettings.FindChecked(CurrentMaterial));
						++CurrentAmortizationIndex;
					}
				}
				break;
			case CleanupMaterialExport:
				{
					if (CurrentAmortizationIndex >= OpenedMaterialExportChannels.Num()) 
					{
						ExportStage = Complete; 
						CurrentAmortizationIndex = 0;
					}
					else
					{
						Swarm.CloseChannel(OpenedMaterialExportChannels[CurrentAmortizationIndex]);
						++CurrentAmortizationIndex;
					}
				}
				break;
			default: UE_LOGF(LogLightmassSolver, Fatal, "Invalid amortization stage hit.");
			}
		}

		Stats.ExportMaterialsTime += ExportTime;

		return ExportStage == Complete;
	}
	else
	{
		return true;
	}
}

float FLightmassExporter::GetAmortizedExportPercentDone() const
{
	int32 EstimatedTotalTaskCount = Materials.Num() * 3;
	int32 CurrentTaskId = ExportStage == BuildMaterials ? CurrentAmortizationIndex:
		ExportStage == ShaderCompilation ? Materials.Num():
		ExportStage == ExportMaterials ? (Materials.Num() + CurrentAmortizationIndex) :
		ExportStage == CleanupMaterialExport ? (Materials.Num() * 2 + CurrentAmortizationIndex) : EstimatedTotalTaskCount;
	return static_cast<float>(CurrentTaskId) / EstimatedTotalTaskCount;
}

void FLightmassExporter::WriteVisibilityData( int32 Channel )
{
	Swarm.WriteChannel( Channel, VisibilityBucketGuids.GetData(), VisibilityBucketGuids.Num() * VisibilityBucketGuids.GetTypeSize() );

	int32 NumVisVolumes = 0;
	for( TObjectIterator<APrecomputedVisibilityVolume> It; It; ++It )
	{
		if (World->ContainsActor(*It) && IsValid(*It))
		{
			NumVisVolumes++;
		}
	}

	if (World->GetWorldSettings()->bPrecomputeVisibility 
		&& NumVisVolumes == 0
		&& World->GetWorldSettings()->bPlaceCellsOnlyAlongCameraTracks == false)
	{
		FMessageLog("LightingResults").Error(LOCTEXT("LightmassError_MissingPrecomputedVisibilityVolume", "Level has bPrecomputeVisibility enabled but no Precomputed Visibility Volumes, precomputed visibility will not be effective."));
	}

	// Export the visibility volumes that indicate to lightmass where to place visibility cells
	Swarm.WriteChannel( Channel, &NumVisVolumes, sizeof(NumVisVolumes) );
	for( TObjectIterator<APrecomputedVisibilityVolume> It; It; ++It )
	{
		APrecomputedVisibilityVolume* Volume = *It;
		if (World->ContainsActor(Volume) && IsValid(Volume))
		{
			FBox3f LMBox(Volume->GetComponentsBoundingBox(true));
			Swarm.WriteChannel(Channel, &LMBox, sizeof(LMBox));

			TArray<FPlane> Planes4d;
			Volume->Brush->GetSurfacePlanes(Volume, Planes4d);			
			TArray<FPlane4f> Planes;
			for (FPlane Plane4d : Planes4d)
			{
				Planes.Add(FPlane4f(Plane4d));
			}
			const int32 NumPlanes = Planes.Num();
			Swarm.WriteChannel( Channel, &NumPlanes, sizeof(NumPlanes) );
			Swarm.WriteChannel( Channel, Planes.GetData(), Planes.Num() * Planes.GetTypeSize() );
		}
	}

	int32 NumOverrideVisVolumes = 0;
	for( TObjectIterator<APrecomputedVisibilityOverrideVolume> It; It; ++It )
	{
		if (World->ContainsActor(*It) && IsValid(*It))
		{
			NumOverrideVisVolumes++;
		}
	}

	Swarm.WriteChannel( Channel, &NumOverrideVisVolumes, sizeof(NumOverrideVisVolumes) );
	for( TObjectIterator<APrecomputedVisibilityOverrideVolume> It; It; ++It )
	{
		APrecomputedVisibilityOverrideVolume* Volume = *It;
		if (World->ContainsActor(Volume) && IsValid(Volume))
		{
			FBox3f LMBox(Volume->GetComponentsBoundingBox(true));
			Swarm.WriteChannel(Channel, &LMBox, sizeof(LMBox));

			TArray<int32> VisibilityIds;
			for (int32 ActorIndex = 0; ActorIndex < Volume->OverrideVisibleActors.Num(); ActorIndex++)
			{
				AActor* CurrentActor = Volume->OverrideVisibleActors[ActorIndex];
				if (CurrentActor)
				{
					for (UActorComponent* Component : CurrentActor->GetComponents())
					{
						UPrimitiveComponent* CurrentComponent = Cast<UPrimitiveComponent>(Component);
						if (CurrentComponent && (CurrentComponent->Mobility == EComponentMobility::Static) && CurrentComponent->VisibilityId != INDEX_NONE)
						{
							VisibilityIds.AddUnique(CurrentComponent->VisibilityId);
						}
					}
				}
			}
			TArray<int32> InvisibilityIds;
			for (int32 RemoveActorIndex = 0; RemoveActorIndex < Volume->OverrideInvisibleActors.Num(); RemoveActorIndex++)
			{
				AActor* RemoveActor = Volume->OverrideInvisibleActors[RemoveActorIndex];
				if (RemoveActor)
				{
					for (UActorComponent* Component : RemoveActor->GetComponents())
					{
						UPrimitiveComponent* RemoveComponent = Cast<UPrimitiveComponent>(Component);
						if (RemoveComponent && (RemoveComponent->Mobility == EComponentMobility::Static) && RemoveComponent->VisibilityId != INDEX_NONE)
						{
							InvisibilityIds.AddUnique(RemoveComponent->VisibilityId);
						}
					}
				}
			}
			for (int32 RemoveLevelIndex = 0; RemoveLevelIndex < Volume->OverrideInvisibleLevels.Num(); RemoveLevelIndex++)
			{
				ULevelStreaming* LevelStreaming = World->GetLevelStreamingForPackageName(Volume->OverrideInvisibleLevels[RemoveLevelIndex]);
				ULevel* Level;
				if( LevelStreaming && (Level = LevelStreaming->GetLoadedLevel()) != NULL )
				{
					for (int32 RemoveActorIndex = 0; RemoveActorIndex < Level->Actors.Num(); RemoveActorIndex++)
					{
						AActor* RemoveActor = Level->Actors[RemoveActorIndex];
						if (RemoveActor)
						{
							for (UActorComponent* Component : RemoveActor->GetComponents())
							{
								UPrimitiveComponent* RemoveComponent = Cast<UPrimitiveComponent>(Component);
								if (RemoveComponent && (RemoveComponent->Mobility == EComponentMobility::Static) && RemoveComponent->VisibilityId != INDEX_NONE)
								{
									InvisibilityIds.AddUnique(RemoveComponent->VisibilityId);
								}
							}
						}
					}
				}
			}
			
			const int32 NumVisibilityIds = VisibilityIds.Num();
			Swarm.WriteChannel( Channel, &NumVisibilityIds, sizeof(NumVisibilityIds) );
			Swarm.WriteChannel( Channel, VisibilityIds.GetData(), VisibilityIds.Num() * VisibilityIds.GetTypeSize() );
			const int32 NumInvisibilityIds = InvisibilityIds.Num();
			Swarm.WriteChannel( Channel, &NumInvisibilityIds, sizeof(NumInvisibilityIds) );
			Swarm.WriteChannel( Channel, InvisibilityIds.GetData(), InvisibilityIds.Num() * InvisibilityIds.GetTypeSize() );
		}
	}

	const float CellSize = World->GetWorldSettings()->VisibilityCellSize;

	// This used to walk along Matinee sequences and write the positions for pre-computed visibility.
	// Leaving the swarm channel code here for fear of breaking swarm.
	TArray<FVector4f> CameraTrackPositions;
	int32 NumCameraPositions = CameraTrackPositions.Num();
	Swarm.WriteChannel( Channel, &NumCameraPositions, sizeof(NumCameraPositions) );
	Swarm.WriteChannel( Channel, CameraTrackPositions.GetData(), CameraTrackPositions.Num() * CameraTrackPositions.GetTypeSize() );
}

void FLightmassExporter::WriteVolumetricLightmapData( int32 Channel )
{
	for (int32 VolumeIndex = 0; VolumeIndex < VolumetricLightmapDensityVolumes.Num(); VolumeIndex++)
	{
		AVolumetricLightmapDensityVolume* DetailVolume = VolumetricLightmapDensityVolumes[VolumeIndex];

		Lightmass::FVolumetricLightmapDensityVolumeData VolumeData;
		VolumeData.Bounds = FBox3f(DetailVolume->GetComponentsBoundingBox(true));
		VolumeData.AllowedMipLevelRange = FIntPoint(DetailVolume->AllowedMipLevelRange.Min, DetailVolume->AllowedMipLevelRange.Max);

		TArray<FPlane> Planes4d;
		DetailVolume->Brush->GetSurfacePlanes(DetailVolume, Planes4d);
		TArray<FPlane4f> Planes;
		for (FPlane Plane4d : Planes4d)
		{
			Planes.Add(FPlane4f(Plane4d));
		}

		VolumeData.NumPlanes = Planes.Num();

		Swarm.WriteChannel( Channel, &VolumeData, sizeof(VolumeData) );
		Swarm.WriteChannel( Channel, Planes.GetData(), Planes.Num() * Planes.GetTypeSize() );
	}

	TArray<FGuid> VolumetricLightmapTaskGuidsArray;
	VolumetricLightmapTaskGuids.GenerateKeyArray(VolumetricLightmapTaskGuidsArray);
	Swarm.WriteChannel( Channel, VolumetricLightmapTaskGuidsArray.GetData(), VolumetricLightmapTaskGuidsArray.Num() * VolumetricLightmapTaskGuidsArray.GetTypeSize() );
}

void FLightmassExporter::WriteLights( int32 Channel )
{
	// Search for the sun light component the same way as in FScene::RemoveLightSceneInfo_RenderThread.
	// If found, we keep the pointer to apply atmosphere transmittance to it in the light loop.
	const UDirectionalLightComponent* AtmosphereLights[NUM_ATMOSPHERE_LIGHTS];
	float AtmosphereLightsBrightness[NUM_ATMOSPHERE_LIGHTS];
	FLinearColor SunLightAtmosphereTransmittance[NUM_ATMOSPHERE_LIGHTS];
	for (uint8 Index = 0; Index < NUM_ATMOSPHERE_LIGHTS; ++Index)
	{
		AtmosphereLights[Index] = nullptr;
		AtmosphereLightsBrightness[Index] = 0.0f;
		SunLightAtmosphereTransmittance[Index] = FLinearColor::White;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Compute a mapping between directional light and trnasmittance to apply. For each AtmosphereSunLightIndex, the brightest lights is kept.
	if (SkyAtmosphereComponent)
	{
		for (int32 LightIndex = 0; LightIndex < DirectionalLights.Num(); ++LightIndex)
		{
			const UDirectionalLightComponent* Light = DirectionalLights[LightIndex];
			if (Light->IsUsedAsAtmosphereSunLight() && Light->GetColoredLightBrightness().GetLuminance() > AtmosphereLightsBrightness[Light->GetAtmosphereSunLightIndex()])
			{
				AtmosphereLights[Light->GetAtmosphereSunLightIndex()] = Light;
				AtmosphereLightsBrightness[Light->GetAtmosphereSunLightIndex()] = Light->GetColoredLightBrightness().GetLuminance();
			}
		}

		for (uint8 Index = 0; Index < NUM_ATMOSPHERE_LIGHTS; ++Index)
		{
			if (AtmosphereLights[Index])
			{
				const FVector SunDirection = -AtmosphereLights[Index]->GetDirection();
				if (SkyAtmosphereComponent)
				{
					FAtmosphereSetup AtmosphereSetup(*SkyAtmosphereComponent);
					SunLightAtmosphereTransmittance[Index] = AtmosphereSetup.GetTransmittanceAtGroundLevel(SunDirection);
				}
				break;
			}
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Export directional lights.
	for ( int32 LightIndex = 0; LightIndex < DirectionalLights.Num(); ++LightIndex )
	{
		const UDirectionalLightComponent* Light = DirectionalLights[LightIndex];
		Lightmass::FLightData LightData;
		Lightmass::FDirectionalLightData DirectionalData;
		Copy( Light, LightData );
		LightData.IndirectLightingSaturation = Light->LightmassSettings.IndirectLightingSaturation;
		LightData.ShadowExponent = Light->LightmassSettings.ShadowExponent;
		LightData.ShadowResolutionScale = Light->ShadowResolutionScale;
		LightData.LightSourceRadius = 0;
		LightData.LightSourceLength = 0;

		// Apply atmosphere transmittance if needed.
		for (uint8 Index = 0; Index < NUM_ATMOSPHERE_LIGHTS; ++Index)
		{
			if (AtmosphereLights[Index] && AtmosphereLights[Index] == Light)
			{
				LightData.Color *= SunLightAtmosphereTransmittance[Index];
				break;
			}
		}

		TArray< uint8 > LightProfileTextureData;
		CopyLightProfile( Light, LightData, LightProfileTextureData );

		DirectionalData.LightSourceAngle = Light->LightmassSettings.LightSourceAngle * (float)PI / 180.0f;
		Swarm.WriteChannel( Channel, &LightData, sizeof(LightData) );
		Swarm.WriteChannel( Channel, LightProfileTextureData.GetData(), LightProfileTextureData.Num() * LightProfileTextureData.GetTypeSize() );
		Swarm.WriteChannel( Channel, &DirectionalData, sizeof(DirectionalData) );
		UpdateExportProgress();
	}

	// Export point lights.
	for ( int32 LightIndex = 0; LightIndex < PointLights.Num(); ++LightIndex )
	{
		const UPointLightComponent* Light = PointLights[LightIndex];
		Lightmass::FLightData LightData;
		Lightmass::FPointLightData PointData;
		Copy( Light, LightData );
		LightData.IndirectLightingSaturation = Light->LightmassSettings.IndirectLightingSaturation;
		LightData.ShadowExponent = Light->LightmassSettings.ShadowExponent;
		LightData.ShadowResolutionScale = Light->ShadowResolutionScale;
		LightData.LightSourceRadius = FMath::Max( 1.0f, Light->SourceRadius );
		LightData.LightSourceLength = Light->SourceLength;

		TArray< uint8 > LightProfileTextureData;
		CopyLightProfile( Light, LightData, LightProfileTextureData );

		PointData.Radius = Light->AttenuationRadius;
		PointData.FalloffExponent = Light->LightFalloffExponent;
		PointData.LightTangent = FVector3f(Light->GetComponentTransform().GetUnitAxis(EAxis::Z));
		Swarm.WriteChannel( Channel, &LightData, sizeof(LightData) );
		Swarm.WriteChannel( Channel, LightProfileTextureData.GetData(), LightProfileTextureData.Num() * LightProfileTextureData.GetTypeSize() );
		Swarm.WriteChannel( Channel, &PointData, sizeof(PointData) );
		UpdateExportProgress();
	}

	// Export spot lights.
	for ( int32 LightIndex = 0; LightIndex < SpotLights.Num(); ++LightIndex )
	{
		const USpotLightComponent* Light = SpotLights[LightIndex];
		Lightmass::FLightData LightData;
		Lightmass::FPointLightData PointData;
		Lightmass::FSpotLightData SpotData;
		Copy( Light, LightData ); 
		LightData.IndirectLightingSaturation = Light->LightmassSettings.IndirectLightingSaturation;
		LightData.ShadowExponent = Light->LightmassSettings.ShadowExponent;
		LightData.ShadowResolutionScale = Light->ShadowResolutionScale;
		LightData.LightSourceRadius = FMath::Max( 1.0f, Light->SourceRadius );
		LightData.LightSourceLength = Light->SourceLength;

		TArray< uint8 > LightProfileTextureData;
		CopyLightProfile( Light, LightData, LightProfileTextureData );

		PointData.Radius = Light->AttenuationRadius;
		PointData.FalloffExponent = Light->LightFalloffExponent;
		PointData.LightTangent = FVector3f(Light->GetComponentTransform().GetUnitAxis(EAxis::Z));
		SpotData.InnerConeAngle = Light->InnerConeAngle; 
		SpotData.OuterConeAngle = Light->OuterConeAngle;
		Swarm.WriteChannel( Channel, &LightData, sizeof(LightData) );
		Swarm.WriteChannel( Channel, LightProfileTextureData.GetData(), LightProfileTextureData.Num() * LightProfileTextureData.GetTypeSize() );
		Swarm.WriteChannel( Channel, &PointData, sizeof(PointData) );
		Swarm.WriteChannel( Channel, &SpotData, sizeof(SpotData) );
		UpdateExportProgress();
	}

	// Export rect lights.
	for ( int32 LightIndex = 0; LightIndex < RectLights.Num(); ++LightIndex )
	{
		const URectLightComponent* Light = RectLights[LightIndex];
		Lightmass::FLightData LightData;
		Copy( Light, LightData );
		LightData.IndirectLightingSaturation = Light->LightmassSettings.IndirectLightingSaturation;
		LightData.ShadowExponent = Light->LightmassSettings.ShadowExponent;
		LightData.ShadowResolutionScale = Light->ShadowResolutionScale;
		LightData.LightSourceRadius = 0.5f * Light->SourceWidth;
		LightData.LightSourceLength = 0.5f * Light->SourceHeight;

		TArray< uint8 > LightProfileTextureData;
		CopyLightProfile( Light, LightData, LightProfileTextureData );

		Lightmass::FPointLightData PointData;
		PointData.Radius = Light->AttenuationRadius;
		PointData.FalloffExponent = 0.0f;
		PointData.LightTangent = FVector3f(Light->GetComponentTransform().GetUnitAxis(EAxis::Z));

		Lightmass::FRectLightData RectData = { 0, 0, FLinearColor::White };
		// TODO export texture data. Below code is written for that in mind but source data doesn't contain mips.
		//TArray< FFloat16Color > SourceTexture;
		if( Light->SourceTexture )
		{
			FTextureSource& Source = Light->SourceTexture->Source;

			if( Source.IsValid() )
			{
				ETextureSourceFormat Format = Source.GetFormat();
				int32 BytesPerPixel = Source.GetBytesPerPixel();
				bool SRGB = Light->SourceTexture->SRGB;

				uint32 NumMips = Source.GetNumMips();
				uint32 MipLevel = NumMips - FMath::Min( NumMips, 8u );

				uint32 SizeX = FMath::Max( Source.GetSizeX() >> MipLevel, 1 );
				uint32 SizeY = FMath::Max( Source.GetSizeY() >> MipLevel, 1 );

				RectData.SourceTextureSizeX = SizeX;
				RectData.SourceTextureSizeY = SizeY;

				ERawImageFormat::Type RawFormat = FImageCoreUtils::ConvertToRawImageFormat(Format);

				if ( SizeX > 1 || SizeY > 1 )
				{
					TArray64< uint8 > MipData;
					verify( Source.GetMipData( MipData, MipLevel ) );

					uint8* Pixel = MipData.GetData();
					uint8* PixelEnd = Pixel + MipData.Num();
					while( Pixel < PixelEnd )
					{
						FLinearColor Color = ERawImageFormat::GetOnePixelLinear(Pixel,RawFormat,SRGB);
						RectData.SourceTextureAvgColor += Color;

						Pixel += BytesPerPixel;
					}

					RectData.SourceTextureAvgColor *= 1.0f / ( SizeX * SizeY );
				}
			}
		}

		Swarm.WriteChannel( Channel, &LightData, sizeof(LightData) );
		Swarm.WriteChannel( Channel, LightProfileTextureData.GetData(), LightProfileTextureData.Num() * LightProfileTextureData.GetTypeSize() );
		Swarm.WriteChannel( Channel, &PointData, sizeof(PointData) );
		Swarm.WriteChannel( Channel, &RectData, sizeof(RectData) );
		//Swarm.WriteChannel( Channel, SourceTexture.GetData(), SourceTexture.Num() * SourceTexture.GetTypeSize() );
		UpdateExportProgress();
	}

	// Export sky lights.
	for ( int32 LightIndex = 0; LightIndex < SkyLights.Num(); ++LightIndex )
	{
		const USkyLightComponent* Light = SkyLights[LightIndex];

		Lightmass::FLightData LightData;
		Lightmass::FSkyLightData SkyData;
		Copy( Light, LightData ); 

		TArray< uint8 > LightProfileTextureData;
		CopyLightProfile( Light, LightData, LightProfileTextureData );

		TArray<FFloat16Color> RadianceMap;

		// Capture the scene's emissive and send it to lightmass
		// Note: FLightmassProcessor::InitiateExport hid all but the current lighting scenario
		Light->CaptureEmissiveRadianceEnvironmentCubeMap(SkyData.IrradianceEnvironmentMap, RadianceMap);

		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.StaticLighting"), TEXT("bUseFilteredCubemapForSkylight"), SkyData.bUseFilteredCubemap, GLightmassIni));
		SkyData.RadianceEnvironmentMapDataSize = RadianceMap.Num();
		Swarm.WriteChannel( Channel, &LightData, sizeof(LightData) );
		Swarm.WriteChannel( Channel, LightProfileTextureData.GetData(), LightProfileTextureData.Num() * LightProfileTextureData.GetTypeSize() );
		Swarm.WriteChannel( Channel, &SkyData, sizeof(SkyData) );
		Swarm.WriteChannel( Channel, RadianceMap.GetData(), RadianceMap.Num() * RadianceMap.GetTypeSize() );
		UpdateExportProgress();
	}
}

/**
 * Exports all UModels to secondary, persistent channels
 */
void FLightmassExporter::WriteModels()
{
	for (int32 ModelIndex = 0; ModelIndex < Models.Num(); ModelIndex++)
	{
		
	}
}

/**
 * Exports all UStaticMeshes to secondary, persistent channels
 */
void FLightmassExporter::WriteStaticMeshes()
{
	// Export geometry resources.
	for (int32 StaticMeshIndex = 0; StaticMeshIndex < StaticMeshes.Num() && !GEditor->GetMapBuildCancelled(); StaticMeshIndex++)
	{
		const UStaticMesh* StaticMesh = StaticMeshes[StaticMeshIndex];

		Lightmass::FBaseMeshData BaseMeshData;
		BaseMeshData.Guid = StaticMesh->GetLightingGuid();

		// create a channel name to write the mesh out to
		FString NewChannelName = Lightmass::CreateChannelNameWithLMExecutableHash(BaseMeshData.Guid, Lightmass::LM_STATICMESH_VERSION, LightmassExecutableHash, Lightmass::LM_STATICMESH_EXTENSION);

		// Warn the user if there is an invalid lightmap UV channel specified.
		if( StaticMesh->GetLightMapCoordinateIndex() > 0
			&& StaticMesh->GetRenderData()
			&& StaticMesh->GetRenderData()->LODResources.Num() > 0 )
		{
			const FStaticMeshLODResources& RenderData = StaticMesh->GetRenderData()->LODResources[0];
			if( StaticMesh->GetLightMapCoordinateIndex() >= (int32)RenderData.VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords() )
			{
				FMessageLog("LightingResults").Warning()
					->AddToken(FUObjectToken::Create(const_cast<UStaticMesh*>(StaticMesh)))
					->AddToken(FTextToken::Create( NSLOCTEXT("Lightmass", "LightmassError_BadLightMapCoordinateIndex", "StaticMesh has invalid LightMapCoordinateIndex.") ) );
			}
		}

		// only export the static mesh if it's not currently in the cache
		if ((GSwarmDebugOptions.bForceContentExport == true) ||
			(Swarm.TestChannel(*NewChannelName) < 0))
		{
			// open the channel
			int32 Channel = Swarm.OpenChannel( *NewChannelName, LM_STATICMESH_CHANNEL_FLAGS );
			if( Channel >= 0 )
			{
				// write out base data
				Swarm.WriteChannel(Channel, &BaseMeshData, sizeof(BaseMeshData));

				Lightmass::FStaticMeshData StaticMeshData;
				StaticMeshData.LightmapCoordinateIndex = StaticMesh->GetLightMapCoordinateIndex();
				StaticMeshData.NumLODs = StaticMesh->GetRenderData()->LODResources.Num();
				Swarm.WriteChannel( Channel, &StaticMeshData, sizeof(StaticMeshData) );
				for (int32 LODIndex = 0; LODIndex < StaticMesh->GetRenderData()->LODResources.Num(); LODIndex++)
				{
					TArray<uint32> Indices;
					const FStaticMeshLODResources& RenderData = StaticMesh->GetRenderData()->LODResources[LODIndex];

					RenderData.IndexBuffer.GetCopy(Indices);
					Lightmass::FStaticMeshLODData SMLODData;
					SMLODData.NumElements = RenderData.Sections.Num();
					SMLODData.NumTriangles = RenderData.GetNumTriangles();
					SMLODData.NumIndices = Indices.Num();
					// the vertex buffer could have double vertices for shadow buffer data, so we use what the render data thinks it has, not what is actually there
					SMLODData.NumVertices = RenderData.VertexBuffers.StaticMeshVertexBuffer.GetNumVertices();
					Swarm.WriteChannel( Channel, &SMLODData, sizeof(SMLODData) );

					int32 NumSections = RenderData.Sections.Num();
					if( NumSections > 0 )
					{
						TArray<Lightmass::FStaticMeshElementData> LMElements;
						LMElements.Empty(NumSections);
						LMElements.AddZeroed(NumSections);
						for (int32 SectionIndex = 0; SectionIndex < NumSections; SectionIndex++)
						{
							const FStaticMeshSection& Section = RenderData.Sections[SectionIndex];
							Lightmass::FStaticMeshElementData& SMElementData = LMElements[SectionIndex];
							SMElementData.FirstIndex = Section.FirstIndex;
							SMElementData.NumTriangles = Section.NumTriangles;
							SMElementData.bEnableShadowCasting = Section.bCastShadow;
						}
						Swarm.WriteChannel( Channel, (void*)(LMElements.GetData()), NumSections * sizeof(Lightmass::FStaticMeshElementData) );
					}

					Swarm.WriteChannel( Channel, (void*)Indices.GetData(), Indices.Num() * sizeof(uint32) );
					
					int32 VertexCount = SMLODData.NumVertices;
					if( VertexCount > 0 )
					{
						TArray<Lightmass::FStaticMeshVertex> LMVertices;
						LMVertices.Empty(VertexCount);
						LMVertices.AddZeroed(VertexCount);
						for (int32 VertexIndex = 0; VertexIndex < VertexCount; VertexIndex++)
						{
							Lightmass::FStaticMeshVertex& Vertex = LMVertices[VertexIndex];
							Vertex.Position = FVector4f(RenderData.VertexBuffers.PositionVertexBuffer.VertexPosition(VertexIndex), 1.0f);
							Vertex.TangentX = FVector4f(RenderData.VertexBuffers.StaticMeshVertexBuffer.VertexTangentX(VertexIndex));
							Vertex.TangentY = FVector4f(RenderData.VertexBuffers.StaticMeshVertexBuffer.VertexTangentY(VertexIndex));
							Vertex.TangentZ = FVector4f(RenderData.VertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(VertexIndex));
							int32 UVCount = FMath::Clamp<int32>(RenderData.VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords(), 0, MAX_TEXCOORDS);
							int32 UVIndex;
							for (UVIndex = 0; UVIndex < UVCount; UVIndex++)
							{
								Vertex.UVs[UVIndex] = RenderData.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(VertexIndex, UVIndex);
							}

							for (; UVIndex < MAX_TEXCOORDS; UVIndex++)
							{
								Vertex.UVs[UVIndex] = FVector2f::ZeroVector;
							}
						}
						Swarm.WriteChannel( Channel, (void*)(LMVertices.GetData()), LMVertices.Num() * sizeof(Lightmass::FStaticMeshVertex));
					}
				}

				// close the channel, the whole mesh is now exported
				Swarm.CloseChannel(Channel);
			}
			else
			{
				UE_LOGF(LogLightmassSolver, Log,  "Error, OpenChannel failed to open %ls with error code %d", *NewChannelName, Channel );
			}
		}
		UpdateExportProgress();
	}
}

void FLightmassExporter::GetMaterialHash(const UMaterialInterface* Material, FSHAHash& OutHash)
{
	FSHA1 HashState;

	TArray<FGuid> MaterialGuids;
	Material->GetLightingGuidChain(true, MaterialGuids);
	MaterialGuids.Sort();

	FGuid LastGuid;
	for (const FGuid& MaterialGuid : MaterialGuids)
	{
		if (MaterialGuid != LastGuid)
		{
			HashState.Update((const uint8*)&MaterialGuid, sizeof(MaterialGuid));
			LastGuid = MaterialGuid;
		}
	}

	if (Substrate::IsSubstrateEnabled())
	{
		uint32 LightmassSubstrateVersion = 0XFA1081D0; // This can be change when the code/logic for converting material to Substrate for lightmap has changed.
		HashState.Update((const uint8*)&LightmassSubstrateVersion, sizeof(LightmassSubstrateVersion));
	}

	HashState.Final();
	HashState.GetHash(&OutHash.Hash[0]);
}

void FLightmassExporter::BuildMaterialMap(UMaterialInterface* Material)
{
	if (ensure(Material))
	{
		FSHAHash MaterialHash;
		GetMaterialHash(Material, MaterialHash);

		// create a channel name to write the material out to
		FString NewChannelName = Lightmass::CreateChannelNameWithLMExecutableHash(MaterialHash, Lightmass::LM_MATERIAL_VERSION, LightmassExecutableHash, Lightmass::LM_MATERIAL_EXTENSION);

		// only export the material if it's not currently in the cache
		int32 ErrorCode;
		if ( GSwarmDebugOptions.bForceContentExport == true )
		{
			// If we're forcing export of content, pretend we didn't find it
			ErrorCode = NSwarm::SWARM_ERROR_FILE_FOUND_NOT;
		}
		else
		{
			// Otherwise, test the channel
			ErrorCode = Swarm.TestChannel(*NewChannelName);
		}

		if ( ErrorCode != NSwarm::SWARM_SUCCESS )
		{
			if( ErrorCode == NSwarm::SWARM_ERROR_FILE_FOUND_NOT )
			{
				static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.NormalMapsForStaticLighting"));
				const bool bUseNormalMapsForLighting = CVar->GetValueOnGameThread() != 0;

				// Only generate normal maps if we'll actually need them for lighting
				MaterialRenderer.BeginGenerateMaterialData(Material, bUseNormalMapsForLighting, NewChannelName, MaterialExportData);
			}
			else
			{
				UE_LOGF(LogLightmassSolver, Warning, "Error in TestChannel() for %ls: %ls", *(MaterialHash.ToString()), *(Material->GetPathName()));
			}
		}
	}
}

void FLightmassExporter::BlockOnShaderCompilation()
{
	// Block until async shader compiling is finished before we try to use the shaders for exporting
	// The code is structured to only block once for all materials, so that shader compiling is able to utilize many cores
	GShaderCompilingManager->FinishAllCompilation();
}

void FLightmassExporter::ExportMaterial(UMaterialInterface* Material, const FLightmassMaterialExportSettings& ExportSettings)
{
	FMaterialExportDataEntry* ExportEntry = MaterialExportData.Find(Material);

	// Only create the Swarm channel if there is something to export
	if (ensure(Material) && ExportEntry)
	{
		Lightmass::FBaseMaterialData BaseMaterialData;
		BaseMaterialData.Guid = Material->GetLightingGuid();

		// Generate the required information
		Lightmass::FMaterialData MaterialData;
		FMemory::Memzero(&MaterialData,sizeof(MaterialData));
		UMaterial* BaseMaterial = Material->GetMaterial();
		MaterialData.bTwoSided = (uint32)Material->IsTwoSided();
		MaterialData.bIsThinSurface = (uint32)Material->IsThinSurface();
		MaterialData.EmissiveBoost = Material->GetEmissiveBoost();
		MaterialData.DiffuseBoost = Material->GetDiffuseBoost();

		TArray<FFloat16Color> MaterialEmissive;
		TArray<FFloat16Color> MaterialDiffuse;
		TArray<FFloat16Color> MaterialTransmission;
		TArray<FFloat16Color> MaterialNormal;

		if (MaterialRenderer.GenerateMaterialData(
			World->Scene,
			*Material,
			ExportSettings,
			MaterialData,
			*ExportEntry,
			MaterialDiffuse,
			MaterialEmissive,
			MaterialTransmission,
			MaterialNormal))
		{
			// open the channel
			int32 Channel = Swarm.OpenChannel( *(ExportEntry->ChannelName), LM_MATERIAL_CHANNEL_FLAGS );
			if( Channel >= 0 )
			{
				// write out base data
				Swarm.WriteChannel(Channel, &BaseMaterialData, sizeof(BaseMaterialData));

				// the material data
				Swarm.WriteChannel(Channel, &MaterialData, sizeof(MaterialData));

				// Write each array of data
				uint8* OutData;
				int32 OutSize;

				OutSize = FMath::Square(MaterialData.EmissiveSize) * sizeof(FFloat16Color);
				if (OutSize > 0)
				{
					OutData = (uint8*)(MaterialEmissive.GetData());
					Swarm.WriteChannel(Channel, OutData, OutSize);
				}

				OutSize = FMath::Square(MaterialData.DiffuseSize) * sizeof(FFloat16Color);
				if (OutSize > 0)
				{
					OutData = (uint8*)(MaterialDiffuse.GetData());
					Swarm.WriteChannel(Channel, OutData, OutSize);
				}
		
				OutSize = FMath::Square(MaterialData.TransmissionSize) * sizeof(FFloat16Color);
				if (OutSize > 0)
				{
					OutData = (uint8*)(MaterialTransmission.GetData());
					Swarm.WriteChannel(Channel, OutData, OutSize);
				}

				OutSize = FMath::Square(MaterialData.NormalSize) * sizeof(FFloat16Color);
				if (OutSize > 0)
				{
					OutData = (uint8*)(MaterialNormal.GetData());
					Swarm.WriteChannel(Channel, OutData, OutSize);
				}

				OpenedMaterialExportChannels.Add(Channel);
			}
			else
			{
				UE_LOGF(LogLightmassSolver, Warning, "Failed to open channel for material data for %ls: %ls", *(Material->GetLightingGuid().ToString()), *(Material->GetPathName()));
			}
		}
		else
		{
			UE_LOGF(LogLightmassSolver, Warning, "Failed to generate material data for %ls: %ls", *(Material->GetLightingGuid().ToString()), *(Material->GetPathName()));
		}
	}
		
	UpdateExportProgress();
}

void FLightmassExporter::WriteBaseMeshInstanceData( int32 Channel, int32 MeshIndex, const FStaticLightingMesh* Mesh, TArray<Lightmass::FMaterialElementData>& MaterialElementData )
{
	Lightmass::FStaticLightingMeshInstanceData MeshInstanceData;
	FMemory::Memzero(&MeshInstanceData,sizeof(MeshInstanceData));
	MeshInstanceData.Guid = Mesh->Guid;
	MeshInstanceData.NumTriangles = Mesh->NumTriangles;
	MeshInstanceData.NumShadingTriangles = Mesh->NumShadingTriangles;
	MeshInstanceData.NumVertices = Mesh->NumVertices;
	MeshInstanceData.NumShadingVertices = Mesh->NumShadingVertices;
	MeshInstanceData.MeshIndex = MeshIndex;
	MeshInstanceData.LevelGuid = LightingContext.GetPersistentLevelGuid();
	check(Mesh->Component);
	bool bFoundLevel = false;
	AActor* ComponentOwner = Mesh->Component->GetOwner();
	if (ComponentOwner && ComponentOwner->GetLevel())
	{
		MeshInstanceData.LevelGuid = LightingContext.GetLevelGuidForActor(ComponentOwner);
		bFoundLevel = true;
	}
	else if (Mesh->Component->IsA(UModelComponent::StaticClass()))
	{
		const UModelComponent* ModelComponent = CastChecked<UModelComponent>(Mesh->Component);
		for (int32 LevelIndex = 0; LevelIndex < World->GetNumLevels(); LevelIndex++)
		{
			if (ModelComponent->GetModel() == World->GetLevel(LevelIndex)->Model)
			{
				MeshInstanceData.LevelGuid = LightingContext.GetLevelGuidForLevel(World->GetLevel(LevelIndex));
				bFoundLevel = true;
				break;
			}
		}
	}
	
	if (!bFoundLevel)
	{
		UE_LOGF(LogLightmassSolver, Warning, "Couldn't determine level for component %ls during Lightmass export, it will be considered in the persistent level!", *Mesh->Component->GetPathName());
	}

	MeshInstanceData.LightingFlags = 0;
	MeshInstanceData.LightingFlags |= Mesh->bCastShadow ? Lightmass::GI_INSTANCE_CASTSHADOW : 0;
	MeshInstanceData.LightingFlags |= Mesh->bTwoSidedMaterial ? Lightmass::GI_INSTANCE_TWOSIDED : 0;
	MeshInstanceData.bCastShadowAsTwoSided = Mesh->Component->bCastShadowAsTwoSided;
	MeshInstanceData.bMovable = (Mesh->Component->Mobility != EComponentMobility::Static);
	MeshInstanceData.NumRelevantLights = Mesh->RelevantLightsGuid.Num();
	MeshInstanceData.BoundingBox = FBox3f(Mesh->BoundingBox);
	Swarm.WriteChannel( Channel, &MeshInstanceData, sizeof(MeshInstanceData) );
	const uint32 LightGuidsSize = Mesh->RelevantLightsGuid.Num() * sizeof(FGuid);
	if( LightGuidsSize > 0 )
	{
		FGuid* LightGuids = (FGuid*)FMemory::Malloc(LightGuidsSize);
		for( int32 LightIdx=0; LightIdx < Mesh->RelevantLightsGuid.Num(); LightIdx++ )
		{
			LightGuids[LightIdx] = Mesh->RelevantLightsGuid[LightIdx];
		}
		Swarm.WriteChannel( Channel, LightGuids, LightGuidsSize );
		FMemory::Free( LightGuids );
	}

	const int32 NumVisibilitiyIds = Mesh->VisibilityIds.Num();
	Swarm.WriteChannel(Channel, &NumVisibilitiyIds, sizeof(NumVisibilitiyIds));
	Swarm.WriteChannel(Channel, Mesh->VisibilityIds.GetData(), Mesh->VisibilityIds.Num() * Mesh->VisibilityIds.GetTypeSize());

	// Always need to have at least one material
	if (MaterialElementData.Num() == 0)
	{
		Lightmass::FMaterialElementData DefaultData;
		GetMaterialHash(UMaterial::GetDefaultMaterial(MD_Surface), DefaultData.MaterialHash);
		MaterialElementData.Add(DefaultData);
	}

	// Write out the materials used by this mesh...
	const int32 NumMaterialElements = MaterialElementData.Num();
	Swarm.WriteChannel(Channel, &NumMaterialElements, sizeof(NumMaterialElements));
	for (int32 MtrlIdx = 0; MtrlIdx < NumMaterialElements; MtrlIdx++)
	{
		Swarm.WriteChannel(Channel, &(MaterialElementData[MtrlIdx]), sizeof(Lightmass::FMaterialElementData));
	}
}

void FLightmassExporter::WriteBaseMappingData( int32 Channel, const FStaticLightingMapping* Mapping )
{
	Lightmass::FStaticLightingMappingData MappingData;
	FMemory::Memzero(&MappingData,sizeof(MappingData));
	MappingData.Guid = Mapping->Mesh->Guid;
	MappingData.StaticLightingMeshInstance = Mapping->Mesh->SourceMeshGuid;
	Swarm.WriteChannel( Channel, &MappingData, sizeof(MappingData) );
}

void FLightmassExporter::WriteBaseTextureMappingData( int32 Channel, const FStaticLightingTextureMapping* TextureMapping )
{
	WriteBaseMappingData( Channel, TextureMapping );
	
	Lightmass::FStaticLightingTextureMappingData TextureMappingData;
	FMemory::Memzero(&TextureMappingData,sizeof(TextureMappingData));
	check(TextureMapping->SizeX > 0 && TextureMapping->SizeY > 0);
	TextureMappingData.SizeX = TextureMapping->SizeX;
	TextureMappingData.SizeY = TextureMapping->SizeY;
	TextureMappingData.LightmapTextureCoordinateIndex = TextureMapping->LightmapTextureCoordinateIndex;
	TextureMappingData.bBilinearFilter = TextureMapping->bBilinearFilter;

	Swarm.WriteChannel( Channel, &TextureMappingData, sizeof(TextureMappingData) );
}


void FLightmassExporter::WriteLandscapeMapping(int32 Channel, const class FLandscapeStaticLightingTextureMapping* LandscapeMapping)
{
	WriteBaseTextureMappingData(Channel, (const FStaticLightingTextureMapping*)LandscapeMapping);
}

struct FMeshAndLODId
{
	int32 MeshIndex;
	int32 LODIndex;
};

void FLightmassExporter::WriteMeshInstances( int32 Channel )
{
	// initially come up with a unique ID for each component
	TMap<const UPrimitiveComponent*, FMeshAndLODId> ComponentToIDMap;

	int32 NextId = 0;
	for( int32 MeshIdx=0; MeshIdx < StaticMeshLightingMeshes.Num(); MeshIdx++ )
	{
		const FStaticMeshStaticLightingMesh* SMLightingMesh = StaticMeshLightingMeshes[MeshIdx];
		const UStaticMesh* StaticMesh = SMLightingMesh->StaticMesh;
		if (StaticMesh)
		{
			const UStaticMeshComponent* Primitive = SMLightingMesh->Primitive;
			if (Primitive)
			{
				// All FStaticMeshStaticLightingMesh's in the OtherMeshLODs array need to get the same MeshIndex but different LODIndex
				// So that they won't shadow each other in Lightmass. HLODs are forced as new meshes and rely on custom handling
				if (SMLightingMesh->HLODTreeIndex)
				{
					FMeshAndLODId NewId;
					NewId.MeshIndex = NextId++;
					NewId.LODIndex = 0;
					ComponentToIDMap.Add(Primitive, NewId);
				}
				else if (SMLightingMesh->OtherMeshLODs.Num() > 0)
				{
					FMeshAndLODId* ExistingLODId = NULL;
					int32 LargestLODIndex = INDEX_NONE;
					for (int32 OtherLODIndex = 0; OtherLODIndex < SMLightingMesh->OtherMeshLODs.Num(); OtherLODIndex++)
					{
						FStaticLightingMesh* OtherLOD = SMLightingMesh->OtherMeshLODs[OtherLODIndex];
						if (OtherLOD->Component)
						{
							FMeshAndLODId* CurrentLODId = ComponentToIDMap.Find(OtherLOD->Component);
							// Find the mesh with the largest index
							if (CurrentLODId && CurrentLODId->LODIndex > LargestLODIndex)
							{
								ExistingLODId = CurrentLODId;
								LargestLODIndex = CurrentLODId->LODIndex;
							}
						}
					}
					if (ExistingLODId)
					{
						FMeshAndLODId NewId;
						// Reuse the mesh index from another LOD
						NewId.MeshIndex = ExistingLODId->MeshIndex;
						// Assign a new unique LOD index
						NewId.LODIndex = ExistingLODId->LODIndex + 1;
						ComponentToIDMap.Add(Primitive, NewId);
					}
					else
					{
						FMeshAndLODId NewId;
						NewId.MeshIndex = NextId++;
						NewId.LODIndex = 0;
						ComponentToIDMap.Add(Primitive, NewId);
					}
				}
				else
				{
					FMeshAndLODId NewId;
					NewId.MeshIndex = NextId++;
					NewId.LODIndex = 0;
					ComponentToIDMap.Add(Primitive, NewId);
				}
			}
		}
	}

	// static mesh instance meshes
	for( int32 MeshIdx=0; MeshIdx < StaticMeshLightingMeshes.Num(); MeshIdx++ )
	{
		const FStaticMeshStaticLightingMesh* SMLightingMesh = StaticMeshLightingMeshes[MeshIdx];

		FMeshAndLODId* MeshId = NULL;

		// Collect the material guids for each element
		TArray<Lightmass::FMaterialElementData> MaterialElementData;
		const UStaticMesh* StaticMesh = SMLightingMesh->StaticMesh;
		check(StaticMesh);

		const UStaticMeshComponent* Primitive = SMLightingMesh->Primitive;
		if (Primitive)
		{
			// get the meshindex from the component
			MeshId = ComponentToIDMap.Find(Primitive);

			if (StaticMesh->GetRenderData() && SMLightingMesh->LODIndex < StaticMesh->GetRenderData()->LODResources.Num())
			{
				const FStaticMeshLODResources& LODRenderData = StaticMesh->GetRenderData()->LODResources[SMLightingMesh->LODIndex];
				for (int32 SectionIndex = 0; SectionIndex < LODRenderData.Sections.Num(); SectionIndex++)
				{
					const FStaticMeshSection& Section = LODRenderData.Sections[ SectionIndex ];
					UMaterialInterface* Material = Primitive->GetMaterial(Section.MaterialIndex);
					if (Material == NULL)
					{
						Material = UMaterial::GetDefaultMaterial(MD_Surface);
					}
					Lightmass::FMaterialElementData NewElementData;
					GetMaterialHash(Material, NewElementData.MaterialHash);
					NewElementData.bUseTwoSidedLighting = Primitive->LightmassSettings.bUseTwoSidedLighting;
					NewElementData.bShadowIndirectOnly = Primitive->LightmassSettings.bShadowIndirectOnly;
					NewElementData.bUseEmissiveForStaticLighting = Primitive->LightmassSettings.bUseEmissiveForStaticLighting;
					NewElementData.bUseVertexNormalForHemisphereGather = Primitive->LightmassSettings.bUseVertexNormalForHemisphereGather;
					// Combine primitive and level boost settings so we don't have to send the level settings over to Lightmass  
					NewElementData.EmissiveLightFalloffExponent = Primitive->LightmassSettings.EmissiveLightFalloffExponent;
					NewElementData.EmissiveLightExplicitInfluenceRadius = Primitive->LightmassSettings.EmissiveLightExplicitInfluenceRadius;
					NewElementData.EmissiveBoost = Primitive->GetEmissiveBoost(SectionIndex) * LevelSettings.EmissiveBoost;
					NewElementData.DiffuseBoost = Primitive->GetDiffuseBoost(SectionIndex) * LevelSettings.DiffuseBoost;
					NewElementData.FullyOccludedSamplesFraction = Primitive->LightmassSettings.FullyOccludedSamplesFraction;
					MaterialElementData.Add(NewElementData);
				}
			}
		}

		WriteBaseMeshInstanceData( Channel, MeshId ? MeshId->MeshIndex : INDEX_NONE, SMLightingMesh, MaterialElementData );

		Lightmass::FStaticMeshStaticLightingMeshData SMInstanceMeshData;
		FMemory::Memzero(&SMInstanceMeshData,sizeof(SMInstanceMeshData));

		// Store HLOD data in upper 16 bits
		SMInstanceMeshData.EncodedLODIndices = SMLightingMesh->LODIndex & 0xFFFF;
		SMInstanceMeshData.EncodedLODIndices |= (SMLightingMesh->HLODTreeIndex & 0xFFFF) << 16;
		SMInstanceMeshData.EncodedHLODRange = SMLightingMesh->HLODChildStartIndex & 0xFFFF;
		SMInstanceMeshData.EncodedHLODRange |= (SMLightingMesh->HLODChildEndIndex & 0xFFFF) << 16;

		SMInstanceMeshData.LocalToWorld = FMatrix44f(SMLightingMesh->LocalToWorld);	// LWC_TODO: Precision loss relevant here?
		SMInstanceMeshData.bReverseWinding = SMLightingMesh->bReverseWinding;
		SMInstanceMeshData.bShouldSelfShadow = true;
		SMInstanceMeshData.StaticMeshGuid = SMLightingMesh->StaticMesh->GetLightingGuid();
		const FSplineMeshParams* SplineParams = SMLightingMesh->GetSplineParameters();
		if (SplineParams && StaticMesh)
		{
			FBoxSphereBounds MeshBounds = StaticMesh->GetBounds();
			const USplineMeshComponent* SplineComponent = CastChecked<USplineMeshComponent>(SMLightingMesh->Component);
			SMInstanceMeshData.bIsSplineMesh = true;
			Copy(*SplineParams, SMInstanceMeshData.SplineParameters);
			SMInstanceMeshData.SplineParameters.SplineUpDir = FVector3f(SplineComponent->SplineUpDir);
			SMInstanceMeshData.SplineParameters.bSmoothInterpRollScale = SplineComponent->bSmoothInterpRollScale;

			if (FMath::IsNearlyEqual(SplineComponent->SplineBoundaryMin, SplineComponent->SplineBoundaryMax))
			{
				// set ranges according to the extents of the mesh
				SMInstanceMeshData.SplineParameters.MeshMinZ = MeshBounds.Origin[SplineComponent->ForwardAxis] - MeshBounds.BoxExtent[SplineComponent->ForwardAxis];
				SMInstanceMeshData.SplineParameters.MeshRangeZ = 2.f * MeshBounds.BoxExtent[SplineComponent->ForwardAxis];
			}
			else
			{
				// set ranges according to the custom boundary min/max
				SMInstanceMeshData.SplineParameters.MeshMinZ = SplineComponent->SplineBoundaryMin;
				SMInstanceMeshData.SplineParameters.MeshRangeZ = SplineComponent->SplineBoundaryMax - SplineComponent->SplineBoundaryMin;
			}

			SMInstanceMeshData.SplineParameters.ForwardAxis = (Lightmass::ESplineMeshAxis::Type)SplineComponent->ForwardAxis.GetValue();
		}
		else
		{
			SMInstanceMeshData.bIsSplineMesh = false;
			FMemory::Memzero(&SMInstanceMeshData.SplineParameters, sizeof(SMInstanceMeshData.SplineParameters));
		}

		Swarm.WriteChannel( Channel, &SMInstanceMeshData, sizeof(SMInstanceMeshData) );

		UpdateExportProgress();
	}
}

void FLightmassExporter::WriteLandscapeInstances( int32 Channel )
{
	// landscape instance meshes
	for (int32 LandscapeIdx = 0; LandscapeIdx < LandscapeLightingMeshes.Num(); LandscapeIdx++)
	{
		const FLandscapeStaticLightingMesh* LandscapeLightingMesh = LandscapeLightingMeshes[LandscapeIdx];

		// Collect the material guids for each element
		TArray<Lightmass::FMaterialElementData> MaterialElementData;
		const ULandscapeComponent* LandscapeComp = LandscapeLightingMesh->LandscapeComponent;
		if (LandscapeComp && LandscapeComp->GetLandscapeProxy())
		{
			UMaterialInterface* Material = LandscapeComp->GetMaterialInstance(0, false);
			if (!Material)
			{
				Material = UMaterial::GetDefaultMaterial(MD_Surface);
			}
			Lightmass::FMaterialElementData NewElementData;
			GetMaterialHash(Material, NewElementData.MaterialHash);
			FLightmassPrimitiveSettings& LMSetting = LandscapeComp->GetLandscapeProxy()->LightmassSettings;
			NewElementData.bUseTwoSidedLighting = LMSetting.bUseTwoSidedLighting;
			NewElementData.bShadowIndirectOnly = LMSetting.bShadowIndirectOnly;
			NewElementData.bUseEmissiveForStaticLighting = LMSetting.bUseEmissiveForStaticLighting;
			NewElementData.bUseVertexNormalForHemisphereGather = LMSetting.bUseVertexNormalForHemisphereGather;
			// Combine primitive and level boost settings so we don't have to send the level settings over to Lightmass  
			NewElementData.EmissiveLightFalloffExponent = LMSetting.EmissiveLightFalloffExponent;
			NewElementData.EmissiveLightExplicitInfluenceRadius = LMSetting.EmissiveLightExplicitInfluenceRadius;
			NewElementData.EmissiveBoost = LandscapeComp->GetEmissiveBoost(0) * LevelSettings.EmissiveBoost;
			NewElementData.DiffuseBoost = LandscapeComp->GetDiffuseBoost(0) * LevelSettings.DiffuseBoost;
			NewElementData.FullyOccludedSamplesFraction = LMSetting.FullyOccludedSamplesFraction;
			MaterialElementData.Add(NewElementData);
		}

		WriteBaseMeshInstanceData(Channel, INDEX_NONE, LandscapeLightingMesh, MaterialElementData);

		Lightmass::FLandscapeStaticLightingMeshData LandscapeInstanceMeshData;
		FMemory::Memzero(&LandscapeInstanceMeshData,sizeof(LandscapeInstanceMeshData));

		LandscapeInstanceMeshData.LocalToWorld = FMatrix44f(LandscapeLightingMesh->LocalToWorld.ToMatrixWithScale());	// LWC_TODO: Precision loss relevant here?
		LandscapeInstanceMeshData.ComponentSizeQuads = LandscapeLightingMesh->ComponentSizeQuads;
		LandscapeInstanceMeshData.LightMapRatio = LandscapeLightingMesh->LightMapRatio;
		LandscapeInstanceMeshData.ExpandQuadsX = LandscapeLightingMesh->ExpandQuadsX;
		LandscapeInstanceMeshData.ExpandQuadsY = LandscapeLightingMesh->ExpandQuadsY;

		Swarm.WriteChannel( Channel, &LandscapeInstanceMeshData, sizeof(LandscapeInstanceMeshData) );

		// write height map data
		uint8* OutData;
		int32 OutSize;

		OutSize = LandscapeLightingMesh->HeightData.Num() * sizeof(FColor);  
		if (OutSize > 0)
		{
			OutData = (uint8*)(LandscapeLightingMesh->HeightData.GetData());
			Swarm.WriteChannel(Channel, OutData, OutSize);
		}

		UpdateExportProgress();
	}
}

struct FLightmassMaterialPair
{
	FLightmassMaterialPair(int32 InLightmassSettingsIndex, UMaterialInterface* InMaterial)
		: LightmassSettingsIndex(InLightmassSettingsIndex)
		, Material(InMaterial)
	{
	}

	/** Index into the Model's LightmassSettings array for this triangle */
	int32 LightmassSettingsIndex;

	/** Materials used by this triangle */
	UMaterialInterface* Material;

	bool operator==( const FLightmassMaterialPair& Other ) const
	{
		return LightmassSettingsIndex == Other.LightmassSettingsIndex && Material == Other.Material;
	}
};


void FLightmassExporter::WriteMappings( int32 Channel )
{
	// bsp mappings
	for( int32 MappingIdx=0; MappingIdx < BSPSurfaceMappings.Num() && !GEditor->GetMapBuildCancelled(); MappingIdx++ )
	{
		const FBSPSurfaceStaticLighting* BSPMapping = BSPSurfaceMappings[MappingIdx];

		TArray<Lightmass::FMaterialElementData> MaterialElementData;
		const UModel* Model = BSPMapping->GetModel();
		check(Model);

		// make a list of the used lightmass settings by this NodeGroup and a mapping from
		// each triangle into this array
		TArray<FLightmassMaterialPair> LocalLightmassSettings;
		TArray<int32> LocalPerTriangleLightmassSettings;

		// go through each triangle, looking for unique settings, and remapping each triangle 
		int32 NumTriangles = BSPMapping->NodeGroup->TriangleSurfaceMap.Num();
		LocalPerTriangleLightmassSettings.Empty(NumTriangles);
		LocalPerTriangleLightmassSettings.AddUninitialized(NumTriangles);
		for (int32 TriangleIndex = 0; TriangleIndex < NumTriangles; TriangleIndex++)
		{
			const FBspSurf& Surf = Model->Surfs[BSPMapping->NodeGroup->TriangleSurfaceMap[TriangleIndex]];
			LocalPerTriangleLightmassSettings[TriangleIndex] = 
				LocalLightmassSettings.AddUnique(FLightmassMaterialPair(Surf.iLightmassIndex, Surf.Material));
		}

		// now for each used setting, export it
		for (int32 SettingIndex = 0; SettingIndex < LocalLightmassSettings.Num(); SettingIndex++)
		{
			const FLightmassMaterialPair& Pair = LocalLightmassSettings[SettingIndex];
			UMaterialInterface* Material = Pair.Material;
			if (Material == NULL)
			{
				Material = UMaterial::GetDefaultMaterial(MD_Surface);
			}
			check(Material);

			// get the settings from the model
			const FLightmassPrimitiveSettings& PrimitiveSettings = Model->LightmassSettings[Pair.LightmassSettingsIndex];

			Lightmass::FMaterialElementData TempData;
			GetMaterialHash(Material, TempData.MaterialHash);
			TempData.bUseTwoSidedLighting = PrimitiveSettings.bUseTwoSidedLighting;
			TempData.bShadowIndirectOnly = PrimitiveSettings.bShadowIndirectOnly;
			TempData.bUseEmissiveForStaticLighting = PrimitiveSettings.bUseEmissiveForStaticLighting;
			TempData.bUseVertexNormalForHemisphereGather = PrimitiveSettings.bUseVertexNormalForHemisphereGather;
			TempData.EmissiveLightFalloffExponent = PrimitiveSettings.EmissiveLightFalloffExponent;
			TempData.EmissiveLightExplicitInfluenceRadius = PrimitiveSettings.EmissiveLightExplicitInfluenceRadius;
			TempData.EmissiveBoost = PrimitiveSettings.EmissiveBoost * LevelSettings.EmissiveBoost;
			TempData.DiffuseBoost = PrimitiveSettings.DiffuseBoost * LevelSettings.DiffuseBoost;
			TempData.FullyOccludedSamplesFraction = PrimitiveSettings.FullyOccludedSamplesFraction;
			MaterialElementData.Add(TempData);
		}
		
		WriteBaseMeshInstanceData(Channel, INDEX_NONE, BSPMapping, MaterialElementData);
		WriteBaseTextureMappingData( Channel, BSPMapping );
		
		Lightmass::FBSPSurfaceStaticLightingData BSPSurfaceMappingData;
		BSPSurfaceMappingData.TangentX = BSPMapping->NodeGroup->TangentX;
		BSPSurfaceMappingData.TangentY = BSPMapping->NodeGroup->TangentY;
		BSPSurfaceMappingData.TangentZ = BSPMapping->NodeGroup->TangentZ;
		BSPSurfaceMappingData.MapToWorld = FMatrix44f(BSPMapping->NodeGroup->MapToWorld);	// LWC_TODO: Precision loss?
		BSPSurfaceMappingData.WorldToMap = FMatrix44f(BSPMapping->NodeGroup->WorldToMap);	// LWC_TODO: Precision loss?
		
		Swarm.WriteChannel( Channel, &BSPSurfaceMappingData, sizeof(BSPSurfaceMappingData) );
	
		const uint32 VertexDataSize = BSPMapping->NodeGroup->Vertices.Num() * sizeof(Lightmass::FStaticLightingVertexData);
		if( VertexDataSize > 0 )
		{
			Lightmass::FStaticLightingVertexData* VertexData = (Lightmass::FStaticLightingVertexData*)FMemory::Malloc(VertexDataSize);
			for( int32 VertIdx=0; VertIdx < BSPMapping->NodeGroup->Vertices.Num(); VertIdx++ )
			{
				const FStaticLightingVertex& SrcVertex = BSPMapping->NodeGroup->Vertices[VertIdx];
				Lightmass::FStaticLightingVertexData& DstVertex = VertexData[VertIdx];

				// LWC_TODO: precision loss
				DstVertex.WorldPosition = (FVector3f)SrcVertex.WorldPosition;
				DstVertex.WorldTangentX = (FVector3f)SrcVertex.WorldTangentX;
				DstVertex.WorldTangentY = (FVector3f)SrcVertex.WorldTangentY;
				DstVertex.WorldTangentZ = (FVector3f)SrcVertex.WorldTangentZ;
				for( int32 CoordIdx=0; CoordIdx < Lightmass::MAX_TEXCOORDS; CoordIdx++ )
				{	
					DstVertex.TextureCoordinates[CoordIdx] = FVector2f(SrcVertex.TextureCoordinates[CoordIdx]);	// LWC_TODO: Precision loss
				}
			}
			Swarm.WriteChannel( Channel, VertexData, VertexDataSize );
			FMemory::Free( VertexData );
		}
		if( BSPMapping->NodeGroup->TriangleVertexIndices.Num() > 0 )
		{
			Swarm.WriteChannel( Channel, (void*)BSPMapping->NodeGroup->TriangleVertexIndices.GetData(), BSPMapping->NodeGroup->TriangleVertexIndices.Num() * sizeof(int32) );
		}

		Swarm.WriteChannel(Channel, LocalPerTriangleLightmassSettings.GetData(), LocalPerTriangleLightmassSettings.Num() * sizeof(int32));
		UpdateExportProgress();
	}	
	
	// static mesh texture mappings
	for( int32 MappingIdx=0; MappingIdx < StaticMeshTextureMappings.Num(); MappingIdx++ )
	{
		const FStaticMeshStaticLightingTextureMapping* SMTextureMapping = StaticMeshTextureMappings[MappingIdx];
		WriteBaseTextureMappingData( Channel, SMTextureMapping );
		UpdateExportProgress();
	}	

	// landscape surface mappings
	for (int32 MappingIdx = 0; MappingIdx < LandscapeTextureMappings.Num(); MappingIdx++)
	{
		const FLandscapeStaticLightingTextureMapping* LandscapeMapping = LandscapeTextureMappings[MappingIdx];
		WriteLandscapeMapping(Channel, LandscapeMapping);
		UpdateExportProgress();
	}

	for (int32 MappingIdx = 0; MappingIdx < VolumeMappings.Num(); MappingIdx++)
	{
		const FStaticLightingGlobalVolumeMapping* VolumeMapping = VolumeMappings[MappingIdx];
		WriteBaseTextureMappingData( Channel, VolumeMapping );
		UpdateExportProgress();
	}

	for (int32 MappingIdx = 0; MappingIdx < LandscapeVolumeMappings.Num(); MappingIdx++)
	{
		const FLandscapeStaticLightingGlobalVolumeMapping* VolumeMapping = LandscapeVolumeMappings[MappingIdx];
		WriteLandscapeMapping( Channel, VolumeMapping );
		UpdateExportProgress();
	}
}

/** Finds the GUID of the mapping that is being debugged. */
bool FLightmassExporter::FindDebugMapping(FGuid& DebugMappingGuid)
{
	const FStaticLightingMapping* FoundDebugMapping = NULL;
	// Only BSP texture, static mesh vertex and texture lightmaps supported for now.
	for( int32 MappingIdx=0; MappingIdx < BSPSurfaceMappings.Num(); MappingIdx++ )
	{
		const FBSPSurfaceStaticLighting* BSPMapping = BSPSurfaceMappings[MappingIdx];
		if (BSPMapping->DebugThisMapping())
		{
			// Only one mapping should be setup for debugging
			check(!FoundDebugMapping);
			FoundDebugMapping = BSPMapping;
		}
	}

	for( int32 MappingIdx=0; MappingIdx < StaticMeshTextureMappings.Num(); MappingIdx++ )
	{
		const FStaticMeshStaticLightingTextureMapping* SMTextureMapping = StaticMeshTextureMappings[MappingIdx];
		if (SMTextureMapping->DebugThisMapping())
		{
			// Only one mapping should be setup for debugging
			check(!FoundDebugMapping);
			FoundDebugMapping = SMTextureMapping;
		}
	}	

	if (FoundDebugMapping)
	{
		DebugMappingGuid = FoundDebugMapping->GetLightingGuid();
		return true;
	}
	else 
	{
		return false;
	}
}

void FLightmassExporter::SetVolumetricLightmapSettings(Lightmass::FVolumetricLightmapSettings& OutSettings)
{
	FBox CombinedImportanceVolume(ForceInit);

	for (int32 i = 0; i < ImportanceVolumes.Num(); i++)
	{
		CombinedImportanceVolume += ImportanceVolumes[i];
	}

	const FVector ImportanceExtent = CombinedImportanceVolume.GetExtent();
	// Guarantee cube voxels.  
	// This means some parts of the volumetric lightmap volume will be outside the lightmass importance volume.
	// We prevent refinement outside of importance volumes in FStaticLightingSystem::ShouldRefineVoxel
	const float MaxExtent = FMath::Max(ImportanceExtent.X, FMath::Max(ImportanceExtent.Y, ImportanceExtent.Z));

	OutSettings.VolumeMin = FVector3f(CombinedImportanceVolume.Min);	//LWC_TODO: Precision loss
	const FVector RequiredVolumeSize = FVector(MaxExtent * 2);

	VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.VolumetricLightmaps"), TEXT("BrickSize"), OutSettings.BrickSize, GLightmassIni));
	VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.VolumetricLightmaps"), TEXT("MaxRefinementLevels"), OutSettings.MaxRefinementLevels, GLightmassIni));
	VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.VolumetricLightmaps"), TEXT("VoxelizationCellExpansionForSurfaceGeometry"), OutSettings.VoxelizationCellExpansionForSurfaceGeometry, GLightmassIni));
	VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.VolumetricLightmaps"), TEXT("VoxelizationCellExpansionForVolumeGeometry"), OutSettings.VoxelizationCellExpansionForVolumeGeometry, GLightmassIni));
	VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.VolumetricLightmaps"), TEXT("VoxelizationCellExpansionForLights"), OutSettings.VoxelizationCellExpansionForLights, GLightmassIni));
	VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.VolumetricLightmaps"), TEXT("MinBrickError"), OutSettings.MinBrickError, GLightmassIni));
	VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.VolumetricLightmaps"), TEXT("SurfaceLightmapMinTexelsPerVoxelAxis"), OutSettings.SurfaceLightmapMinTexelsPerVoxelAxis, GLightmassIni));
	VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.VolumetricLightmaps"), TEXT("bCullBricksBelowLandscape"), OutSettings.bCullBricksBelowLandscape, GLightmassIni));
	VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.VolumetricLightmaps"), TEXT("LightBrightnessSubdivideThreshold"), OutSettings.LightBrightnessSubdivideThreshold, GLightmassIni));

	const FLightmassWorldInfoSettings& WorldInfoSettings = World->GetWorldSettings()->LightmassSettings;

	float Smoothing = FMath::Clamp(WorldInfoSettings.VolumetricLightmapSphericalHarmonicSmoothing, SMALL_NUMBER, 1000.0f);
	OutSettings.WindowingTargetLaplacian = 1.0f / Smoothing;

	OutSettings.BrickSize = FMath::RoundUpToPowerOfTwo(OutSettings.BrickSize);
	OutSettings.MaxRefinementLevels = FMath::Clamp(OutSettings.MaxRefinementLevels, 1, 6);
	OutSettings.VoxelizationCellExpansionForSurfaceGeometry = FMath::Max(OutSettings.VoxelizationCellExpansionForSurfaceGeometry, 0.0f);
	OutSettings.VoxelizationCellExpansionForVolumeGeometry = FMath::Max(OutSettings.VoxelizationCellExpansionForVolumeGeometry, 0.0f);
	OutSettings.VoxelizationCellExpansionForLights = FMath::Max(OutSettings.VoxelizationCellExpansionForLights, 0.0f);

	const float TargetDetailCellSize = WorldInfoSettings.VolumetricLightmapDetailCellSize;

	FIntVector FullGridSize(
		FMath::TruncToInt(2 * ImportanceExtent.X / TargetDetailCellSize) + 1,
		FMath::TruncToInt(2 * ImportanceExtent.Y / TargetDetailCellSize) + 1,
		FMath::TruncToInt(2 * ImportanceExtent.Z / TargetDetailCellSize) + 1);

	// Make sure size of indirection texture does not exceed INT_MAX
	const int32 MaxGridDimension = FMath::Pow(2048000000 / 4, 1.0f / 3) * OutSettings.BrickSize;
	if (FullGridSize.GetMax() > MaxGridDimension)
	{
		UE_LOGF(LogLightmassSolver, Warning, 
			"Volumetric lightmap grid size is too large which can cause potential crashes or long build time, clamping from (%d, %d, %d) to (%d, %d, %d)",
			FullGridSize.X,
			FullGridSize.Y,
			FullGridSize.Z,
			FMath::Min(FullGridSize.X, MaxGridDimension),
			FMath::Min(FullGridSize.Y, MaxGridDimension),
			FMath::Min(FullGridSize.Z, MaxGridDimension)
			);
		FullGridSize.X = FMath::Min(FullGridSize.X, MaxGridDimension);
		FullGridSize.Y = FMath::Min(FullGridSize.Y, MaxGridDimension);
		FullGridSize.Z = FMath::Min(FullGridSize.Z, MaxGridDimension);
	}

	const int32 BrickSizeLog2 = FMath::FloorLog2(OutSettings.BrickSize);
	const int32 DetailCellsPerTopLevelBrick = 1 << (OutSettings.MaxRefinementLevels * BrickSizeLog2);

	OutSettings.TopLevelGridSize = FIntVector::DivideAndRoundUp(FullGridSize, DetailCellsPerTopLevelBrick);

	OutSettings.VolumeSize = FVector3f(OutSettings.TopLevelGridSize) * DetailCellsPerTopLevelBrick * TargetDetailCellSize;
}

/** Fills out the Scene's settings, read from the engine ini. */
void FLightmassExporter::WriteSceneSettings( Lightmass::FSceneFileHeader& Scene )
{
	bool bConfigBool = false;
	//@todo - need a mechanism to automatically catch when a new setting has been added but doesn't get initialized
	{
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.StaticLighting"), TEXT("bAllowMultiThreadedStaticLighting"), bConfigBool, GLightmassIni));
		Scene.GeneralSettings.bAllowMultiThreadedStaticLighting = bConfigBool;
		Scene.GeneralSettings.NumUnusedLocalCores = NumUnusedLocalCores;
		Scene.GeneralSettings.NumIndirectLightingBounces = LevelSettings.NumIndirectLightingBounces;
		Scene.GeneralSettings.NumSkyLightingBounces = LevelSettings.NumSkyLightingBounces;
		Scene.GeneralSettings.IndirectLightingSmoothness = LevelSettings.IndirectLightingSmoothness;
		Scene.GeneralSettings.IndirectLightingQuality = LevelSettings.IndirectLightingQuality;

		if (QualityLevel == Quality_Preview)
		{
			Scene.GeneralSettings.IndirectLightingQuality = FMath::Min(Scene.GeneralSettings.IndirectLightingQuality, 1.0f);
		}

		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.StaticLighting"), TEXT("ViewSingleBounceNumber"), Scene.GeneralSettings.ViewSingleBounceNumber, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.StaticLighting"), TEXT("bUseConservativeTexelRasterization"), bConfigBool, GLightmassIni));
		Scene.GeneralSettings.bUseConservativeTexelRasterization = bConfigBool;
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.StaticLighting"), TEXT("bAccountForTexelSize"), bConfigBool, GLightmassIni));
		Scene.GeneralSettings.bAccountForTexelSize = bConfigBool;
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.StaticLighting"), TEXT("bUseMaxWeight"), bConfigBool, GLightmassIni));
		Scene.GeneralSettings.bUseMaxWeight = bConfigBool;
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.StaticLighting"), TEXT("MaxTriangleLightingSamples"), Scene.GeneralSettings.MaxTriangleLightingSamples, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.StaticLighting"), TEXT("MaxTriangleIrradiancePhotonCacheSamples"), Scene.GeneralSettings.MaxTriangleIrradiancePhotonCacheSamples, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.StaticLighting"), TEXT("bUseEmbree"), bConfigBool, GLightmassIni));
		Scene.GeneralSettings.bUseEmbree = bConfigBool;
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.StaticLighting"), TEXT("bVerifyEmbree"), bConfigBool, GLightmassIni));
		Scene.GeneralSettings.bVerifyEmbree = Scene.GeneralSettings.bUseEmbree && bConfigBool;
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.StaticLighting"), TEXT("bUseEmbreePacketTracing"), Scene.GeneralSettings.bUseEmbreePacketTracing, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.StaticLighting"), TEXT("bUseFastVoxelization"), Scene.GeneralSettings.bUseFastVoxelization, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.StaticLighting"), TEXT("bUseEmbreeInstancing"), Scene.GeneralSettings.bUseEmbreeInstancing, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.StaticLighting"), TEXT("MappingSurfaceCacheDownsampleFactor"), Scene.GeneralSettings.MappingSurfaceCacheDownsampleFactor, GLightmassIni));

		int32 CheckQualityLevel;
		GConfig->GetInt( TEXT("LightingBuildOptions"), TEXT("QualityLevel"), CheckQualityLevel, GEditorPerProjectIni);
		CheckQualityLevel = FMath::Clamp<int32>(CheckQualityLevel, Quality_Preview, Quality_Production);
		UE_LOGF(LogLightmassSolver, Log, "LIGHTMASS: Writing scene settings: Quality level %d (%d in INI)", (int32)(QualityLevel), CheckQualityLevel);
		if (CheckQualityLevel != QualityLevel)
		{
			UE_LOGF(LogLightmassSolver, Warning, "LIGHTMASS: Writing scene settings w/ QualityLevel mismatch! %d vs %d (ini setting)", (int32)QualityLevel, CheckQualityLevel);
		}

		switch (QualityLevel)
		{
		case Quality_High:
		case Quality_Production:
			Scene.GeneralSettings.bUseErrorColoring = false;
			Scene.GeneralSettings.UnmappedTexelColor = FLinearColor(0.0f, 0.0f, 0.0f);
			break;
		default:
			{
				bool bUseErrorColoring = false;
				GConfig->GetBool( TEXT("LightingBuildOptions"), TEXT("UseErrorColoring"),		bUseErrorColoring,					GEditorPerProjectIni );
				Scene.GeneralSettings.bUseErrorColoring = bUseErrorColoring;
				if (bUseErrorColoring == false)
				{
					Scene.GeneralSettings.UnmappedTexelColor = FLinearColor(0.0f, 0.0f, 0.0f);
				}
				else
				{
					Scene.GeneralSettings.UnmappedTexelColor = FLinearColor(0.7f, 0.7f, 0.0f);
				}
			}
			break;
		}
	}
	{
		float GlobalLevelScale = 1.0f;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.StaticLightingSceneConstants"), TEXT("StaticLightingLevelScale"), GlobalLevelScale, GLightmassIni));
		Scene.SceneConstants.StaticLightingLevelScale = GlobalLevelScale * LevelSettings.StaticLightingLevelScale;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.StaticLightingSceneConstants"), TEXT("VisibilityRayOffsetDistance"), Scene.SceneConstants.VisibilityRayOffsetDistance, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.StaticLightingSceneConstants"), TEXT("VisibilityNormalOffsetDistance"), Scene.SceneConstants.VisibilityNormalOffsetDistance, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.StaticLightingSceneConstants"), TEXT("VisibilityNormalOffsetSampleRadiusScale"), Scene.SceneConstants.VisibilityNormalOffsetSampleRadiusScale, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.StaticLightingSceneConstants"), TEXT("VisibilityTangentOffsetSampleRadiusScale"), Scene.SceneConstants.VisibilityTangentOffsetSampleRadiusScale, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.StaticLightingSceneConstants"), TEXT("SmallestTexelRadius"), Scene.SceneConstants.SmallestTexelRadius, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.StaticLightingSceneConstants"), TEXT("LightGridSize"), Scene.SceneConstants.LightGridSize, GLightmassIni));
	}
	{
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.StaticLightingMaterial"), TEXT("bUseDebugMaterial"), bConfigBool, GLightmassIni));
		Scene.MaterialSettings.bUseDebugMaterial = bConfigBool;
		FString ShowMaterialAttributeName;
		VERIFYLIGHTMASSINI(GConfig->GetString(TEXT("DevOptions.StaticLightingMaterial"), TEXT("ShowMaterialAttribute"), ShowMaterialAttributeName, GLightmassIni));

		Scene.MaterialSettings.ViewMaterialAttribute = Lightmass::VMA_None;
		if (ShowMaterialAttributeName.Contains(TEXT("Emissive")) )
		{
			Scene.MaterialSettings.ViewMaterialAttribute = Lightmass::VMA_Emissive;
		}
		else if (ShowMaterialAttributeName.Contains(TEXT("Diffuse"))
			|| LevelSettings.bVisualizeMaterialDiffuse)
		{
			Scene.MaterialSettings.ViewMaterialAttribute = Lightmass::VMA_Diffuse;
		}
		else if (ShowMaterialAttributeName.Contains(TEXT("Transmission")) )
		{
			Scene.MaterialSettings.ViewMaterialAttribute = Lightmass::VMA_Transmission;
		}
		else if (ShowMaterialAttributeName.Contains(TEXT("Normal")) )
		{
			Scene.MaterialSettings.ViewMaterialAttribute = Lightmass::VMA_Normal;
		}

		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.StaticLightingMaterial"), TEXT("EmissiveSampleSize"), Scene.MaterialSettings.EmissiveSize, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.StaticLightingMaterial"), TEXT("DiffuseSampleSize"), Scene.MaterialSettings.DiffuseSize, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.StaticLightingMaterial"), TEXT("TransmissionSampleSize"), Scene.MaterialSettings.TransmissionSize, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.StaticLightingMaterial"), TEXT("NormalSampleSize"), Scene.MaterialSettings.NormalSize, GLightmassIni));

		const FString DiffuseStr = GConfig->GetStr(TEXT("DevOptions.StaticLightingMaterial"), TEXT("DebugDiffuse"), GLightmassIni);
		VERIFYLIGHTMASSINI(FParse::Value(*DiffuseStr, TEXT("R="), Scene.MaterialSettings.DebugDiffuse.R));
		VERIFYLIGHTMASSINI(FParse::Value(*DiffuseStr, TEXT("G="), Scene.MaterialSettings.DebugDiffuse.G));
		VERIFYLIGHTMASSINI(FParse::Value(*DiffuseStr, TEXT("B="), Scene.MaterialSettings.DebugDiffuse.B));

		Scene.MaterialSettings.EnvironmentColor = FLinearColor(LevelSettings.EnvironmentColor) * LevelSettings.EnvironmentIntensity;

		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.NormalMapsForStaticLighting"));
		Scene.MaterialSettings.bUseNormalMapsForLighting = CVar->GetValueOnGameThread() != 0;
	}
	{
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.MeshAreaLights"), TEXT("bVisualizeMeshAreaLightPrimitives"), bConfigBool, GLightmassIni));
		Scene.MeshAreaLightSettings.bVisualizeMeshAreaLightPrimitives = bConfigBool;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.MeshAreaLights"), TEXT("EmissiveIntensityThreshold"), Scene.MeshAreaLightSettings.EmissiveIntensityThreshold, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.MeshAreaLights"), TEXT("MeshAreaLightGridSize"), Scene.MeshAreaLightSettings.MeshAreaLightGridSize, GLightmassIni));
		float MeshAreaLightSimplifyNormalAngleThreshold;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.MeshAreaLights"), TEXT("MeshAreaLightSimplifyNormalAngleThreshold"), MeshAreaLightSimplifyNormalAngleThreshold, GLightmassIni));
		Scene.MeshAreaLightSettings.MeshAreaLightSimplifyNormalCosAngleThreshold = FMath::Cos(FMath::Clamp(MeshAreaLightSimplifyNormalAngleThreshold, 0.0f, 90.0f) * (float)PI / 180.0f);
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.MeshAreaLights"), TEXT("MeshAreaLightSimplifyCornerDistanceThreshold"), Scene.MeshAreaLightSettings.MeshAreaLightSimplifyCornerDistanceThreshold, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.MeshAreaLights"), TEXT("MeshAreaLightSimplifyMeshBoundingRadiusFractionThreshold"), Scene.MeshAreaLightSettings.MeshAreaLightSimplifyMeshBoundingRadiusFractionThreshold, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.MeshAreaLights"), TEXT("MeshAreaLightGeneratedDynamicLightSurfaceOffset"), Scene.MeshAreaLightSettings.MeshAreaLightGeneratedDynamicLightSurfaceOffset, GLightmassIni));
	}
	{
		Scene.AmbientOcclusionSettings.bUseAmbientOcclusion = LevelSettings.bUseAmbientOcclusion;
		Scene.AmbientOcclusionSettings.bGenerateAmbientOcclusionMaterialMask = LevelSettings.bGenerateAmbientOcclusionMaterialMask;
		Scene.AmbientOcclusionSettings.bVisualizeAmbientOcclusion = LevelSettings.bVisualizeAmbientOcclusion;
		Scene.AmbientOcclusionSettings.DirectIlluminationOcclusionFraction = LevelSettings.DirectIlluminationOcclusionFraction;
		Scene.AmbientOcclusionSettings.IndirectIlluminationOcclusionFraction = LevelSettings.IndirectIlluminationOcclusionFraction;
		Scene.AmbientOcclusionSettings.OcclusionExponent = LevelSettings.OcclusionExponent;
		Scene.AmbientOcclusionSettings.FullyOccludedSamplesFraction = LevelSettings.FullyOccludedSamplesFraction;
		Scene.AmbientOcclusionSettings.MaxOcclusionDistance = LevelSettings.MaxOcclusionDistance;
	}
	{
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.PrecomputedDynamicObjectLighting"), TEXT("bVisualizeVolumeLightSamples"), bConfigBool, GLightmassIni));
		Scene.DynamicObjectSettings.bVisualizeVolumeLightSamples = bConfigBool;
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.PrecomputedDynamicObjectLighting"), TEXT("bVisualizeVolumeLightInterpolation"), bConfigBool, GLightmassIni));
		Scene.DynamicObjectSettings.bVisualizeVolumeLightInterpolation = bConfigBool;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.PrecomputedDynamicObjectLighting"), TEXT("NumHemisphereSamplesScale"), Scene.DynamicObjectSettings.NumHemisphereSamplesScale, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.PrecomputedDynamicObjectLighting"), TEXT("SurfaceLightSampleSpacing"), Scene.DynamicObjectSettings.SurfaceLightSampleSpacing, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.PrecomputedDynamicObjectLighting"), TEXT("FirstSurfaceSampleLayerHeight"), Scene.DynamicObjectSettings.FirstSurfaceSampleLayerHeight, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.PrecomputedDynamicObjectLighting"), TEXT("SurfaceSampleLayerHeightSpacing"), Scene.DynamicObjectSettings.SurfaceSampleLayerHeightSpacing, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.PrecomputedDynamicObjectLighting"), TEXT("NumSurfaceSampleLayers"), Scene.DynamicObjectSettings.NumSurfaceSampleLayers, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.PrecomputedDynamicObjectLighting"), TEXT("DetailVolumeSampleSpacing"), Scene.DynamicObjectSettings.DetailVolumeSampleSpacing, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.PrecomputedDynamicObjectLighting"), TEXT("VolumeLightSampleSpacing"), Scene.DynamicObjectSettings.VolumeLightSampleSpacing, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.PrecomputedDynamicObjectLighting"), TEXT("MaxVolumeSamples"), Scene.DynamicObjectSettings.MaxVolumeSamples, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.PrecomputedDynamicObjectLighting"), TEXT("bUseMaxSurfaceSampleNum"), bConfigBool, GLightmassIni));
		Scene.DynamicObjectSettings.bUseMaxSurfaceSampleNum = bConfigBool;
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.PrecomputedDynamicObjectLighting"), TEXT("MaxSurfaceLightSamples"), Scene.DynamicObjectSettings.MaxSurfaceLightSamples, GLightmassIni));

		Scene.DynamicObjectSettings.SurfaceLightSampleSpacing *= LevelSettings.VolumeLightSamplePlacementScale;
		Scene.DynamicObjectSettings.VolumeLightSampleSpacing *= LevelSettings.VolumeLightSamplePlacementScale;
		Scene.DynamicObjectSettings.DetailVolumeSampleSpacing *= LevelSettings.VolumeLightSamplePlacementScale;
	}
	{
		SetVolumetricLightmapSettings(Scene.VolumetricLightmapSettings);
	}
	{
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.PrecomputedVisibility"), TEXT("bVisualizePrecomputedVisibility"), bConfigBool, GLightmassIni));
		Scene.PrecomputedVisibilitySettings.bVisualizePrecomputedVisibility = bConfigBool;
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.PrecomputedVisibility"), TEXT("bPlaceCellsOnOpaqueOnly"), bConfigBool, GLightmassIni));
		Scene.PrecomputedVisibilitySettings.bPlaceCellsOnOpaqueOnly = bConfigBool;
		Scene.PrecomputedVisibilitySettings.bPlaceCellsOnlyAlongCameraTracks = World->GetWorldSettings()->bPlaceCellsOnlyAlongCameraTracks;
		Scene.PrecomputedVisibilitySettings.CellSize = World->GetWorldSettings()->VisibilityCellSize;
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.PrecomputedVisibility"), TEXT("NumCellDistributionBuckets"), Scene.PrecomputedVisibilitySettings.NumCellDistributionBuckets, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.PrecomputedVisibility"), TEXT("PlayAreaHeight"), Scene.PrecomputedVisibilitySettings.PlayAreaHeight, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.PrecomputedVisibility"), TEXT("MeshBoundsScale"), Scene.PrecomputedVisibilitySettings.MeshBoundsScale, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.PrecomputedVisibility"), TEXT("MinMeshSamples"), Scene.PrecomputedVisibilitySettings.MinMeshSamples, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.PrecomputedVisibility"), TEXT("MaxMeshSamples"), Scene.PrecomputedVisibilitySettings.MaxMeshSamples, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.PrecomputedVisibility"), TEXT("NumCellSamples"), Scene.PrecomputedVisibilitySettings.NumCellSamples, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.PrecomputedVisibility"), TEXT("NumImportanceSamples"), Scene.PrecomputedVisibilitySettings.NumImportanceSamples, GLightmassIni));
	}
	if (World->GetWorldSettings()->VisibilityAggressiveness != VIS_LeastAggressive)
	{
		const TCHAR* AggressivenessSectionNames[VIS_Max] = {
			TEXT(""), 
			TEXT("DevOptions.PrecomputedVisibilityModeratelyAggressive"), 
			TEXT("DevOptions.PrecomputedVisibilityMostAggressive")};
		const TCHAR* ActiveSection = AggressivenessSectionNames[FMath::Clamp((int32)World->GetWorldSettings()->VisibilityAggressiveness, 0, VIS_Max - 1)];
		VERIFYLIGHTMASSINI(GConfig->GetFloat(ActiveSection, TEXT("MeshBoundsScale"), Scene.PrecomputedVisibilitySettings.MeshBoundsScale, GLightmassIni));
	}
	{
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.VolumeDistanceField"), TEXT("VoxelSize"), Scene.VolumeDistanceFieldSettings.VoxelSize, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.VolumeDistanceField"), TEXT("VolumeMaxDistance"), Scene.VolumeDistanceFieldSettings.VolumeMaxDistance, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.VolumeDistanceField"), TEXT("NumVoxelDistanceSamples"), Scene.VolumeDistanceFieldSettings.NumVoxelDistanceSamples, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.VolumeDistanceField"), TEXT("MaxVoxels"), Scene.VolumeDistanceFieldSettings.MaxVoxels, GLightmassIni));
	}
	{
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.StaticShadows"), TEXT("bUseZeroAreaLightmapSpaceFilteredLights"), bConfigBool, GLightmassIni));
		Scene.ShadowSettings.bUseZeroAreaLightmapSpaceFilteredLights = bConfigBool;
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.StaticShadows"), TEXT("NumShadowRays"), Scene.ShadowSettings.NumShadowRays, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.StaticShadows"), TEXT("NumPenumbraShadowRays"), Scene.ShadowSettings.NumPenumbraShadowRays, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.StaticShadows"), TEXT("NumBounceShadowRays"), Scene.ShadowSettings.NumBounceShadowRays, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.StaticShadows"), TEXT("bFilterShadowFactor"), bConfigBool, GLightmassIni));
		Scene.ShadowSettings.bFilterShadowFactor = bConfigBool;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.StaticShadows"), TEXT("ShadowFactorGradientTolerance"), Scene.ShadowSettings.ShadowFactorGradientTolerance, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.StaticShadows"), TEXT("bAllowSignedDistanceFieldShadows"), bConfigBool, GLightmassIni));
		Scene.ShadowSettings.bAllowSignedDistanceFieldShadows = bConfigBool;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.StaticShadows"), TEXT("MaxTransitionDistanceWorldSpace"), Scene.ShadowSettings.MaxTransitionDistanceWorldSpace, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.StaticShadows"), TEXT("ApproximateHighResTexelsPerMaxTransitionDistance"), Scene.ShadowSettings.ApproximateHighResTexelsPerMaxTransitionDistance, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.StaticShadows"), TEXT("MinDistanceFieldUpsampleFactor"), Scene.ShadowSettings.MinDistanceFieldUpsampleFactor, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.StaticShadows"), TEXT("StaticShadowDepthMapTransitionSampleDistanceX"), Scene.ShadowSettings.StaticShadowDepthMapTransitionSampleDistanceX, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.StaticShadows"), TEXT("StaticShadowDepthMapTransitionSampleDistanceY"), Scene.ShadowSettings.StaticShadowDepthMapTransitionSampleDistanceY, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.StaticShadows"), TEXT("StaticShadowDepthMapSuperSampleFactor"), Scene.ShadowSettings.StaticShadowDepthMapSuperSampleFactor, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.StaticShadows"), TEXT("StaticShadowDepthMapMaxSamples"), Scene.ShadowSettings.StaticShadowDepthMapMaxSamples, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.StaticShadows"), TEXT("MinUnoccludedFraction"), Scene.ShadowSettings.MinUnoccludedFraction, GLightmassIni));
	}
	{
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.ImportanceTracing"), TEXT("bUseStratifiedSampling"), bConfigBool, GLightmassIni));
		Scene.ImportanceTracingSettings.bUseStratifiedSampling = bConfigBool;
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.ImportanceTracing"), TEXT("NumHemisphereSamples"), Scene.ImportanceTracingSettings.NumHemisphereSamples, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.ImportanceTracing"), TEXT("NumAdaptiveRefinementLevels"), Scene.ImportanceTracingSettings.NumAdaptiveRefinementLevels, GLightmassIni));
		float MaxHemisphereAngleDegrees;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.ImportanceTracing"), TEXT("MaxHemisphereRayAngle"), MaxHemisphereAngleDegrees, GLightmassIni));
		Scene.ImportanceTracingSettings.MaxHemisphereRayAngle = MaxHemisphereAngleDegrees * (float)PI / 180.0f;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.ImportanceTracing"), TEXT("AdaptiveBrightnessThreshold"), Scene.ImportanceTracingSettings.AdaptiveBrightnessThreshold, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.ImportanceTracing"), TEXT("AdaptiveFirstBouncePhotonConeAngle"), Scene.ImportanceTracingSettings.AdaptiveFirstBouncePhotonConeAngle, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.ImportanceTracing"), TEXT("AdaptiveSkyVarianceThreshold"), Scene.ImportanceTracingSettings.AdaptiveSkyVarianceThreshold, GLightmassIni));

		float AdaptiveFirstBouncePhotonConeAngle;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.ImportanceTracing"), TEXT("AdaptiveFirstBouncePhotonConeAngle"), AdaptiveFirstBouncePhotonConeAngle, GLightmassIni));
		Scene.ImportanceTracingSettings.AdaptiveFirstBouncePhotonConeAngle = FMath::Clamp(AdaptiveFirstBouncePhotonConeAngle, 0.0f, 90.0f) * (float)PI / 180.0f;

		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.ImportanceTracing"), TEXT("bUseRadiositySolverForSkylightMultibounce"), Scene.ImportanceTracingSettings.bUseRadiositySolverForSkylightMultibounce, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.ImportanceTracing"), TEXT("bCacheFinalGatherHitPointsForRadiosity"), Scene.ImportanceTracingSettings.bCacheFinalGatherHitPointsForRadiosity, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.ImportanceTracing"), TEXT("bUseRadiositySolverForLightMultibounce"), Scene.ImportanceTracingSettings.bUseRadiositySolverForLightMultibounce, GLightmassIni));
	}
	{
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.PhotonMapping"), TEXT("bUsePhotonMapping"), bConfigBool, GLightmassIni));
		Scene.PhotonMappingSettings.bUsePhotonMapping = bConfigBool;
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.PhotonMapping"), TEXT("bUseFinalGathering"), bConfigBool, GLightmassIni));
		Scene.PhotonMappingSettings.bUseFinalGathering = bConfigBool;
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.PhotonMapping"), TEXT("bUsePhotonDirectLightingInFinalGather"), bConfigBool, GLightmassIni));
		Scene.PhotonMappingSettings.bUsePhotonDirectLightingInFinalGather = bConfigBool;
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.PhotonMapping"), TEXT("bVisualizeCachedApproximateDirectLighting"), bConfigBool, GLightmassIni));
		Scene.PhotonMappingSettings.bVisualizeCachedApproximateDirectLighting = bConfigBool;
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.PhotonMapping"), TEXT("bUseIrradiancePhotons"), bConfigBool, GLightmassIni));
		Scene.PhotonMappingSettings.bUseIrradiancePhotons = bConfigBool;
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.PhotonMapping"), TEXT("bCacheIrradiancePhotonsOnSurfaces"), bConfigBool, GLightmassIni));
		Scene.PhotonMappingSettings.bCacheIrradiancePhotonsOnSurfaces = bConfigBool;
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.PhotonMapping"), TEXT("bVisualizePhotonPaths"), bConfigBool, GLightmassIni));
		Scene.PhotonMappingSettings.bVisualizePhotonPaths = bConfigBool;
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.PhotonMapping"), TEXT("bVisualizePhotonGathers"), bConfigBool, GLightmassIni));
		Scene.PhotonMappingSettings.bVisualizePhotonGathers = bConfigBool;
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.PhotonMapping"), TEXT("bVisualizePhotonImportanceSamples"), bConfigBool, GLightmassIni));
		Scene.PhotonMappingSettings.bVisualizePhotonImportanceSamples = bConfigBool;
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.PhotonMapping"), TEXT("bVisualizeIrradiancePhotonCalculation"), bConfigBool, GLightmassIni));
		Scene.PhotonMappingSettings.bVisualizeIrradiancePhotonCalculation = bConfigBool;
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.PhotonMapping"), TEXT("bEmitPhotonsOutsideImportanceVolume"), bConfigBool, GLightmassIni));
		Scene.PhotonMappingSettings.bEmitPhotonsOutsideImportanceVolume = bConfigBool;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.PhotonMapping"), TEXT("ConeFilterConstant"), Scene.PhotonMappingSettings.ConeFilterConstant, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.PhotonMapping"), TEXT("NumIrradianceCalculationPhotons"), Scene.PhotonMappingSettings.NumIrradianceCalculationPhotons, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.PhotonMapping"), TEXT("FinalGatherImportanceSampleFraction"), Scene.PhotonMappingSettings.FinalGatherImportanceSampleFraction, GLightmassIni));
		float FinalGatherImportanceSampleConeAngle;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.PhotonMapping"), TEXT("FinalGatherImportanceSampleConeAngle"), FinalGatherImportanceSampleConeAngle, GLightmassIni));
		Scene.PhotonMappingSettings.FinalGatherImportanceSampleCosConeAngle = FMath::Cos(FMath::Clamp(FinalGatherImportanceSampleConeAngle, 0.0f, 90.0f) * (float)PI / 180.0f);
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.PhotonMapping"), TEXT("IndirectPhotonEmitDiskRadius"), Scene.PhotonMappingSettings.IndirectPhotonEmitDiskRadius, GLightmassIni));
		float IndirectPhotonEmitConeAngleDegrees;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.PhotonMapping"), TEXT("IndirectPhotonEmitConeAngle"), IndirectPhotonEmitConeAngleDegrees, GLightmassIni));
		Scene.PhotonMappingSettings.IndirectPhotonEmitConeAngle = FMath::Clamp(IndirectPhotonEmitConeAngleDegrees, 0.0f, 90.0f) * (float)PI / 180.0f;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.PhotonMapping"), TEXT("MaxImportancePhotonSearchDistance"), Scene.PhotonMappingSettings.MaxImportancePhotonSearchDistance, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.PhotonMapping"), TEXT("MinImportancePhotonSearchDistance"), Scene.PhotonMappingSettings.MinImportancePhotonSearchDistance, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.PhotonMapping"), TEXT("NumImportanceSearchPhotons"), Scene.PhotonMappingSettings.NumImportanceSearchPhotons, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.PhotonMapping"), TEXT("OutsideImportanceVolumeDensityScale"), Scene.PhotonMappingSettings.OutsideImportanceVolumeDensityScale, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.PhotonMapping"), TEXT("DirectPhotonDensity"), Scene.PhotonMappingSettings.DirectPhotonDensity, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.PhotonMapping"), TEXT("DirectIrradiancePhotonDensity"), Scene.PhotonMappingSettings.DirectIrradiancePhotonDensity, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.PhotonMapping"), TEXT("DirectPhotonSearchDistance"), Scene.PhotonMappingSettings.DirectPhotonSearchDistance, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.PhotonMapping"), TEXT("IndirectPhotonPathDensity"), Scene.PhotonMappingSettings.IndirectPhotonPathDensity, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.PhotonMapping"), TEXT("IndirectPhotonDensity"), Scene.PhotonMappingSettings.IndirectPhotonDensity, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.PhotonMapping"), TEXT("IndirectIrradiancePhotonDensity"), Scene.PhotonMappingSettings.IndirectIrradiancePhotonDensity, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.PhotonMapping"), TEXT("IndirectPhotonSearchDistance"), Scene.PhotonMappingSettings.IndirectPhotonSearchDistance, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.PhotonMapping"), TEXT("PhotonSearchAngleThreshold"), Scene.PhotonMappingSettings.PhotonSearchAngleThreshold, GLightmassIni));
		float IrradiancePhotonSearchConeAngle;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.PhotonMapping"), TEXT("IrradiancePhotonSearchConeAngle"), IrradiancePhotonSearchConeAngle, GLightmassIni));
		Scene.PhotonMappingSettings.MinCosIrradiancePhotonSearchCone = FMath::Cos((90.0f - FMath::Clamp(IrradiancePhotonSearchConeAngle, 1.0f, 90.0f)) * (float)PI / 180.0f);
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.PhotonMapping"), TEXT("bUsePhotonSegmentsForVolumeLighting"), Scene.PhotonMappingSettings.bUsePhotonSegmentsForVolumeLighting, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.PhotonMapping"), TEXT("PhotonSegmentMaxLength"), Scene.PhotonMappingSettings.PhotonSegmentMaxLength, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.PhotonMapping"), TEXT("GeneratePhotonSegmentChance"), Scene.PhotonMappingSettings.GeneratePhotonSegmentChance, GLightmassIni));
	}
	{
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.IrradianceCache"), TEXT("bAllowIrradianceCaching"), bConfigBool, GLightmassIni));
		Scene.IrradianceCachingSettings.bAllowIrradianceCaching = bConfigBool;
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.IrradianceCache"), TEXT("bUseIrradianceGradients"), bConfigBool, GLightmassIni));
		Scene.IrradianceCachingSettings.bUseIrradianceGradients = bConfigBool;
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.IrradianceCache"), TEXT("bShowGradientsOnly"), bConfigBool, GLightmassIni));
		Scene.IrradianceCachingSettings.bShowGradientsOnly = bConfigBool;
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.IrradianceCache"), TEXT("bVisualizeIrradianceSamples"), bConfigBool, GLightmassIni));
		Scene.IrradianceCachingSettings.bVisualizeIrradianceSamples = bConfigBool;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.IrradianceCache"), TEXT("RecordRadiusScale"), Scene.IrradianceCachingSettings.RecordRadiusScale, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.IrradianceCache"), TEXT("InterpolationMaxAngle"), Scene.IrradianceCachingSettings.InterpolationMaxAngle, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.IrradianceCache"), TEXT("PointBehindRecordMaxAngle"), Scene.IrradianceCachingSettings.PointBehindRecordMaxAngle, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.IrradianceCache"), TEXT("DistanceSmoothFactor"), Scene.IrradianceCachingSettings.DistanceSmoothFactor, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.IrradianceCache"), TEXT("AngleSmoothFactor"), Scene.IrradianceCachingSettings.AngleSmoothFactor, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.IrradianceCache"), TEXT("SkyOcclusionSmoothnessReduction"), Scene.IrradianceCachingSettings.SkyOcclusionSmoothnessReduction, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.IrradianceCache"), TEXT("MaxRecordRadius"), Scene.IrradianceCachingSettings.MaxRecordRadius, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.IrradianceCache"), TEXT("CacheTaskSize"), Scene.IrradianceCachingSettings.CacheTaskSize, GLightmassIni));
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.IrradianceCache"), TEXT("InterpolateTaskSize"), Scene.IrradianceCachingSettings.InterpolateTaskSize, GLightmassIni));
	}

	// Modify settings based on the quality level required
	// Preview is assumed to have a scale of 1 for all settings and therefore is not in the ini
	if (QualityLevel != Quality_Preview)
	{
		const TCHAR* QualitySectionNames[Quality_MAX] = {
			TEXT(""), 
			TEXT("DevOptions.StaticLightingMediumQuality"), 
			TEXT("DevOptions.StaticLightingHighQuality"), 
			TEXT("DevOptions.StaticLightingProductionQuality")};

		float NumShadowRaysScale;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(QualitySectionNames[QualityLevel], TEXT("NumShadowRaysScale"), NumShadowRaysScale, GLightmassIni));
		Scene.ShadowSettings.NumShadowRays = FMath::TruncToInt(Scene.ShadowSettings.NumShadowRays * NumShadowRaysScale);

		float NumPenumbraShadowRaysScale;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(QualitySectionNames[QualityLevel], TEXT("NumPenumbraShadowRaysScale"), NumPenumbraShadowRaysScale, GLightmassIni));
		Scene.ShadowSettings.NumPenumbraShadowRays = FMath::TruncToInt(Scene.ShadowSettings.NumPenumbraShadowRays * NumPenumbraShadowRaysScale);

		float ApproximateHighResTexelsPerMaxTransitionDistanceScale;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(QualitySectionNames[QualityLevel], TEXT("ApproximateHighResTexelsPerMaxTransitionDistanceScale"), ApproximateHighResTexelsPerMaxTransitionDistanceScale, GLightmassIni));
		Scene.ShadowSettings.ApproximateHighResTexelsPerMaxTransitionDistance = FMath::TruncToInt(Scene.ShadowSettings.ApproximateHighResTexelsPerMaxTransitionDistance * ApproximateHighResTexelsPerMaxTransitionDistanceScale);

		VERIFYLIGHTMASSINI(GConfig->GetInt(QualitySectionNames[QualityLevel], TEXT("MinDistanceFieldUpsampleFactor"), Scene.ShadowSettings.MinDistanceFieldUpsampleFactor, GLightmassIni));

		float NumHemisphereSamplesScale;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(QualitySectionNames[QualityLevel], TEXT("NumHemisphereSamplesScale"), NumHemisphereSamplesScale, GLightmassIni));
		Scene.ImportanceTracingSettings.NumHemisphereSamples = FMath::TruncToInt(Scene.ImportanceTracingSettings.NumHemisphereSamples * NumHemisphereSamplesScale);

		float NumImportanceSearchPhotonsScale;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(QualitySectionNames[QualityLevel], TEXT("NumImportanceSearchPhotonsScale"), NumImportanceSearchPhotonsScale, GLightmassIni));
		Scene.PhotonMappingSettings.NumImportanceSearchPhotons = FMath::TruncToInt(Scene.PhotonMappingSettings.NumImportanceSearchPhotons * NumImportanceSearchPhotonsScale);

		float NumDirectPhotonsScale;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(QualitySectionNames[QualityLevel], TEXT("NumDirectPhotonsScale"), NumDirectPhotonsScale, GLightmassIni));
		Scene.PhotonMappingSettings.DirectPhotonDensity = Scene.PhotonMappingSettings.DirectPhotonDensity * NumDirectPhotonsScale;
		Scene.PhotonMappingSettings.DirectIrradiancePhotonDensity = Scene.PhotonMappingSettings.DirectIrradiancePhotonDensity * NumDirectPhotonsScale; 

		float DirectPhotonSearchDistanceScale;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(QualitySectionNames[QualityLevel], TEXT("DirectPhotonSearchDistanceScale"), DirectPhotonSearchDistanceScale, GLightmassIni));
		Scene.PhotonMappingSettings.DirectPhotonSearchDistance = Scene.PhotonMappingSettings.DirectPhotonSearchDistance * DirectPhotonSearchDistanceScale;

		float NumIndirectPhotonPathsScale;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(QualitySectionNames[QualityLevel], TEXT("NumIndirectPhotonPathsScale"), NumIndirectPhotonPathsScale, GLightmassIni));
		Scene.PhotonMappingSettings.IndirectPhotonPathDensity = Scene.PhotonMappingSettings.IndirectPhotonPathDensity * NumIndirectPhotonPathsScale;

		float NumIndirectPhotonsScale;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(QualitySectionNames[QualityLevel], TEXT("NumIndirectPhotonsScale"), NumIndirectPhotonsScale, GLightmassIni));
		Scene.PhotonMappingSettings.IndirectPhotonDensity = Scene.PhotonMappingSettings.IndirectPhotonDensity * NumIndirectPhotonsScale;

		float NumIndirectIrradiancePhotonsScale;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(QualitySectionNames[QualityLevel], TEXT("NumIndirectIrradiancePhotonsScale"), NumIndirectIrradiancePhotonsScale, GLightmassIni));
		Scene.PhotonMappingSettings.IndirectIrradiancePhotonDensity = Scene.PhotonMappingSettings.IndirectIrradiancePhotonDensity * NumIndirectIrradiancePhotonsScale;

		float RecordRadiusScaleScale;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(QualitySectionNames[QualityLevel], TEXT("RecordRadiusScaleScale"), RecordRadiusScaleScale, GLightmassIni));
		Scene.IrradianceCachingSettings.RecordRadiusScale = Scene.IrradianceCachingSettings.RecordRadiusScale * RecordRadiusScaleScale;

		float InterpolationMaxAngleScale;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(QualitySectionNames[QualityLevel], TEXT("InterpolationMaxAngleScale"), InterpolationMaxAngleScale, GLightmassIni));
		Scene.IrradianceCachingSettings.InterpolationMaxAngle = Scene.IrradianceCachingSettings.InterpolationMaxAngle * InterpolationMaxAngleScale;

		float IrradianceCacheSmoothFactor;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(QualitySectionNames[QualityLevel], TEXT("IrradianceCacheSmoothFactor"), IrradianceCacheSmoothFactor, GLightmassIni));
		Scene.IrradianceCachingSettings.DistanceSmoothFactor *= IrradianceCacheSmoothFactor;
		Scene.IrradianceCachingSettings.AngleSmoothFactor *= IrradianceCacheSmoothFactor;

		VERIFYLIGHTMASSINI(GConfig->GetInt(QualitySectionNames[QualityLevel], TEXT("NumAdaptiveRefinementLevels"), Scene.ImportanceTracingSettings.NumAdaptiveRefinementLevels, GLightmassIni));

		float AdaptiveBrightnessThresholdScale;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(QualitySectionNames[QualityLevel], TEXT("AdaptiveBrightnessThresholdScale"), AdaptiveBrightnessThresholdScale, GLightmassIni));
		Scene.ImportanceTracingSettings.AdaptiveBrightnessThreshold = Scene.ImportanceTracingSettings.AdaptiveBrightnessThreshold * AdaptiveBrightnessThresholdScale;

		float AdaptiveFirstBouncePhotonConeAngleScale;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(QualitySectionNames[QualityLevel], TEXT("AdaptiveFirstBouncePhotonConeAngleScale"), AdaptiveFirstBouncePhotonConeAngleScale, GLightmassIni));
		Scene.ImportanceTracingSettings.AdaptiveFirstBouncePhotonConeAngle = Scene.ImportanceTracingSettings.AdaptiveFirstBouncePhotonConeAngle * AdaptiveFirstBouncePhotonConeAngleScale;

		float AdaptiveSkyVarianceThresholdScale;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(QualitySectionNames[QualityLevel], TEXT("AdaptiveSkyVarianceThresholdScale"), AdaptiveSkyVarianceThresholdScale, GLightmassIni));
		Scene.ImportanceTracingSettings.AdaptiveSkyVarianceThreshold = Scene.ImportanceTracingSettings.AdaptiveSkyVarianceThreshold * AdaptiveSkyVarianceThresholdScale;
	}
}

/** Fills InputData with debug information */
void FLightmassExporter::WriteDebugInput( Lightmass::FDebugLightingInputData& InputData, FGuid& DebugMappingGuid )
{
	InputData.bRelaySolverStats = UE_LOG_ACTIVE(LogLightmassSolver, Log);

	if (IsTexelDebuggingEnabled())
	{
		FindDebugMapping(DebugMappingGuid);
	}
	
	InputData.MappingGuid = DebugMappingGuid;
	InputData.NodeIndex = GCurrentSelectedLightmapSample.NodeIndex;
	InputData.Position = FVector4f(FVector3f(GCurrentSelectedLightmapSample.Position), 0);
	InputData.LocalX = GCurrentSelectedLightmapSample.LocalX;
	InputData.LocalY = GCurrentSelectedLightmapSample.LocalY;
	InputData.MappingSizeX = GCurrentSelectedLightmapSample.MappingSizeX;
	InputData.MappingSizeY = GCurrentSelectedLightmapSample.MappingSizeY;
	FVector4 ViewPosition(0, 0, 0, 0);
	for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		if (LevelVC->IsPerspective())
		{
			ViewPosition = LevelVC->GetViewLocation();
		}
	}
	InputData.CameraPosition = (FVector4f)ViewPosition;
	int32 DebugVisibilityId = INDEX_NONE;
	bool bVisualizePrecomputedVisibility = false;
	VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.PrecomputedVisibility"), TEXT("bVisualizePrecomputedVisibility"), bVisualizePrecomputedVisibility, GLightmassIni));
	if (bVisualizePrecomputedVisibility)
	{
		for (FSelectedActorIterator It(World); It; ++It)
		{
			for (UActorComponent* Comp : It->GetComponents())
			{
				if (UPrimitiveComponent* Component = Cast<UPrimitiveComponent>(Comp))
				{
					if (DebugVisibilityId == INDEX_NONE)
					{
						DebugVisibilityId = Component->VisibilityId;
					}
					else if (DebugVisibilityId != Component->VisibilityId)
					{
						UE_LOGF(LogLightmassSolver, Warning, "Not debugging visibility for component %ls with vis id %u, as it was not the first component on the selected actor.",
							*Component->GetPathName(), Component->VisibilityId);
					}
				}
			}
		}

		for (int32 LevelIndex= 0; LevelIndex < World->GetNumLevels(); LevelIndex++)
		{
			ULevel* Level = World->GetLevel(LevelIndex);
			for (int32 SurfIdx = 0; SurfIdx < Level->Model->Surfs.Num(); SurfIdx++)
			{
				const FBspSurf& Surf = Level->Model->Surfs[SurfIdx];
				if ((Surf.PolyFlags & PF_Selected) != 0)
				{
					for (int32 NodeIdx = 0; NodeIdx < Level->Model->Nodes.Num(); NodeIdx++)
					{
						const FBspNode& Node = Level->Model->Nodes[NodeIdx];
						if (Node.iSurf == SurfIdx)
						{
							UModelComponent* SomeModelComponent = Level->ModelComponents[Node.ComponentIndex];
							if (DebugVisibilityId == INDEX_NONE)
							{
								DebugVisibilityId = SomeModelComponent->VisibilityId;
							}
							else if (DebugVisibilityId != SomeModelComponent->VisibilityId)
							{
								UE_LOGF(LogLightmassSolver, Warning, "Not debugging visibility for model component %ls with vis id %u!",
									*SomeModelComponent->GetPathName(), SomeModelComponent->VisibilityId);
							}
						}
					}
				}
			}
		}
	}
	InputData.DebugVisibilityId = DebugVisibilityId;
}

void FLightmassExporter::UpdateExportProgress()
{
	CurrentProgress++;

	// Update rarely to reduce time spent redrawing the status, which can be significant
	const int32 ProgressUpdateFrequency = FMath::Max<int32>(TotalProgress / 20,1);
	if (CurrentProgress % ProgressUpdateFrequency == 0)
	{
		GWarn->UpdateProgress(CurrentProgress, TotalProgress);
	}
}

void FLightmassExporter::AddLight(ULightComponentBase* Light)
{
	UDirectionalLightComponent* DirectionalLight = Cast<UDirectionalLightComponent>(Light);
	UPointLightComponent* PointLight = Cast<UPointLightComponent>(Light);
	USpotLightComponent* SpotLight = Cast<USpotLightComponent>(Light);
	URectLightComponent* RectLight = Cast<URectLightComponent>(Light);
	USkyLightComponent* SkyLight = Cast<USkyLightComponent>(Light);

	if( DirectionalLight )
	{
		DirectionalLights.AddUnique(DirectionalLight);
	}
	else if( SpotLight )
	{
		SpotLights.AddUnique(SpotLight);
	}
	else if( PointLight )
	{
		PointLights.AddUnique(PointLight);
	}
	else if( RectLight )
	{
		RectLights.AddUnique(RectLight);
	}
	else if( SkyLight )
	{
		SkyLights.AddUnique(SkyLight);
	}
}

/*-----------------------------------------------------------------------------
	FLightmassProcessor
-----------------------------------------------------------------------------*/

/** 
 * Constructor
 * 
 * @param bInDumpBinaryResults true if it should dump out raw binary lighting data to disk
 */
FLightmassProcessor::FLightmassProcessor(const FStaticLightingSystem& InSystem, bool bInDumpBinaryResults, bool bInOnlyBuildVisibility)
:	Exporter(NULL)
,	Importer(NULL)
,	System(InSystem)
,	Swarm( NSwarm::FSwarmInterface::Get() )
,	bProcessingSuccessful(false)
,	bProcessingFailed(false)
,	bQuitReceived(false)
,	NumCompletedTasks(0)
,	bRunningLightmass(false)
,	bOnlyBuildVisibility( bInOnlyBuildVisibility )
,	bDumpBinaryResults( bInDumpBinaryResults )
,	bImportCompletedMappingsImmediately(false)
,	MappingToProcessIndex(0)
{
	// Since these can be set by the commandline, we need to update them here...
	GLightmassDebugOptions.bDebugMode = GLightmassDebugMode;
	GLightmassDebugOptions.bStatsEnabled = GLightmassStatsMode;

	NSwarm::TLogFlags LogFlags = NSwarm::SWARM_LOG_NONE;
	if (GLightmassDebugOptions.bStatsEnabled)
	{
		LogFlags = NSwarm::TLogFlags(LogFlags | NSwarm::SWARM_LOG_TIMINGS);
	}

	FString OptionsFolder = FPaths::Combine(*FPaths::GameAgnosticSavedDir(), TEXT("Swarm"));
	OptionsFolder = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*OptionsFolder);
	int32 ConnectionHandle = Swarm.OpenConnection( SwarmCallback, this,  LogFlags, *OptionsFolder );
	bSwarmConnectionIsValid = (ConnectionHandle >= 0);
	Exporter = new FLightmassExporter( System.GetLightingContext() );
	check(Exporter);
	Exporter->bSwarmConnectionIsValid = bSwarmConnectionIsValid;

	DeferredMappingsDirectory = InSystem.Options.MappingsDirectory;

	Messages.Add( TEXT("UseErrorColoringButton_Tooltip"), LOCTEXT("UseErrorColoringButton_Tooltip", "Display objects with lighting errors in identifying colors rather than black (Lightmass only).") );
	Messages.Add( TEXT("LightmassError_SupportFP"), LOCTEXT("LightmassError_SupportFP", "Lightmass requires a graphics card with support for floating point rendertargets. Aborting!") );
	Messages.Add( TEXT("LightmassError_MissingImportanceVolume"), LOCTEXT("LightmassError_MissingImportanceVolume", "No importance volume found - lighting build will take a long time and have poor quality.") );
	Messages.Add( TEXT("LightmassError_FailedToAllocateShadowmapChannel"), LOCTEXT("LightmassError_FailedToAllocateShadowmapChannel", "Severe performance loss: Failed to allocate shadowmap channel for stationary light due to overlap - light will fall back to dynamic shadows!") );
	Messages.Add( TEXT("LightmassError_MissingPrecomputedVisibilityVolume"), LOCTEXT("LightmassError_MissingPrecomputedVisibilityVolume", "Level has bPrecomputeVisibility enabled but no Precomputed Visibility Volumes, precomputed visibility will not be effective.") );
	Messages.Add( TEXT("LightmassError_BuildSelected"), LOCTEXT("LightmassError_BuildSelected", "Building selected actors only, lightmap memory and quality will be sub-optimal until the next full rebuild.") );
	Messages.Add( TEXT("LightmassError_BuildSelectedNothingSelected"), LOCTEXT("LightmassError_BuildSelectedNothingSelected", "Building selected actors and BSP only, but no actors or BSP selected!") );
	Messages.Add( TEXT("LightmassError_ObjectWrappedUVs"), LOCTEXT("LightmassError_ObjectWrappedUVs", "Object has wrapping UVs.") );
	Messages.Add( TEXT("LightmassError_ObjectOverlappedUVs"), LOCTEXT("LightmassError_ObjectOverlappedUVs", "Object has overlapping UVs.") );
	Messages.Add( TEXT("LightmassError_EmissiveMeshHighPolyCount"), LOCTEXT("LightmassError_EmissiveMeshHighPolyCount", "Object has a large number of polygons (more than 3000) and will result in a long lighting build.") );
	Messages.Add( TEXT("LightmassError_EmissiveMeshExtremelyHighPolyCount"), LOCTEXT("LightmassError_EmissiveMeshExtremelyHighPolyCount", "Object did not create emissive lights due to excessive polycount (more than 5000).") );
	Messages.Add( TEXT("LightmassError_BadLightMapCoordinateIndex"), LOCTEXT("LightmassError_BadLightMapCoordinateIndex", "StaticMesh has invalid LightMapCoordinateIndex.") );
	Messages.Add( TEXT("LightmassError_ObjectMultipleDominantLights"), LOCTEXT("LightmassError_ObjectMultipleDominantLights", "Object has multiple dominant lights.") );
}

FLightmassProcessor::~FLightmassProcessor()
{
	// Note: the connection must be closed before deleting anything that SwarmCallback accesses
	Swarm.CloseConnection();

	delete Exporter;
	delete Importer;

	for ( TMap<FGuid, FMappingImportHelper*>::TIterator It(ImportedMappings); It; ++It )
	{
		FMappingImportHelper* ImportData = It.Value();
		delete ImportData;
	}
	ImportedMappings.Empty();

	delete DeferredMappings;

	FLandscapeStaticLightingMesh::LandscapeUpscaleHeightDataCache.Empty();
}

/** Retrieve an exporter for the given channel name */
FLightmassExporter* FLightmassProcessor::GetLightmassExporter()
{
	return Exporter;
}

FString FLightmassProcessor::GetMappingFileExtension(const FStaticLightingMapping* InMapping)
{
	// Determine the input file name
	FString FileExtension = TEXT("");
	if (InMapping)
	{
		if (InMapping->IsTextureMapping() == true)
		{
			FileExtension = Lightmass::LM_TEXTUREMAPPING_EXTENSION;
		}
	}

	return FileExtension;
}

int32 FLightmassProcessor_GetMappingFileVersion(const FStaticLightingMapping* InMapping)
{
	// Determine the input file name
	int32 ReturnVersion = 0;
	if (InMapping)
	{
		if (InMapping->IsTextureMapping() == true)
		{
			ReturnVersion = Lightmass::LM_TEXTUREMAPPING_VERSION;
		}
	}
	return ReturnVersion;
}

bool FLightmassProcessor::OpenJob()
{
	// Start the Job
	int32 ErrorCode = Swarm.OpenJob( Exporter->SceneGuid );
	if( ErrorCode < 0 )
	{
		UE_LOGF(LogLightmassSolver, Log,  "Error, OpenJob failed with error code %d", ErrorCode );
		return false;
	}
	return true;
}

bool FLightmassProcessor::CloseJob()
{
	// All done, end the Job
	int32 ErrorCode = Swarm.CloseJob();
	if( ErrorCode < 0 )
	{
		UE_LOGF(LogLightmassSolver, Log,  "Error, CloseJob failed with error code %d", ErrorCode );
		return false;
	}
	return true;
}

void FLightmassProcessor::InitiateExport()
{
	FLightmassStatistics::FScopedGather ExportStatScope(Statistics.ExportTime);

	// If the Job started successfully, export the scene
	GWarn->StatusUpdate( 0, 100, LOCTEXT("BeginExportingTheSceneTask", "Exporting the scene...") );
	double StartTime = FPlatformTime::Seconds();

	int32 NumCellDistributionBuckets;
	VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.PrecomputedVisibility"), TEXT("NumCellDistributionBuckets"), NumCellDistributionBuckets, GLightmassIni));
	
	if (System.GetWorld()->GetWorldSettings()->bPrecomputeVisibility)
	{
		for (int32 DistributionBucketIndex = 0; DistributionBucketIndex < NumCellDistributionBuckets; DistributionBucketIndex++)
		{
			Exporter->VisibilityBucketGuids.Add(FGuid::NewGuid());
		}
	}

	if (System.GetWorld()->GetWorldSettings()->LightmassSettings.VolumeLightingMethod == VLM_VolumetricLightmap
		&& !bOnlyBuildVisibility 
		&& !System.Options.bApplyDeferedActorMappingPass )
	{
		Lightmass::FVolumetricLightmapSettings VolumetricLightmapSettings;
		GetLightmassExporter()->SetVolumetricLightmapSettings(VolumetricLightmapSettings);

		const int32 NumTopLevelBricks = VolumetricLightmapSettings.TopLevelGridSize.X * VolumetricLightmapSettings.TopLevelGridSize.Y * VolumetricLightmapSettings.TopLevelGridSize.Z;
		
		int32 TargetNumVolumetricLightmapTasks;
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.VolumetricLightmaps"), TEXT("TargetNumVolumetricLightmapTasks"), TargetNumVolumetricLightmapTasks, GLightmassIni));

		const int32 NumTasksPerTopLevelBrick = FMath::Clamp(TargetNumVolumetricLightmapTasks / NumTopLevelBricks, 1, VolumetricLightmapSettings.BrickSize * VolumetricLightmapSettings.BrickSize * VolumetricLightmapSettings.BrickSize);

		// Generate task guids for top level volumetric lightmap cells
		for (int32 VolumetricLightmapTaskIndex = 0; 
			VolumetricLightmapTaskIndex < NumTopLevelBricks * NumTasksPerTopLevelBrick; 
			VolumetricLightmapTaskIndex++)
		{
			Exporter->VolumetricLightmapTaskGuids.Add(FGuid::NewGuid(), VolumetricLightmapTaskIndex);
		}
	}

	TArray<ULevel*> LevelsToRestore;

	// Hide all other lighting scenarios before we export skylights, which capture the rendered scene
	for (ULevel* Level : System.GetWorld()->GetLevels())
	{
		if (Level->bIsVisible && !System.ShouldOperateOnLevel(Level))
		{
			LevelsToRestore.Add(Level);
			EditorLevelUtils::SetLevelVisibilityTemporarily(Level, false);
		}
	}

	Exporter->WriteToChannel(Statistics, DebugMappingGuid);

	// Restore level state
	for (ULevel* Level : LevelsToRestore)
	{
		EditorLevelUtils::SetLevelVisibilityTemporarily(Level, true);
	}
}

bool FLightmassProcessor::ExecuteAmortizedMaterialExport()
{
	FLightmassStatistics::FScopedGather ExportStatScope(Statistics.ExportTime);

	return Exporter->WriteToMaterialChannel(Statistics);
}

void FLightmassProcessor::IssueStaticShadowDepthMapTask(const ULightComponent* Light, int32 EstimatedCost)
{
	if (Light->HasStaticShadowing() && !Light->HasStaticLighting())
	{
		NSwarm::FTaskSpecification NewTaskSpecification(Light->LightGuid, TEXT("StaticShadowDepthMaps"), NSwarm::JOB_TASK_FLAG_USE_DEFAULTS );
		NewTaskSpecification.Cost = EstimatedCost;
		int32 ErrorCode = Swarm.AddTask( NewTaskSpecification );
		if( ErrorCode >= 0 )
		{
			NumTotalSwarmTasks++;
		}
		else
		{
			UE_LOGF(LogLightmassSolver, Warning,  "Error, AddTask for StaticShadowDepthMaps failed with error code %d", ErrorCode );
			bProcessingFailed = true;
		}
	}
}

// Helper class for creating a list of paths relative to the engine directory, which can be queried as a list of raw strings.
class FEngineDependencyPaths
{
public:
	FEngineDependencyPaths(std::initializer_list<const TCHAR*> Paths, const TArray<FString>& InDynamicDependencyPaths = TArray<FString>())
	{
		for(const TCHAR* Path : Paths)
		{
			Strings.Add(FPaths::EngineDir() / Path);
		}
		for(const FString& Path : InDynamicDependencyPaths)
		{
			Strings.Add(FPaths::EngineDir() / Path);
		}

		RawStrings.Reserve(Strings.Num());
		for(const FString& String : Strings)
		{
			RawStrings.Add(*String);
		}
	}

	const TCHAR** GetArray() const
	{
		return const_cast<const TCHAR**>(RawStrings.GetData());
	}

	int32 Num() const
	{
		return RawStrings.Num();
	}

private:
	TArray<FString> Strings;
	TArray<const TCHAR*> RawStrings;
};

static TArray<FString> GetDynamicDependencyPathsFromManifest(const TCHAR* BinDir)
{
	TArray<FString> DynamicDependencyPaths;
	FString FileData;
	if (FFileHelper::LoadFileToString(FileData, *(FPaths::EngineDir() / BinDir / "UnrealLightmass.target")))
	{
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(FileData);
		TSharedPtr<FJsonObject> RootJsonObject;
		FJsonSerializer::Deserialize(Reader, RootJsonObject);
		if (RootJsonObject)
		{
			const TArray<TSharedPtr<FJsonValue>>* BuildProducts = nullptr;
			if (RootJsonObject->TryGetArrayField(TEXT("BuildProducts"), BuildProducts))
			{
				for (const auto& BuildProductField : *BuildProducts)
				{
					auto Obj = BuildProductField->AsObject();
					FString TypeStr;
					if (Obj && Obj->TryGetStringField(TEXT("Type"), TypeStr) && TypeStr == TEXT("DynamicLibrary"))
					{
						FString Tmp;
						if (Obj->TryGetStringField(TEXT("Path"), Tmp) && Tmp.RemoveFromStart(TEXT("$(EngineDir)/")))
						{
							DynamicDependencyPaths.Emplace(Tmp);
						}
					}
				}
			}
		}
	}
	return DynamicDependencyPaths;
}

bool FLightmassProcessor::BeginRun()
{
	{
		FLightmassStatistics::FScopedGather ExportStatScope(Statistics.ExportTime);
		bool bGarbageCollectAfterExport = false;
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.StaticLighting"), TEXT("bGarbageCollectAfterExport"), bGarbageCollectAfterExport, GLightmassIni));
		if (bGarbageCollectAfterExport == true)
		{
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, true);
		}
	}

	double SwarmJobStartTime = FPlatformTime::Seconds();
	VolumeSampleTaskCompleted = 0;
	MeshAreaLightDataTaskCompleted = 0;
	VolumeDistanceFieldTaskCompleted = 0;

	// Check if we can use 64-bit Lightmass.
	bool bUse64bitProcess = false;
	bool bAllow64bitProcess = true;
	VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.StaticLighting"), TEXT("bAllow64bitProcess"), bAllow64bitProcess, GLightmassIni));
	if ( bAllow64bitProcess )
	{
		bUse64bitProcess = true;
	}

	// Setup dependencies for 32bit.
	static const FString LightmassExecutable32 = FPaths::EngineDir() / TEXT("Binaries/Win32/UnrealLightmass.exe");
	const FEngineDependencyPaths RequiredDependencyPaths32(
	{
		TEXT("Binaries/DotNET/SwarmInterface.dll"),
	}, GetDynamicDependencyPathsFromManifest(TEXT("Binaries/Win32/")));

	// Setup dependencies for 64bit.
#if PLATFORM_WINDOWS
	static const FString LightmassExecutable64 = FPaths::EngineDir() / TEXT("Binaries/Win64/UnrealLightmass.exe");
	const FEngineDependencyPaths RequiredDependencyPaths64( 
	{ 
		TEXT("Binaries/DotNET/SwarmInterface.dll"),
	}, GetDynamicDependencyPathsFromManifest(TEXT("Binaries/Win64/")));
#elif PLATFORM_MAC
	static const FString LightmassExecutable64 = FPaths::EngineDir() / TEXT("Binaries/Mac/UnrealLightmass");
	const FEngineDependencyPaths RequiredDependencyPaths64( 
	{ 
		TEXT("Binaries/DotNET/Mac/AgentInterface.dll"),
	}, GetDynamicDependencyPathsFromManifest(TEXT("Binaries/Mac/")));
#elif PLATFORM_LINUX
	static const FString LightmassExecutable64 = FPaths::EngineDir() / TEXT("Binaries/Linux/UnrealLightmass");
	const FEngineDependencyPaths RequiredDependencyPaths64( 
	{ 
		TEXT("Binaries/DotNET/Linux/AgentInterface.dll"),
	}, GetDynamicDependencyPathsFromManifest(TEXT("Binaries/Linux/")));
#else // PLATFORM_LINUX
#error "Unknown Lightmass platform"
#endif

	// Set up optional dependencies.  These might not exist in Launcher distributions, for example.
	const FEngineDependencyPaths OptionalDependencyPaths32 =
	{
		TEXT("Binaries/Win32/UnrealLightmass.pdb")
	};

	const FEngineDependencyPaths OptionalDependencyPaths64 =
	{
		TEXT("Binaries/Win64/UnrealLightmass.pdb")
	};

	// Set up the description for the Job
	const TCHAR* DescriptionKeys[] =
	{
		TEXT("MapName"),
		TEXT("GameName"),
		TEXT("QualityLevel")
	};

	// Get the map name
	FString MapNameStr = System.GetWorld()->GetMapName();
	const TCHAR* MapName = MapNameStr.GetCharArray().GetData();
	// Get the game name
	const TCHAR* GameName = FApp::GetProjectName();
	// Get the quality level
	TCHAR QualityLevel[MAX_SPRINTF] = TEXT("");
	FCString::Sprintf( QualityLevel, TEXT("%d"), ( int32 )Exporter->QualityLevel );

	const TCHAR* DescriptionValues[] =
	{
		MapName,
		GameName,
		QualityLevel
	};

	// Create the job - one task per mapping.
	bProcessingSuccessful = false;
	bProcessingFailed = false;
	bQuitReceived = false;
	NumCompletedTasks = 0;
	bRunningLightmass = false;
	
	Statistics.SwarmJobOpenTime += FPlatformTime::Seconds() - SwarmJobStartTime;
	
	UE_LOGF(LogLightmassSolver, Log,  "Swarm launching: %ls %ls", bUse64bitProcess ? *LightmassExecutable64 : *LightmassExecutable32, *Exporter->SceneGuid.ToString() );

	SwarmJobStartTime = FPlatformTime::Seconds();

	// If using Debug Mode (off by default), we use a hard-coded job GUID and Lightmass must be executed manually
	// (e.g. through a debugger), using the -debug command line parameter.
	// Lightmass will read all the cached input files and process the whole job locally without notifying
	// Swarm or Unreal that the job is completed. This also means that Lightmass can be executed as many times
	// as required (the input files will still be there in the Swarm cache) and Unreal doesn't need to be
	// running concurrently.
	int32 JobFlags = NSwarm::JOB_FLAG_USE_DEFAULTS;
	if (GLightmassDebugOptions.bDebugMode)
	{
		UE_LOGF(LogLightmassSolver, Log,  "Waiting for UnrealLightmass.exe to be launched manually..." );
		UE_LOGF(LogLightmassSolver, Log,  "Note: This Job will not be distributed" );
		JobFlags |= NSwarm::JOB_FLAG_MANUAL_START;
	}
	else
	{
		// Enable Swarm Job distribution, if requested
		if (GSwarmDebugOptions.bDistributionEnabled)
		{
			UE_LOGF(LogLightmassSolver, Log,  "Swarm will be allowed to distribute this job" );
			JobFlags |= NSwarm::JOB_FLAG_ALLOW_REMOTE;
		}
		else
		{
			UE_LOGF(LogLightmassSolver, Log,  "Swarm will be not be allowed to distribute this job; it will run locally only" );
		}
	}

	// Check to see if swarm should be run minimized (it should by default)
	bool bMinimizeSwarm = true;
	GConfig->GetBool(TEXT("LightingBuildOptions"), TEXT("MinimizeSwarm"), bMinimizeSwarm, GEditorSettingsIni);
	if ( bMinimizeSwarm )
	{
		UE_LOGF(LogLightmassSolver, Log,  "Swarm will be run minimized" );
		JobFlags |= NSwarm::JOB_FLAG_MINIMIZED;
	}

	FString CommandLineParameters = *Exporter->SceneGuid.ToString();
	if (GLightmassDebugOptions.bStatsEnabled)
	{
		CommandLineParameters += TEXT(" -stats");
	}

	NSwarm::FJobSpecification JobSpecification32, JobSpecification64;
	if ( !bUse64bitProcess )
	{
		JobSpecification32 = NSwarm::FJobSpecification( *LightmassExecutable32, *CommandLineParameters, ( NSwarm::TJobTaskFlags )JobFlags );
		JobSpecification32.AddDependencies( RequiredDependencyPaths32.GetArray(), RequiredDependencyPaths32.Num(), OptionalDependencyPaths32.GetArray(), OptionalDependencyPaths32.Num() );
		JobSpecification32.AddDescription( DescriptionKeys, DescriptionValues, UE_ARRAY_COUNT(DescriptionKeys) );
	}
	else
	{
		JobSpecification64 = NSwarm::FJobSpecification( *LightmassExecutable64, *CommandLineParameters, ( NSwarm::TJobTaskFlags )JobFlags );
		JobSpecification64.AddDependencies( RequiredDependencyPaths64.GetArray(), RequiredDependencyPaths64.Num(), OptionalDependencyPaths64.GetArray(), OptionalDependencyPaths64.Num() );
		JobSpecification64.AddDescription( DescriptionKeys, DescriptionValues, UE_ARRAY_COUNT(DescriptionKeys) );
	}
	int32 ErrorCode = Swarm.BeginJobSpecification( JobSpecification32, JobSpecification64 );
	if( ErrorCode < 0 )
	{
		UE_LOGF(LogLightmassSolver, Log,  "Error, BeginJobSpecification failed with error code %d", ErrorCode );
		bProcessingFailed = true;
	}

	// Count the number of tasks given to Swarm
	NumTotalSwarmTasks = 0;

	if (System.GetWorld()->GetWorldSettings()->bPrecomputeVisibility)
	{
		for (int32 TaskIndex = 0; TaskIndex < Exporter->VisibilityBucketGuids.Num(); TaskIndex++)
		{
			NSwarm::FTaskSpecification NewTaskSpecification( Exporter->VisibilityBucketGuids[TaskIndex], TEXT("PrecomputedVisibility"), NSwarm::JOB_TASK_FLAG_USE_DEFAULTS );
			//@todo - accurately estimate cost
			NewTaskSpecification.Cost = 10000;
			ErrorCode = Swarm.AddTask( NewTaskSpecification );
			if( ErrorCode >= 0 )
			{
				NumTotalSwarmTasks++;
			}
			else
			{
				UE_LOGF(LogLightmassSolver, Log,  "Error, AddTask failed with error code %d", ErrorCode );
			}
		}
	}

	if (!bOnlyBuildVisibility)
	{
		const EVolumeLightingMethod VolumeLightingMethod = System.GetWorld()->GetWorldSettings()->LightmassSettings.VolumeLightingMethod;

		if (VolumeLightingMethod == VLM_VolumetricLightmap)
		{
			for (TMap<FGuid, int32>::TIterator It(Exporter->VolumetricLightmapTaskGuids); It; ++It )
			{
				NSwarm::FTaskSpecification NewTaskSpecification(It.Key(), TEXT("VolumetricLightmap"), NSwarm::JOB_TASK_FLAG_USE_DEFAULTS );
				//@todo - accurately estimate cost
				NewTaskSpecification.Cost = 10000;
				ErrorCode = Swarm.AddTask( NewTaskSpecification );

				UE_LOGF(LogLightmassSolver, Verbose, "AddTask VLM %ls", *It.Key().ToString() );

				if( ErrorCode >= 0 )
				{
					NumTotalSwarmTasks++;
				}
				else
				{
					UE_LOGF(LogLightmassSolver, Warning,  "Error, AddTask failed with error code %d", ErrorCode );
					bProcessingFailed = true;
				}
			}
		}
		else
		{
			check(VolumeLightingMethod == VLM_SparseVolumeLightingSamples);
			NSwarm::FTaskSpecification NewTaskSpecification( Lightmass::PrecomputedVolumeLightingGuid, TEXT("VolumeSamples"), NSwarm::JOB_TASK_FLAG_USE_DEFAULTS );
			//@todo - accurately estimate cost
			// Changed estimated cost: this should be the maximum cost, because it became really big if there are WORLD_MAX size light-mapping
			NewTaskSpecification.Cost = INT_MAX;
			ErrorCode = Swarm.AddTask( NewTaskSpecification );
			if( ErrorCode >= 0 )
			{
				NumTotalSwarmTasks++;
			}
			else
			{
				UE_LOGF(LogLightmassSolver, Warning,  "Error, AddTask failed with error code %d", ErrorCode );
				bProcessingFailed = true;
			}
		}

		{
			NSwarm::FTaskSpecification NewTaskSpecification( Lightmass::MeshAreaLightDataGuid, TEXT("MeshAreaLightData"), NSwarm::JOB_TASK_FLAG_USE_DEFAULTS );
			NewTaskSpecification.Cost = 1000;
			ErrorCode = Swarm.AddTask( NewTaskSpecification );
			UE_LOGF(LogLightmassSolver, Verbose, "AddTask MeshAreasLights %ls", *Lightmass::MeshAreaLightDataGuid.ToString());

			if( ErrorCode >= 0 )
			{
				NumTotalSwarmTasks++;
			}
			else
			{
				UE_LOGF(LogLightmassSolver, Log,  "Error, AddTask failed with error code %d", ErrorCode );
				bProcessingFailed = true;
			}
		}

		{
			for (int32 LightIndex = 0; LightIndex < Exporter->DirectionalLights.Num(); LightIndex++)
			{
				const ULightComponent* Light = Exporter->DirectionalLights[LightIndex];
				IssueStaticShadowDepthMapTask(Light, INT_MAX);
			}
			
			for (int32 LightIndex = 0; LightIndex < Exporter->SpotLights.Num(); LightIndex++)
			{
				const ULightComponent* Light = Exporter->SpotLights[LightIndex];
				IssueStaticShadowDepthMapTask(Light, 10000);
			}

			for (int32 LightIndex = 0; LightIndex < Exporter->PointLights.Num(); LightIndex++)
			{
				const ULightComponent* Light = Exporter->PointLights[LightIndex];
				IssueStaticShadowDepthMapTask(Light, 10000);
			}

			for (int32 LightIndex = 0; LightIndex < Exporter->RectLights.Num(); LightIndex++)
			{
				const ULightComponent* Light = Exporter->RectLights[LightIndex];
				IssueStaticShadowDepthMapTask(Light, 10000);
			}
		}

		// Add BSP mapping tasks.
		for( int32 MappingIdx=0; (ErrorCode >= 0) && MappingIdx < Exporter->BSPSurfaceMappings.Num() && !GEditor->GetMapBuildCancelled(); MappingIdx++ )
		{
			FBSPSurfaceStaticLighting* BSPMapping = Exporter->BSPSurfaceMappings[MappingIdx];
			if (BSPMapping->bProcessMapping == true)
			{
				PendingBSPMappings.Add(BSPMapping->GetLightingGuid(), BSPMapping);

				NSwarm::FTaskSpecification NewTaskSpecification( BSPMapping->GetLightingGuid(), TEXT("BSPMapping"), NSwarm::JOB_TASK_FLAG_USE_DEFAULTS );
				NewTaskSpecification.Cost = BSPMapping->GetTexelCount();
				ErrorCode = Swarm.AddTask( NewTaskSpecification );
				if( ErrorCode >= 0 )
				{
					NumTotalSwarmTasks++;
				}
				else
				{
					UE_LOGF(LogLightmassSolver, Log,  "Error, AddTask failed with error code %d", ErrorCode );
				}
			}
		}

		// Add static mesh texture mappings tasks.
		for( int32 MappingIdx=0; (ErrorCode >= 0) && MappingIdx < Exporter->StaticMeshTextureMappings.Num() && !GEditor->GetMapBuildCancelled(); MappingIdx++ )
		{
			FStaticMeshStaticLightingTextureMapping* SMTextureMapping = Exporter->StaticMeshTextureMappings[MappingIdx];
			if (SMTextureMapping->bProcessMapping == true)
			{
				PendingTextureMappings.Add(SMTextureMapping->GetLightingGuid(), SMTextureMapping);

				NSwarm::FTaskSpecification NewTaskSpecification( SMTextureMapping->GetLightingGuid(), TEXT("SMTextureMapping"), NSwarm::JOB_TASK_FLAG_USE_DEFAULTS );
				NewTaskSpecification.Cost = SMTextureMapping->GetTexelCount();
				ErrorCode = Swarm.AddTask( NewTaskSpecification );

				UE_LOGF(LogLightmassSolver, Verbose, "AddTask Mapping %ls", *SMTextureMapping->GetLightingGuid().ToString());

				if( ErrorCode >= 0 )
				{
					NumTotalSwarmTasks++;
				}
				else
				{
					UE_LOGF(LogLightmassSolver, Log,  "Error, AddTask failed with error code %d", ErrorCode );
				}
			}
		}

		// Add Landscape mapping tasks.
		for( int32 MappingIdx=0; (ErrorCode >= 0) && MappingIdx < Exporter->LandscapeTextureMappings.Num() && !GEditor->GetMapBuildCancelled(); MappingIdx++ )
		{
			FLandscapeStaticLightingTextureMapping* LandscapeMapping = Exporter->LandscapeTextureMappings[MappingIdx];
			if (LandscapeMapping->bProcessMapping == true)
			{
				PendingLandscapeMappings.Add(LandscapeMapping->GetLightingGuid(), LandscapeMapping);

				NSwarm::FTaskSpecification NewTaskSpecification( LandscapeMapping->GetLightingGuid(), TEXT("LandscapeMapping"), NSwarm::JOB_TASK_FLAG_USE_DEFAULTS );
				NewTaskSpecification.Cost = LandscapeMapping->GetTexelCount();
				ErrorCode = Swarm.AddTask( NewTaskSpecification );

				UE_LOGF(LogLightmassSolver, Verbose, "AddTask Landscape Mapping %ls", *LandscapeMapping->GetLightingGuid().ToString());

				if( ErrorCode >= 0 )
				{
					NumTotalSwarmTasks++;
				}
				else
				{
					UE_LOGF(LogLightmassSolver, Log,  "Error, AddTask failed with error code %d", ErrorCode );
				}
			}
		}
	}

	int32 EndJobErrorCode = Swarm.EndJobSpecification();
	if( EndJobErrorCode < 0 )
	{
		UE_LOGF(LogLightmassSolver, Log,  "Error, EndJobSpecification failed with error code %d", ErrorCode );
	}

	if ( ErrorCode < 0 || EndJobErrorCode < 0 )
	{
		bProcessingFailed = true;
	}

	int32 NumTotalTasks = NumTotalSwarmTasks;
	
	// In deterministic mode, we import and process the mappings after Lightmass is done, so we have twice the steps.
	NumTotalTasks *= 2;

	GWarn->StatusUpdate( NumCompletedTasks, NumTotalTasks, LOCTEXT("BeginRunningLightmassTask", "Running Lightmass...") );

	Statistics.SwarmJobOpenTime += FPlatformTime::Seconds() - SwarmJobStartTime;
	
	LightmassStartTime = FPlatformTime::Seconds();

#if USE_LOCAL_SWARM_INTERFACE
	// @todo Mac: add proper progress reporting from UnrealLightmass
	bProcessingSuccessful = true;
#endif

	return !bProcessingFailed;
}

int32 FLightmassProcessor::GetAsyncPercentDone() const
{
	return NumCompletedTasks * 100 / NumTotalSwarmTasks;
}

float FLightmassProcessor::GetAmortizedExportPercentDone() const
{
	return Exporter->GetAmortizedExportPercentDone();
}

bool FLightmassProcessor::Update()
{
	bool bIsFinished = false;
	if ( !bQuitReceived && !bProcessingFailed && !GEditor->GetMapBuildCancelled() )
	{
		bool bAllTaskAreComplete = (FPlatformAtomics::AtomicRead(&NumCompletedTasks) == NumTotalSwarmTasks ? true : false);

#if USE_LOCAL_SWARM_INTERFACE
		if (IsRunningCommandlet())
		{
			FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
		}
#endif

		GLog->Flush();

		bIsFinished = bAllTaskAreComplete && bProcessingSuccessful;

		if (bIsFinished)
		{
			Statistics.LightmassTime += FPlatformTime::Seconds() - LightmassStartTime;
		}
	}
	else
	{
		bIsFinished = true;
	}

	ProcessAlertMessages();

#if USE_LOCAL_SWARM_INTERFACE
	int32 Status = 0;
	const bool bIsLightmassRunning = Swarm.IsJobProcessRunning(&Status);
	if (!bIsLightmassRunning)
	{
		bIsFinished = true;
		bProcessingFailed = Status != 0;
		bProcessingSuccessful = !bProcessingFailed;
	}
#endif

	return bIsFinished;
}

void FLightmassProcessor::ImportDeferredMappings()
{
	TArray<FString> Files;
	IFileManager::Get().FindFiles(Files, *DeferredMappingsDirectory, TEXT(".lm"));

	DeferredMappings = new FDeferredMappingsBundle;
	
	// load the mappings
	for(const FString& FileName : Files)
	{
		TArray<uint8> FileData;
		int32 BaseMappingIndex  = DeferredMappings->Mappings.Num();
		FString FullFileName = FString::Printf(TEXT("%s\\%s"), *DeferredMappingsDirectory, *FileName);

		if (FFileHelper::LoadFileToArray(FileData, *FullFileName))
		{
			FObjectMemoryReader Reader(FileData);
		
			FDeferredMappingsBundle Bundle;
			Bundle.Serialize(Reader, false);

			for (FLightmassProcessor::FTextureMappingImportHelper& Mapping : Bundle.Mappings)
			{
				// Those mapping guid are just numbered 0 to n, so we rebase them to be unique in the combined mappings
				Mapping.MappingGuid.D += BaseMappingIndex;
			}

			// transfer all bundle mappings
			DeferredMappings->Mappings.Append(Bundle.Mappings);
		}
		else
		{
			UE_LOGF(LogLightmassSolver, Warning, "Error, couldn't import deferred mapping %ls", *FullFileName);
			bProcessingFailed = true;
		}
	}

	DeferredMappings->MapHelpers();

	for (FLightmassProcessor::FTextureMappingImportHelper& Mapping : DeferredMappings->Mappings)
	{
		ImportedMappings.Add(Mapping.MappingGuid, &Mapping);
	}	
}

void FLightmassProcessor::ClearImportedDeferredMappings()
{
	if (DeferredMappings)
	{
		for (FLightmassProcessor::FTextureMappingImportHelper& Mapping : DeferredMappings->Mappings)
		{
			ImportedMappings.Remove(Mapping.MappingGuid);
		}
	}
}


void FLightmassProcessor::ExportDeferredMappings()
{
	if (DeferredMappings && !System.Options.bApplyDeferedActorMappingPass)
	{		
		FGuid BundleGuid = FGuid::NewGuid();

		FString FileName = FString::Printf(TEXT("Mappings_%s.lm"), *BundleGuid.ToString());

		TArray<uint8> FileData;
		FObjectMemoryWriter Writer(FileData);

		DeferredMappings->Serialize(Writer, false);

		FString FullFileName = FString::Printf(TEXT("%s\\%s"), *DeferredMappingsDirectory, *FileName);

		if (!FFileHelper::SaveArrayToFile(FileData, *FullFileName))
		{
			UE_LOGF(LogLightmassSolver, Warning, "Error, couldn't export deferred mapping %ls", *FullFileName);
			bProcessingFailed = true;
		}
	}
}

bool FLightmassProcessor::IsDeferredMapping(const FGuid& Guid)
{
	if (const FStaticLightingMapping* Mapping = Exporter->FindMappingByGuid(Guid))
	{
		return Mapping->IsDeferred();
	}
	return false;
}

void FLightmassProcessor::DeferMapping(FTextureMappingImportHelper* ImportHelper)
{
	if (!DeferredMappings)
	{
		DeferredMappings = new FDeferredMappingsBundle();
	}

 	DeferredMappings->Mappings.Add(*ImportHelper);
	DeferredMappings->OwnerToMapping.Add(ImportHelper->OwnerGuid, ImportHelper->MappingGuid);
}

bool FLightmassProcessor::CompleteRun()
{
	bRunningLightmass = false;
	
	double ImportStartTime = FPlatformTime::Seconds();
	double OriginalApplyTime = Statistics.ApplyTimeInProcessing;

	if ( !bProcessingFailed && !GEditor->GetMapBuildCancelled() )
	{
		ImportVolumeSamples();

		if (Exporter->VolumetricLightmapTaskGuids.Num() > 0)
		{
			ImportVolumetricLightmap();
		}
		
		ImportPrecomputedVisibility();
		ImportMeshAreaLightData();
		ImportVolumeDistanceFieldData();

		if (bImportCompletedMappingsImmediately)
		{
			if (System.Options.bApplyDeferedActorMappingPass)
			{
				ImportDeferredMappings();
			}

			// Import any outstanding completed mappings.
			ImportMappings(false);

			{
				// Detach all components
				// This must be done globally because different mappings will
				FGlobalComponentRecreateRenderStateContext ReregisterContext;

				// Block until the RT processes the unregister before modifying variables that it may need to access
				FlushRenderingCommands();

				ProcessAvailableMappings();
			}

			// Export mapping which we'll not process on this run (this will remove then from available mappings)
			ExportDeferredMappings();
		}

		ApplyPrecomputedVisibility();
		ProcessAlertMessages();
	}
	CompletedMappingTasks.Clear();
	CompletedVisibilityTasks.Clear();
	CompletedVolumetricLightmapTasks.Clear();

	ClearImportedDeferredMappings();

	double ApplyTimeDelta = Statistics.ApplyTimeInProcessing - OriginalApplyTime;
	Statistics.ImportTimeInProcessing += FPlatformTime::Seconds() - ImportStartTime - ApplyTimeDelta;

	return bProcessingSuccessful;
}

bool FLightmassProcessor::IsProcessingCompletedSuccessfully() const
{
	return bProcessingSuccessful &&
		!bQuitReceived && !bProcessingFailed && !GEditor->GetMapBuildCancelled();
}

/**
 * Import all mappings that have been completed so far.
 *	@param	bProcessImmediately		If true, immediately process the mapping
 *									If false, store it off for later processing
 */
void FLightmassProcessor::ImportMappings(bool bProcessImmediately)
{
	// This will return a list of all the completed Guids
	TList<FGuid>* Element = CompletedMappingTasks.ExtractAll();

	// Reverse the list, so we have the mappings in the same order that they came in
	{
		TList<FGuid>* PrevElement = NULL;
		TList<FGuid>* NextElement = NULL;
		while (Element)
		{
			NextElement = Element->Next;
			Element->Next = PrevElement;
			PrevElement = Element;
			Element = NextElement;
		}
		// Assign the new head of the list
		Element = PrevElement;
	}
	
	while ( Element )
	{
		TList<FGuid>* NextElement = Element->Next;
		ImportMapping( Element->Element, bProcessImmediately );
		delete Element;
		Element = NextElement;
	}
}

static_assert(LM_NUM_SH_COEFFICIENTS == NUM_INDIRECT_LIGHTING_SH_COEFFICIENTS, "Lightmass SH generation must match engine SH expectations.");

/** Imports volume lighting samples from Lightmass and adds them to the appropriate levels. */
void FLightmassProcessor::ImportVolumeSamples()
{
	if (VolumeSampleTaskCompleted > 0)
	{
		{
			static_assert(sizeof(FDebugVolumeLightingSample) == sizeof(Lightmass::FDebugVolumeLightingSample), "Debug type sizes must match.");
			const FString ChannelName = Lightmass::CreateChannelName(Lightmass::VolumeLightingDebugOutputGuid, Lightmass::LM_VOLUMEDEBUGOUTPUT_VERSION, Lightmass::LM_VOLUMEDEBUGOUTPUT_EXTENSION);
			const int32 Channel = Swarm.OpenChannel( *ChannelName, LM_VOLUMEDEBUGOUTPUT_CHANNEL_FLAGS );
			if (Channel >= 0)
			{
				ReadArray(Channel, GDebugStaticLightingInfo.VolumeLightingSamples);
				Swarm.CloseChannel(Channel);
			}
		}

		const FString ChannelName = Lightmass::CreateChannelName(Lightmass::PrecomputedVolumeLightingGuid, Lightmass::LM_VOLUMESAMPLES_VERSION, Lightmass::LM_VOLUMESAMPLES_EXTENSION);
		const int32 Channel = Swarm.OpenChannel( *ChannelName, LM_VOLUMESAMPLES_CHANNEL_FLAGS );
		if (Channel >= 0)
		{
			FVector4f UnusedVolumeCenter;
			Swarm.ReadChannel(Channel, &UnusedVolumeCenter, sizeof(UnusedVolumeCenter));
			FVector4f UnusedVolumeExtent;
			Swarm.ReadChannel(Channel, &UnusedVolumeExtent, sizeof(UnusedVolumeExtent));

			int32 NumStreamLevels = System.GetWorld()->GetStreamingLevels().Num();
			int32 NumVolumeSampleArrays;
			Swarm.ReadChannel(Channel, &NumVolumeSampleArrays, sizeof(NumVolumeSampleArrays));
			for (int32 ArrayIndex = 0; ArrayIndex < NumVolumeSampleArrays; ArrayIndex++)
			{
				FGuid LevelGuid;
				Swarm.ReadChannel(Channel, &LevelGuid, sizeof(LevelGuid));
				TArray<Lightmass::FVolumeLightingSampleData> VolumeSamples;
				ReadArray(Channel, VolumeSamples);
				ULevel* CurrentLevel = System.LightingContext.GetLevelForGuid(LevelGuid).Get();

				// Only build precomputed light for visible streamed levels
				if (CurrentLevel && CurrentLevel->bIsVisible)
				{
					UMapBuildDataRegistry* CurrentRegistry = System.LightingContext.GetOrCreateRegistryForLevel(CurrentLevel);
					FPrecomputedLightVolumeData& CurrentLevelData = CurrentRegistry->AllocateLevelPrecomputedLightVolumeBuildData(CurrentLevel->LevelBuildDataId);

					FBox3f LevelVolumeBounds(ForceInit);

					for (int32 SampleIndex = 0; SampleIndex < VolumeSamples.Num(); SampleIndex++)
					{
						const Lightmass::FVolumeLightingSampleData& CurrentSample = VolumeSamples[SampleIndex];
						FVector4f SampleMin = CurrentSample.PositionAndRadius - FVector3f(CurrentSample.PositionAndRadius.W);
						FVector4f SampleMax = CurrentSample.PositionAndRadius + FVector3f(CurrentSample.PositionAndRadius.W);
						LevelVolumeBounds += FBox3f(SampleMin, SampleMax);
					}

					CurrentLevelData.Initialize(FBox(LevelVolumeBounds));

					for (int32 SampleIndex = 0; SampleIndex < VolumeSamples.Num(); SampleIndex++)
					{
						const Lightmass::FVolumeLightingSampleData& CurrentSample = VolumeSamples[SampleIndex];
						FVolumeLightingSample NewHighQualitySample;
						NewHighQualitySample.Position = FVector4f(CurrentSample.PositionAndRadius);
						NewHighQualitySample.Radius = CurrentSample.PositionAndRadius.W;
						NewHighQualitySample.SetPackedSkyBentNormal(FVector3d(CurrentSample.SkyBentNormal)); 
						NewHighQualitySample.DirectionalLightShadowing = CurrentSample.DirectionalLightShadowing;

						for (int32 CoefficientIndex = 0; CoefficientIndex < NUM_INDIRECT_LIGHTING_SH_COEFFICIENTS; CoefficientIndex++)
						{
							NewHighQualitySample.Lighting.R.V[CoefficientIndex] = CurrentSample.HighQualityCoefficients[CoefficientIndex][0];
							NewHighQualitySample.Lighting.G.V[CoefficientIndex] = CurrentSample.HighQualityCoefficients[CoefficientIndex][1];
							NewHighQualitySample.Lighting.B.V[CoefficientIndex] = CurrentSample.HighQualityCoefficients[CoefficientIndex][2];
						}							

						FVolumeLightingSample NewLowQualitySample;
						NewLowQualitySample.Position = FVector4f(CurrentSample.PositionAndRadius);
						NewLowQualitySample.Radius = CurrentSample.PositionAndRadius.W;
						NewLowQualitySample.DirectionalLightShadowing = CurrentSample.DirectionalLightShadowing;
						NewLowQualitySample.SetPackedSkyBentNormal(FVector3d(CurrentSample.SkyBentNormal)); 

						for (int32 CoefficientIndex = 0; CoefficientIndex < NUM_INDIRECT_LIGHTING_SH_COEFFICIENTS; CoefficientIndex++)
						{
							NewLowQualitySample.Lighting.R.V[CoefficientIndex] = CurrentSample.LowQualityCoefficients[CoefficientIndex][0];
							NewLowQualitySample.Lighting.G.V[CoefficientIndex] = CurrentSample.LowQualityCoefficients[CoefficientIndex][1];
							NewLowQualitySample.Lighting.B.V[CoefficientIndex] = CurrentSample.LowQualityCoefficients[CoefficientIndex][2];
						}							

						CurrentLevelData.AddHighQualityLightingSample(NewHighQualitySample);
						CurrentLevelData.AddLowQualityLightingSample(NewLowQualitySample);
					}

					CurrentLevelData.FinalizeSamples();
				}
			}

			Swarm.CloseChannel(Channel);
		}
		else
		{
			UE_LOGF(LogLightmassSolver, Log,  "Error, OpenChannel failed to open %ls with error code %d", *ChannelName, Channel );
		}
		FPlatformAtomics::InterlockedExchange(&VolumeSampleTaskCompleted, 0);
	}
}

/** Imports precomputed visibility */
void FLightmassProcessor::ImportPrecomputedVisibility()
{
	TList<FGuid>* Element = CompletedVisibilityTasks.ExtractAll();

	// Reverse the list, so we have the tasks in the same order that they came in
	{
		TList<FGuid>* PrevElement = NULL;
		TList<FGuid>* NextElement = NULL;
		while (Element)
		{
			NextElement = Element->Next;
			Element->Next = PrevElement;
			PrevElement = Element;
			Element = NextElement;
		}
		// Assign the new head of the list
		Element = PrevElement;
	}
	
	while ( Element )
	{
		// If this task has not already been imported, import it now
		TList<FGuid>* NextElement = Element->Next;

		const FString ChannelName = Lightmass::CreateChannelName(Element->Element, Lightmass::LM_PRECOMPUTEDVISIBILITY_VERSION, Lightmass::LM_PRECOMPUTEDVISIBILITY_EXTENSION);
		const int32 Channel = Swarm.OpenChannel( *ChannelName, LM_PRECOMPUTEDVISIBILITY_CHANNEL_FLAGS );
		if (Channel >= 0)
		{
			// Find the index of this visibility task in VisibilityBucketGuids
			const int32 ArrayIndex = Exporter->VisibilityBucketGuids.Find(Element->Element);
			check(ArrayIndex >= 0);

			if (CompletedPrecomputedVisibilityCells.Num() == 0)
			{
				CompletedPrecomputedVisibilityCells.AddZeroed(Exporter->VisibilityBucketGuids.Num());
			}
			
			int32 NumCells = 0;
			Swarm.ReadChannel(Channel, &NumCells, sizeof(NumCells));

			for (int32 CellIndex = 0; CellIndex < NumCells; CellIndex++)
			{
				FBox3f Bounds;
				Swarm.ReadChannel(Channel, &Bounds, sizeof(Bounds));

				// Use the same index for this task guid as it has in VisibilityBucketGuids, so that visibility cells are processed in a deterministic order
				CompletedPrecomputedVisibilityCells[ArrayIndex].AddZeroed();
				FUncompressedPrecomputedVisibilityCell& CurrentCell = CompletedPrecomputedVisibilityCells[ArrayIndex].Last();
				CurrentCell.Bounds = FBox(Bounds);
				ReadArray(Channel, CurrentCell.VisibilityData);
			}

			TArray<FDebugStaticLightingRay> DebugRays;
			ReadArray(Channel, DebugRays);
			GDebugStaticLightingInfo.PrecomputedVisibilityRays.Append(DebugRays);

			Swarm.CloseChannel(Channel);
		}
		else
		{
			UE_LOGF(LogLightmassSolver, Log,  "Error, OpenChannel failed to open %ls with error code %d", *ChannelName, Channel );
		}

		delete Element;
		Element = NextElement;
	}
}

static bool IsMeshVisible(const TArray<uint8>& VisibilityData, int32 MeshId)
{
	return (VisibilityData[MeshId / 8] & 1 << (MeshId % 8)) != 0;
}

static int32 AccumulateVisibility(const TArray<uint8>& OtherCellData, TArray<uint8>& CellData)
{
	int32 NumAdded = 0;

	checkSlow(OtherCellData.Num() == CellData.Num());
	for (int32 i = 0; i < OtherCellData.Num(); i++)
	{
		if (OtherCellData[i] != 0)
		{
			for (int32 BitIndex = 0; BitIndex < 8; BitIndex++)
			{
				NumAdded += IsMeshVisible(OtherCellData, i * 8 + BitIndex) && !IsMeshVisible(CellData, i * 8 + BitIndex) ? 1 : 0;
			}
		}

		CellData[i] |= OtherCellData[i];
	}
	return NumAdded;
}

struct FPrecomputedVisibilitySortGridCell
{
	TArray<FUncompressedPrecomputedVisibilityCell, TInlineAllocator<2>> Cells;
};

void SpreadVisibilityCell(
	float CellSize, 
	float PlayAreaHeight,
	const FUncompressedPrecomputedVisibilityCell& OtherCell,
	FUncompressedPrecomputedVisibilityCell& VisibilityCell,
	int32& QueriesVisibleFromSpreadingNeighbors)
{
	// Determine whether the cell is a world space neighbor
	if (!(OtherCell.Bounds.Min == VisibilityCell.Bounds.Min && OtherCell.Bounds.Max == VisibilityCell.Bounds.Max)
		&& FMath::Abs(VisibilityCell.Bounds.Min.X - OtherCell.Bounds.Min.X) < CellSize + KINDA_SMALL_NUMBER
		&& FMath::Abs(VisibilityCell.Bounds.Min.Y - OtherCell.Bounds.Min.Y) < CellSize + KINDA_SMALL_NUMBER
		// Don't spread from cells below, they're probably below the ground and see too much
		&& OtherCell.Bounds.Min.Z - VisibilityCell.Bounds.Min.Z > -PlayAreaHeight * 0.5f
		// Only spread from one cell above
		&& OtherCell.Bounds.Min.Z - VisibilityCell.Bounds.Min.Z < PlayAreaHeight * 1.5f)
	{
		// Combine the neighbor's visibility with the current cell's visibility
		// This reduces visibility errors at the cost of less effective culling
		QueriesVisibleFromSpreadingNeighbors += AccumulateVisibility(OtherCell.VisibilityData, VisibilityCell.VisibilityData);
	}
}

void FLightmassProcessor::ApplyPrecomputedVisibility()
{
	TArray<FUncompressedPrecomputedVisibilityCell> CombinedPrecomputedVisibilityCells;
	for (int32 ArrayIndex = 0; ArrayIndex < CompletedPrecomputedVisibilityCells.Num(); ArrayIndex++)
	{
		CombinedPrecomputedVisibilityCells.Append(CompletedPrecomputedVisibilityCells[ArrayIndex]);
	}
	CompletedPrecomputedVisibilityCells.Empty();

	if (CombinedPrecomputedVisibilityCells.Num() > 0)
	{
		const double StartTime = FPlatformTime::Seconds();
		int32 VisibilitySpreadingIterations;

		const TCHAR* AggressivenessSectionNames[VIS_Max] = {
			TEXT("DevOptions.PrecomputedVisibility"), 
			TEXT("DevOptions.PrecomputedVisibilityModeratelyAggressive"), 
			TEXT("DevOptions.PrecomputedVisibilityMostAggressive")};
		const TCHAR* ActiveSection = AggressivenessSectionNames[FMath::Clamp((int32)System.GetWorld()->GetWorldSettings()->VisibilityAggressiveness, 0, VIS_Max - 1)];
		VERIFYLIGHTMASSINI(GConfig->GetInt(ActiveSection, TEXT("VisibilitySpreadingIterations"), VisibilitySpreadingIterations, GLightmassIni));
		bool bCompressVisibilityData;
		VERIFYLIGHTMASSINI(GConfig->GetBool(TEXT("DevOptions.PrecomputedVisibility"), TEXT("bCompressVisibilityData"), bCompressVisibilityData, GLightmassIni));
		const float CellSize = System.GetWorld()->GetWorldSettings()->VisibilityCellSize;
		float PlayAreaHeight = 0;
		VERIFYLIGHTMASSINI(GConfig->GetFloat(TEXT("DevOptions.PrecomputedVisibility"), TEXT("PlayAreaHeight"), PlayAreaHeight, GLightmassIni));
		int32 CellBucketSize = 0;
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.PrecomputedVisibility"), TEXT("CellRenderingBucketSize"), CellBucketSize, GLightmassIni));
		int32 NumCellBuckets = 0;
		VERIFYLIGHTMASSINI(GConfig->GetInt(TEXT("DevOptions.PrecomputedVisibility"), TEXT("NumCellRenderingBuckets"), NumCellBuckets, GLightmassIni));

		int32 TotalNumQueries = 0;
		int32 QueriesVisibleFromSpreadingNeighbors = 0;

		for (int32 IterationIndex = 0; IterationIndex < VisibilitySpreadingIterations; IterationIndex++)
		{
			FBox AllCellsBounds(ForceInit);

			for (int32 CellIndex = 0; CellIndex < CombinedPrecomputedVisibilityCells.Num(); CellIndex++)
			{
				AllCellsBounds += CombinedPrecomputedVisibilityCells[CellIndex].Bounds;
			}

			int32 GridSizeX = FMath::TruncToInt(AllCellsBounds.GetSize().X / CellSize + .5f);
			int32 GridSizeY = FMath::TruncToInt(AllCellsBounds.GetSize().Y / CellSize + .5f);

			const int32 GridSizeMax = 10000;

			if (GridSizeX < GridSizeMax && GridSizeY < GridSizeMax)
			{
				TArray<FPrecomputedVisibilitySortGridCell> SortGrid;
				SortGrid.Empty(GridSizeX * GridSizeY);
				SortGrid.AddZeroed(GridSizeX * GridSizeY);

				// Add visibility cells into a 2d grid
				// Note that visibility data is duplicated so that the next pass can read from original neighbor visibility data
				for (int32 CellIndex = 0; CellIndex < CombinedPrecomputedVisibilityCells.Num(); CellIndex++)
				{
					float CellXFloat = (CombinedPrecomputedVisibilityCells[CellIndex].Bounds.GetCenter().X - AllCellsBounds.Min.X) / CellSize;
					int32 CellX = FMath::Clamp(FMath::TruncToInt(CellXFloat), 0, GridSizeX - 1);

					float CellYFloat = (CombinedPrecomputedVisibilityCells[CellIndex].Bounds.GetCenter().Y - AllCellsBounds.Min.Y) / CellSize;
					int32 CellY = FMath::Clamp(FMath::TruncToInt(CellYFloat), 0, GridSizeY - 1);

					FPrecomputedVisibilitySortGridCell& GridCell = SortGrid[CellY * GridSizeX + CellX];
					GridCell.Cells.Add(CombinedPrecomputedVisibilityCells[CellIndex]);
				}

				// Gather visibility from neighbors, using the 2d grid to accelerate the neighbor search
				for (int32 CellIndex = 0; CellIndex < CombinedPrecomputedVisibilityCells.Num(); CellIndex++)
				{
					FUncompressedPrecomputedVisibilityCell& CurrentCell = CombinedPrecomputedVisibilityCells[CellIndex];

					float CellXFloat = (CombinedPrecomputedVisibilityCells[CellIndex].Bounds.GetCenter().X - AllCellsBounds.Min.X) / CellSize;
					int32 CellX = FMath::Clamp(FMath::TruncToInt(CellXFloat), 0, GridSizeX - 1);

					float CellYFloat = (CombinedPrecomputedVisibilityCells[CellIndex].Bounds.GetCenter().Y - AllCellsBounds.Min.Y) / CellSize;
					int32 CellY = FMath::Clamp(FMath::TruncToInt(CellYFloat), 0, GridSizeY - 1);

					TotalNumQueries += CurrentCell.VisibilityData.Num() * 8;

					const FPrecomputedVisibilitySortGridCell& GridCell = SortGrid[CellY * GridSizeX + CellX];

					for (int32 YOffset = -1; YOffset <= 1; YOffset++)
					{
						for (int32 XOffset = -1; XOffset <= 1; XOffset++)
						{
							int32 FinalCellX = CellX + XOffset;
							int32 FinalCellY = CellY + YOffset;

							if (FinalCellX >= 0 && FinalCellX < GridSizeX && FinalCellY >= 0 && FinalCellY < GridSizeY)
							{
								const TArray<FUncompressedPrecomputedVisibilityCell, TInlineAllocator<2>>& CurrentSortCell = SortGrid[FinalCellY * GridSizeX + FinalCellX].Cells;

								for (int32 VisibilityCellIndex = 0; VisibilityCellIndex < CurrentSortCell.Num(); VisibilityCellIndex++)
								{
									const FUncompressedPrecomputedVisibilityCell& OtherCell = CurrentSortCell[VisibilityCellIndex];
									SpreadVisibilityCell(CellSize, PlayAreaHeight, OtherCell, CurrentCell, QueriesVisibleFromSpreadingNeighbors);
								}
							}
						}
					}
				}
			}
			else
			{
				// Brute force O(N^2) neighbor spreading version
				// Copy the original data since we read from outside the current cell
				TArray<FUncompressedPrecomputedVisibilityCell> OriginalPrecomputedVisibilityCells(CombinedPrecomputedVisibilityCells);
				for (int32 CellIndex = 0; CellIndex < CombinedPrecomputedVisibilityCells.Num(); CellIndex++)
				{
					FUncompressedPrecomputedVisibilityCell& CurrentCell = CombinedPrecomputedVisibilityCells[CellIndex];
					TotalNumQueries += CurrentCell.VisibilityData.Num() * 8;
					for (int32 OtherCellIndex = 0; OtherCellIndex < OriginalPrecomputedVisibilityCells.Num(); OtherCellIndex++)
					{
						const FUncompressedPrecomputedVisibilityCell& OtherCell = OriginalPrecomputedVisibilityCells[OtherCellIndex];

						SpreadVisibilityCell(CellSize, PlayAreaHeight, OtherCell, CurrentCell, QueriesVisibleFromSpreadingNeighbors);
					}
				}
			}
		}	

		const FVector2D CellBucketOriginXY(CombinedPrecomputedVisibilityCells[0].Bounds.Min.X, CombinedPrecomputedVisibilityCells[0].Bounds.Min.Y);

		TArray<TArray<const FUncompressedPrecomputedVisibilityCell*> > CellRenderingBuckets;
		CellRenderingBuckets.Empty(NumCellBuckets * NumCellBuckets);
		CellRenderingBuckets.AddZeroed(NumCellBuckets * NumCellBuckets);
		SIZE_T UncompressedSize = 0;
		// Sort the cells into buckets based on their position
		//@todo - sort cells inside buckets based on locality, to reduce visibility cache misses
		for (int32 CellIndex = 0; CellIndex < CombinedPrecomputedVisibilityCells.Num(); CellIndex++)
		{
			const FUncompressedPrecomputedVisibilityCell& CurrentCell = CombinedPrecomputedVisibilityCells[CellIndex];

			const float FloatOffsetX = (CurrentCell.Bounds.Min.X - CellBucketOriginXY.X + .5f * CellSize) / CellSize;
			// FMath::TruncToInt rounds toward 0, we want to always round down
			const int32 BucketIndexX = FMath::Abs((FMath::TruncToInt(FloatOffsetX) - (FloatOffsetX < 0.0f ? 1 : 0)) / CellBucketSize % NumCellBuckets);
			const float FloatOffsetY = (CurrentCell.Bounds.Min.Y - CellBucketOriginXY.Y + .5f * CellSize) / CellSize;
			const int32 BucketIndexY = FMath::Abs((FMath::TruncToInt(FloatOffsetY) - (FloatOffsetY < 0.0f ? 1 : 0)) / CellBucketSize % NumCellBuckets);

			const int32 BucketIndex = BucketIndexY * CellBucketSize + BucketIndexX;
			CellRenderingBuckets[BucketIndex].Add(&CurrentCell);
			UncompressedSize += CurrentCell.VisibilityData.Num();
		}

		System.GetWorld()->PersistentLevel->MarkPackageDirty();

		// Set all the level parameters needed to access visibility
		System.GetWorld()->PersistentLevel->PrecomputedVisibilityHandler.PrecomputedVisibilityCellBucketOriginXY = CellBucketOriginXY;
		System.GetWorld()->PersistentLevel->PrecomputedVisibilityHandler.PrecomputedVisibilityCellSizeXY = CellSize;
		System.GetWorld()->PersistentLevel->PrecomputedVisibilityHandler.PrecomputedVisibilityCellSizeZ = PlayAreaHeight;
		System.GetWorld()->PersistentLevel->PrecomputedVisibilityHandler.PrecomputedVisibilityCellBucketSizeXY = CellBucketSize;
		System.GetWorld()->PersistentLevel->PrecomputedVisibilityHandler.PrecomputedVisibilityNumCellBuckets = NumCellBuckets;
		System.GetWorld()->PersistentLevel->PrecomputedVisibilityHandler.PrecomputedVisibilityCellBuckets.Empty(NumCellBuckets * NumCellBuckets);
		System.GetWorld()->PersistentLevel->PrecomputedVisibilityHandler.PrecomputedVisibilityCellBuckets.AddZeroed(NumCellBuckets * NumCellBuckets);

		// Split visibility data into ~32Kb chunks, to limit decompression time
		const int32 ChunkSizeTarget = 32 * 1024;
		TArray<uint8> UncompressedVisibilityData;
		SIZE_T TotalCompressedSize = 0;
		for (int32 BucketIndex = 0; BucketIndex < CellRenderingBuckets.Num(); BucketIndex++)
		{
			FPrecomputedVisibilityBucket& OutputBucket = System.GetWorld()->PersistentLevel->PrecomputedVisibilityHandler.PrecomputedVisibilityCellBuckets[BucketIndex];
			OutputBucket.CellDataSize = CombinedPrecomputedVisibilityCells[0].VisibilityData.Num();
			int32 ChunkIndex = 0;
			UncompressedVisibilityData.Reset();
			for (int32 CellIndex = 0; CellIndex < CellRenderingBuckets[BucketIndex].Num(); CellIndex++)
			{	
				const FUncompressedPrecomputedVisibilityCell& CurrentCell = *CellRenderingBuckets[BucketIndex][CellIndex];
				FPrecomputedVisibilityCell NewCell;
				NewCell.Min = CurrentCell.Bounds.Min;
				// We're only storing Min per cell with a shared SizeXY and SizeZ for reduced memory storage
				checkSlow(CurrentCell.Bounds.Max.Equals(CurrentCell.Bounds.Min + FVector(CellSize, CellSize, PlayAreaHeight), KINDA_SMALL_NUMBER * 10.0f));
				NewCell.ChunkIndex = ChunkIndex;
				NewCell.DataOffset = UncompressedVisibilityData.Num();
				OutputBucket.Cells.Add(NewCell);
				UncompressedVisibilityData.Append(CurrentCell.VisibilityData);
				// Create a new chunk if we've reached the size limit or this is the last cell in a bucket
				if (UncompressedVisibilityData.Num() > ChunkSizeTarget || CellIndex == CellRenderingBuckets[BucketIndex].Num() - 1)
				{
					// Don't compress small amounts of data because FCompression::CompressMemory will fail
					if (bCompressVisibilityData && UncompressedVisibilityData.Num() > 32)
					{
						TArray<uint8> TempCompressionOutput;
						// Compressed output can be larger than the input, so we use temporary storage to hold the compressed output for now
						TempCompressionOutput.Empty(UncompressedVisibilityData.Num() * 4 / 3);
						TempCompressionOutput.AddUninitialized(UncompressedVisibilityData.Num() * 4 / 3);
						int32 CompressedSize = TempCompressionOutput.Num();
						verify(FCompression::CompressMemory(
							// Using zlib since it is supported on all platforms, otherwise we would need to compress on cook
							NAME_Zlib, 
							TempCompressionOutput.GetData(), 
							CompressedSize, 
							UncompressedVisibilityData.GetData(), 
							UncompressedVisibilityData.Num(),
							COMPRESS_BiasMemory));

						OutputBucket.CellDataChunks.AddZeroed();
						FCompressedVisibilityChunk& NewChunk = OutputBucket.CellDataChunks.Last();
						NewChunk.UncompressedSize = UncompressedVisibilityData.Num();
						NewChunk.bCompressed = true;
						NewChunk.Data.Empty(CompressedSize);
						NewChunk.Data.AddUninitialized(CompressedSize);
						FMemory::Memcpy(NewChunk.Data.GetData(), TempCompressionOutput.GetData(), CompressedSize);
						ChunkIndex++;
						TotalCompressedSize += CompressedSize;
						UncompressedVisibilityData.Reset();
					}
					else
					{
						OutputBucket.CellDataChunks.AddZeroed();
						FCompressedVisibilityChunk& NewChunk = OutputBucket.CellDataChunks.Last();
						NewChunk.UncompressedSize = UncompressedVisibilityData.Num();
						NewChunk.bCompressed = false;
						NewChunk.Data = UncompressedVisibilityData;
						ChunkIndex++;
						TotalCompressedSize += UncompressedVisibilityData.Num();
						UncompressedVisibilityData.Reset();
					}
				}
			}
		}

		System.GetWorld()->PersistentLevel->PrecomputedVisibilityHandler.UpdateVisibilityStats(true);

		UE_LOGF(LogStaticLightingSystem, Log, "ApplyPrecomputedVisibility %.1fs with %u cells, %.1f%% of all queries changed to visible from spreading neighbors, compressed %.3fMb to %.3fMb (%.1f ratio)",
			FPlatformTime::Seconds() - StartTime,
			CombinedPrecomputedVisibilityCells.Num(),
			100.0f * QueriesVisibleFromSpreadingNeighbors / TotalNumQueries,
			UncompressedSize / 1024.0f / 1024.0f,
			TotalCompressedSize / 1024.0f / 1024.0f,
			UncompressedSize / (float)TotalCompressedSize);
	}
	else
	{
		System.GetWorld()->PersistentLevel->PrecomputedVisibilityHandler.Invalidate(System.GetWorld()->Scene);
	}
}

/** Imports data from Lightmass about the mesh area lights generated for the scene, and creates AGeneratedMeshAreaLight's for them. */
void FLightmassProcessor::ImportMeshAreaLightData()
{
	if (MeshAreaLightDataTaskCompleted > 0)
	{
		const FString ChannelName = Lightmass::CreateChannelName(Lightmass::MeshAreaLightDataGuid, Lightmass::LM_MESHAREALIGHTDATA_VERSION, Lightmass::LM_MESHAREALIGHTDATA_EXTENSION);
		const int32 Channel = Swarm.OpenChannel( *ChannelName, LM_MESHAREALIGHT_CHANNEL_FLAGS );
		if (Channel >= 0)
		{
			int32 NumMeshAreaLights = 0;
			Swarm.ReadChannel(Channel, &NumMeshAreaLights, sizeof(NumMeshAreaLights));
			for (int32 LightIndex = 0; LightIndex < NumMeshAreaLights; LightIndex++)
			{
				Lightmass::FMeshAreaLightData LMCurrentLightData;
				Swarm.ReadChannel(Channel, &LMCurrentLightData, sizeof(LMCurrentLightData));
				const ULevel* CurrentLevel = System.LightingContext.GetLevelForGuid(LMCurrentLightData.LevelGuid).Get();
				if (CurrentLevel && CurrentLevel->Actors.Num() > 0)
				{
					// Find the level that the mesh area light was in
					FVector4 Position;
					Position = (FVector4)LMCurrentLightData.Position;
					FVector4 Direction;
					Direction = (FVector4)LMCurrentLightData.Direction;
					// Spawn a AGeneratedMeshAreaLight to handle the light's influence on dynamic objects
					FActorSpawnParameters SpawnInfo;
					SpawnInfo.Owner = CurrentLevel->GetWorldSettings();
					AGeneratedMeshAreaLight* NewGeneratedLight = CurrentLevel->OwningWorld->SpawnActor<AGeneratedMeshAreaLight>(Position, Direction.Rotation());
					USpotLightComponent* SpotComponent = CastChecked<USpotLightComponent>(NewGeneratedLight->GetLightComponent());
					// Unregister the component before we change its attributes
					FComponentReregisterContext Reregister(SpotComponent);
					// Setup spotlight properties to approximate a mesh area light
					SpotComponent->AttenuationRadius = LMCurrentLightData.Radius;
					SpotComponent->OuterConeAngle = LMCurrentLightData.ConeAngle * 180.0f / PI;
					SpotComponent->LightColor = LMCurrentLightData.Color;
					SpotComponent->Intensity = LMCurrentLightData.Brightness;
					SpotComponent->LightFalloffExponent = LMCurrentLightData.FalloffExponent;
				}
			}
			Swarm.CloseChannel(Channel);
		}
		else
		{
			UE_LOGF(LogLightmassSolver, Log,  "Error, OpenChannel failed to open %ls with error code %d", *ChannelName, Channel );
		}
		FPlatformAtomics::InterlockedExchange(&MeshAreaLightDataTaskCompleted, 0);
	}
}


/** Imports the volume distance field from Lightmass. */
void FLightmassProcessor::ImportVolumeDistanceFieldData()
{
	if (VolumeDistanceFieldTaskCompleted > 0)
	{
		const FString ChannelName = Lightmass::CreateChannelName(Lightmass::VolumeDistanceFieldGuid, Lightmass::LM_MESHAREALIGHTDATA_VERSION, Lightmass::LM_MESHAREALIGHTDATA_EXTENSION);
		const int32 Channel = Swarm.OpenChannel( *ChannelName, LM_MESHAREALIGHT_CHANNEL_FLAGS );
		if (Channel >= 0)
		{
			FPrecomputedVolumeDistanceField& DistanceField = System.GetWorld()->PersistentLevel->PrecomputedVolumeDistanceField;
			Swarm.ReadChannel(Channel, &DistanceField.VolumeSizeX, sizeof(DistanceField.VolumeSizeX));
			Swarm.ReadChannel(Channel, &DistanceField.VolumeSizeY, sizeof(DistanceField.VolumeSizeY));
			Swarm.ReadChannel(Channel, &DistanceField.VolumeSizeZ, sizeof(DistanceField.VolumeSizeZ));
			Swarm.ReadChannel(Channel, &DistanceField.VolumeMaxDistance, sizeof(DistanceField.VolumeMaxDistance));
			
			FVector4 BoxMin;
			Swarm.ReadChannel(Channel, &BoxMin, sizeof(BoxMin));
			FVector4 BoxMax;
			Swarm.ReadChannel(Channel, &BoxMax, sizeof(BoxMax));
			DistanceField.VolumeBox = FBox(BoxMin, BoxMax);

			ReadArray(Channel, DistanceField.Data);

			Swarm.CloseChannel(Channel);
		}
		else
		{
			UE_LOGF(LogLightmassSolver, Log,  "Error, OpenChannel failed to open %ls with error code %d", *ChannelName, Channel );
		}
		FPlatformAtomics::InterlockedExchange(&VolumeDistanceFieldTaskCompleted, 0);
	}
}

/**
 *	Import the texture mapping 
 *	@param	TextureMapping			The mapping being imported.
 *	@param	bProcessImmediately		If true, immediately process the mapping
 *									If false, store it off for later processing
 */
void FLightmassProcessor::ImportStaticLightingTextureMapping( const FGuid& MappingGuid, bool bProcessImmediately )
{
	FString ChannelName = Lightmass::CreateChannelName(MappingGuid, Lightmass::LM_TEXTUREMAPPING_VERSION, Lightmass::LM_TEXTUREMAPPING_EXTENSION);

	// We need to check if there's a channel with this name for each completed mapping,
	// even if the mapping has been imported as part of a previous channel.
	// Example:
	// 1. If the remote agent gets reassigned, it might have written out a merged channel
	//    (mappings A, B, C and D in one channel) but only sent out a "completed" message for
	//    some of the mappings (e.g. A and B).
	// 2. Unreal imports A, B, C and D when it receives the "completed" message for A.
	// 3. A new remote agent will process C, D and some new mappings E and F, and write out
	//    a merged channel named "C", containing C, D, E, F.
	// 4. Unreal must now read the "C" channel - even if C has been imported already - in order
	//    to import E and F.
	int32 Channel = Swarm.OpenChannel( *ChannelName, LM_TEXTUREMAPPING_CHANNEL_FLAGS );
	if (Channel >= 0)
	{
		// Read in how many mappings this channel contains
		uint32 MappingsImported = 0;
		uint32 NumMappings = 0;
		Swarm.ReadChannel(Channel, &NumMappings, sizeof(NumMappings));

		// Read in each of the mappings
		while (MappingsImported != NumMappings)
		{
			// Read in the next GUID and look up its mapping
			FGuid NextMappingGuid;
			Swarm.ReadChannel(Channel, &NextMappingGuid, sizeof(FGuid));
			FStaticLightingTextureMapping* TextureMapping = GetStaticLightingTextureMapping(NextMappingGuid);

			if (TextureMapping && !TextureMapping->IsValidMapping())
			{
				// Mapping is invalid (such as in the case of BSP being invalidated), discard the rest of the file
				break;
			}

			// If we don't have a mapping pending, check to see if we've already imported
			// it which can *possibly* happen if a disconnection race condition occured
			// where we got the results for a task, but didn't get the message that it
			// had finished before we re-queued/re-assigned the task to another agent,
			// which could result in duplicate results. If we get a duplicate, just
			// re-import the redundant results.
			bool bReimporting = false;
			if ( TextureMapping == NULL )
			{
				// Remove the mapping from ImportedMappings and re-import it.
				FMappingImportHelper** pImportData = ImportedMappings.Find( NextMappingGuid );
				check( pImportData && *pImportData && (*pImportData)->Type == SLT_Texture );
				FTextureMappingImportHelper* pTextureImportData = (*pImportData)->GetTextureMappingHelper();
				TextureMapping = pTextureImportData->TextureMapping.GetReference();
				bReimporting = true;
				if ( GLightmassStatsMode )
				{
					UE_LOGF(LogLightmassSolver, Log, "Re-importing texture mapping: %ls", *NextMappingGuid.ToString());
				}
			}

			if( ensureMsgf( (TextureMapping != NULL), TEXT("Opened mapping channel %s to Swarm, then tried to find texture mapping %s (number %d of %d) and failed."), *MappingGuid.ToString(), *NextMappingGuid.ToString(), MappingsImported, NumMappings) )
			{
				//UE_LOGF(LogLightmassSolver, Log, "Importing %32ls %ls", *(TextureMapping->GetDescription()), *(TextureMapping->GetLightingGuid().ToString()));

				FTextureMappingImportHelper* ImportData = new FTextureMappingImportHelper();
				ImportData->TextureMapping = TextureMapping;
				ImportData->MappingGuid = NextMappingGuid;
				if (ImportTextureMapping(Channel, *ImportData) == true)
				{
					if ( bReimporting == false )
					{
						ImportedMappings.Add(ImportData->MappingGuid, ImportData);
					}
					if (bProcessImmediately)
					{
						ProcessMapping(ImportData->MappingGuid);
					}
				}
				else
				{
					UE_LOGF(LogLightmassSolver, Warning, "Failed to import TEXTure mapping results!");
				}

				// Completed this mapping, increment
				MappingsImported++;
			}
			else
			{
				// Report an error for this mapping
				UObject* Object = NULL;
				const FStaticLightingMapping* FoundMapping = Exporter->FindMappingByGuid(NextMappingGuid);
				if (FoundMapping)
				{
					Object = FoundMapping->GetMappedObject();
				}

				FMessageLog("LightingResults").Error()
					->AddToken(FUObjectToken::Create(Object))
					->AddToken(FTextToken::Create(LOCTEXT("LightmassError_LightingBuildError", "Lighting build error")));

				// We can't trust the rest of this file, so we'll need to bail now
				break;
			}
		}
		Swarm.CloseChannel( Channel );
	}
	// File not found?
	else if ( Channel == NSwarm::SWARM_ERROR_CHANNEL_NOT_FOUND )
	{
		// If the channel doesn't exist, then this mapping could've been part of another channel
		// that has already been imported, so attempt to remove the mapping
		FStaticLightingTextureMapping* TextureMapping = GetStaticLightingTextureMapping(MappingGuid);
		// Alternatively, this channel could be part of an invalidated mapping
	}
	// Other error
	else
	{
		UE_LOGF(LogLightmassSolver, Log,  "Error, OpenChannel failed to open %ls with error code %d", *ChannelName, Channel );
	}
}

/** Determines whether the specified mapping is a texture mapping */
bool FLightmassProcessor::IsStaticLightingTextureMapping( const FGuid& MappingGuid )
{
	if (PendingBSPMappings.Contains(MappingGuid))
	{
		return true;
	}
	else if (PendingTextureMappings.Contains(MappingGuid))
	{
		return true;
	}
	else if (PendingLandscapeMappings.Contains(MappingGuid))
	{
		return true;
	}
	else
	{
		FMappingImportHelper** pImportData = ImportedMappings.Find(MappingGuid);
		if ( pImportData && (*pImportData)->Type == SLT_Texture )
		{
			return true;
		}
	}
	return false;
}

/** Gets the texture mapping for the specified GUID */
FStaticLightingTextureMapping* FLightmassProcessor::GetStaticLightingTextureMapping( const FGuid& MappingGuid )
{
	FBSPSurfaceStaticLighting* BSPMapping = NULL;
	FStaticMeshStaticLightingTextureMapping* SMTextureMapping = NULL;
	FLandscapeStaticLightingTextureMapping* LandscapeMapping = NULL;

	if (PendingBSPMappings.RemoveAndCopyValue(MappingGuid, BSPMapping))
	{
		return BSPMapping->GetTextureMapping();
	}
	else if (PendingTextureMappings.RemoveAndCopyValue(MappingGuid, SMTextureMapping))
	{
		return SMTextureMapping->GetTextureMapping();
	}
	else if (PendingLandscapeMappings.RemoveAndCopyValue(MappingGuid, LandscapeMapping))
	{
		return LandscapeMapping->GetTextureMapping();
	}
	return NULL;
}

void FLightmassProcessor::ImportStaticShadowDepthMap(ULightComponent* Light)
{
	const FString ChannelName = Lightmass::CreateChannelName(Light->LightGuid, Lightmass::LM_DOMINANTSHADOW_VERSION, Lightmass::LM_DOMINANTSHADOW_EXTENSION);
	const int32 Channel = Swarm.OpenChannel( *ChannelName, LM_DOMINANTSHADOW_CHANNEL_FLAGS );
	if (Channel >= 0)
	{
		UMapBuildDataRegistry* CurrentRegistry = System.LightingContext.GetOrCreateRegistryForActor(Light->GetOwner());
		FLightComponentMapBuildData& CurrentLightData = CurrentRegistry->FindOrAllocateLightBuildData(Light->LightGuid, true);

		Lightmass::FStaticShadowDepthMapData ShadowMapData;
		Swarm.ReadChannel(Channel, &ShadowMapData, sizeof(ShadowMapData));

		BeginReleaseResource(&Light->StaticShadowDepthMap);
		CurrentLightData.DepthMap.Empty();

		CurrentLightData.DepthMap.WorldToLight = FMatrix(ShadowMapData.WorldToLight);
		CurrentLightData.DepthMap.ShadowMapSizeX = ShadowMapData.ShadowMapSizeX;
		CurrentLightData.DepthMap.ShadowMapSizeY = ShadowMapData.ShadowMapSizeY;

		ReadArray(Channel, CurrentLightData.DepthMap.DepthSamples);
		Swarm.CloseChannel(Channel);

		CurrentLightData.FinalizeLoad();
	}
	else
	{
		UE_LOGF(LogLightmassSolver, Log,  "Error, OpenChannel failed to open %ls with error code %d", *ChannelName, Channel );
	}
}

/**
 * Import the mapping specified by a Guid.
 *	@param MappingGuid				Guid that identifies a mapping
 *	@param	bProcessImmediately		If true, immediately process the mapping
 *									If false, store it off for later processing
 */
void FLightmassProcessor::ImportMapping( const FGuid& MappingGuid, bool bProcessImmediately )
{
	double ImportAndApplyStartTime = FPlatformTime::Seconds();
	double OriginalApplyTime = Statistics.ApplyTimeInProcessing;

	if (IsStaticLightingTextureMapping(MappingGuid))
	{
		ImportStaticLightingTextureMapping(MappingGuid, bProcessImmediately);
	}
	else
	{
		ULightComponent* Light = FindLight(MappingGuid);

		if (Light)
		{
			ImportStaticShadowDepthMap(Light);
		}
		else
		{
			FMappingImportHelper** pImportData = ImportedMappings.Find(MappingGuid);
			if ((pImportData == NULL) || (*pImportData == NULL))
			{
				UE_LOGF(LogLightmassSolver, Warning, "Mapping not found for %ls", *(MappingGuid.ToString()));
			}
		}
	}

	if ( !bRunningLightmass )
	{
		double ApplyTime = Statistics.ApplyTimeInProcessing - OriginalApplyTime;
		double ImportTime = FPlatformTime::Seconds() - ImportAndApplyStartTime - ApplyTime;
		Statistics.ImportTimeInProcessing += ImportTime;
	}
}

/**
 * Process the mapping specified by a Guid.
 * @param MappingGuid	Guid that identifies a mapping
 **/
void FLightmassProcessor::ProcessMapping( const FGuid& MappingGuid )
{
	double ApplyStartTime = FPlatformTime::Seconds();

	FMappingImportHelper** pImportData = ImportedMappings.Find(MappingGuid);
	if ( pImportData && *pImportData )
	{
		if ( (*pImportData)->bProcessed == false )
		{
			FMappingImportHelper* ImportData = *pImportData;
			switch (ImportData->Type)
			{
			case SLT_Texture:
				{
					FTextureMappingImportHelper* TImportData = (FTextureMappingImportHelper*)ImportData;
					if(TImportData->TextureMapping)
					{
						//UE_LOGF(LogLightmassSolver, Log, "Processing %32ls: %ls", *(TImportData->TextureMapping->GetDescription()), *(TImportData->TextureMapping->GetLightingGuid().ToString()));
						System.ApplyMapping(TImportData->TextureMapping, TImportData->QuantizedData, TImportData->ShadowMapData);
					}
					else
					{
						//UE_LOGF(LogLightmassSolver, Log, "Processing texture mapping %ls failed due to missing mapping!", *MappingGuid.ToString());
					}
				}
				break;
			default:
				{
					UE_LOGF(LogLightmassSolver, Warning, "Unknown mapping type in the ImportedMappings: 0x%08x", (uint32)(ImportData->Type));
				}
			}

			(*pImportData)->bProcessed = true;
		}
		else
		{
			// Just to be able to set a breakpoint here.
			int32 DebugDummy = 0;
		}
	}
	else
	{
		UE_LOGF(LogLightmassSolver, Warning, "Failed to find imported mapping %ls", *(MappingGuid.ToString()));
	}

	if ( !bRunningLightmass )
	{
		Statistics.ApplyTimeInProcessing += FPlatformTime::Seconds() - ApplyStartTime;
	}
}

/**
 * Process any available mappings.
 **/
void FLightmassProcessor::ProcessAvailableMappings()
{
	bool bDoneProcessing = false;
	int32 ProcessedCount = 0;
	int32 ImportedMappingsCount = ImportedMappings.Num();
	while (!bDoneProcessing)
	{
		FGuid NextGuid = FGuid(0,0,0,MappingToProcessIndex);
		FMappingImportHelper** pImportData = ImportedMappings.Find(NextGuid);
		if (pImportData)
		{
			if ( *pImportData && (*pImportData)->bProcessed == false )
			{
				if (IsDeferredMapping(NextGuid))
				{
					DeferMapping((*pImportData)->GetTextureMappingHelper());
				}
				else
				{
					ProcessMapping(NextGuid);
				}
			}
			ProcessedCount++;
		}

		MappingToProcessIndex++;

		if (ProcessedCount >= ImportedMappingsCount)
		{
			bDoneProcessing = true;
		}
	}
}

void FLightmassProcessor::ImportDebugOutputStruct(int32 Channel)
{
	static_assert(sizeof(FDebugStaticLightingRay) == sizeof(Lightmass::FDebugStaticLightingRay), "Debug type sizes must match for FDebugStaticLightingRay.");
	static_assert(sizeof(FDebugStaticLightingVertex) == sizeof(Lightmass::FDebugStaticLightingVertex), "Debug type sizes must match for FDebugStaticLightingVertex.");
	static_assert(sizeof(FDebugLightingCacheRecord) == sizeof(Lightmass::FDebugLightingCacheRecord), "Debug type sizes must match for FDebugLightingCacheRecord.");
	static_assert(STRUCT_OFFSET(FDebugLightingCacheRecord, RecordId) == STRUCT_OFFSET(Lightmass::FDebugLightingCacheRecord, RecordId), "Debug struct offset must match for FDebugLightingCacheRecord::RecordId.");
	static_assert(sizeof(FDebugPhoton) == sizeof(Lightmass::FDebugPhoton), "Debug type sizes must match for FDebugPhoton.");
	static_assert(sizeof(FDebugOctreeNode) == sizeof(Lightmass::FDebugOctreeNode), "Debug type sizes must match for FDebugOctreeNode.");
	static_assert(NumTexelCorners == Lightmass::NumTexelCorners, "Debug type sizes must match for NumTexelCorners.");

	bool bDebugInfoValid = false;
	Swarm.ReadChannel(Channel, &bDebugInfoValid, sizeof(bDebugInfoValid));

	if (bDebugInfoValid)
	{
		if (GDebugStaticLightingInfo.bValid)
		{
			UE_LOGF(LogLightmassSolver, Log, "Error, importing valid debug info, but GDebugStaticLightingInfo was already valid (multiple sources of debug info from Lightmass)");
		}

		GDebugStaticLightingInfo.bValid = true;
		ReadArray(Channel, GDebugStaticLightingInfo.PathRays);
		ReadArray(Channel, GDebugStaticLightingInfo.ShadowRays);
		ReadArray(Channel, GDebugStaticLightingInfo.IndirectPhotonPaths);
		ReadArray(Channel, GDebugStaticLightingInfo.SelectedVertexIndices);
		ReadArray(Channel, GDebugStaticLightingInfo.Vertices);
		ReadArray(Channel, GDebugStaticLightingInfo.CacheRecords);
		ReadArray(Channel, GDebugStaticLightingInfo.DirectPhotons);
		ReadArray(Channel, GDebugStaticLightingInfo.IndirectPhotons);
		ReadArray(Channel, GDebugStaticLightingInfo.IrradiancePhotons);
		ReadArray(Channel, GDebugStaticLightingInfo.GatheredPhotons);
		ReadArray(Channel, GDebugStaticLightingInfo.GatheredImportancePhotons);
		ReadArray(Channel, GDebugStaticLightingInfo.GatheredPhotonNodes);
		Swarm.ReadChannel(Channel, &GDebugStaticLightingInfo.bDirectPhotonValid, sizeof(GDebugStaticLightingInfo.bDirectPhotonValid));
		Swarm.ReadChannel(Channel, &GDebugStaticLightingInfo.GatheredDirectPhoton, sizeof(GDebugStaticLightingInfo.GatheredDirectPhoton));
		Swarm.ReadChannel(Channel, &GDebugStaticLightingInfo.TexelCorners, sizeof(GDebugStaticLightingInfo.TexelCorners));
		Swarm.ReadChannel(Channel, &GDebugStaticLightingInfo.bCornerValid, sizeof(GDebugStaticLightingInfo.bCornerValid));
		Swarm.ReadChannel(Channel, &GDebugStaticLightingInfo.SampleRadius, sizeof(GDebugStaticLightingInfo.SampleRadius));
	}
}

/**
 *	Retrieve the light for the given Guid
 *
 *	@param	LightGuid			The guid of the light we are looking for
 *
 *	@return	ULightComponent*	The corresponding light component.
 *								NULL if not found.
 */
ULightComponent* FLightmassProcessor::FindLight(const FGuid& LightGuid)
{
	if (Exporter)
	{
		int32 LightIndex;
		for (LightIndex = 0; LightIndex < Exporter->DirectionalLights.Num(); LightIndex++)
		{
			const UDirectionalLightComponent* Light = Exporter->DirectionalLights[LightIndex];
			if (Light)
			{
				if (Light->LightGuid == LightGuid)
				{
					return (ULightComponent*)Light;
				}
			}
		}
		for (LightIndex = 0; LightIndex < Exporter->PointLights.Num(); LightIndex++)
		{
			const UPointLightComponent* Light = Exporter->PointLights[LightIndex];
			if (Light)
			{
				if (Light->LightGuid == LightGuid)
				{
					return (ULightComponent*)Light;
				}
			}
		}
		for (LightIndex = 0; LightIndex < Exporter->SpotLights.Num(); LightIndex++)
		{
			const USpotLightComponent* Light = Exporter->SpotLights[LightIndex];
			if (Light)
			{
				if (Light->LightGuid == LightGuid)
				{
					return (ULightComponent*)Light;
				}
			}
		}
		for (LightIndex = 0; LightIndex < Exporter->RectLights.Num(); LightIndex++)
		{
			const URectLightComponent* Light = Exporter->RectLights[LightIndex];
			if (Light)
			{
				if (Light->LightGuid == LightGuid)
				{
					return (ULightComponent*)Light;
				}
			}
		}
	}

	return NULL;
}

/**
 *	Retrieve the static mehs for the given Guid
 *
 *	@param	Guid				The guid of the static mesh we are looking for
 *
 *	@return	UStaticMesh*		The corresponding static mesh.
 *								NULL if not found.
 */
UStaticMesh* FLightmassProcessor::FindStaticMesh(FGuid& Guid)
{
	if (Exporter)
	{
		for (int32 SMIdx = 0; SMIdx < Exporter->StaticMeshes.Num(); SMIdx++)
		{
			const UStaticMesh* StaticMesh = Exporter->StaticMeshes[SMIdx];
			if (StaticMesh && (StaticMesh->GetLightingGuid() == Guid))
			{
				return (UStaticMesh*)StaticMesh;
			}
		}
	}
	return NULL;
}

ULevel* FLightmassProcessor::FindLevel(const FGuid& Guid)
{
	if (Exporter)
	{
		const TWeakObjectPtr<ULevel> Level = Exporter->LightingContext.GetLevelForGuid(Guid);
		return Level.IsValid() ? Level.Get() : NULL;
	}
	return NULL;
}

/**
 *	Import light map data from the given channel.
 *
 *	@param	Channel				The channel to import from.
 *	@param	QuantizedData		The quantized lightmap data to fill in
 *	@param	UncompressedSize	Size the data will be after uncompressing it (if compressed)
 *	@param	CompressedSize		Size of the source data if compressed
 *
 *	@return	bool				true if successful, false otherwise.
 */
bool FLightmassProcessor::ImportLightMapData2DData(int32 Channel, FQuantizedLightmapData* QuantizedData, int32 UncompressedSize, int32 CompressedSize)
{
	check(QuantizedData);
	
	int32 SizeX = QuantizedData->SizeX;
	int32 SizeY = QuantizedData->SizeY;

	// make space for the samples
	QuantizedData->Data.Empty(SizeX * SizeY);
	QuantizedData->Data.AddUninitialized(SizeX * SizeY);
	FLightMapCoefficients* DataBuffer = QuantizedData->Data.GetData();

	int32 DataBufferSize = SizeX * SizeY * sizeof(FLightMapCoefficients);

	check(DataBufferSize == UncompressedSize);

	// read in the compressed data
	void* CompressedBuffer = FMemory::Malloc(CompressedSize);
	Swarm.ReadChannel(Channel, CompressedBuffer, CompressedSize);

	// decompress the temp buffer into another temp buffer 
	if (!FCompression::UncompressMemory(NAME_Zlib, DataBuffer, UncompressedSize, CompressedBuffer, CompressedSize))
	{
		checkf(false, TEXT("Uncompress failed, which is unexpected"));
	}

	// can free one buffer now
	FMemory::Free(CompressedBuffer);

	return true;
}

bool FLightmassProcessor::ImportSignedDistanceFieldShadowMapData2D(int32 Channel, TMap<ULightComponent*,FShadowMapData2D*>& OutShadowMapData, int32 ShadowMapCount)
{
	for (int32 SMIndex = 0; SMIndex < ShadowMapCount; SMIndex++)
	{
		FGuid LightGuid;
		Swarm.ReadChannel(Channel, &LightGuid, sizeof(FGuid));

		ULightComponent* LightComp = FindLight(LightGuid);
		if (LightComp == NULL)
		{
			UE_LOGF(LogLightmassSolver, Warning, "Failed to find light for texture mapping: %ls", *LightGuid.ToString());
		}

		Lightmass::FShadowMapData2DData SMData(0,0);
		Swarm.ReadChannel(Channel, &SMData, sizeof(Lightmass::FShadowMapData2DData));

		static_assert(sizeof(FQuantizedSignedDistanceFieldShadowSample) == sizeof(Lightmass::FQuantizedSignedDistanceFieldShadowSampleData), "Sample data sizes must match.");

		FQuantizedShadowSignedDistanceFieldData2D* ShadowMapData = new FQuantizedShadowSignedDistanceFieldData2D(SMData.SizeX, SMData.SizeY);
		check(ShadowMapData);

		FQuantizedSignedDistanceFieldShadowSample* DataBuffer = ShadowMapData->GetData();
		uint32 DataBufferSize = SMData.SizeX * SMData.SizeY * sizeof(Lightmass::FQuantizedSignedDistanceFieldShadowSampleData);

		uint32 CompressedSize = SMData.CompressedDataSize;
		uint32 UncompressedSize = SMData.UncompressedDataSize;
		check(DataBufferSize == UncompressedSize);

		// Read in the compressed data
		void* CompressedBuffer = FMemory::Malloc(CompressedSize);
		Swarm.ReadChannel(Channel, CompressedBuffer, CompressedSize);

		// Decompress the temp buffer into another temp buffer 
		if (!FCompression::UncompressMemory(NAME_Zlib, DataBuffer, UncompressedSize, CompressedBuffer, CompressedSize))
		{
			checkf(false, TEXT("Uncompress failed, which is unexpected"));
		}
		FMemory::Free(CompressedBuffer);

		if (LightComp)
		{
			OutShadowMapData.Add(LightComp, ShadowMapData);
		}
		else
		{
			delete ShadowMapData;
		}
	}

	return true;
}

/**
 *	Import a complete texture mapping....
 *
 *	@param	Channel			The channel to import from.
 *	@param	TMImport		The texture mapping information that will be imported.
 *
 *	@return	bool			true if successful, false otherwise.
 */
bool FLightmassProcessor::ImportTextureMapping(int32 Channel, FTextureMappingImportHelper& TMImport)
{
	bool bResult = true;

	// Additional information for this mapping
	Swarm.ReadChannel(Channel, &(TMImport.ExecutionTime), sizeof(double));

	// The resulting light map data for this mapping (shared header and TArray) 
	Lightmass::FLightMapData2DData LMLightmapData2DData(0,0);
	Swarm.ReadChannel(Channel, &LMLightmapData2DData, sizeof(Lightmass::FLightMapData2DData));
	check(TMImport.TextureMapping->SizeX == LMLightmapData2DData.SizeX);
	check(TMImport.TextureMapping->SizeY == LMLightmapData2DData.SizeY);
	Swarm.ReadChannel(Channel, &TMImport.NumShadowMaps, sizeof(TMImport.NumShadowMaps));
	Swarm.ReadChannel(Channel, &TMImport.NumSignedDistanceFieldShadowMaps, sizeof(TMImport.NumSignedDistanceFieldShadowMaps));

	int32 NumLights = 0;
	TArray<FGuid> LightGuids;
	LightGuids.Empty(NumLights);
	Swarm.ReadChannel(Channel, &NumLights, sizeof(NumLights));
	for (int32 i = 0; i < NumLights; i++)
	{
		const int32 NewLightIndex = LightGuids.AddUninitialized();
		Swarm.ReadChannel(Channel, &LightGuids[NewLightIndex], sizeof(LightGuids[NewLightIndex]));
	}

	// allocate space to store the quantized data
	TMImport.QuantizedData = new FQuantizedLightmapData;
	FMemory::Memcpy(TMImport.QuantizedData->Scale, LMLightmapData2DData.Multiply, sizeof(TMImport.QuantizedData->Scale));
	FMemory::Memcpy(TMImport.QuantizedData->Add, LMLightmapData2DData.Add, sizeof(TMImport.QuantizedData->Add));
	TMImport.QuantizedData->SizeX = LMLightmapData2DData.SizeX;
	TMImport.QuantizedData->SizeY = LMLightmapData2DData.SizeY;
	TMImport.QuantizedData->LightGuids = LightGuids;
	TMImport.QuantizedData->bHasSkyShadowing = LMLightmapData2DData.bHasSkyShadowing;

	if (ImportLightMapData2DData(Channel, TMImport.QuantizedData, LMLightmapData2DData.UncompressedDataSize, LMLightmapData2DData.CompressedDataSize) == false)
	{
		bResult = false;
	}

	int32 NumUnmappedTexels = 0;
	for (int32 DebugIdx = 0; DebugIdx < TMImport.QuantizedData->Data.Num(); DebugIdx++)
	{
		if (TMImport.QuantizedData->Data[DebugIdx].Coverage == 0.0f)
		{
			NumUnmappedTexels++;
		}
	}

	if (TMImport.QuantizedData->Data.Num() > 0)
	{
		TMImport.UnmappedTexelsPercentage = 100.0f * (float)NumUnmappedTexels / (float)TMImport.QuantizedData->Data.Num();
	}
	else
	{
		TMImport.UnmappedTexelsPercentage = 0.0f;
	}

	if (ImportSignedDistanceFieldShadowMapData2D(Channel, TMImport.ShadowMapData, TMImport.NumSignedDistanceFieldShadowMaps) == false)
	{
		bResult = false;
	}

	ImportDebugOutputStruct(Channel);

	// Update the LightingBuildInfo list
	UObject* MappedObject = TMImport.TextureMapping->GetMappedObject();
	float MemoryAmount = float(NumUnmappedTexels);
	float TotalMemoryAmount = float(TMImport.QuantizedData->Data.Num());
	//@todo. Move this into some common place... it's defined in several places now!
	const float MIP_FACTOR = 4.0f / 3.0f;
	// Assume compressed == 4 bits/pixel, really this is platform specific.
	float BytesPerPixel = 1.0f;

	float LightMapTypeModifier = NUM_HQ_LIGHTMAP_COEF;
	if (!AllowHighQualityLightmaps(GMaxRHIFeatureLevel))
	{
		LightMapTypeModifier = NUM_LQ_LIGHTMAP_COEF;
	}

	int32 WastedMemory = FMath::TruncToInt(MemoryAmount * BytesPerPixel * MIP_FACTOR * LightMapTypeModifier);
	int32 TotalMemory = FMath::TruncToInt(TotalMemoryAmount * BytesPerPixel * MIP_FACTOR * LightMapTypeModifier);
	
	FStatsViewerModule& StatsViewerModule = FModuleManager::Get().LoadModuleChecked<FStatsViewerModule>(TEXT("StatsViewer"));
	ULightingBuildInfo* LightingBuildInfo = NewObject<ULightingBuildInfo>();
	LightingBuildInfo->Set( MappedObject, TMImport.ExecutionTime, TMImport.UnmappedTexelsPercentage, WastedMemory, TotalMemory );
	StatsViewerModule.GetPage(EStatsPage::LightingBuildInfo)->AddEntry( LightingBuildInfo );

	return bResult;
}

void FLightmassProcessor::FMappingImportHelper::Serialize(FArchive& Ar)
{
	static_assert(sizeof(Type) == sizeof(int32));
	Ar << (int32&)Type;
	Ar << MappingGuid;
	Ar << OwnerGuid;
	Ar << ExecutionTime;
	Ar << bProcessed;
}

void FLightmassProcessor::FTextureMappingImportHelper::Serialize(FArchive& Ar)
{
	FMappingImportHelper::Serialize(Ar);

	if (Ar.IsLoading())
	{
		int32 MappingType;
		Ar << MappingType;
		
		switch (MappingType)
		{
			case 0:
				TextureMapping = new FStaticMeshStaticLightingTextureMapping(Ar);
				break;
			case 1:
				TextureMapping = new FLandscapeStaticLightingTextureMapping(Ar);
				break;			
			case 2:
				TextureMapping = new FStaticLightingTextureMapping_InstancedStaticMesh(Ar);
		}
		
		// Ptr will be deleted during FStaticLightingTextureMapping::Apply
		QuantizedData = new FQuantizedLightmapData;
	}
	else
	{
		static  TMap<FString, int32> DescToType = {{"SMTextureMapping", 0}, { "LandscapeMapping", 1 }, {"InstancedSMLightingMapping", 2}};
		
		int32  mappingType = DescToType.FindChecked(TextureMapping->GetDescription());
		Ar << mappingType;
	}

	TextureMapping->Serialize(Ar);
	QuantizedData->Serialize(Ar);
	
	Ar << UnmappedTexelsPercentage;
	Ar << NumShadowMaps;
	Ar << NumSignedDistanceFieldShadowMaps;

	int32 NbShadowMapData = ShadowMapData.Num();
	Ar << NbShadowMapData;

	if (Ar.IsLoading())
	{
		for (int32 i = 0; i < NbShadowMapData; i++)
		{
			FSoftObjectPath	Path;
			Ar << Path;

			int32 ShadowMapType;
			Ar << ShadowMapType;

			int32 SizeX;
			int32 SizeY;
			Ar << SizeX;
			Ar << SizeY;

			// Ptr will be deleted during FStaticLightingTextureMapping::Apply
			FShadowMapData2D* Data = nullptr;

			switch ((FShadowMapData2D::ShadowMapDataType)ShadowMapType)
			{
				case FShadowMapData2D::SHADOW_SIGNED_DISTANCE_FIELD_DATA:
					Data = new FShadowSignedDistanceFieldData2D(SizeX, SizeY);
					break;
				case FShadowMapData2D::SHADOW_SIGNED_DISTANCE_FIELD_DATA_QUANTIZED:
					Data = new FQuantizedShadowSignedDistanceFieldData2D(SizeX, SizeY);
					break;

				default:
				case FShadowMapData2D::SHADOW_FACTOR_DATA:
				case FShadowMapData2D::SHADOW_FACTOR_DATA_QUANTIZED:
					ensure(false);
					return;
			}

			Data->Serialize(&Ar);

			ShadowMapData.Add(Cast<ULightComponent>(Path.ResolveObject()), Data);
		}
	}
	else
	{
		for (auto& it : ShadowMapData)
		{
			FSoftObjectPath Path(it.Key);
			Ar << Path;

			int32 ShadowMapType = (int32)it.Value->GetType();
			Ar << ShadowMapType;

			int32 SizeX = it.Value->GetSizeX();
			int32 SizeY = it.Value->GetSizeY();
			Ar << SizeX;
			Ar << SizeY;

			it.Value->Serialize(&Ar);
		}

	}	
}

#undef LOCTEXT_NAMESPACE
