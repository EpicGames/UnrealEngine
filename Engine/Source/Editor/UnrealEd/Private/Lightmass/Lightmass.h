// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Lightmass.h: lightmass import/export definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "GameFramework/WorldSettings.h"
#include "Lightmass/LightmassCharacterIndirectDetailVolume.h"
#include "Lightmass/VolumetricLightmapDensityVolume.h"
#include "StaticLightingSystem/StaticLightingPrivate.h"
#include "Lightmass/LightmassImportanceVolume.h"
#include "Components/LightmassPortalComponent.h"
#include "Components/SkyAtmosphereComponent.h"
#if PLATFORM_WINDOWS && !PLATFORM_CPU_ARM_FAMILY
#include "Windows/AllowWindowsPlatformTypes.h"
#endif
	#include "SwarmInterface.h"
	#include "Lightmass/LightmassRender.h"
#if PLATFORM_WINDOWS && !PLATFORM_CPU_ARM_FAMILY
#include "Windows/HideWindowsPlatformTypes.h"
#endif

class FBSPSurfaceStaticLighting;
class FLandscapeStaticLightingMesh;
class FLandscapeStaticLightingTextureMapping;
class FShadowMapData2D;
class FStaticMeshStaticLightingTextureMapping;
class ULightComponent;
class ULightComponentBase;
class UMaterialInterface;
class UStaticMesh;
struct FQuantizedLightmapData;

template <class ElementType> class TList;

/** Forward declarations of Lightmass types */
namespace Lightmass
{
	struct FSceneFileHeader;
	struct FDebugLightingInputData;
	struct FMaterialData;
	struct FMaterialElementData;
	class FVolumetricLightmapSettings;
}

struct FLightmassMaterialExportSettings
{
	const FStaticLightingMesh* UnwrapMesh;

	friend bool operator ==(const FLightmassMaterialExportSettings& lhs, const FLightmassMaterialExportSettings& rhs)
	{
		return lhs.UnwrapMesh == rhs.UnwrapMesh;
	}
};

/** Lightmass Exporter class */
class FLightmassExporter
{
public:

	FLightmassExporter(const FStaticLightingBuildContext& LightingContext);
	virtual ~FLightmassExporter();

	void SetLevelSettings(FLightmassWorldInfoSettings& InLevelSettings)
	{
		LevelSettings = InLevelSettings;
	}

	void SetNumUnusedLocalCores(int32 InNumUnusedLocalCores)
	{
		NumUnusedLocalCores = InNumUnusedLocalCores;
	}

	void SetQualityLevel(ELightingBuildQuality InQualityLevel)
	{
		QualityLevel = InQualityLevel;
	}

	void SetLevelName(const FString& InName)
	{
		LevelName = InName;
	}

	void ClearImportanceVolumes()
	{
		ImportanceVolumes.Empty();
	}

	void AddImportanceVolume(const ALightmassImportanceVolume* InImportanceVolume)
	{
		ImportanceVolumes.Add(InImportanceVolume->GetComponentsBoundingBox(true));
	}

	void AddImportanceVolumeBoundingBox(const FBox& Bounds)
	{
		ImportanceVolumes.Add(Bounds);
	}

	const TArray<FBox>& GetImportanceVolumes() const
	{
		return ImportanceVolumes;
	}

	void AddCharacterIndirectDetailVolume(const ALightmassCharacterIndirectDetailVolume* InDetailVolume)
	{
		CharacterIndirectDetailVolumes.Add(InDetailVolume->GetComponentsBoundingBox(true));
	}

	void AddPortal(const ULightmassPortalComponent* InPortalComponent)
	{
		Portals.Add(InPortalComponent->GetComponentTransform().ToMatrixWithScale());
	}

	void SetSkyAtmosphereComponent(const USkyAtmosphereComponent* SkyAtmosphere)
	{
		SkyAtmosphereComponent = SkyAtmosphere;
	}

	// if provided, InStaticLightingMesh is used to UV unwrap the material into the static lighting textures
	void AddMaterial(UMaterialInterface* InMaterialInterface, const FStaticLightingMesh* InStaticLightingMesh = nullptr);

	void AddLight(ULightComponentBase* Light);

	const FStaticLightingMapping* FindMappingByGuid(FGuid FindGuid) const;
	
	float GetAmortizedExportPercentDone() const;

	/** Guids of visibility tasks. */
	TArray<FGuid> VisibilityBucketGuids;

	TMap<FGuid, int32> VolumetricLightmapTaskGuids;

	TArray<AVolumetricLightmapDensityVolume*> VolumetricLightmapDensityVolumes;

private:

	void SetVolumetricLightmapSettings(Lightmass::FVolumetricLightmapSettings& OutSettings);

	void WriteToChannel( FLightmassStatistics& Stats, FGuid& DebugMappingGuid );
	bool WriteToMaterialChannel(FLightmassStatistics& Stats);

	/** Exports visibility Data. */
	void WriteVisibilityData(int32 Channel);

	void WriteVolumetricLightmapData(int32 Channel);

	void WriteLights( int32 Channel );

	/**
	 * Exports all UModels to secondary, persistent channels
	 */
	void WriteModels();
	
	/**
	 * Exports all UStaticMeshes to secondary, persistent channels
	 */
	void WriteStaticMeshes();

	void WriteLandscapes();

	/**
	 *	Exports all of the materials to secondary, persistent channels
	 */
	void BuildMaterialMap(UMaterialInterface* Material);
	void BlockOnShaderCompilation();
	void ExportMaterial(UMaterialInterface* Material, const FLightmassMaterialExportSettings& ExportSettings);

	void WriteMeshInstances( int32 Channel );
	void WriteLandscapeInstances( int32 Channel );

	void WriteMappings( int32 Channel );

	void WriteBaseMeshInstanceData( int32 Channel, int32 MeshIndex, const class FStaticLightingMesh* Mesh, TArray<Lightmass::FMaterialElementData>& MaterialElementData );
	void WriteBaseMappingData( int32 Channel, const class FStaticLightingMapping* Mapping );
	void WriteBaseTextureMappingData( int32 Channel, const class FStaticLightingTextureMapping* TextureMapping );
	void WriteVertexData( int32 Channel, const struct FStaticLightingVertex* Vertex );
	void WriteLandscapeMapping( int32 Channel, const class FLandscapeStaticLightingTextureMapping* LandscapeMapping );

	void GetMaterialHash(const UMaterialInterface* Material, FSHAHash& OutHash);

	/** Finds the GUID of the mapping that is being debugged. */
	bool FindDebugMapping(FGuid& DebugMappingGuid);

	/** Fills out the Scene's settings, read from the engine ini. */
	void WriteSceneSettings( Lightmass::FSceneFileHeader& Scene );

	/** Fills InputData with debug information */
	void WriteDebugInput( Lightmass::FDebugLightingInputData& InputData, FGuid& DebugMappingGuid );

	void UpdateExportProgress();

	TMap<const FStaticLightingMesh*,int32> MeshToIndexMap;

	//
	NSwarm::FSwarmInterface&	Swarm;
	bool						bSwarmConnectionIsValid;
	FGuid						SceneGuid;
	FSHAHash					LightmassExecutableHash;
	FString						ChannelName;

	TArray<FBox>				ImportanceVolumes;
	TArray<FBox>				CharacterIndirectDetailVolumes;
	TArray<FMatrix>				Portals;

	const USkyAtmosphereComponent*	SkyAtmosphereComponent;

	FLightmassWorldInfoSettings LevelSettings;
	/** The number of local cores to leave unused */
	int32 NumUnusedLocalCores;
	/** The quality level of the lighting build */
	ELightingBuildQuality QualityLevel;

	/** Amortize Export stage that we currently are in */
	enum AmortizedExportStage
	{
		NotRunning,
		BuildMaterials,
		ShaderCompilation,
		ExportMaterials,
		CleanupMaterialExport,
		Complete,
	};
	AmortizedExportStage ExportStage;
	/** The current index (multi-use) for the current stage */
	int32 CurrentAmortizationIndex;
	/** List of all channels that we've opened in swarm during amortized export that we need to amortized close */
	TArray<int32> OpenedMaterialExportChannels;

	FString LevelName;

	// lights objects
	TArray<const class UDirectionalLightComponent*> DirectionalLights;
	TArray<const class UPointLightComponent*> PointLights;
	TArray<const class USpotLightComponent*> SpotLights;
	TArray<const class URectLightComponent*> RectLights;
	TArray<const class USkyLightComponent*> SkyLights;

	// BSP mappings
	TArray<class FBSPSurfaceStaticLighting*> BSPSurfaceMappings;
	TArray<const class UModel*> Models;
	
	// static mesh mappings
	TArray<const class FStaticMeshStaticLightingMesh*> StaticMeshLightingMeshes;
	TArray<class FStaticMeshStaticLightingTextureMapping*> StaticMeshTextureMappings;
	TArray<const class UStaticMesh*> StaticMeshes;

	// Landscape
	TArray<const FLandscapeStaticLightingMesh*> LandscapeLightingMeshes;
	TArray<FLandscapeStaticLightingTextureMapping*> LandscapeTextureMappings;

	TArray<class FStaticLightingGlobalVolumeMapping*> VolumeMappings;
	TArray<class FLandscapeStaticLightingGlobalVolumeMapping*> LandscapeVolumeMappings;

	// materials
	TArray<UMaterialInterface*> Materials;
	TMap<UMaterialInterface*, FLightmassMaterialExportSettings> MaterialExportSettings;
	TMap<UMaterialInterface*, FMaterialExportDataEntry> MaterialExportData;

	/** Exporting progress bar maximum value. */
	int32		TotalProgress;
	/** Exporting progress bar current value. */
	int32		CurrentProgress;

	/** The material renderers */
	FLightmassMaterialRenderer MaterialRenderer;

	/** The world we are exporting from */
	UWorld* World;

	/** The ligthing context we are exporting for, used to acquire some mappings & GUIDs that are contextual  */
	const FStaticLightingBuildContext&	LightingContext;

	/** Friends */
	friend class FBSPSurfaceStaticLighting;
	friend class FStaticMeshStaticLightingMesh;
	friend class FStaticMeshStaticLightingTextureMapping;
	friend class FLightmassProcessor;
	friend class FLandscapeStaticLightingMesh;
	friend class FLandscapeStaticLightingTextureMapping;
	friend class FStaticLightingGlobalVolumeMapping;
	friend class FLandscapeStaticLightingGlobalVolumeMapping;
};

/** Lightmass Importer class */
class FLightmassImporter
{
};

/** Thread-safe single-linked list (lock-free). */
template<typename ElementType>
class TListThreadSafe
{	public:

	/** Initialization constructor. */
	TListThreadSafe():
		FirstElement(NULL)
	{}

	/**
	 * Adds an element to the list.
	 * @param Element	Newly allocated and initialized list element to add.
	 */
	void AddElement(TList<ElementType>* Element)
	{
		// Link the element at the beginning of the list.
		TList<ElementType>* LocalFirstElement;
		do 
		{
			LocalFirstElement = FirstElement;
			Element->Next = LocalFirstElement;
		}
		while(FPlatformAtomics::InterlockedCompareExchangePointer((void**)&FirstElement,Element,LocalFirstElement) != LocalFirstElement);
	}

	/**
	 * Clears the list and returns the elements.
	 * @return	List of all current elements. The original list is cleared. Make sure to delete all elements when you're done with them!
	 */
	TList<ElementType>* ExtractAll()
	{
		// Atomically read the complete list and clear the shared head pointer.
		TList<ElementType>* LocalFirstElement;
		do
		{
			LocalFirstElement = FirstElement;
		}
		while(FPlatformAtomics::InterlockedCompareExchangePointer((void**)&FirstElement,NULL,LocalFirstElement) != LocalFirstElement);
		return LocalFirstElement;
	}

	/**
	 *	Clears the list.
	 */
	void Clear()
	{
		while (FirstElement)
		{
			// Atomically read the complete list and clear the shared head pointer.
			TList<ElementType>* Element = ExtractAll();

			// Delete all elements in the local list.
			while (Element)
			{
				TList<ElementType>* NextElement = Element->Next;
				delete Element;
				Element = NextElement;
			};
		};
	}

private:

	TList<ElementType>* FirstElement;
};

/** Stores the data for a visibility cell imported from Lightmass before compression. */
class FUncompressedPrecomputedVisibilityCell
{
public:
	FBox Bounds;
	/** Precomputed visibility data, the bits are indexed by VisibilityId of a primitive component. */
	TArray<uint8> VisibilityData;
};

class FLightmassAlertMessage
{
public:
	FGuid ObjectId;
	FString MessageText;
	int32 Type;
	int32 Severity;
};

/** Lightmass Processor class */
class FLightmassProcessor
{
public:
	/** 
	 * Constructor
	 * 
	 * @param bInDumpBinaryResults true if it should dump out raw binary lighting data to disk
	 */
	FLightmassProcessor(const FStaticLightingSystem& InSystem, bool bInDumpBinaryResults, bool bInOnlyBuildVisibility);

	~FLightmassProcessor();

	/** Retrieve an exporter for the given channel name */
	FLightmassExporter* GetLightmassExporter();

	/** Is the connection to Swarm valid? */
	bool IsSwarmConnectionIsValid() const
	{
		return bSwarmConnectionIsValid;
	}

	void SetImportCompletedMappingsImmediately(bool bInImportCompletedMappingsImmediately)
	{
		bImportCompletedMappingsImmediately = bInImportCompletedMappingsImmediately;
	}
	
	void SetVolumetricLightMapImportMode(bool bUseVLMCellGrid);

	/** Exports everything but the materials */
	void InitiateExport();
	/** Exports some materials, amortized, returning true if completely done */
	bool ExecuteAmortizedMaterialExport();

	void IssueStaticShadowDepthMapTask(const ULightComponent* Light, int32 EstimatedCost);

	/** Starts the asynchronous lightmass process */
	bool BeginRun();

	/** Checks to see if processing is finished or canceled. If so, it will return true */
	bool Update();

	/** Runs all the synchronous import and process mappings code */
	bool CompleteRun();

	/** True if the build has completed without errors and without being canceled */
	bool IsProcessingCompletedSuccessfully() const;

	/** The percentage of lightmass tasks which are complete (0-100) */
	int32 GetAsyncPercentDone() const;
	float GetAmortizedExportPercentDone() const;

	/** Defined the period during which Job APIs can be used, including opening and using Job Channels */
	bool	OpenJob();
	bool	CloseJob();

	FString GetMappingFileExtension(const FStaticLightingMapping* InMapping);

	/** Returns the Lightmass statistics. */
	const FLightmassStatistics& GetStatistics() const
	{
		return Statistics;
	}

	/**
	 * Import the mapping specified by a Guid.
	 *	@param	MappingGuid				Guid that identifies a mapping
	 *	@param	bProcessImmediately		If true, immediately process the mapping
	 *									If false, store it off for later processing
	 **/
	void	ImportMapping( const FGuid& MappingGuid, bool bProcessImmediately );

	/**
	 * Process the mapping specified by a Guid.
	 * @param MappingGuid	Guid that identifies a mapping
	 **/
	void	ProcessMapping( const FGuid& MappingGuid );

	/**
	 * Process any available mappings.
	 **/
	void	ProcessAvailableMappings();

protected:

	friend struct FDeferredMappingsBundle;
	
	enum StaticLightingType
	{
		SLT_Texture		// FStaticLightingTextureMapping
	};

	struct FTextureMappingImportHelper;

	/**
	 *	Helper struct for importing mappings
	 */
	struct FMappingImportHelper
	{
		/** The type of lighting mapping */
		StaticLightingType	Type;
		/** The mapping guid read in */
		FGuid				MappingGuid;
		/** The owner guid for this mapping */
		FGuid				OwnerGuid;

		/** The execution time this mapping took */
		double				ExecutionTime;
		/** Whether the mapping has been processed yet */
		bool				bProcessed;

		FMappingImportHelper()
		{
			Type = SLT_Texture;
			ExecutionTime = 0.0;
			bProcessed = false;
		}

		FMappingImportHelper(const FMappingImportHelper& InHelper)
		{
			Type = InHelper.Type;
			MappingGuid = InHelper.MappingGuid;
			ExecutionTime = InHelper.ExecutionTime;
			bProcessed = InHelper.bProcessed;
		}

		virtual ~FMappingImportHelper() = default;

		virtual FTextureMappingImportHelper* GetTextureMappingHelper()
		{
			return NULL;
		}

		virtual void Serialize(FArchive& Ar);
	};

	/**
	 *	Helper struct for importing texture mappings
	 */
	struct FTextureMappingImportHelper : public FMappingImportHelper
	{
		/** The texture mapping being imported */
		TRefCountPtr<FStaticLightingTextureMapping> TextureMapping;
		/** The imported quantized lightmap data */
		FQuantizedLightmapData* QuantizedData;
		/** The percentage of unmapped texels */
		float UnmappedTexelsPercentage;
		/** Number of shadow maps to import */
		int32 NumShadowMaps;
		int32 NumSignedDistanceFieldShadowMaps;
		TMap<ULightComponent*,FShadowMapData2D*> ShadowMapData;

		FTextureMappingImportHelper()
		{
			TextureMapping = NULL;
			QuantizedData = NULL;
			UnmappedTexelsPercentage = 0.0f;
			NumShadowMaps = 0;
			NumSignedDistanceFieldShadowMaps = 0;
			Type = SLT_Texture;
		}

		FTextureMappingImportHelper(const FTextureMappingImportHelper& InHelper)
		:	FMappingImportHelper(InHelper)
		{
			TextureMapping = InHelper.TextureMapping;
			QuantizedData = InHelper.QuantizedData;
			UnmappedTexelsPercentage = InHelper.UnmappedTexelsPercentage;
			NumShadowMaps = InHelper.NumShadowMaps;
			NumSignedDistanceFieldShadowMaps = InHelper.NumSignedDistanceFieldShadowMaps;
		}

		virtual ~FTextureMappingImportHelper() = default;

		virtual FTextureMappingImportHelper* GetTextureMappingHelper()
		{
			return this;
		}

		virtual void Serialize(FArchive& Ar) override;

		friend FArchive& operator<<(FArchive& Ar, FTextureMappingImportHelper& Helper)
		{
			Helper.Serialize(Ar);
			return Ar;
		}

	};

	/**
	 *	Retrieve the light for the given Guid
	 *
	 *	@param	LightGuid			The guid of the light we are looking for
	 *
	 *	@return	ULightComponent*	The corresponding light component.
	 *								NULL if not found.
	 */
	ULightComponent* FindLight(const FGuid& LightGuid);

	/**
	 *	Retrieve the static mehs for the given Guid
	 *
	 *	@param	Guid				The guid of the static mesh we are looking for
	 *
	 *	@return	UStaticMesh*		The corresponding static mesh.
	 *								NULL if not found.
	 */
	UStaticMesh* FindStaticMesh(FGuid& Guid);

	ULevel* FindLevel(const FGuid& Guid);

	/**
	 *	Import light map data from the given channel.
	 *
	 *	@param	Channel				The channel to import from.
	 *	@param	QuantizedData		The quantized lightmap data to fill in
	 *	@param	UncompressedSize	Size the data will be after uncompressing it (if compressed)
	 *	@param	CompressedSize		Size of the source data if compressed
	 *
	 *	@return	bool				true if successful, false otherwise.
	 */
	bool ImportLightMapData2DData(int32 Channel, FQuantizedLightmapData* QuantizedData, int32 UncompressedSize, int32 CompressedSize);

	/**
	 *	Import signed distance field shadow map data from the given channel.
	 *
	 *	@param	Channel				The channel to import from.
	 *	@param	OutShadowMapData	The light component --> shadow map data mapping that should be filled in.
	 *	@param	ShadowMapCount		Number of shadow maps to import
	 *
	 *	@return	UBOOL				TRUE if successful, FALSE otherwise.
	 */
	bool ImportSignedDistanceFieldShadowMapData2D(int32 Channel, TMap<ULightComponent*,FShadowMapData2D*>& OutShadowMapData, int32 ShadowMapCount);

	/**
	 *	Import a complete texture mapping....
	 *
	 *	@param	Channel			The channel to import from.
	 *	@param	TMImport		The texture mapping information that will be imported.
	 *
	 *	@return	bool			true if successful, false otherwise.
	 */
	bool ImportTextureMapping(int32 Channel, FTextureMappingImportHelper& TMImport);

	//
	FLightmassExporter* Exporter;
	FLightmassImporter* Importer;
	const FStaticLightingSystem& System;
	struct FDeferredMappingsBundle* DeferredMappings = nullptr;

	NSwarm::FSwarmInterface&	Swarm;
	bool						bSwarmConnectionIsValid;
	/** Whether lightmass has completed the job successfully. */
	bool						bProcessingSuccessful;
	/** Whether lightmass has completed the job with a failure. */
	bool						bProcessingFailed;
	/** Whether lightmass has received a quit message from Swarm. */
	bool						bQuitReceived;
	/** Number of completed tasks, as reported from Swarm. */
	volatile int32				NumCompletedTasks;
	/** Number of completed volumetric lightmap tasks, for chasing down UE-80351. */
	volatile int32				NumCompletedVolumetricLightmapTasks = 0;
	/** Whether Lightmass is currently running. */
	bool						bRunningLightmass;
	/** Lightmass statistics*/
	FLightmassStatistics		Statistics;

	TMap< FString, FText > Messages;

	/** If true, only visibility will be rebuilt. */
	bool bOnlyBuildVisibility;
	/** If true, this will dump out raw binary lighting data to disk */
	bool bDumpBinaryResults;
	/** If true, and in Deterministic mode, mappings will be imported but not processed as they are completed... */
	bool bImportCompletedMappingsImmediately;

	/** The index of the next mapping to process when available. */
	int32 MappingToProcessIndex;
	/** The number of available mappings to process before yielding back to the importing. */
	static int32 MaxProcessAvailableCount;

	/** Imported visibility cells, one array per visibility task. */
	TArray<TArray<FUncompressedPrecomputedVisibilityCell> > CompletedPrecomputedVisibilityCells;

	/** BSP mappings that are not completed by Lightmass yet. */
	TMap<FGuid, FBSPSurfaceStaticLighting*>					PendingBSPMappings;
	/** Texture mappings that are not completed by Lightmass yet. */
	TMap<FGuid, FStaticMeshStaticLightingTextureMapping*>	PendingTextureMappings;
	/** Landscape mappings that are not completed by Lightmass yet. */
	TMap<FGuid, FLandscapeStaticLightingTextureMapping*>	PendingLandscapeMappings;


	/** Mappings that are completed by Lightmass. */
	TListThreadSafe<FGuid>									CompletedMappingTasks;

	/** Positive if the volume sample task is complete. */
	static volatile int32										VolumeSampleTaskCompleted;

	/** List of completed visibility tasks. */
	TListThreadSafe<FGuid>									CompletedVisibilityTasks;

	TListThreadSafe<FGuid>									CompletedVolumetricLightmapTasks;

	/** Positive if the mesh area light data task is complete. */
	static volatile int32										MeshAreaLightDataTaskCompleted;

	/** Positive if the volume distance field task is complete. */
	static volatile int32										VolumeDistanceFieldTaskCompleted;

	/** Mappings that have been imported but not processed. */
	TMap<FGuid, FMappingImportHelper*>						ImportedMappings;
	
	/** Guid of the mapping that is being debugged. */
	FGuid DebugMappingGuid;
	
	int32 NumTotalSwarmTasks;

	/** Must cache off stats due to async */
	double LightmassStartTime;

	/** Must be acquired before read / writing SwarmCallbackMessages */
	FCriticalSection SwarmCallbackMessagesSection;

	/** Queue of messages from the swarm callback, to be processed by the main thread */
	TArray<FLightmassAlertMessage> SwarmCallbackMessages;
	
	FString DeferredMappingsDirectory;
	bool bSplitToVLMCellGrid = false;

	/**
	 *	Import all mappings that have been completed so far.
	 *
	 *	@param	bProcessImmediately		If true, immediately process the mapping
	 *									If false, store it off for later processing
	 **/
	void	ImportMappings(bool bProcessImmediately);

	/** Imports volume lighting samples from Lightmass and adds them to the appropriate levels. */
	void	ImportVolumeSamples();
	
	void	ImportIrradianceTasks(bool& bGenerateSkyShadowing, TArray<struct FImportedVolumetricLightmapTaskData>& TaskDataArray);

	void	ImportVolumetricLightmap();

	/** Imports precomputed visibility */
	void	ImportPrecomputedVisibility();

	/** Processes the imported visibility, compresses it and applies it to the current level. */
	void	ApplyPrecomputedVisibility();

	/** Imports data from Lightmass about the mesh area lights generated for the scene, and creates AGeneratedMeshAreaLight's for them. */
	void	ImportMeshAreaLightData();

	/** Imports the volume distance field from Lightmass. */
	void	ImportVolumeDistanceFieldData();

	/** Determines whether the specified mapping is a texture mapping */
	bool	IsStaticLightingTextureMapping( const FGuid& MappingGuid );

	/** Gets the texture mapping for the specified GUID */
	FStaticLightingTextureMapping*	GetStaticLightingTextureMapping( const FGuid& MappingGuid );

	/** Deferred mappings handling */
	void DeferMapping(FTextureMappingImportHelper* ImportHelper);
	bool IsDeferredMapping(const FGuid& Guid);
	void ExportDeferredMappings();
	void ImportDeferredMappings();
	void ClearImportedDeferredMappings();

	/**
	 *	Import the texture mapping 
	 *	@param	TextureMapping			The mapping being imported.
	 *	@param	bProcessImmediately		If true, immediately process the mapping
	 *									If false, store it off for later processing
	 */
	void	ImportStaticLightingTextureMapping( const FGuid& MappingGuid, bool bProcessImmediately );

	void ImportStaticShadowDepthMap(ULightComponent* Light);

	/** Reads in a TArray from the given channel. */
	template<class T>
	void ReadArray(int32 Channel, TArray<T>& Array)
	{
		int32 ArrayNum = 0;
		Swarm.ReadChannel(Channel, &ArrayNum, sizeof(ArrayNum));
		if (ArrayNum > 0)
		{
			Array.Empty(ArrayNum);
			Array.AddZeroed(ArrayNum);
			Swarm.ReadChannel(Channel, Array.GetData(), Array.GetTypeSize() * ArrayNum);
		}
	}

	void ImportDebugOutputStruct(int32 Channel);

	void ProcessAlertMessages();

	/**
	 * Swarm callback function.
	 * @param CallbackMessage	Incoming message sent from Swarm
	 * @param CallbackData		Type-casted pointer to the FLightmassProcessor
	 **/
	static void SwarmCallback( NSwarm::FMessage* CallbackMessage, void* CallbackData );
};
