// Copyright Epic Games, Inc. All Rights Reserved.

#include "Lightmass/LightmassLandscapeRender.h"
#include "RHI.h"
#include "RenderingThread.h"
#include "RenderResource.h"
#include "VertexFactory.h"
#include "PackedNormal.h"
#include "LandscapeLight.h"
#include "ShowFlags.h"
#include "HitProxies.h"
#include "RHIStaticStates.h"
#include "SceneView.h"
#include "PrimitiveUniformShaderParameters.h"
#include "LocalVertexFactory.h"
#include "CanvasRender.h"
#include "MeshBatch.h"
#include "MeshBuilderOneFrameResources.h"

#include "LandscapeProxy.h"
#include "LandscapeInfo.h"
#include "RendererInterface.h"
#include "LandscapeComponent.h"
#include "EngineModule.h"
#include "LandscapeEdit.h"
#include "DynamicMeshBuilder.h"
#include "MeshPassProcessor.h"
#include "UnrealClient.h"

void RenderLandscapeMaterialForLightmass(const FLandscapeStaticLightingMesh* LandscapeMesh, FMaterialRenderProxy* MaterialProxy, const FRenderTarget* RenderTarget)
{
	const ULandscapeComponent* LandscapeComponent = CastChecked<ULandscapeComponent>(LandscapeMesh->Component);

	const int32 SubsectionSizeQuads = LandscapeComponent->SubsectionSizeQuads;
	const int32 NumSubsections = LandscapeComponent->NumSubsections;
	const int32 ComponentSizeQuads = LandscapeComponent->ComponentSizeQuads;

	int32 PatchExpandCountX = 0;
	int32 PatchExpandCountY = 0;
	int32 DesiredSize = 1;
	const float LightMapOffset = 0.0f;
	const float LightMapRes = LandscapeComponent->StaticLightingResolution > 0.f ? LandscapeComponent->StaticLightingResolution : LandscapeComponent->GetLandscapeProxy()->StaticLightingResolution;
	const int32 LightingLOD = LandscapeComponent->GetLandscapeProxy()->StaticLightingLOD;
	const float LightMapRatio = ::GetTerrainExpandPatchCount(LightMapRes, PatchExpandCountX, PatchExpandCountY, ComponentSizeQuads, (NumSubsections * (SubsectionSizeQuads + 1)), DesiredSize, LightingLOD);

	const FVector2D PatchExpandOffset = FVector2D((float)PatchExpandCountX / (ComponentSizeQuads + 2 * PatchExpandCountX), (float)PatchExpandCountY / (ComponentSizeQuads + 2 * PatchExpandCountY)) * FVector2D(RenderTarget->GetSizeXY());
	const FVector2D PatchExpandScale = FVector2D((float)ComponentSizeQuads / (ComponentSizeQuads + 2 * PatchExpandCountX), (float)ComponentSizeQuads / (ComponentSizeQuads + 2 * PatchExpandCountY));

	TArray<FDynamicMeshVertex> Vertices;
	TArray<uint32> Indices;
	Vertices.Reserve(FMath::Square(NumSubsections) * 4);
	Indices.Reserve(FMath::Square(NumSubsections) * 6);

	const float fraction = 1.0f / NumSubsections;
	const FVector2D PositionScale = FVector2D(RenderTarget->GetSizeXY()) * fraction * PatchExpandScale;
	const FVector::FReal LayerScale = SubsectionSizeQuads;
	const float WeightmapSubsection = LandscapeComponent->WeightmapSubsectionOffset;
	const FVector2D WeightmapBias = FVector2D(LandscapeComponent->WeightmapScaleBias.Z, LandscapeComponent->WeightmapScaleBias.W);
	const FVector2D WeightmapScale = FVector2D(LandscapeComponent->WeightmapScaleBias.X, LandscapeComponent->WeightmapScaleBias.Y) * SubsectionSizeQuads;

	const int32 SubsectionX_Start = PatchExpandCountX > 0 ? -1 : 0;
	const int32 SubsectionX_End = NumSubsections + (PatchExpandCountX > 0 ? 1 : 0);
	const int32 SubsectionY_Start = PatchExpandCountY > 0 ? -1 : 0;
	const int32 SubsectionY_End = NumSubsections + (PatchExpandCountY > 0 ? 1 : 0);
	for (int32 SubsectionY = SubsectionY_Start; SubsectionY < SubsectionY_End; ++SubsectionY)
	{
		for (int32 SubsectionX = SubsectionX_Start; SubsectionX < SubsectionX_End; ++SubsectionX)
		{
			const FIntPoint UVSubsection = FIntPoint((SubsectionX >= 0 ? SubsectionX : 0),
			                                         (SubsectionY >= 0 ? SubsectionY : 0));
			const FVector2D UVScale = FVector2D((SubsectionX >= 0 && SubsectionX < NumSubsections ? 1 : 0),
			                                    (SubsectionY >= 0 && SubsectionY < NumSubsections ? 1 : 0));

			const FVector2D BasePosition = PatchExpandOffset + FVector2D(SubsectionX, SubsectionY) * PositionScale;
			const FVector2D BaseLayerCoords = FVector2D(LandscapeComponent->SectionBaseX, LandscapeComponent->SectionBaseY) + FVector2D(UVSubsection) * LayerScale;
			const FVector2f BaseWeightmapCoords = FVector2f(WeightmapBias) + FVector2f(UVSubsection) * WeightmapSubsection;	// LWC_TODO: Precision loss

			int32 Index = Vertices.Add(FDynamicMeshVertex(FVector3f(FVector2f(BasePosition) /*FVector2D(0, 0) * PositionScale*/, 0), FVector3f(FVector2f(BaseLayerCoords) /*FVector2D(0, 0) * UVScale * LayerScale*/, 0), BaseWeightmapCoords /*FVector2D(0, 0) * UVScale * WeightmapScale*/));
			verifySlow(   Vertices.Add(FDynamicMeshVertex(FVector3f(FVector2f(BasePosition + FVector2D(1, 0) * PositionScale),   0), FVector3f(FVector2f(BaseLayerCoords + FVector2D(1, 0) * UVScale * LayerScale),   0), BaseWeightmapCoords + FVector2f(1, 0) * FVector2f(UVScale * WeightmapScale))) == Index + 1);	// LWC_TODO: Precision loss
			verifySlow(   Vertices.Add(FDynamicMeshVertex(FVector3f(FVector2f(BasePosition + FVector2D(0, 1) * PositionScale),   0), FVector3f(FVector2f(BaseLayerCoords + FVector2D(0, 1) * UVScale * LayerScale),   0), BaseWeightmapCoords + FVector2f(0, 1) * FVector2f(UVScale * WeightmapScale))) == Index + 2);	// LWC_TODO: Precision loss
			verifySlow(   Vertices.Add(FDynamicMeshVertex(FVector3f(FVector2f(BasePosition + FVector2D(1, 1) * PositionScale),   0), FVector3f(FVector2f(BaseLayerCoords + FVector2D(1, 1) * UVScale * LayerScale),   0), BaseWeightmapCoords + FVector2f(1, 1) * FVector2f(UVScale * WeightmapScale))) == Index + 3);	// LWC_TODO: Precision loss

			checkSlow(Index + 3 <= MAX_uint16);
			Indices.Add(Index);
			Indices.Add(Index + 3);
			Indices.Add(Index + 1);
			Indices.Add(Index);
			Indices.Add(Index + 2);
			Indices.Add(Index + 3);
		}
	}

	FSceneViewFamily ViewFamily(FSceneViewFamily::ConstructionValues(
		RenderTarget,
		NULL,
		FEngineShowFlags(ESFIM_Game))
		.SetTime(FGameTime()));

	FDynamicMeshBuilder DynamicMeshBuilder(ViewFamily.GetFeatureLevel(), 4, 0, true);
	DynamicMeshBuilder.AddVertices(Vertices);
	DynamicMeshBuilder.AddTriangles(Indices);

	const FIntRect ViewRect(FIntPoint(0, 0), RenderTarget->GetSizeXY());

	// make a temporary view
	FSceneViewInitOptions ViewInitOptions;
	ViewInitOptions.ViewFamily = &ViewFamily;
	ViewInitOptions.SetViewRectangle(ViewRect);
	ViewInitOptions.ViewOrigin = FVector::ZeroVector;
	ViewInitOptions.ViewRotationMatrix = FMatrix::Identity;
	ViewInitOptions.ProjectionMatrix = FCanvas::CalcBaseTransform2D(RenderTarget->GetSizeXY().X, RenderTarget->GetSizeXY().Y);
	ViewInitOptions.BackgroundColor = FLinearColor::Black;
	ViewInitOptions.OverlayColor = FLinearColor::White;

	ENQUEUE_RENDER_COMMAND(CanvasFlushSetupCommand)(
		[RenderTarget, &DynamicMeshBuilder, ViewInitOptions, MaterialProxy](FRHICommandListImmediate& RHICmdList)
		{
			FMeshBatch Mesh;
			FMeshBuilderOneFrameResources OneFrameResource;
			DynamicMeshBuilder.GetMeshElement(FMatrix::Identity, MaterialProxy, SDPG_Foreground, true, false, 0, OneFrameResource, Mesh);

			//SCOPED_DRAW_EVENT(RHICmdList, CanvasFlush);
			
			if (OneFrameResource.IsValidForRendering())
			{
				const FIntRect RTViewRect = FIntRect(0, 0, RenderTarget->GetRenderTargetTexture()->GetSizeX(), RenderTarget->GetRenderTargetTexture()->GetSizeY());

				FRDGBuilder GraphBuilder(RHICmdList, RDG_EVENT_NAME("LightmassLandscapeMaterial"));
				FSceneView View(ViewInitOptions);

				FMeshPassProcessorRenderState DrawRenderState;

				// disable depth test & writes
				DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Always>::GetRHI());

				//SCOPED_DRAW_EVENT(RHICmdList, RenderLandscapeMaterialToTexture);
				FCanvasRenderContext RenderContext(GraphBuilder, RenderTarget, RTViewRect, FIntRect(0, 0, 0, 0), false);
				GetRendererModule().DrawTileMesh(RenderContext, DrawRenderState, View, Mesh, false, FHitProxyId());

				GraphBuilder.Execute();
			}
		});
	FlushRenderingCommands();
}

void GetLandscapeOpacityData(const FLandscapeStaticLightingMesh* LandscapeMesh, int32& InOutSizeX, int32& InOutSizeY, TArray<FFloat16Color>& OutMaterialSamples)
{
	const ULandscapeComponent* LandscapeComponent = CastChecked<ULandscapeComponent>(LandscapeMesh->Component);

	const int32 SubsectionSizeQuads = LandscapeComponent->SubsectionSizeQuads;
	const int32 NumSubsections = LandscapeComponent->NumSubsections;
	const int32 ComponentSizeQuads = LandscapeComponent->ComponentSizeQuads;

	int32 PatchExpandCountX = 0;
	int32 PatchExpandCountY = 0;
	int32 DesiredSize = 1;
	const float LightMapOffset = 0.0f;
	const float LightMapRes = LandscapeComponent->StaticLightingResolution > 0.f ? LandscapeComponent->StaticLightingResolution : LandscapeComponent->GetLandscapeProxy()->StaticLightingResolution;
	const int32 LightingLOD = LandscapeComponent->GetLandscapeProxy()->StaticLightingLOD;
	const float LightMapRatio = ::GetTerrainExpandPatchCount(LightMapRes, PatchExpandCountX, PatchExpandCountY, ComponentSizeQuads, (NumSubsections * (SubsectionSizeQuads + 1)), DesiredSize, LightingLOD);

	ULandscapeInfo* const LandscapeInfo = LandscapeComponent->GetLandscapeInfo();
	check(LandscapeInfo);

	FLandscapeEditDataInterface DataInterface = FLandscapeEditDataInterface(LandscapeInfo);
	const int32 X1 = LandscapeComponent->SectionBaseX - PatchExpandCountX;
	const int32 X2 = LandscapeComponent->SectionBaseX + ComponentSizeQuads + PatchExpandCountX + 1;
	const int32 Y1 = LandscapeComponent->SectionBaseY - PatchExpandCountY;
	const int32 Y2 = LandscapeComponent->SectionBaseY + ComponentSizeQuads + PatchExpandCountY + 1;
	const int32 XSize = X2 - X1 + 1;
	const int32 YSize = Y2 - Y1 + 1;

	TArray<uint8> Data;
	Data.AddUninitialized(YSize * XSize);
	FMemory::Memset(Data.GetData(), 255, Data.Num());

	DataInterface.GetWeightDataFast(ALandscapeProxy::VisibilityLayer, X1, Y1, X2, Y2, Data.GetData(), 0);

	const int32 BaseComponentX = LandscapeComponent->SectionBaseX / ComponentSizeQuads;
	const int32 BaseComponentY = LandscapeComponent->SectionBaseY / ComponentSizeQuads;
	for (int32 ComponentY = BaseComponentY - 1; ComponentY <= BaseComponentY + 1; ++ComponentY)
	{
		for (int32 ComponentX = BaseComponentX - 1; ComponentX <= BaseComponentX + 1; ++ComponentX)
		{
			if (ComponentX == 0 && ComponentY == 0)
			{
				continue;
			}

			ULandscapeComponent* Component = LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(ComponentX, ComponentY));
			if (Component)
			{
				const TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapLayerAllocations = Component->GetWeightmapLayerAllocations();

				if (ComponentWeightmapLayerAllocations.ContainsByPredicate([](const FWeightmapLayerAllocationInfo& Allocation) { return Allocation.LayerInfo == ALandscapeProxy::VisibilityLayer; }))
				{
					// filled by GetWeightDataFast()
				}
				else
				{
					// fill with 0 (fully opaque)
					// we don't worry about the border between components, we assume the value is identical on both components (it should be anyway)
					const int32 X_Start = FMath::Clamp(ComponentX  * ComponentSizeQuads + PatchExpandCountX                         , 0, XSize - 1);
					const int32 X_End   = FMath::Clamp(ComponentX  * ComponentSizeQuads + PatchExpandCountX + ComponentSizeQuads + 1, 0, XSize - 1);
					const int32 Y_Start = FMath::Clamp(ComponentY  * ComponentSizeQuads + PatchExpandCountX                         , 0, YSize - 1);
					const int32 Y_End   = FMath::Clamp(ComponentY  * ComponentSizeQuads + PatchExpandCountX + ComponentSizeQuads + 1, 0, YSize - 1);
					for (int32 Y = Y_Start; Y < Y_End; ++Y)
					{
						uint8* DataPtr = &Data[Y * XSize + X_Start];
						FMemory::Memset(DataPtr, 0, X_End - X_Start);
					}
				}
			}
			else
			{
				// filled with 255 (hole) by the Memset above.
				// not done here due to the complexity of handling the shared border between existing / non-existing components
				// it would be better if we could remove the triangles for non-existent components from the expanded landscape lighting mesh, but we don't yet do that
			}
		}
	}

	// Scale up the hole map to compensate for lightmass using point-sampling
	static const int32 ScaleFactor = 3;
	InOutSizeX = (XSize - 1) * ScaleFactor;
	InOutSizeY = (YSize - 1) * ScaleFactor;

	OutMaterialSamples.Empty(InOutSizeX * InOutSizeY);
	for (int32 Y = 0; Y < InOutSizeY; ++Y)
	{
		for (int32 X = 0; X < InOutSizeX; ++X)
		{
			// Adding a half-pixel offset to compensate for lightmass using point-sampling
			const FVector2D SampleUV = FVector2D(X + 0.5f, Y + 0.5f) / ScaleFactor;
			const int32 X0 = FMath::FloorToInt(SampleUV.X);
			const float Xf = SampleUV.X - X0;
			const int32 Y0 = FMath::FloorToInt(SampleUV.Y);
			const float Yf = SampleUV.Y - Y0;
			const uint8 Value = FMath::BiLerp(
				Data[FMath::Clamp(Y0    , 0, YSize - 1) * XSize + FMath::Clamp(X0    , 0, XSize - 1)],
				Data[FMath::Clamp(Y0    , 0, YSize - 1) * XSize + FMath::Clamp(X0 + 1, 0, XSize - 1)],
				Data[FMath::Clamp(Y0 + 1, 0, YSize - 1) * XSize + FMath::Clamp(X0    , 0, XSize - 1)],
				Data[FMath::Clamp(Y0 + 1, 0, YSize - 1) * XSize + FMath::Clamp(X0 + 1, 0, XSize - 1)],
				Xf, Yf);
			const float Opacity = (255 - Value) / 255.0f;
			OutMaterialSamples.Add(FLinearColor(Opacity, Opacity, Opacity));
		}
	}
}
