// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FLandscapeStaticLightingMesh;
class FMaterialRenderProxy;
class FRenderTarget;

extern void RenderLandscapeMaterialForLightmass(const FLandscapeStaticLightingMesh* LandscapeMesh, FMaterialRenderProxy* MaterialProxy, const FRenderTarget* RenderTarget);
extern void GetLandscapeOpacityData(const FLandscapeStaticLightingMesh* LandscapeMesh, int32& InOutSizeX, int32& InOutSizeY, TArray<FFloat16Color>& OutMaterialSamples);
