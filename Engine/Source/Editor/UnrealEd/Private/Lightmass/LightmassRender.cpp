// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LightmassRender.cpp: lightmass rendering-related implementation.
=============================================================================*/

#include "Lightmass/LightmassRender.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/Guid.h"
#include "RenderingThread.h"
#include "MaterialDomain.h"
#include "MaterialShared.h"
#include "Materials/Material.h"
#include "Materials/MaterialAttributeDefinitionMap.h"
#include "Materials/MaterialRenderProxy.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "Engine/TextureRenderTarget2D.h"
#include "MaterialExport.h"
#include "Misc/ConfigCacheIni.h"
#include "LandscapeLight.h"
#include "Lightmass/Lightmass.h"
#include "MaterialCompiler.h"
#include "LightMap.h"
#include "Lightmass/LightmassLandscapeRender.h"
#include "LandscapeMaterialInstanceConstant.h"
#include "EngineModule.h"
#include "EngineUtils.h"
#include "TextureResource.h"

DEFINE_LOG_CATEGORY_STATIC(LogLightmassRender, Error, All);

// FLightmassMaterialCompiler - A proxy compiler that overrides various compiler functions for potential problem expressions.
struct FLightmassMaterialCompiler : public FProxyMaterialCompiler
{
	FLightmassMaterialCompiler(FMaterialCompiler* InCompiler) :
		FProxyMaterialCompiler(InCompiler)
	{}

	// gets value stored by SetMaterialProperty()
	virtual EShaderFrequency GetCurrentShaderFrequency() const override
	{
		// not used by Lightmass
		return SF_Pixel;
	}

	virtual FMaterialShadingModelField GetMaterialShadingModels() const override
	{ 
		// not used by Lightmass
		return MSM_MAX;
	}

	virtual FMaterialShadingModelField GetCompiledShadingModels() const override
	{
		// not used by Lightmass
		return MSM_MAX;
	}

	virtual EMaterialValueType GetParameterType(int32 Index) const override
	{
		return MCT_Unknown;
	}

	virtual FMaterialUniformExpression* GetParameterUniformExpression(int32 Index) const override
	{
		return nullptr;
	}

	virtual int32 ParticleMacroUV() override
	{
		return Compiler->ParticleMacroUV();
	}

	virtual int32 ParticleRelativeTime() override
	{
		return Compiler->Constant(0.0f);
	}

	virtual int32 ParticleMotionBlurFade() override
	{
		return Compiler->Constant(1.0f);
	}

	virtual int32 ParticleRandom() override
	{
		return Compiler->Constant(0.0f);
	}

	virtual int32 ParticleDirection() override
	{
		return Compiler->Constant3(0.0f, 0.0f, 0.0f);
	}

	virtual int32 ParticleSpeed() override
	{
		return Compiler->Constant(0.0f);
	}
	
	virtual int32 ParticleSize() override
	{
		return Compiler->Constant2(0.0f,0.0f);
	}

	virtual int32 ParticleSpriteRotation() override
	{
		return Compiler->Constant2(0.0f, 0.0f);
	}

	virtual int32 WorldPosition(EWorldPositionIncludedOffsets WorldPositionIncludedOffsets) override
	{
		//UE_LOGF(LogLightmassRender, Log, "Lightmass material compiler has encountered WorldPosition... Forcing constant (0.0f,0.0f,0.0f).");
		return Compiler->Constant3(0.0f,0.0f,0.0f);
	}

	virtual int32 ObjectWorldPosition(EPositionOrigin OriginType) override
	{
		//UE_LOGF(LogLightmassRender, Log, "Lightmass material compiler has encountered ObjectWorldPosition... Forcing constant (0.0f,0.0f,0.0f).");
		return Compiler->Constant3(0.0f,0.0f,0.0f);
	}

	virtual int32 ObjectRadius() override
	{
		//UE_LOGF(LogLightmassRender, Log, "Lightmass material compiler has encountered ObjectRadius... Forcing constant 500.0f.");
		return Compiler->Constant(500);
	}

	virtual int32 ObjectBounds() override
	{
		//UE_LOGF(LogLightmassRender, Log, "Lightmass material compiler has encountered ObjectBounds... Forcing constant (0,0,0).");
		return Compiler->Constant3(0,0,0);
	}

	virtual int32 PreSkinnedLocalBounds(int32 OutputIndex) override
	{
		return Compiler->Constant3(0, 0, 0);
	}

	virtual int32 DistanceCullFade() override
	{
		return Compiler->Constant(1.0f);
	}

	virtual int32 ActorWorldPosition(EPositionOrigin OriginType) override
	{
		return Compiler->Constant3(0.0f,0.0f,0.0f);
	}

	virtual int32 CameraVector() override
	{
		//UE_LOGF(LogLightmassRender, Log, "Lightmass material compiler has encountered CameraVector... Forcing constant (0.0f,0.0f,1.0f).");
		return Compiler->Constant3(0.0f,0.0f,1.0f);
	}

	virtual int32 LightVector() override
	{
		//UE_LOGF(LogLightmassRender, Log, "Lightmass material compiler has encountered LightVector... Forcing constant (1.0f,0.0f,0.0f).");
		return Compiler->Constant3(1.0f,0.0f,0.0f);
	}

	virtual int32 ReflectionVector() override
	{
		//UE_LOGF(LogLightmassRender, Log, "Lightmass material compiler has encountered ReflectionVector... Forcing constant (0.0f,0.0f,-1.0f).");
		return Compiler->Constant3(0.0f,0.0f,-1.0f);
	}

	virtual int32 ReflectionAboutCustomWorldNormal(int32 CustomWorldNormal, int32 bNormalizeCustomWorldNormal) override
	{
		//UE_LOGF(LogLightmassRender, Log, "Lightmass material compiler has encountered ReflectionAboutCustomNormalVector... Forcing constant (0.0f,0.0f,-1.0f).");
		return Compiler->Constant3(0.0f,0.0f,-1.0f);
	}

	virtual int32 TransformVector(EMaterialCommonBasis SourceCoordBasis, EMaterialCommonBasis DestCoordBasis, FTransformParameters& Parameters, int32 A) override
	{
		//UE_LOGF(LogLightmassRender, Log, "Lightmass material compiler has encountered TransformVector... Passing thru source vector untouched.");
		return A;
	}

	virtual int32 TransformPosition(EMaterialCommonBasis SourceCoordBasis, EMaterialCommonBasis DestCoordBasis, FTransformParameters& Parameters, int32 A) override
	{
		//UE_LOGF(LogLightmassRender, Log, "Lightmass material compiler has encountered TransformPosition... Passing thru source vector untouched.");
		return A;
	}

	virtual int32 VertexColor() override
	{
		//UE_LOGF(LogLightmassRender, Log, "Lightmass material compiler has encountered VertexColor... Forcing constant (1.0f,1.0f,1.0f,1.0f).");
		return Compiler->Constant4(1.0f,1.0f,1.0f,1.0f);
	}

	virtual int32 PreSkinnedPosition() override
	{
		return Compiler->Constant3(0.f,0.f,0.f);
	}

	virtual int32 PreSkinnedNormal() override
	{
		return Compiler->Constant3(0.f,0.f,1.f);
	}

	virtual int32 VertexInterpolator(uint32 InterpolatorIndex) override
	{
		return Compiler->VertexInterpolator(InterpolatorIndex);
	}

	virtual int32 RealTime(bool bPeriodic, float Period) override
	{
		//UE_LOGF(LogLightmassRender, Log, "Lightmass material compiler has encountered RealTime... Forcing constant 0.0f.");
		return Compiler->Constant(0.0f);
	}

	virtual int32 GameTime(bool bPeriodic, float Period) override
	{
		//UE_LOGF(LogLightmassRender, Log, "Lightmass material compiler has encountered GameTime... Forcing constant 0.0f.");
		return Compiler->Constant(0.0f);
	}

	virtual int32 DecalColor() override
	{
		return Compiler->Constant4(1.0f, 1.0f, 1.0f, 1.0f);
	}

	virtual int32 DecalLifetimeOpacity() override
	{
		return Compiler->Constant(0.0f);
	}

	virtual int32 GIReplace(int32 Direct, int32 StaticIndirect, int32 DynamicIndirect) override { return StaticIndirect; }

	virtual EMaterialCompilerType GetCompilerType() const override { return EMaterialCompilerType::Lightmass; }

#if WITH_EDITOR
	virtual int32 MaterialBakingWorldPosition() override
	{
		return Compiler->MaterialBakingWorldPosition();
	}
#endif
};

/**
 * Class for rendering previews of material expressions in the material editor's linked object viewport.
 */
class FLightmassMaterialProxy : public FEditorMaterialResourceBase, public FMaterialRenderProxy
{
public:
	FLightmassMaterialProxy(): FEditorMaterialResourceBase(), FMaterialRenderProxy(TEXT("FLightmassMaterialProxy"))
	{
		SetQualityLevelProperties(GMaxRHIShaderPlatform);
	}

	/** Initializes the material proxy and kicks off async shader compiling. */
	void BeginCompiling(UMaterialInterface* InMaterialInterface, EMaterialProperty InPropertyToCompile, EMaterialShaderMapUsage::Type InUsage)
	{
		if (InMaterialInterface)
		{
			bSubstrateEnabled = Substrate::IsSubstrateEnabled() 
				&& InMaterialInterface->GetMaterial()
				&& InMaterialInterface->GetMaterial()->HasSubstrateFrontMaterialConnected(); // Only enable substrate export path for materials that have the front material connected, otherise use the hidden conversion / legacy path.
			MaterialInterface = InMaterialInterface;
			Material = MaterialInterface ? MaterialInterface->GetMaterial() : NULL;
			PropertyToCompile = InPropertyToCompile;
			Usage = InUsage;

			ReferencedTextures = MaterialInterface->GetDefaultTextures();
			ReferencedTextureCollections = MaterialInterface->GetReferencedTextureCollections();

			FMaterialResource* Resource = InMaterialInterface->GetMaterialResource(GMaxRHIShaderPlatform);
			if (Resource)
			{
				FMaterialShaderMapId ResourceId;
				Resource->BuildShaderMapId(ResourceId, nullptr);

				{
					TArray<FShaderType*> ShaderTypes;
					TArray<FVertexFactoryType*> VFTypes;
					TArray<const FShaderPipelineType*> ShaderPipelineTypes;
					GetDependentShaderAndVFTypes(ResourceId.LayoutParams, ShaderTypes, ShaderPipelineTypes, VFTypes);

					// Overwrite the shader map Id's dependencies with ones that came from the FMaterial actually being compiled (this)
					// This is necessary as we change FMaterial attributes like GetShadingModels(), which factor into the ShouldCache functions that determine dependent shader types
					ResourceId.SetShaderDependencies(ShaderTypes, ShaderPipelineTypes, VFTypes, GMaxRHIShaderPlatform);
				}

				// Override with a special usage so we won't re-use the shader map used by the material for rendering
				ResourceId.Usage = GetShaderMapUsage();
				CacheShaders(ResourceId);
			}
		}
	}

	/**
	 * Should the shader for this material with the given platform, shader type and vertex 
	 * factory type combination be compiled
	 *
	 * @param ShaderType	Which shader is being compiled
	 * @param VertexFactory	Which vertex factory is being compiled (can be NULL)
	 *
	 * @return true if the shader should be compiled
	 */
	virtual bool ShouldCache(const FShaderType* ShaderType, const FVertexFactoryType* VertexFactoryType) const override
	{
		if (VertexFactoryType == FindVertexFactoryType(FName(TEXT("FLocalVertexFactory"), FNAME_Find)))
		{
			// we only need the non-light-mapped, base pass, local vertex factory shaders for drawing an opaque Material Tile
			// @todo: Added a FindShaderType by fname or something"

			if(FCString::Stristr(ShaderType->GetName(), TEXT("BasePassVSFNoLightMapPolicy")))
			{
				return true;
			}
			else if(FCString::Stristr(ShaderType->GetName(), TEXT("Simple")))
			{
				return true;
			}
			else if(FCString::Stristr(ShaderType->GetName(), TEXT("BasePassPSFNoLightMapPolicy")))
			{
				return true;
			}
		}

		return false;
	}

	////////////////
	// FMaterialRenderProxy interface.
	virtual const FMaterial* GetMaterialNoFallback(ERHIFeatureLevel::Type InFeatureLevel) const override
	{
		if (GetRenderingThreadShaderMap())
		{
			return this;
		}
		return nullptr;
	}

	virtual const FMaterialRenderProxy* GetFallback(ERHIFeatureLevel::Type InFeatureLevel) const override
	{
		return UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
	}

	virtual bool GetParameterValue(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, FMaterialParameterValue& OutValue, const FMaterialRenderContext& Context) const override
	{
		return MaterialInterface->GetRenderProxy()->GetParameterValue(Type, ParameterInfo, OutValue, Context);
	}

	// Material properties.

	/** Entry point for compiling a specific material property.  This must call SetMaterialProperty. */
	virtual int32 CompilePropertyAndSetMaterialProperty(EMaterialProperty Property, FMaterialCompiler* Compiler, EShaderFrequency OverrideShaderFrequency, bool bUsePreviousFrameTime) const override
	{
		// needs to be called in this function!!
		Compiler->SetMaterialProperty(Property, OverrideShaderFrequency, bUsePreviousFrameTime);

		int32 Ret = CompilePropertyAndSetMaterialPropertyWithoutCast(Property, Compiler);

		return Compiler->ForceCast(Ret, FMaterialAttributeDefinitionMap::GetValueType(Property));
	}

	/** helper for CompilePropertyAndSetMaterialProperty() */
	int32 CompilePropertyAndSetMaterialPropertyWithoutCast(EMaterialProperty Property, FMaterialCompiler* Compiler) const
	{
		// MAKE SURE THIS MATCHES THE CHART IN WillFillData
		// 						  RETURNED VALUES (F16 'textures')
		// 	BLEND MODE  | DIFFUSE     | SPECULAR     | EMISSIVE    | NORMAL    | TRANSMISSIVE              |
		// 	------------+-------------+--------------+-------------+-----------+---------------------------|
		// 	Opaque      | Diffuse     | Spec,SpecPwr | Emissive    | Normal    | 0 (EMPTY)                 |
		// 	Masked      | Diffuse     | Spec,SpecPwr | Emissive    | Normal    | Opacity Mask              |
		// 	Translucent | 0 (EMPTY)   | 0 (EMPTY)    | Emissive    | 0 (EMPTY) | (Emsv | Diffuse)*Opacity  |
		// 	Additive    | 0 (EMPTY)   | 0 (EMPTY)    | Emissive    | 0 (EMPTY) | (Emsv | Diffuse)*Opacity  |
		// 	Modulative  | 0 (EMPTY)   | 0 (EMPTY)    | Emissive    | 0 (EMPTY) | Emsv | Diffuse            |
		// 	------------+-------------+--------------+-------------+-----------+---------------------------|

		const uint32 ForceCast_Exact_Replicate = MFCF_ForceCast | MFCF_ExactMatch | MFCF_ReplicateValue;
		const EMaterialProperty DiffuseInput = MP_BaseColor;

		if (bSubstrateEnabled)
		{
			uint8 BlendMode = static_cast<uint8>(MaterialInterface->GetBlendMode());
			ESubstrateMaterialExportContext SubstrateMaterialExportContext = IsOpaqueOrMaskedBlendMode(*MaterialInterface) ? ESubstrateMaterialExportContext::SMEC_Opaque : ESubstrateMaterialExportContext::SMEC_Translucent;

			if (Usage == EMaterialShaderMapUsage::LightmassExportDiffuse)
			{
				Compiler->SetSubstrateMaterialExportType(SME_BaseColorPostCoverage, SubstrateMaterialExportContext, BlendMode);
			}
			else if (Usage == EMaterialShaderMapUsage::LightmassExportNormal)
			{
				Compiler->SetSubstrateMaterialExportType(SME_Normal, SubstrateMaterialExportContext, BlendMode);
			}
			else if (Usage == EMaterialShaderMapUsage::LightmassExportOpacity)
			{
				Compiler->SetSubstrateMaterialExportType(SME_Transmittance, SubstrateMaterialExportContext, BlendMode);
			}
			else if (Usage == EMaterialShaderMapUsage::LightmassExportEmissive)
			{
				Compiler->SetSubstrateMaterialExportType(SME_Emissive, SubstrateMaterialExportContext, BlendMode);
			}
		}

		if( Property == MP_EmissiveColor )
		{
			UMaterial* ProxyMaterial = MaterialInterface->GetMaterial();
			bool bIsMaterialUnlit = MaterialInterface->GetShadingModels().IsUnlit();
			const bool bIsOpaque = IsOpaqueBlendMode(*MaterialInterface);
			const bool bIsMasked = IsMaskedBlendMode(*MaterialInterface);
			const bool bIsModulate = IsModulateBlendMode(*MaterialInterface);
			const bool bIsTranslucentOnly = IsTranslucentOnlyBlendMode(*MaterialInterface);
			const bool bIsAlphaHoldout = IsAlphaHoldoutBlendMode(*MaterialInterface);
			const bool bIsAdditive = IsAdditiveBlendMode(*MaterialInterface);
			const bool bIsAlphaComposite = IsAlphaCompositeBlendMode(*MaterialInterface);
			check(ProxyMaterial);
			FLightmassMaterialCompiler ProxyCompiler(Compiler);

			switch (PropertyToCompile)
			{
			case MP_EmissiveColor:
				// Emissive is ALWAYS returned...
				return Compiler->Max(MaterialInterface->CompileProperty(&ProxyCompiler,MP_EmissiveColor, ForceCast_Exact_Replicate), Compiler->Constant3(0, 0, 0));
			case MP_DiffuseColor:
				// Only return for Opaque and Masked...
				if (bIsOpaque || bIsMasked)
				{
					return Compiler->Saturate(MaterialInterface->CompileProperty(&ProxyCompiler, DiffuseInput, ForceCast_Exact_Replicate));
				}
				break;
			case MP_SpecularColor: 
				// Only return for Opaque and Masked...
				if (bIsOpaque || bIsMasked)
				{
					return Compiler->AppendVector(
						Compiler->Saturate(MaterialInterface->CompileProperty(&ProxyCompiler, MP_SpecularColor, ForceCast_Exact_Replicate)), 
						Compiler->Saturate(MaterialInterface->CompileProperty(&ProxyCompiler, MP_Roughness, MFCF_ForceCast)));
				}
				break;
			case MP_Normal:
				// Only return for Opaque and Masked...
				if (bIsOpaque || bIsMasked)
				{
					return MaterialInterface->CompileProperty(&ProxyCompiler, MP_Normal, ForceCast_Exact_Replicate);
				}
				break;
			
			case MP_Opacity:
				if (bIsMasked)
				{
					return MaterialInterface->CompileProperty(&ProxyCompiler, MP_OpacityMask);
				}
				else if (IsTranslucentBlendMode(*MaterialInterface) && ProxyMaterial->GetCastShadowAsMasked())
				{
					return MaterialInterface->CompileProperty(&ProxyCompiler, MP_Opacity);
				}
				else if (bIsModulate)
				{
					if (bIsMaterialUnlit)
					{
						return MaterialInterface->CompileProperty(Compiler, MP_EmissiveColor, ForceCast_Exact_Replicate);
					}
					else
					{
						return Compiler->Saturate(MaterialInterface->CompileProperty(Compiler, DiffuseInput, ForceCast_Exact_Replicate));
					}
				}
				else if (bIsTranslucentOnly || bIsAdditive || bIsAlphaComposite || bIsAlphaHoldout)
				{
					int32 ColoredOpacity = INDEX_NONE;
					if (bIsMaterialUnlit)
					{
						ColoredOpacity = MaterialInterface->CompileProperty(Compiler, MP_EmissiveColor, ForceCast_Exact_Replicate);
					}
					else
					{
						ColoredOpacity = Compiler->Saturate(MaterialInterface->CompileProperty(Compiler, DiffuseInput, ForceCast_Exact_Replicate));
					}
					return Compiler->Lerp(Compiler->Constant3(1.0f, 1.0f, 1.0f), ColoredOpacity, Compiler->Saturate(MaterialInterface->CompileProperty(&ProxyCompiler,MP_Opacity,MFCF_ForceCast)));
				}
				break;
			case MP_ShadingModel:
				return MaterialInterface->CompileProperty(&ProxyCompiler, MP_ShadingModel);
			case MP_FrontMaterial:
				if (bSubstrateEnabled)
				{
					// When using Substrate, material property always compile from material. 
					// We cannot use rediction so instead we instruct the compiler the type of data export we are looking for.
					return MaterialInterface->CompileProperty(&ProxyCompiler, MP_FrontMaterial);
				}
				else
				{
					return ProxyCompiler.SubstrateCreateAndRegisterNullMaterial();
				}
			default:
				return Compiler->Constant(1.0f);
			}
	
			return Compiler->Constant(0.0f);
		}
		else if (Property == MP_WorldPositionOffset || Property == MP_PixelDepthOffset || Property == MP_Displacement)
		{
			//This property MUST return 0 as a default or during the process of rendering textures out for lightmass to use, pixels will be off by 1.
			return Compiler->Constant(0.0f);
		}
		else if (Property >= MP_CustomizedUVs0 && Property <= MP_CustomizedUVs7)
		{
			// Pass through customized UVs
			return MaterialInterface->CompileProperty(Compiler, Property);
		}
		else if (Property == MP_ShadingModel)
		{
			return MaterialInterface->CompileProperty(Compiler, MP_ShadingModel); // useless with Substrate
		}
		// When using Substrate, we need to actually compile more root node inputs, 
		// and then we handle what needs to actually be exported from the shader code (see SUBSTRATE_MATERIAL_EXPORT_TYPE).
		else if (Property == MP_OpacityMask)
		{
			return MaterialInterface->CompileProperty(Compiler, MP_OpacityMask);
		}
		else if (Property == MP_SurfaceThickness)
		{
			return MaterialInterface->CompileProperty(Compiler, MP_SurfaceThickness);
		}
		else if (Property == MP_FrontMaterial)
		{
			if (bSubstrateEnabled)
			{
				return MaterialInterface->CompileProperty(Compiler, MP_FrontMaterial);
			}
			else
			{
				return Compiler->SubstrateCreateAndRegisterNullMaterial();
			}
		}
		else
		{
			return Compiler->Constant(1.0f);
		}
	}

	/** 
	 * Gets the shader map usage of the material, which will be included in the DDC key.
	 * This mechanism allows derived material classes to create different DDC keys with the same base material.
	 * For example lightmass exports diffuse and emissive, each of which requires a material resource with the same base material.
	 */
	virtual EMaterialShaderMapUsage::Type GetShaderMapUsage() const override { return Usage; }

	virtual FString GetMaterialUsageDescription() const override { return FString::Printf(TEXT("%s FLightmassMaterialRenderer"), MaterialInterface ? *MaterialInterface->GetName() : TEXT("NULL")); }
	
	virtual EMaterialDomain GetMaterialDomain() const override
	{
		if (Material)
		{
			return Material->MaterialDomain;
		}
		return MD_Surface;
	}
	virtual bool IsTwoSided() const override
	{
		if (MaterialInterface)
		{
			return MaterialInterface->IsTwoSided();
		}
		return false;
	}
	virtual bool IsThinSurface() const override
	{
		if (MaterialInterface)
		{
			return MaterialInterface->IsThinSurface();
		}
		return false;
	}
	virtual bool IsDitheredLODTransition() const override
	{
		if (MaterialInterface)
		{
			return MaterialInterface->IsDitheredLODTransition();
		}
		return false;
	}
	virtual bool IsLightFunction() const override
	{
		if (Material)
		{
			return (Material->MaterialDomain == MD_LightFunction);
		}
		return false;
	}
	virtual bool IsDeferredDecal() const override
	{
		return Material && Material->MaterialDomain == MD_DeferredDecal;
	}
	virtual bool IsUIMaterial() const override
	{
		return Material && Material->MaterialDomain == MD_UI;
	}
	virtual bool IsVolumetricPrimitive() const override
	{
		return Material && Material->MaterialDomain == MD_Volume;
	}
	virtual bool IsSpecialEngineMaterial() const override
	{
		if (Material)
		{
			return (Material->bUsedAsSpecialEngineMaterial == 1);
		}
		return false;
	}
	virtual bool IsWireframe() const override
	{
		if (Material)
		{
			return (Material->Wireframe == 1);
		}
		return false;
	}
	virtual bool IsMasked() const override									{ return false; }
	virtual enum EBlendMode GetBlendMode() const override					{ return BLEND_Opaque; }
	virtual enum ERefractionMode GetRefractionMode() const override			{ return Material ? (ERefractionMode)Material->RefractionMethod : RM_None; }
	virtual bool GetRootNodeOverridesDefaultRefraction()const override		{ return false; }
	virtual float GetOpacityMaskClipValue() const override					{ return 0.5f; }
	virtual bool GetCastDynamicShadowAsMasked() const override				{ return false; }
	virtual FString GetFriendlyName() const override { return FString::Printf(TEXT("FLightmassMaterialRenderer %s"), MaterialInterface ? *MaterialInterface->GetName() : TEXT("NULL")); }

	virtual bool IsShadingModelFromMaterialExpression() const override		{ return false; }
	virtual FMaterialShadingModelField GetShadingModels() const override	
	{ 
		// Substrate needs the real material shading model since the expressions access GetMaterialShadingModels() through the compiler to generate the substrate operators, 
		// and we do not want unlit materials when it is a Slab or a ShadingModel node.
		return Substrate::IsSubstrateEnabled() ? Material->GetShadingModels() : MSM_Unlit; 
	} 
	/**
	 * Should shaders compiled for this material be saved to disk?
	 */
	virtual bool IsPersistent() const override { return true; }

	virtual FGuid GetMaterialId() const override 
	{ 
		// Reuse the base material's Id
		// Normally this would cause a bug as the shader map would try to be shared by both, 
		// But FLightmassMaterialProxy::GetShaderMapUsage() allows this to work
		return Material->StateId;
	}

	virtual UMaterialInterface* GetMaterialInterface() const override
	{
		return MaterialInterface;
	}

	friend FArchive& operator<< ( FArchive& Ar, FLightmassMaterialProxy& V )
	{
		return Ar << V.MaterialInterface;
	}

	bool IsMaterialInputConnected(UMaterial* InMaterial, EMaterialProperty MaterialInput)
	{
		bool bConnected = false;
		UMaterialEditorOnlyData* MaterialEditorOnly = InMaterial->GetEditorOnlyData();

		if (bSubstrateEnabled)
		{
			// Material attribute do not override the FrontMaterial input
			bConnected = MaterialEditorOnly->FrontMaterial.Expression != nullptr;
		}
		else
		{
			switch (MaterialInput)
			{
			case MP_EmissiveColor:
				bConnected = MaterialEditorOnly->EmissiveColor.Expression != nullptr;
				break;
			case MP_DiffuseColor:
				bConnected = MaterialEditorOnly->BaseColor.Expression != nullptr;
				break;
			case MP_SpecularColor:
				bConnected = MaterialEditorOnly->Specular.Expression != nullptr;
				break;
			case MP_Normal:
				bConnected = MaterialEditorOnly->Normal.Expression != nullptr;
				break;
			case MP_Opacity:
				bConnected = MaterialEditorOnly->Opacity.Expression != nullptr;
				break;
			case MP_OpacityMask:
				bConnected = MaterialEditorOnly->OpacityMask.Expression != nullptr;
				break;
			default:
				break;
			}

			// Note: only checking to see whether the entire material attributes connection exists.  
			// This means materials using the material attributes input will export more attributes than is necessary.
			bConnected = InMaterial->bUseMaterialAttributes ? MaterialEditorOnly->MaterialAttributes.Expression != NULL : bConnected;
		}

		return bConnected;
	}

	/**
	 *	Checks if the configuration of the material proxy will generate a uniform
	 *	value across the sampling... (Ie, nothing is hooked to the property)
	 *
	 *	@param	OutUniformValue		The value that will be returned.
	 *
	 *	@return	bool				true if a single value would be generated.
	 *								false if not.
	 */
	bool WillGenerateUniformData(FFloat16Color& OutUniformValue)
	{
		// Pre-fill the value...
		OutUniformValue.R = 0.0f;
		OutUniformValue.G = 0.0f;
		OutUniformValue.B = 0.0f;
		OutUniformValue.A = 0.0f;

		check(Material);
		bool bExpressionIsNULL = false;

		if (bSubstrateEnabled)
		{
			const bool bIsOpaque = IsOpaqueBlendMode(*MaterialInterface);
			const bool bIsMasked = IsMaskedBlendMode(*MaterialInterface);

			switch (Usage)
			{
			case EMaterialShaderMapUsage::LightmassExportEmissive:
				bExpressionIsNULL = !IsMaterialInputConnected(Material, MP_FrontMaterial);
				break;
			case EMaterialShaderMapUsage::LightmassExportDiffuse:
				if (bIsOpaque || bIsMasked)
				{
					bExpressionIsNULL = !IsMaterialInputConnected(Material, MP_FrontMaterial);
				}
				break;
			case EMaterialShaderMapUsage::LightmassExportOpacity:
				if (!bIsOpaque)
				{
					bExpressionIsNULL = !IsMaterialInputConnected(Material, MP_FrontMaterial);
					OutUniformValue.A = 15.0f;
				}
				break;
			case EMaterialShaderMapUsage::LightmassExportNormal:
				if (bIsOpaque || bIsMasked)
				{
					bExpressionIsNULL = !IsMaterialInputConnected(Material, MP_FrontMaterial);
					OutUniformValue.B = 1.0f;	// Default normal is (0,0,1)
				}
				break;
			default:
				UE_LOGF(LogLightmassRender, Error, "WillGenerateUniformData - cannot export a requested property for %ls", *(Material->GetPathName()));
				break;
			}
		}
		else
		{
			const bool bIsOpaqueOrMasked = IsOpaqueOrMaskedBlendMode(*MaterialInterface);
			bool bIsMaterialUnlit = MaterialInterface->GetShadingModels().IsUnlit();
			EBlendMode BlendMode = MaterialInterface->GetBlendMode();

			switch (PropertyToCompile)
			{
			case MP_EmissiveColor:
				// Emissive is ALWAYS returned...
				bExpressionIsNULL = !IsMaterialInputConnected(Material, PropertyToCompile);
				break;
			case MP_DiffuseColor:
				// Only return for Opaque and Masked...
				if (bIsOpaqueOrMasked)
				{
					bExpressionIsNULL = !IsMaterialInputConnected(Material, PropertyToCompile);
				}
				break;
			case MP_SpecularColor: 
				// Only return for Opaque and Masked...
				if (bIsOpaqueOrMasked)
				{
					bExpressionIsNULL = !IsMaterialInputConnected(Material, PropertyToCompile);
					OutUniformValue.A = 15.0f;
				}
				break;
			case MP_Normal:
				// Only return for Opaque and Masked...
				if (bIsOpaqueOrMasked)
				{
					bExpressionIsNULL = !IsMaterialInputConnected(Material, PropertyToCompile);
					OutUniformValue.B = 1.0f;	// Default normal is (0,0,1)
				}
				break;
			case MP_Opacity:
				if (BlendMode == BLEND_Masked)
				{
					bExpressionIsNULL = !IsMaterialInputConnected(Material, MP_OpacityMask);
					OutUniformValue.R = 1.0f;
					OutUniformValue.G = 1.0f;
					OutUniformValue.B = 1.0f;
					OutUniformValue.A = 1.0f;
				}
				else
				if ((BlendMode == BLEND_Modulate) ||
					(BlendMode == BLEND_Translucent) || 
					(BlendMode == BLEND_Additive) ||
					(BlendMode == BLEND_AlphaComposite) ||
					(BlendMode == BLEND_AlphaHoldout))
				{
					bool bColorInputIsNULL = false;
					if (bIsMaterialUnlit)
					{
						bColorInputIsNULL = !IsMaterialInputConnected(Material, MP_EmissiveColor);
					}
					else
					{
						bColorInputIsNULL = !IsMaterialInputConnected(Material, MP_DiffuseColor);
					}
					if (BlendMode == BLEND_Translucent
						|| BlendMode == BLEND_Additive
						|| BlendMode == BLEND_AlphaComposite
						|| BlendMode == BLEND_AlphaHoldout)
					{
						bExpressionIsNULL = bColorInputIsNULL && !IsMaterialInputConnected(Material, PropertyToCompile);
					}
					else
					{
						bExpressionIsNULL = bColorInputIsNULL;
					}
				}
				break;
			}
		}

		return bExpressionIsNULL;
	}

	/**
	 *	Retrieves the desired render target format and size for the given property.
	 *	This will allow for overriding the format and/or size based on the material and property of interest.
	 *
	 *	@param	InMaterialProperty	The material property that is going to be captured in the render target.
	 *	@param	OutFormat			The format the render target should use.
	 *	@param	OutSizeX			The width to use for the render target.
	 *	@param	OutSizeY			The height to use for the render target.
	 *
	 *	@return	bool				true if data is good, false if not (do not create render target)
	 */
	bool GetRenderTargetFormatAndSize(EMaterialProperty InMaterialProperty, EPixelFormat& OutFormat, float SizeScale, int32& OutSizeX, int32& OutSizeY)
	{
		OutFormat = PF_FloatRGBA;

		int32 GlobalSize = 0;
		// For now, just look them up in the config file...
		if (InMaterialProperty == MP_DiffuseColor)
		{
			verify(GConfig->GetInt(TEXT("DevOptions.StaticLightingMaterial"), TEXT("DiffuseSampleSize"), GlobalSize, GLightmassIni));
		}
		else
		if (InMaterialProperty == MP_SpecularColor)
		{
			verify(GConfig->GetInt(TEXT("DevOptions.StaticLightingMaterial"), TEXT("SpecularSampleSize"), GlobalSize, GLightmassIni));
		}
		else
		if (InMaterialProperty == MP_EmissiveColor)
		{
			verify(GConfig->GetInt(TEXT("DevOptions.StaticLightingMaterial"), TEXT("EmissiveSampleSize"), GlobalSize, GLightmassIni));
		}
		else
		if (InMaterialProperty == MP_Normal)
		{
			verify(GConfig->GetInt(TEXT("DevOptions.StaticLightingMaterial"), TEXT("NormalSampleSize"), GlobalSize, GLightmassIni));
		}
		else
		if (InMaterialProperty == MP_Opacity)
		{
			verify(GConfig->GetInt(TEXT("DevOptions.StaticLightingMaterial"), TEXT("TransmissionSampleSize"), GlobalSize, GLightmassIni));
		}
		else
		{
			OutSizeX = 0;
			OutSizeY = 0;
			return false;
		}
		OutSizeX = OutSizeY = FMath::TruncToInt(GlobalSize * SizeScale);
		return true;
	}

	static bool WillFillData(EBlendMode InBlendMode, EMaterialProperty InMaterialProperty, bool bSubstrateEnabled)
	{
		// MAKE SURE THIS MATCHES THE CHART IN CompileProperty
		// 						  RETURNED VALUES (F16 'textures')
		// 	BLEND MODE  | DIFFUSE     | SPECULAR     | EMISSIVE    | NORMAL    | TRANSMISSIVE              |
		// 	------------+-------------+--------------+-------------+-----------+---------------------------|
		// 	Opaque      | Diffuse     | Spec,SpecPwr | Emissive    | Normal    | 0 (EMPTY)                 |
		// 	Masked      | Diffuse     | Spec,SpecPwr | Emissive    | Normal    | Opacity Mask              |
		// 	Translucent | 0 (EMPTY)   | 0 (EMPTY)    | Emissive    | 0 (EMPTY) | (Emsv | Diffuse)*Opacity  |
		// 	Additive    | 0 (EMPTY)   | 0 (EMPTY)    | Emissive    | 0 (EMPTY) | (Emsv | Diffuse)*Opacity  |
		// 	Modulative  | 0 (EMPTY)   | 0 (EMPTY)    | Emissive    | 0 (EMPTY) | Emsv | Diffuse            |
		// 	------------+-------------+--------------+-------------+-----------+---------------------------|

		// Emissive will always fill data.
		if (InMaterialProperty == MP_EmissiveColor)
		{
			return true;
		}

		if (bSubstrateEnabled)
		{
			switch (InMaterialProperty)
			{
			case MP_DiffuseColor:
			{
				return InBlendMode == BLEND_Opaque || InBlendMode == BLEND_Masked;
			}
			case MP_Normal:
			{
				return InBlendMode == BLEND_Opaque || InBlendMode == BLEND_Masked;
			}
			case MP_Opacity:
			{
				return InBlendMode != BLEND_Opaque;
			}
			default:
			{
				UE_LOGF(LogLightmassRender, Error, "FLightmassMaterialProxy::WillFillData - cannot export a requested property for");
			}
			}
		}

		switch (InBlendMode)
		{
		case BLEND_Opaque:
			{
				switch (InMaterialProperty)
				{
				case MP_DiffuseColor:	return true;
				case MP_SpecularColor:	return true;
				case MP_Normal:			return true;
				case MP_Opacity:		return false;
				}
			}
			break;
		case BLEND_Masked:
			{
				switch (InMaterialProperty)
				{
				case MP_DiffuseColor:	return true;
				case MP_SpecularColor:	return true;
				case MP_Normal:			return true;
				case MP_Opacity:		return true;
				}
			}
			break;
		case BLEND_Translucent:
		case BLEND_Additive:
		case BLEND_AlphaComposite:
		case BLEND_AlphaHoldout:
			{
				switch (InMaterialProperty)
				{
				case MP_DiffuseColor:	return false;
				case MP_SpecularColor:	return false;
				case MP_Normal:			return false;
				case MP_Opacity:		return true;
				}
			}
			break;
		case BLEND_Modulate:
			{
				switch (InMaterialProperty)
				{
				case MP_DiffuseColor:	return false;
				case MP_SpecularColor:	return false;
				case MP_Normal:			return false;
				case MP_Opacity:		return true;
				}
			}
			break;
		}
		return false;
	}

	virtual void GatherExpressionsForCustomInterpolators(TArray<UMaterialExpression*>& OutExpressions) const override
	{
		if (Material)
		{
			Material->GetAllExpressionsForCustomInterpolators(OutExpressions);
		}
	}

	virtual void GetStaticParameterSet(FStaticParameterSet& OutSet) const override
	{
		if (const FMaterialResource* Resource = MaterialInterface->GetMaterialResource(GMaxRHIShaderPlatform))
		{
			Resource->GetStaticParameterSet(OutSet);
		}
	}

	virtual bool CheckInValidStateForCompilation(class FMaterialCompiler* Compiler) const override
	{
		return Material && Material->CheckInValidStateForCompilation(Compiler);
	}

private:
	/** The material interface for this proxy */
	UMaterialInterface* MaterialInterface;
	UMaterial* Material;
	/** The property to compile for rendering the sample */
	EMaterialProperty PropertyToCompile;
	/** Stores which exported attribute this proxy is compiling for. */
	EMaterialShaderMapUsage::Type Usage;
	/** If Substrate is enabled, we need to specify things differently since redirection cannot straiforwardly be used*/
	bool bSubstrateEnabled;
};

FMaterialExportDataEntry::~FMaterialExportDataEntry()
{
	FLightmassMaterialProxy* LocalDiffuseMaterialProxy = DiffuseMaterialProxy;
	FLightmassMaterialProxy* LocalEmissiveMaterialProxy = EmissiveMaterialProxy;
	FLightmassMaterialProxy* LocalOpacityMaterialProxy = OpacityMaterialProxy;
	FLightmassMaterialProxy* LocalNormalMaterialProxy = NormalMaterialProxy;

	if (LocalDiffuseMaterialProxy )
	{
		FMaterial::DeferredDelete(LocalDiffuseMaterialProxy);		
	}

	if (LocalEmissiveMaterialProxy)
	{
		FMaterial::DeferredDelete(LocalEmissiveMaterialProxy);
	}

	if (LocalOpacityMaterialProxy)
	{
		FMaterial::DeferredDelete(LocalOpacityMaterialProxy);
	}

	if (LocalNormalMaterialProxy)
	{	
		FMaterial::DeferredDelete(LocalNormalMaterialProxy);
	}
}

//
// FLightmassMaterialRenderer
//
FLightmassMaterialRenderer::~FLightmassMaterialRenderer()
{
	if (!GExitPurge && RenderTarget)
	{
		RenderTarget->RemoveFromRoot();
	}
	RenderTarget = NULL;
	delete Canvas;
	Canvas = NULL;
}

void FLightmassMaterialRenderer::BeginGenerateMaterialData(
	UMaterialInterface* InMaterial, 
	bool bInWantNormals, 
	const FString& ChannelName,
	TMap<UMaterialInterface*, FMaterialExportDataEntry>& MaterialExportData)
{
	UMaterial* BaseMaterial = InMaterial->GetMaterial();

	EBlendMode BlendMode = InMaterial->GetBlendMode();

	const bool bIsLandscapeMaterial = InMaterial->IsA<ULandscapeMaterialInstanceConstant>();

	if (BaseMaterial)
	{
		check(!MaterialExportData.Contains(InMaterial));
		const bool bSubstrateEnabled = Substrate::IsSubstrateEnabled();

		FMaterialExportDataEntry& MaterialData = MaterialExportData.Add(InMaterial, FMaterialExportDataEntry(ChannelName));

		if (FLightmassMaterialProxy::WillFillData(BlendMode, MP_DiffuseColor, bSubstrateEnabled))
		{
			MaterialData.DiffuseMaterialProxy = new FLightmassMaterialProxy();
			MaterialData.DiffuseMaterialProxy->BeginCompiling(InMaterial, MP_DiffuseColor, EMaterialShaderMapUsage::LightmassExportDiffuse);
		}

		if (FLightmassMaterialProxy::WillFillData(BlendMode, MP_EmissiveColor, bSubstrateEnabled))
		{
			MaterialData.EmissiveMaterialProxy = new FLightmassMaterialProxy();
			MaterialData.EmissiveMaterialProxy->BeginCompiling(InMaterial, MP_EmissiveColor, EMaterialShaderMapUsage::LightmassExportEmissive);
		}

		if (FLightmassMaterialProxy::WillFillData(BlendMode, MP_Opacity, bSubstrateEnabled))
		{
			// Landscape opacity is generated from the hole mask, not the material
			if (!bIsLandscapeMaterial)
			{
				MaterialData.OpacityMaterialProxy = new FLightmassMaterialProxy();
				MaterialData.OpacityMaterialProxy->BeginCompiling(InMaterial, MP_Opacity, EMaterialShaderMapUsage::LightmassExportOpacity);
			}
		}

		if (bInWantNormals && FLightmassMaterialProxy::WillFillData(BlendMode, MP_Normal, bSubstrateEnabled))
		{
			MaterialData.NormalMaterialProxy = new FLightmassMaterialProxy();
			MaterialData.NormalMaterialProxy->BeginCompiling(InMaterial, MP_Normal, EMaterialShaderMapUsage::LightmassExportNormal);
		}
	}
}

/**
 *	Generate the required material data for the given material.
 *
 *	@param	Material				The material of interest.
 *	@param	bInWantNormals			True if normals should be generated as well
 *	@param	MaterialEmissive		The emissive samples for the material.
 *	@param	MaterialDiffuse			The diffuse samples for the material.
 *	@param	MaterialTransmission	The transmission samples for the material.
 *
 *	@return	bool					true if successful, false if not.
 */
bool FLightmassMaterialRenderer::GenerateMaterialData(
	FSceneInterface* InSceneInterface,
	UMaterialInterface& InMaterial,
	const FLightmassMaterialExportSettings& InExportSettings,
	Lightmass::FMaterialData& OutMaterialData,
	FMaterialExportDataEntry& MaterialExportEntry,
	TArray<FFloat16Color>& OutMaterialDiffuse,
	TArray<FFloat16Color>& OutMaterialEmissive,
	TArray<FFloat16Color>& OutMaterialTransmission,
	TArray<FFloat16Color>& OutMaterialNormal)
{
	bool bResult = true;
	UMaterial* BaseMaterial = InMaterial.GetMaterial();
	check(BaseMaterial);

	EBlendMode BlendMode = InMaterial.GetBlendMode();
	const bool bSubstrateEnabled = Substrate::IsSubstrateEnabled();

	FMaterialShadingModelField ShadingModels = InMaterial.GetShadingModels();
 	if (!bSubstrateEnabled &&		// Shading models are irrelevant when using Substrate
		!ShadingModels.HasShadingModel(MSM_DefaultLit) &&
		!ShadingModels.HasShadingModel(MSM_Unlit) &&
		!ShadingModels.HasShadingModel(MSM_Subsurface) &&
		!ShadingModels.HasShadingModel(MSM_PreintegratedSkin) &&
		!ShadingModels.HasShadingModel(MSM_SubsurfaceProfile))
	{
		UE_LOGF(LogLightmassRender, Warning, "LIGHTMASS: Material has an unsupported shading model: %d on %ls", 
			(int32)ShadingModels.GetShadingModelField(),
			*(InMaterial.GetPathName()));
	}

	// Set the blend mode
	static_assert(EBlendMode::BLEND_MAX == (EBlendMode)Lightmass::BLEND_MAX, "Debug type sizes must match.");
	OutMaterialData.BlendMode = (Lightmass::EBlendMode)((int32)BlendMode);

	// Set the two-sided flag
	OutMaterialData.bTwoSided = (uint32)InMaterial.IsTwoSided();
	OutMaterialData.bIsThinSurface = (uint32)InMaterial.IsThinSurface();
	OutMaterialData.OpacityMaskClipValue = InMaterial.GetOpacityMaskClipValue();
	// Cast shadow as masked feature need to access transmission texture. Only allow
	// if transmission/opacity data exists
	OutMaterialData.bCastShadowAsMasked = MaterialExportEntry.OpacityMaterialProxy
		&& InMaterial.GetCastShadowAsMasked();
	OutMaterialData.bSurfaceDomain = BaseMaterial->MaterialDomain == MD_Surface;

	const bool bIsLandscapeMaterial = InMaterial.IsA<ULandscapeMaterialInstanceConstant>();

	// due to landscape using an expanded mesh, we have to mask out the edge data even on opaque components (sigh)
	const bool bIsOpaque = OutMaterialData.BlendMode == Lightmass::BLEND_Opaque;
	if (bIsLandscapeMaterial && bIsOpaque)
	{
		OutMaterialData.BlendMode = Lightmass::BLEND_Masked;
	}

	// Diffuse
	if (MaterialExportEntry.DiffuseMaterialProxy)
	{
		if (!GenerateMaterialPropertyData(InSceneInterface, InMaterial, InExportSettings, MaterialExportEntry.DiffuseMaterialProxy, MP_DiffuseColor, OutMaterialData.DiffuseSize, OutMaterialData.DiffuseSize, OutMaterialDiffuse))
		{
			UE_LOGF(LogLightmassRender, Warning, "Failed to generate diffuse material samples for %ls: %ls",
				*(InMaterial.GetLightingGuid().ToString()), *(InMaterial.GetPathName()));
			bResult = false;
			OutMaterialData.DiffuseSize = 0;
		}
	}

	// Emissive
	if (MaterialExportEntry.EmissiveMaterialProxy)
	{
		if (!GenerateMaterialPropertyData(InSceneInterface, InMaterial, InExportSettings, MaterialExportEntry.EmissiveMaterialProxy, MP_EmissiveColor, OutMaterialData.EmissiveSize, OutMaterialData.EmissiveSize, OutMaterialEmissive))
		{
			UE_LOGF(LogLightmassRender, Warning, "Failed to generate emissive material samples for %ls: %ls",
				*(InMaterial.GetLightingGuid().ToString()), *(InMaterial.GetPathName()));
			bResult = false;
			OutMaterialData.EmissiveSize = 0;
		}
	}

	// Transmission
	// Landscape opacity is generated from the hole mask, not the material
	if (MaterialExportEntry.OpacityMaterialProxy || bIsLandscapeMaterial)
	{
		if (!GenerateMaterialPropertyData(InSceneInterface, InMaterial, InExportSettings, MaterialExportEntry.OpacityMaterialProxy, MP_Opacity, OutMaterialData.TransmissionSize, OutMaterialData.TransmissionSize, OutMaterialTransmission))
		{
			UE_LOGF(LogLightmassRender, Warning, "Failed to generate transmissive material samples for %ls: %ls",
				*(InMaterial.GetLightingGuid().ToString()), *(InMaterial.GetPathName()));
			bResult = false;
			OutMaterialData.TransmissionSize = 0;
		}
	}

	// Normal
	if (MaterialExportEntry.NormalMaterialProxy)
	{
		if (!GenerateMaterialPropertyData(InSceneInterface, InMaterial, InExportSettings, MaterialExportEntry.NormalMaterialProxy, MP_Normal, OutMaterialData.NormalSize, OutMaterialData.NormalSize, OutMaterialNormal))
		{
			UE_LOGF(LogLightmassRender, Warning, "Failed to generate normal material samples for %ls: %ls",
				*(InMaterial.GetLightingGuid().ToString()), *(InMaterial.GetPathName()));
			bResult = false;
			OutMaterialData.NormalSize = 0;
		}
	}

	return bResult;
}

void LightmassDebugExportMaterial(UMaterialInterface& InMaterial, EMaterialProperty InMaterialProperty, FFloat16Color* InMaterialSamples, int32 InSizeX, int32 InSizeY)
{
	TArray<FColor> OutputBuffer;
	OutputBuffer.Empty(InSizeX * InSizeY);
	bool bSRGB = InMaterialProperty != MP_Normal;
	for (int32 i = 0; i < InSizeX * InSizeY; ++i)
	{
		FLinearColor LinearColor(InMaterialSamples[i]);
		OutputBuffer.Add(LinearColor.ToFColor(bSRGB));
	}

	// Create screenshot folder if not already present.
	// Save the contents of the array to a bitmap file.
	FString TempPath = FPaths::ScreenShotDir();
	TempPath += TEXT("/Materials");
	IFileManager::Get().MakeDirectory(*TempPath, true);
	FString TempName = InMaterial.GetPathName();
	TempName = TempName.Replace(TEXT("."), TEXT("_"));
	TempName = TempName.Replace(TEXT(":"), TEXT("_"));
	FString OutputName = TempPath / TempName;
	OutputName += TEXT("_");
	switch (InMaterialProperty)
	{
	case MP_DiffuseColor:	OutputName += TEXT("Diffuse");			break;
	case MP_EmissiveColor:	OutputName += TEXT("Emissive");			break;
	case MP_SpecularColor:	OutputName += TEXT("Specular");			break;
	case MP_Normal:			OutputName += TEXT("Normal");			break;
	case MP_Opacity:		OutputName += TEXT("Transmissive");		break;
	}
	OutputName += TEXT(".BMP");
	FFileHelper::CreateBitmap(*OutputName, InSizeX, InSizeY, OutputBuffer.GetData());
}

/**
 *	Generate the material data for the given material and it's given property.
 *
 *	@param	InMaterial				The material of interest.
 *	@param	InMaterialProperty		The property to generate the samples for
 *	@param	InOutSizeX				The desired X size of the sample to capture (in), the resulting size (out)
 *	@param	InOutSizeY				The desired Y size of the sample to capture (in), the resulting size (out)
 *	@param	OutMaterialSamples		The samples for the material.
 *
 *	@return	bool					true if successful, false if not.
 */
bool FLightmassMaterialRenderer::GenerateMaterialPropertyData(
	FSceneInterface* InSceneInterface,
	UMaterialInterface& InMaterial,
	const FLightmassMaterialExportSettings& InExportSettings,
	FLightmassMaterialProxy* MaterialProxy,
	EMaterialProperty InMaterialProperty,
	int32& InOutSizeX,
	int32& InOutSizeY,
	TArray<FFloat16Color>& OutMaterialSamples)
{
	bool bResult = true;

	FFloat16Color UniformValue;

	const bool bIsLandscapeMaterial = InMaterial.IsA<ULandscapeMaterialInstanceConstant>();

	// Landscape opacity needs to be handled specially because it needs to look at the neighbor components
	// trying to actually use the neighbor materials is all but impossible so we read the data from the hole mask ourself
	if (bIsLandscapeMaterial && InMaterialProperty == MP_Opacity)
	{
		auto* LandscapeMesh = static_cast<const FLandscapeStaticLightingMesh*>(InExportSettings.UnwrapMesh);
		GetLandscapeOpacityData(LandscapeMesh, InOutSizeX, InOutSizeY, OutMaterialSamples);

		if (GLightmassDebugOptions.bDebugMaterials == true)
		{
			LightmassDebugExportMaterial(InMaterial, InMaterialProperty, OutMaterialSamples.GetData(), InOutSizeX, InOutSizeY);
		}
	}
	else if (MaterialProxy->WillGenerateUniformData(UniformValue))
	{
		// Single value... fill it in.
		InOutSizeX = 1;
		InOutSizeY = 1;
		OutMaterialSamples.Empty(1);
		OutMaterialSamples.AddZeroed(1);
		OutMaterialSamples[0] = UniformValue;
	}
	else
	{
		// Verify that async compiling has completed for this material
		// If the ShaderMap is NULL that's because it failed to compile, which is ok as the default material will be used for exporting
		check(!MaterialProxy->GetGameThreadShaderMap() || MaterialProxy->GetGameThreadShaderMap()->IsValidForRendering());

		//@todo The format may be determined by the material property...
		// For example, if Diffuse doesn't need to be F16 it can create a standard RGBA8 target.
		EPixelFormat Format = PF_FloatRGBA;
		if (MaterialProxy->GetRenderTargetFormatAndSize(InMaterialProperty, Format, InMaterial.GetExportResolutionScale(), InOutSizeX, InOutSizeY))
		{
			if (CreateRenderTarget(Format, InOutSizeX, InOutSizeY) == false)
			{
				UE_LOGF(LogLightmassRender, Warning, "Failed to create %4dx%4d render target!", InOutSizeX, InOutSizeY);
				bResult = false;
			}
			else
			{
				ENQUEUE_RENDER_COMMAND(InitializeSystemTextures)(
					[](FRHICommandListImmediate& RHICmdList)
					{
						GetRendererModule().InitializeSystemTextures(RHICmdList);
					});

				// Prefetch all virtual textures so that we have content available
				//todo[vt]: Move this to calling function to avoid multiple prefetches
				if (UseVirtualTexturing(GMaxRHIShaderPlatform))
				{					
					const ERHIFeatureLevel::Type FeatureLevel = GMaxRHIFeatureLevel;
					const FVector2D ScreenSpaceSize(InOutSizeX, InOutSizeY);

					UE::RenderCommandPipe::FSyncScope SyncScope;

					ENQUEUE_RENDER_COMMAND(LoadTiles)(
						[FeatureLevel, ScreenSpaceSize](FRHICommandListImmediate& RHICmdList)
					{
						GetRendererModule().RequestVirtualTextureTiles(ScreenSpaceSize, -1);
						GetRendererModule().LoadPendingVirtualTextureTiles(RHICmdList, FeatureLevel);
					});

					FlushRenderingCommands();
				}

				if (bIsLandscapeMaterial)
				{
					// Landscape needs special handling because it uses multiple UVs, which isn't yet supported by lightmass's regular pipeline
					auto* LandscapeMesh = static_cast<const FLandscapeStaticLightingMesh*>(InExportSettings.UnwrapMesh);
					RenderLandscapeMaterialForLightmass(LandscapeMesh, MaterialProxy, RenderTarget->GameThread_GetRenderTargetResource());
				}
				else
				{
					// At this point, we can't just return false at failure since we have some clean-up to do...
					Canvas->SetRenderTarget_GameThread(RenderTarget->GameThread_GetRenderTargetResource());

					// Clear the render target to black
					// This is necessary because the below DrawTile doesn't write to the first column and first row
					//@todo - figure out and fix DrawTile issues when rendering a full render target quad
					Canvas->Clear(FLinearColor(0, 0, 0, 0));
					FCanvasTileItem TileItem(FVector2D(0.0f, 0.0f), MaterialProxy, FVector2D(InOutSizeX, InOutSizeY));
					TileItem.bFreezeTime = true;
					Canvas->DrawItem(TileItem);
					Canvas->Flush_GameThread();
					FlushRenderingCommands();
					Canvas->SetRenderTarget_GameThread(NULL);
					FlushRenderingCommands();
				}

				// Read in the data
				//@todo UE4. Check the format! RenderTarget->Format
				// If we are going to allow non-F16 formats, then the storage will have to be aware of it!
				if (RenderTarget->GameThread_GetRenderTargetResource()->ReadFloat16Pixels(OutMaterialSamples) == false)
				{
					UE_LOGF(LogLightmassRender, Warning, "Failed to read Float16Pixels for 0x%08x property of %ls: %ls",
						(uint32)InMaterialProperty, *(InMaterial.GetLightingGuid().ToString()), *(InMaterial.GetPathName()));
					bResult = false;
				}

				if (GLightmassDebugOptions.bDebugMaterials == true)
				{
					LightmassDebugExportMaterial(InMaterial, InMaterialProperty, OutMaterialSamples.GetData(), InOutSizeX, InOutSizeY);
				}
			}
		}
		else
		{
			UE_LOGF(LogLightmassRender, Warning, "Failed to get render target format and size for 0x%08x property of %ls: %ls",
				(uint32)InMaterialProperty, *(InMaterial.GetLightingGuid().ToString()), *(InMaterial.GetPathName()));
			bResult = false;
		}
	}

	return bResult;
}

/**
 *	Create the required render target.
 *
 *	@param	InFormat	The format of the render target
 *	@param	InSizeX		The X resolution of the render target
 *	@param	InSizeY		The Y resolution of the render target
 *
 *	@return	bool		true if it was successful, false if not
 */
bool FLightmassMaterialRenderer::CreateRenderTarget(EPixelFormat InFormat, int32 InSizeX, int32 InSizeY)
{
	if (RenderTarget && 
		((RenderTarget->OverrideFormat != InFormat) || (RenderTarget->SizeX != InSizeX) || (RenderTarget->SizeY != InSizeY))
		)
	{
		RenderTarget->RemoveFromRoot();
		RenderTarget = NULL;
		delete Canvas;
		Canvas = NULL;
	}

	if (RenderTarget == NULL)
	{
		RenderTarget = NewObject<UTextureRenderTarget2D>();
		check(RenderTarget);
		RenderTarget->AddToRoot();
		RenderTarget->ClearColor = FLinearColor(0.0f, 0.0f, 0.0f, 0.0f);
		RenderTarget->InitCustomFormat(InSizeX, InSizeY, InFormat, false);

		Canvas = new FCanvas(RenderTarget->GameThread_GetRenderTargetResource(), NULL, FGameTime(), GMaxRHIFeatureLevel);
		check(Canvas);
	}

	return (RenderTarget != NULL);
}
