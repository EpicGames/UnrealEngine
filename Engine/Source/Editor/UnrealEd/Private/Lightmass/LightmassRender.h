// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LightmassRender.h: lightmass rendering-related definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "PixelFormat.h"
#include "SceneTypes.h"

class FCanvas;
class FLightmassMaterialProxy;
class UMaterialInterface;
class FSceneInterface;
class UTextureRenderTarget2D;
struct FLightmassMaterialExportSettings;

/** Forward declarations of Lightmass types */
namespace Lightmass
{
	struct FMaterialData;
}

class FLightmassMaterialProxy;
struct FLightmassMaterialExportSettings;

/** Information about a material to be exported to lightmass, which is generated by a first pass over the materials. */
class FMaterialExportDataEntry
{
public:
	/** Swarm channel name. */
	FString ChannelName;

	/** Material proxies to use for exporting, or NULL if the attribute should not be exported. */
	FLightmassMaterialProxy* DiffuseMaterialProxy;
	FLightmassMaterialProxy* EmissiveMaterialProxy;
	FLightmassMaterialProxy* OpacityMaterialProxy;
	FLightmassMaterialProxy* NormalMaterialProxy;

	FMaterialExportDataEntry(const FString& InChannelName) :
		ChannelName(InChannelName),
		DiffuseMaterialProxy(NULL),
		EmissiveMaterialProxy(NULL),
		OpacityMaterialProxy(NULL),
		NormalMaterialProxy(NULL)
	{}

	~FMaterialExportDataEntry();
};

/**
 * Class for rendering sample 'textures' of various material properties.
 */
class FLightmassMaterialRenderer
{
public:
	FLightmassMaterialRenderer() : 
		  RenderTarget(NULL)
		, Canvas(NULL)
	{
	}

	virtual ~FLightmassMaterialRenderer();

	/** Initiates generation of material export data by kicking off async shader compiles. */
	void BeginGenerateMaterialData(
		UMaterialInterface* InMaterial,
		bool bInWantNormals,
		const FString& ChannelName,
		TMap<UMaterialInterface*, FMaterialExportDataEntry>& MaterialExportData);

	/**
	 *	Generate the required material data for the given material.
	 *
	 *	@param	Material				The material of interest.
	 *	@param	bInWantNormals			True if normals should be generated as well
	 *	@param	MaterialEmissive		The emissive samples for the material.
	 *	@param	MaterialDiffuse			The diffuse samples for the material.
	 *	@param	MaterialTransmission	The transmission samples for the material.
	 *
	 *	@return	bool					true if successful, false if not.
	 */
	bool GenerateMaterialData(
		FSceneInterface* InSceneInterface,
		UMaterialInterface& InMaterial,
		const FLightmassMaterialExportSettings& InExportSettings,
		Lightmass::FMaterialData& OutMaterialData,
		FMaterialExportDataEntry& MaterialExportEntry,
		TArray<FFloat16Color>& OutMaterialDiffuse,
		TArray<FFloat16Color>& OutMaterialEmissive,
		TArray<FFloat16Color>& OutMaterialTransmission,
		TArray<FFloat16Color>& OutMaterialNormal);

protected:
	/**
	 *	Create the required render target.
	 *
	 *	@param	InFormat			The format of the render target
	 *	@param	InSizeX				The X resolution of the render target
	 *	@param	InSizeY				The Y resolution of the render target
	 *
	 *	@return	bool		true if it was successful, false if not
	 */
	bool CreateRenderTarget(EPixelFormat InFormat, int32 InSizeX, int32 InSizeY);

	/**
	 *	Generate the material data for the given material and it's given property.
	 *
	 *	@param	InMaterial				The material of interest.
	 *	@param	InMaterialProperty		The property to generate the samples for
	 *	@param	InOutSizeX				The desired X size of the sample to capture (in), the resulting size (out)
	 *	@param	InOutSizeY				The desired Y size of the sample to capture (in), the resulting size (out)
	 *	@param	OutMaterialSamples		The samples for the material.
	 *
	 *	@return	bool					true if successful, false if not.
	 */
	bool GenerateMaterialPropertyData(
		FSceneInterface* InSceneInterface,
		UMaterialInterface& InMaterial, 
		const FLightmassMaterialExportSettings& InExportSettings,
		FLightmassMaterialProxy* MaterialProxy,
		EMaterialProperty InMaterialProperty, 
		int32& InOutSizeX, 
		int32& InOutSizeY, 
		TArray<FFloat16Color>& OutMaterialSamples);

	UTextureRenderTarget2D* RenderTarget;
	FCanvas* Canvas;
};

