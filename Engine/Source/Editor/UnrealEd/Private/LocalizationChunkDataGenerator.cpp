// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalizationChunkDataGenerator.h"

#include "Containers/Map.h"
#include "IPlatformFileSandboxWrapper.h"
#include "Interfaces/IPluginManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "Internationalization/InternationalizationManifest.h"
#include "Internationalization/Text.h"
#include "Internationalization/TextLocalizationResource.h"
#include "LocTextHelper.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Templates/UnrealTemplate.h"
#include "TextLocalizationResourceGenerator.h"
#include "Trace/Detail/Channel.h"
#include "UObject/NameTypes.h"
#include "Async/ParallelFor.h"

class FArchiveEntry;

DEFINE_LOG_CATEGORY_STATIC(LogLocalizationChunkDataGenerator, Log, All);

FLocalizationChunkDataGenerator::FLocalizationChunkDataGenerator(const int32 InCatchAllChunkId, TArray<FString> InLocalizationTargetsToChunk, TArray<FString> InAllCulturesToCook)
	: CatchAllChunkId(InCatchAllChunkId)
	, LocalizationTargetsToChunk(MoveTemp(InLocalizationTargetsToChunk))
	, AllCulturesToCook(MoveTemp(InAllCulturesToCook))
{
	AllPotentialContentRoots.Add(TEXT("/Engine/"));
	AllPotentialContentRoots.Add(TEXT("/Game/"));

	// Cache information about potential plugin content roots, including plugins that aren't currently loaded that we may have gathered localization data from
	{
		TArray<TSharedRef<IPlugin>> AllPlugins = IPluginManager::Get().GetDiscoveredPlugins();
		for (TSharedRef<IPlugin> Plugin : AllPlugins)
		{
			if (!Plugin->CanContainContent())
			{
				continue;
			}

			AllPotentialContentRoots.Add(FString::Printf(TEXT("/%s/"), *Plugin->GetName()));

			// Some plugins may re-map their content onto /Game during cook, so we need to take this into account when chunking the localization data
			const FString PluginConfigFilename = Plugin->GetBaseDir() / TEXT("Config") / FString::Printf(TEXT("Default%s.ini"), *Plugin->GetName());

			// Note: We don't use GConfig directly here, as not all of these plugins may currently be loaded
			// This means we need to load their config file directly to access this setting...
			FConfigFile PluginConfig;
			PluginConfig.Read(PluginConfigFilename);

			bool bShouldRemap = false;
			PluginConfig.GetBool(TEXT("PluginSettings"), TEXT("RemapPluginContentToGame"), bShouldRemap);
			
			if (bShouldRemap)
			{
				PluginContentRootsMappedToGameRoot.Add(FString::Printf(TEXT("/%s/"), *Plugin->GetName()));
			}
		}
	}
}

void FLocalizationChunkDataGenerator::GenerateChunkDataFiles(const int32 InChunkId, const TSet<FName>& InPackagesInChunk, const ITargetPlatform* TargetPlatform, FSandboxPlatformFile* InSandboxFile, TArray<FString>& OutChunkFilenames)
{
	const FString InPlatformName = TargetPlatform->PlatformName();
	const FString LocalizationContentRoot = (InSandboxFile->GetSandboxDirectory() / InSandboxFile->GetGameSandboxDirectoryName() / TEXT("Content") / TEXT("Localization")).Replace(TEXT("[Platform]"), *InPlatformName);
	const FString LocalizationMetadataRoot = (InSandboxFile->GetSandboxDirectory() / InSandboxFile->GetGameSandboxDirectoryName() / TEXT("Metadata") / TEXT("Localization")).Replace(TEXT("[Platform]"), *InPlatformName);
	GenerateChunkDataFilesForPaths(InChunkId, InPackagesInChunk, TargetPlatform, LocalizationContentRoot, LocalizationMetadataRoot, OutChunkFilenames);
}

void FLocalizationChunkDataGenerator::GenerateChunkDataFilesForPaths(const int32 InChunkId, const TSet<FName>& InPackagesInChunk, const ITargetPlatform* TargetPlatform, const FString& LocalizationContentRoot, const FString& LocalizationMetadataRoot, TArray<FString>& OutChunkFilenames)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FLocalizationChunkDataGenerator::GenerateChunkDataFiles);

	// The primary chunk doesn't gain a suffix to make it unique, as it is replacing the offline localization data that is usually staged verbatim
	const bool bIsPrimaryChunk = InChunkId == 0;

	// The catch-all chunk is the only chunk that can contain non-asset localization data (this is chunk 0 by default)
	const bool bIsCatchAllChunk = InChunkId == CatchAllChunkId;

	// We can skip empty non-primary and non-catch-all chunks
	if (!bIsPrimaryChunk && !bIsCatchAllChunk && InPackagesInChunk.Num() == 0)
	{
		return;
	}

	ConditionalCacheLocalizationTargetData();

	// We can skip this if we're not actually chunking any localization data
	if (CachedLocalizationTargetData.Num() == 0)
	{
		return;
	}

	for (const FCachedLocalizationTargetData& CachedLocalizationTargetDataEntry : CachedLocalizationTargetData)
	{
		const TSharedPtr<FLocTextHelper>& SourceLocTextHelper = CachedLocalizationTargetDataEntry.LocTextHelper;

		// If this target has no helper, then it was either an invalid target or failed to load when caching - skip it here
		if (!SourceLocTextHelper)
		{
			continue;
		}

		// Prepare to produce the localization target for this chunk
		const TArray<FString> AvailableCulturesToCook = SourceLocTextHelper->GetAllCultures();
		const FString ChunkTargetName = TextLocalizationResourceUtil::GetLocalizationTargetNameForChunkId(SourceLocTextHelper->GetTargetName(), InChunkId);
		const FString ChunkTargetContentRoot = LocalizationContentRoot / ChunkTargetName;
		const FString ChunkTargetMetadataRoot = LocalizationMetadataRoot / ChunkTargetName;

		// Produce a filtered set of data that can be used to produce the LocRes for each chunk
		bool bChunkHasText = false;
		FLocTextHelper ChunkLocTextHelper(ChunkTargetMetadataRoot, FString::Printf(TEXT("%s.manifest"), *ChunkTargetName), FString::Printf(TEXT("%s.archive"), *ChunkTargetName), SourceLocTextHelper->GetNativeCulture(), SourceLocTextHelper->GetForeignCultures(), nullptr);
		ChunkLocTextHelper.LoadAll(ELocTextHelperLoadFlags::Create); // Create the in-memory manifest and archives
		FCriticalSection EnumerateGuard;
		SourceLocTextHelper->EnumerateSourceTextsParallel([this, bIsCatchAllChunk, SourceLocTextHelper, &bChunkHasText, &ChunkLocTextHelper, &InPackagesInChunk, &AvailableCulturesToCook, &EnumerateGuard](TSharedRef<FManifestEntry> InManifestEntry)
		{
			for (const FManifestContext& ManifestContext : InManifestEntry->Contexts)
			{
				bool bIncludeInChunk = bIsCatchAllChunk;
				{
					// Note: We can't use FPackageName::IsValidLongPackageName in the tests below, as not all plugins we gathered localization from may be loaded during cook meaning the unmapped path wouldn't be recognized as a valid package root
					FString SourcePackageName = FPackageName::ObjectPathToPackageName(ManifestContext.SourceLocation);

					// Some plugins may re-map their content onto /Game during cook, so we need to take this into account when chunking the localization data
					for (const FString& PluginContentRootMappedToGameRoot : PluginContentRootsMappedToGameRoot)
					{
						if (SourcePackageName.RemoveFromStart(PluginContentRootMappedToGameRoot))
						{
							SourcePackageName.InsertAt(0, TEXT("/Game/"));
							break;
						}
					}

					// Is this from a potential content root? If so, treat it as an asset and test whether it was cooked into this chunk
					for (const FString& PotentialContentRoot : AllPotentialContentRoots)
					{
						if (SourcePackageName.StartsWith(PotentialContentRoot))
						{
							bIncludeInChunk = InPackagesInChunk.Contains(*SourcePackageName);
							break;
						}
					}
				}

				if (bIncludeInChunk)
				{
					FScopeLock _(&EnumerateGuard);

					bChunkHasText = true;
					ChunkLocTextHelper.AddSourceText(InManifestEntry->Namespace, InManifestEntry->Source, ManifestContext);
					for (const FString& CultureToCook : AvailableCulturesToCook)
					{
						TSharedPtr<FArchiveEntry> SourceTranslationEntry = SourceLocTextHelper->FindTranslation(CultureToCook, InManifestEntry->Namespace, ManifestContext.Key, ManifestContext.KeyMetadataObj);
						if (SourceTranslationEntry)
						{
							ChunkLocTextHelper.AddTranslation(CultureToCook, SourceTranslationEntry.ToSharedRef());
						}
					}
				}
			}

			return true; // continue enumeration
		}, false);

		// We can skip empty non-primary chunks
		if (!bIsPrimaryChunk && !bChunkHasText)
		{
			// We still need to write a dummy LocMeta file, so that FTextLocalizationManager::OnPakFileMounted validates that this chunked target has no data
			// Without the dummy file it would early out and prevent any other chunked targets (that might have data for this chunk) from being loaded too!
			{
				const FString ChunkLocMetaFilename = ChunkTargetContentRoot / FString::Printf(TEXT("%s.locmeta"), *ChunkTargetName);

				FTextLocalizationMetaDataResource ChunkDummyLocMeta;
				if (ChunkDummyLocMeta.SaveToFile(ChunkLocMetaFilename))
				{
					OutChunkFilenames.Add(ChunkLocMetaFilename);
				}
				else
				{
					UE_LOGF(LogLocalizationChunkDataGenerator, Error, "Failed to generate dummy meta-data for localization target '%ls' when chunking localization data.", *ChunkTargetName);
				}
			}
			continue;
		}

		// Save the manifest and archives for debug purposes within the metadata folder
		// This won't add them to the build, and we don't care if this fails as it's only for debugging
		ChunkLocTextHelper.SaveAll();

		// Sort the manifest to ensure the LocRes output is deterministic, as this was populated via a ParallelFor
		ChunkLocTextHelper.SortManifest();

		// Produce the LocMeta file for the chunk target
		{
			const FString ChunkLocMetaFilename = ChunkTargetContentRoot / FString::Printf(TEXT("%s.locmeta"), *ChunkTargetName);
			
			FTextLocalizationMetaDataResource ChunkLocMeta;
			if (FTextLocalizationResourceGenerator::GenerateLocMeta(ChunkLocTextHelper, FString::Printf(TEXT("%s.locres"), *ChunkTargetName), ChunkLocMeta) && ChunkLocMeta.SaveToFile(ChunkLocMetaFilename))
			{
				OutChunkFilenames.Add(ChunkLocMetaFilename);
			}
			else
			{
				UE_LOGF(LogLocalizationChunkDataGenerator, Error, "Failed to generate meta-data for localization target '%ls' when chunking localization data.", *ChunkTargetName);
			}
		}

		// Produce the LocRes files for each culture of the chunk target
		for (const FString& CultureToCook : AvailableCulturesToCook)
		{
			// This is an extra sanity check as the native culture of the target may not be being cooked
			if (!AllCulturesToCook.Contains(CultureToCook))
			{
				continue;
			}

			const FString ChunkLocResFilename = ChunkTargetContentRoot / CultureToCook / FString::Printf(TEXT("%s.locres"), *ChunkTargetName);

			FTextLocalizationResource ChunkLocRes;
			TMap<FName, TSharedRef<FTextLocalizationResource>> UnusedPerPlatformLocRes;
			if (FTextLocalizationResourceGenerator::GenerateLocRes(ChunkLocTextHelper, CultureToCook, CachedLocalizationTargetDataEntry.GenerateLocResFlags, ChunkLocResFilename, ChunkLocRes, UnusedPerPlatformLocRes) && ChunkLocRes.SaveToFile(ChunkLocResFilename))
			{
				OutChunkFilenames.Add(ChunkLocResFilename);
			}
			else
			{
				UE_LOGF(LogLocalizationChunkDataGenerator, Error, "Failed to generate resource data for localization target '%ls' when chunking localization data.", *ChunkTargetName);
			}
		}
	}
}

void FLocalizationChunkDataGenerator::ConditionalCacheLocalizationTargetData()
{
	// We can skip this if we're not actually chunking or staging any localization data
	if (LocalizationTargetsToChunk.Num() == 0 || AllCulturesToCook.Num() == 0)
	{
		return;
	}

	// Already cached?
	if (LocalizationTargetsToChunk.Num() == CachedLocalizationTargetData.Num())
	{
		return;
	}

	auto DoesTargetAllowStaleTranslations =
		[](const FString& LocalizationTarget) -> bool
		{
			// Note: We have to try and load this from the gather config data, as this information is not mirrored in the LocMeta file
			const FString PotentialConfigPaths[] = {
				FPaths::Combine(FPaths::ProjectConfigDir(), TEXTVIEW("Localization"), LocalizationTarget + TEXTVIEW("_Compile.ini")),
				FPaths::Combine(FPaths::ProjectConfigDir(), TEXTVIEW("Localization"), LocalizationTarget + TEXTVIEW(".ini")),
			};

			for (const FString& PotentialConfigPath : PotentialConfigPaths)
			{
				if (!FPaths::FileExists(PotentialConfigPath))
				{
					continue;
				}

				const FConfigFile* LocalizationConfig = GConfig->Find(PotentialConfigPath); // Note: Find also loads if needed
				if (!LocalizationConfig)
				{
					continue;
				}

				UE_LOGF(LogLocalizationChunkDataGenerator, Log, "Loaded config for localization target '%ls' (from: '%ls')", *LocalizationTarget, *PotentialConfigPath);

				// Look for bSkipSourceCheck in any section of this config file
				bool bSkipSourceCheck = false;
				for (const TTuple<FString, FConfigSection>& SectionPair : *LocalizationConfig)
				{
					if (SectionPair.Value.GetBool(TEXT("bSkipSourceCheck"), bSkipSourceCheck))
					{
						break;
					}
				}

				// Clean-up the config we loaded above
				GConfig->UnloadFile(PotentialConfigPath);

				return bSkipSourceCheck;
			}

			UE_LOGF(LogLocalizationChunkDataGenerator, Log, "Could not find any config for localization target '%ls'", *LocalizationTarget);

			return false;
		};

	CachedLocalizationTargetData.Reset(LocalizationTargetsToChunk.Num());
	for (const FString& LocalizationTarget : LocalizationTargetsToChunk)
	{
		FCachedLocalizationTargetData& CachedLocalizationTargetDataEntry = CachedLocalizationTargetData.AddDefaulted_GetRef();
		CachedLocalizationTargetDataEntry.GenerateLocResFlags = DoesTargetAllowStaleTranslations(LocalizationTarget) ? EGenerateLocResFlags::AllowStaleTranslations : EGenerateLocResFlags::None;

		// Does this target exist?
		// Note: We only allow game localization targets to be chunked, and the layout is assumed to follow our standard pattern (as used by the localization dashboard and FLocTextHelper)
		const FString SourceRootPath = FPaths::ProjectContentDir() / TEXT("Localization") / LocalizationTarget;
		if (!FPaths::DirectoryExists(SourceRootPath))
		{
			UE_LOGF(LogLocalizationChunkDataGenerator, Warning, "Failed to find localization target for '%ls' when chunking localization data. Is it a valid project localization target? - %ls", *LocalizationTarget, *SourceRootPath);
			continue;
		}

		// Work out what the native culture is
		FString SourceNativeCulture;
		{
			const FString SourceLocMetaFilename = SourceRootPath / FString::Printf(TEXT("%s.locmeta"), *LocalizationTarget);

			FTextLocalizationMetaDataResource SourceLocMeta;
			if (!SourceLocMeta.LoadFromFile(SourceLocMetaFilename))
			{
				UE_LOGF(LogLocalizationChunkDataGenerator, Error, "Failed to load meta-data for localization target '%ls' when chunking localization data. Re-compile the localization target to generate the LocMeta file.", *LocalizationTarget);
				continue;
			}

			SourceNativeCulture = SourceLocMeta.NativeCulture;
		}

		// Work out which of the desired cultures this target actually supports
		TArray<FString> SourceForeignCulturesToCook;
		for (const FString& CultureToCook : AllCulturesToCook)
		{
			if (CultureToCook == SourceNativeCulture)
			{
				continue;
			}

			const FString LocalizationTargetCulturePath = SourceRootPath / CultureToCook;
			if (FPaths::DirectoryExists(LocalizationTargetCulturePath))
			{
				SourceForeignCulturesToCook.Add(CultureToCook);
			}
		}

		// Load the data for this target
		CachedLocalizationTargetDataEntry.LocTextHelper = MakeShared<FLocTextHelper>(SourceRootPath, FString::Printf(TEXT("%s.manifest"), *LocalizationTarget), FString::Printf(TEXT("%s.archive"), *LocalizationTarget), SourceNativeCulture, SourceForeignCulturesToCook, nullptr);
		{
			FText LoadError;
			if (CachedLocalizationTargetDataEntry.LocTextHelper->LoadAll(ELocTextHelperLoadFlags::Load, &LoadError))
			{
				UE_LOGF(LogLocalizationChunkDataGenerator, Log, "Loaded data for localization target '%ls'", *LocalizationTarget);
			}
			else
			{
				CachedLocalizationTargetDataEntry.LocTextHelper.Reset();
				UE_LOGF(LogLocalizationChunkDataGenerator, Error, "Failed to load data for localization target '%ls' when chunking localization data: %ls", *LocalizationTarget, *LoadError.ToString());
				continue;
			}
		}
	}
}
