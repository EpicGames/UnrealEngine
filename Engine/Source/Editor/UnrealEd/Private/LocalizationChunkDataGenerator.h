// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commandlets/IChunkDataGenerator.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"

enum class EGenerateLocResFlags : uint8;

class FLocTextHelper;
class FName;
class FSandboxPlatformFile;
class ITargetPlatform;

/**
 * Implementation for splitting localization data into chunks when creating streaming install manifests.
 */
class FLocalizationChunkDataGenerator : public IChunkDataGenerator
{
public:
	FLocalizationChunkDataGenerator(const int32 InCatchAllChunkId, TArray<FString> InLocalizationTargetsToChunk, TArray<FString> InAllCulturesToCook);
	virtual ~FLocalizationChunkDataGenerator() = default;

	//~ IChunkDataGenerator
	virtual void GenerateChunkDataFiles(const int32 InChunkId, const TSet<FName>& InPackagesInChunk, const ITargetPlatform* TargetPlatform, FSandboxPlatformFile* InSandboxFile, TArray<FString>& OutChunkFilenames) override;

	void GenerateChunkDataFilesForPaths(const int32 InChunkId, const TSet<FName>& InPackagesInChunk, const ITargetPlatform* TargetPlatform, const FString& InLocalizationContentRoot, const FString& InLocalizationMetadataRoot, TArray<FString>& OutChunkFilenames);

private:
	/** Update CachedLocalizationTargetData if needed */
	void ConditionalCacheLocalizationTargetData();

	/** The chunk ID that should be used as the catch-all chunk for any non-asset localized strings */
	int32 CatchAllChunkId;

	/** List of localization targets that should be chunked */
	TArray<FString> LocalizationTargetsToChunk;

	/** Complete list of cultures to cook data for, including inferred parent cultures */
	TArray<FString> AllCulturesToCook;

	struct FCachedLocalizationTargetData
	{
		TSharedPtr<FLocTextHelper> LocTextHelper;
		EGenerateLocResFlags GenerateLocResFlags = {};
	};
	/** Cached localization target data, to avoid redundant work for each chunk */
	TArray<FCachedLocalizationTargetData> CachedLocalizationTargetData;

	/** Array of potential content roots, including plugins that aren't currently loaded */
	TArray<FString> AllPotentialContentRoots;

	/** Array of plugin content roots that should be mapped onto /Game during cook */
	TArray<FString> PluginContentRootsMappedToGameRoot;
};
