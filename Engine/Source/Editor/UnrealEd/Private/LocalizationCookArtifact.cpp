// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalizationCookArtifact.h"

#include "HAL/FileManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "Logging/LogMacros.h"
#include "Misc/App.h"
#include "Misc/Paths.h"
#include "Serialization/CompactBinaryWriter.h"
#include "UObject/ICookInfo.h"

#if WITH_EDITOR

DEFINE_LOG_CATEGORY_STATIC(LogLocalizationCookArtifact, Log, All);


FLocalizationCookArtifact::FLocalizationCookArtifact(const TArray<FString>& InAllCulturesToCook)
	: CulturesToCook(InAllCulturesToCook)
{
}

FString FLocalizationCookArtifact::GetArtifactName() const
{
	return TEXT("LocalizationCookArtifact");
}

void FLocalizationCookArtifact::StoreDataInOplog(UE::Cook::Artifact::FStoreDataInOplogContext& Context)
{
	FString WorkspaceRoot = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir() / TEXT(".."));
	FPaths::NormalizeDirectoryName(WorkspaceRoot);
	if (!WorkspaceRoot.EndsWith(TEXT("/")))
	{
		WorkspaceRoot += TEXT("/");
	}

	TArray<FString> LocFiles;
	TArray<FString> FoundFiles;
	IFileManager::Get().FindFilesRecursive(FoundFiles, *FPaths::ProjectDir(), TEXT("*.locres"), true, false);
	IFileManager::Get().FindFilesRecursive(FoundFiles, *FPaths::ProjectDir(), TEXT("*.locmeta"), true, false, false);

	LocFiles.Reserve(FoundFiles.Num());
	for (FString& FilePath : FoundFiles)
	{
		FPaths::NormalizeFilename(FilePath);
		FilePath = FPaths::ConvertRelativePathToFull(FilePath);
		if (FilePath.StartsWith(WorkspaceRoot, ESearchCase::IgnoreCase))
		{
			LocFiles.Add(FilePath.Mid(WorkspaceRoot.Len()));
		}
	}

	UE_LOG(LogLocalizationCookArtifact, Log, TEXT("Cook.Localization: found %d files."), LocFiles.Num());

	// ---- Write oplog op ----
	FCbWriter Writer;
	Writer.BeginObject();

	Writer.BeginArray("files");
	for (const FString& File : LocFiles)
	{
		Writer << File;
	}
	Writer.EndArray();
	Writer.BeginArray("cultures");
	for (const FString& Culture : CulturesToCook)
	{
		Writer << Culture;
	}
	Writer.EndArray();

	Writer.EndObject();
	Context.AppendOp(TEXT("Cook.Localization"), Writer.Save().AsObject());
}

#endif // WITH_EDITOR
