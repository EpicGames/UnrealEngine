// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Cooker/CookArtifact.h"
#include "Containers/UnrealString.h"

class ITargetPlatform;

#if WITH_EDITOR
/**
 * CookArtifact that discovers all localization files (.locres, .locmeta) at the end of cook
 * and emits a "Cook.Localization" oplog op for the chunk assigner to consume.
 *
 * Two categories are emitted:
 *   files — all .locres and .locmeta files found in the project and plugin directories.
 *   cultures — the cultures that were enabled for the current cook.
 */
class FLocalizationCookArtifact : public UE::Cook::ICookArtifact
{
public:
	explicit FLocalizationCookArtifact(const TArray<FString>& InAllCulturesToCook);

	virtual FString GetArtifactName() const override;

	virtual void StoreDataInOplog(UE::Cook::Artifact::FStoreDataInOplogContext& Context) override;

private:
	TArray<FString> CulturesToCook;
};
#endif // WITH_EDITOR
