// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalizedAssetUtil.h"

#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "LocalizationSourceControlUtil.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "ObjectTools.h"
#include "PackageHelperFunctions.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "UObject/TopLevelAssetPath.h"

DEFINE_LOG_CATEGORY_STATIC(LogLocalizedAssetUtil, Log, All);

bool FLocalizedAssetSCCUtil::SaveAssetWithSCC(const TSharedPtr<FLocalizationSCC>& InSourceControlInfo, UObject* InAsset)
{
	UPackage* const AssetPackage = InAsset->GetOutermost();

	if (!AssetPackage)
	{
		UE_LOGF(LogLocalizedAssetUtil, Error, "Unable to find package for %ls '%ls'.", *InAsset->GetClass()->GetName(), *InAsset->GetPathName());
		return false;
	}

	return SavePackageWithSCC(InSourceControlInfo, AssetPackage);
}

bool FLocalizedAssetSCCUtil::SaveAssetWithSCC(const TSharedPtr<FLocalizationSCC>& InSourceControlInfo, UObject* InAsset, const FString& InFilename)
{
	UPackage* const AssetPackage = InAsset->GetOutermost();

	if (!AssetPackage)
	{
		UE_LOGF(LogLocalizedAssetUtil, Error, "Unable to find package for %ls '%ls'.", *InAsset->GetClass()->GetName(), *InAsset->GetPathName());
		return false;
	}

	return SavePackageWithSCC(InSourceControlInfo, AssetPackage, InFilename);
}

bool FLocalizedAssetSCCUtil::SavePackageWithSCC(const TSharedPtr<FLocalizationSCC>& InSourceControlInfo, UPackage* InPackage)
{
	const FString PackageFileName = FPackageName::LongPackageNameToFilename(InPackage->GetPathName(), FPackageName::GetAssetPackageExtension());
	return SavePackageWithSCC(InSourceControlInfo, InPackage, PackageFileName);
}

bool FLocalizedAssetSCCUtil::SavePackageWithSCC(const TSharedPtr<FLocalizationSCC>& InSourceControlInfo, UPackage* InPackage, const FString& InFilename)
{
	const bool bPackageExistedOnDisk = FPaths::FileExists(InFilename);

	// If the package already exists on disk, then we need to check it out before writing to it
	if (bPackageExistedOnDisk && InSourceControlInfo.IsValid())
	{
		FText SCCErrorStr;
		if (!InSourceControlInfo->CheckOutFile(InFilename, SCCErrorStr))
		{
			UE_LOGF(LogLocalizedAssetUtil, Error, "Failed to check-out package '%ls' at '%ls'. %ls", *InPackage->GetPathName(), *InFilename, *SCCErrorStr.ToString());
		}
	}

	if (!SavePackageHelper(InPackage, InFilename))
	{
		UE_LOGF(LogLocalizedAssetUtil, Error, "Unable to save updated package '%ls' to '%ls'", *InPackage->GetPathName(), *InFilename);
		return false;
	}

	// If the package didn't exist on disk, then we need to check it out after writing to it (which will perform an add)
	if (!bPackageExistedOnDisk && InSourceControlInfo.IsValid())
	{
		FText SCCErrorStr;
		if (!InSourceControlInfo->CheckOutFile(InFilename, SCCErrorStr))
		{
			UE_LOGF(LogLocalizedAssetUtil, Error, "Failed to check-out package '%ls' at '%ls'. %ls", *InPackage->GetPathName(), *InFilename, *SCCErrorStr.ToString());
		}
	}

	return true;
}

bool FLocalizedAssetSCCUtil::DeleteAssetWithSCC(const TSharedPtr<FLocalizationSCC>& InSourceControlInfo, UObject* InAsset)
{
	UPackage* const AssetPackage = InAsset->GetOutermost();

	if (!AssetPackage)
	{
		UE_LOGF(LogLocalizedAssetUtil, Error, "Unable to find package for %ls '%ls'.", *InAsset->GetClass()->GetName(), *InAsset->GetPathName());
		return false;
	}

	return DeletePackageWithSCC(InSourceControlInfo, AssetPackage);
}

bool FLocalizedAssetSCCUtil::DeletePackageWithSCC(const TSharedPtr<FLocalizationSCC>& InSourceControlInfo, UPackage* InPackage)
{
	if (!ObjectTools::DeleteSingleObject(InPackage, /*bPerformReferenceCheck*/false))
	{
		return false;
	}

	TArray<UPackage*> DeletedPackages;
	DeletedPackages.Add(InPackage);
	ObjectTools::CleanupAfterSuccessfulDelete(DeletedPackages, /*bPerformReferenceCheck*/false);

	return true;
}

bool FLocalizedAssetSCCUtil::SaveFileWithSCC(const TSharedPtr<FLocalizationSCC>& InSourceControlInfo, const FString& InFilename, const FSaveFileCallback& InSaveFileCallback)
{
	const bool bFileExistedOnDisk = FPaths::FileExists(InFilename);

	// If the file already exists on disk, then we need to check it out before writing to it
	if (bFileExistedOnDisk && InSourceControlInfo.IsValid())
	{
		FText SCCErrorStr;
		if (!InSourceControlInfo->CheckOutFile(InFilename, SCCErrorStr))
		{
			UE_LOGF(LogLocalizedAssetUtil, Error, "Failed to check-out file at '%ls'. %ls", *InFilename, *SCCErrorStr.ToString());
		}
	}

	if (!InSaveFileCallback(InFilename))
	{
		UE_LOGF(LogLocalizedAssetUtil, Error, "Unable to save updated file to '%ls'", *InFilename);
		return false;
	}

	// If the file didn't exist on disk, then we need to check it out after writing to it (which will perform an add)
	if (!bFileExistedOnDisk && InSourceControlInfo.IsValid())
	{
		FText SCCErrorStr;
		if (!InSourceControlInfo->CheckOutFile(InFilename, SCCErrorStr))
		{
			UE_LOGF(LogLocalizedAssetUtil, Error, "Failed to check-out file at '%ls'. %ls", *InFilename, *SCCErrorStr.ToString());
		}
	}

	return true;
}


bool FLocalizedAssetUtil::GetAssetsByPathAndClass(IAssetRegistry& InAssetRegistry, const FName InPackagePath, const FTopLevelAssetPath& InClassName, const bool bIncludeLocalizedAssets, TArray<FAssetData>& OutAssets)
{
	TArray<FName> PackagePaths;
	PackagePaths.Add(InPackagePath);

	return GetAssetsByPathAndClass(InAssetRegistry, PackagePaths, InClassName, bIncludeLocalizedAssets, OutAssets);
}

bool FLocalizedAssetUtil::GetAssetsByPathAndClass(IAssetRegistry& InAssetRegistry, const TArray<FName>& InPackagePaths, const FTopLevelAssetPath& InClassName, const bool bIncludeLocalizedAssets, TArray<FAssetData>& OutAssets)
{
	FARFilter AssetFilter;
	AssetFilter.PackagePaths = InPackagePaths;
	AssetFilter.bRecursivePaths = true;
	AssetFilter.ClassPaths.Add(InClassName);
	AssetFilter.bRecursiveClasses = true;

	if (!InAssetRegistry.GetAssets(AssetFilter, OutAssets))
	{
		return false;
	}

	if (!bIncludeLocalizedAssets)
	{
		// We need to manually exclude any localized assets
		OutAssets.RemoveAll([&](const FAssetData& InAssetData) -> bool
		{
			return FPackageName::IsLocalizedPackage(InAssetData.PackageName.ToString());
		});
	}

	return true;
}
