// Copyright Epic Games, Inc. All Rights Reserved.


#include "MRUFavoritesList.h"

#include "CoreTypes.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Misc/PackageName.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Notifications/SNotificationList.h"

FMainMRUFavoritesList::FMainMRUFavoritesList()
	: FMRUList( TEXT("MRU") )
	, INIFavoritesSection(TEXT("FavoriteFiles"))
{
}
FMainMRUFavoritesList::FMainMRUFavoritesList(const FString& IniSectionOverride, const int32 InitMaxItems)
	: FMRUList(IniSectionOverride, InitMaxItems)
	, INIFavoritesSection(TEXT("FavoriteFiles"))
{
}

FMainMRUFavoritesList::FMainMRUFavoritesList(const FString& IniSectionOverride, const FString& IniFavoritesSectionOverride, const int32 InitMaxItems)
	: FMRUList(IniSectionOverride, InitMaxItems)
	, INIFavoritesSection(IniFavoritesSectionOverride)
{
}

/** Destructor */
FMainMRUFavoritesList::~FMainMRUFavoritesList()
{
	FavoriteItems.Empty();
}

/** Populate MRU/Favorites list by reading saved values from the relevant INI file */
void FMainMRUFavoritesList::ReadFromINI()
{
	// Read in the MRU items
	InternalReadINI( Items, INISection, TEXT("MRUItem"), GetMaxItems() );

	// Read in the Favorite items
	InternalReadINI( FavoriteItems, INIFavoritesSection, TEXT("FavoritesItem"), MaxItems );
}

/** Save off the state of the MRU and favorites lists to the relevant INI file */
void FMainMRUFavoritesList::WriteToINI() const
{
	InternalWriteINI( Items, INISection, TEXT("MRUItem") );
	InternalWriteINI( FavoriteItems, INIFavoritesSection, TEXT("FavoritesItem") );
}

/**
 * Add a new item to the favorites list
 *
 * @param	Item	Package name of the item to add to the favorites list
 */
void FMainMRUFavoritesList::AddFavoritesItem( const FString& Item )
{
	if (ensureMsgf(FPackageName::IsValidLongPackageName(Item), TEXT("Item is not a valid long package name: '%s'"), *Item))
	{
		// Only add the item if it isn't already a favorite!
		if ( !FavoriteItems.Contains( Item ) )
		{
			FavoriteItems.Insert( Item, 0 );
			WriteToINI();
		}
	}
}

/**
 * Remove an item from the favorites list
 *
 * @param	Item	Package name of the item to remove from the favorites list
 */
void FMainMRUFavoritesList::RemoveFavoritesItem( const FString& Item )
{
	if (ensureMsgf(FPackageName::IsValidLongPackageName(Item), TEXT("Item is not a valid long package name: '%s'"), *Item))
	{
		const int32 ItemIndex = FavoriteItems.Find( Item );
		if ( ItemIndex != INDEX_NONE )
		{
			FavoriteItems.RemoveAt( ItemIndex );
			WriteToINI();
		}
	}
}

/**
 * Moves the specified favorites item to the head of the list
 *
 * @param	Item	Package name of the item to move
 */
void FMainMRUFavoritesList::MoveFavoritesItemToHead(const FString& Item)
{
	if (ensureMsgf(FPackageName::IsValidLongPackageName(Item), TEXT("Item is not a valid long package name: '%s'"), *Item))
	{
		if ( FavoriteItems.RemoveSingle(Item) == 1 )
		{
			FavoriteItems.Insert( Item, 0 );
			WriteToINI();
		}
	}
}

/**
 * Returns whether a package name is favorited or not
 *
 * @param	Item	Package name of the item to check
 *
 * @return	true if the provided item is in the favorite's list; false if it is not
 */
bool FMainMRUFavoritesList::ContainsFavoritesItem( const FString& Item ) const
{
	if (ensureMsgf(FPackageName::IsValidLongPackageName(Item), TEXT("Item is not a valid long package name: '%s'"), *Item))
	{
		return FavoriteItems.Contains( Item );
	}
	return false;
}

/**
 * Return the favorites item specified by the provided index
 *
 * @param	ItemIndex	Index of the favorites item to return
 *
 * @return	The favorites item specified by the provided index
 */
FString FMainMRUFavoritesList::GetFavoritesItem( int32 ItemIndex ) const
{
	check( FavoriteItems.IsValidIndex( ItemIndex ) );
	return FavoriteItems[ ItemIndex ];
}

/**
 * Verifies that the favorites item specified by the provided index still exists. If it does not, the item
 * is removed from the favorites list and the user is notified.
 *
 * @param	ItemIndex	Index of the favorites item to check
 *
 * @return	true if the item specified by the index was verified and still exists; false if it does not
 */
bool FMainMRUFavoritesList::VerifyFavoritesFile( int32 ItemIndex )
{
	check( FavoriteItems.IsValidIndex( ItemIndex ) );
	const FString& CurPackageName = FavoriteItems[ ItemIndex ];

	FString CurFileName;
	bool bSuccess = FPackageName::TryConvertLongPackageNameToFilename(CurPackageName, CurFileName, FPackageName::GetMapPackageExtension());

	// If the file doesn't exist any more, remove it from the favorites list and alert the user
	if ( !bSuccess || IFileManager::Get().FileSize(*CurFileName) == INDEX_NONE )
	{
		FNotificationInfo Info(FText::Format(NSLOCTEXT("UnrealEd", "Error_FavoritesFileDoesNotExist", "Map '{0}' does not exist - it will be removed from the Favorites list."), FText::FromString(CurPackageName)));
		Info.bUseThrobber = false;
		Info.ExpireDuration = 8.0f;
		FSlateNotificationManager::Get().AddNotification(Info)->SetCompletionState(SNotificationItem::CS_Fail);
		RemoveFavoritesItem(CurPackageName);
		return false;
	}

	return true;
}

/**
* Supplies an optional delegate that can be used to filter a given MRUFavorites item
* Useful for dynamically verifying which items should be utilized at a given time
*
* @param DoesMRUFavoritesItemPassFilterDelegate The delegate to use
*/
void FMainMRUFavoritesList::RegisterDoesMRUFavoritesItemPassFilterDelegate(FDoesMRUFavoritesItemPassFilter DoesMRUFavoritesItemPassFilterDelegate)
{
	DoesMRUFavoritesItemPassFilter = DoesMRUFavoritesItemPassFilterDelegate;
}

/**
* Unregisters the optional filter delegate
*/
void FMainMRUFavoritesList::UnregisterDoesMRUFavoritesItemPassFilterDelegate()
{
	DoesMRUFavoritesItemPassFilter = FDoesMRUFavoritesItemPassFilter();
}

/**
* Checks the favorites item specified by the provided index against the optional 'DoesMRUFavoritesItemPassFilterDelegate'.
*
* @param ItemIndex Index of the favorites item to check
*
* @return true if the item specified by the index passes the filter or if no filter has been provided; false if it does not pass the filter
*/
bool FMainMRUFavoritesList::FavoritesItemPassesCurrentFilter(int32 ItemIndex) const
{
	check(FavoriteItems.IsValidIndex(ItemIndex));
	if (DoesMRUFavoritesItemPassFilter.IsBound())
	{
		const FString& FavoriteItem = FavoriteItems[ItemIndex];
		return DoesMRUFavoritesItemPassFilter.Execute(FavoriteItem);
	}
	return true;
}

/**
* Checks the MRU item specified by the provided index against the optional 'DoesMRUFavoritesItemPassFilterDelegate'.
*
* @param ItemIndex Index of the MRU item to check
*
* @return true if the item specified by the index passes the filter or if no filter has been provided; false if it does not pass the filter
*/
bool FMainMRUFavoritesList::MRUItemPassesCurrentFilter(int32 ItemIndex) const
{
	check(Items.IsValidIndex(ItemIndex));
	if (DoesMRUFavoritesItemPassFilter.IsBound())
	{
		const FString& MRUItem = Items[ItemIndex];
		return DoesMRUFavoritesItemPassFilter.Execute(MRUItem);
	}
	return true;
}
