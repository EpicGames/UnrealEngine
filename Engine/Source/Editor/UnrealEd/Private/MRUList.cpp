// Copyright Epic Games, Inc. All Rights Reserved.

#include "MRUList.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "CoreGlobals.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Logging/MessageLog.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreMiscDefines.h"
#include "Misc/PackageName.h"
#include "Modules/ModuleManager.h"
#include "UObject/NameTypes.h"

FMRUList::FMRUList(const FString& InINISection, const int32 InitMaxItems)
	:	MaxItems( InitMaxItems ),
		INISection( InINISection )
{
}


FMRUList::~FMRUList()
{
	Items.Empty();
}


void FMRUList::Cull()
{
	while( Items.Num() > GetMaxItems() )
	{
		Items.RemoveAt( Items.Num()-1 );
	}
}

void FMRUList::ClearMRUItems()
{
	Items.Empty();

	WriteToINI();
}

void FMRUList::ReadFromINI()
{
	InternalReadINI( Items, INISection, TEXT("MRUItem"), GetMaxItems() );
}


void FMRUList::WriteToINI() const
{
	InternalWriteINI( Items, INISection, TEXT("MRUItem") );
}


void FMRUList::MoveToTop(int32 InItem)
{
	check( InItem > -1 && InItem < Items.Num() );

	TArray<FString> WkArray;
	WkArray = Items;

	const FString Save = WkArray[InItem];
	WkArray.RemoveAt( InItem );

	Items.Empty();
	new(Items)FString( *Save );
	Items += WkArray;
}


void FMRUList::AddMRUItem(const FString& InItem)
{
	checkf(FPackageName::IsValidLongPackageName(InItem), TEXT("FMRUList::AddMRUItem called with an invalid package name: %s"), *InItem);

	// See if the item already exists in the list.  If so,
	// move it to the top of the list and leave.
	const int32 ItemIndex = Items.Find(InItem);
	if ( ItemIndex != INDEX_NONE )
	{
		MoveToTop( ItemIndex );
	}
	else
	{
		// Item is new, so add it to the bottom of the list.
		if(InItem.Len() )
		{
			new(Items) FString( *InItem );
			MoveToTop( Items.Num()-1 );
		}

		Cull();
	}
	
	WriteToINI();
}


int32 FMRUList::FindMRUItemIdx(const FString& InItem) const
{
	check(FPackageName::IsValidLongPackageName(InItem));

	for( int32 mru = 0 ; mru < Items.Num() ; ++mru )
	{
		if( Items[mru] == InItem )
		{
			return mru;
		}
	}

	return INDEX_NONE;
}


void FMRUList::RemoveMRUItem(const FString& InItem)
{
	RemoveMRUItem( FindMRUItemIdx( InItem ) );
}


void FMRUList::RemoveMRUItem(int32 InItem)
{
	// Find the item and remove it.
	check( InItem > -1 && InItem < GetMaxItems() );
	Items.RemoveAt( InItem );
}


void FMRUList::InternalReadINI( TArray<FString>& OutItems, const FString& INISection, const FString& INIKeyBase, int32 NumElements )
{
	// Clear existing items
	OutItems.Empty();

	bool bConvertedToNewFormat = false;

	// Iterate over the maximum number of provided elements
	for( int32 ItemIdx = 0 ; ItemIdx < NumElements ; ++ItemIdx )
	{
		// Try to find data for a key formed as "INIKeyBaseItemIdx" for the provided INI section. If found, add the data to the output item array.
		FString CurItem;
		if (GConfig->GetString(*INISection, *FString::Printf(TEXT("%s%d"), *INIKeyBase, ItemIdx), CurItem, GEditorPerProjectIni))
		{
			if (!FPackageName::IsValidLongPackageName(CurItem))
			{
				FString NewItem;
				if (FPackageName::TryConvertFilenameToLongPackageName(CurItem, NewItem))
				{
					if (NewItem != CurItem)
					{
						CurItem = NewItem;
						bConvertedToNewFormat = true;
					}
					
					OutItems.AddUnique(CurItem);
				}
				else
				{
					bConvertedToNewFormat = true;
				}
			}
			else
			{
				OutItems.AddUnique(CurItem);
			}
		}
	}

	if (bConvertedToNewFormat)
	{
		InternalWriteINI(OutItems, INISection, INIKeyBase);
	}
}


void FMRUList::InternalWriteINI( const TArray<FString>& InItems, const FString& INISection, const FString& INIKeyBase )
{
	if (FConfigFile* ConfigFile = GConfig->Find(GEditorPerProjectIni))
	{
		ConfigFile->Remove(*INISection);

		for (int32 ItemIdx = 0; ItemIdx < InItems.Num(); ++ItemIdx)
		{
			ConfigFile->SetString(*INISection, *FString::Printf(TEXT("%s%d"), *INIKeyBase, ItemIdx), *InItems[ItemIdx]);
		}

		GConfig->Flush(false, GEditorPerProjectIni);
	}
}


bool FMRUList::VerifyMRUFile(int32 InItem, FString& OutPackageName)
{
	check( InItem > -1 && InItem < GetMaxItems() );

	// Handle redirector
	const FString OriginalPackageName = Items[InItem];
	const FSoftObjectPath OriginalObjectPath = FSoftObjectPath(FTopLevelAssetPath(*OriginalPackageName, *FPackageName::GetShortName(OriginalPackageName)));
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	const FSoftObjectPath RedirectedObjectPath = AssetRegistryModule.Get().GetRedirectedObjectPath(OriginalObjectPath);

	FString PackageName;
	FString RedirectedPackageName;
	if (RedirectedObjectPath != OriginalObjectPath && FPackageName::TryConvertFilenameToLongPackageName(RedirectedObjectPath.ToString(), RedirectedPackageName))
	{
		PackageName = RedirectedPackageName;
	}
	else
	{
		PackageName = OriginalPackageName;
	}

	FString Filename;
	bool bSuccess = FPackageName::TryConvertLongPackageNameToFilename(PackageName, Filename, FPackageName::GetMapPackageExtension());

	// If the file doesn't exist, tell the user about it, remove the file from the list
	if( !bSuccess || IFileManager::Get().FileSize( *Filename) == INDEX_NONE )
	{
		FMessageLog EditorErrors("EditorErrors");
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("PackageName"), FText::FromString(PackageName));
		EditorErrors.Warning(FText::Format( NSLOCTEXT("MRUList", "Error_FileDoesNotExist", "Map '{PackageName}' does not exist.  It will be removed from the recent items list."), Arguments ) );
		EditorErrors.Notify(NSLOCTEXT("MRUList", "Notification_PackageDoesNotExist", "Map does not exist! Removed from recent items list!"));
		RemoveMRUItem( InItem );
		WriteToINI();

		return false;
	}
	else if (!RedirectedPackageName.IsEmpty())
	{
		// Remove old path, add new path
		RemoveMRUItem( InItem );
		// Note: WriteToINI is called by AddMRUItem
		AddMRUItem(PackageName);
	}
	else
	{
		// Otherwise, move the file to the top of the list
		MoveToTop( InItem );
		WriteToINI();
	}

	OutPackageName = PackageName;

	return true;
}
