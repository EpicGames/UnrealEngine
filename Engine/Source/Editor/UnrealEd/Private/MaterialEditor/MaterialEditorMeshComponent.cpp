// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialEditor/MaterialEditorMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "PhysicsEngine/BodySetup.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialEditorMeshComponent)

FBoxSphereBounds UMaterialEditorMeshComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	if (GetStaticMesh())
	{
		// Graphics bounds.
		FBoxSphereBounds NewBounds = GetStaticMesh()->GetBounds().TransformBy(LocalToWorld);

		// Add bounds of collision geometry (if present).
		if (GetStaticMesh()->GetBodySetup())
		{
			// Use more accurate but expensive bounds function for the material editor only
			FBoxSphereBounds AggGeomBounds;
			GetStaticMesh()->GetBodySetup()->AggGeom.CalcBoxSphereBounds(AggGeomBounds, LocalToWorld);
			if (AggGeomBounds.SphereRadius != 0.f)
			{
				NewBounds = Union(NewBounds,AggGeomBounds);
			}
		}

		NewBounds.BoxExtent *= BoundsScale;
		NewBounds.SphereRadius *= BoundsScale;

		return NewBounds;
	}
	else
	{
		return FBoxSphereBounds(LocalToWorld.GetLocation(), FVector::ZeroVector, 0.f);
	}
}
