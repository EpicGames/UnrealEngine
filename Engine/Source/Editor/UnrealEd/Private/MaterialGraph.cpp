// Copyright Epic Games, Inc. All Rights Reserved.

/////////////////////////////////////////////////////
// UMaterialGraph

#include "MaterialGraph/MaterialGraph.h"
#include "MaterialGraph/MaterialGraphNode_Operator.h"
#include "MaterialGraph/MaterialGraphNode_Comment.h"
#include "MaterialGraph/MaterialGraphNode_Custom.h"
#include "MaterialGraph/MaterialGraphNode_Composite.h"
#include "MaterialGraph/MaterialGraphNode_PinBase.h"
#include "MaterialGraph/MaterialGraphNode.h"
#include "MaterialGraph/MaterialGraphNode_Root.h"
#include "MaterialGraph/MaterialGraphSchema.h"

#include "Materials/MaterialAttributeDefinitionMap.h"
#include "Materials/MaterialExpressionComment.h"
#include "Materials/MaterialExpressionComposite.h"
#include "Materials/MaterialExpressionCustom.h"
#include "Materials/MaterialExpressionPinBase.h"
#include "Materials/MaterialExpressionOperator.h"
#include "Materials/MaterialExpressionFunctionOutput.h"
#include "Materials/MaterialExpressionCustomOutput.h"
#include "Materials/MaterialExpressionReroute.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialExpressionNamedReroute.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialInsights.h"

#include "MaterialGraph/MaterialShaderValueTypeObject.h"
#include "MaterialGraphNode_Knot.h"

#include "Kismet2/BlueprintEditorUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialGraph)

#define LOCTEXT_NAMESPACE "MaterialGraph"

UMaterialGraph::UMaterialGraph(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UMaterialGraph::UMaterialGraph() = default;

UMaterialGraph::UMaterialGraph(FVTableHelper& Helper)
	: Super(Helper)
{
}

UMaterialGraph::~UMaterialGraph()
{
}

void UMaterialGraph::RebuildGraph()
{
	check(Material);

	// Pre-group expressions & comments per subgraph to avoid unnecessary iteration over all material expressions
	TMap<UMaterialExpression*, TArray<UMaterialExpression*>> SubgraphExpressionMap;
	TMap<UMaterialExpression*, TArray<UMaterialExpressionComment*>> SubgraphCommentMap;
	for (UMaterialExpression* Expression : Material->GetExpressions())
	{
		SubgraphExpressionMap.FindOrAdd(Expression->SubgraphExpression).Add(Expression);
	}
	for (UMaterialExpressionComment* Comment : Material->GetEditorComments())
	{
		if (Comment)
		{
			SubgraphCommentMap.FindOrAdd(Comment->SubgraphExpression).Add(Comment);
		}
	}

	RebuildGraphInternal(SubgraphExpressionMap, SubgraphCommentMap);
}

template<typename NodeType>
static UMaterialGraphNode* InitExpressionNewNode(UMaterialGraph* Graph, UMaterialExpression* Expression, bool bUserInvoked)
{
	UMaterialGraphNode* NewNode = nullptr;

	FGraphNodeCreator<NodeType> NodeCreator(*Graph);
	if (bUserInvoked)
	{
		NewNode = NodeCreator.CreateUserInvokedNode();
	}
	else
	{
		NewNode = NodeCreator.CreateNode(false);
	}
	NewNode->MaterialExpression = Expression;
	NewNode->RealtimeDelegate = Graph->RealtimeDelegate;
	NewNode->MaterialDirtyDelegate = Graph->MaterialDirtyDelegate;
	Expression->GraphNode = NewNode;
	Expression->SubgraphExpression = Graph->SubgraphExpression;
	NodeCreator.Finalize();

	return NewNode;
}

namespace Private
{

UMaterialGraphNode_Base* FindGraphNodeForObject(const UObject* Object)
{
	if (const UMaterialExpression* MaterialExpression = Cast<UMaterialExpression>(Object))
	{
		if (MaterialExpression->GraphNode)
		{
			return CastChecked<UMaterialGraphNode_Base>(MaterialExpression->GraphNode);
		}
	}
	else if (const UMaterial* Material = Cast<UMaterial>(Object))
	{
		const UMaterialGraph* MaterialGraph = Material->MaterialGraph;
		if (MaterialGraph)
		{
			return MaterialGraph->RootNode;
		}
	}
	return nullptr;
}
} // namespace Private

void UMaterialGraph::UpdatePinTypes()
{
	if (!Material || !Material->IsUsingNewHLSLGenerator())
	{
		return;
	}

	const FMaterialInsights* Insight = Material->MaterialInsight.Get();
	if (!Insight)
	{
		return;
	}

	// Clear any previously-set value type objects, preserving UEnum objects used for inline editing.
	for (UEdGraphNode* Node : Nodes)
	{
		for (UEdGraphPin* Pin : Node->Pins)
		{
			if (Cast<UMaterialShaderValueTypeObject>(Pin->PinType.PinSubCategoryObject.Get()))
			{
				Pin->PinType.PinSubCategoryObject = nullptr;
			}
		}
	}

	for (const FMaterialInsights::FConnectionInsight& ConnectionInsight : Insight->ConnectionInsights)
	{
		UMaterialShaderValueTypeObject* ValueTypeObj = UMaterialShaderValueTypeObject::Get(ConnectionInsight.ValueType);

		if (UMaterialGraphNode_Base* InputNode = ::Private::FindGraphNodeForObject(ConnectionInsight.InputObject))
		{
			const int32 InputIndex = InputNode->GetSourceIndexForInputIndex(ConnectionInsight.InputIndex);
			if (UEdGraphPin* InputPin = InputNode->GetInputPin(InputIndex))
			{
				InputPin->PinType.PinSubCategoryObject = ValueTypeObj;
			}
		}

		if (UMaterialGraphNode_Base* OutputNode = ::Private::FindGraphNodeForObject(ConnectionInsight.OutputExpression))
		{
			if (UEdGraphPin* OutputPin = OutputNode->GetOutputPin(ConnectionInsight.OutputIndex))
			{
				OutputPin->PinType.PinSubCategoryObject = ValueTypeObj;
			}
		}
	}
}

void UMaterialGraph::RebuildGraphInternal(const TMap<UMaterialExpression*, TArray<UMaterialExpression*>>& SubgraphExpressionMap, const TMap<UMaterialExpression*, TArray<UMaterialExpressionComment*>>& SubgraphCommentMap)
{
	Modify();

	RemoveAllNodes();

	if (!MaterialFunction && !SubgraphExpression)
	{
		// This needs to be done before building the new material inputs to guarantee that the shading model field is up to date
		Material->RebuildShadingModelField();

		// Initialize the material input list.
		MaterialInputs.Add(FMaterialInputInfo(FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(MP_BaseColor, Material), MP_BaseColor, LOCTEXT("BaseColorToolTip", "Defines the overall color of the Material. Each channel is automatically clamped between 0 and 1")));
		MaterialInputs.Add(FMaterialInputInfo(FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(MP_Metallic, Material), MP_Metallic, LOCTEXT("MetallicToolTip", "Controls how \"metal-like\" your surface looks like")));
		MaterialInputs.Add(FMaterialInputInfo(FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(MP_Specular, Material), MP_Specular, LOCTEXT("SpecularToolTip", "Used to scale the current amount of specularity on non-metallic surfaces and is a value between 0 and 1, default at 0.5")));
		MaterialInputs.Add(FMaterialInputInfo(FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(MP_Roughness, Material), MP_Roughness, LOCTEXT("RoughnessToolTip", "Controls how rough the Material is. Roughness of 0 (smooth) is a mirror reflection and 1 (rough) is completely matte or diffuse")));
		MaterialInputs.Add(FMaterialInputInfo(FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(MP_Anisotropy, Material), MP_Anisotropy, LOCTEXT("AnisotropyToolTip", "Determines the extent the specular highlight is stretched along the tangent. Anisotropy from 0 to 1 results in a specular highlight that stretches from uniform to maximally stretched along the tangent direction.")));
		MaterialInputs.Add(FMaterialInputInfo(FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(MP_EmissiveColor, Material), MP_EmissiveColor, LOCTEXT("EmissiveToolTip", "Controls which parts of your Material will appear to glow")));
		MaterialInputs.Add(FMaterialInputInfo(FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(MP_Opacity, Material), MP_Opacity, LOCTEXT("OpacityToolTip", "Controls the translucency of the Material. When Substrate is enabled, this input override alpha blending over the background when AlphaComposite blend mode is selected.")));
		MaterialInputs.Add(FMaterialInputInfo(FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(MP_OpacityMask, Material), MP_OpacityMask, LOCTEXT("OpacityMaskToolTip", "When in Masked mode, a Material is either completely visible or completely invisible")));
		MaterialInputs.Add(FMaterialInputInfo(FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(MP_Normal, Material), MP_Normal, LOCTEXT("NormalToolTip", "Takes the input of a normal map")));
		MaterialInputs.Add(FMaterialInputInfo(FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(MP_Tangent, Material), MP_Tangent, LOCTEXT("TangentToolTip", "Takes the input of a tangent map. Useful for specifying anisotropy direction.")));
		MaterialInputs.Add(FMaterialInputInfo(FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(MP_WorldPositionOffset, Material), MP_WorldPositionOffset, LOCTEXT("WorldPositionOffsetToolTip", "Allows for the vertices of a mesh to be manipulated in world space by the Material")));
		MaterialInputs.Add(FMaterialInputInfo(FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(MP_SubsurfaceColor, Material), MP_SubsurfaceColor, LOCTEXT("SubsurfaceToolTip", "Allows you to add a color to your Material to simulate shifts in color when light passes through the surface")));
		MaterialInputs.Add(FMaterialInputInfo(FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(MP_CustomData0, Material), MP_CustomData0, FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(MP_CustomData0, Material)));
		MaterialInputs.Add(FMaterialInputInfo(FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(MP_CustomData1, Material), MP_CustomData1, FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(MP_CustomData1, Material)));
		MaterialInputs.Add(FMaterialInputInfo(FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(MP_AmbientOcclusion, Material), MP_AmbientOcclusion, LOCTEXT("AmbientOcclusionToolTip", "Simulate the self-shadowing that happens within crevices of a surface, or of a volume for volumetric clouds only")));
		MaterialInputs.Add(FMaterialInputInfo(FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(MP_Refraction, Material), MP_Refraction, LOCTEXT("RefractionToolTip", "Takes in a texture or value that simulates the index of refraction of the surface")));

		for (int32 UVIndex = 0; UVIndex < UE_ARRAY_COUNT(Material->GetEditorOnlyData()->CustomizedUVs); UVIndex++)
		{
			MaterialInputs.Add(FMaterialInputInfo(FText::FromString(FString::Printf(TEXT("Customized UV%u"), UVIndex)), (EMaterialProperty)(MP_CustomizedUVs0 + UVIndex), 
				FText::Format(LOCTEXT("CustomizedUVTooltip", "Customized UV{0} allows running calculations in the vertex shader to increase performance over running them per-pixel.\n"
				"Use the TexCoords{0} node (or ConstCoordinate {0} in the TextureSample node) to access the interpolated custom UV in the pixel shader."), UVIndex)));
		}

		MaterialInputs.Add(FMaterialInputInfo(FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(MP_PixelDepthOffset, Material), MP_PixelDepthOffset, LOCTEXT("PixelDepthOffsetToolTip", "Pixel Depth Offset")));
		MaterialInputs.Add(FMaterialInputInfo(FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(MP_ShadingModel, Material), MP_ShadingModel, LOCTEXT("ShadingModelToolTip", "Selects which shading model should be used per pixel")));
		MaterialInputs.Add(FMaterialInputInfo(FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(MP_SurfaceThickness, Material), MP_SurfaceThickness, LOCTEXT("SurfaceThicknessToolTip", "Defines the surface's thickness when IsThinSurface is enabled on this node. Only available when Substrate is enabled for the project.")));
		MaterialInputs.Add(FMaterialInputInfo(FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(MP_Displacement, Material), MP_Displacement, LOCTEXT("DisplacementToolTip", "Specifies scalar vertex displacement."))); 

		//^^^ New material properties go above here. ^^^^
		MaterialInputs.Add(FMaterialInputInfo(LOCTEXT("MaterialAttributes", "Material Attributes"), MP_MaterialAttributes, LOCTEXT("MaterialAttributesToolTip", "Material Attributes")));

		// FrontMaterial is always added the latest for UX purpose
		MaterialInputs.Add(FMaterialInputInfo(FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(MP_FrontMaterial, Material), MP_FrontMaterial, LOCTEXT("FrontMaterialToolTip", "Specify the front facing material")));

		// Add Root Node
		{
			FGraphNodeCreator<UMaterialGraphNode_Root> NodeCreator(*this);
			RootNode = NodeCreator.CreateNode();
			RootNode->Material = Material;
			NodeCreator.Finalize();
		}
	}

	TArray<UMaterialExpression*> ChildSubGraphExpressions;

	// Composite's use reroutes under the hood that we don't want to create nodes for, gather their expressions for checking
	TArray<UMaterialExpressionReroute*> CompositeRerouteExpressions;
	if (UMaterialExpressionComposite* SubgraphParentComposite = Cast<UMaterialExpressionComposite>(SubgraphExpression))
	{
		CompositeRerouteExpressions = SubgraphParentComposite->GetCurrentReroutes();
	}

	if (const TArray<UMaterialExpression*>* Expressions = SubgraphExpressionMap.Find(SubgraphExpression))
	{
		for (UMaterialExpression* Expression : *Expressions)
		{
			if (!CompositeRerouteExpressions.Contains(Expression))
			{
				AddExpression(Expression, false);

				//@TODO: Make a better way to check if an expression represents a subgraph than by type.
				if (Cast<UMaterialExpressionComposite>(Expression))
				{
					ChildSubGraphExpressions.Add(Expression);
				}
			}
		}
	}

	if (const TArray<UMaterialExpressionComment*>* Comments = SubgraphCommentMap.Find(SubgraphExpression))
	{
		for (UMaterialExpressionComment* Comment : *Comments)
		{
			AddComment(Comment);
		}
	}

	for (UMaterialExpression* ChildSubGraphExpression : ChildSubGraphExpressions)
	{
		UMaterialGraph* Subgraph = AddSubGraph(ChildSubGraphExpression);

		if (UMaterialGraphNode_Composite* CompositeNode = Cast<UMaterialGraphNode_Composite>(ChildSubGraphExpression->GraphNode))
		{
			CompositeNode->BoundGraph = Subgraph;
			Subgraph->Rename(*CastChecked<UMaterialExpressionComposite>(CompositeNode->MaterialExpression)->SubgraphName);
		}

		Subgraph->RebuildGraphInternal(SubgraphExpressionMap, SubgraphCommentMap);
	}

	LinkGraphNodesFromMaterial();
}

UMaterialGraphNode* UMaterialGraph::AddExpression(UMaterialExpression* Expression, bool bUserInvoked)
{
	UMaterialGraphNode* Node = nullptr;
	if (Expression)
	{
		Modify();

		if (Expression->IsA(UMaterialExpressionReroute::StaticClass()))
		{
			Node = InitExpressionNewNode<UMaterialGraphNode_Knot>(this, Expression, false);
		}
		else if (Expression->IsA(UMaterialExpressionComposite::StaticClass()))
		{
			Node = InitExpressionNewNode<UMaterialGraphNode_Composite>(this, Expression, false);
		}
		else if (Expression->IsA(UMaterialExpressionPinBase::StaticClass()))
		{
			Node = InitExpressionNewNode<UMaterialGraphNode_PinBase>(this, Expression, false);
		}
		else if (Expression->IsA(UMaterialExpressionOperator::StaticClass()))
		{
			Node = InitExpressionNewNode<UMaterialGraphNode_Operator>(this, Expression, false);
		}
		else if (Expression->IsA(UMaterialExpressionCustom::StaticClass()))
		{
			// This is for backward compatibility. We don't want people's custom HLSL nodes to suddenly explode
			// when they open their materials for the first time with inline HLSL custom nodes working in the engine.
			// This ensures that only new nodes or nodes that are subsequently saved as uncollapsed show the code.
			UMaterialExpressionCustom* CustomNode = Cast<UMaterialExpressionCustom>(Expression);
			if (bUserInvoked)
				CustomNode->ShowCode = 1;

			Node = InitExpressionNewNode<UMaterialGraphNode_Custom>(this, Expression, bUserInvoked);
		}
		else 
		{
			Node = InitExpressionNewNode<UMaterialGraphNode>(this, Expression, bUserInvoked);
		}
	}

	if (Node && bUserInvoked)
	{
		UpdatePinTypes();
	}

	return Node;
}

UMaterialGraphNode_Comment* UMaterialGraph::AddComment(UMaterialExpressionComment* Comment, bool bIsUserInvoked)
{
	UMaterialGraphNode_Comment* NewComment = NULL;
	if (Comment)
	{
		Modify();
		FGraphNodeCreator<UMaterialGraphNode_Comment> NodeCreator(*this);
		if (bIsUserInvoked)
		{
			NewComment = NodeCreator.CreateUserInvokedNode(true);
		}
		else
		{
			NewComment = NodeCreator.CreateNode(false);
		}
		NewComment->MaterialExpressionComment = Comment;
		NewComment->MaterialDirtyDelegate = MaterialDirtyDelegate;
		Comment->GraphNode = NewComment;
		Comment->SubgraphExpression = SubgraphExpression;
		NodeCreator.Finalize();
	}

	return NewComment;
}

UMaterialGraph* UMaterialGraph::AddSubGraph(UMaterialExpression* InSubgraphExpression)
{
	UMaterialGraph* SubGraph = CastChecked<UMaterialGraph>(FBlueprintEditorUtils::CreateNewGraph(InSubgraphExpression->GraphNode, NAME_None, UMaterialGraph::StaticClass(), Schema));
	check(SubGraph);

	SubGraph->Material = Material;
	SubGraph->MaterialFunction = MaterialFunction;
	SubGraph->RealtimeDelegate = RealtimeDelegate;
	SubGraph->MaterialDirtyDelegate = MaterialDirtyDelegate;
	SubGraph->ToggleCollapsedDelegate = ToggleCollapsedDelegate;
	SubGraph->SubgraphExpression = InSubgraphExpression;
	SubGraphs.Add(SubGraph);

	// If we are a subgraph ourselves, mark that on the expression.
	InSubgraphExpression->SubgraphExpression = SubgraphExpression;

	return SubGraph;
}

void UMaterialGraph::LinkGraphNodesFromMaterial()
{
	struct ExpressionMatchesPredicate
	{
		ExpressionMatchesPredicate(UMaterialExpressionReroute* InCompositeReroute)
			: CompositeReroute(InCompositeReroute)
		{}

		bool operator()(const FCompositeReroute& Reroute)
		{
			return Reroute.Expression == CompositeReroute;
		}

		UMaterialExpressionReroute* CompositeReroute;
	};

	for (int32 Index = 0; Index < Nodes.Num(); ++Index)
	{
		Nodes[Index]->BreakAllNodeLinks();
	}

	if (RootNode)
	{
		// Use Material Inputs to make GraphNode Connections
		for (int32 Index = 0; Index < MaterialInputs.Num(); ++Index)
		{
			UEdGraphPin* InputPin = RootNode->GetInputPin(Index);
			auto ExpressionInput = MaterialInputs[Index].GetExpressionInput(Material);

			if (ExpressionInput.Expression)
			{
				if (UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(ExpressionInput.Expression->GraphNode))
				{
					InputPin->MakeLinkTo(GraphNode->GetOutputPin(GetValidOutputIndex(&ExpressionInput)));
				}
				else if (UMaterialExpressionReroute* CompositeReroute = Cast<UMaterialExpressionReroute>(ExpressionInput.Expression))
				{
					// This is an unseen composite reroute expression, find the actual expression output to connect to.
					UMaterialExpressionComposite* OwningComposite = CastChecked<UMaterialExpressionComposite>(CompositeReroute->SubgraphExpression);

					if (OwningComposite && OwningComposite->InputExpressions && OwningComposite->OutputExpressions)
					{
						UMaterialGraphNode* OutputGraphNode;
						int32 OutputPinIndex = OwningComposite->InputExpressions->ReroutePins.FindLastByPredicate(ExpressionMatchesPredicate(CompositeReroute));
						if (OutputPinIndex != INDEX_NONE)
						{
							OutputGraphNode = CastChecked<UMaterialGraphNode>(OwningComposite->InputExpressions->GraphNode);
						}
						else
						{
							// Output pin base in the subgraph cannot have outputs, if this reroute isn't in the inputs connect to composite's outputs
							OutputPinIndex = OwningComposite->OutputExpressions->ReroutePins.FindLastByPredicate(ExpressionMatchesPredicate(CompositeReroute));
							OutputGraphNode = CastChecked<UMaterialGraphNode>(OwningComposite->GraphNode);
						}
						InputPin->MakeLinkTo(OutputGraphNode->GetOutputPin(OutputPinIndex));
					}
				}
			}
		}
	}

	for (UMaterialExpression* Expression : Material->GetExpressions())
	{
		if (!Expression)
		{
			continue;
		}

		UMaterialGraphNode* MaterialGraphNode = Cast<UMaterialGraphNode>(Expression->GraphNode);
		if (!MaterialGraphNode)
		{
			continue;
		}

		for (UEdGraphPin* Pin : MaterialGraphNode->Pins)
		{
			if (Pin->Direction == EGPD_Input && Pin->PinType.PinCategory != UMaterialGraphSchema::PC_Exec)
			{
				// Implicitly generated property inputs are not returned by GetInputs(), so check index is within valid range.
				FExpressionInput* SourceInput = Expression->GetInput(Pin->SourceIndex);
				if (SourceInput && SourceInput->Expression)
				{
					// Unclear why this is null sometimes outside of composite reroute, but this is safer than crashing
					if (UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(SourceInput->Expression->GraphNode))
					{
						// if GraphNode is a material function call for a missing material function, it may not have any output pins
						UEdGraphPin* OutputPin = GraphNode->GetOutputPin(GetValidOutputIndex(Expression->GetInput(Pin->SourceIndex)));
						if (LIKELY(OutputPin))
						{
							Pin->MakeLinkTo(OutputPin);
						}
					}
					else if (UMaterialExpressionReroute* CompositeReroute = Cast<UMaterialExpressionReroute>(Expression->GetInput(Pin->SourceIndex)->Expression))
					{
						// This is an unseen composite reroute expression, find the actual expression output to connect to.
						UMaterialExpressionComposite* OwningComposite = Cast<UMaterialExpressionComposite>(CompositeReroute->SubgraphExpression);
						
						// If the input- and output expressions are valid, look for the output pin in the reroute lists and make a link to the current Pin.
						if (OwningComposite && OwningComposite->InputExpressions && OwningComposite->OutputExpressions)
						{
							UMaterialGraphNode* OutputGraphNode;
							int32 OutputPinIndex = OwningComposite->InputExpressions->ReroutePins.FindLastByPredicate(ExpressionMatchesPredicate(CompositeReroute));
							if (OutputPinIndex != INDEX_NONE)
							{
								OutputGraphNode = CastChecked<UMaterialGraphNode>(OwningComposite->InputExpressions->GraphNode);
							}
							else
							{
								// Output pin base in the subgraph cannot have outputs, if this reroute isn't in the inputs connect to composite's outputs
								OutputPinIndex = OwningComposite->OutputExpressions->ReroutePins.FindLastByPredicate(ExpressionMatchesPredicate(CompositeReroute));
								OutputGraphNode = CastChecked<UMaterialGraphNode>(OwningComposite->GraphNode);
							}
							Pin->MakeLinkTo(OutputGraphNode->GetOutputPin(OutputPinIndex));
						}
					}
				}
			}
		}
	}

	NotifyGraphChanged();

	UpdatePinTypes();
}

void UMaterialGraph::LinkMaterialExpressionsFromGraph()
{
	// Use GraphNodes to make Material Expression Connections
	for (int32 NodeIndex = 0; NodeIndex < Nodes.Num(); ++NodeIndex)
	{
		if (RootNode && RootNode == Nodes[NodeIndex])
		{
			// Setup Material's inputs from root node
			Material->Modify();
			Material->EditorX = RootNode->NodePosX;
			Material->EditorY = RootNode->NodePosY;
			
			for (UEdGraphPin* Pin : RootNode->Pins)
			{
				if (Pin->Direction == EGPD_Input && Pin->PinType.PinCategory != UMaterialGraphSchema::PC_Exec)
				{
					FExpressionInput& MaterialInput = MaterialInputs[Pin->SourceIndex].GetExpressionInput(Material);
					if (Pin->LinkedTo.Num() > 0)
					{
						UEdGraphPin* ConnectedPin = Pin->LinkedTo[0];
						UMaterialGraphNode* ConnectedNode = CastChecked<UMaterialGraphNode>(ConnectedPin->GetOwningNode());
						check(ConnectedPin->SourceIndex != INDEX_NONE);

						if (!ConnectedNode->MaterialExpression->IsExpressionConnected(&MaterialInput, ConnectedPin->SourceIndex))
						{
							ConnectedNode->MaterialExpression->Modify();
							MaterialInput.Connect(ConnectedPin->SourceIndex, ConnectedNode->MaterialExpression);
						}
					}
					else if (MaterialInput.Expression)
					{
						MaterialInput.Expression = NULL;
					}

					RootNode->UpdateInputUseConstant(Pin, MaterialInput.Expression == nullptr);
				}
			}
		}
		else
		{
			if (UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(Nodes[NodeIndex]))
			{
				// Need to be sure that we are changing the expression before calling modify -
				// triggers a rebuild of its preview when it is called
				UMaterialExpression* Expression = GraphNode->MaterialExpression;
				bool bModifiedExpression = false;
				if (Expression)
				{
					if (Expression->MaterialExpressionEditorX != GraphNode->NodePosX
						|| Expression->MaterialExpressionEditorY != GraphNode->NodePosY
						|| Expression->Desc != GraphNode->NodeComment)
					{
						bModifiedExpression = true;

						Expression->Modify();

						// Update positions and comments
						Expression->MaterialExpressionEditorX = GraphNode->NodePosX;
						Expression->MaterialExpressionEditorY = GraphNode->NodePosY;
						Expression->Desc = GraphNode->NodeComment;
					}

					for (UEdGraphPin* Pin : GraphNode->Pins)
					{
						if (Pin->Direction == EGPD_Input && Pin->PinType.PinCategory != UMaterialGraphSchema::PC_Exec)
						{
							// Wire up non-execution input pins
							// Implicitly generated property inputs are not returned by GetInputs(), so check index is within valid range.
							if (FExpressionInput* ExpressionInput = Expression->GetInput(Pin->SourceIndex))
							{
								if (Pin->LinkedTo.Num() > 0)
								{
									UEdGraphPin* ConnectedPin = Pin->LinkedTo[0];
									UMaterialGraphNode* ConnectedNode = Cast<UMaterialGraphNode>(ConnectedPin->GetOwningNode());

									if (ConnectedNode && !ConnectedNode->MaterialExpression->IsExpressionConnected(ExpressionInput, ConnectedPin->SourceIndex))
									{
										if (!bModifiedExpression)
										{
											bModifiedExpression = true;
											Expression->Modify();
										}

										ConnectedNode->MaterialExpression->Modify();
										ExpressionInput->Connect(ConnectedPin->SourceIndex, ConnectedNode->MaterialExpression);
									}
								}
								else if (ExpressionInput->Expression)
								{
									if (!bModifiedExpression)
									{
										bModifiedExpression = true;
										Expression->Modify();
									}
									ExpressionInput->Expression = NULL;
								}
							}
						}
					}
				}
			}
			else if (UMaterialGraphNode_Comment* CommentNode = Cast<UMaterialGraphNode_Comment>(Nodes[NodeIndex]))
			{
				UMaterialExpressionComment* Comment = CommentNode->MaterialExpressionComment;
				if (Comment)
				{
					if (Comment->MaterialExpressionEditorX != CommentNode->NodePosX
						|| Comment->MaterialExpressionEditorY != CommentNode->NodePosY
						|| Comment->Text != CommentNode->NodeComment
						|| Comment->SizeX != CommentNode->NodeWidth
						|| Comment->SizeY != CommentNode->NodeHeight
						|| Comment->CommentColor != CommentNode->CommentColor
						|| Comment->bCommentBubbleVisible_InDetailsPanel != CommentNode->bCommentBubbleVisible_InDetailsPanel
						|| Comment->bGroupMode != (CommentNode->MoveMode == ECommentBoxMode::GroupMovement)
						|| Comment->bColorCommentBubble != CommentNode->bColorCommentBubble)
					{
						Comment->Modify();

						// Update positions and comments
						Comment->MaterialExpressionEditorX = CommentNode->NodePosX;
						Comment->MaterialExpressionEditorY = CommentNode->NodePosY;
						Comment->Text = CommentNode->NodeComment;
						Comment->SizeX = CommentNode->NodeWidth;
						Comment->SizeY = CommentNode->NodeHeight;
						Comment->CommentColor = CommentNode->CommentColor;
						Comment->bCommentBubbleVisible_InDetailsPanel = CommentNode->bCommentBubbleVisible_InDetailsPanel;
						Comment->bGroupMode = (CommentNode->MoveMode == ECommentBoxMode::GroupMovement);
						Comment->bColorCommentBubble = CommentNode->bColorCommentBubble;
					}
				}
			}
		}
	}

	// Also link subgraphs?
	for (UEdGraph* SubGraph : SubGraphs)
	{
		CastChecked<UMaterialGraph>(SubGraph)->LinkMaterialExpressionsFromGraph();
	}
}

bool UMaterialGraph::IsInputActive(UEdGraphPin* GraphPin) const
{
	if (Material &&
		RootNode && // No inputs without a root node
		GraphPin->Direction == EGPD_Input &&
		GraphPin->PinType.PinCategory != UMaterialGraphSchema::PC_Exec &&
		GraphPin->SourceIndex < MaterialInputs.Num() &&
		GraphPin->GetOwningNode() == RootNode // we only care about pins from the root node here
		) 
	{
		const EMaterialProperty Property = MaterialInputs[GraphPin->SourceIndex].GetProperty();
		return Material->IsPropertyActiveInEditor(Property);
	}
	return true;
}

int32 UMaterialGraph::GetInputIndexForProperty(EMaterialProperty Property) const
{
	for (int32 Index = 0; Index < MaterialInputs.Num(); ++Index)
	{
		if (MaterialInputs[Index].GetProperty() == Property)
		{
			return Index;
		}
	}
	return INDEX_NONE;
}

void UMaterialGraph::GetUnusedMaterialExpressions(UMaterial* Material, TArray<UMaterialExpression*>& OutUnused)
{
	OutUnused.Empty();
	if (!Material)
	{
		return;
	}

	TArray<UMaterialExpression*> ToCheck;

	// Seed from all material property inputs
	for (int32 InputIndex = 0; InputIndex < MP_MAX; ++InputIndex)
	{
		FExpressionInput* Input = Material->GetExpressionInputForProperty(EMaterialProperty(InputIndex));
		if (Input && Input->Expression)
		{
			ToCheck.Push(Input->Expression);
		}
	}

	// CustomOutput expressions are always considered used regardless of connections
	for (UMaterialExpression* Expression : Material->GetExpressions())
	{
		if (Expression && Expression->IsA<UMaterialExpressionCustomOutput>())
		{
			ToCheck.Push(Expression);
		}
	}

	// Depth-first reachability analysis on raw expression data
	TSet<UMaterialExpression*> Reachable;
	while (ToCheck.Num() > 0)
	{
		UMaterialExpression* Expr = ToCheck.Pop();
		if (!Expr || Reachable.Contains(Expr))
		{
			continue;
		}
		Reachable.Add(Expr);

		for (FExpressionInputIterator It(Expr); It; ++It)
		{
			if (It->Expression)
			{
				ToCheck.Push(It->Expression);
			}
		}

		// Named reroute usage has no input pins — manually follow its declaration
		if (const UMaterialExpressionNamedRerouteUsage* RerouteUsage = Cast<UMaterialExpressionNamedRerouteUsage>(Expr))
		{
			if (RerouteUsage->Declaration)
			{
				ToCheck.Push(RerouteUsage->Declaration);
			}
		}
	}

	for (UMaterialExpression* Expression : Material->GetExpressions())
	{
		if (Expression && !Reachable.Contains(Expression))
		{
			OutUnused.Add(Expression);
		}
	}
}

void UMaterialGraph::GetUnusedExpressions(TArray<UEdGraphNode*>& UnusedNodes) const
{
	UnusedNodes.Empty();

	if (RootNode)
	{
		// Material case: delegate to the expression-based algorithm and map to graph nodes
		TArray<UMaterialExpression*> UnusedExpressions;
		GetUnusedMaterialExpressions(Material, UnusedExpressions);
		for (UMaterialExpression* Expression : UnusedExpressions)
		{
			if (Expression->GraphNode)
			{
				UnusedNodes.Add(Expression->GraphNode);
			}
		}
		return;
	}

	TArray<UEdGraphNode*> NodesToCheck;

	if (MaterialFunction)
	{
		for (int32 Index = 0; Index < Nodes.Num(); Index++)
		{
			UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(Nodes[Index]);
			if (GraphNode)
			{
				UMaterialExpressionFunctionOutput* FunctionOutput = Cast<UMaterialExpressionFunctionOutput>(GraphNode->MaterialExpression);
				if (FunctionOutput)
				{
					NodesToCheck.Push(GraphNode);
				}
			}
		}
	}

	// Depth-first traverse the material expression graph.
	TArray<UEdGraphNode*> UsedNodes;
	TMap<UEdGraphNode*, int32> ReachableNodes;
	while (NodesToCheck.Num() > 0)
	{
		UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(NodesToCheck.Pop());
		if (GraphNode)
		{
			int32* AlreadyVisited = ReachableNodes.Find(GraphNode);
			if (!AlreadyVisited)
			{
				// Mark the expression as reachable.
				ReachableNodes.Add(GraphNode, 0);
				UsedNodes.Add(GraphNode);

				// Iterate over the expression's inputs and add them to the pending stack.
				for (UEdGraphPin* Pin : GraphNode->Pins)
				{
					if (Pin->Direction == EGPD_Input &&
						Pin->PinType.PinCategory != UMaterialGraphSchema::PC_Exec &&
						Pin->LinkedTo.Num() > 0 &&
						Pin->LinkedTo[0])
					{
						NodesToCheck.Push(Pin->LinkedTo[0]->GetOwningNode());
					}
				}

				// Since named reroute nodes don't have any input pins, we manually push the declaration node here
				if (const UMaterialExpressionNamedRerouteUsage* NamedRerouteUsage = Cast<UMaterialExpressionNamedRerouteUsage>(GraphNode->MaterialExpression))
				{
					if (NamedRerouteUsage->Declaration && NamedRerouteUsage->Declaration->GraphNode)
					{
						NodesToCheck.Push(NamedRerouteUsage->Declaration->GraphNode);
					}
				}
			}
		}
	}

	for (int32 Index = 0; Index < Nodes.Num(); ++Index)
	{
		UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(Nodes[Index]);

		if (GraphNode && !UsedNodes.Contains(GraphNode))
		{
			UnusedNodes.Add(GraphNode);
		}
	}
}

void UMaterialGraph::NotifyGraphChanged(const FEdGraphEditAction& Action)
{
	Super::NotifyGraphChanged(Action);
}

void UMaterialGraph::NotifyGraphChanged()
{
	Super::NotifyGraphChanged();
}

void UMaterialGraph::RemoveAllNodes()
{
	MaterialInputs.Empty();

	RootNode = NULL;

	TArray<UEdGraphNode*> NodesToRemove = Nodes;
	for (int32 NodeIndex = 0; NodeIndex < NodesToRemove.Num(); ++NodeIndex)
	{
		NodesToRemove[NodeIndex]->Modify();
		RemoveNode(NodesToRemove[NodeIndex]);
	}
}

int32 UMaterialGraph::GetValidOutputIndex(FExpressionInput* Input) const
{
	int32 OutputIndex = 0;

	if (Input->Expression)
	{
		TArray<FExpressionOutput>& Outputs = Input->Expression->GetOutputs();

		if (Outputs.Num() > 0)
		{
			const bool bOutputIndexIsValid = Outputs.IsValidIndex(Input->OutputIndex)
				// Attempt to handle legacy connections before OutputIndex was used that had a mask
				&& (Input->OutputIndex != 0 || Input->Mask == 0);

			for( ; OutputIndex < Outputs.Num() ; ++OutputIndex )
			{
				const FExpressionOutput& Output = Outputs[OutputIndex];

				if((bOutputIndexIsValid && OutputIndex == Input->OutputIndex)
					|| (!bOutputIndexIsValid
					&& Output.Mask == Input->Mask
					&& Output.MaskR == Input->MaskR
					&& Output.MaskG == Input->MaskG
					&& Output.MaskB == Input->MaskB
					&& Output.MaskA == Input->MaskA))
				{
					break;
				}
			}

			if (OutputIndex >= Outputs.Num())
			{
				// Work around for non-reproducible crash where OutputIndex would be out of bounds
				OutputIndex = Outputs.Num() - 1;
			}
		}
	}

	return OutputIndex;
}

#undef LOCTEXT_NAMESPACE
