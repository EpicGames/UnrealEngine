// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MaterialGraphNode.cpp
=============================================================================*/

#include "MaterialGraph/MaterialGraphNode.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IMaterialEditor.h"
#include "ToolMenus.h"
#include "MaterialGraph/MaterialGraphSchema.h"

#include "Materials/MaterialExpressionComposite.h"
#include "Materials/MaterialExpressionPinBase.h"

#include "Materials/MaterialExpressionComponentMask.h"
#include "Materials/MaterialExpressionConstant.h"
#include "Materials/MaterialExpressionConstant2Vector.h"
#include "Materials/MaterialExpressionConstant3Vector.h"
#include "Materials/MaterialExpressionConstant4Vector.h"
#include "Materials/MaterialExpressionCustomOutput.h"
#include "Materials/MaterialExpressionFontSample.h"
#include "Materials/MaterialExpressionFunctionInput.h"
#include "Materials/MaterialExpressionFunctionOutput.h"
#include "Materials/MaterialExpressionMaterialAttributeLayers.h"
#include "Materials/MaterialExpressionRuntimeVirtualTextureSample.h"
#include "Materials/MaterialExpressionSparseVolumeTextureSample.h"
#include "Materials/MaterialExpressionSparseVolumeTextureObject.h"
#include "Materials/MaterialExpressionScalarParameter.h"
#include "Materials/MaterialExpressionStaticBool.h"
#include "Materials/MaterialExpressionStaticBoolParameter.h"
#include "Materials/MaterialExpressionTextureBase.h"
#include "Materials/MaterialExpressionTextureCoordinate.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Materials/MaterialExpressionTextureObject.h"
#include "Materials/MaterialExpressionTextureProperty.h"
#include "Materials/MaterialExpressionVectorParameter.h"
#include "Materials/MaterialExpressionDoubleVectorParameter.h"
#include "Materials/MaterialExpressionViewProperty.h"
#include "Materials/MaterialExpressionMaterialLayerOutput.h"
#include "Materials/MaterialExpressionFontSampleParameter.h"
#include "Materials/MaterialExpressionTextureObjectParameter.h"
#include "Materials/MaterialExpressionNamedReroute.h"
#include "Materials/MaterialExpressionReroute.h"
#include "Materials/MaterialExpressionCurveAtlasRowParameter.h"
#include "Materials/MaterialExpressionSubstrate.h"
#include "Materials/MaterialFunction.h"
#include "MaterialEditorUtilities.h"
#include "MaterialEditorActions.h"
#include "GraphEditorActions.h"
#include "GraphEditorSettings.h"
#include "Framework/Commands/GenericCommands.h"
#include "ScopedTransaction.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "MaterialEditor/MaterialNodes/SGraphNodeMaterialBase.h"
#include "SourceCodeNavigation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialGraphNode)

#define LOCTEXT_NAMESPACE "MaterialGraphNode"

static FText SpaceText = LOCTEXT("Space", " ");

/////////////////////////////////////////////////////
// UMaterialGraphNode

UMaterialGraphNode::UMaterialGraphNode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bPreviewNeedsUpdate(false)
	, bIsErrorExpression(false)
	, bIsPreviewExpression(false)
{
}

void UMaterialGraphNode::PostCopyNode()
{
	// Make sure the MaterialExpression goes back to being owned by the Material after copying.
	ResetMaterialExpressionOwner();
}

FMaterialRenderProxy* UMaterialGraphNode::GetExpressionPreview()
{
	return MaterialExpression ? FMaterialEditorUtilities::GetExpressionPreview(GetGraph(), MaterialExpression) : nullptr;
}

void UMaterialGraphNode::RecreateAndLinkNode()
{
	// Throw away the original pins
	for (int32 PinIndex = 0; PinIndex < Pins.Num(); ++PinIndex)
	{
		UEdGraphPin* Pin = Pins[PinIndex];
		Pin->Modify();
		Pin->BreakAllPinLinks();

		UEdGraphNode::DestroyPin(Pin);
	}
	EmptyPins();

	AllocateDefaultPins();

	CastChecked<UMaterialGraph>(GetGraph())->LinkGraphNodesFromMaterial();
}

void UMaterialGraphNode::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		const FName PropertyName = PropertyChangedEvent.Property->GetFName();
		if (PropertyName == FName(TEXT("NodeComment")))
		{
			if (MaterialExpression)
			{
				MaterialExpression->Modify();

				// Note: This if prevents an infinite loop where the comment update triggers a node update that then triggers a comment update.
				if (MaterialExpression->Desc != NodeComment)
				{
					MaterialExpression->Desc = NodeComment;
					MaterialExpression->bCommentBubbleVisible = !NodeComment.IsEmpty();
					MaterialExpression->Material->PropagateExpressionParameterChanges(MaterialExpression);
				}
			}
		}
	}
}

void UMaterialGraphNode::PostEditImport()
{
	// Make sure this MaterialExpression is owned by the Material it's being pasted into.
	ResetMaterialExpressionOwner();
}

void UMaterialGraphNode::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	if (!bDuplicateForPIE)
	{
		CreateNewGuid();
	}
}

bool UMaterialGraphNode::CanPasteHere(const UEdGraph* TargetGraph) const
{
	if (Super::CanPasteHere(TargetGraph) && MaterialExpression)
	{
		const UMaterialGraph* MaterialGraph = Cast<const UMaterialGraph>(TargetGraph);
		if (MaterialGraph)
		{
			// Check whether we're trying to paste a material function into a function that depends on it
			UMaterialExpressionMaterialFunctionCall* FunctionExpression = Cast<UMaterialExpressionMaterialFunctionCall>(MaterialExpression);
			bool bIsValidFunctionExpression = true;

			if (MaterialGraph->MaterialFunction 
				&& FunctionExpression 
				&& FunctionExpression->MaterialFunction
				&& FunctionExpression->MaterialFunction->IsDependent(MaterialGraph->MaterialFunction))
			{
				bIsValidFunctionExpression = false;
			}

			if (bIsValidFunctionExpression && MaterialExpression && MaterialExpression->IsAllowedIn(MaterialGraph->GetMaterialOrFunction()))
			{
				return true;
			}
		}
	}
	return false;
}

FText UMaterialGraphNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	TArray<FString> Captions;
	if (MaterialExpression)
	{
		MaterialExpression->GetCaption(Captions);
	}

	if (TitleType == ENodeTitleType::EditableTitle)
	{
		return FText::FromString(GetParameterName());
	}
	else if (MaterialExpression && (TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle))
	{
		return FText::FromString(MaterialExpression->GetClass()->GetDescription());
	}
	else
	{
		// More useful to display multi line parameter captions in reverse order
		// TODO: May have to choose order based on expression type if others need correct order
		int32 CaptionIndex = Captions.Num() - 1;

		FTextBuilder NodeTitle;
		if (Captions.IsValidIndex(CaptionIndex))
		{
			NodeTitle.AppendLine(Captions[CaptionIndex]);
		}
		for (; CaptionIndex > 0; )
		{
			CaptionIndex--;
			NodeTitle.AppendLine(Captions[CaptionIndex]);
		}

		if (MaterialExpression && MaterialExpression->bShaderInputData && (MaterialExpression->bHidePreviewWindow || MaterialExpression->bCollapsed))
		{
			if (MaterialExpression->IsA<UMaterialExpressionTextureProperty>())
			{
				NodeTitle.AppendLine(LOCTEXT("TextureProperty", "Texture Property"));
			}
			else if (MaterialExpression->IsA<UMaterialExpressionViewProperty>())
			{
				NodeTitle.AppendLine(LOCTEXT("ViewProperty", "View Property"));
			}
			else
			{
				NodeTitle.AppendLine(LOCTEXT("InputData", "Input Data"));
			}
		}

		if (bIsPreviewExpression)
		{
			NodeTitle.AppendLine();
			NodeTitle.AppendLine(LOCTEXT("PreviewExpression", "Previewing"));
		}

		return NodeTitle.ToText();
	}
}

FLinearColor UMaterialGraphNode::GetNodeTitleColor() const
{
	UMaterial* Material = CastChecked<UMaterialGraph>(GetGraph())->Material;
	const UGraphEditorSettings* Settings = GetDefault<UGraphEditorSettings>();
	if (bIsPreviewExpression)
	{
		// If we are currently previewing a node, its border should be the preview color.
		return Settings->PreviewNodeTitleColor;
	}

	if (UsesBoolColour(MaterialExpression))
	{
		return Settings->BooleanPinTypeColor;
	}
	else if (UsesFloatColour(MaterialExpression))
	{
		return Settings->FloatPinTypeColor;
	}
	else if (UsesVectorColour(MaterialExpression))
	{
		return Settings->VectorPinTypeColor;
	}
	else if (UsesObjectColour(MaterialExpression))
	{
		return Settings->ObjectPinTypeColor;
	}
	else if (UsesEventColour(MaterialExpression))
	{
		return Settings->EventNodeTitleColor;
	}
	else if (MaterialExpression->IsA(UMaterialExpressionMaterialFunctionCall::StaticClass()))
	{
		// Previously FColor(0, 116, 255);
		return Settings->FunctionCallNodeTitleColor;
	}
	else if (MaterialExpression->IsA(UMaterialExpressionMaterialAttributeLayers::StaticClass()))
	{
		return Settings->FunctionCallNodeTitleColor;
	}
	else if (MaterialExpression->IsA(UMaterialExpressionFunctionInput::StaticClass()))
	{
		return Settings->FunctionCallNodeTitleColor;
	}
	else if (MaterialExpression->IsA(UMaterialExpressionFunctionOutput::StaticClass()))
	{
		// Previously FColor(255, 155, 0);
		return Settings->ResultNodeTitleColor;
	}
	else if (MaterialExpression->IsA(UMaterialExpressionMaterialLayerOutput::StaticClass()))
	{
		// Previously FColor(255, 155, 0);
		return Settings->ResultNodeTitleColor;
	}
	else if (MaterialExpression->IsA(UMaterialExpressionCustomOutput::StaticClass()))
	{
		// Previously FColor(255, 155, 0);
		return Settings->ResultNodeTitleColor;
	}
	else if (const UMaterialExpressionNamedRerouteDeclaration* RerouteDeclaration = Cast<UMaterialExpressionNamedRerouteDeclaration>(MaterialExpression))
	{
		// If it's a declaration node, we simply get the color from it 
		return RerouteDeclaration->NodeColor;
	}
	else if (const UMaterialExpressionNamedRerouteUsage* RerouteUsage = Cast<UMaterialExpressionNamedRerouteUsage>(MaterialExpression))
	{
		// Return the color of the declaration
		if (RerouteUsage->Declaration)
		{
			return RerouteUsage->Declaration->NodeColor;
		}
		
		return FLinearColor::Black;
	}
	else if (UMaterial::IsParameter(MaterialExpression))
	{
		if (Material->HasDuplicateParameters(MaterialExpression))
		{
			return FColor( 0, 255, 255 );
		}
		else
		{
			return FColor( 0, 128, 128 );
		}
	}
	else if (UMaterial::IsDynamicParameter(MaterialExpression))
	{
		if (Material->HasDuplicateDynamicParameters(MaterialExpression))
		{
			return FColor( 0, 255, 255 );
		}
		else
		{
			return FColor( 0, 128, 128 );
		}
	}
	else if (MaterialExpression->IsA<UMaterialExpressionSubstrateBSDF>() 
		|| MaterialExpression->IsA<UMaterialExpressionSubstrateUtilityBase>())
	{
		return FColor(181, 29, 230);
	}

	// Assume that most material expressions act like pure functions and don't affect anything else
	return Settings->PureFunctionCallNodeTitleColor;
}

FText UMaterialGraphNode::GetTooltipText() const
{
	if (MaterialExpression)
	{
		TArray<FString> ToolTips;
		MaterialExpression->GetExpressionToolTip(ToolTips);

		if (ToolTips.Num() > 0)
		{
			FString ToolTip = ToolTips[0];

			for (int32 Index = 1; Index < ToolTips.Num(); ++Index)
			{
				ToolTip += TEXT("\n");
				ToolTip += ToolTips[Index];
			}

			return FText::FromString(ToolTip);
		}
	}
	return FText::GetEmpty();
}

void UMaterialGraphNode::PrepareForCopying()
{
	if (MaterialExpression)
	{
		// Temporarily take ownership of the MaterialExpression, so that it is not deleted when cutting
		MaterialExpression->Rename(NULL, this, REN_DontCreateRedirectors);
	}
}

void UMaterialGraphNode::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	UMaterialGraph* MaterialGraph = CastChecked<UMaterialGraph>(GetGraph());

	if (Context->Node)
	{
		FToolMenuInsert MenuPosition = FToolMenuInsert(NAME_None, EToolMenuInsertType::First);
		FToolMenuSection& NodeSection = Menu->AddSection("MaterialSchemaNodeActions", LOCTEXT("NodeActionsMenuHeader", "Node Actions"), MenuPosition);
		if (MaterialExpression)
		{
			if (MaterialExpression->IsA(UMaterialExpressionTextureBase::StaticClass()) || MaterialExpression->IsA(UMaterialExpressionSparseVolumeTextureBase::StaticClass()))
			{
				{
					NodeSection.AddMenuEntry(FMaterialEditorCommands::Get().UseCurrentTexture);
				}

				// Add a 'Convert To Texture' option for convertible types
				{
					if ( (MaterialExpression->IsA(UMaterialExpressionTextureSample::StaticClass()) || MaterialExpression->IsA(UMaterialExpressionSparseVolumeTextureSample::StaticClass())) && !MaterialExpression->HasAParameterName())
					{
						NodeSection.AddMenuEntry(FMaterialEditorCommands::Get().ConvertToTextureObjects);
					}
					else if ( MaterialExpression->IsA(UMaterialExpressionTextureObject::StaticClass()) || MaterialExpression->IsA(UMaterialExpressionSparseVolumeTextureObject::StaticClass()) )
					{
						NodeSection.AddMenuEntry(FMaterialEditorCommands::Get().ConvertToTextureSamples);
					}
				}
			}

			// Add a 'Convert to Local Variables' option to reroute nodes
			if (MaterialExpression->IsA(UMaterialExpressionReroute::StaticClass()))
			{
				NodeSection.AddMenuEntry(FMaterialEditorCommands::Get().ConvertRerouteToNamedReroute);
			}

			// Add local variables selection & conversion to reroute nodes
			if (MaterialExpression->IsA(UMaterialExpressionNamedRerouteBase::StaticClass()))
			{
				if (MaterialExpression->IsA(UMaterialExpressionNamedRerouteDeclaration::StaticClass()))
				{
					NodeSection.AddMenuEntry(FMaterialEditorCommands::Get().SelectNamedRerouteUsages);
					NodeSection.AddMenuEntry(FMaterialEditorCommands::Get().CreateRerouteUsageFromDeclaration);
				}
				if (MaterialExpression->IsA(UMaterialExpressionNamedRerouteUsage::StaticClass()))
				{
					NodeSection.AddMenuEntry(FMaterialEditorCommands::Get().SelectNamedRerouteDeclaration);
				}
				NodeSection.AddMenuEntry(FMaterialEditorCommands::Get().ConvertNamedRerouteToReroute);
			}

			// Add a 'Convert To Parameter' option for convertible types
			if (MaterialExpression->IsA(UMaterialExpressionConstant::StaticClass())
				|| MaterialExpression->IsA(UMaterialExpressionConstant2Vector::StaticClass())
				|| MaterialExpression->IsA(UMaterialExpressionConstant3Vector::StaticClass())
				|| MaterialExpression->IsA(UMaterialExpressionConstant4Vector::StaticClass())
				|| (MaterialExpression->IsA(UMaterialExpressionTextureSample::StaticClass()) && !MaterialExpression->HasAParameterName())
				|| (MaterialExpression->IsA(UMaterialExpressionRuntimeVirtualTextureSample::StaticClass()) && !MaterialExpression->HasAParameterName())
				|| (MaterialExpression->IsA(UMaterialExpressionSparseVolumeTextureSample::StaticClass()) && !MaterialExpression->HasAParameterName())
				|| MaterialExpression->IsA(UMaterialExpressionTextureObject::StaticClass())
				|| MaterialExpression->IsA(UMaterialExpressionSparseVolumeTextureObject::StaticClass())
				|| MaterialExpression->IsA(UMaterialExpressionComponentMask::StaticClass()))
			{
				{
					NodeSection.AddMenuEntry(FMaterialEditorCommands::Get().ConvertObjects);
				}
			}

			// Constants cannot be converted back to a CurveAtlasRowParameter
			const bool bIsAStandardScalarParam = MaterialExpression->IsA(UMaterialExpressionScalarParameter::StaticClass())
											     && !MaterialExpression->IsA(UMaterialExpressionCurveAtlasRowParameter::StaticClass());

			// Add a 'Convert To Constant' option for convertible types
			if (bIsAStandardScalarParam
				|| MaterialExpression->IsA(UMaterialExpressionVectorParameter::StaticClass())
				|| MaterialExpression->IsA(UMaterialExpressionTextureObjectParameter::StaticClass())
				|| MaterialExpression->IsA(UMaterialExpressionSparseVolumeTextureObjectParameter::StaticClass()))
			{
				{
					NodeSection.AddMenuEntry(FMaterialEditorCommands::Get().ConvertToConstant);
				}
			}

			// Add 'Promote to Double' option for float types
			if (MaterialExpression->IsA(UMaterialExpressionVectorParameter::StaticClass()))
			{
				NodeSection.AddMenuEntry(FMaterialEditorCommands::Get().PromoteToDouble);
			}

			// Add 'Promote to Float' option for double types
			if (MaterialExpression->IsA(UMaterialExpressionDoubleVectorParameter::StaticClass()))
			{
				NodeSection.AddMenuEntry(FMaterialEditorCommands::Get().PromoteToFloat);
			}

			{
				// Don't show preview option for bools
				if (!MaterialExpression->IsA(UMaterialExpressionStaticBool::StaticClass())
					&& !MaterialExpression->IsA(UMaterialExpressionStaticBoolParameter::StaticClass()))
				{
					// Add a preview node option if only one node is selected
					if (bIsPreviewExpression)
					{
						// If we are already previewing the selected node, the menu option should tell the user that this will stop previewing
						NodeSection.AddMenuEntry(FMaterialEditorCommands::Get().StopPreviewNode);
					}
					else
					{
						// The menu option should tell the user this node will be previewed.
						NodeSection.AddMenuEntry(FMaterialEditorCommands::Get().StartPreviewNode);
					}
				}

				if (MaterialExpression->bRealtimePreview)
				{
					NodeSection.AddMenuEntry(FMaterialEditorCommands::Get().DisableRealtimePreviewNode);
				}
				else
				{
					NodeSection.AddMenuEntry(FMaterialEditorCommands::Get().EnableRealtimePreviewNode);
				}
			}
		}

		// Break all links
		{
			NodeSection.AddMenuEntry(FGraphEditorCommands::Get().BreakNodeLinks);

			if (Context->Node->CanJumpToDefinition())
			{
				NodeSection.AddMenuEntry(FGraphEditorCommands::Get().GoToDefinition);
			}
		}

		{
			FToolMenuSection& Section = Menu->AddSection("MaterialSchemaNodeNavigation", LOCTEXT("NodeNavigationMenuHeader", "Navigation"));
			Section.AddMenuEntry(FGraphEditorCommands::Get().OpenInNewTab);
		}
		
		// Separate the above frequently used options from the below less frequently used common options
		
		{
			FToolMenuSection& Section = Menu->AddSection("MaterialEditorMenu3", LOCTEXT("GeneralNodeActionsMenuHeader", "General"));
			Section.AddMenuEntry( FGenericCommands::Get().Delete );
			Section.AddMenuEntry( FGenericCommands::Get().Cut );
			Section.AddMenuEntry( FGenericCommands::Get().Copy );
			Section.AddMenuEntry( FGenericCommands::Get().Duplicate );

			// Select upstream and downstream nodes
			Section.AddMenuEntry(FMaterialEditorCommands::Get().SelectDownstreamNodes);
			Section.AddMenuEntry(FMaterialEditorCommands::Get().SelectUpstreamNodes);
		}
			
		{
			FToolMenuSection& Section = Menu->AddSection("EdGraphSchemaOrganization", LOCTEXT("OrganizationHeader", "Organization"));
			Section.AddMenuEntry(FGraphEditorCommands::Get().CollapseNodes);
			Section.AddMenuEntry(FGraphEditorCommands::Get().ExpandNodes);
			Section.AddMenuEntry(FGraphEditorCommands::Get().CollapseSelectionToFunction);
			Section.AddSubMenu(
				"Alignment",
				LOCTEXT("AlignmentHeader", "Alignment"),
				FText(),
				FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
				{
					{
						FToolMenuSection& SubMenuSection = InMenu->AddSection("EdGraphSchemaAlignment", LOCTEXT("AlignHeader", "Align"));
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesTop);
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesMiddle);
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesBottom);
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesLeft);
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesCenter);
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesRight);
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().StraightenConnections);
					}

					{
						FToolMenuSection& SubMenuSection = InMenu->AddSection("EdGraphSchemaDistribution", LOCTEXT("DistributionHeader", "Distribution"));
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().DistributeNodesHorizontally);
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().DistributeNodesVertically);
					}
				}));
		}

		{
			FToolMenuSection& Section = Menu->AddSection("MaterialEditorMenuDocumentation");
			Section.AddMenuEntry(FGraphEditorCommands::Get().GoToDocumentation);
		}

		// Handle the favorites options
		if (MaterialExpression)
		{
			{
				FToolMenuSection& Section = Menu->AddSection("MaterialEditorMenuFavorites");
				if (FMaterialEditorUtilities::IsMaterialExpressionInFavorites(MaterialExpression))
				{
					Section.AddMenuEntry(FMaterialEditorCommands::Get().RemoveFromFavorites);
				}
				else
				{
					Section.AddMenuEntry(FMaterialEditorCommands::Get().AddToFavorites);
				}
			}
		}
	}
}

namespace MaterialPinNames
{
	static const FName Coordinates(TEXT("Coordinates"));
	static const FName UVs(TEXT("UVs"));
	static const FName TextureObject(TEXT("TextureObject"));
	static const FName Tex(TEXT("Tex"));
	static const FName Input(TEXT("Input"));
	static const FName Exponent(TEXT("Exponent"));
	static const FName Exp(TEXT("Exp"));
	static const FName AGreaterThanB(TEXT("AGreaterThanB"));
	static const FName CompactAGreaterThanB(TEXT("A > B"));
	static const FName AEqualsB(TEXT("AEqualsB"));
	static const FName CompactAEqualsB(TEXT("A == B"));
	static const FName ALessThanB(TEXT("ALessThanB"));
	static const FName CompactALessThanB(TEXT("A < B"));
	static const FName MipLevel(TEXT("MipLevel"));
	static const FName Level(TEXT("Level"));
	static const FName MipBias(TEXT("MipBias"));
	static const FName Bias(TEXT("Bias"));
}

FName UMaterialGraphNode::GetShortenPinName(const FName PinName)
{
	FName InputName = PinName;

	// Shorten long expression input names.
	if (PinName == MaterialPinNames::Coordinates)
	{
		InputName = MaterialPinNames::UVs;
	}
	else if (PinName == MaterialPinNames::TextureObject)
	{
		InputName = MaterialPinNames::Tex;
	}
	else if (PinName == MaterialPinNames::Input)
	{
		InputName = NAME_None;
	}
	else if (PinName == MaterialPinNames::Exponent)
	{
		InputName = MaterialPinNames::Exp;
	}
	else if (PinName == MaterialPinNames::AGreaterThanB)
	{
		InputName = MaterialPinNames::CompactAGreaterThanB;
	}
	else if (PinName == MaterialPinNames::AEqualsB)
	{
		InputName = MaterialPinNames::CompactAEqualsB;
	}
	else if (PinName == MaterialPinNames::ALessThanB)
	{
		InputName = MaterialPinNames::CompactALessThanB;
	}
	else if (PinName == MaterialPinNames::MipLevel)
	{
		InputName = MaterialPinNames::Level;
	}
	else if (PinName == MaterialPinNames::MipBias)
	{
		InputName = MaterialPinNames::Bias;
	}

	return InputName;
}

EMaterialValueType UMaterialGraphNode::GetPinMaterialValueType(const UEdGraphPin* Pin) const
{
	if (Pin->PinType.PinCategory == UMaterialGraphSchema::PC_Exec)
	{
		return MCT_Execution;
	}
	
	if (Pin->Direction == EGPD_Input)
	{
		return MaterialExpression->GetInputValueType(Pin->SourceIndex);
	}
	else
	{
		return MaterialExpression->GetOutputValueType(Pin->SourceIndex);
	}
}

void UMaterialGraphNode::PinDefaultValueChanged(UEdGraphPin* Pin)
{
	MaterialExpression->PinDefaultValueChanged(Pin->SourceIndex, Pin->DefaultValue);
}

TSharedPtr<SGraphNode> UMaterialGraphNode::CreateVisualWidget()
{
	// First check to see if CreateCustomGraphNodeWidget returns a valid widget
	TSharedPtr<SGraphNode> OutGraphNode = MaterialExpression->CreateCustomGraphNodeWidget();
	
	// If CreateCustomGraphNodeWidget returned nullptr, create default SGraphNodeMaterialBase
	if (!OutGraphNode)
	{
		OutGraphNode = SNew(SGraphNodeMaterialBase, this);
	}

	return OutGraphNode;
}

bool UMaterialGraphNode::CanJumpToDefinition() const
{
	return true;
}

void UMaterialGraphNode::JumpToDefinition() const
{
	FSourceCodeNavigation::NavigateToClass(MaterialExpression->GetClass());
}

void UMaterialGraphNode::CreateInputPins()
{
	int32 InputIndex = 0;
	while (FExpressionInput* Input = MaterialExpression->GetInput(InputIndex))
	{
		FName InputName = MaterialExpression->GetInputName(InputIndex);
		InputName = GetShortenPinName(InputName);

		FName PinCategory;
		if (MaterialExpression->IsInputConnectionRequired(InputIndex))
		{
			PinCategory = UMaterialGraphSchema::PC_Required;
		}
		else
		{
			PinCategory = UMaterialGraphSchema::PC_Optional;
		}

		FName PinSubCategory = MaterialExpression->GetInputPinSubCategory(InputIndex);
		UObject* PinSubCategoryObject = MaterialExpression->GetInputPinSubCategoryObject(InputIndex);

		UEdGraphPin* NewPin = CreatePin(EGPD_Input, PinCategory, PinSubCategory, PinSubCategoryObject, InputName);
		NewPin->SourceIndex = InputIndex;
		NewPin->DefaultValue = MaterialExpression->GetInputPinDefaultValue(InputIndex);
		if (NewPin->PinName.IsNone())
		{
			// Makes sure pin has a name for lookup purposes but user will never see it
			NewPin->PinName = CreateUniquePinName(TEXT("Input"));
			NewPin->PinFriendlyName = SpaceText;
		}

		InputIndex += 1;
	}

	// Next create pins for property inputs
	int32 AdvancedPins = 0;
	static FName ShowAsInputPinMetaData(TEXT("ShowAsInputPin"));
	const TArray<FProperty*> PropertyInputs = MaterialExpression->GetPropertyInputs();
	for (int32 i = 0; i < PropertyInputs.Num(); ++i)
	{
		const FProperty* Property = PropertyInputs[i];
		FString InputName = Property->GetDisplayNameText().ToString();
		FName PinCategory = UMaterialGraphSchema::PC_Optional;

		int32 PinIndex = InputIndex + i;
		FName PinSubCategory = MaterialExpression->GetInputPinSubCategory(PinIndex);
		UObject* PinSubCategoryObject = MaterialExpression->GetInputPinSubCategoryObject(PinIndex);

		UEdGraphPin* NewPin = CreatePin(EGPD_Input, PinCategory, PinSubCategory, PinSubCategoryObject, FName(InputName));
		NewPin->SourceIndex = PinIndex;
		NewPin->DefaultValue = MaterialExpression->GetInputPinDefaultValue(PinIndex);
		// Property pins can't connect externally
		NewPin->bNotConnectable = true;
		NewPin->bDefaultValueIsReadOnly = !MaterialExpression->CanEditChange(Property);
		const FString ShowAsInputPin = Property->GetMetaData(ShowAsInputPinMetaData);
		NewPin->bAdvancedView = (ShowAsInputPin == TEXT("Advanced"));
		if (NewPin->PinName.IsNone())
		{
			// Makes sure pin has a name for lookup purposes but user will never see it
			NewPin->PinName = CreateUniquePinName(TEXT("PropertyInput"));
			NewPin->PinFriendlyName = SpaceText;
		}

		if (NewPin->bAdvancedView)
		{
			AdvancedPins++;
		}
	}
	if (AdvancedPins > 0)
	{
		// Turn on the advanced view arrow button
		if (AdvancedPinDisplay == ENodeAdvancedPins::NoPins)
		{
			AdvancedPinDisplay = ENodeAdvancedPins::Hidden;
		}
	}
}

void UMaterialGraphNode::CreateOutputPins()
{
	TArray<FExpressionOutput>& Outputs = MaterialExpression->GetOutputs();
	for(int32 Index = 0; Index < Outputs.Num(); ++Index)
	{
		const FExpressionOutput& ExpressionOutput = Outputs[Index];

		FName PinCategory;
		FName PinSubCategory;
		FName OutputName;
		if (MaterialExpression->bShowOutputNameOnPin)
		{
			OutputName = ExpressionOutput.OutputName;
		}

		if (MaterialExpression->bShowMaskColorsOnPin && ExpressionOutput.Mask)
		{
			PinCategory = UMaterialGraphSchema::PC_Mask;

			if (ExpressionOutput.MaskR && !ExpressionOutput.MaskG && !ExpressionOutput.MaskB && !ExpressionOutput.MaskA)
			{
				PinSubCategory = UMaterialGraphSchema::PSC_Red;
			}
			else if (!ExpressionOutput.MaskR && ExpressionOutput.MaskG && !ExpressionOutput.MaskB && !ExpressionOutput.MaskA)
			{
				PinSubCategory = UMaterialGraphSchema::PSC_Green;
			}
			else if (!ExpressionOutput.MaskR && !ExpressionOutput.MaskG && ExpressionOutput.MaskB && !ExpressionOutput.MaskA)
			{
				PinSubCategory = UMaterialGraphSchema::PSC_Blue;
			}
			else if (!ExpressionOutput.MaskR && !ExpressionOutput.MaskG && !ExpressionOutput.MaskB && ExpressionOutput.MaskA)
			{
				PinSubCategory = UMaterialGraphSchema::PSC_Alpha;
			}
			else if (ExpressionOutput.MaskR && ExpressionOutput.MaskG && ExpressionOutput.MaskB && ExpressionOutput.MaskA)
			{
				PinSubCategory = UMaterialGraphSchema::PSC_RGBA;
			}
		}

		UEdGraphPin* NewPin = CreatePin(EGPD_Output, PinCategory, PinSubCategory, OutputName);
		NewPin->SourceIndex = Index;
		if (NewPin->PinName.IsNone())
		{
			// Makes sure pin has a name for lookup purposes but user will never see it
			NewPin->PinName = CreateUniquePinName(TEXT("Output"));
			NewPin->PinFriendlyName = SpaceText;
		}
	}
}

void UMaterialGraphNode::PropagatePropertyChange()
{
	// If we've already propagated an update to this node return
	if (MaterialExpression->bNeedToUpdatePreview)
	{
		return;
	}

	MaterialExpression->bNeedToUpdatePreview = true;

	// Loop through all the output pins.
	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin->PinType.PinCategory == UMaterialGraphSchema::PC_Exec ||
			Pin->Direction != EGPD_Output)
		{
			continue;
		}

		// We only care about pins which are outputs. We call PropagatePropertyChange
		// on the nodes that are linked to this particular output pin.
		for (UEdGraphPin* LinkedPins : Pin->LinkedTo)
		{
			// Retrieve the linked node. Check that it is of type UMaterialGraphNode.
			UEdGraphNode* Node = LinkedPins->GetOwningNode();
			UMaterialGraphNode* CastedNode = Cast<UMaterialGraphNode>(Node);
			
			if (!CastedNode)
			{
				continue;
			}

			CastedNode->PropagatePropertyChange();
		}
	}
}

void UMaterialGraphNode::ToggleNodePreview()
{
	if (!MaterialExpression->Material)
	{
		return;
	}

	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	IAssetEditorInstance* FoundAssetEditor = AssetEditorSubsystem->FindEditorForAsset(MaterialExpression->Material, true);
	if (!FoundAssetEditor)
	{
		return;
	}

	IMaterialEditor* MaterialEditor = static_cast<IMaterialEditor*>(FoundAssetEditor);
	MaterialEditor->PreviewNode(this);
}

void UMaterialGraphNode::ResetMaterialExpressionOwner()
{
	if (MaterialExpression)
	{
		// Ensures MaterialExpression is owned by the Material or Function
		UMaterialGraph* MaterialGraph = CastChecked<UMaterialGraph>(GetGraph());
		UObject* ExpressionOuter = MaterialGraph->Material;
		if (MaterialGraph->MaterialFunction)
		{
			ExpressionOuter = MaterialGraph->MaterialFunction;
		}
		MaterialExpression->Rename(NULL, ExpressionOuter, REN_DontCreateRedirectors);

		// Set up the back pointer for newly created material nodes
		MaterialExpression->GraphNode = this;
	}
}

void UMaterialGraphNode::PostPlacedNewNode()
{
	if (MaterialExpression)
	{
		NodeComment = MaterialExpression->Desc;
		bCommentBubbleVisible = MaterialExpression->bCommentBubbleVisible;
		NodePosX = MaterialExpression->MaterialExpressionEditorX;
		NodePosY = MaterialExpression->MaterialExpressionEditorY;
		bCanRenameNode = MaterialExpression->CanRenameNode();
	}
}

void UMaterialGraphNode::NodeConnectionListChanged()
{
	Super::NodeConnectionListChanged();

	const UEdGraphSchema* Schema = GetSchema();
	if (Schema != nullptr)
	{
		Schema->ForceVisualizationCacheClear();
	}
}

void UMaterialGraphNode::OnRenameNode(const FString& NewName)
{
	MaterialExpression->Modify();
	SetParameterName(NewName);
	MaterialExpression->MarkPackageDirty();
	MaterialExpression->ValidateParameterName();
	FProperty* NameProperty = nullptr;
	if (Cast<UMaterialExpressionParameter>(MaterialExpression))
	{
		NameProperty = FindFieldChecked<FProperty>(UMaterialExpressionParameter::StaticClass(), GET_MEMBER_NAME_CHECKED(UMaterialExpressionParameter, ParameterName));
	}
	else if (Cast<UMaterialExpressionFontSampleParameter>(MaterialExpression))
	{
		NameProperty = FindFieldChecked<FProperty>(UMaterialExpressionFontSampleParameter::StaticClass(), GET_MEMBER_NAME_CHECKED(UMaterialExpressionFontSampleParameter, ParameterName));
	}
	else if (Cast<UMaterialExpressionTextureSampleParameter>(MaterialExpression))
	{
		NameProperty = FindFieldChecked<FProperty>(UMaterialExpressionTextureSampleParameter::StaticClass(), GET_MEMBER_NAME_CHECKED(UMaterialExpressionTextureSampleParameter, ParameterName));
	}
	if(NameProperty)
	{
		FPropertyChangedEvent PropertyChangeEvent(NameProperty, EPropertyChangeType::ValueSet);
		MaterialExpression->PostEditChangeProperty(PropertyChangeEvent);
	}
	MaterialDirtyDelegate.ExecuteIfBound();
}

void UMaterialGraphNode::OnUpdateCommentText( const FString& NewComment )
{
	const FScopedTransaction Transaction( LOCTEXT( "CommentCommitted", "Comment Changed" ) );
	// Update the Node comment
	Modify();
	NodeComment	= NewComment;
	// Update the Material Expresssion desc to match the comment
	if( MaterialExpression )
	{
		MaterialExpression->Modify();

		// Note: This if prevents an infinite loop where the comment update triggers a node update that then triggers a comment update.
		if (MaterialExpression->Desc != NewComment)
		{
			MaterialExpression->Desc = NewComment;
			MaterialExpression->bCommentBubbleVisible = !NewComment.IsEmpty();
			MaterialExpression->Material->PropagateExpressionParameterChanges(MaterialExpression);
		}
		MaterialDirtyDelegate.ExecuteIfBound();
	}
}

void UMaterialGraphNode::OnCommentBubbleToggled( bool bInCommentBubbleVisible )
{
	if ( MaterialExpression )
	{
		MaterialExpression->Modify();
		MaterialExpression->bCommentBubbleVisible = bInCommentBubbleVisible;
		MaterialDirtyDelegate.ExecuteIfBound();
	}
}

void UMaterialGraphNode::GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const
{
	Super::GetPinHoverText(Pin, HoverTextOut);

	if (HoverTextOut.IsEmpty())
	{
		TArray<FString> ToolTips;

		int32 DirIndex = -1;
		int32 PinIndex = INDEX_NONE;
		for (int32 Index = 0; Index < Pins.Num(); ++Index)
		{
			if (Pin.Direction == Pins[Index]->Direction)
			{
				++DirIndex;
				if (Pins[Index] == &Pin)
				{
					PinIndex = DirIndex;
					break;
				}
			}
		}

		if (Pin.Direction == EEdGraphPinDirection::EGPD_Input)
		{
			MaterialExpression->GetConnectorToolTip(PinIndex, INDEX_NONE, ToolTips);
		}
		else
		{
			MaterialExpression->GetConnectorToolTip(INDEX_NONE, PinIndex, ToolTips);
		}

		if (ToolTips.Num() > 0)
		{
			HoverTextOut = ToolTips[0];

			for (int32 Index = 1; Index < ToolTips.Num(); ++Index)
			{
				HoverTextOut += TEXT("\n");
				HoverTextOut += ToolTips[Index];
			}
		}
	}
}

FString UMaterialGraphNode::GetParameterName() const
{
	return MaterialExpression->GetEditableName();
}

void UMaterialGraphNode::SetParameterName(const FString& NewName)
{
	MaterialExpression->SetEditableName(NewName);

	//@TODO: Push into the SetEditableName interface
	CastChecked<UMaterialGraph>(GetGraph())->Material->UpdateExpressionParameterName(MaterialExpression);
}

bool UMaterialGraphNode::UsesBoolColour(UMaterialExpression* Expression)
{
	if (Expression->IsA<UMaterialExpressionStaticBool>())
	{
		return true;
	}
	// Explicitly check for bool param as switch params inherit from it
	else if (Expression->GetClass() == UMaterialExpressionStaticBoolParameter::StaticClass())
	{
		return true;
	}

	return false;
}

bool UMaterialGraphNode::UsesFloatColour(UMaterialExpression* Expression)
{
	if (Expression->IsA<UMaterialExpressionConstant>())
	{
		return true;
	}
	else if (Expression->IsA<UMaterialExpressionScalarParameter>())
	{
		return true;
	}

	return false;
}

bool UMaterialGraphNode::UsesVectorColour(UMaterialExpression* Expression)
{
	if (Expression->IsA<UMaterialExpressionConstant2Vector>())
	{
		return true;
	}
	else if (Expression->IsA<UMaterialExpressionConstant3Vector>())
	{
		return true;
	}
	else if (Expression->IsA<UMaterialExpressionConstant4Vector>())
	{
		return true;
	}
	else if (Expression->IsA<UMaterialExpressionVectorParameter>())
	{
		return true;
	}

	return false;
}

bool UMaterialGraphNode::UsesObjectColour(UMaterialExpression* Expression)
{
	if (Expression->IsA<UMaterialExpressionTextureBase>())
	{
		return true;
	}
	else if (Expression->IsA<UMaterialExpressionSparseVolumeTextureBase>())
	{
		return true;
	}
	else if (Expression->IsA<UMaterialExpressionFontSample>())
	{
		return true;
	}

	return false;
}

bool UMaterialGraphNode::UsesEventColour(UMaterialExpression* Expression)
{
	if (Expression->bShaderInputData && !Expression->IsA<UMaterialExpressionStaticBool>())
	{
		return true;
	}
	else if (Expression->IsA<UMaterialExpressionFunctionInput>())
	{
		return true;
	}
	else if (Expression->IsA<UMaterialExpressionTextureCoordinate>())
	{
		return true;
	}

	return false;
}

FString UMaterialGraphNode::GetDocumentationExcerptName() const
{
	// Default the node to searching for an excerpt named for the C++ node class name, including the U prefix.
	// This is done so that the excerpt name in the doc file can be found by find-in-files when searching for the full class name.
	UClass* MyClass = (MaterialExpression != NULL) ? MaterialExpression->GetClass() : this->GetClass();
	return FString::Printf(TEXT("%s%s"), MyClass->GetPrefixCPP(), *MyClass->GetName());
}

bool UMaterialGraphNode::CanUserDeleteNode() const
{
	if (MaterialExpression != NULL)
	{
		return MaterialExpression->CanUserDeleteExpression();
	}
	return true;
}


#undef LOCTEXT_NAMESPACE
