// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MaterialGraphNode_Base.cpp
=============================================================================*/

#include "MaterialGraph/MaterialGraphNode_Base.h"
#include "EdGraph/EdGraphSchema.h"
#include "MaterialGraph/MaterialGraph.h"
#include "MaterialGraph/MaterialGraphSchema.h"
#include "MaterialValueType.h"

/////////////////////////////////////////////////////
// UMaterialGraphNode_Base

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialGraphNode_Base)

UMaterialGraphNode_Base::UMaterialGraphNode_Base(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UMaterialGraphNode_Base::GetSourceIndexForInputIndex(int32 InputIndex) const
{
	// For most node types, SourceIndex==InputIndex
	return InputIndex;
}

UEdGraphPin* UMaterialGraphNode_Base::GetInputPin(int32 InputIndex) const
{
	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin->PinType.PinCategory != UMaterialGraphSchema::PC_Exec &&
			Pin->Direction == EGPD_Input &&
			Pin->SourceIndex == InputIndex)
		{
			return Pin;
		}
	}
	return nullptr;
}

UEdGraphPin* UMaterialGraphNode_Base::GetInputPin(const FName& PinName) const
{
	// Return the first input pin matching the name
	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin->PinType.PinCategory != UMaterialGraphSchema::PC_Exec &&
			Pin->Direction == EGPD_Input &&
			Pin->PinName == PinName)
		{
			return Pin;
		}
	}
	return nullptr;
}

UEdGraphPin* UMaterialGraphNode_Base::GetOutputPin(int32 OutputIndex) const
{
	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin->PinType.PinCategory != UMaterialGraphSchema::PC_Exec &&
			Pin->Direction == EGPD_Output &&
			Pin->SourceIndex == OutputIndex)
		{
			return Pin;
		}
	}
	return nullptr;
}

UEdGraphPin* UMaterialGraphNode_Base::GetExecInputPin() const
{
	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin->PinType.PinCategory == UMaterialGraphSchema::PC_Exec &&
			Pin->Direction == EGPD_Input)
		{
			check(Pin->SourceIndex == 0);
			return Pin;
		}
	}
	return nullptr;
}

UEdGraphPin* UMaterialGraphNode_Base::GetExecOutputPin(int32 OutputIndex) const
{
	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin->PinType.PinCategory == UMaterialGraphSchema::PC_Exec &&
			Pin->Direction == EGPD_Output &&
			Pin->SourceIndex == OutputIndex)
		{
			return Pin;
		}
	}
	return nullptr;
}

EMaterialValueType UMaterialGraphNode_Base::GetOutputValueType(const UEdGraphPin* OutputPin) const
{
	return GetPinMaterialValueType(OutputPin);
}

EMaterialValueType UMaterialGraphNode_Base::GetInputValueType(const UEdGraphPin* InputPin) const
{
	return GetPinMaterialValueType(InputPin);
}

EMaterialValueType UMaterialGraphNode_Base::GetPinMaterialValueType(const UEdGraphPin* Pin) const
{
	if (Pin->PinType.PinCategory == UMaterialGraphSchema::PC_Exec)
	{
		return MCT_Execution;
	}
	return MCT_Unknown;
}

void UMaterialGraphNode_Base::ReplaceNode(UMaterialGraphNode_Base* OldNode)
{
	check(OldNode);
	check(OldNode != this);

	for (int32 PinIndex = 0; PinIndex < OldNode->Pins.Num(); ++PinIndex)
	{
		UEdGraphPin* OldPin = OldNode->Pins[PinIndex];
		if (OldPin->Direction == EGPD_Input)
		{
			UEdGraphPin* NewPin = nullptr;
			if(OldPin->PinType.PinCategory == UMaterialGraphSchema::PC_Exec) NewPin = GetExecInputPin();
			else NewPin = GetInputPin(OldPin->SourceIndex);

			if (NewPin)
			{
				ModifyAndCopyPersistentPinData(*NewPin, *OldPin);
			}
		}
		else
		{
			// Try to find an equivalent output in this node
			int32 FoundPinIndex = -1;
			{
				// First check names
				for (int32 NewPinIndex = 0; NewPinIndex < Pins.Num(); NewPinIndex++)
				{
					UEdGraphPin* NewPin = Pins[NewPinIndex];
					if (NewPin->Direction == EGPD_Output &&
						NewPin->PinType.PinCategory == OldPin->PinType.PinCategory &&
						NewPin->PinName == OldPin->PinName)
					{
						FoundPinIndex = NewPinIndex;
						break;
					}
				}
			}
			if (FoundPinIndex == -1)
			{
				// Now check types
				for (int32 NewPinIndex = 0; NewPinIndex < Pins.Num(); NewPinIndex++)
				{
					UEdGraphPin* NewPin = Pins[NewPinIndex];
					if (NewPin->Direction == EGPD_Output &&
						NewPin->PinType == OldPin->PinType)
					{
						FoundPinIndex = NewPinIndex;
						break;
					}
				}
			}

			// If we can't find an equivalent output in this node, just use the first
			// The user will have to fix up any issues from the mismatch
			FoundPinIndex = FMath::Max(FoundPinIndex, 0);
			ModifyAndCopyPersistentPinData(*Pins[FoundPinIndex], *OldPin);
		}
	}

	// Break the original pin links
	for (int32 OldPinIndex = 0; OldPinIndex < OldNode->Pins.Num(); ++OldPinIndex)
	{
		UEdGraphPin* OldPin = OldNode->Pins[OldPinIndex];
		OldPin->Modify();
		OldPin->BreakAllPinLinks();
	}
}

void UMaterialGraphNode_Base::InsertNewNode(UEdGraphPin* FromPin, UEdGraphPin* NewLinkPin, TSet<UEdGraphNode*>& OutNodeList)
{
	const UMaterialGraphSchema* Schema = CastChecked<UMaterialGraphSchema>(GetSchema());

	// The pin we are creating from already has a connection that needs to be broken. We want to "insert" the new node in between, so that the output of the new node is hooked up too
	UEdGraphPin* OldLinkedPin = FromPin->LinkedTo[0];
	check(OldLinkedPin);

	FromPin->BreakAllPinLinks();

	// Hook up the old linked pin to the first valid output pin on the new node
	for (int32 OutpinPinIdx=0; OutpinPinIdx<Pins.Num(); OutpinPinIdx++)
	{
		UEdGraphPin* OutputPin = Pins[OutpinPinIdx];
		check(OutputPin);
		if (ECanCreateConnectionResponse::CONNECT_RESPONSE_MAKE == Schema->CanCreateConnection(OldLinkedPin, OutputPin).Response)
		{
			if (Schema->TryCreateConnection(OldLinkedPin, OutputPin))
			{
				OutNodeList.Add(OldLinkedPin->GetOwningNode());
				OutNodeList.Add(this);
			}
			break;
		}
	}

	if (Schema->TryCreateConnection(FromPin, NewLinkPin))
	{
		OutNodeList.Add(FromPin->GetOwningNode());
		OutNodeList.Add(this);
	}
}

void UMaterialGraphNode_Base::AllocateDefaultPins()
{
	check(Pins.Num() == 0);
	CreateInputPins();
	CreateOutputPins();
}

void UMaterialGraphNode_Base::PostPasteNode()
{
	int32 NumInputDataPins = 0;
	int32 NumOutputDataPins = 0;
	int32 NumInputExecPins = 0;
	int32 NumOutputExecPins = 0;
	for (UEdGraphPin* Pin : Pins)
	{
		int32 Index = INDEX_NONE;
		if (Pin->PinType.PinCategory == UMaterialGraphSchema::PC_Exec)
		{
			if (Pin->Direction == EGPD_Input) Index = NumInputExecPins++;
			else Index = NumOutputExecPins++;
		}
		else
		{
			if (Pin->Direction == EGPD_Input) Index = NumInputDataPins++;
			else Index = NumOutputDataPins++;
		}
		Pin->SourceIndex = Index;
	}
}

void UMaterialGraphNode_Base::EmptyPins()
{
	Pins.Reset();
}

void UMaterialGraphNode_Base::ReconstructNode()
{
	Modify();

	// Break any links to 'orphan' pins
	for (int32 PinIndex = 0; PinIndex < Pins.Num(); ++PinIndex)
	{
		UEdGraphPin* Pin = Pins[PinIndex];
		TArray<class UEdGraphPin*>& LinkedToRef = Pin->LinkedTo;
		for (int32 LinkIdx=0; LinkIdx < LinkedToRef.Num(); LinkIdx++)
		{
			UEdGraphPin* OtherPin = LinkedToRef[LinkIdx];
			// If we are linked to a pin that its owner doesn't know about, break that link
			if (!OtherPin->GetOwningNode()->Pins.Contains(OtherPin))
			{
				Pin->LinkedTo.Remove(OtherPin);
			}
		}
	}

	// Move the existing pins to a saved array
	TArray<UEdGraphPin*> OldPins = MoveTemp(Pins);

	EmptyPins();

	// Recreate the new pins
	AllocateDefaultPins();

	for (int32 OldPinIndex = 0; OldPinIndex < OldPins.Num(); ++OldPinIndex)
	{
		UEdGraphPin* OldPin = OldPins[OldPinIndex];
		UEdGraphPin* NewPin = nullptr;
		if (OldPin->PinType.PinCategory == UMaterialGraphSchema::PC_Exec)
		{
			if (OldPin->Direction == EGPD_Input) 
			{
				NewPin = GetExecInputPin();
			}
			else 
			{
				NewPin = GetExecOutputPin(OldPin->SourceIndex);
			}
		}
		else
		{
			if (OldPin->Direction == EGPD_Input)
			{
				NewPin = GetInputPin(OldPin->PinName);
				if (NewPin == nullptr)
				{
					NewPin = GetInputPin(OldPin->SourceIndex);
				}
			}
			else 
			{
				NewPin = GetOutputPin(OldPin->SourceIndex);
			}
		}
		if (NewPin)
		{
			NewPin->MovePersistentDataFromOldPin(*OldPin);
		}
	}

	for (UEdGraphPin* OldPin : OldPins)
	{
		// Throw away the original pins
		OldPin->Modify();
		UEdGraphNode::DestroyPin(OldPin);
	}

	GetGraph()->NotifyGraphChanged();
}

void UMaterialGraphNode_Base::RemovePinAt(const int32 PinIndex, const EEdGraphPinDirection PinDirection)
{
	UEdGraphPin* Pin = GetPinWithDirectionAt(PinIndex, PinDirection);
	check(Pin);

	// Shift down indices to account for the pin we removed
	for (UEdGraphPin* CheckPin : Pins)
	{
		if (CheckPin->PinType.PinCategory == Pin->PinType.PinCategory &&
			CheckPin->Direction == Pin->Direction &&
			CheckPin->SourceIndex > Pin->SourceIndex)
		{
			CheckPin->SourceIndex--;
		}
	}

	Super::RemovePinAt(PinIndex, PinDirection);

	UMaterialGraph* MaterialGraph = CastChecked<UMaterialGraph>(GetGraph());
	MaterialGraph->LinkMaterialExpressionsFromGraph();
}

void UMaterialGraphNode_Base::AutowireNewNode(UEdGraphPin* FromPin)
{
	if (FromPin != NULL)
	{
		const UMaterialGraphSchema* Schema = CastChecked<UMaterialGraphSchema>(GetSchema());

		TSet<UEdGraphNode*> NodeList;

		// auto-connect from dragged pin to first compatible pin on the new node
		for (int32 i=0; i<Pins.Num(); i++)
		{
			UEdGraphPin* Pin = Pins[i];
			check(Pin);
			FPinConnectionResponse Response = Schema->CanCreateConnection(FromPin, Pin);
			if (ECanCreateConnectionResponse::CONNECT_RESPONSE_MAKE == Response.Response) //-V1051
			{
				if (Schema->TryCreateConnection(FromPin, Pin))
				{
					NodeList.Add(FromPin->GetOwningNode());
					NodeList.Add(this);
				}
				break;
			}
			else if(ECanCreateConnectionResponse::CONNECT_RESPONSE_BREAK_OTHERS_A == Response.Response)
			{
				InsertNewNode(FromPin, Pin, NodeList);
				break;
			}
		}

		// Send all nodes that received a new pin connection a notification
		for (auto It = NodeList.CreateConstIterator(); It; ++It)
		{
			UEdGraphNode* Node = (*It);
			Node->NodeConnectionListChanged();
		}
	}
}

bool UMaterialGraphNode_Base::CanCreateUnderSpecifiedSchema(const UEdGraphSchema* Schema) const
{
	return Schema->IsA(UMaterialGraphSchema::StaticClass());
}

void UMaterialGraphNode_Base::ModifyAndCopyPersistentPinData(UEdGraphPin& TargetPin, const UEdGraphPin& SourcePin) const
{
	if (SourcePin.LinkedTo.Num() > 0)
	{
		TargetPin.Modify();

		for (int32 LinkIndex = 0; LinkIndex < SourcePin.LinkedTo.Num(); ++LinkIndex)
		{
			UEdGraphPin* OtherPin = SourcePin.LinkedTo[LinkIndex];
			OtherPin->Modify();
		}
	}

	TargetPin.CopyPersistentDataFromOldPin(SourcePin);
}

FString UMaterialGraphNode_Base::GetDocumentationLink() const
{
	return TEXT("Shared/GraphNodes/Material");
}

