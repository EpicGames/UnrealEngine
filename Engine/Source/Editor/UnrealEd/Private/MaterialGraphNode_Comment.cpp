// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MaterialGraphNode_Comment.cpp
=============================================================================*/

#include "MaterialGraph/MaterialGraphNode_Comment.h"
#include "ToolMenus.h"
#include "MaterialGraph/MaterialGraphSchema.h"
#include "Materials/MaterialExpressionComment.h"
#include "MaterialNodes/SGraphNodeMaterialComment.h"
#include "Materials/MaterialFunction.h"
#include "Framework/Commands/GenericCommands.h"
#include "GraphEditorActions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialGraphNode_Comment)

#define LOCTEXT_NAMESPACE "MaterialGraphNode_Comment"

/////////////////////////////////////////////////////
// UMaterialGraphNode_Comment

UMaterialGraphNode_Comment::UMaterialGraphNode_Comment(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UMaterialGraphNode_Comment::PostCopyNode()
{
	// Make sure the MaterialExpression goes back to being owned by the Material after copying.
	ResetMaterialExpressionOwner();
}

void UMaterialGraphNode_Comment::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		const FName PropertyName = PropertyChangedEvent.Property->GetFName();
		if (PropertyName == FName(TEXT("NodeComment")))
		{
			if (MaterialExpressionComment)
			{
				MaterialExpressionComment->Modify();
				MaterialExpressionComment->Text = NodeComment;
			}
		}
	}
}

void UMaterialGraphNode_Comment::PostEditImport()
{
	// Make sure this MaterialExpression is owned by the Material it's being pasted into.
	ResetMaterialExpressionOwner();
}

void UMaterialGraphNode_Comment::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	if (!bDuplicateForPIE)
	{
		CreateNewGuid();
	}
}

TSharedPtr<SGraphNode> UMaterialGraphNode_Comment::CreateVisualWidget()
{
	return SNew(SGraphNodeMaterialComment, this);
}

void UMaterialGraphNode_Comment::PrepareForCopying()
{
	if (MaterialExpressionComment)
	{
		// Temporarily take ownership of the MaterialExpression, so that it is not deleted when cutting
		MaterialExpressionComment->Rename(NULL, this, REN_DontCreateRedirectors);
	}
}

void UMaterialGraphNode_Comment::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	if (Context->Node && !Context->Pin)
	{
		// frequently used common options
		{
			FToolMenuSection& Section = Menu->AddSection("MaterialEditorCommentMenu");
			Section.AddMenuEntry(FGenericCommands::Get().Delete);
			Section.AddMenuEntry(FGenericCommands::Get().Cut);
			Section.AddMenuEntry(FGenericCommands::Get().Copy);
			Section.AddMenuEntry(FGenericCommands::Get().Duplicate);
		}
		
		{
			FToolMenuSection& Section = Menu->AddSection("EdGraphSchemaOrganization", LOCTEXT("OrganizationHeader", "Organization"));
			Section.AddMenuEntry(FGraphEditorCommands::Get().CollapseNodes);
			Section.AddMenuEntry(FGraphEditorCommands::Get().ExpandNodes);

			Section.AddSubMenu(
				"Alignment",
				LOCTEXT("AlignmentHeader", "Alignment"),
				FText(),
				FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
				{
					{
						FToolMenuSection& SubMenuSection = InMenu->AddSection("EdGraphSchemaAlignment", LOCTEXT("AlignHeader", "Align"));
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesTop);
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesMiddle);
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesBottom);
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesLeft);
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesCenter);
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesRight);
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().StraightenConnections);
					}

					{
						FToolMenuSection& SubMenuSection = InMenu->AddSection("EdGraphSchemaDistribution", LOCTEXT("DistributionHeader", "Distribution"));
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().DistributeNodesHorizontally);
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().DistributeNodesVertically);
					}
				}));
		}
	}
}

bool UMaterialGraphNode_Comment::CanCreateUnderSpecifiedSchema(const UEdGraphSchema* Schema) const
{
	return Schema->IsA(UMaterialGraphSchema::StaticClass());
}

void UMaterialGraphNode_Comment::PostPlacedNewNode()
{
	// always used for material comments
	MoveMode = ECommentBoxMode::GroupMovement;

	if (MaterialExpressionComment)
	{
		NodeComment = MaterialExpressionComment->Text;
		NodePosX = MaterialExpressionComment->MaterialExpressionEditorX;
		NodePosY = MaterialExpressionComment->MaterialExpressionEditorY;
		NodeWidth = MaterialExpressionComment->SizeX;
		NodeHeight = MaterialExpressionComment->SizeY;
		CommentColor = MaterialExpressionComment->CommentColor;
		bCommentBubbleVisible_InDetailsPanel = MaterialExpressionComment->bCommentBubbleVisible_InDetailsPanel;
		bCommentBubbleVisible = MaterialExpressionComment->bCommentBubbleVisible_InDetailsPanel;
		bCommentBubblePinned = MaterialExpressionComment->bCommentBubbleVisible_InDetailsPanel;
		MoveMode = MaterialExpressionComment->bGroupMode ? ECommentBoxMode::GroupMovement : ECommentBoxMode::NoGroupMovement;
		bColorCommentBubble = MaterialExpressionComment->bColorCommentBubble;
	}
}

void UMaterialGraphNode_Comment::OnRenameNode(const FString& NewName)
{
	// send property changed events
	FProperty* NodeCommentProperty = FindFProperty<FProperty>(GetClass(), "NodeComment");
	if(NodeCommentProperty != NULL)
	{
		PreEditChange(NodeCommentProperty);

		NodeComment = NewName;

		FPropertyChangedEvent NodeCommentPropertyChangedEvent(NodeCommentProperty);
		PostEditChangeProperty(NodeCommentPropertyChangedEvent);
	}
}

void UMaterialGraphNode_Comment::ResizeNode(const FVector2f& NewSize)
{
	Super::ResizeNode(NewSize);

	if(MaterialExpressionComment)
	{
		// Set position as well since may have been resized from top corner
		MaterialExpressionComment->SizeX = NodeWidth;
		MaterialExpressionComment->SizeY = NodeHeight;
		MaterialExpressionComment->MaterialExpressionEditorX = NodePosX;
		MaterialExpressionComment->MaterialExpressionEditorY = NodePosY;
		MaterialExpressionComment->MarkPackageDirty();
		MaterialDirtyDelegate.ExecuteIfBound();
	}
}

int32 UMaterialGraphNode_Comment::GetFontSize() const
{
	if (MaterialExpressionComment)
	{
		return MaterialExpressionComment->FontSize;
	}
	return Super::GetFontSize();
}

void UMaterialGraphNode_Comment::ResetMaterialExpressionOwner()
{
	if (MaterialExpressionComment)
	{
		// Ensures MaterialExpression is owned by the Material or Function
		UMaterialGraph* MaterialGraph = CastChecked<UMaterialGraph>(GetGraph());
		UObject* ExpressionOuter = MaterialGraph->Material;
		if (MaterialGraph->MaterialFunction)
		{
			ExpressionOuter = MaterialGraph->MaterialFunction;
		}
		MaterialExpressionComment->Rename(NULL, ExpressionOuter, REN_DontCreateRedirectors);

		// Set up the back pointer for newly created material nodes
		MaterialExpressionComment->GraphNode = this;
	}
}


#undef LOCTEXT_NAMESPACE
