// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MaterialGraphNode_Composite.cpp
=============================================================================*/

#include "MaterialGraph/MaterialGraphNode_Composite.h"
#include "ToolMenus.h"
#include "MaterialGraph/MaterialGraphSchema.h"
#include "MaterialGraph/MaterialGraphNode.h"
#include "MaterialGraph/MaterialGraphNode_Comment.h"
#include "Materials/MaterialExpressionComposite.h"
#include "Materials/MaterialExpressionPinBase.h"
#include "MaterialEditingLibrary.h"
#include "MaterialEditorUtilities.h"
#include "EdGraphUtilities.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Framework/Commands/GenericCommands.h"
#include "MaterialEditor/MaterialNodes/SGraphNodeMaterialComposite.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialGraphNode_Composite)

#define LOCTEXT_NAMESPACE "MaterialGraphNode_Composite"

/////////////////////////////////////////////////////
// UMaterialGraphNode_Composite

UMaterialGraphNode_Composite::UMaterialGraphNode_Composite(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UMaterialGraphNode_Composite::PostEditUndo()
{
	Super::PostEditUndo();
	FixupInputAndOutputPinBases();
}

void UMaterialGraphNode_Composite::PostCopyNode()
{
	Super::PostCopyNode();

	if (BoundGraph != nullptr)
	{
		for (UEdGraphNode* Node : BoundGraph->Nodes)
		{
			if (UMaterialGraphNode* MaterialNode = Cast<UMaterialGraphNode>(Node))
			{
				MaterialNode->PostCopyNode();
			}
			else if (UMaterialGraphNode_Comment* CommentNode = Cast<UMaterialGraphNode_Comment>(Node))
			{
				CommentNode->PostCopyNode();
			}
		}
	}
}

UObject* UMaterialGraphNode_Composite::GetJumpTargetForDoubleClick() const
{
	return BoundGraph;
}

void UMaterialGraphNode_Composite::DestroyNode()
{
	UEdGraph* GraphToRemove = BoundGraph;

	BoundGraph = nullptr;
	Super::DestroyNode();

	if (GraphToRemove)
	{
		GraphToRemove->Modify();
		FBlueprintEditorUtils::RemoveGraph(nullptr, GraphToRemove, EGraphRemoveFlags::None);
	}
}

void UMaterialGraphNode_Composite::PrepareForCopying()
{
	if (BoundGraph != nullptr)
	{
		for (UEdGraphNode* Node : BoundGraph->Nodes)
		{
			Node->PrepareForCopying();
		}
	}

	Super::PrepareForCopying();
}

void UMaterialGraphNode_Composite::PostPasteNode()
{
	Super::PostPasteNode();

	if (BoundGraph != nullptr)
	{
		UEdGraph* ParentGraph = CastChecked<UEdGraph>(GetOuter());
		ensure(BoundGraph != ParentGraph);

		// Update the InputSinkNode / OutputSourceNode pointers to point to the new graph
		TSet<UEdGraphNode*> BoundaryNodes;
		for (int32 NodeIndex = 0; NodeIndex < BoundGraph->Nodes.Num(); ++NodeIndex)
		{
			UEdGraphNode* Node = BoundGraph->Nodes[NodeIndex];
			BoundaryNodes.Add(Node);
		}

		ensure(BoundGraph->SubGraphs.Find(ParentGraph) == INDEX_NONE);

		//Nested composites will already be in the SubGraph array
		if (ParentGraph->SubGraphs.Find(BoundGraph) == INDEX_NONE)
		{
			ParentGraph->SubGraphs.Add(BoundGraph);
		}

		FEdGraphUtilities::PostProcessPastedNodes(BoundaryNodes);
	}
}

void UMaterialGraphNode_Composite::OnRenameNode(const FString& NewName)
{
	MaterialExpression->Modify();
	MaterialExpression->SetEditableName(NewName);
	MaterialExpression->MarkPackageDirty();
	MaterialExpression->ValidateParameterName();

	CastChecked<UMaterialGraph>(GetGraph())->Material->UpdateExpressionParameterName(MaterialExpression);
	MaterialDirtyDelegate.ExecuteIfBound();
}

void UMaterialGraphNode_Composite::ReconstructNode()
{
	FixupInputAndOutputPinBases();
}

TSharedPtr<SGraphNode> UMaterialGraphNode_Composite::CreateVisualWidget()
{
	return SNew(SGraphNodeMaterialComposite, this);
}

void UMaterialGraphNode_Composite::FixupInputAndOutputPinBases()
{
	if (BoundGraph)
	{
		for (UEdGraphNode* Node : BoundGraph->Nodes)
		{
			if (UMaterialGraphNode* MaterialNode = Cast<UMaterialGraphNode>(Node))
			{
				UMaterialExpressionPinBase* PinBase = Cast<UMaterialExpressionPinBase>(MaterialNode->MaterialExpression);

				if (PinBase && PinBase->PinDirection == EGPD_Input)
				{
					CastChecked<UMaterialExpressionComposite>(MaterialExpression)->OutputExpressions = PinBase;
					PinBase->SubgraphExpression = MaterialExpression;
				}
				else if (PinBase && PinBase->PinDirection == EGPD_Output)
				{
					CastChecked<UMaterialExpressionComposite>(MaterialExpression)->InputExpressions = PinBase;
					PinBase->SubgraphExpression = MaterialExpression;
				}
			}
		}

		UMaterialGraphNode_Base::ReconstructNode();
	}
}

#undef LOCTEXT_NAMESPACE
