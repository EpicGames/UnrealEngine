// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialGraph/MaterialGraphNode_Custom.h"
#include "MaterialEditor/MaterialNodes/SGraphNodeMaterialCustom.h"

#include "Materials/MaterialExpressionCustom.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialGraphNode_Custom)

#define CODE_PROPERTY_NAME TEXT("Code")
#define SHOW_CODE_PROPERTY_NAME TEXT("ShowCode")

UMaterialGraphNode_Custom::UMaterialGraphNode_Custom(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

TSharedPtr<SGraphNode> UMaterialGraphNode_Custom::CreateVisualWidget()
{
	return SNew(SGraphNodeMaterialCustom, this);
}

FText UMaterialGraphNode_Custom::GetHlslText() const
{
	UMaterialExpressionCustom* CustomExpression = CastChecked<UMaterialExpressionCustom>(MaterialExpression.Get());
	return FText::FromString(CustomExpression->Code);
}

void UMaterialGraphNode_Custom::OnCustomHlslTextCommitted(const FText& InText, ETextCommit::Type InType)
{
	UMaterialExpressionCustom* CustomExpression = CastChecked<UMaterialExpressionCustom>(MaterialExpression.Get());

	FString NewValue = InText.ToString();
	if (!NewValue.Equals(CustomExpression->Code, ESearchCase::CaseSensitive))
	{
		CustomExpression->Code = NewValue;

		ChangeProperty(CODE_PROPERTY_NAME);
	}
}

void UMaterialGraphNode_Custom::OnCodeViewChanged(const ECheckBoxState NewCheckedState)
{
	UMaterialExpressionCustom* Expression = Cast<UMaterialExpressionCustom>(MaterialExpression.Get());
	if (!Expression)
		return;

	Expression->ShowCode = (NewCheckedState == ECheckBoxState::Checked);

	ChangeProperty(SHOW_CODE_PROPERTY_NAME, false);
}

ECheckBoxState UMaterialGraphNode_Custom::IsCodeViewChecked() const
{
	UMaterialExpressionCustom* Expression = Cast<UMaterialExpressionCustom>(MaterialExpression.Get());
	if (!Expression)
		return ECheckBoxState::Checked;

	const bool bIsShown = (bool)Expression->ShowCode;
	return bIsShown ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void UMaterialGraphNode_Custom::ChangeProperty(FString PropertyName, bool bUpdatePreview)
{
	UMaterialExpressionCustom* Expression = Cast<UMaterialExpressionCustom>(MaterialExpression.Get());
	for (TFieldIterator<FProperty> InputIt(Expression->GetClass(), EFieldIteratorFlags::IncludeSuper, EFieldIteratorFlags::ExcludeDeprecated); InputIt; ++InputIt)
	{
		FProperty* Property = *InputIt;
		FString Name = Property->GetName();

		if (Name == PropertyName)
		{
			Expression->ForcePropertyValueChanged(Property, bUpdatePreview);
			break;
		}
	}
}
