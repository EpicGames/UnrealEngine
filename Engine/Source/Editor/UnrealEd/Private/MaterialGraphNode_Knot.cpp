// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialGraphNode_Knot.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "BlueprintNodeSpawner.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "MaterialEditorUtilities.h"
#include "EdGraph/EdGraphPin.h"

#include "SGraphNodeKnot.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialGraphNode_Knot)

#define LOCTEXT_NAMESPACE "MaterialGraphNode_Knot"

static const char* PC_Wildcard = "wildcard";

/////////////////////////////////////////////////////
// UMaterialGraphNode_Knot

UMaterialGraphNode_Knot::UMaterialGraphNode_Knot(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCanRenameNode = true;
}

void UMaterialGraphNode_Knot::AllocateDefaultPins()
{
	const FName InputPinName(TEXT("InputPin"));
	const FName OutputPinName(TEXT("OutputPin"));

	UEdGraphPin* MyInputPin = CreatePin(EGPD_Input, PC_Wildcard, InputPinName);
	MyInputPin->SourceIndex = 0;
	MyInputPin->bDefaultValueIsIgnored = true;

	UEdGraphPin* MyOutputPin = CreatePin(EGPD_Output, PC_Wildcard, OutputPinName);
	MyOutputPin->SourceIndex = 0;
}

FText UMaterialGraphNode_Knot::GetTooltipText() const
{
	//@TODO: Should pull the tooltip from the source pin
	return MaterialExpression->GetCreationDescription();
}

FText UMaterialGraphNode_Knot::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (TitleType == ENodeTitleType::EditableTitle)
	{
		return FText::FromString(NodeComment);
	}
	else if (TitleType == ENodeTitleType::MenuTitle)
	{
		return MaterialExpression->GetCreationName();
	}
	else
	{
		return LOCTEXT("KnotTitle", "Reroute Node");
	}
}


bool UMaterialGraphNode_Knot::ShouldOverridePinNames() const
{
	return true;
}

FText UMaterialGraphNode_Knot::GetPinNameOverride(const UEdGraphPin& Pin) const
{
	// Keep the pin size tiny
	return FText::GetEmpty();
}

void UMaterialGraphNode_Knot::OnRenameNode(const FString& NewName)
{
	NodeComment = NewName;
}


bool UMaterialGraphNode_Knot::CanSplitPin(const UEdGraphPin* Pin) const
{
	return false;
}

TSharedPtr<class INameValidatorInterface> UMaterialGraphNode_Knot::MakeNameValidator() const
{
	// Comments can be duplicated, etc...
	return MakeShareable(new FDummyNameValidator(EValidatorResult::Ok));
}

UEdGraphPin* UMaterialGraphNode_Knot::GetPassThroughPin(const UEdGraphPin* FromPin) const
{
	if(FromPin && Pins.Contains(FromPin))
	{
		return FromPin == Pins[0] ? Pins[1] : Pins[0];
	}

	return nullptr;
}

TSharedPtr<SGraphNode> UMaterialGraphNode_Knot::CreateVisualWidget()
{
	return SNew(SGraphNodeKnot, this);
}


/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
