// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialGraph/MaterialGraphNode_Operator.h"
#include "MaterialEditor/MaterialNodes/SGraphNodeMaterialOperator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialGraphNode_Operator)

UMaterialGraphNode_Operator::UMaterialGraphNode_Operator(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

TSharedPtr<SGraphNode> UMaterialGraphNode_Operator::CreateVisualWidget()
{
	return SNew(SGraphNodeMaterialOperator, this);
}
