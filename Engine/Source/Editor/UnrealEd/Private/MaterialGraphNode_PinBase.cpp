// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MaterialGraphNode_PinBase.cpp
=============================================================================*/

#include "MaterialGraph/MaterialGraphNode_PinBase.h"
#include "MaterialGraph/MaterialGraphSchema.h"
#include "MaterialGraph/MaterialGraphNode.h"
#include "MaterialGraph/MaterialGraphNode_Comment.h"
#include "Materials/MaterialExpressionPinBase.h"
#include "Materials/MaterialExpressionReroute.h"
#include "MaterialEditingLibrary.h"
#include "MaterialEditorUtilities.h"
#include "EdGraphUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialGraphNode_PinBase)

#define LOCTEXT_NAMESPACE "MaterialGraphNode_PinBase"

/////////////////////////////////////////////////////
// UMaterialGraphNode_PinBase

void UMaterialGraphNode_PinBase::PostCopyNode()
{
	Super::PostCopyNode();

	if (UMaterialExpressionPinBase* PinBase = Cast<UMaterialExpressionPinBase>(MaterialExpression))
	{
		for (FCompositeReroute& ReroutePin : PinBase->ReroutePins)
		{
			ReroutePin.Expression->Rename(nullptr, GetOuter(), REN_DontCreateRedirectors);
		}
	}
}

void UMaterialGraphNode_PinBase::PrepareForCopying()
{
	Super::PrepareForCopying();

	if (UMaterialExpressionPinBase* PinBase = Cast<UMaterialExpressionPinBase>(MaterialExpression))
	{
		for (FCompositeReroute& ReroutePin : PinBase->ReroutePins)
		{
			ReroutePin.Expression->Rename(nullptr, GetOuter(), REN_DontCreateRedirectors);
		}
	}
}

#undef LOCTEXT_NAMESPACE
