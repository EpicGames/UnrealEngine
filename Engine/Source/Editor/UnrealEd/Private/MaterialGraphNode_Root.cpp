// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MaterialGraphNode_Root.cpp
=============================================================================*/

#include "MaterialGraph/MaterialGraphNode_Root.h"
#include "MaterialShared.h"
#include "MaterialGraph/MaterialGraph.h"
#include "MaterialGraph/MaterialGraphSchema.h"
#include "MaterialEditorUtilities.h"
#include "Materials/MaterialAttributeDefinitionMap.h"
#include "GraphEditorSettings.h"

#include "MaterialEditor/MaterialNodes/SGraphNodeMaterialResult.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialGraphNode_Root)

#define LOCTEXT_NAMESPACE "MaterialGraphNode_Root"

/////////////////////////////////////////////////////
// UMaterialGraphNode_Root

UMaterialGraphNode_Root::UMaterialGraphNode_Root(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UMaterialGraphNode_Root::UpdateInputUseConstant(UEdGraphPin* Pin, bool bUseConstant)
{
	const UMaterialGraph* MaterialGraph = CastChecked<UMaterialGraph>(GetGraph());
	const FMaterialInputInfo& MaterialInput = MaterialGraph->MaterialInputs[Pin->SourceIndex];
	
	UMaterialEditorOnlyData* EditorOnlyData = Material->GetEditorOnlyData();
	EMaterialProperty Property = MaterialInput.GetProperty();
	switch (Property)
	{
	case MP_EmissiveColor:		EditorOnlyData->EmissiveColor.UseConstant = bUseConstant; break;
	case MP_Opacity:			EditorOnlyData->Opacity.UseConstant = bUseConstant; break;
	case MP_OpacityMask:		EditorOnlyData->OpacityMask.UseConstant = bUseConstant; break;
	case MP_BaseColor:			EditorOnlyData->BaseColor.UseConstant = bUseConstant; break;
	case MP_Metallic:			EditorOnlyData->Metallic.UseConstant = bUseConstant; break;
	case MP_Specular:			EditorOnlyData->Specular.UseConstant = bUseConstant; break;
	case MP_Roughness:			EditorOnlyData->Roughness.UseConstant = bUseConstant; break;
	case MP_Normal:				EditorOnlyData->Normal.UseConstant = bUseConstant; break;
	case MP_Tangent:			EditorOnlyData->Tangent.UseConstant = bUseConstant; break;
	case MP_SubsurfaceColor:	EditorOnlyData->SubsurfaceColor.UseConstant = bUseConstant; break;
	case MP_CustomData0:		EditorOnlyData->ClearCoat.UseConstant = bUseConstant; break;
	case MP_CustomData1:		EditorOnlyData->ClearCoatRoughness.UseConstant = bUseConstant; break;
	case MP_AmbientOcclusion:	EditorOnlyData->AmbientOcclusion.UseConstant = bUseConstant; break;
	case MP_Refraction:			EditorOnlyData->Refraction.UseConstant = bUseConstant; break;
	case MP_ShadingModel:		break; // TODO, see notes in CreateInputPins
	case MP_FrontMaterial:		break; // TODO, see notes in CreateInputPins
	case MP_SurfaceThickness:	EditorOnlyData->SurfaceThickness.UseConstant = bUseConstant; break;

	// TODO: Disabling these for the time being, as these values cause issues/artefacts when directly modified inline (UE-219232)
	case MP_PixelDepthOffset:	EditorOnlyData->PixelDepthOffset.UseConstant = false; break;
	case MP_Anisotropy:			EditorOnlyData->Anisotropy.UseConstant = false; break;
	case MP_WorldPositionOffset:EditorOnlyData->WorldPositionOffset.UseConstant = false; break;
	case MP_Displacement:		EditorOnlyData->Displacement.UseConstant = false; break;
	default:
		if (Property >= MP_CustomizedUVs0 && Property <= MP_CustomizedUVs7)
		{
			EditorOnlyData->CustomizedUVs[Property - MP_CustomizedUVs0].UseConstant = bUseConstant; break;
		}
		break;
	}
}

FText UMaterialGraphNode_Root::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FMaterialEditorUtilities::GetOriginalObjectName(this->GetGraph());
}

FLinearColor UMaterialGraphNode_Root::GetNodeTitleColor() const
{
	return GetDefault<UGraphEditorSettings>()->ResultNodeTitleColor;
}

FText UMaterialGraphNode_Root::GetTooltipText() const
{
	return LOCTEXT("MaterialNode", "Result node of the Material");
}

void UMaterialGraphNode_Root::PostPlacedNewNode()
{
	if (Material)
	{
		NodePosX = Material->EditorX;
		NodePosY = Material->EditorY;
	}
}

TSharedPtr<SGraphNode> UMaterialGraphNode_Root::CreateVisualWidget()
{
	return SNew(SGraphNodeMaterialResult, this);
}

void UMaterialGraphNode_Root::PinDefaultValueChanged(UEdGraphPin* Pin)
{
	const UMaterialGraph* MaterialGraph = CastChecked<UMaterialGraph>(GetGraph());
	const FMaterialInputInfo& MaterialInput = MaterialGraph->MaterialInputs[Pin->SourceIndex];

	UMaterialEditorOnlyData* EditorOnlyData = Material->GetEditorOnlyData();
	EMaterialProperty Property = MaterialInput.GetProperty();
	switch (Property)
	{
	case MP_EmissiveColor:		EditorOnlyData->EmissiveColor.DefaultValueChanged(Pin->DefaultValue); break;
	case MP_Opacity:			EditorOnlyData->Opacity.DefaultValueChanged(Pin->DefaultValue); break;
	case MP_OpacityMask:		EditorOnlyData->OpacityMask.DefaultValueChanged(Pin->DefaultValue); break;
	case MP_BaseColor:			EditorOnlyData->BaseColor.DefaultValueChanged(Pin->DefaultValue); break;
	case MP_Metallic:			EditorOnlyData->Metallic.DefaultValueChanged(Pin->DefaultValue); break;
	case MP_Specular:			EditorOnlyData->Specular.DefaultValueChanged(Pin->DefaultValue); break;
	case MP_Roughness:			EditorOnlyData->Roughness.DefaultValueChanged(Pin->DefaultValue); break;
	case MP_Normal:				EditorOnlyData->Normal.DefaultValueChanged(Pin->DefaultValue); break;
	case MP_Tangent:			EditorOnlyData->Tangent.DefaultValueChanged(Pin->DefaultValue); break;
	case MP_SubsurfaceColor:	EditorOnlyData->SubsurfaceColor.DefaultValueChanged(Pin->DefaultValue); break;
	case MP_CustomData0:		EditorOnlyData->ClearCoat.DefaultValueChanged(Pin->DefaultValue); break;
	case MP_CustomData1:		EditorOnlyData->ClearCoatRoughness.DefaultValueChanged(Pin->DefaultValue); break;
	case MP_AmbientOcclusion:	EditorOnlyData->AmbientOcclusion.DefaultValueChanged(Pin->DefaultValue); break;
	case MP_Refraction:			EditorOnlyData->Refraction.DefaultValueChanged(Pin->DefaultValue); break;
	case MP_ShadingModel:		break; // TODO, see notes in CreateInputPins
	case MP_FrontMaterial:		break; // TODO, see notes in CreateInputPins
	case MP_SurfaceThickness:	EditorOnlyData->SurfaceThickness.DefaultValueChanged(Pin->DefaultValue); break;

	// TODO: Disabling these for the time being, as these values cause issues/artefacts when directly modified inline (UE-219232)
	// case MP_Anisotropy:			EditorOnlyData->Anisotropy.DefaultValueChanged(Pin->DefaultValue); break;
	// case MP_PixelDepthOffset:	EditorOnlyData->PixelDepthOffset.DefaultValueChanged(Pin->DefaultValue); break;
	// case MP_WorldPositionOffset:EditorOnlyData->WorldPositionOffset.DefaultValueChanged(Pin->DefaultValue); break;
	// case MP_Displacement:		EditorOnlyData->Displacement.DefaultValueChanged(Pin->DefaultValue); break; 
	default:
		if (Property >= MP_CustomizedUVs0 && Property <= MP_CustomizedUVs7)
		{
			EditorOnlyData->CustomizedUVs[Property - MP_CustomizedUVs0].DefaultValueChanged(Pin->DefaultValue); break;
		}
		break;
	}

	FMaterialEditorUtilities::UpdateMaterialAfterGraphChange(MaterialGraph);
}

UObject* UMaterialGraphNode_Root::GetMaterialNodeOwner() const
{
	return Material;
}

int32 UMaterialGraphNode_Root::GetSourceIndexForInputIndex(int32 InputIndex) const
{
	const UMaterialGraph* MaterialGraph = CastChecked<UMaterialGraph>(GetGraph());
	for (int32 SourceIndex = 0; SourceIndex < MaterialGraph->MaterialInputs.Num(); ++SourceIndex)
	{
		// InputIndex will be the EMaterialProperty of the input
		// SourceIndex will be an index into the 'MaterialInputs' array
		if (InputIndex == (int32)MaterialGraph->MaterialInputs[SourceIndex].GetProperty())
		{
			return SourceIndex;
		}
	}
	return INDEX_NONE;
}

EMaterialValueType UMaterialGraphNode_Root::GetPinMaterialValueType(const UEdGraphPin* Pin) const
{
	if (Pin->PinType.PinCategory == UMaterialGraphSchema::PC_Exec)
	{
		return MCT_Execution;
	}

	const UMaterialGraph* MaterialGraph = CastChecked<UMaterialGraph>(GetGraph());
	const FMaterialInputInfo& MaterialInput = MaterialGraph->MaterialInputs[Pin->SourceIndex];
	EMaterialProperty Property = MaterialInput.GetProperty();

	EMaterialValueType MaterialType{};
	if (Property == MP_MaterialAttributes)
	{
		MaterialType = MCT_MaterialAttributes;
	}
	else if (Property == MP_FrontMaterial)
	{
		MaterialType = MCT_Substrate;
	}
	else
	{
		MaterialType = FMaterialAttributeDefinitionMap::GetValueType(Property);
	}
	return MaterialType;
}

void UMaterialGraphNode_Root::CreateInputPins()
{
	UMaterialGraph* MaterialGraph = CastChecked<UMaterialGraph>(GetGraph());

	UMaterialEditorOnlyData* EditorOnlyData = Material->GetEditorOnlyData();
	for (int32 Index = 0; Index < MaterialGraph->MaterialInputs.Num(); ++Index)
	{
		const FMaterialInputInfo& MaterialInput = MaterialGraph->MaterialInputs[Index];
		EMaterialProperty Property = MaterialInput.GetProperty();
	
		FName MaterialInputName = *MaterialInput.GetName().ToString();
		FName PinSubCategory;
		FString TempStr;
		switch (Property)
		{
			case MP_Metallic:
			case MP_Specular:
			case MP_Roughness:
			case MP_Opacity:
			case MP_OpacityMask:
			case MP_CustomData0:
			case MP_CustomData1:
			case MP_AmbientOcclusion:
			case MP_SurfaceThickness:
				PinSubCategory = UMaterialGraphSchema::PSC_Red;
				break;

			case MP_PixelDepthOffset:
				// TODO: Disabling these for the time being, as these values cause issues/artefacts when directly modified inline (UE-219232)
				// PinSubCategory = UMaterialGraphSchema::PSC_Red;
				// switch (MaterialGraph->Material->PixelDepthOffsetMode)
				// {
				// case EPixelDepthOffsetMode::PDOM_Legacy:
				// 	TempStr = TEXT("Legacy");
				// 	break;
				// case EPixelDepthOffsetMode::PDOM_AlongCameraVector:
				// 	TempStr = TEXT("Camera Vector");
				// 	break;
				// default:
				// 	TempStr = TEXT("UNKNOWN EPixelDepthOffsetMode");
				// }
				// MaterialInputName = *FString::Printf(TEXT("%s (%s)"), *MaterialInputName.ToString(), *TempStr);
				break;
				
			case MP_Refraction:
				PinSubCategory = UMaterialGraphSchema::PSC_Red;
				switch (MaterialGraph->Material->RefractionMethod)
				{
				case ERefractionMode::RM_None:
					TempStr = TEXT("Disabled");
					break;
				case ERefractionMode::RM_IndexOfRefraction:
					TempStr = TEXT("IOR");
					break;
				case ERefractionMode::RM_PixelNormalOffset:
					TempStr = TEXT("Pixel Normal Offset");
					break;
				case ERefractionMode::RM_2DOffset:
					TempStr = TEXT("2D Offset");
					break;
				case ERefractionMode::RM_IndexOfRefractionFromF0:
					TempStr = TEXT("IOR from F0");
					break;
				default:
					TempStr = TEXT("UNKNOWN ERefractionMode");
				}
				MaterialInputName = *FString::Printf(TEXT("%s (%s)"), *MaterialInputName.ToString(), *TempStr);
				break;
				
			case MP_Normal:
			case MP_Tangent:
				PinSubCategory = UMaterialGraphSchema::PSC_RGB;
				break;
				
			case MP_BaseColor:
			case MP_EmissiveColor:
			case MP_SubsurfaceColor:
				PinSubCategory = UMaterialGraphSchema::PSC_RGBA;
				break;

			case MP_ShadingModel: // TODO FMaterialInput<uint32>
			case MP_FrontMaterial: // TODO FMaterialInput<uint32>
				// TODO: Not sure if we want to simply show a Num Pin UI here.
				// Skipping these for now.
				// If these are based on Enum types, Enum pin construction requires a UEnum object to create UI with
				// which these constants don't have.
				break;

			// TODO: Disabling these for the time being, as these values cause issues/artefacts when directly modified inline (UE-219232)
			case MP_WorldPositionOffset:
			case MP_Anisotropy:
			case MP_Displacement:
				break;
			
			default:
				if (Property >= MP_CustomizedUVs0 && Property <= MP_CustomizedUVs7)
				{
					PinSubCategory = UMaterialGraphSchema::PSC_RG;
				}
				break;
		}
		
		UEdGraphPin* InputPin = CreatePin(EGPD_Input, UMaterialGraphSchema::PC_MaterialInput, PinSubCategory/*, *FString::Printf(TEXT("%d"), (int32)Property)*/, MaterialInputName);
		InputPin->SourceIndex = Index;

		switch (Property)
		{
			case MP_EmissiveColor:		InputPin->DefaultValue = EditorOnlyData->EmissiveColor.GetDefaultValue(); break;
			case MP_Opacity:			InputPin->DefaultValue = EditorOnlyData->Opacity.GetDefaultValue(); break;
			case MP_OpacityMask:		InputPin->DefaultValue = EditorOnlyData->OpacityMask.GetDefaultValue(); break;
			case MP_BaseColor:			InputPin->DefaultValue = EditorOnlyData->BaseColor.GetDefaultValue(); break;
			case MP_Metallic:			InputPin->DefaultValue = EditorOnlyData->Metallic.GetDefaultValue(); break;
			case MP_Specular:			InputPin->DefaultValue = EditorOnlyData->Specular.GetDefaultValue(); break;
			case MP_Roughness:			InputPin->DefaultValue = EditorOnlyData->Roughness.GetDefaultValue(); break;
			
			case MP_Normal:				InputPin->DefaultValue = EditorOnlyData->Normal.GetDefaultValue(); break;
			case MP_Tangent:			InputPin->DefaultValue = EditorOnlyData->Tangent.GetDefaultValue(); break;
			case MP_SubsurfaceColor:	InputPin->DefaultValue = EditorOnlyData->SubsurfaceColor.GetDefaultValue(); break;
			case MP_CustomData0:		InputPin->DefaultValue = EditorOnlyData->ClearCoat.GetDefaultValue(); break;
			case MP_CustomData1:		InputPin->DefaultValue = EditorOnlyData->ClearCoatRoughness.GetDefaultValue(); break;
			case MP_AmbientOcclusion:	InputPin->DefaultValue = EditorOnlyData->AmbientOcclusion.GetDefaultValue(); break;
			case MP_Refraction:			InputPin->DefaultValue = EditorOnlyData->Refraction.GetDefaultValue(); break;
			case MP_ShadingModel:		break; // TODO
			case MP_FrontMaterial:		break; // TODO
			case MP_SurfaceThickness:	InputPin->DefaultValue = EditorOnlyData->SurfaceThickness.GetDefaultValue(); break;

			// TODO: Disabling these for the time being, as these values cause issues/artefacts when directly modified inline (UE-219232)
			// case MP_WorldPositionOffset:InputPin->DefaultValue = EditorOnlyData->WorldPositionOffset.GetDefaultValue(); break;
			// case MP_PixelDepthOffset:	InputPin->DefaultValue = EditorOnlyData->PixelDepthOffset.GetDefaultValue(); break;
			// case MP_Anisotropy:			InputPin->DefaultValue = EditorOnlyData->Anisotropy.GetDefaultValue(); break;
			// case MP_Displacement:		InputPin->DefaultValue = EditorOnlyData->Displacement.GetDefaultValue(); break;
			default:
				if (Property >= MP_CustomizedUVs0 && Property <= MP_CustomizedUVs7)
				{
					InputPin->DefaultValue = EditorOnlyData->CustomizedUVs[Property - MP_CustomizedUVs0].GetDefaultValue(); break;
				}
				break;
		}
	}
}

#undef LOCTEXT_NAMESPACE
