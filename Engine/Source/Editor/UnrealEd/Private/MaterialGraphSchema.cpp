// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MaterialGraphSchema.cpp
=============================================================================*/

#include "MaterialGraph/MaterialGraphSchema.h"
#include "Misc/FeedbackContext.h"
#include "Modules/ModuleManager.h"
#include "UObject/UnrealType.h"
#include "UObject/PropertyPortFlags.h"
#include "Textures/SlateIcon.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ToolMenus.h"
#include "EdGraph/EdGraph.h"
#include "HAL/IConsoleManager.h"
#include "IMaterialEditor.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialFunction.h"
#include "MaterialGraph/MaterialGraph.h"
#include "Engine/Texture.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"
#include "MaterialGraph/MaterialGraphNode_Base.h"
#include "MaterialGraph/MaterialGraphNode_Comment.h"
#include "MaterialGraph/MaterialGraphNode.h"
#include "MaterialGraph/MaterialGraphNode_Root.h"
#include "Materials/MaterialParameterCollection.h"

#include "Materials/MaterialExpressionCollectionParameter.h"
#include "Materials/MaterialExpressionComment.h"
#include "Materials/MaterialExpressionComposite.h"
#include "Materials/MaterialExpressionPinBase.h"
#include "Materials/MaterialExpressionFunctionInput.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Materials/MaterialExpressionSparseVolumeTextureSample.h"
#include "Materials/MaterialExpressionFunctionOutput.h"
#include "Materials/MaterialExpressionReroute.h"
#include "Materials/MaterialExpressionNamedReroute.h"

#include "ScopedTransaction.h"
#include "MaterialEditorUtilities.h"
#include "GraphEditorActions.h"
#include "GraphEditorSettings.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Editor/AssetReferenceFilter.h"
#include "IAssetTools.h"
#include "MaterialEditorActions.h"
#include "MaterialGraph/MaterialShaderValueTypeObject.h"
#include "MaterialGraphNode_Knot.h"
#include "RenderUtils.h"
#include "MaterialEditorSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialGraphSchema)

#define LOCTEXT_NAMESPACE "MaterialGraphSchema"

int32 UMaterialGraphSchema::CurrentCacheRefreshID = 0;

////////////////////////////////////////
// FMaterialGraphSchemaAction_NewNode //

UEdGraphNode* FMaterialGraphSchemaAction_NewNode::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	check(MaterialExpressionClass);

	const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "MaterialEditorNewExpression", "Material Editor: New Expression"));

	// When FromPin is nullptr and Shift is pressed, we attempt to use the selected node's output pin as our FromPin
	if (FromPin == nullptr && FSlateApplication::Get().GetModifierKeys().IsShiftDown())
	{
		// Determine Previously Selected Node beforce creating the new node (which can select itself if bSelectNewNode is true)
		if (TSharedPtr<IMaterialEditor> MaterialEditor = FMaterialEditorUtilities::GetIMaterialEditorForObject(ParentGraph))
		{
			TSet<UObject*> SelectedNodes = MaterialEditor->GetSelectedNodes();
			if (SelectedNodes.Num() == 1)
			{
				for (UObject* SelectedNode : SelectedNodes)
				{
					if (UMaterialGraphNode_Base* SelectedMaterialGraphNode_Base = Cast<UMaterialGraphNode_Base>(SelectedNode))
					{
						FromPin = SelectedMaterialGraphNode_Base->GetOutputPin(0);
					}
				}
			}
		}
	}

	UMaterialExpression* NewExpression = FMaterialEditorUtilities::CreateNewMaterialExpression(ParentGraph, MaterialExpressionClass, FDeprecateSlateVector2D(Location), bSelectNewNode, /*bAutoAssignResource*/true);
	PostCreationDelegate.ExecuteIfBound(NewExpression);

	if (NewExpression)
	{
		if (MaterialExpressionClass == UMaterialExpressionFunctionInput::StaticClass() && FromPin)
		{
			// Set this to be an input of the type we dragged from
			SetFunctionInputType(CastChecked<UMaterialExpressionFunctionInput>(NewExpression), UMaterialGraphSchema::GetMaterialIOValueType(FromPin));
		}

		NewExpression->GraphNode->AutowireNewNode(FromPin);

		return NewExpression->GraphNode;
	}

	return NULL;
}

void FMaterialGraphSchemaAction_NewNode::SetFunctionInputType(UMaterialExpressionFunctionInput* FunctionInput, uint32 MaterialValueType) const
{
	return SetFunctionInputType(FunctionInput, static_cast<EMaterialValueType>(MaterialValueType));
}

void FMaterialGraphSchemaAction_NewNode::SetFunctionInputType(UMaterialExpressionFunctionInput* FunctionInput, EMaterialValueType MaterialValueType) const
{
	switch (MaterialValueType)
	{
	case MCT_Float:
	case MCT_Float1:
		FunctionInput->InputType = FunctionInput_Scalar;
		break;
	case MCT_Float2:
		FunctionInput->InputType = FunctionInput_Vector2;
		break;
	case MCT_Float3:
		FunctionInput->InputType = FunctionInput_Vector3;
		break;
	case MCT_Float4:
		FunctionInput->InputType = FunctionInput_Vector4;
		break;
	case MCT_Texture:
	case MCT_Texture2D:
		FunctionInput->InputType = FunctionInput_Texture2D;
		break;
	case MCT_TextureCube:
		FunctionInput->InputType = FunctionInput_TextureCube;
		break;
	case MCT_Texture2DArray:
		FunctionInput->InputType = FunctionInput_Texture2DArray;
		break;
	case MCT_TextureExternal:
		FunctionInput->InputType = FunctionInput_TextureExternal;
		break;
	case MCT_VolumeTexture:
		FunctionInput->InputType = FunctionInput_VolumeTexture;
		break;
	case MCT_StaticBool:
		FunctionInput->InputType = FunctionInput_StaticBool;
		break;
	case MCT_Bool:
		FunctionInput->InputType = FunctionInput_Bool;
		break;
	case MCT_MaterialAttributes:
		FunctionInput->InputType = FunctionInput_MaterialAttributes;
		break;
	case MCT_Substrate:
		FunctionInput->InputType = FunctionInput_Substrate;
		break;
	default:
		break;
	}
}

////////////////////////////////////////////////
// FMaterialGraphSchemaAction_NewFunctionCall //

UEdGraphNode* FMaterialGraphSchemaAction_NewFunctionCall::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "MaterialEditorNewFunctionCall", "Material Editor: New Function Call"));

	UMaterialExpressionMaterialFunctionCall* FunctionNode = CastChecked<UMaterialExpressionMaterialFunctionCall>(
																FMaterialEditorUtilities::CreateNewMaterialExpression(
																	ParentGraph, UMaterialExpressionMaterialFunctionCall::StaticClass(), FDeprecateSlateVector2D(Location), bSelectNewNode, /*bAutoAssignResource*/false));

	if (!FunctionNode->MaterialFunction)
	{
		UMaterialFunction* MaterialFunction = LoadObject<UMaterialFunction>(NULL, *FunctionPath, NULL, 0, NULL);
		UMaterialGraph* MaterialGraph = CastChecked<UMaterialGraph>(ParentGraph);
		if(FunctionNode->SetMaterialFunction(MaterialFunction))
		{
			FunctionNode->PostEditChange();
			FMaterialEditorUtilities::UpdateSearchResults(ParentGraph);
			FunctionNode->GraphNode->AutowireNewNode(FromPin);
			return FunctionNode->GraphNode;
		}
		else
		{
			FMaterialEditorUtilities::AddToSelection(ParentGraph, FunctionNode);
			FMaterialEditorUtilities::DeleteSelectedNodes(ParentGraph);
		}
	}

	return NULL;
}

/////////////////////////////////////////////
// FMaterialGraphSchemaAction_NewComposite //

UEdGraphNode* FMaterialGraphSchemaAction_NewComposite::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode)
{
	return SpawnNode(ParentGraph, Location);
}

UEdGraphNode* FMaterialGraphSchemaAction_NewComposite::SpawnNode(UEdGraph* ParentGraph, const UE::Slate::FDeprecateVector2DParameter& Location)
{
	const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "MaterialEditorNewComposite", "Material Editor: New Composite"));

	UMaterialExpressionComposite* NewComposite = FMaterialEditorUtilities::CreateNewMaterialExpressionComposite(ParentGraph, FDeprecateSlateVector2D(Location));

	if (NewComposite)
	{
		return NewComposite->GraphNode;
	}

	return nullptr;
}

///////////////////////////////////////////
// FMaterialGraphSchemaAction_NewComment //

UEdGraphNode* FMaterialGraphSchemaAction_NewComment::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "MaterialEditorNewComment", "Material Editor: New Comment") );

	UMaterialExpressionComment* NewComment = FMaterialEditorUtilities::CreateNewMaterialExpressionComment(ParentGraph, FDeprecateSlateVector2D(Location));

	if (NewComment)
	{
		return NewComment->GraphNode;
	}

	return NULL;
}

//////////////////////////////////////////////////////
// FMaterialGraphSchemaAction_NewNamedRerouteUsage //

UEdGraphNode* FMaterialGraphSchemaAction_NewNamedRerouteUsage::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode /*= true*/)
{
	check(Declaration);

	const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "MaterialEditorNewNamedRerouteUsage", "Material Editor: New Named Reroute Usage") );
	
	UMaterialExpression* NewExpression = FMaterialEditorUtilities::CreateNewMaterialExpression(ParentGraph, UMaterialExpressionNamedRerouteUsage::StaticClass(), FDeprecateSlateVector2D(Location), bSelectNewNode, /*bAutoAssignResource*/true);

	if (NewExpression)
	{
		auto* Usage = CastChecked<UMaterialExpressionNamedRerouteUsage>(NewExpression);
		Usage->Declaration = Declaration;
		Usage->DeclarationGuid = Declaration->VariableGuid;
		NewExpression->GraphNode->AutowireNewNode(FromPin);
		return NewExpression->GraphNode;
	}

	return NULL;
}

//////////////////////////////////////
// FMaterialGraphSchemaAction_Paste //

UEdGraphNode* FMaterialGraphSchemaAction_Paste::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	FMaterialEditorUtilities::PasteNodesHere(ParentGraph, FDeprecateSlateVector2D(Location));
	return NULL;
}

//////////////////////////
// UMaterialGraphSchema //

const FName UMaterialGraphSchema::PC_Mask(TEXT("mask"));
const FName UMaterialGraphSchema::PC_Required(TEXT("required"));
const FName UMaterialGraphSchema::PC_Optional(TEXT("optional"));
const FName UMaterialGraphSchema::PC_MaterialInput(TEXT("materialinput"));
const FName UMaterialGraphSchema::PC_Exec(TEXT("exec"));

const FName UMaterialGraphSchema::PSC_Red(TEXT("red"));
const FName UMaterialGraphSchema::PSC_Green(TEXT("green"));
const FName UMaterialGraphSchema::PSC_Blue(TEXT("blue"));
const FName UMaterialGraphSchema::PSC_Alpha(TEXT("alpha"));
const FName UMaterialGraphSchema::PSC_RGBA(TEXT("rgba"));
const FName UMaterialGraphSchema::PSC_RGB(TEXT("rgb"));
const FName UMaterialGraphSchema::PSC_RG(TEXT("rg"));
const FName UMaterialGraphSchema::PSC_Int(TEXT("int"));
const FName UMaterialGraphSchema::PSC_Byte(TEXT("byte"));
const FName UMaterialGraphSchema::PSC_Bool(TEXT("bool"));
const FName UMaterialGraphSchema::PSC_Float(TEXT("float"));
const FName UMaterialGraphSchema::PSC_Vector4(TEXT("vector4"));

const FName UMaterialGraphSchema::PN_Execute("execute");

const FLinearColor UMaterialGraphSchema::ActivePinColor = FLinearColor::White;
const FLinearColor UMaterialGraphSchema::InactivePinColor = FLinearColor(0.05f, 0.05f, 0.05f);
const FLinearColor UMaterialGraphSchema::AlphaPinColor = FLinearColor(0.5f, 0.5f, 0.5f);

UMaterialGraphSchema::UMaterialGraphSchema(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UMaterialGraphSchema::OnConnectToFunctionOutput(UEdGraphPin* InGraphPin, UEdGraphPin* InFuncPin)
{
	const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "GraphEd_CreateConnection", "Create Pin Link") );

	TryCreateConnection(InGraphPin, InFuncPin);
}

void UMaterialGraphSchema::OnConnectToMaterial(class UEdGraphPin* InGraphPin, int32 ConnIndex)
{
	const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "GraphEd_CreateConnection", "Create Pin Link") );

	UMaterialGraph* MaterialGraph = CastChecked<UMaterialGraph>(InGraphPin->GetOwningNode()->GetGraph());

	TryCreateConnection(InGraphPin, MaterialGraph->RootNode->GetInputPin(ConnIndex));
}

void UMaterialGraphSchema::GetPaletteActions(FGraphActionMenuBuilder& ActionMenuBuilder, const FString& CategoryName, bool bMaterialFunction) const
{
	UObject* MaterialOrFunction = bMaterialFunction ? UMaterialFunction::StaticClass()->GetDefaultObject() : UMaterial::StaticClass()->GetDefaultObject();
	return GetPaletteActions(ActionMenuBuilder, CategoryName, MaterialOrFunction);
}

void UMaterialGraphSchema::GetPaletteActions(FGraphActionMenuBuilder& ActionMenuBuilder, const FString& CategoryName, const UObject* MaterialOrFunction) const
{
	if (CategoryName != TEXT("Functions"))
	{
		FMaterialEditorUtilities::GetMaterialExpressionActions(ActionMenuBuilder, MaterialOrFunction);
		GetCommentAction(ActionMenuBuilder);
	}
	if (CategoryName != TEXT("Expressions"))
	{
		GetMaterialFunctionActions(ActionMenuBuilder);
	}
}

bool UMaterialGraphSchema::ConnectionCausesLoop(const UEdGraphPin* InputPin, const UEdGraphPin* OutputPin) const
{
	if (UMaterialGraphNode* OutputNode = Cast<UMaterialGraphNode>(OutputPin->GetOwningNode()))
	{
		TArray<UMaterialExpression*> InputExpressions;
		OutputNode->MaterialExpression->GetAllInputExpressions(InputExpressions);

		if (UMaterialGraphNode* InputNode = Cast<UMaterialGraphNode>(InputPin->GetOwningNode()))
		{
			return InputExpressions.Contains(InputNode->MaterialExpression);
		}
	}

	// Simple connection to root node
	return false;
}

bool UMaterialGraphSchema::ArePinsCompatible_Internal(const UEdGraphPin* InputPin, const UEdGraphPin* OutputPin, FText& ResponseMessage) const
{
	EMaterialValueType InputType = GetMaterialIOValueType(InputPin);
	EMaterialValueType OutputType = GetMaterialIOValueType(OutputPin);

	if (InputPin->bNotConnectable)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PinName"), FText::FromName(InputPin->PinName));
		ResponseMessage = FText::Format(LOCTEXT("PinNotConnectable", "Pin '{PinName}' is not connectable"), Args);	
		return false;
	}

	bool bPinsCompatible = CanConnectMaterialValueTypes(InputType, OutputType);
	if (!bPinsCompatible)
	{
		TArray<FText> InputDescriptions;
		TArray<FText> OutputDescriptions;
		GetMaterialValueTypeDescriptions(InputType, InputDescriptions);
		GetMaterialValueTypeDescriptions(OutputType, OutputDescriptions);

		FString CombinedInputDescription;
		FString CombinedOutputDescription;
		for (int32 Index = 0; Index < InputDescriptions.Num(); ++Index)
		{
			if ( CombinedInputDescription.Len() > 0 )
			{
				CombinedInputDescription += TEXT(", ");
			}
			CombinedInputDescription += InputDescriptions[Index].ToString();
		}
		for (int32 Index = 0; Index < OutputDescriptions.Num(); ++Index)
		{
			if ( CombinedOutputDescription.Len() > 0 )
			{
				CombinedOutputDescription += TEXT(", ");
			}
			CombinedOutputDescription += OutputDescriptions[Index].ToString();
		}

		FFormatNamedArguments Args;
		Args.Add( TEXT("InputType"), FText::FromString(CombinedInputDescription) );
		Args.Add( TEXT("OutputType"), FText::FromString(CombinedOutputDescription) );
		ResponseMessage = FText::Format( LOCTEXT("IncompatibleDesc", "{OutputType} is not compatible with {InputType}"), Args );
	}

	return bPinsCompatible;
}

uint32 UMaterialGraphSchema::GetMaterialValueType(const UEdGraphPin* MaterialPin)
{
	return static_cast<uint32>(GetMaterialIOValueType(MaterialPin));
}

EMaterialValueType UMaterialGraphSchema::GetMaterialIOValueType(const UEdGraphPin* MaterialPin)
{
	const UMaterialGraphNode_Base* OwningNode = CastChecked<UMaterialGraphNode_Base>(MaterialPin->GetOwningNode());
	if (MaterialPin->Direction == EGPD_Output)
	{
		return OwningNode->GetOutputValueType(MaterialPin);
	}
	else
	{
		return OwningNode->GetInputValueType(MaterialPin);
	}
}

void UMaterialGraphSchema::GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const
{
	const UMaterialGraph* MaterialGraph = CastChecked<UMaterialGraph>(ContextMenuBuilder.CurrentGraph);

	// Run through all nodes and add any menu items they want to add
	Super::GetGraphContextActions(ContextMenuBuilder);

	// Get the Context Actions from Material Editor Module
	FMaterialEditorUtilities::GetMaterialExpressionActions(ContextMenuBuilder, MaterialGraph->GetMaterialOrFunction());

	// Get the Material Functions as well
	GetMaterialFunctionActions(ContextMenuBuilder);

	GetCommentAction(ContextMenuBuilder, ContextMenuBuilder.CurrentGraph);

	GetNamedRerouteActions(ContextMenuBuilder, ContextMenuBuilder.CurrentGraph);

	// Add Paste here if appropriate
	if (!ContextMenuBuilder.FromPin && FMaterialEditorUtilities::CanPasteNodes(ContextMenuBuilder.CurrentGraph))
	{
		const FText PasteDesc = LOCTEXT("PasteDesc", "Paste Here");
		const FText PasteToolTip = LOCTEXT("PasteToolTip", "Pastes copied items at this location.");
		TSharedPtr<FMaterialGraphSchemaAction_Paste> PasteAction(new FMaterialGraphSchemaAction_Paste(FText::GetEmpty(), PasteDesc, PasteToolTip, 0));
		ContextMenuBuilder.AddAction(PasteAction);
	}
}

void UMaterialGraphSchema::GetContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	if (Context && Context->Pin)
	{
		const UEdGraphPin* InGraphPin = Context->Pin;
		const UMaterialGraph* MaterialGraph = CastChecked<UMaterialGraph>(Context->Graph);

		// add menu items to expression output for material connection
		if ( InGraphPin->Direction == EGPD_Output )
		{
			{
				FToolMenuSection& Section = Menu->AddSection("MaterialEditorMenuConnector2");
				// If we are editing a material function, display options to connect to function outputs
				if (MaterialGraph->MaterialFunction)
				{
					for (int32 Index = 0; Index < MaterialGraph->Nodes.Num(); Index++)
					{
						UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(MaterialGraph->Nodes[Index]);
						if (GraphNode)
						{
							UMaterialExpressionFunctionOutput* FunctionOutput = Cast<UMaterialExpressionFunctionOutput>(GraphNode->MaterialExpression);
							if (FunctionOutput)
							{
								FFormatNamedArguments Arguments;
								Arguments.Add(TEXT("Name"), FText::FromName( FunctionOutput->OutputName ));
								const FText Label = FText::Format( LOCTEXT( "ConnectToFunction", "Connect To {Name}" ), Arguments );
								const FText ToolTip = FText::Format( LOCTEXT( "ConnectToFunctionTooltip", "Connects to the function output {Name}" ), Arguments );
								Section.AddMenuEntry(
									NAME_None,
									Label,
									ToolTip,
									FSlateIcon(),
									FUIAction(FExecuteAction::CreateUObject(const_cast<UMaterialGraphSchema*>(this), &UMaterialGraphSchema::OnConnectToFunctionOutput, const_cast< UEdGraphPin* >(InGraphPin), GraphNode->GetInputPin(0))));
							}
						}
					}
				}
				else
				{
					for (int32 Index = 0; Index < MaterialGraph->MaterialInputs.Num(); ++Index)
					{
						if(MaterialGraph->MaterialInputs[Index].IsVisiblePin(MaterialGraph->Material))
						{
							FFormatNamedArguments Arguments;
							Arguments.Add(TEXT("Name"), MaterialGraph->MaterialInputs[Index].GetName());
							const FText Label = FText::Format( LOCTEXT( "ConnectToInput", "Connect To {Name}" ), Arguments );
							const FText ToolTip = FText::Format( LOCTEXT( "ConnectToInputTooltip", "Connects to the material input {Name}" ), Arguments );
							Section.AddMenuEntry(
								NAME_None,
								Label,
								ToolTip,
								FSlateIcon(),
								FUIAction(FExecuteAction::CreateUObject(const_cast<UMaterialGraphSchema*>(this), &UMaterialGraphSchema::OnConnectToMaterial, const_cast< UEdGraphPin* >(InGraphPin), Index)));
						}
					}
				}
			}
		}
	}
	else
	{
		//Moved all functionality to relevant node classes
	}

	Super::GetContextMenuActions(Menu, Context);
}

static TAutoConsoleVariable<int32> CVarPreventInvalidMaterialConnections(
	TEXT("r.PreventInvalidMaterialConnections"),
	1,
	TEXT("Controls whether users can make connections in the material editor if the system\n")
	TEXT("determines that they may cause compile errors\n")
	TEXT("0: Allow all connections\n")
	TEXT("1: Prevent invalid connections"),
	ECVF_Cheat);

const FPinConnectionResponse UMaterialGraphSchema::CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const
{
	bool bPreventInvalidConnections = CVarPreventInvalidMaterialConnections.GetValueOnGameThread() != 0;

	// Make sure the pins are not on the same node
	if (A->GetOwningNode() == B->GetOwningNode())
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectionSameNode", "Both are on the same node"));
	}

	// Compare the directions
	const UEdGraphPin* InputPin = NULL;
	const UEdGraphPin* OutputPin = NULL;

	if (!CategorizePinsByDirection(A, B, /*out*/ InputPin, /*out*/ OutputPin))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectionIncompatible", "Directions are not compatible"));
	}

	// Check for new and existing loops
	FText ResponseMessage;
	if (ConnectionCausesLoop(InputPin, OutputPin))
	{
		ResponseMessage = LOCTEXT("ConnectionLoop", "Connection could cause loop");
		// TODO: re-enable this if loops are going to be removed completely
		//return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectionLoop", "Connection would cause loop").ToString());
	}

	// Check for incompatible pins and get description if they cannot connect
	if (!ArePinsCompatible_Internal(InputPin, OutputPin, ResponseMessage) && bPreventInvalidConnections)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, ResponseMessage);
	}

	// For non-exec pins, break existing connections on inputs only - multiple output connections are acceptable
	if (InputPin->LinkedTo.Num() > 0)
	{
		const EMaterialValueType InputType = GetMaterialIOValueType(InputPin);
		if (!(InputType & MCT_Execution))
		{
			ECanCreateConnectionResponse ReplyBreakOutputs;
			if (InputPin == A)
			{
				ReplyBreakOutputs = CONNECT_RESPONSE_BREAK_OTHERS_A;
			}
			else
			{
				ReplyBreakOutputs = CONNECT_RESPONSE_BREAK_OTHERS_B;
			}
			if (ResponseMessage.IsEmpty())
			{
				ResponseMessage = LOCTEXT("ConnectionReplace", "Replace existing connections");
			}
			return FPinConnectionResponse(ReplyBreakOutputs, ResponseMessage);
		}
	}

	// For exec pins, reverse is true - multiple input connections are acceptable
	if (OutputPin->LinkedTo.Num() > 0)
	{
		const EMaterialValueType OutputType = GetMaterialIOValueType(InputPin);
		if (OutputType & MCT_Execution)
		{
			ECanCreateConnectionResponse ReplyBreakInputs;
			if (OutputPin == A)
			{
				ReplyBreakInputs = CONNECT_RESPONSE_BREAK_OTHERS_A;
			}
			else
			{
				ReplyBreakInputs = CONNECT_RESPONSE_BREAK_OTHERS_B;
			}
			if (ResponseMessage.IsEmpty())
			{
				ResponseMessage = LOCTEXT("ConnectionReplace", "Replace existing connections");
			}
			return FPinConnectionResponse(ReplyBreakInputs, ResponseMessage);
		}
	}

	return FPinConnectionResponse(CONNECT_RESPONSE_MAKE, ResponseMessage);
}

bool UMaterialGraphSchema::TryCreateConnection(UEdGraphPin* A, UEdGraphPin* B) const
{
	bool bModified = UEdGraphSchema::TryCreateConnection(A, B);

	if (bModified)
	{
		FMaterialEditorUtilities::UpdateMaterialAfterGraphChange(A->GetOwningNode()->GetGraph());
	}

	return bModified;
}

FLinearColor UMaterialGraphSchema::GetColorForConnectionType(UE::Shader::EValueType ConnectionType)
{
	using namespace UE::Shader;
	const UGraphEditorSettings* Settings = GetDefault<UGraphEditorSettings>();
	if (ConnectionType == EValueType::Any)
	{
		return Settings->WildcardPinTypeColor;
	}
	else if (ConnectionType == EValueType::Struct)
	{
		return Settings->StructPinTypeColor;
	}
	else if (ConnectionType == EValueType::Object)
	{
		return Settings->ObjectPinTypeColor;
	}
	else
	{
		const FValueTypeDescription TypeDesc = GetValueTypeDescription(ConnectionType);
		if (TypeDesc.ComponentType == EValueComponentType::Float ||
			TypeDesc.ComponentType == EValueComponentType::Numeric)
		{
			if (TypeDesc.NumComponents == 1)
			{
				return Settings->FloatPinTypeColor;
			}
			else
			{
				return Settings->VectorPinTypeColor;
			}
		}
		else if (TypeDesc.ComponentType == EValueComponentType::Double)
		{
			// Tentatively use these two color pin settings for LWC inputs to make them stand out better in the graph, since LWC math is expensive.
			// We could in the future add unique color pin settings for these.
			if (TypeDesc.NumComponents == 1)
			{
				return Settings->SoftObjectPinTypeColor;	// Default cyan
			}
			else
			{
				return Settings->SoftClassPinTypeColor;		// Default pink
			}
		}
		else if (TypeDesc.ComponentType == EValueComponentType::Bool)
		{
			return Settings->BooleanPinTypeColor;
		}
		else if (TypeDesc.ComponentType == EValueComponentType::Int)
		{
			return Settings->IntPinTypeColor;
		}
	}

	return Settings->DefaultPinTypeColor;
}

FLinearColor UMaterialGraphSchema::GetPinTypeColor(const FEdGraphPinType& PinType) const
{
	if (PinType.PinCategory == PC_Mask)
	{
		if (PinType.PinSubCategory == PSC_Red)
		{
			return FLinearColor::Red;
		}
		else if (PinType.PinSubCategory == PSC_Green)
		{
			return FLinearColor::Green;
		}
		else if (PinType.PinSubCategory == PSC_Blue)
		{
			return FLinearColor::Blue;
		}
		else if (PinType.PinSubCategory == PSC_Alpha)
		{
			return AlphaPinColor;
		}
	}
	else if (PinType.PinCategory == PC_Required)
	{
		return ActivePinColor;
	}
	else if (PinType.PinCategory == PC_Optional)
	{
		return InactivePinColor;
	}

	return ActivePinColor;
}

FLinearColor UMaterialGraphSchema::GetPinColor(const UEdGraphPin* InPin) const
{
	if (InPin)
	{
		if (const UMaterialShaderValueTypeObject* ValueTypeObj = Cast<UMaterialShaderValueTypeObject>(InPin->PinType.PinSubCategoryObject.Get()))
		{
			return GetColorForConnectionType(ValueTypeObj->ValueType);
		}
		return GetPinTypeColor(InPin->PinType);
	}
	return FLinearColor::Black;
}

void UMaterialGraphSchema::BreakNodeLinks(UEdGraphNode& TargetNode) const
{
	bool bHasLinksToBreak = false;
	for (auto PinIt = TargetNode.Pins.CreateConstIterator(); PinIt; ++PinIt)
	{
		UEdGraphPin* Pin = *PinIt;
		for (auto LinkIt = Pin->LinkedTo.CreateConstIterator(); LinkIt; ++LinkIt)
		{
			if (*LinkIt)
			{
				bHasLinksToBreak = true;
			}
		}
	}

	Super::BreakNodeLinks(TargetNode);

	if (bHasLinksToBreak)
	{
		FMaterialEditorUtilities::UpdateMaterialAfterGraphChange(TargetNode.GetGraph());
	}
}

void UMaterialGraphSchema::BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotifcation) const
{
	const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "GraphEd_BreakPinLinks", "Break Pin Links") );

	bool bHasLinksToBreak = false;
	for (auto LinkIt = TargetPin.LinkedTo.CreateConstIterator(); LinkIt; ++LinkIt)
	{
		if (*LinkIt)
		{
			bHasLinksToBreak = true;
		}
	}

	Super::BreakPinLinks(TargetPin, bSendsNodeNotifcation);

	// if this would notify the node then we need to re-compile material
	if (bSendsNodeNotifcation && bHasLinksToBreak)
	{
		FMaterialEditorUtilities::UpdateMaterialAfterGraphChange(TargetPin.GetOwningNode()->GetGraph());
	}
}

void UMaterialGraphSchema::BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const
{
	const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "GraphEd_BreakSinglePinLink", "Break Pin Link") );

	bool bHasLinkToBreak = false;
	for (auto LinkIt = SourcePin->LinkedTo.CreateConstIterator(); LinkIt; ++LinkIt)
	{
		if (*LinkIt == TargetPin)
		{
			bHasLinkToBreak = true;
		}
	}

	Super::BreakSinglePinLink(SourcePin, TargetPin);

	if (bHasLinkToBreak)
	{
		FMaterialEditorUtilities::UpdateMaterialAfterGraphChange(SourcePin->GetOwningNode()->GetGraph());
	}
}

bool UMaterialGraphSchema::CanEncapuslateNode(UEdGraphNode const& TestNode) const
{
	if (TestNode.IsA(UMaterialGraphNode_Comment::StaticClass()))
	{
		return true;
	}

	// Disallow output nodes from encapsulation, everything else (including parameters) is fair game for materials.
	const UMaterialGraphNode* MaterialGraphNode = Cast<UMaterialGraphNode>(&TestNode);

	return  MaterialGraphNode && MaterialGraphNode->MaterialExpression 
		&& !MaterialGraphNode->MaterialExpression->IsA(UMaterialExpressionFunctionOutput::StaticClass())
		&& !MaterialGraphNode->MaterialExpression->IsA(UMaterialExpressionPinBase::StaticClass())
		&& !TestNode.IsA(UMaterialGraphNode_Root::StaticClass());
}

void UMaterialGraphSchema::DroppedAssetsOnGraph(const TArray<struct FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const
{
	UMaterialGraph* MaterialGraph = CastChecked<UMaterialGraph>(Graph);
	const int32 LocOffsetBetweenNodes = 32;

	FVector2f ExpressionPosition = GraphPosition;

	TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = FMaterialEditorUtilities::MakeMaterialAssetReferenceFilter(Graph);
	for (const FAssetData& AssetData : Assets)
	{
		if (AssetReferenceFilter.IsValid() && !AssetReferenceFilter->PassesFilter(AssetData))
		{
			continue;
		}

		bool bAddedNode = false;
		UObject* Asset = AssetData.GetAsset();
		UClass* MaterialExpressionClass = Cast<UClass>(Asset);
		UMaterialFunctionInterface* Func = Cast<UMaterialFunctionInterface>(Asset);
		UTexture* Tex = Cast<UTexture>(Asset);
		USparseVolumeTexture* SparseVolumeTexture = Cast<USparseVolumeTexture>(Asset);
		UMaterialParameterCollection* ParameterCollection = Cast<UMaterialParameterCollection>(Asset);
		
		if (MaterialExpressionClass && MaterialExpressionClass->IsChildOf(UMaterialExpression::StaticClass()))
		{
			FMaterialEditorUtilities::CreateNewMaterialExpression(Graph, MaterialExpressionClass, ExpressionPosition, true, true);
			bAddedNode = true;
		}
		else if ( Func )
		{
			UMaterialExpressionMaterialFunctionCall* FunctionNode = CastChecked<UMaterialExpressionMaterialFunctionCall>(
				FMaterialEditorUtilities::CreateNewMaterialExpression(Graph, UMaterialExpressionMaterialFunctionCall::StaticClass(), ExpressionPosition, true, false));

			if (!FunctionNode->MaterialFunction)
			{
				if(FunctionNode->SetMaterialFunction(Func))
				{
					FunctionNode->PostEditChange();
					FMaterialEditorUtilities::UpdateSearchResults(Graph);
				}
				else
				{
					FMaterialEditorUtilities::AddToSelection(Graph, FunctionNode);
					FMaterialEditorUtilities::DeleteSelectedNodes(Graph);

					continue;
				}
			}

			bAddedNode = true;
		}
		else if ( Tex )
		{
			UMaterialExpressionTextureSample* TextureSampleNode = CastChecked<UMaterialExpressionTextureSample>(
				FMaterialEditorUtilities::CreateNewMaterialExpression(Graph, UMaterialExpressionTextureSample::StaticClass(), ExpressionPosition, true, true) );
			TextureSampleNode->Texture = Tex;
			TextureSampleNode->AutoSetSampleType();

			FMaterialEditorUtilities::ForceRefreshExpressionPreviews(Graph);

			bAddedNode = true;
		}
		else if ( SparseVolumeTexture )
		{
			UMaterialExpressionSparseVolumeTextureSample* SparseVolumeTextureSampleNode = CastChecked<UMaterialExpressionSparseVolumeTextureSample>(
				FMaterialEditorUtilities::CreateNewMaterialExpression(Graph, UMaterialExpressionSparseVolumeTextureSample::StaticClass(), ExpressionPosition, true, true));
			SparseVolumeTextureSampleNode->SparseVolumeTexture = SparseVolumeTexture;

			FMaterialEditorUtilities::ForceRefreshExpressionPreviews(Graph);

			bAddedNode = true;
		}
		else if ( ParameterCollection )
		{
			UMaterialExpressionCollectionParameter* CollectionParameterNode = CastChecked<UMaterialExpressionCollectionParameter>(
				FMaterialEditorUtilities::CreateNewMaterialExpression(Graph, UMaterialExpressionCollectionParameter::StaticClass(), ExpressionPosition, true, true) );
			CollectionParameterNode->Collection = ParameterCollection;

			FMaterialEditorUtilities::ForceRefreshExpressionPreviews(Graph);

			bAddedNode = true;
		}

		if ( bAddedNode )
		{
			ExpressionPosition.X += LocOffsetBetweenNodes;
			ExpressionPosition.Y += LocOffsetBetweenNodes;
		}
	}
}

void UMaterialGraphSchema::DroppedAssetsOnNode(const TArray<struct FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraphNode* Node) const
{
	const UMaterialGraphNode* MaterialGraphNode = Cast<UMaterialGraphNode>(Node);
	if (!MaterialGraphNode || !MaterialGraphNode->MaterialExpression)
	{
		return;
	}
	UMaterialExpression* Expression = MaterialGraphNode->MaterialExpression;

	TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = FMaterialEditorUtilities::MakeMaterialAssetReferenceFilter(MaterialGraphNode->GetGraph());

	// Drop a texture onto a TextureSample node -> replace texture
	if (UMaterialExpressionTextureSample* TextureSampleNode = Cast<UMaterialExpressionTextureSample>(Expression))
	{
		if (Assets.Num() < 1 || (AssetReferenceFilter.IsValid() && !AssetReferenceFilter->PassesFilter(Assets[0])))
		{
			return;
		}

		if (UTexture* Tex = Cast<UTexture>(Assets[0].GetAsset()))
		{
			const FScopedTransaction Transaction(LOCTEXT("MaterialEditorReplaceTexture", "Material Editor: Replace texture"));
			TextureSampleNode->Modify();
			TextureSampleNode->Texture = Tex;
			TextureSampleNode->AutoSetSampleType();

			FMaterialEditorUtilities::ForceRefreshExpressionPreviews(Node->GetGraph());
		}
	}
}

void UMaterialGraphSchema::UpdateMaterialOnDefaultValueChanged(const UEdGraph* Graph) const
{
	FMaterialEditorUtilities::UpdateMaterialAfterGraphChange(Graph);
}

void UMaterialGraphSchema::MarkMaterialDirty(const UEdGraph* Graph) const
{
	FMaterialEditorUtilities::MarkMaterialDirty(Graph);
}

void UMaterialGraphSchema::UpdateDetailView(const UEdGraph* Graph) const
{
	FMaterialEditorUtilities::UpdateDetailView(Graph);
}

int32 UMaterialGraphSchema::GetNodeSelectionCount(const UEdGraph* Graph) const
{
	return FMaterialEditorUtilities::GetNumberOfSelectedNodes(Graph);
}

TSharedPtr<FEdGraphSchemaAction> UMaterialGraphSchema::GetCreateCommentAction() const
{
	return TSharedPtr<FEdGraphSchemaAction>(static_cast<FEdGraphSchemaAction*>(new FMaterialGraphSchemaAction_NewComment));
}

void UMaterialGraphSchema::GetMaterialFunctionActions(FGraphActionMenuBuilder& ActionMenuBuilder) const
{
	// Get type of dragged pin
	EMaterialValueType FromPinType{};
	if (ActionMenuBuilder.FromPin)
	{
		FromPinType = GetMaterialIOValueType(ActionMenuBuilder.FromPin);
	}

	// Load the asset registry module
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetTools& AssetTools = IAssetTools::Get();

	// Collect a full list of assets with the specified class
	TArray<FAssetData> AssetDataList;
	AssetRegistryModule.Get().GetAssetsByClass(UMaterialFunction::StaticClass()->GetClassPathName(), AssetDataList);

	for (const FAssetData& AssetData : AssetDataList)
	{
		const bool bExposeToLibrary = AssetData.GetTagValueRef<bool>("bExposeToLibrary");

		// If this was a function that was selected to be exposed to the library
		if ( bExposeToLibrary )
		{
			if(AssetData.IsAssetLoaded())
			{
				if(AssetData.GetAsset()->GetOutermost() == GetTransientPackage())
				{
					continue;
				}
			}

			if (!AssetTools.IsAssetVisible(AssetData))
			{
				continue;
			}

			if (!ActionMenuBuilder.FromPin || HasCompatibleConnection(AssetData, FromPinType, ActionMenuBuilder.FromPin->Direction))
			{
				// Gather the relevant information from the asset data
				const FString FunctionPathName = AssetData.GetObjectPathString();
				TArray<FString> LibraryCategories;
				{
					const FString LibraryCategoriesString = AssetData.GetTagValueRef<FString>("LibraryCategories");
					if ( !LibraryCategoriesString.IsEmpty() )
					{
						if (FArrayProperty* LibraryCategoriesProperty = FindFieldChecked<FArrayProperty>(UMaterialFunction::StaticClass(), TEXT("LibraryCategories")))
						{
							uint8* DestAddr = (uint8*)(&LibraryCategories);
							LibraryCategoriesProperty->ImportText_Direct(*LibraryCategoriesString, DestAddr, NULL, PPF_None, GWarn);
						}
					}
				}
				TArray<FText> LibraryCategoriesText;
				{
					const FString LibraryCategoriesString = AssetData.GetTagValueRef<FString>("LibraryCategoriesText");
					if ( !LibraryCategoriesString.IsEmpty() )
					{
						FArrayProperty* LibraryCategoriesProperty = FindFieldChecked<FArrayProperty>(UMaterialFunction::StaticClass(), GET_MEMBER_NAME_CHECKED(UMaterialFunction, LibraryCategoriesText));
						uint8* DestAddr = (uint8*)(&LibraryCategoriesText);
						LibraryCategoriesProperty->ImportText_Direct(*LibraryCategoriesString, DestAddr, NULL, PPF_None, GWarn);
					}

					for (const FString& Category : LibraryCategories)
					{
						if (!LibraryCategoriesText.ContainsByPredicate(
							[&Category](const FText& Text)
							{
								return Text.ToString() == Category;
							}
							))
						{
							LibraryCategoriesText.Add(FText::FromString(Category));
						}
					}

					if ( LibraryCategoriesText.Num() == 0 )
					{
						LibraryCategoriesText.Add( LOCTEXT("UncategorizedMaterialFunction", "Uncategorized") );
					}

					// When Substrate is disabled, skip all material function related to Substrate
					// SUBSTRATE_TODO: remove this when Substrate becomes the only shading path
					bool bSkipMaterialFunction = false;
					for (const FText& Category : LibraryCategoriesText)
					{
						if (Category.ToString().Contains("Substrate"))
						{
							bSkipMaterialFunction = !Substrate::IsSubstrateEnabled();
							break;
						}
					}
					if (bSkipMaterialFunction)
					{
						continue;
					}
				}

				FString FunctionName = FunctionPathName;

				const FString UserExposedCaption = AssetData.GetTagValueRef<FString>("UserExposedCaption");
				if (!UserExposedCaption.IsEmpty())
				{
					// If the UI user exposed name name is not empty, use it directly
					FunctionName = UserExposedCaption;
				}
				else
				{
					// Extract the object name from the path
					int32 PeriodIndex = FunctionPathName.Find(TEXT("."), ESearchCase::CaseSensitive, ESearchDir::FromEnd);

					if (PeriodIndex != INDEX_NONE)
					{
						FunctionName = FunctionPathName.Right(FunctionPathName.Len() - PeriodIndex - 1);
					}
				}
				// Remove leading slash
				const FText AssetPath = FText::FromString(AssetData.PackagePath.ToString().RightChop(1));
				const FText Description = AssetData.GetTagValueRef<FText>("Description");
				const FText Tooltip = FText::Format(LOCTEXT("MaterialFunctionActionTooltip", "Location: {0}\n{1}"), AssetPath, Description);
				// For each category the function should belong to...
				for (const FText& CategoryName : LibraryCategoriesText)
				{
					TSharedPtr<FMaterialGraphSchemaAction_NewFunctionCall> NewFunctionAction(new FMaterialGraphSchemaAction_NewFunctionCall(
						CategoryName,
						FText::FromString(FunctionName),
						Tooltip, 0));
					ActionMenuBuilder.AddAction(NewFunctionAction);
					NewFunctionAction->FunctionPath = FunctionPathName;
				}
			}
		}
	}
}

void UMaterialGraphSchema::GetCompositeAction(FGraphActionMenuBuilder& ActionMenuBuilder, const UEdGraph* CurrentGraph) const
{
	if (!ActionMenuBuilder.FromPin)
	{
		const bool bIsManyNodesSelected = CurrentGraph ? (FMaterialEditorUtilities::GetNumberOfSelectedNodes(CurrentGraph) > 0) : false;
		const FText CompositeDesc = LOCTEXT("CompositeDesc", "New Composite");
		const FText CompositeToolTip = LOCTEXT("CompositeToolTip", "Create a composite node that holds a subgraph.");
		TSharedPtr<FMaterialGraphSchemaAction_NewComposite> NewAction(new FMaterialGraphSchemaAction_NewComposite(FText::GetEmpty(), CompositeDesc, CompositeToolTip, 0));
		ActionMenuBuilder.AddAction(NewAction);
	}
}

void UMaterialGraphSchema::GetCommentAction(FGraphActionMenuBuilder& ActionMenuBuilder, const UEdGraph* CurrentGraph) const
{
	if (!ActionMenuBuilder.FromPin)
	{
		const bool bIsManyNodesSelected = CurrentGraph ? (FMaterialEditorUtilities::GetNumberOfSelectedNodes(CurrentGraph) > 0) : false;
		const FText CommentDesc = LOCTEXT("CommentDesc", "New Comment");
		const FText MultiCommentDesc = LOCTEXT("MultiCommentDesc", "Create Comment from Selection");
		const FText CommentToolTip = LOCTEXT("CommentToolTip", "Creates a comment.");
		const FText MenuDescription = bIsManyNodesSelected ? MultiCommentDesc : CommentDesc;
		TSharedPtr<FMaterialGraphSchemaAction_NewComment> NewAction(new FMaterialGraphSchemaAction_NewComment(FText::GetEmpty(), MenuDescription, CommentToolTip, 0));
		ActionMenuBuilder.AddAction( NewAction );
	}
}

void UMaterialGraphSchema::GetNamedRerouteActionsForGraph(FGraphActionListBuilderBase& OutAllActions, const UEdGraph* CurrentGraph)
{
	if (CurrentGraph)
	{
		for (UEdGraphNode* GraphNode : CurrentGraph->Nodes)
		{
			if (auto* MaterialGraphNode = Cast<UMaterialGraphNode>(GraphNode))
			{
				if (auto Declaration = Cast<UMaterialExpressionNamedRerouteDeclaration>(MaterialGraphNode->MaterialExpression))
				{
					static const FText Category = LOCTEXT("NamedRerouteCategory", "Named Reroutes");
					const FText Name = FText::FromString(Declaration->Name.ToString());
					const FText Tooltip = FText::Format(LOCTEXT("NamedRerouteTooltip", "Add a usage of {0} here"), Name);
					TSharedPtr<FMaterialGraphSchemaAction_NewNamedRerouteUsage> NewAction(new FMaterialGraphSchemaAction_NewNamedRerouteUsage(Category, Name, Tooltip, 1 /* We want named reroutes to be on top */));
					NewAction->Declaration = Declaration;
					OutAllActions.AddAction(NewAction);
				}
			}
		}
	}
}

void UMaterialGraphSchema::GetNamedRerouteActions(FGraphActionMenuBuilder& ActionMenuBuilder, const UEdGraph* CurrentGraph) const
{
	GetNamedRerouteActionsForGraph(ActionMenuBuilder, CurrentGraph);
}

bool UMaterialGraphSchema::HasCompatibleConnection(const FAssetData& FunctionAssetData, EMaterialValueType TestType, EEdGraphPinDirection TestDirection) const
{
	if (TestType != 0)
	{
		auto CombinedInputTypes = static_cast<EMaterialValueType>(FunctionAssetData.GetTagValueRef<uint64>(GET_MEMBER_NAME_CHECKED(UMaterialFunctionInterface, CombinedInputTypes)));
		auto CombinedOutputTypes = static_cast<EMaterialValueType>(FunctionAssetData.GetTagValueRef<uint64>(GET_MEMBER_NAME_CHECKED(UMaterialFunctionInterface, CombinedOutputTypes)));

		if (CombinedOutputTypes == 0)
		{
			// Need to load function to build combined output types
			UMaterialFunctionInterface* MaterialFunction = Cast<UMaterialFunctionInterface>(FunctionAssetData.GetAsset());
			if (MaterialFunction != nullptr)
			{
				CombinedInputTypes = static_cast<EMaterialValueType>(MaterialFunction->CombinedInputTypes);
				CombinedOutputTypes = static_cast<EMaterialValueType>(MaterialFunction->CombinedOutputTypes);
			}
		}

		if (TestDirection == EGPD_Output)
		{
			if (CanConnectMaterialValueTypes(CombinedInputTypes, TestType))
			{
				return true;
			}
		}
		else
		{
			if (CanConnectMaterialValueTypes(TestType, CombinedOutputTypes))
			{
				return true;
			}
		}
	}

	return false;
}

bool UMaterialGraphSchema::IsCacheVisualizationOutOfDate(int32 InVisualizationCacheID) const
{
	return CurrentCacheRefreshID != InVisualizationCacheID;
}

int32 UMaterialGraphSchema::GetCurrentVisualizationCacheID() const
{
	return CurrentCacheRefreshID;
}

void UMaterialGraphSchema::ForceVisualizationCacheClear() const
{
	++CurrentCacheRefreshID;
}

void UMaterialGraphSchema::OnPinConnectionDoubleCicked(UEdGraphPin* PinA, UEdGraphPin* PinB, const FVector2f& GraphPosition) const
{
	const FScopedTransaction Transaction(LOCTEXT("CreateRerouteNodeOnWire", "Create Reroute Node"));

	//@TODO: This constant is duplicated from inside of SGraphNodeKnot
	const FVector2f NodeSpacerSize(42.0f, 24.0f);
	const FVector2f KnotTopLeft = GraphPosition - (NodeSpacerSize * 0.5f);

	// Create a new knot
	UEdGraph* ParentGraph = PinA->GetOwningNode()->GetGraph();

	if (UMaterialExpression* Expression = FMaterialEditorUtilities::CreateNewMaterialExpression(ParentGraph, UMaterialExpressionReroute::StaticClass(), FDeprecateSlateVector2D(KnotTopLeft), true, true))
	{
		// Move the connections across (only notifying the knot, as the other two didn't really change)
		PinA->BreakLinkTo(PinB);
		PinA->MakeLinkTo((PinA->Direction == EGPD_Output) ? CastChecked<UMaterialGraphNode_Knot>(Expression->GraphNode)->GetInputPin() : CastChecked<UMaterialGraphNode_Knot>(Expression->GraphNode)->GetOutputPin());
		PinB->MakeLinkTo((PinB->Direction == EGPD_Output) ? CastChecked<UMaterialGraphNode_Knot>(Expression->GraphNode)->GetInputPin() : CastChecked<UMaterialGraphNode_Knot>(Expression->GraphNode)->GetOutputPin());
		FMaterialEditorUtilities::UpdateMaterialAfterGraphChange(ParentGraph);
	}
}

bool UMaterialGraphSchema::SafeDeleteNodeFromGraph(UEdGraph* Graph, UEdGraphNode* NodeToDelete) const
{
	if (NodeToDelete == nullptr || Graph == nullptr || NodeToDelete->GetGraph() != Graph)
	{
		return false;
	}

	TArray<UEdGraphNode*> NodesToDelete;
	NodesToDelete.Add(NodeToDelete);
	FMaterialEditorUtilities::DeleteNodes(Graph, NodesToDelete);
	return true;
}

void UMaterialGraphSchema::GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const
{
	OutTooltipText.Reset();
	OutOkIcon = false;

	TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = FMaterialEditorUtilities::MakeMaterialAssetReferenceFilter(HoverGraph);
	for (const FAssetData& AssetData : Assets)
	{
		UObject* Asset = AssetData.GetAsset();
		UClass* MaterialExpressionClass = Cast<UClass>(Asset);
		UMaterialFunctionInterface* Func = Cast<UMaterialFunctionInterface>(Asset);
		UTexture* Tex = Cast<UTexture>(Asset);
		USparseVolumeTexture* SparseVolumeTexture = Cast<USparseVolumeTexture>(Asset);
		UMaterialParameterCollection* ParameterCollection = Cast<UMaterialParameterCollection>(Asset);

		if ((MaterialExpressionClass != nullptr && MaterialExpressionClass->IsChildOf(UMaterialExpression::StaticClass())) ||
			Func != nullptr ||
			Tex != nullptr ||
			SparseVolumeTexture != nullptr ||
			ParameterCollection != nullptr)
		{
			FText FailureReason;
			if (AssetReferenceFilter.IsValid() && !AssetReferenceFilter->PassesFilter(AssetData, &FailureReason))
			{
				OutTooltipText = FailureReason.ToString();
			}
			else
			{
				OutOkIcon = true;
			}
		}
	}

	if (OutOkIcon)
	{
		OutTooltipText.Reset();
	}
}

void UMaterialGraphSchema::GetAssetsNodeHoverMessage(const TArray<struct FAssetData>& Assets, const UEdGraphNode* HoverNode, FString& OutTooltipText, bool& OutOkIcon) const
{
	OutTooltipText = FString();
	OutOkIcon = false;

	const UMaterialGraphNode* MaterialGraphNode = Cast<UMaterialGraphNode>(HoverNode);
	if (!MaterialGraphNode || !MaterialGraphNode->MaterialExpression)
	{
		return;
	}
	UMaterialExpression* Expression = MaterialGraphNode->MaterialExpression;

	if (UMaterialExpressionTextureSample* TextureSampleNode = Cast<UMaterialExpressionTextureSample>(Expression))
	{
		if (Assets.Num() < 1)
		{
			return;
		}

		const FAssetData& AssetData = Assets[0];
		if (UTexture* Tex = Cast<UTexture>(AssetData.GetAsset()))
		{
			TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = FMaterialEditorUtilities::MakeMaterialAssetReferenceFilter(MaterialGraphNode->GetGraph());

			FText FailureReason;
			if (AssetReferenceFilter.IsValid() && !AssetReferenceFilter->PassesFilter(AssetData, &FailureReason))
			{
				OutTooltipText = FailureReason.ToString();
			}
			else
			{
				OutOkIcon = true;
			}
		}
	}
}

#if WITH_EDITORONLY_DATA
//////////////////////////////////////////////////////////////////////////

FGraphSchemaSearchWeightModifiers UMaterialGraphSchema::GetSearchWeightModifiers() const
{
	const UMaterialEditorSettings* MaterialSettings = GetDefault<UMaterialEditorSettings>();
	FGraphSchemaSearchWeightModifiers Modifiers;
	Modifiers.NodeTitleWeight = MaterialSettings->ContextMenuNodeTitleWeight;
	Modifiers.KeywordWeight = MaterialSettings->ContextMenuKeywordWeight;
	Modifiers.DescriptionWeight = MaterialSettings->ContextMenuDescriptionWeight;
	Modifiers.CategoryWeight = MaterialSettings->ContextMenuDescriptionWeight;
	Modifiers.WholeMatchLocalizedWeightMultiplier = MaterialSettings->ContextMenuWholeMatchLocalizedWeightMultiplier;
	Modifiers.WholeMatchWeightMultiplier = MaterialSettings->ContextMenuWholeMatchWeightMultiplier;
	Modifiers.StartsWithBonusWeightMultiplier = MaterialSettings->ContextMenuStartsWithBonusWeightMultiplier;
	Modifiers.PercentageMatchWeightMultiplier = MaterialSettings->ContextMenuPercentageMatchWeightMultiplier;
	Modifiers.ShorterMatchWeight = MaterialSettings->ContextMenuShorterMatchWeight;
	return Modifiers;
}

float UMaterialGraphSchema::GetActionFilteredWeight(const FEdGraphSchemaAction& InCurrentAction, const TArray<FString>& InFilterTerms, const TArray<FString>& InSanitizedFilterTerms, const TArray<UEdGraphPin*>& DraggedFromPins) const
{
	// The overall 'weight'
	float TotalWeight = 0.0f;

	// Setup an array of arrays so we can do a weighted search			
	TArray<FGraphSchemaSearchTextWeightInfo> WeightedArrayList;
	FGraphSchemaSearchTextDebugInfo DebugInfo;

	FGraphSchemaSearchWeightModifiers WeightModifiers = GetSearchWeightModifiers();
	int32 NonLocalizedFirstIndex = CollectSearchTextWeightInfo(InCurrentAction, WeightModifiers, WeightedArrayList, &DebugInfo);

	const UMaterialEditorSettings* MaterialSettings = GetDefault<UMaterialEditorSettings>();

	// Now iterate through all the filter terms and calculate a 'weight' using the values and multipliers
	for (int32 FilterIndex = 0; FilterIndex < InFilterTerms.Num(); ++FilterIndex)
	{
		const FString& EachTerm = InFilterTerms[FilterIndex];
		const FString& EachTermSanitized = InSanitizedFilterTerms[FilterIndex];
		// Now check the weighted lists
		for (int32 iFindCount = 0; iFindCount < WeightedArrayList.Num(); iFindCount++)
		{
			float WeightPerList = 0.0f;
			const TArray<FString>& WordArray = *WeightedArrayList[iFindCount].Array;
			float ArrayWeight = WeightedArrayList[iFindCount].WeightModifier;
			int32 WholeMatchCount = 0;
			float WholeMatchMultiplier = (iFindCount < NonLocalizedFirstIndex) ? MaterialSettings->ContextMenuWholeMatchLocalizedWeightMultiplier : MaterialSettings->ContextMenuWholeMatchWeightMultiplier;

			// Count of how many words in this array contain a search term that the user has typed in
			int32 WordMatchCount = 0;
			// The number of characters in the best matching word
			int32 BestMatchCharLength = 0;

			for (int32 iEachWord = 0; iEachWord < WordArray.Num(); iEachWord++)
			{
				float WeightPerWord = 0.0f;

				// If a word contains the search phrase that the user has typed in, then give it weight					
				if (WordArray[iEachWord].Contains(EachTermSanitized, ESearchCase::CaseSensitive) || WordArray[iEachWord].Contains(EachTerm, ESearchCase::CaseSensitive))
				{
					++WordMatchCount;
					WeightPerWord += ArrayWeight * WholeMatchMultiplier;

					// If the word starts with the search term, give it extra boost of weight
					if (WordArray[iEachWord].StartsWith(EachTermSanitized, ESearchCase::CaseSensitive) || WordArray[iEachWord].StartsWith(EachTerm, ESearchCase::CaseSensitive))
					{
						WeightPerWord += ArrayWeight * MaterialSettings->ContextMenuStartsWithBonusWeightMultiplier;
					}
				}

				if (WeightPerWord > WeightPerList)
				{
					// Use the best word match weight, we don't want to count similar words more than one
					WeightPerList = WeightPerWord;
					BestMatchCharLength = WordArray[iEachWord].Len();
				}
			}

			if (BestMatchCharLength > 0 && WeightPerList > 0)
			{
				// Higher number of matching words contributes to higher weight
				float PercentMatch = (float)WordMatchCount / (float)WordArray.Num();
				float PercentMatchWeight = (WeightPerList * PercentMatch * MaterialSettings->ContextMenuPercentageMatchWeightMultiplier);
				WeightPerList += PercentMatchWeight;
				DebugInfo.PercentMatchWeight += PercentMatchWeight;
				DebugInfo.PercentMatch += PercentMatch;

				// The shorter the best matched word, the larger bonus it gets
				float ShorterMatchFactor = (float)EachTerm.Len() / (float)BestMatchCharLength;
				float ShorterMatchWeight = ShorterMatchFactor * MaterialSettings->ContextMenuShorterMatchWeight;
				WeightPerList += ShorterMatchWeight;
				DebugInfo.ShorterMatchWeight += ShorterMatchWeight;
			}

			if (WeightedArrayList[iFindCount].DebugWeight)
			{
				*WeightedArrayList[iFindCount].DebugWeight += WeightPerList;
			}

			TotalWeight += WeightPerList;
		}

		DebugInfo.TotalWeight = TotalWeight;
		PrintSearchTextDebugInfo(InFilterTerms, InCurrentAction, &DebugInfo);
	}

	return TotalWeight;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
float UMaterialGraphSchema::GetActionFilteredWeight(const FGraphActionListBuilderBase::ActionGroup& InCurrentAction, const TArray<FString>& InFilterTerms, const TArray<FString>& InSanitizedFilterTerms, const TArray<UEdGraphPin*>& DraggedFromPins) const
{
	int32 Action = 0;
	if (InCurrentAction.Actions[Action].IsValid() == true)
	{
		return GetActionFilteredWeight(*InCurrentAction.Actions[Action], InFilterTerms, InSanitizedFilterTerms, DraggedFromPins);
	}
	return 0.f;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#endif // WITH_EDITORONLY_DATA

#undef LOCTEXT_NAMESPACE
