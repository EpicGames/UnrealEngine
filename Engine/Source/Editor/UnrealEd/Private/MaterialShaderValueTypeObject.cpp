// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialGraph/MaterialShaderValueTypeObject.h"
#include "Subsystems/MaterialShaderValueTypeSubsystem.h"
#include "Editor.h"

UMaterialShaderValueTypeObject* UMaterialShaderValueTypeObject::Get(UE::Shader::EValueType InType)
{
	check(IsInGameThread());

	if (InType == UE::Shader::EValueType::Void)
	{
		return nullptr;
	}

	if (!GEditor)
	{
		return nullptr;
	}

	UMaterialShaderValueTypeSubsystem* Subsystem = GEditor->GetEditorSubsystem<UMaterialShaderValueTypeSubsystem>();
	if (!Subsystem)
	{
		return nullptr;
	}

	return Subsystem->GetOrCreate(InType);
}
