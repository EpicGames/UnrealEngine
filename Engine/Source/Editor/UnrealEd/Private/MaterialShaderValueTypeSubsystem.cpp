// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/MaterialShaderValueTypeSubsystem.h"
#include "MaterialGraph/MaterialShaderValueTypeObject.h"

UMaterialShaderValueTypeObject* UMaterialShaderValueTypeSubsystem::GetOrCreate(UE::Shader::EValueType InType)
{
	TObjectPtr<UMaterialShaderValueTypeObject>& Slot = Flyweights.FindOrAdd(static_cast<uint8>(InType));
	if (!Slot)
	{
		Slot = NewObject<UMaterialShaderValueTypeObject>(this);
		Slot->ValueType = InType;
	}
	return Slot;
}
