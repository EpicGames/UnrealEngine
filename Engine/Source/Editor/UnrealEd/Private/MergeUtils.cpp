// Copyright Epic Games, Inc. All Rights Reserved.

#include "MergeUtils.h"

#include "DiffUtils.h"
#include "ISourceControlModule.h"
#include "ISourceControlRevision.h"
#include "SDetailsDiff.h"
#include "SourceControlOperations.h"
#include "AsyncTreeDifferences.h"
#include "Algo/RandomShuffle.h"
#include "Editor.h"
#include "HAL/PlatformFileManager.h"
#include "Engine/Blueprint.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "UObject/Linker.h"
#include "AssetRegistry/AssetRegistryModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MergeUtils)

#define LOCTEXT_NAMESPACE "MergeUtils"

struct FScopedMergeResolveTransaction
{
	FScopedMergeResolveTransaction(UObject* InManagedObject, EMergeFlags InFlags)
		: ManagedObject(InManagedObject)
		, Flags(InFlags)
	{
		if (Flags & MF_HANDLE_SOURCE_CONTROL)
		{
			UndoHandler = NewObject<UUndoableResolveHandler>();
			UndoHandler->SetFlags(RF_Transactional);
			UndoHandler->SetManagedObject(InManagedObject);
			
			TransactionNum = GEditor->BeginTransaction(LOCTEXT("ResolveMerge", "ResolveAutoMerge"));
			ensure(UndoHandler->Modify());
			if (InManagedObject)
			{
				ensure(InManagedObject->Modify());
			}
		}
	}

	void Cancel()
	{
		bCanceled = true;
	}
	
	~FScopedMergeResolveTransaction()
	{
		if (Flags & MF_HANDLE_SOURCE_CONTROL)
		{
			if (!bCanceled)
			{
				UndoHandler->MarkResolved();
				GEditor->EndTransaction();
			}
			else if (ManagedObject.IsValid())
			{
				ManagedObject->GetPackage()->SetDirtyFlag(false);
				GEditor->CancelTransaction(TransactionNum);
			}
		}
	}

	TWeakObjectPtr<UObject> ManagedObject;
	UUndoableResolveHandler* UndoHandler = nullptr;
	EMergeFlags Flags;
	int TransactionNum = 0;
	bool bCanceled = false;
};


UPackage* MergeUtils::LoadPackageForMerge(const FString& SCFile, const FString& Revision, const UPackage* LocalPackage)
{
	return DiffUtils::LoadPackageForDiff(FPackagePath::FromLocalPath(LoadSCFileForMerge(SCFile, Revision)), LocalPackage->GetLoadedPath());
}

FString MergeUtils::LoadSCFileForMerge(const FString& SCFile, const FString& Revision)
{
	const FString FileWithRevision = SCFile + TEXT("#") + Revision;
	const TSharedRef<FDownloadFile, ESPMode::ThreadSafe> DownloadFileOperation = ISourceControlOperation::Create<FDownloadFile>(FPaths::DiffDir());
	ISourceControlModule::Get().GetProvider().Execute(DownloadFileOperation, FileWithRevision, EConcurrency::Synchronous);
	const FString DownloadPath = FPaths::ConvertRelativePathToFull(FPaths::DiffDir() / FPaths::GetCleanFilename(FileWithRevision));

	// move downloaded file to renamed path so it meets ue asset name requirements
	const FString Directory = FPaths::GetPath(DownloadPath);
	FString Filename = FPaths::GetCleanFilename(DownloadPath);
	Filename.ReplaceInline(TEXT(".uasset"), TEXT(""));
	Filename.ReplaceCharInline('#', '-');
	Filename.ReplaceCharInline('.', '-');
	Filename += TEXT("-");
	FString ResultPath = FPaths::CreateTempFilename(*Directory, *Filename, TEXT(".uasset"));
	checkf(FPaths::DirectoryExists(Directory), TEXT("Tried to move file to a directory that doesn't exist"));
	checkf(!FPaths::FileExists(ResultPath), TEXT("Tried to rename file to a name that's already taken"));
	if (ensure(FPlatformFileManager::Get().GetPlatformFile().MoveFile(*ResultPath, *DownloadPath)))
	{
		return ResultPath;
	}
	return {};
}

void UUndoableResolveHandler::SetManagedObject(UObject* Object)
{
	ManagedObject = Object;
	const UPackage* Package = ManagedObject->GetPackage();
	const FString Filepath = FPaths::ConvertRelativePathToFull(Package->GetLoadedPath().GetLocalFullPath());
	
	ISourceControlProvider& Provider = ISourceControlModule::Get().GetProvider();
	const FSourceControlStatePtr SourceControlState = Provider.GetState(Package, EStateCacheUsage::Use);
	const ISourceControlState::FResolveInfo ResolveInfo = SourceControlState->GetResolveInfo();
	BaseRevisionNumber = SourceControlState->GetResolveInfo().BaseRevision;
	if (const TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> CurrentRevision = SourceControlState->GetCurrentRevision())
	{
		CurrentRevisionNumber = FString::FromInt(CurrentRevision->GetRevisionNumber());
	}
	else
	{
		CurrentRevisionNumber = {};
	}
	CheckinIdentifier = SourceControlState->GetCheckInIdentifier();

	// save package and copy the package to a temp file so it can be reverted
	const FString BaseFilename = FPaths::GetBaseFilename(Filepath);
	const FString Directory = FPaths::ProjectSavedDir()/TEXT("Temp");
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	BackupFilepath = FPaths::CreateTempFilename(*Directory, *BaseFilename.Left(32));
	ensure(PlatformFile.CreateDirectoryTree(*Directory));
	ensure(PlatformFile.CopyFile(*BackupFilepath, *Filepath));
}

void UUndoableResolveHandler::MarkResolved()
{
	if (ManagedObject.IsValid())
	{
		const UPackage* Package = ManagedObject->GetPackage();
		const FString Filepath = FPaths::ConvertRelativePathToFull(Package->GetLoadedPath().GetLocalFullPath());
		
		ISourceControlProvider& Provider = ISourceControlModule::Get().GetProvider();
		Provider.Execute(ISourceControlOperation::Create<FResolve>(), TArray{Filepath}, EConcurrency::Synchronous);
		bShouldBeResolved = true;
	}
}

void UUndoableResolveHandler::PostEditUndo()
{
	if (bShouldBeResolved) // redo resolution
	{
		MarkResolved();
	}
	else if (ManagedObject.IsValid())// undo resolution
	{
		UPackage* Package = ManagedObject->GetPackage();
		const FString Filepath = FPaths::ConvertRelativePathToFull(Package->GetLoadedPath().GetLocalFullPath());
		
		if (BaseRevisionNumber.IsEmpty() || CurrentRevisionNumber.IsEmpty())
		{
			ensure(FPlatformFileManager::Get().GetPlatformFile().CopyFile(*Filepath, *BackupFilepath));
			return;
		}
		
		// to force the file to revert to it's pre-resolved state, we must revert, sync back to base revision,
		// apply the conflicting changes, then sync forward again.
		ISourceControlProvider& Provider = ISourceControlModule::Get().GetProvider();
		{
			const TSharedRef<FSync> SyncOperation = ISourceControlOperation::Create<FSync>();
			SyncOperation->SetRevision(BaseRevisionNumber);
			Provider.Execute(SyncOperation, Filepath, EConcurrency::Synchronous);
		}
		
		ResetLoaders(Package);
		Provider.Execute( ISourceControlOperation::Create<FRevert>(), Filepath, EConcurrency::Synchronous);

		{
			const TSharedRef<FCheckOut> CheckoutOperation = ISourceControlOperation::Create<FCheckOut>();
			Provider.Execute(CheckoutOperation, CheckinIdentifier, {Filepath}, EConcurrency::Synchronous);
		}

		ensure(FPlatformFileManager::Get().GetPlatformFile().CopyFile(*Filepath, *BackupFilepath));
		
		{
			const TSharedRef<FSync> SyncOperation = ISourceControlOperation::Create<FSync>();
			SyncOperation->SetRevision(CurrentRevisionNumber);
			Provider.Execute(SyncOperation, Filepath, EConcurrency::Synchronous);
		}

		Provider.Execute(ISourceControlOperation::Create<FUpdateStatus>(), {Filepath}, EConcurrency::Asynchronous);
	}
	UObject::PostEditUndo();
}

struct FBPReferenceFinder : public FArchiveUObject
{
	FBPReferenceFinder(const UObject* Obj, TArray<const UObject*>& InReferences)
		: FArchiveUObject()
		, References(InReferences)
		, OwningPackage(Obj->GetOutermost())
	{
		// Copying FPackageHarvester:
		SetIsPersistent(true);
		SetIsSaving(true);
		ArIsObjectReferenceCollector = true;
		ArShouldSkipBulkData = true;
		if (Obj->HasAnyFlags(RF_ClassDefaultObject))
		{
			Obj->GetClass()->SerializeDefaultObject(const_cast<UObject*>(Obj), *this);
		}
		else
		{
			const_cast<UObject*>(Obj)->Serialize(*this);
		}
	}

private:
	virtual FArchive& operator<<(UObject*& ObjRef) override
	{
		if (ObjRef != nullptr &&
			(!ObjRef->HasAnyFlags(RF_Transient) || ObjRef->IsNative()) &&
			!ObjRef->IsIn(GetTransientPackage()) &&
			ObjRef != OwningPackage)
		{
			// Set to null any pointer to an external asset
			References.Add(ObjRef);
		}

		return *this;
	}

	TArray<const UObject*>& References;
	UPackage* OwningPackage;
};

static TArray<FString> GatherExportPaths(const UObject* Object)
{
	TArray<FString> OutExports;
	
	// find everything roots references that is in package and put it into
	// OutExports, unless it is in the disallow list.. puts other references
	// into OutImports:
	
	// Object won't be modified. const_cast just simplifies code
	TArray PendingRefs{const_cast<UObject*>(Object)};

	TSet<UObject*> RefsProcessed;
	RefsProcessed.Append(PendingRefs);

	TArray<UObject*> ScratchRefs; // just keeping this allocation alive across iterations
	UPackage* Package = Object->GetPackage();

	while (PendingRefs.Num())
	{
		const UObject* Iter = PendingRefs.Pop();
		if (ensure(Iter->GetPackage() == Package))
		{
			const FString PathName = Iter->GetPathName(Package);
            if (ensure(Iter == FindObject<UObject>(Package, *PathName)))
            {
            	OutExports.Add(PathName);
            }
		}
		
		ScratchRefs.Add(Iter->GetClass());
		FBPReferenceFinder ReferencedObjects(Iter, (TArray<const UObject*>&)ScratchRefs);
		for (UObject* Obj : ScratchRefs)
		{
			if (RefsProcessed.Contains(Obj))
			{
				continue;
			}
			RefsProcessed.Add(Obj);

			if (!Obj->IsIn(Package))
			{
				// found import. ignore.
				continue;
			}

			PendingRefs.Add(Obj);
		}
		ScratchRefs.Reset();
	}
	
	OutExports.Sort();
	
	return OutExports;
}

enum class EObjectPointerType : uint8
{
	Nullptr,
	Import,
	Export,
};

static UObject* DuplicateForMerge(const UObject* Source, UObject* DestinationOuter)
{
	FObjectDuplicationParameters DuplicateParams(const_cast<UObject*>(Source), DestinationOuter);
	DuplicateParams.PortFlags |= PPF_DuplicateVerbatim;
	DuplicateParams.bSkipPostLoad = true;
	DuplicateParams.DestName = Source->GetFName();
	return StaticDuplicateObjectEx(DuplicateParams);
}

struct FPropertyInstance
{
	explicit FPropertyInstance() = default;
	FPropertyInstance(const void* InObject, const FProperty* InProperty)
		: Object(InObject), KeyProperty(InProperty), ValProperty(InProperty)
	{}
	FPropertyInstance(const void* InObject, const FProperty* InValProperty, const FProperty* InKeyProperty)
		: Object(InObject), KeyProperty(InKeyProperty), ValProperty(InValProperty)
	{}

	operator bool() const
	{
		return Object != nullptr && KeyProperty != nullptr && ValProperty != nullptr;
	}

	bool operator==(const FPropertyInstance& Other) const
	{
		if (!ValProperty || !Other.ValProperty)
		{
			return ValProperty == Other.ValProperty;
		}
		
		if (ValProperty == Other.ValProperty)
		{
			const void* DataA = ValProperty->ContainerPtrToValuePtr<void*>(Object);
			const void* DataB = Other.ValProperty->ContainerPtrToValuePtr<void*>(Other.Object);
			check(ValProperty->GetElementSize() == Other.ValProperty->GetElementSize());
			check(ValProperty->GetClass() == Other.ValProperty->GetClass());
			
			return ValProperty->Identical(DataA, DataB, PPF_DeepComparison);
		}
		return false;
	}

	void GetChildren(TArray<FPropertyInstance>& OutChildren) const
	{
		auto TryAddChild = [&OutChildren](const void* Data, const FProperty* NewValProperty, const FProperty* NewKeyProperty = nullptr)
		{
			// ignore transient properties
			if (!NewValProperty->HasAnyPropertyFlags(CPF_Transient | CPF_DuplicateTransient | CPF_NonPIEDuplicateTransient))
			{
				OutChildren.Emplace(Data, NewValProperty, NewKeyProperty ? NewKeyProperty : NewValProperty);
			}
		};
		
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(ValProperty))
		{
			// treat FSoftObjectPath as a leaf
			if (StructProperty->Struct != TBaseStructure<FSoftObjectPath>::Get())
			{
				const void* StructPtr = StructProperty->ContainerPtrToValuePtr<void*>(Object);
                for (TFieldIterator<FProperty> PropertyIt(StructProperty->Struct); PropertyIt; ++PropertyIt)
                {
                	TryAddChild(StructPtr, *PropertyIt);
                }
			}
		}
		else if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(ValProperty))
		{
			FScriptArrayHelper Helper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(Object));
			for (int32 Index = 0; Index < Helper.Num(); ++Index)
			{
				TryAddChild(Helper.GetElementPtr(Index), ArrayProperty->Inner);
			}
		}
		else if (const FSetProperty* SetProperty = CastField<FSetProperty>(ValProperty))
		{
			FScriptSetHelper Helper(SetProperty, SetProperty->ContainerPtrToValuePtr<void>(Object));
			for (FScriptSetHelper::FIterator It = Helper.CreateIterator(); It; ++It)
			{
				TryAddChild(Helper.GetElementPtr(It), SetProperty->ElementProp);
			}
		}
		else if (const FMapProperty* MapProperty = CastField<FMapProperty>(ValProperty))
		{
			FScriptMapHelper MapHelper(MapProperty, MapProperty->ContainerPtrToValuePtr<void>(Object));
			for (FScriptMapHelper::FIterator MapIt = MapHelper.CreateIterator(); MapIt; ++MapIt)
			{
				TryAddChild(MapHelper.GetPairPtr(MapIt), MapProperty->ValueProp, MapProperty->KeyProp);
			}
		}

		// UObject* is treated a leaf so we don't need to enter FObjectProperty
	}

	const void* Object = nullptr;
	const FProperty* KeyProperty = nullptr; // equal to ValProperty unless this is a map element
	const FProperty* ValProperty = nullptr;
};

// methods that make FPropertyInstance diffable
template<>
class TTreeDiffSpecification<FPropertyInstance>
{
public:

	bool AreValuesEqual(const FPropertyInstance& TreeNodeA, const FPropertyInstance& TreeNodeB, TArray<FPropertySoftPath>* OutDifferingProperties = nullptr) const
	{
		return TreeNodeA == TreeNodeB;
	}
	
	bool AreMatching(const FPropertyInstance& TreeNodeA, const FPropertyInstance& TreeNodeB, TArray<FPropertySoftPath>* OutDifferingProperties = nullptr) const
	{
		if (TreeNodeA.KeyProperty->GetName() == TreeNodeB.KeyProperty->GetName())
		{
			if (CastField<FSetProperty>(TreeNodeA.KeyProperty->Owner.ToField()) || CastField<FMapProperty>(TreeNodeA.KeyProperty->Owner.ToField()))
			{
				const void* DataA = TreeNodeA.KeyProperty->ContainerPtrToValuePtr<void*>(TreeNodeA.Object);
				const void* DataB = TreeNodeB.KeyProperty->ContainerPtrToValuePtr<void*>(TreeNodeB.Object);
				return TreeNodeA.KeyProperty->Identical(DataA, DataB, PPF_DeepComparison);
			}
			else
			{
				return true;
			}
		}
		return false;
	}
	
	void GetChildren(const FPropertyInstance& InParent, TArray<FPropertyInstance>& OutChildren) const
	{
		return InParent.GetChildren(OutChildren);
	}

	bool ShouldMatchByValue(const FPropertyInstance& TreeNodeA) const
	{
		// array elements should match by value
		return CastField<FArrayProperty>(TreeNodeA.KeyProperty->Owner.ToField()) != nullptr;
	}
	
	bool ShouldInheritEqualFromChildren(const FPropertyInstance& TreeNodeA, const FPropertyInstance& TreeNodeB) const
	{
		return true;
	}
};

static TAsyncTreeDifferences<FPropertyInstance> ObjectPropTreeDiff(const UObject* ObjectA, const UObject* ObjectB)
{
	TArray<FPropertyInstance> ChildrenA;
	if (ObjectA)
	{
		for (TFieldIterator<FProperty> PropertyIt(ObjectA->GetClass()); PropertyIt; ++PropertyIt)
		{
			if (!PropertyIt->HasAnyPropertyFlags(CPF_Transient | CPF_DuplicateTransient | CPF_NonPIEDuplicateTransient))
			{
				ChildrenA.Emplace(ObjectA, *PropertyIt);
			}
		}
	}
	TArray<FPropertyInstance> ChildrenB;
	if (ObjectB)
	{
		for (TFieldIterator<FProperty> PropertyIt(ObjectB->GetClass()); PropertyIt; ++PropertyIt)
		{
			if (!PropertyIt->HasAnyPropertyFlags(CPF_Transient | CPF_DuplicateTransient | CPF_NonPIEDuplicateTransient))
			{
				ChildrenB.Emplace(ObjectB, *PropertyIt);
			}
		}
	}
	TAsyncTreeDifferences<FPropertyInstance> TreeDifferences(ChildrenA, ChildrenB);
	TreeDifferences.FlushQueue();
	return TreeDifferences;
}

// converts a TAsyncTreeDifferences<FPropertyInstance>::DiffNodeType into a FPropertySoftPath
using DiffNode = TAsyncTreeDifferences<FPropertyInstance>::DiffNodeType;
static FPropertySoftPath ToPropertySoftPath(const TUniquePtr<DiffNode>& Node)
{
	FString Path;
	TArray<FName> Chain;
	for (const DiffNode* Current = Node.Get(); Current->Parent; Current = Current->Parent)
	{
		const FPropertyInstance& Instance = (Current->ValueA.ValProperty) ? Current->ValueA : Current->ValueB;
		const FPropertyInstance& Parent = (Current->ValueA.ValProperty) ? Current->Parent->ValueA : Current->Parent->ValueB;
		FName Name = Instance.ValProperty->GetFName();
		if (Parent.ValProperty)
		{
			if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Parent.ValProperty))
            {
            	FScriptArrayHelper Helper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(Parent.Object));
				// possible optimization: use pointer arithmetic to avoid loop?
            	for (int32 I = 0; I < Helper.Num(); ++I)
            	{
            		if (Helper.GetElementPtr(I) == Instance.ValProperty->ContainerPtrToValuePtr<void>(Instance.Object))
            		{
            			Name = FName(FString::FromInt(I));
            			break;
            		}
            	}
            }
            else if (const FMapProperty* MapProperty = CastField<FMapProperty>(Parent.ValProperty))
            {
            	FScriptMapHelper Helper(MapProperty, MapProperty->ContainerPtrToValuePtr<void>(Parent.Object));
            	const int32 I = Helper.FindMapIndexWithKey(Instance.KeyProperty->ContainerPtrToValuePtr<void>(Instance.Object));
            	Name = FName(FString::FromInt(I));
            }
            else if (const FSetProperty* SetProperty = CastField<FSetProperty>(Parent.ValProperty))
            {
	            FScriptSetHelper Helper(SetProperty, SetProperty->ContainerPtrToValuePtr<void>(Parent.Object));
            	const int32 I = Helper.FindElementIndex(Instance.KeyProperty->ContainerPtrToValuePtr<void>(Instance.Object));
            	Name = FName(FString::FromInt(I));
            }
		}
		Chain.Add(Name);
	}
	Algo::Reverse(Chain);
	return FPropertySoftPath(Chain);
}

template<typename CppType>
static void TryRelinkProperty(void* Data, const TFObjectPropertyBase<CppType>* Property, const UPackage* FromPackage, UPackage* ToPackage)
{
	CppType* CppTypePtr = Property->GetPropertyValuePtr(Data);
	const UObject* Object = CppTypePtr->Get();
	if (Object && Object->IsIn(FromPackage))
	{
		if (UObject* Found = FindObjectChecked<UObject>(ToPackage, *Object->GetPathName(FromPackage)))
		{
			check(Found->IsIn(ToPackage));
			*CppTypePtr = CppType(Found);
		}
	}
}

static void TryRelinkProperty(void* Data, const FStructProperty* Property, const UPackage* FromPackage, UPackage* ToPackage)
{
    if (Property->Struct == TBaseStructure<FSoftObjectPath>::Get())
    {
    	FSoftObjectPath* PathPtr = reinterpret_cast<FSoftObjectPath*>(Data);
    	const UObject* Object = PathPtr->ResolveObject();
    	if (Object && Object->IsIn(FromPackage))
    	{
    		if (UObject* Found = FindObjectChecked<UObject>(ToPackage, *Object->GetPathName(FromPackage)))
    		{
    			check(Found->IsIn(ToPackage));
    			*PathPtr = FSoftObjectPath(Found);
    		}
    	}
    }
}

static void TryRelinkProperty(void* Data, const FInterfaceProperty* Property, const UPackage* FromPackage, UPackage* ToPackage)
{
	FScriptInterface* Interface = Property->GetPropertyValuePtr(Data);
	const UObject* Object = Interface->GetObject();
    if (Object && Object->IsIn(FromPackage))
    {
    	if (UObject* Found = FindObjectChecked<UObject>(ToPackage, *Object->GetPathName(FromPackage)))
    	{
    		check(Found->IsIn(ToPackage));
    		*Interface = FScriptInterface(Found, Found->GetInterfaceAddress(Property->InterfaceClass));
    	}
    }
}

// redirect pointers to UObjects in FromPackage to UObjects in ToPackage
static void RelinkObjectProperties(const TArray<FPropertyInstance>& Props, UPackage* FromPackage, UPackage* ToPackage)
{
	check(FromPackage && ToPackage);
	for (const FPropertyInstance& Instance : Props)
	{
		void *Data = const_cast<void*>(Instance.ValProperty->ContainerPtrToValuePtr<void>(Instance.Object));
		
		if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Instance.ValProperty))
		{
			TryRelinkProperty(Data, ObjectProperty, FromPackage, ToPackage);
		}
		else if (const FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(Instance.ValProperty))
		{
			TryRelinkProperty(Data, SoftObjectProperty, FromPackage, ToPackage);
		}
		else if (const FWeakObjectProperty* WeakObjectProperty = CastField<FWeakObjectProperty>(Instance.ValProperty))
		{
			TryRelinkProperty(Data, WeakObjectProperty, FromPackage, ToPackage);
		}
		else if (const FLazyObjectProperty* LazyObjectProperty = CastField<FLazyObjectProperty>(Instance.ValProperty))
		{
			TryRelinkProperty(Data, LazyObjectProperty, FromPackage, ToPackage);
		}
		else if (const FStructProperty* StructProperty = CastField<FStructProperty>(Instance.ValProperty))
		{
			TryRelinkProperty(Data, StructProperty, FromPackage, ToPackage);
		}
		else if (const FInterfaceProperty* InterfaceProperty = CastField<FInterfaceProperty>(Instance.ValProperty))
		{
			TryRelinkProperty(Data, InterfaceProperty, FromPackage, ToPackage);
		}
		
		TArray<FPropertyInstance> Children;
		Instance.GetChildren(Children);
		if (Children.Num())
		{
			RelinkObjectProperties(Children, FromPackage, ToPackage);
		}
	}
}

// redirect pointers to UObjects in FromPackage to UObjects in ToPackage
static void RelinkObjectProperties(UObject* Object, UPackage* FromPackage, UPackage* ToPackage)
{
	TArray<FPropertyInstance> PropertyInstances;
	for (TFieldIterator<FProperty> PropertyIt(Object->GetClass()); PropertyIt; ++PropertyIt)
	{
		PropertyInstances.Emplace(Object, *PropertyIt);
	}

	RelinkObjectProperties(PropertyInstances, FromPackage, ToPackage);
}

template<typename CppType>
bool SoftCompareProperty(const TFObjectPropertyBase<CppType>* Property, void* DataA, void* DataB, const UPackage* PackageA, const UPackage* PackageB)
{
	CppType* CppTypePtrA = Property->GetPropertyValuePtr(DataA);
	CppType* CppTypePtrB = Property->GetPropertyValuePtr(DataB);
	const UObject* ObjectA = CppTypePtrA->Get();
	const UObject* ObjectB = CppTypePtrB->Get();
	if (ObjectA && ObjectA->IsIn(PackageA) && ObjectB && ObjectB->IsIn(PackageB))
	{
		return ObjectA->GetPathName(PackageA) == ObjectB->GetPathName(PackageB);
	}
	return ObjectA == ObjectB;
}

bool SoftCompareProperty(const FStructProperty* Property, void* DataA, void* DataB, const UPackage* PackageA, const UPackage* PackageB)
{
	if (Property->Struct == TBaseStructure<FSoftObjectPath>::Get())
	{
		const UObject* ObjectA = reinterpret_cast<FSoftObjectPath*>(DataA)->ResolveObject();
		const UObject* ObjectB = reinterpret_cast<FSoftObjectPath*>(DataB)->ResolveObject();
		if (ObjectA && ObjectA->IsIn(PackageA) && ObjectB && ObjectB->IsIn(PackageB))
		{
			return ObjectA->GetPathName(PackageA) == ObjectB->GetPathName(PackageB);
		}
		return ObjectA == ObjectB;
	}
	return Property->Identical(DataA, DataB, PPF_DeepComparison);
}

bool SoftCompareProperty(const FInterfaceProperty* Property, void* DataA, void* DataB, const UPackage* PackageA, const UPackage* PackageB)
{
	const UObject* ObjectA = Property->GetPropertyValuePtr(DataA)->GetObject();
	const UObject* ObjectB = Property->GetPropertyValuePtr(DataB)->GetObject();
	if (ObjectA && ObjectA->IsIn(PackageA) && ObjectB && ObjectB->IsIn(PackageB))
	{
		return ObjectA->GetPathName(PackageA) == ObjectB->GetPathName(PackageB);
	}
	return ObjectA == ObjectB;
}

bool SoftCompareProperty(const FFieldPathProperty* Property, void* DataA, void* DataB, const UPackage* PackageA, const UPackage* PackageB)
{
	const FFieldPath FieldPathA = Property->GetPropertyValue(DataA);
	const FFieldPath FieldPathB = Property->GetPropertyValue(DataB);
	FString PathAString = FieldPathA.ToString();
	FString PathBString = FieldPathB.ToString();
	if (PathAString.RemoveFromStart(PackageA->GetName() + TEXT(".")) && PathBString.RemoveFromStart(PackageB->GetName() + TEXT(".")))
	{
		// if the field path is within the merging packages, compare the path without the package
		return PathAString == PathBString;
	}
	return FieldPathA == FieldPathB;
}

static bool SoftCompare(const FPropertyInstance& A, const FPropertyInstance& B, const UPackage* PackageA, const UPackage* PackageB)
{
	void *DataA = const_cast<void*>(A.ValProperty->ContainerPtrToValuePtr<void>(A.Object));
	void *DataB = const_cast<void*>(B.ValProperty->ContainerPtrToValuePtr<void>(B.Object));
	
	if (const FObjectProperty* Prop = CastField<FObjectProperty>(A.ValProperty))
	{
		return SoftCompareProperty(Prop, DataA, DataB, PackageA, PackageB);
	}
	if (const FSoftObjectProperty* Prop = CastField<FSoftObjectProperty>(A.ValProperty))
	{
		return SoftCompareProperty(Prop, DataA, DataB, PackageA, PackageB);
	}
	if (const FWeakObjectProperty* Prop = CastField<FWeakObjectProperty>(A.ValProperty))
	{
		return SoftCompareProperty(Prop, DataA, DataB, PackageA, PackageB);
	}
	if (const FLazyObjectProperty* Prop = CastField<FLazyObjectProperty>(A.ValProperty))
	{
		return SoftCompareProperty(Prop, DataA, DataB, PackageA, PackageB);
	}
	if (const FStructProperty* Prop = CastField<FStructProperty>(A.ValProperty))
	{
		return SoftCompareProperty(Prop, DataA, DataB, PackageA, PackageB);
	}
	if (const FInterfaceProperty* Prop = CastField<FInterfaceProperty>(A.ValProperty))
	{
		return SoftCompareProperty(Prop, DataA, DataB, PackageA, PackageB);
	}
	if (const FFieldPathProperty* Prop = CastField<FFieldPathProperty>(A.ValProperty))
	{
		return SoftCompareProperty(Prop, DataA, DataB, PackageA, PackageB);
	}
	return A.ValProperty->Identical(DataA, DataB, PPF_DeepComparison);
}

// diff algorithm that gets all the property differences from every object in ExportPaths
// note that FObjectProperties are shallow diffed
static TMap<FString, TMap<FPropertySoftPath, ETreeDiffResult>> GetDifferences(const TArray<FString>& ExportPaths, UPackage* PackageA, UPackage* PackageB)
{
	TMap<FString, TMap<FPropertySoftPath, ETreeDiffResult>> Result;
	for (const FString& Path : ExportPaths)
	{
		const UObject* ObjectA = PackageA ? FindObject<UObject>(PackageA, *Path) : nullptr;
		const UObject* ObjectB =  PackageB ? FindObject<UObject>(PackageB, *Path) : nullptr;

		const TAsyncTreeDifferences<FPropertyInstance> TreeDiff = ObjectPropTreeDiff(ObjectA, ObjectB);

		// find every item that differs between source and default
		TreeDiff.ForEach(ETreeTraverseOrder::PreOrder,
			[&Result, &Path, PackageA, PackageB](const TUniquePtr<DiffNode>& Node)->ETreeTraverseControl
			{
				switch(Node->DiffResult)
				{
					case ETreeDiffResult::MissingFromTree1:
						Result.FindOrAdd(Path).Add(ToPropertySoftPath(Node), Node->DiffResult);
						break;
					case ETreeDiffResult::MissingFromTree2:
						Result.FindOrAdd(Path).Add(ToPropertySoftPath(Node), Node->DiffResult);
						break;
					case ETreeDiffResult::DifferentValues:
						{
							// if this isn't a leaf, continue to children to find more detailed diff info
							if (!Node->Children.IsEmpty())
							{
								return ETreeTraverseControl::Continue;
							}

							if (!SoftCompare(Node->ValueA, Node->ValueB, PackageA, PackageB))
							{
								Result.FindOrAdd(Path).Add(ToPropertySoftPath(Node), Node->DiffResult);
							}
						}
						break;
					case ETreeDiffResult::Identical:
						break;
					default: check(false);
				}
				
				return ETreeTraverseControl::Continue;
			});
	}
	return Result;
}

// looks for subobjects of Template that aren't in Object's package and duplicates them
static void DuplicateMissingSubobjects(UObject* Object, const UObject* Template)
{
	check(Object && Template);
	UPackage* Package = Object->GetPackage();
	UPackage* TemplatePackage = Template->GetPackage();
	const TArray<FString> ExportPaths = GatherExportPaths(Template);
	
	for (const FString& ExportPath : ExportPaths)
	{
		if (!FindObject<UObject>(Package, *ExportPath))
		{
			const UObject* CurTemplate = FindObjectChecked<UObject>(TemplatePackage, *ExportPath);
			UObject* Outer;
			if (CurTemplate->GetOuter()->IsA<UPackage>())
			{
				Outer = Package;
			}
			else
			{
				FString OuterPath = CurTemplate->GetOuter()->GetPathName(TemplatePackage);
				Outer = FindObjectChecked<UObject>(Package, *OuterPath);
			}
			check(Outer);
			DuplicateForMerge(CurTemplate, Outer);
		}
	}
}

static void NotifyPropertyChange(UObject* Object, const TUniquePtr<DiffNode>& Node, EPropertyChangeType::Type Type)
{			
	if (Object)
	{
		const DiffNode* Itr = Node.Get();
		while(Itr->Parent->ValueA.ValProperty != nullptr)
		{
			Itr = Itr->Parent;
		}
		
		// notify property change
		FPropertyChangedEvent Event(
			const_cast<FProperty*>(Itr->ValueA.ValProperty),
			Type
		);
		Object->PostEditChangeProperty(Event);
	}
};

// assigns Node->ValueB to Node->ValueA
static void HandleMergeAssign(UObject* Object, const TUniquePtr<DiffNode>& Node)
{
	check(Node->DiffResult == ETreeDiffResult::DifferentValues);
	const void* Source = Node->ValueB.ValProperty->ContainerPtrToValuePtr<void>(Node->ValueB.Object);
	void* Dest = const_cast<void*>(Node->ValueA.ValProperty->ContainerPtrToValuePtr<void>(Node->ValueA.Object));
	Node->ValueA.ValProperty->CopyCompleteValue(Dest, Source);
	NotifyPropertyChange(Object, Node, EPropertyChangeType::ValueSet);
}

// inserts Node->ValueB into Node->Parent->ValueA
static void HandleMergeInsert(UObject* Object, const TUniquePtr<DiffNode>& Node)
{
	check(Node->DiffResult == ETreeDiffResult::MissingFromTree1);
	if (Node->Parent->ValueA)
	{
        const FProperty* SourceProperty = Node->ValueB.ValProperty;
        const void* SourceValue = Node->ValueB.ValProperty->ContainerPtrToValuePtr<void>(Node->ValueB.Object);
        check(SourceProperty && SourceValue)
        
        const FProperty* ParentResultProperty = Node->Parent->ValueA.ValProperty;
        const void* ParentResultObject = Node->Parent->ValueA.Object;
        check(ParentResultProperty && ParentResultObject);
        
        if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(ParentResultProperty))
        {
        	FScriptArrayHelper Helper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(ParentResultObject));
    
        	// count the number of valid result siblings before this node to get the insert index
        	int32 InsertIndex = 0;
        	for (const TUniquePtr<DiffNode>& Sibling : Node->Parent->Children)
        	{
        		if (Sibling == Node)
        		{
        			break;
        		}
        		if (Sibling->DiffResult != ETreeDiffResult::MissingFromTree1)
        		{
        			++InsertIndex;
        		}
        	}
    
        	Helper.InsertValues(InsertIndex, 1);
    
        	SourceProperty->CopyCompleteValue(Helper.GetElementPtr(InsertIndex), SourceValue);
        	
        	NotifyPropertyChange(Object, Node, EPropertyChangeType::ArrayAdd);
        	return;
        }
        else if (const FSetProperty* SetProperty = CastField<FSetProperty>(ParentResultProperty))
        {
        	FScriptSetHelper Helper(SetProperty, SetProperty->ContainerPtrToValuePtr<void>(ParentResultObject));
        	Helper.AddElement(SourceValue);
        	NotifyPropertyChange(Object, Node, EPropertyChangeType::ArrayAdd);
        	return;
        }
        else if (const FMapProperty* MapProperty = CastField<FMapProperty>(ParentResultProperty))
        {
        	FScriptMapHelper Helper(MapProperty, MapProperty->ContainerPtrToValuePtr<void>(ParentResultObject));
        	const FProperty* SourceKeyProperty = Node->ValueB.KeyProperty;
        	
        	Helper.AddPair(
        		SourceKeyProperty->ContainerPtrToValuePtr<void>(Node->ValueB.Object),
        		SourceValue
        	);
        	NotifyPropertyChange(Object, Node, EPropertyChangeType::ArrayAdd);
        	return;
        }
	}

	// if you get this warning it's likely because of a type mismatch between the objects being merged.
	UE_LOGF(LogSourceControl, Warning, "Data loss in Merge: property: [%ls] in object: %ls", *ToPropertySoftPath(Node).ToDisplayName(), *Object->GetName())
}

// removes Node->ValueA from Node->Parent->ValueA
static void HandleMergeRemove(UObject* Object, const TUniquePtr<DiffNode>& Node)
{
	check(Node->DiffResult == ETreeDiffResult::MissingFromTree2);
	if (Node->Parent->ValueA)
	{
		const FProperty* ResultProperty = Node->ValueA.ValProperty;
		const void* ResultObject = Node->ValueA.Object;
		check(ResultProperty && ResultObject)
		
		const FProperty* ParentResultProperty = Node->Parent->ValueA.ValProperty;
		const void* ParentResultObject = Node->Parent->ValueA.Object;
		check(ParentResultProperty && ParentResultObject);
		
		if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(ParentResultProperty))
        {
            FScriptArrayHelper Helper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(ParentResultObject));
            // possible optimization: use pointer arithmetic to avoid loop?
            for (int I = 0; I < Helper.Num(); ++I)
            {
                if (Helper.GetElementPtr(I) == ResultProperty->ContainerPtrToValuePtr<void>(ResultObject))
                {
                    Helper.RemoveValues(I, 1);
                    break;
                }
            }
            NotifyPropertyChange(Object, Node, EPropertyChangeType::ArrayRemove);
        	return;
        }
		else if (const FSetProperty* SetProperty = CastField<FSetProperty>(ParentResultProperty))
        {
            FScriptSetHelper Helper(SetProperty, SetProperty->ContainerPtrToValuePtr<void>(ParentResultObject));
            Helper.RemoveElement(ResultProperty->ContainerPtrToValuePtr<void>(ResultObject));
            NotifyPropertyChange(Object, Node, EPropertyChangeType::ArrayRemove);
        	return;
        }
		else if (const FMapProperty* MapProperty = CastField<FMapProperty>(ParentResultProperty))
        {
            FScriptMapHelper Helper(MapProperty, MapProperty->ContainerPtrToValuePtr<void>(ParentResultObject));
            const FProperty* ResultKeyProperty = Node->ValueA.KeyProperty;
            Helper.RemovePair(ResultKeyProperty->ContainerPtrToValuePtr<void>(ResultObject));
            NotifyPropertyChange(Object, Node, EPropertyChangeType::ArrayRemove);
        	return;
        }
	}
	
	// if you get this warning it's likely because of a type mismatch between the objects being merged.
	UE_LOGF(LogSourceControl, Warning, "Data loss in Merge: property: [%ls] in object: %ls", *ToPropertySoftPath(Node).ToDisplayName(), *Object->GetName())
}

 /**
  *	PseudoCode:\n
  *	foreach (CurResultObject, CurSourceObject, CurDefaultObject) in (ResultObject, SourceObject, DefaultObject).SubObjects():
  *		foreach (ResultProperty, SourceProperty, DefaultProperty) in (CurResultObject, CurSourceObject, CurDefaultObject).Properties():
  *			if (SourceProperty != DefaultProperty && ResultProperty != SourceProperty):
  *				ResultProperty = SourceProperty
  */
static void CopyDeltaProperties(UObject* ResultObject, const UObject* SourceObject, const UObject* DefaultObject = nullptr)
{
	check(ResultObject && SourceObject);
	UPackage* ResultPackage = ResultObject->GetPackage();
	UPackage* SourcePackage = SourceObject->GetPackage();
	UPackage* DefaultPackage = DefaultObject ? DefaultObject->GetPackage() : nullptr;
	
	// if SourceObject has extra exports, create them
	DuplicateMissingSubobjects(ResultObject, SourceObject);

	TArray<FString> ExportPaths = GatherExportPaths(SourceObject);
	TMap<FString, TMap<FPropertySoftPath, ETreeDiffResult>> SourceDefaultDiff = GetDifferences(ExportPaths, SourcePackage, DefaultPackage);
	
	for (const FString& Path : ExportPaths)
	{
		const UObject* CurSource = FindObjectChecked<UObject>(SourcePackage, *Path);
		UObject* CurResult = FindObject<UObject>(ResultPackage, *Path);
		if (!CurResult)
		{
			UObject* Outer = ResultPackage;
			if (CurSource->GetOuter() != SourcePackage)
			{
				Outer = FindObjectChecked<UObject>(ResultPackage, *CurSource->GetOuter()->GetPathName(SourcePackage));
			}
			CurResult = DuplicateForMerge(CurSource, Outer);
		}
		
		const TMap<FPropertySoftPath, ETreeDiffResult>* CurSourceDefaultDiff = SourceDefaultDiff.Find(Path);
		
		check(CurResult);
		if (CurSource && CurSourceDefaultDiff)
		{
			const TAsyncTreeDifferences<FPropertyInstance> ResultSourceTreeDiff = ObjectPropTreeDiff(CurResult, CurSource);
		
			ResultSourceTreeDiff.ForEach(ETreeTraverseOrder::ReversePreOrder,
				[CurSourceDefaultDiff, CurResult, SourcePackage, ResultPackage] (const TUniquePtr<DiffNode>& Node)->ETreeTraverseControl
				{
					// only copy leaf nodes
					if (!Node->Children.IsEmpty())
					{
						// even if insert/delete has children, treat it as a leaf.
						if (Node->DiffResult != ETreeDiffResult::MissingFromTree1 && Node->DiffResult != ETreeDiffResult::MissingFromTree2)
						{
							return ETreeTraverseControl::Continue;
						}
					}

					// find cached diff result between SourceObject and DefaultObject for this property
					ETreeDiffResult SourceDefaultDiffResult = ETreeDiffResult::Identical;
					if (const ETreeDiffResult* Found = CurSourceDefaultDiff->Find(ToPropertySoftPath(Node)))
					{
						SourceDefaultDiffResult = *Found;
					}
					
					// for every item that differs between source and default, copy source to result
					if (SourceDefaultDiffResult != ETreeDiffResult::Identical)
					{
						switch(Node->DiffResult)
						{
						case ETreeDiffResult::MissingFromTree1:
							HandleMergeInsert(CurResult, Node);
							break;
						case ETreeDiffResult::MissingFromTree2:
							HandleMergeRemove(CurResult, Node);
							break;
						case ETreeDiffResult::DifferentValues:
							{
								if (!SoftCompare(Node->ValueA, Node->ValueB, ResultPackage, SourcePackage))
								{
                            		HandleMergeAssign(CurResult, Node);
								}
							}
							
							break;
						case ETreeDiffResult::Identical: break;
						default: check(false);
						}
					}

					return ETreeTraverseControl::SkipChildren;
				});
		}

		// redirect pointers to UObjects in SourcePackage to UObjects in ResultPackage
		RelinkObjectProperties(CurResult, SourcePackage, ResultPackage);
    }
}

// Copies BranchA's changes before BranchB's changes so that BranchB always has precedence in the returned object
static UObject* AutoMerge(const UObject* BaseRevision, const UObject* BranchA, const UObject* BranchB, FName PackageName)
{
	// apply changes from BranchA first by simply duplicating it over to the result
	PackageName = MakeUniqueObjectName(nullptr, BranchA->GetClass(), PackageName, EUniqueObjectNameOptions::GloballyUnique);
	UPackage* MergedPackage = CreatePackage(*(TEXT("/Temp/") + PackageName.ToString()));

	// Base revision into MergedPackage
	UObject* Merged = DuplicateForMerge(BaseRevision, MergedPackage);
	
	// deep copy BranchA's changes into Merged Object
	CopyDeltaProperties(Merged, BranchA, BaseRevision);
	
	// deep copy BranchB's changes into Merged Object
	CopyDeltaProperties(Merged, BranchB, BaseRevision);
	
	return Merged;
}

EAssetCommandResult MergeUtils::Merge(const FAssetAutomaticMergeArgs& MergeArgs)
{
	if (!ensure(MergeArgs.LocalAsset))
	{
		return EAssetCommandResult::Unhandled;
	}
	
	FAssetManualMergeArgs ManualMergeArgs;
	ManualMergeArgs.LocalAsset = MergeArgs.LocalAsset;
	ManualMergeArgs.ResolutionCallback = MergeArgs.ResolutionCallback;
	ManualMergeArgs.Flags = MergeArgs.Flags;
	const UPackage* LocalPackage = ManualMergeArgs.LocalAsset->GetPackage();
	
	
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	const TSharedRef<FUpdateStatus, ESPMode::ThreadSafe> UpdateStatusOperation = ISourceControlOperation::Create<FUpdateStatus>();
	UpdateStatusOperation->SetUpdateHistory(true);
	SourceControlProvider.Execute(UpdateStatusOperation, LocalPackage);
	
	// Get the SCC state
	const FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(LocalPackage, EStateCacheUsage::Use);

	// If we have an asset and its in SCC..
	if( SourceControlState.IsValid() && SourceControlState->IsSourceControlled() )
	{
		const ISourceControlState::FResolveInfo ResolveInfo = SourceControlState->GetResolveInfo();
		check(ResolveInfo.IsValid());
		
		if(UPackage* TempPackage = LoadPackageForMerge(ResolveInfo.RemoteFile, ResolveInfo.RemoteRevision, LocalPackage))
		{
			// Grab the old asset from that old package
			ManualMergeArgs.RemoteAsset = FindObject<UObject>(TempPackage, *ManualMergeArgs.LocalAsset->GetName());

			// Recovery for package names that don't match
			if (ManualMergeArgs.RemoteAsset == nullptr)
			{
				ManualMergeArgs.RemoteAsset = TempPackage->FindAssetInPackage();
			}
		}
		
		if(UPackage* TempPackage = LoadPackageForMerge(ResolveInfo.BaseFile, ResolveInfo.BaseRevision, LocalPackage))
		{
			// Grab the old asset from that old package
			ManualMergeArgs.BaseAsset = FindObject<UObject>(TempPackage, *ManualMergeArgs.LocalAsset->GetName());

			// Recovery for package names that don't match
			if (ManualMergeArgs.BaseAsset == nullptr)
			{
				ManualMergeArgs.BaseAsset = TempPackage->FindAssetInPackage();
			}
		}
	}

	// single asset merging is only supported for assets in a conflicted state in source control
	if (!ensure(ManualMergeArgs.BaseAsset && ManualMergeArgs.RemoteAsset && ManualMergeArgs.LocalAsset))
	{
		return EAssetCommandResult::Unhandled;
	}
	
	return Merge(ManualMergeArgs);
}

EAssetCommandResult MergeUtils::Merge(const FAssetManualMergeArgs& MergeArgs)
{
	auto NotifyResolution = [MergeArgs](EAssetMergeResult Result)
	{
		FAssetMergeResults Results;
		Results.Result = Result;
		Results.MergedPackage = MergeArgs.LocalAsset->GetPackage();
		MergeArgs.ResolutionCallback.ExecuteIfBound(Results);
		return EAssetCommandResult::Handled;
	};

	// apply changes in different orders to come up with two possible merge options
	const UObject* FavorRemote = AutoMerge(MergeArgs.BaseAsset,MergeArgs.LocalAsset, MergeArgs.RemoteAsset, TEXT("FavorRemote"));
	const UObject* FavorLocal = AutoMerge(MergeArgs.BaseAsset,MergeArgs.RemoteAsset,MergeArgs.LocalAsset, TEXT("FavorLocal"));
	
	const TArray<FString> FavorRemoteExports = GatherExportPaths(FavorRemote);
	const TArray<FString> FavorLocalExports = GatherExportPaths(FavorLocal);
	ensureMsgf(FavorRemoteExports == FavorLocalExports, TEXT("Merge Result has a non-deterministic export list. This is likely because of object names with guids in them."));
	check(FavorRemoteExports.Num() == FavorLocalExports.Num());
	
	TMap<FString, TMap<FPropertySoftPath, ETreeDiffResult>> Conflicts = GetDifferences(
		FavorLocalExports, FavorLocal->GetPackage(), FavorRemote->GetPackage());

	if (!Conflicts.IsEmpty())
	{
		// some properties like uuids mutate on duplicate so this will catch all of those properties and we can ignore them
		const UObject* FavorLocal2 = AutoMerge(MergeArgs.BaseAsset,MergeArgs.RemoteAsset,MergeArgs.LocalAsset, TEXT("FavorLocal"));
	
		const TMap<FString, TMap<FPropertySoftPath, ETreeDiffResult>> FalsePositives = GetDifferences(
			FavorLocalExports, FavorLocal->GetPackage(), FavorLocal2->GetPackage());
		
		for (const auto& [ObjectPath, Differences] : FalsePositives)
		{
			for (auto& [Path, Diff] : Differences)
			{
				if (TMap<FPropertySoftPath, ETreeDiffResult>* Found = Conflicts.Find(ObjectPath))
				{
					Found->Remove(Path);
					if (Found->IsEmpty())
					{
						Conflicts.Remove(ObjectPath);
					}
				}
			}
		}
	}

	
	// if both merge options are the same, we have no conflicts
	if (Conflicts.IsEmpty())
	{
		// auto-merge successful!
		FScopedMergeResolveTransaction UndoHandler(MergeArgs.LocalAsset, MergeArgs.Flags);
		
		// copy changes over to the local asset
		CopyDeltaProperties(MergeArgs.LocalAsset, FavorLocal);
		
		return NotifyResolution(EAssetMergeResult::Completed);
	}
	
	// conflicts detected. We need to ask the user to manually resolve them
	if (!(MergeArgs.Flags & MF_NO_GUI))
	{
		const TSharedRef<SDetailsDiff> DiffView = SDetailsDiff::CreateDiffWindow(MergeArgs.RemoteAsset, MergeArgs.LocalAsset, {}, {}, FavorRemote->GetClass());

		// construct center panel object and copy FavorLocal into it
		const FName ResultPackageName = MakeUniqueObjectName(nullptr, FavorLocal->GetClass(), TEXT("MergeResult"), EUniqueObjectNameOptions::GloballyUnique);
		UPackage* ResultPackage = CreatePackage(*(TEXT("/Temp/") + ResultPackageName.ToString()));
		UObject* ResultObject = DuplicateForMerge(FavorLocal, ResultPackage);
		
		DiffView->ReportMergeConflicts(Conflicts);
		DiffView->SetOutputObject(ResultObject);
		
		DiffView->OnWindowClosedEvent.AddLambda([MergeArgs, NotifyResolution](const TSharedRef<SDetailsDiff>& DiffView)
		{
			FScopedMergeResolveTransaction UndoHandler(MergeArgs.LocalAsset, MergeArgs.Flags);
			
			// copy changes over to the local asset
			CopyDeltaProperties(MergeArgs.LocalAsset, DiffView->GetOutputObject());

			if (UBlueprint* AsBlueprint = Cast<UBlueprint>(MergeArgs.LocalAsset))
			{
				// because merging may have changed the parent class, we should recompile
				// what we really need from this is to get the CDO up to date so it's type
				// matches the UBlueprint::ParentClass
				EBlueprintCompileOptions CompileOptions
                {
                		EBlueprintCompileOptions::SkipSave
                	|	EBlueprintCompileOptions::UseDeltaSerializationDuringReinstancing
                	|	EBlueprintCompileOptions::SkipNewVariableDefaultsDetection
                };
    
                FKismetEditorUtilities::CompileBlueprint(AsBlueprint, CompileOptions);
			}
			
			NotifyResolution(EAssetMergeResult::Completed);
		});
		
		return EAssetCommandResult::Handled;
	}
	
	return NotifyResolution(EAssetMergeResult::Cancelled);
}

#if false && WITH_DEV_AUTOMATION_TESTS
#include "Misc/AutomationTest.h"
namespace UE::MergeUtilsTests
{

	IMPLEMENT_COMPLEX_AUTOMATION_TEST(FMergeWithSelfTests, "ReviewDiffMerge.MergeWithSelf", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
	void FMergeWithSelfTests::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		TArray<FAssetData> Assets;
		AssetRegistryModule.Get().GetAssetsByClass(UBlueprint::StaticClass()->GetClassPathName(), Assets, true);
		Algo::RandomShuffle(Assets);
		for (const FAssetData& Asset : Assets)
		{
			UBlueprint* BP = CastChecked<UBlueprint>(Asset.GetAsset());
			if (FBlueprintEditorUtils::IsDataOnlyBlueprint(BP))
			{
				OutBeautifiedNames.Add(Asset.AssetName.ToString());
				OutTestCommands.Add(Asset.PackageName.ToString());
				if (OutTestCommands.Num() >= 5)
				{
					break;
				}
			}
		}
	}

	bool FMergeWithSelfTests::RunTest(const FString& PackageName)
	{
		const FPackagePath Path = FPackagePath::FromPackageNameChecked(PackageName);
		if (const UPackage* Package = DiffUtils::LoadPackageForDiff(Path, {}))
		{
			UBlueprint* BP = CastChecked<UBlueprint>(Package->FindAssetInPackage());
         	FAssetManualMergeArgs Args;
         	Args.BaseAsset = BP;
         	Args.LocalAsset = BP;
         	Args.RemoteAsset = BP;
         	Args.Flags = MF_NO_GUI;
         	bool bSuccess = false;
         	Args.ResolutionCallback = FOnAssetMergeResolved::CreateLambda(
         		[&bSuccess](const FAssetMergeResults& Results)
         		{
         			bSuccess = Results.Result == EAssetMergeResult::Completed;
         		});
         	FString CheckName = FString::Format(TEXT("Self merge: {0}"), {BP->GetFriendlyName()});
         	MergeUtils::Merge(Args);
         	UTEST_TRUE(CheckName, bSuccess);
		}
        return true;
	}
}
#endif

#undef LOCTEXT_NAMESPACE
