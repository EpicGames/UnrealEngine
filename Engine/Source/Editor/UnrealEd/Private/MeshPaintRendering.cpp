// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================
	MeshPaintRendering.cpp: Mesh texture paint brush rendering
================================================================================*/

#include "MeshPaintRendering.h"
#include "ShaderParameters.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Shader.h"
#include "StaticBoundShaderState.h"
#include "RHIStaticStates.h"
#include "BatchedElements.h"
#include "GlobalShader.h"
#include "ShaderParameterUtils.h"
#include "TextureResource.h"
#include "Engine/TextureRenderTarget2D.h"
#include "PipelineStateCache.h"

namespace MeshPaintRendering
{

	/** Mesh paint vertex shader */
	class TMeshPaintVertexShader : public FGlobalShader
	{
		DECLARE_SHADER_TYPE( TMeshPaintVertexShader, Global );

	public:

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) && !IsConsolePlatform(Parameters.Platform);
		}

		/** Default constructor. */
		TMeshPaintVertexShader() {}

		/** Initialization constructor. */
		TMeshPaintVertexShader( const ShaderMetaType::CompiledShaderInitializerType& Initializer )
			: FGlobalShader( Initializer )
		{
			TransformParameter.Bind( Initializer.ParameterMap, TEXT( "c_Transform" ) );
		}

		void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FMatrix44f& InTransform )
		{
			SetShaderValue(BatchedParameters, TransformParameter, InTransform );
		}

	private:
		LAYOUT_FIELD(FShaderParameter, TransformParameter);
	};


	IMPLEMENT_SHADER_TYPE( , TMeshPaintVertexShader, TEXT( "/Engine/Private/MeshPaintVertexShader.usf" ), TEXT( "Main" ), SF_Vertex);



	/** Mesh paint pixel shader */
	class TMeshPaintPixelShader : public FGlobalShader
	{
		DECLARE_SHADER_TYPE( TMeshPaintPixelShader, Global );
	public:

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) && !IsConsolePlatform(Parameters.Platform);
		}

		/** Default constructor. */
		TMeshPaintPixelShader() {}

		/** Initialization constructor. */
		TMeshPaintPixelShader( const ShaderMetaType::CompiledShaderInitializerType& Initializer )
			: FGlobalShader( Initializer )
		{
			CloneTextureParameter.Bind( Initializer.ParameterMap, TEXT( "s_CloneTexture" ) );
			CloneTextureParameterSampler.Bind( Initializer.ParameterMap, TEXT( "s_CloneTextureSampler" ));
			WorldToBrushMatrixParameter.Bind( Initializer.ParameterMap, TEXT( "c_WorldToBrushMatrix" ) );
			BrushMetricsParameter.Bind( Initializer.ParameterMap, TEXT( "c_BrushMetrics" ) );
			BrushStrengthParameter.Bind( Initializer.ParameterMap, TEXT( "c_BrushStrength" ) );
			BrushColorParameter.Bind( Initializer.ParameterMap, TEXT( "c_BrushColor" ) );
			ChannelFlagsParameter.Bind( Initializer.ParameterMap, TEXT( "c_ChannelFlags") );
			GenerateMaskFlagParameter.Bind( Initializer.ParameterMap, TEXT( "c_GenerateMaskFlag") );
			GammaParameter.Bind( Initializer.ParameterMap, TEXT( "c_Gamma" ) );
			PaintBrushTextureParameter.Bind(Initializer.ParameterMap, TEXT("s_PaintBrushTexture"));
			PaintBrushTextureParameterSampler.Bind(Initializer.ParameterMap, TEXT("s_PaintBrushTextureSampler"));
			RotationDirectionParameter.Bind(Initializer.ParameterMap, TEXT("RotationDirection"));
			RotationOffsetParameter.Bind(Initializer.ParameterMap, TEXT("RotationOffset"));
		}

		void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const float InGamma, const FMeshPaintShaderParameters& InShaderParams )
		{
			SetTextureParameter(
				BatchedParameters,
				CloneTextureParameter,
				CloneTextureParameterSampler,
				TStaticSamplerState< SF_Point, AM_Clamp, AM_Clamp, AM_Clamp >::GetRHI(),
				InShaderParams.CloneTexture->GetRenderTargetResource()->TextureRHI );

			SetShaderValue(BatchedParameters, WorldToBrushMatrixParameter, (FMatrix44f)InShaderParams.WorldToBrushMatrix );

			FVector4f BrushMetrics;
			BrushMetrics.X = InShaderParams.BrushRadius;
			BrushMetrics.Y = InShaderParams.BrushRadialFalloffRange;
			BrushMetrics.Z = InShaderParams.BrushDepth;
			BrushMetrics.W = InShaderParams.BrushDepthFalloffRange;
			SetShaderValue(BatchedParameters, BrushMetricsParameter, BrushMetrics );

			FVector4f BrushStrength4( InShaderParams.BrushStrength, 0.0f, 0.0f, 0.0f );
			SetShaderValue(BatchedParameters, BrushStrengthParameter, BrushStrength4 );

			SetShaderValue(BatchedParameters, BrushColorParameter, InShaderParams.BrushColor );

			FVector4f ChannelFlags;
			ChannelFlags.X = InShaderParams.RedChannelFlag;
			ChannelFlags.Y = InShaderParams.GreenChannelFlag;
			ChannelFlags.Z = InShaderParams.BlueChannelFlag;
			ChannelFlags.W = InShaderParams.AlphaChannelFlag;
			SetShaderValue(BatchedParameters, ChannelFlagsParameter, ChannelFlags );
			
			float MaskVal = InShaderParams.GenerateMaskFlag ? 1.0f : 0.0f;
			SetShaderValue(BatchedParameters, GenerateMaskFlagParameter, MaskVal );

			// @todo MeshPaint
			SetShaderValue(BatchedParameters, GammaParameter, InGamma );
			if (InShaderParams.PaintBrushTexture)
			{
				SetTextureParameter(
					BatchedParameters,
					PaintBrushTextureParameter,
					PaintBrushTextureParameterSampler,
					TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI(),
					InShaderParams.PaintBrushTexture->GetRenderTargetResource()->TextureRHI);
				SetShaderValue(BatchedParameters, RotationOffsetParameter, InShaderParams.PaintBrushRotationOffset);

				if (InShaderParams.bRotateBrushTowardsDirection)
				{
					SetShaderValue(BatchedParameters, RotationDirectionParameter, InShaderParams.PaintBrushDirectionVector);
				}
			}
		}

		class FMeshPaintUsePaintBrush : SHADER_PERMUTATION_BOOL("MESH_PAINT_USE_PAINTBRUSH");
		class FMeshPaintUseRotateTowardDirection : SHADER_PERMUTATION_BOOL("MESH_PAINT_ROTATE_TOWARD_DIRECTION");
		class FMeshPaintUseFillBucket : SHADER_PERMUTATION_BOOL("MESH_PAINT_USE_FILL_BUCKET");

		using FPermutationDomain = TShaderPermutationDomain<FMeshPaintUsePaintBrush, FMeshPaintUseRotateTowardDirection, FMeshPaintUseFillBucket>;

		LAYOUT_FIELD(FShaderResourceParameter, PaintBrushTextureParameter);
		LAYOUT_FIELD(FShaderResourceParameter, PaintBrushTextureParameterSampler);

		LAYOUT_FIELD(FShaderParameter, RotationDirectionParameter);
		LAYOUT_FIELD(FShaderParameter, RotationOffsetParameter);

	private:
		/** Texture that is a clone of the destination render target before we start drawing */
		LAYOUT_FIELD(FShaderResourceParameter, CloneTextureParameter);
		LAYOUT_FIELD(FShaderResourceParameter, CloneTextureParameterSampler);

		/** Brush -> World matrix */
		LAYOUT_FIELD(FShaderParameter, WorldToBrushMatrixParameter);

		/** Brush metrics: x = radius, y = falloff range, z = depth, w = depth falloff range */
		LAYOUT_FIELD(FShaderParameter, BrushMetricsParameter);

		/** Brush strength */
		LAYOUT_FIELD(FShaderParameter, BrushStrengthParameter);

		/** Brush color */
		LAYOUT_FIELD(FShaderParameter, BrushColorParameter);

		/** Flags that control paining individual channels: x = Red, y = Green, z = Blue, w = Alpha */
		LAYOUT_FIELD(FShaderParameter, ChannelFlagsParameter);
		
		/** Flag to control brush mask generation or paint blending */
		LAYOUT_FIELD(FShaderParameter, GenerateMaskFlagParameter);

		/** Gamma */
		// @todo MeshPaint: Remove this?
		LAYOUT_FIELD(FShaderParameter, GammaParameter);
	};


	IMPLEMENT_SHADER_TYPE( , TMeshPaintPixelShader, TEXT( "/Engine/Private/MeshPaintPixelShader.usf" ), TEXT( "Main" ), SF_Pixel );


	/** Mesh paint dilate vertex shader */
	class TMeshPaintDilateVertexShader : public FGlobalShader
	{
		DECLARE_SHADER_TYPE( TMeshPaintDilateVertexShader, Global );

	public:

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) && !IsConsolePlatform(Parameters.Platform);
		}

		/** Default constructor. */
		TMeshPaintDilateVertexShader() {}

		/** Initialization constructor. */
		TMeshPaintDilateVertexShader( const ShaderMetaType::CompiledShaderInitializerType& Initializer )
			: FGlobalShader( Initializer )
		{
			TransformParameter.Bind( Initializer.ParameterMap, TEXT( "c_Transform" ) );
		}

		void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FMatrix44f& InTransform )
		{
			SetShaderValue(BatchedParameters, TransformParameter, InTransform );
		}

	private:

		LAYOUT_FIELD(FShaderParameter, TransformParameter);
	};


	IMPLEMENT_SHADER_TYPE( , TMeshPaintDilateVertexShader, TEXT( "/Engine/Private/meshpaintdilatevertexshader.usf" ), TEXT( "Main" ), SF_Vertex );



	/** Mesh paint pixel shader */
	class TMeshPaintDilatePixelShader : public FGlobalShader
	{
		DECLARE_SHADER_TYPE( TMeshPaintDilatePixelShader, Global );

	public:

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) && !IsConsolePlatform(Parameters.Platform);
		}

		/** Default constructor. */
		TMeshPaintDilatePixelShader() {}

		/** Initialization constructor. */
		TMeshPaintDilatePixelShader( const ShaderMetaType::CompiledShaderInitializerType& Initializer )
			: FGlobalShader( Initializer )
		{
			Texture0Parameter.Bind( Initializer.ParameterMap, TEXT( "Texture0" ) );
			Texture0ParameterSampler.Bind( Initializer.ParameterMap, TEXT( "Texture0Sampler" ) );
			Texture1Parameter.Bind( Initializer.ParameterMap, TEXT( "Texture1") );
			Texture1ParameterSampler.Bind( Initializer.ParameterMap, TEXT( "Texture1Sampler") );
			Texture2Parameter.Bind( Initializer.ParameterMap, TEXT( "Texture2") );
			Texture2ParameterSampler.Bind( Initializer.ParameterMap, TEXT( "Texture2Sampler") );
			WidthPixelOffsetParameter.Bind( Initializer.ParameterMap, TEXT( "WidthPixelOffset") );
			HeightPixelOffsetParameter.Bind( Initializer.ParameterMap, TEXT( "HeightPixelOffset") );
			GammaParameter.Bind( Initializer.ParameterMap, TEXT( "Gamma" ) );
		}

		void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const float InGamma, const FMeshPaintDilateShaderParameters& InShaderParams )
		{
			SetTextureParameter(
				BatchedParameters,
				Texture0Parameter,
				Texture0ParameterSampler,
				TStaticSamplerState< SF_Point, AM_Clamp, AM_Clamp, AM_Clamp >::GetRHI(),
				InShaderParams.Texture0->GetRenderTargetResource()->TextureRHI );

			SetTextureParameter(
				BatchedParameters,
				Texture1Parameter,
				Texture1ParameterSampler,
				TStaticSamplerState< SF_Point, AM_Clamp, AM_Clamp, AM_Clamp >::GetRHI(),
				InShaderParams.Texture1->GetRenderTargetResource()->TextureRHI );

			SetTextureParameter(
				BatchedParameters,
				Texture2Parameter,
				Texture2ParameterSampler,
				TStaticSamplerState< SF_Point, AM_Clamp, AM_Clamp, AM_Clamp >::GetRHI(),
				InShaderParams.Texture2->GetRenderTargetResource()->TextureRHI );

			SetShaderValue(BatchedParameters, WidthPixelOffsetParameter, InShaderParams.WidthPixelOffset );
			SetShaderValue(BatchedParameters, HeightPixelOffsetParameter, InShaderParams.HeightPixelOffset );
			SetShaderValue(BatchedParameters, GammaParameter, InGamma );
		}


	private:

		/** Texture0 */
		LAYOUT_FIELD(FShaderResourceParameter, Texture0Parameter);
		LAYOUT_FIELD(FShaderResourceParameter, Texture0ParameterSampler);

		/** Texture1 */
		LAYOUT_FIELD(FShaderResourceParameter, Texture1Parameter);
		LAYOUT_FIELD(FShaderResourceParameter, Texture1ParameterSampler);

		/** Texture2 */
		LAYOUT_FIELD(FShaderResourceParameter, Texture2Parameter);
		LAYOUT_FIELD(FShaderResourceParameter, Texture2ParameterSampler);

		/** Pixel size width */
		LAYOUT_FIELD(FShaderParameter, WidthPixelOffsetParameter);
		
		/** Pixel size height */
		LAYOUT_FIELD(FShaderParameter, HeightPixelOffsetParameter);

		/** Gamma */
		// @todo MeshPaint: Remove this?
		LAYOUT_FIELD(FShaderParameter, GammaParameter);
	};


	IMPLEMENT_SHADER_TYPE( , TMeshPaintDilatePixelShader, TEXT( "/Engine/Private/meshpaintdilatepixelshader.usf" ), TEXT( "Main" ), SF_Pixel );


	/** Mesh paint vertex format */
	typedef FSimpleElementVertex FMeshPaintVertex;


	/** Mesh paint vertex declaration resource */
	typedef FSimpleElementVertexDeclaration FMeshPaintVertexDeclaration;


	/** Global mesh paint vertex declaration resource */
	TGlobalResource< FMeshPaintVertexDeclaration > GMeshPaintVertexDeclaration;



	typedef FSimpleElementVertex FMeshPaintDilateVertex;
	typedef FSimpleElementVertexDeclaration FMeshPaintDilateVertexDeclaration;
	TGlobalResource< FMeshPaintDilateVertexDeclaration > GMeshPaintDilateVertexDeclaration;


	/** Binds the mesh paint vertex and pixel shaders to the graphics device */
	void SetMeshPaintShaders(FRHICommandList& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit, ERHIFeatureLevel::Type InFeatureLevel, const FMatrix& InTransform,
										   const float InGamma,
										   const FMeshPaintShaderParameters& InShaderParams )
	{
		TShaderMapRef< TMeshPaintVertexShader > VertexShader(GetGlobalShaderMap(InFeatureLevel));

		TMeshPaintPixelShader::FPermutationDomain MeshPaintPermutationVector;
		MeshPaintPermutationVector.Set<TMeshPaintPixelShader::FMeshPaintUsePaintBrush>(InShaderParams.PaintBrushTexture != nullptr);
		MeshPaintPermutationVector.Set<TMeshPaintPixelShader::FMeshPaintUseRotateTowardDirection>(InShaderParams.bRotateBrushTowardsDirection);
		MeshPaintPermutationVector.Set<TMeshPaintPixelShader::FMeshPaintUseFillBucket>(InShaderParams.bUseFillBucket);
		TShaderMapRef<TMeshPaintPixelShader> PixelShader(GetGlobalShaderMap(InFeatureLevel), MeshPaintPermutationVector);

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GMeshPaintVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;

		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		// Set vertex shader parameters
		SetShaderParametersLegacyVS(RHICmdList, VertexShader, FMatrix44f(InTransform) );	// LWC_TODO: Precision loss
		
		// Set pixel shader parameters
		SetShaderParametersLegacyPS(RHICmdList, PixelShader, InGamma, InShaderParams );

		// @todo MeshPaint: Make sure blending/color writes are setup so we can write to ALPHA if needed!
	}

	/** Binds the mesh paint vertex and pixel shaders to the graphics device */
	void SetMeshPaintDilateShaders(FRHICommandList& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit, ERHIFeatureLevel::Type InFeatureLevel, const FMatrix& InTransform,
												 const float InGamma,
												 const FMeshPaintDilateShaderParameters& InShaderParams )
	{
		TShaderMapRef< TMeshPaintDilateVertexShader > VertexShader(GetGlobalShaderMap(InFeatureLevel));
		TShaderMapRef< TMeshPaintDilatePixelShader > PixelShader(GetGlobalShaderMap(InFeatureLevel));

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GMeshPaintDilateVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;

		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		// Set vertex shader parameters
		SetShaderParametersLegacyVS(RHICmdList, VertexShader, FMatrix44f(InTransform) );	// LWC_TODO: Precision loss

		// Set pixel shader parameters
		SetShaderParametersLegacyPS(RHICmdList, PixelShader, InGamma, InShaderParams );
	}

}

