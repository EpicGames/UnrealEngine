// Copyright Epic Games, Inc. All Rights Reserved.

#include "MiniCurveEditor.h"
#include "Widgets/Layout/SBorder.h"
#include "Styling/AppStyle.h"
#include "SCurveEditor.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Editor.h"

void SMiniCurveEditor::Construct(const FArguments& InArgs)
{
	ViewMinInput=0.f;
	ViewMaxInput=5.f;

	this->ChildSlot
	[
		SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
		.Padding(0.0f)
		[
			SAssignNew(TrackWidget, SCurveEditor)
			.ViewMinInput(this, &SMiniCurveEditor::GetViewMinInput)
			.ViewMaxInput(this, &SMiniCurveEditor::GetViewMaxInput)
			.TimelineLength(this, &SMiniCurveEditor::GetTimelineLength)
			.OnSetInputViewRange(this, &SMiniCurveEditor::SetInputViewRange)
			.HideUI(false)
			.AlwaysDisplayColorCurves(true)
		]
	];

	check(TrackWidget.IsValid());
	TrackWidget->SetCurveOwner(InArgs._CurveOwner);

	WidgetWindow = InArgs._ParentWindow;

	bHasOwner = InArgs._OwnerObject != nullptr;

	// Only notify if the Owner is present - it's not when spawned from the DataTable editor, for example
	if (bHasOwner)
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->NotifyAssetOpened(InArgs._OwnerObject, this);
	}
}

SMiniCurveEditor::~SMiniCurveEditor()
{
	if (bHasOwner)
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->NotifyEditorClosed(this);
	}
}

float SMiniCurveEditor::GetTimelineLength() const
{
	return 0.f;
}

void SMiniCurveEditor::SetInputViewRange(float InViewMinInput, float InViewMaxInput)
{
	ViewMaxInput = InViewMaxInput;
	ViewMinInput = InViewMinInput;
}

FName SMiniCurveEditor::GetEditorName() const
{
	return FName("MiniCurveEditor");
}

void SMiniCurveEditor::FocusWindow(UObject* ObjectToFocusOn)
{	
	if(WidgetWindow.IsValid())
	{
		//WidgetWindow.Pin()->ShowWindow();
		WidgetWindow.Pin()->BringToFront(true);
	}
}

bool SMiniCurveEditor::CloseWindow()
{
	return CloseWindow(EAssetEditorCloseReason::AssetEditorHostClosed);
}

bool SMiniCurveEditor::CloseWindow(EAssetEditorCloseReason InCloseReason)
{
	if(WidgetWindow.IsValid())
	{
		WidgetWindow.Pin()->RequestDestroyWindow();
	}

	return true;
}

TSharedPtr<class FTabManager> SMiniCurveEditor::GetAssociatedTabManager()
{
	//@TODO: This editor should probably derive from FAssetEditorToolkit instead!
	return TSharedPtr<class FTabManager>();
}

double SMiniCurveEditor::GetLastActivationTime()
{
	//@TODO: This editor should probably derive from FAssetEditorToolkit instead!
	return 0.0;
}

void SMiniCurveEditor::RemoveEditingAsset(UObject* Asset)
{
	//@TODO: This editor should probably derive from FAssetEditorToolkit instead!
}
