// Copyright Epic Games, Inc. All Rights Reserved.


#include "MouseDeltaTracker.h"
#include "EngineDefines.h"
#include "SceneView.h"
#include "EditorViewportClient.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Editor.h"
#include "EditorDragTools.h"
#include "SnappingUtils.h"
#include "UnrealWidget.h"
#include "EditorInteractiveGizmoManager.h"
#include "Misc/AxisDisplayInfo.h"
#include "EditorModeManager.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"
#include "ViewportInteractions/ViewportInteractionsUtilities.h"

#define LOCTEXT_NAMESPACE "MouseDeltaTracker"

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// FMouseDeltaTracker
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FMouseDeltaTracker::FMouseDeltaTracker()
	: Start( FVector::ZeroVector )
	, StartSnapped( FVector::ZeroVector )
	, StartScreen( FVector::ZeroVector )
	, End( FVector::ZeroVector )
	, EndSnapped( FVector::ZeroVector )
	, EndScreen( FVector::ZeroVector )
	, RawDelta( FVector::ZeroVector )
	, ReductionAmount( FVector::ZeroVector )
	, DragTool( NULL )
	, bHasAttemptedDragTool(false)
	, bUsedDragModifier(false)
	, bIsDeletingDragTool(false)
{
}

FMouseDeltaTracker::~FMouseDeltaTracker()
{
}

/**
 * Sets the current axis of the widget for the specified viewport.
 *
 * @param	InViewportClient		The viewport whose widget axis is to be set.
 */
void FMouseDeltaTracker::DetermineCurrentAxis(FEditorViewportClient* InViewportClient)
{
	const bool AltDown = InViewportClient->IsAltPressed();
	const bool ShiftDown = InViewportClient->IsShiftPressed();
	const bool ControlDown = InViewportClient->IsCtrlPressed();
	const bool LeftMouseButtonDown = InViewportClient->Viewport->KeyState(EKeys::LeftMouseButton);
	const bool RightMouseButtonDown = InViewportClient->Viewport->KeyState(EKeys::RightMouseButton);
	const bool MiddleMouseButtonDown = InViewportClient->Viewport->KeyState(EKeys::MiddleMouseButton);

	const bool bIsRotateObjectMode = InViewportClient->IsOrtho() && ControlDown && RightMouseButtonDown;
	// Ctrl + LEFT/RIGHT mouse button acts the same as dragging the most appropriate widget handle.
	if( (!InViewportClient->ShouldOrbitCamera() && bIsRotateObjectMode ) || ((!bIsRotateObjectMode && ControlDown && !AltDown ) &&
		(LeftMouseButtonDown || RightMouseButtonDown)) )
	{
		// Only try to pick an axis if we're not dragging by widget handle.
		if ( InViewportClient->GetCurrentWidgetAxis() == EAxisList::None )
		{
			switch( InViewportClient->GetWidgetMode() )
			{
				case UE::Widget::WM_Scale:
					// Non-uniform scale when shift is down, uniform when it is up
					if (ShiftDown)
					{
						switch( InViewportClient->ViewportType )
						{
							case LVT_Perspective:
								if( LeftMouseButtonDown && !RightMouseButtonDown )
								{
									InViewportClient->SetCurrentWidgetAxis( EAxisList::X );
								}
								else if( !LeftMouseButtonDown && RightMouseButtonDown )
								{
									InViewportClient->SetCurrentWidgetAxis( EAxisList::Y );
								}
								else if( LeftMouseButtonDown && RightMouseButtonDown )
								{
									InViewportClient->SetCurrentWidgetAxis( EAxisList::Z );
								}
								break;
							case LVT_OrthoXY:
							case LVT_OrthoNegativeXY:
								InViewportClient->SetCurrentWidgetAxis( EAxisList::XY );
								break;
							case LVT_OrthoXZ:
							case LVT_OrthoNegativeXZ:
								InViewportClient->SetCurrentWidgetAxis( EAxisList::XZ );
								break;
							case LVT_OrthoYZ:
							case LVT_OrthoNegativeYZ:
								InViewportClient->SetCurrentWidgetAxis( EAxisList::YZ );
								break;
							default:
								break;
						}
					}
					else
					{
						InViewportClient->SetCurrentWidgetAxis( EAxisList::XYZ );
					}
					break;

				case UE::Widget::WM_Translate:
				case UE::Widget::WM_TranslateRotateZ:
				case UE::Widget::WM_2D:
					switch( InViewportClient->ViewportType )
					{
						case LVT_Perspective:
							if( LeftMouseButtonDown && !RightMouseButtonDown )
							{
								InViewportClient->SetCurrentWidgetAxis( EAxisList::X );
							}
							else if( !LeftMouseButtonDown && RightMouseButtonDown )
							{
								InViewportClient->SetCurrentWidgetAxis( EAxisList::Y );
							}
							else if( LeftMouseButtonDown && RightMouseButtonDown )
							{
								InViewportClient->SetCurrentWidgetAxis( EAxisList::Z );
							}
							break;
						case LVT_OrthoXY:
						case LVT_OrthoNegativeXY:
							InViewportClient->SetCurrentWidgetAxis(EAxisList::XY);
							break;
						case LVT_OrthoXZ:
						case LVT_OrthoNegativeXZ:
							InViewportClient->SetCurrentWidgetAxis(EAxisList::XZ);
							break;
						case LVT_OrthoYZ:
						case LVT_OrthoNegativeYZ:
							InViewportClient->SetCurrentWidgetAxis(EAxisList::YZ);
							break;
						default:
							break;
					}
				break;

				case UE::Widget::WM_Rotate:
					switch( InViewportClient->ViewportType )
					{
						case LVT_Perspective:
							if (LeftMouseButtonDown && !RightMouseButtonDown)
							{
								InViewportClient->SetCurrentWidgetAxis(EAxisList::X);
							}
							else if (!LeftMouseButtonDown && RightMouseButtonDown)
							{
								InViewportClient->SetCurrentWidgetAxis(EAxisList::Y);
							}
							else if (LeftMouseButtonDown && RightMouseButtonDown)
							{
								InViewportClient->SetCurrentWidgetAxis(EAxisList::Z);
							}
							break;
						case LVT_OrthoXY:
						case LVT_OrthoNegativeXY:
							InViewportClient->SetCurrentWidgetAxis(EAxisList::Z);
							break;
						case LVT_OrthoXZ:
						case LVT_OrthoNegativeXZ:
							InViewportClient->SetCurrentWidgetAxis(EAxisList::Y);
							break;
						case LVT_OrthoYZ:
						case LVT_OrthoNegativeYZ:
							InViewportClient->SetCurrentWidgetAxis(EAxisList::X);
							break;
						default:
							break;
					}
				break;
			
				default:
					break;
			}
			//if we now have a widget axis we must have used a modifier to get it
			if( InViewportClient->GetCurrentWidgetAxis() != EAxisList::None )
			{
				bUsedDragModifier = true;				
			}
		}
	}
}

/**
 * Begin tracking at the specified location for the specified viewport.
 */
void FMouseDeltaTracker::StartTracking(FEditorViewportClient* InViewportClient, const int32 InX, const int32 InY, const FInputEventState& InInputState, bool bNudge, bool bResetDragToolState)
{
	DetermineCurrentAxis(InViewportClient);

	// Initialize widget axis (in case it hasn't been set by the hovered hit proxy)

	if (InViewportClient->Widget && InViewportClient->GetCurrentWidgetAxis() == EAxisList::None)
	{
		check(InViewportClient->Viewport);
		HHitProxy* HitProxy = InViewportClient->Viewport->GetHitProxy(InX, InY);
		if (HitProxy && HitProxy->IsA(HWidgetAxis::StaticGetType()))
		{
			EAxisList::Type ProxyAxis = ((HWidgetAxis*)HitProxy)->Axis;
			InViewportClient->SetCurrentWidgetAxis(ProxyAxis);
		}
	}

	const bool AltDown = InViewportClient->IsAltPressed();
	const bool ShiftDown = InViewportClient->IsShiftPressed();
	const bool ControlDown = InViewportClient->IsCtrlPressed();
	const bool LeftMouseButtonDown = InViewportClient->Viewport->KeyState(EKeys::LeftMouseButton);
	const bool RightMouseButtonDown = InViewportClient->Viewport->KeyState(EKeys::RightMouseButton);
	const bool MiddleMouseButtonDown = InViewportClient->Viewport->KeyState(EKeys::MiddleMouseButton);

	constexpr bool bAllowWidgetManipulationWithNewTRSGizmo = false;
	const bool bUsesNewGizmo = InViewportClient->GetModeTools()->UsesViewportITF();

	const bool bAllowWidgetManipulation = !bUsesNewGizmo || bAllowWidgetManipulationWithNewTRSGizmo;

	bool bIsDragging = ((ControlDown || ShiftDown) && (LeftMouseButtonDown || RightMouseButtonDown || MiddleMouseButtonDown)) ||
		(InViewportClient->GetCurrentWidgetAxis() != EAxisList::None) || bNudge;

	// Update bWidgetAxisControlledByDrag since we now know that we have begun dragging an object with the mouse.
	if ( bIsDragging && bAllowWidgetManipulation )
	{
		InViewportClient->bWidgetAxisControlledByDrag = true;
	}

	InViewportClient->TrackingStarted( InInputState, bIsDragging, bNudge );

	if (InViewportClient->Widget && bAllowWidgetManipulation)
	{
		InViewportClient->Widget->SetDragStartPosition(FVector2D(InX, InY));
		InViewportClient->Widget->SetDragging(bIsDragging);
		if (InViewportClient->GetWidgetMode() == UE::Widget::WM_Rotate)
		{
			InViewportClient->Invalidate();
		}
	}

	// Clear bool that tracks whether AddDelta has been called
	bHasReceivedAddDelta = false;

	if( bResetDragToolState )
	{
		bHasAttemptedDragTool = false;
	}


	ensure( !DragTool.IsValid() );

	StartSnapped = Start = StartScreen = FVector( InX, InY, 0 );
	RawDelta = FVector::ZeroVector;
	TrackingWidgetMode = InViewportClient->GetWidgetMode();

	// No drag tool is active, so handle snapping.
	switch( TrackingWidgetMode )
	{
		case UE::Widget::WM_Translate:
			FSnappingUtils::SnapPointToGrid( StartSnapped, FVector(GEditor->GetGridSize(),GEditor->GetGridSize(),GEditor->GetGridSize()) );
			break;

		case UE::Widget::WM_Scale:
			FSnappingUtils::SnapScale( StartSnapped, FVector(GEditor->GetGridSize(),GEditor->GetGridSize(),GEditor->GetGridSize()) );
			break;

		case UE::Widget::WM_Rotate:
		{
			FRotator Rotation( StartSnapped.X, StartSnapped.Y, StartSnapped.Z );
			FSnappingUtils::SnapRotatorToGrid( Rotation );
			StartSnapped = FVector( Rotation.Pitch, Rotation.Yaw, Rotation.Roll );
		}
		break;
		case UE::Widget::WM_TranslateRotateZ:
		case UE::Widget::WM_2D:
			FSnappingUtils::SnapPointToGrid( StartSnapped, FVector(GEditor->GetGridSize(),GEditor->GetGridSize(),GEditor->GetGridSize()) );
			break;

		default:
			break;
	}

	// Clear any snapping helpers on new movement
	const bool bClearImmediatley = true;
	FSnappingUtils::ClearSnappingHelpers( bClearImmediatley );

	End = EndScreen = Start;
	EndSnapped = StartSnapped;

	bExternalMovement = false;	//no external movement has occurred yet.

	if (bAllowWidgetManipulation)
	{
		InViewportClient->Widget->ResetDeltaRotation();	
	}
}

/**
 * Called when a mouse button has been released.  If there are no other
 * mouse buttons being held down, the internal information is reset.
 */
bool FMouseDeltaTracker::EndTracking(FEditorViewportClient* InViewportClient)
{
	DetermineCurrentAxis( InViewportClient );

	if (InViewportClient->Widget)
	{
		InViewportClient->Widget->SetDragging(false);
	}

	InViewportClient->TrackingStopped();

	InViewportClient->Widget->ResetDeltaRotation();

	Start = StartSnapped = StartScreen = End = EndSnapped = EndScreen = RawDelta = ReductionAmount = FVector::ZeroVector;

	if (!bIsDeletingDragTool)
	{
		// Ending the drag tool may pop up a modal dialog which can cause unwanted reentrancy - protect against this.
		TGuardValue<bool> RecursionGuard(bIsDeletingDragTool, true);

		// Delete the drag tool if one exists.
		if (DragTool.IsValid())
		{
			if (DragTool->IsDragging())
			{
				DragTool->EndDrag();
			}
			DragTool.Reset();
			return false;
		}
	}

	// Do not fade snapping indicators over time if this viewport is not real time
	bool bClearImmediately = !InViewportClient->IsRealtime();
	FSnappingUtils::ClearSnappingHelpers( bClearImmediately );
	return true;
}

void FMouseDeltaTracker::ConditionalBeginUsingDragTool( FEditorViewportClient* InViewportClient )
{
	if (InViewportClient->GetModeTools()->UsesViewportITF())
	{
		return;
	}

	const bool LeftMouseButtonDown = InViewportClient->Viewport->KeyState(EKeys::LeftMouseButton);
	const bool RightMouseButtonDown = InViewportClient->Viewport->KeyState(EKeys::RightMouseButton);
	const bool MiddleMouseButtonDown = InViewportClient->Viewport->KeyState(EKeys::MiddleMouseButton);
	const bool bAltDown = InViewportClient->IsAltPressed();
	const bool bShiftDown = InViewportClient->IsShiftPressed();
	const bool bControlDown = InViewportClient->IsCtrlPressed();

	// Has there been enough mouse movement to begin using a drag tool. We don't want to start using a tool for clicks(could have very small mouse movements)
	bool bEnoughMouseMovement = GetRawDelta().SizeSquared() > MOUSE_CLICK_DRAG_DELTA;

	if( bEnoughMouseMovement )
	{
		const bool bCanDrag = !DragTool.IsValid() && !RightMouseButtonDown && InViewportClient->CanUseDragTool();

		if (bCanDrag && !bHasAttemptedDragTool)
		{

			// Create a drag tool.
			if (!(bAltDown + bShiftDown) && bControlDown && MiddleMouseButtonDown && !LeftMouseButtonDown && !RightMouseButtonDown)
			{
				DragTool = InViewportClient->MakeDragTool(EDragTool::ViewportChange);
			}
			else
			{
				if (InViewportClient->IsOrtho())
				{
					if (LeftMouseButtonDown)
					{
						DragTool = InViewportClient->MakeDragTool(EDragTool::BoxSelect);
					}
					else if (!(bControlDown + bAltDown + bShiftDown) && MiddleMouseButtonDown)
					{
						DragTool = InViewportClient->MakeDragTool(EDragTool::Measure);
					}
				}
				else
				{
					if (LeftMouseButtonDown && bControlDown && bAltDown)
					{
						DragTool = InViewportClient->MakeDragTool(EDragTool::FrustumSelect);
					}
				}
			}

			if (DragTool.IsValid())
			{
				DragTool->StartDrag(InViewportClient, GEditor->ClickLocation, FVector2D(StartScreen));
			}
		}

		// Can not attempt to use a drag tool the rest of this tracking session
		bHasAttemptedDragTool = true;
	}
}

/**
 * Adds delta movement into the tracker.
 */
void FMouseDeltaTracker::AddDelta(FEditorViewportClient* InViewportClient, FKey InKey, const int32 InDelta, bool InNudge)
{
	const bool LeftMouseButtonDown = InViewportClient->Viewport->KeyState(EKeys::LeftMouseButton);
	const bool RightMouseButtonDown = InViewportClient->Viewport->KeyState(EKeys::RightMouseButton);
	const bool MiddleMouseButtonDown = InViewportClient->Viewport->KeyState(EKeys::MiddleMouseButton);
	const bool bAltDown = InViewportClient->IsAltPressed();
	const bool bShiftDown = InViewportClient->IsShiftPressed();
	const bool bControlDown = InViewportClient->IsCtrlPressed();

	if( !LeftMouseButtonDown && !MiddleMouseButtonDown && !RightMouseButtonDown && !InNudge )
	{
		return;
	}

	// Accumulate raw delta
	RawDelta += FVector(InKey == EKeys::MouseX ? InDelta : 0,
						InKey == EKeys::MouseY ? InDelta : 0,
						0);

	// Note that AddDelta has been called since StartTracking
	bHasReceivedAddDelta = true;

	// If we are using a drag tool, the widget isn't involved so set it to having no active axis.  This
	// means we will get unmodified mouse movement returned to us by other functions.

	const EAxisList::Type SaveAxis = InViewportClient->GetCurrentWidgetAxis();

	// If the user isn't dragging with the left mouse button, clear out the axis 
	// as the widget only responds to the left mouse button.
	//
	// We allow an exception for dragging with the left and/or right mouse button while holding control
	// as that simulates moving objects with the gizmo
	//
	// We also allow the exception of the middle mouse button when Alt is pressed, or when the current axis is the pivot centre, as it 
	// allows movement of only the pivot
	const bool bIsOrthoObjectRotation = bControlDown && InViewportClient->IsOrtho();
	const bool bUsingDragTool = UsingDragTool();
	const bool bUsingAxis = !bUsingDragTool && (LeftMouseButtonDown || (bAltDown && MiddleMouseButtonDown) || (SaveAxis == EAxisList::Screen && MiddleMouseButtonDown) || ((bIsOrthoObjectRotation || bControlDown) && RightMouseButtonDown));

	ConditionalBeginUsingDragTool( InViewportClient );

	if( bUsingDragTool || !InViewportClient->IsTracking() || !bUsingAxis )
	{
		InViewportClient->SetCurrentWidgetAxis( EAxisList::None );
	}

	FVector Wk = InViewportClient->TranslateDelta( InKey, static_cast<float>(InDelta), InNudge );

	EndScreen += Wk;

	if( InViewportClient->GetCurrentWidgetAxis() != EAxisList::None )
	{
		// Affect input delta by the camera speed

		UE::Widget::EWidgetMode WidgetMode = InViewportClient->GetWidgetMode();
		bool bIsRotation = (WidgetMode == UE::Widget::WM_Rotate) 
			|| ( ( WidgetMode == UE::Widget::WM_TranslateRotateZ ) && ( InViewportClient->GetCurrentWidgetAxis() == EAxisList::ZRotation ) )
			|| ( ( WidgetMode == UE::Widget::WM_2D) && (InViewportClient->GetCurrentWidgetAxis() == EAxisList::Rotate2D ) );
		if (bIsRotation)
		{
			Wk *= GetDefault<ULevelEditorViewportSettings>()->MouseSensitivty;
		}
		else if( WidgetMode == UE::Widget::WM_Scale && !GEditor->UsePercentageBasedScaling() )
		{
			const float ScaleSpeedMultipler = 0.01f;
			Wk *= ScaleSpeedMultipler;
		}

		// Make rotations occur at the same speed, regardless of ortho zoom

		if( InViewportClient->IsOrtho() )
		{
			if (bIsRotation)
			{
				double Scale = 1.0;

				if( InViewportClient->IsOrtho() )
				{
					Scale = DEFAULT_ORTHOZOOM / InViewportClient->GetOrthoZoom();
				}

				Wk *= Scale;
			}
		}
		//if Absolute Translation, and not just moving the camera around
		else if (InViewportClient->IsUsingAbsoluteTranslation(false))
		{
			// Compute a view.
			FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues( 
				InViewportClient->Viewport, 
				InViewportClient->GetScene(),
				InViewportClient->EngineShowFlags )
				.SetRealtimeUpdate( InViewportClient->IsRealtime() ));

			FSceneView* View = InViewportClient->CalcSceneView( &ViewFamily );

			//calculate mouse position
			check(InViewportClient->Viewport);
			FVector2D MousePosition(InViewportClient->Viewport->GetMouseX(), InViewportClient->Viewport->GetMouseY());
			FVector WidgetPosition = InViewportClient->GetWidgetLocation();

			FRotator TempRot;
			FVector TempScale;
			InViewportClient->Widget->AbsoluteTranslationConvertMouseMovementToAxisMovement(View, InViewportClient, WidgetPosition, MousePosition, Wk, TempRot, TempScale );
		}
	}

	End += Wk;
	EndSnapped = End;

	
	if( UsingDragTool() )
	{
		FVector Drag = Wk;
		if( DragTool->bConvertDelta )
		{
			FRotator Rot;
			InViewportClient->ConvertMovementToDragRot( Wk, Drag, Rot );
		}

		if ( InViewportClient->IsPerspective() )
		{
			DragTool->AddDelta(Wk);
		}
		else
		{
			DragTool->AddDelta( Drag );
		}

		InViewportClient->SetCurrentWidgetAxis( SaveAxis );
	}
	else
	{
		switch( InViewportClient->GetWidgetMode() )
		{
			case UE::Widget::WM_Translate:
				FSnappingUtils::SnapPointToGrid( EndSnapped, FVector(GEditor->GetGridSize(),GEditor->GetGridSize(),GEditor->GetGridSize()) );
				break;

			case UE::Widget::WM_Scale:
				FSnappingUtils::SnapScale( EndSnapped, FVector(GEditor->GetGridSize(),GEditor->GetGridSize(),GEditor->GetGridSize()) );
				break;

			case UE::Widget::WM_Rotate:
			{
				FRotator Rotation( EndSnapped.X, EndSnapped.Y, EndSnapped.Z );
				FSnappingUtils::SnapRotatorToGrid( Rotation );
				EndSnapped = FVector( Rotation.Pitch, Rotation.Yaw, Rotation.Roll );
			}
			break;
			case UE::Widget::WM_TranslateRotateZ:
			case UE::Widget::WM_2D:
			{
				if (InViewportClient->GetCurrentWidgetAxis() == EAxisList::Rotate2D)
				{
					FRotator Rotation( EndSnapped.X, EndSnapped.Y, EndSnapped.Z );
					FSnappingUtils::SnapRotatorToGrid( Rotation );
					EndSnapped = FVector( Rotation.Pitch, Rotation.Yaw, Rotation.Roll );
				}
				else
				{
					//translation (either xy plane or z)
					FSnappingUtils::SnapPointToGrid( EndSnapped, FVector(GEditor->GetGridSize(),GEditor->GetGridSize(),GEditor->GetGridSize()) );
				}
			}

			default:
				break;
		}
	}
}

/**
 * Returns the raw mouse delta, in pixels.
 */
const FVector FMouseDeltaTracker::GetRawDelta() const
{
	return RawDelta;
}

/**
 * Returns the current delta.
 */
const FVector FMouseDeltaTracker::GetDelta() const
{
	const FVector Delta( End - Start );
	return Delta;
}

/**
 * Returns the current snapped delta.
 */
const FVector FMouseDeltaTracker::GetDeltaSnapped() const
{
	const FVector SnappedDelta( EndSnapped - StartSnapped );
	return SnappedDelta;
}

/**
* Returns the absolute delta since dragging started.
*/
const FVector FMouseDeltaTracker::GetAbsoluteDelta() const
{
	const FVector Delta( End - Start + ReductionAmount );
	return Delta;
}

/**
* Returns the absolute snapped delta since dragging started. 
*/
const FVector FMouseDeltaTracker::GetAbsoluteDeltaSnapped() const
{
	const FVector SnappedDelta( EndSnapped - StartSnapped + ReductionAmount );
	return SnappedDelta;
}

/**
 * Returns the screen space delta since dragging started.
 */
const FVector FMouseDeltaTracker::GetScreenDelta() const
{
	const FVector Delta( EndScreen - StartScreen );
	return Delta;
}

/**
 * Converts the delta movement to drag/rotation/scale based on the viewport type or widget axis
 */
void FMouseDeltaTracker::ConvertMovementDeltaToDragRot(FSceneView* InView, FEditorViewportClient* InViewportClient, FVector& InOutDragDelta, FVector& OutDrag, FRotator& OutRotation, FVector& OutScale) const
{
	OutDrag = FVector::ZeroVector;
	OutRotation = FRotator::ZeroRotator;
	OutScale = FVector::ZeroVector;

	if( InViewportClient->GetCurrentWidgetAxis() != EAxisList::None )
	{
		InViewportClient->Widget->ConvertMouseMovementToAxisMovement( InView, InViewportClient, bUsedDragModifier, InOutDragDelta, OutDrag, OutRotation, OutScale );
	}
	else
	{
		InViewportClient->ConvertMovementToDragRot( InOutDragDelta, OutDrag, OutRotation );
	}
}

/**
 * Absolute Translation conversion from mouse position on the screen to widget axis movement/rotation.
 */
void FMouseDeltaTracker::AbsoluteTranslationConvertMouseToDragRot(FSceneView* InView, FEditorViewportClient* InViewportClient,FVector& OutDrag, FRotator& OutRotation, FVector& OutScale ) const
{
	OutDrag = FVector::ZeroVector;
	OutRotation = FRotator::ZeroRotator;
	OutScale = FVector::ZeroVector;

	check ( InViewportClient->GetCurrentWidgetAxis() != EAxisList::None );

	//calculate mouse position
	check(InViewportClient->Viewport);
	FVector2D MousePosition(InViewportClient->Viewport->GetMouseX(), InViewportClient->Viewport->GetMouseY());

	InViewportClient->Widget->AbsoluteTranslationConvertMouseMovementToAxisMovement(InView, InViewportClient, InViewportClient->GetWidgetLocation(), MousePosition, OutDrag, OutRotation, OutScale );
}

/**
 * Subtracts the specified value from End and EndSnapped.
 */
void FMouseDeltaTracker::ReduceBy(const FVector& In)
{
	End -= In;
	EndSnapped -= In;
	ReductionAmount += In;
}

/**
 * @return		true if a drag tool is being used by the tracker, false otherwise.
 */
bool FMouseDeltaTracker::UsingDragTool() const
{
	return DragTool.IsValid() && DragTool->IsDragging() ;
}

/**
 * Renders the drag tool.  Does nothing if no drag tool exists.
 */
void FMouseDeltaTracker::Render3DDragTool(const FSceneView* View,FPrimitiveDrawInterface* PDI)
{
	if ( DragTool.IsValid() )
	{
		DragTool->Render3D( View, PDI );
	}
}

/**
 * Renders the drag tool.  Does nothing if no drag tool exists.
 */
void FMouseDeltaTracker::RenderDragTool(const FSceneView* View,FCanvas* Canvas)
{
	if ( DragTool.IsValid() )
	{
		DragTool->Render( View, Canvas );
	}
}

const FVector FMouseDeltaTracker::GetDragStartPos() const
{
	return Start;
}

const bool FMouseDeltaTracker::GetUsedDragModifier() const
{
	return bUsedDragModifier;
}

void FMouseDeltaTracker::ResetUsedDragModifier()
{
	bUsedDragModifier = false;	
}

#undef LOCTEXT_NAMESPACE
