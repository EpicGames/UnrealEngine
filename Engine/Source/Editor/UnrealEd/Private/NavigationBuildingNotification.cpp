// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationBuildingNotification.h"
#include "NavigationSystem.h"
#include "Editor/EditorEngine.h"
#include "Settings/LevelEditorMiscSettings.h"
#include "EngineGlobals.h"
#include "EditorViewportClient.h"
#include "Kismet2/DebuggerCommands.h"
#include "EditorBuildUtils.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

void FNavigationBuildingNotificationImpl::BuildStarted()
{
	UEditorEngine* const EEngine = Cast<UEditorEngine>(GEngine);
	const bool bUserRequestedBuild = (EEngine != NULL && (FEditorBuildUtils::IsBuildingNavigationFromUserRequest()));
	LastEnableTime = FPlatformTime::Seconds();

	if (NavigationBuildNotificationPtr.IsValid())
	{
		if (!bUserRequestedBuild)
		{
			return;
		}
		else
		{
			NavigationBuildNotificationPtr.Pin()->ExpireAndFadeout();
		}
	}

	if ( NavigationBuiltCompleteNotification.IsValid() )
	{
		NavigationBuiltCompleteNotification.Pin()->ExpireAndFadeout();
	}

	FNotificationInfo Info( GetNotificationText() );
	Info.bFireAndForget = false;
	Info.FadeOutDuration = 0.0f;
	Info.ExpireDuration = 0.0f;

	Info.ButtonDetails.Add(FNotificationButtonInfo(NSLOCTEXT("NavigationBuild", "Cancel", "Cancel"), FText::GetEmpty(), FSimpleDelegate::CreateRaw(this, &FNavigationBuildingNotificationImpl::CancelBuild)));

	NavigationBuildNotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
	if (NavigationBuildNotificationPtr.IsValid())
	{
		NavigationBuildNotificationPtr.Pin()->SetCompletionState(SNotificationItem::CS_Pending);
	}
}

void FNavigationBuildingNotificationImpl::CancelBuild()
{
	UEditorEngine* const EEngine = Cast<UEditorEngine>(GEngine);
	if (EEngine != nullptr)
	{
		FWorldContext &EditorContext = EEngine->GetEditorWorldContext();
		UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(EditorContext.World());
		if (NavSys)
		{
			NavSys->CancelBuild();
		}
	}
}

void FNavigationBuildingNotificationImpl::BuildFinished()
{
	// Finished all requests! Notify the UI.
	UEditorEngine* const EEngine = Cast<UEditorEngine>(GEngine);
	TSharedPtr<SNotificationItem> NotificationItem = NavigationBuildNotificationPtr.Pin();
	if (NotificationItem.IsValid())
	{
		NotificationItem->SetText( NSLOCTEXT("NavigationBuild", "NavigationBuildingComplete", "Navigation building done!") );
		NotificationItem->SetCompletionState(SNotificationItem::CS_Success);
		NotificationItem->ExpireAndFadeout();

		if (EEngine)
		{
			// request update for all viewports with disabled real time but with visible navmesh
			for(FEditorViewportClient* Viewport : EEngine->GetAllViewportClients())
			{
				if (Viewport && Viewport->IsRealtime() == false && Viewport->EngineShowFlags.Navigation)
				{
					Viewport->bNeedsRedraw = true;
					EEngine->UpdateSingleViewportClient(Viewport, true, false);
				}
			}
		}

		NavigationBuildNotificationPtr.Reset();
	}

	if (EEngine != NULL && (FEditorBuildUtils::IsBuildingNavigationFromUserRequest()))
	{
		// remove existing item, if any
		ClearCompleteNotification();

		FNotificationInfo Info( NSLOCTEXT("NavigationBuild", "NavigationBuildDoneMessage", "Navigation building completed.") );
		Info.bFireAndForget = true;
		Info.bUseThrobber = false;
		Info.FadeOutDuration = 3.0f;
		Info.ExpireDuration = 3.0f;

		NavigationBuiltCompleteNotification = FSlateNotificationManager::Get().AddNotification(Info);
		if (NavigationBuiltCompleteNotification.IsValid())
		{
			NavigationBuiltCompleteNotification.Pin()->SetCompletionState(SNotificationItem::CS_Pending);
		}
	}

	FEditorBuildUtils::PathBuildingFinished();
}

void FNavigationBuildingNotificationImpl::ClearCompleteNotification()
{
	if ( NavigationBuiltCompleteNotification.IsValid() )
	{
		NavigationBuiltCompleteNotification.Pin()->SetCompletionState(SNotificationItem::CS_None);
		NavigationBuiltCompleteNotification.Pin()->ExpireAndFadeout();
		NavigationBuiltCompleteNotification.Reset();
	}
}

FText FNavigationBuildingNotificationImpl::GetNotificationText() const
{
	int32 RemainingTasks = 0;
	UEditorEngine* const EEngine = Cast<UEditorEngine>(GEngine);
	if (EEngine)
	{
		FWorldContext &EditorContext = EEngine->GetEditorWorldContext();
		UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(EditorContext.World());
		if (NavSys)
		{
			RemainingTasks = NavSys->GetNumRemainingBuildTasks();
		}
	}
		
	FFormatNamedArguments Args;
	Args.Add(TEXT("RemainingTasks"), FText::AsNumber(RemainingTasks));
	return FText::Format(NSLOCTEXT("NavigationBuild", "NavigationBuildingInProgress", "Building Navigation ({RemainingTasks})"), Args);
}


void FNavigationBuildingNotificationImpl::Tick(float DeltaTime)
{
	if (FPlayWorldCommandCallbacks::IsInPIE_AndRunning())
	{
		return;
	}

	UEditorEngine* const EEngine = Cast<UEditorEngine>(GEngine);
	if (EEngine != NULL)
	{
		const bool bUserRequestedBuild = FEditorBuildUtils::IsBuildingNavigationFromUserRequest();
		FWorldContext &EditorContext = EEngine->GetEditorWorldContext();
		
		UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(EditorContext.World());
		const bool bBuildInProgress = NavSys != nullptr
			&& (NavSys->GetIsAutoUpdateEnabled() ? NavSys->IsNavigationBuildInProgress() : NavSys->GetNumRunningBuildTasks() > 0)
			&& NavSys->GetNumRemainingBuildTasks() > 0;

		if (!bPreviouslyDetectedBuild && bBuildInProgress)
		{
			TimeOfStartedBuild = FPlatformTime::Seconds();
		}
		else if(bPreviouslyDetectedBuild && !bBuildInProgress)
		{
			TimeOfStoppedBuild = FPlatformTime::Seconds();
		}

		if( bBuildInProgress && bPreviouslyDetectedBuild && 
			!NavigationBuildNotificationPtr.IsValid() && 
			(bUserRequestedBuild || (!bUserRequestedBuild && (FPlatformTime::Seconds() - TimeOfStartedBuild) > 0.1))
		) 
		{
			BuildStarted();
		}
		// Disable the notification when we are no longer doing an async compile
		else if (!bBuildInProgress && !bPreviouslyDetectedBuild && (FPlatformTime::Seconds() - TimeOfStoppedBuild) > 1.0)
		{
			BuildFinished();
		}
		else if (bBuildInProgress && NavigationBuildNotificationPtr.IsValid())
		{
			NavigationBuildNotificationPtr.Pin()->SetText(GetNotificationText());
		}

		bPreviouslyDetectedBuild = bBuildInProgress;
	}
}

TStatId FNavigationBuildingNotificationImpl::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FNavigationBuildingNotificationImpl, STATGROUP_Tickables);
}

