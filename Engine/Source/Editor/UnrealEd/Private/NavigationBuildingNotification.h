// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Stats/Stats.h"
#include "TickableEditorObject.h"

/** Notification class for asynchronous shader compiling. */
class FNavigationBuildingNotificationImpl
	: public FTickableEditorObject
{

public:

	FNavigationBuildingNotificationImpl()
	{
		bPreviouslyDetectedBuild = false;
		LastEnableTime = 0;
	}

	/** Starts the notification. */
	void BuildStarted();

	/** Ends the notification. */
	void BuildFinished();

protected:
	/** FTickableEditorObject interface */
	virtual void Tick(float DeltaTime) override;
	virtual ETickableTickType GetTickableTickType() const override
	{
		return ETickableTickType::Always;
	}
	virtual TStatId GetStatId() const override;

private:
	void ClearCompleteNotification();
	FText GetNotificationText() const;

	void CancelBuild();

	bool bPreviouslyDetectedBuild;
	double TimeOfStartedBuild;
	double TimeOfStoppedBuild;

	/** Tracks the last time the notification was started, used to avoid spamming. */
	double LastEnableTime;

	TWeakPtr<SNotificationItem> NavigationBuiltCompleteNotification;

	/** The source code symbol query in progress message */
	TWeakPtr<SNotificationItem> NavigationBuildNotificationPtr;
};

