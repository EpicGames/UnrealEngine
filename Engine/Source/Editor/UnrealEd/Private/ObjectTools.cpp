// Copyright Epic Games, Inc. All Rights Reserved.


#include "ObjectTools.h"
#include "Algo/RemoveIf.h"
#include "Engine/World.h"
#include "Engine/Level.h"
#include "UObject/UnrealType.h"
#include "CollectionManagerModule.h"
#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"
#include "Engine/Blueprint.h"
#include "Exporters/Exporter.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/MessageDialog.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/App.h"
#include "Misc/FileHelper.h"
#include "Misc/NamePermissionList.h"
#include "Misc/StringOutputDevice.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "Serialization/FindReferencersArchive.h"
#include "Serialization/ArchiveReferenceMarker.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SWindow.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "RHI.h"
#include "Materials/MaterialInterface.h"
#include "RenderingThread.h"
#include "Materials/Material.h"
#include "MaterialShared.h"
#include "CanvasTypes.h"
#include "Engine/Brush.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "ISourceControlOperation.h"
#include "SourceControlOperations.h"
#include "ISourceControlModule.h"
#include "Engine/SkeletalMesh.h"
#include "Editor/UnrealEdEngine.h"
#include "TextureResource.h"
#include "Engine/Texture.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "ThumbnailRendering/TextureThumbnailRenderer.h"
#include "ThumbnailExternalCache.h"
#include "Engine/StaticMesh.h"
#include "Factories/Factory.h"
#include "AssetToolsModule.h"
#include "Sound/SoundWave.h"
#include "GameFramework/Volume.h"
#include "UObject/MetaData.h"
#include "Serialization/ArchiveReplaceObjectRef.h"
#include "Serialization/ArchiveReplaceObjectAndStructPropertyRef.h"
#include "GameFramework/WorldSettings.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "Engine/Selection.h"
#include "Engine/TextureRenderTarget2D.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Animation/MorphTarget.h"
#include "Editor.h"
#include "Editor/Transactor.h"
#include "EditorDirectories.h"
#include "FileHelpers.h"
#include "Dialogs/Dialogs.h"
#include "Dialog/SMessageDialog.h"
#include "UnrealEdGlobals.h"
#include "PackageTools.h"
#include "Internationalization/TextPackageNamespaceUtil.h"
#include "Framework/Application/SlateApplication.h"
#include "ILocalizedAssetTools.h"

#include "BusyCursor.h"
#include "Dialogs/DlgMoveAssets.h"
#include "Dialogs/DlgReferenceTree.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetDeleteModel.h"
#include "Dialogs/SPrivateAssetsDialog.h"
#include "Dialogs/SDeleteAssetsDialog.h"
#include "AudioDevice.h"
#include "ReferencedAssetsUtils.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "PackagesDialog.h"
#include "PropertyEditorModule.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/KismetReinstanceUtilities.h"
#include "BlueprintCompilationManager.h"
#include "PackageHelperFunctions.h"
#include "EditorLevelUtils.h"
#include "DesktopPlatformModule.h"
#include "Engine/LevelStreaming.h"
#include "LevelUtils.h"
#include "ContentStreaming.h"
#include "ComponentRecreateRenderStateContext.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Layers/LayersSubsystem.h"
#include "Engine/SCS_Node.h"
#include "ShaderCompiler.h"
#include "Templates/UniquePtr.h"
#include "Engine/MapBuildDataRegistry.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "UObject/PropertyBagRepository.h"
#include "UObject/ReferencerFinder.h"
#include "Containers/Set.h"
#include "UObject/StrongObjectPtr.h"
#include "Logging/LogMacros.h"
#include "UncontrolledChangelistsModule.h"
#include "AssetCompilingManager.h"
#include "ObjectEditorUtils.h"
#include "Settings/EditorProjectSettings.h"
#include "Settings/EditorStyleSettings.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "Engine/DataTable.h"
#include "RHIGPUReadback.h"

DEFINE_LOG_CATEGORY_STATIC(LogObjectTools, Log, All);

#define LOCTEXT_NAMESPACE "ObjectTools"

static TAutoConsoleVariable<bool> CVarUseLegacyGetReferencersForDeletion(
	TEXT("Editor.UseLegacyGetReferencersForDeletion"),
	false,
	TEXT("Choose the algorithm to be used when detecting referencers of any assets/objects being deleted.\n\n")
	TEXT("0: Use the most optimized version (default)\n")
	TEXT("1: Use the slower legacy version (for debug/comparison)"),
	ECVF_Default
	);

// This function should ONLY be needed by ConsolidateObjects and ForceDeleteObjects
// Use anywhere else could be dangerous as this involves a map transition and GC
void ReloadEditorWorldForReferenceReplacementIfNecessary(TArray< TWeakObjectPtr<UObject> >& InOutObjectsToReplace)
{
	// If we are force-deleting or consolidating the editor world, first transition to an empty map to prevent reference problems.
	// Then, re-load the world from disk to set it up for delete as an inactive world which isn't attached to the editor engine or other systems.
	UWorld* EditorWorld = GEditor->GetEditorWorldContext().World();

	// Also get the map build data, we'll need reqquire it after reloading the level because it will be gc'd when NewMap is called.
	UMapBuildDataRegistry* MapBuildData = EditorWorld->PersistentLevel->MapBuildData;

	// Remove the world from ObjectsToDelete since NewMap() will delete the object naturally
	int32 NumEntriesRemoved = InOutObjectsToReplace.Remove(EditorWorld);
	if (NumEntriesRemoved > 0)
	{
		bool bMapBuildDataRemoved = false;
		if (MapBuildData)
		{
			bMapBuildDataRemoved = InOutObjectsToReplace.Remove(MapBuildData) == 1;
		}

		const FString ObjectPath = EditorWorld->GetPathName();

		// Transition to a new map. This will invoke garbage collection and destroy the EditorWorld
		GEditor->NewMap();

		// Attempt to reload the editor world so we can make sure the file gets deleted and everything is handled normally.
		// It is okay for this to fail. If we could not reload the world, it is not on disk and is gone.
		UWorld* ReloadedEditorWorld = LoadObject<UWorld>(nullptr, *ObjectPath, nullptr, LOAD_Quiet | LOAD_NoWarn);
		if (ReloadedEditorWorld)
		{
			InOutObjectsToReplace.Add(ReloadedEditorWorld);

			if (bMapBuildDataRemoved && ReloadedEditorWorld->PersistentLevel->MapBuildData)
			{
				InOutObjectsToReplace.Add(ReloadedEditorWorld->PersistentLevel->MapBuildData);
			}
		}


	}
}

template <class T>
class FArchiveReplaceObjectRefDataTableRows : public FArchiveReplaceObjectRef<T>
{
public:

	FArchiveReplaceObjectRefDataTableRows(UObject* InSearchObject, const TMap<T*, T*>& InReplacementMap)
		: FArchiveReplaceObjectRef<T>(InSearchObject, InReplacementMap, EArchiveReplaceObjectFlags::DelayStart)
	{
		// Note: We intentionally add the 'DelayStart' flag above; otherwise the base class ctor will use the base archive type for serialization, and our overrides won't get called.
		FArchiveReplaceObjectRef<T>::SerializeSearchObject();
	}

	virtual void SerializeObject(UObject* ObjectToSerialize) override
	{
		UDataTable* DataTableObject = Cast<UDataTable>(ObjectToSerialize);

		if (DataTableObject)
		{
			if (DataTableObject->RowStruct != nullptr && DataTableObject->RowStruct->RefLink != nullptr)
			{
				for (const TPair<FName, uint8*>& Pair : DataTableObject->GetRowMap())
				{
					if (uint8* RowData = Pair.Value)
					{
						DataTableObject->RowStruct->SerializeItem(*this, RowData, nullptr);
					}
				}
			}
		}
		else
		{
			FArchiveReplaceObjectRef<T>::SerializeObject(ObjectToSerialize);
		}
	}
};

namespace ObjectTools
{
	static int32 MaxTimesToCheckSameObject = 3;
	static FAutoConsoleVariableRef CVarMaxTimesToCheckSameObject(TEXT("ObjectTools.MaxTimesToCheckSameObject"),
		MaxTimesToCheckSameObject,
		TEXT("Number of times to recurse on the same object when mapping property chains to objects."));

	static int32 MaxRecursionDepth = 4;
	static FAutoConsoleVariableRef CVarMaxRecursionDepth(TEXT("ObjectTools.MaxRecursionDepth"),
		MaxRecursionDepth,
		TEXT("How many times to recurse to find the object to search for"));


	/** Returns true if the specified object can be displayed in a content browser */
	bool IsObjectBrowsable( UObject* Obj )	// const
	{
		bool bIsSupported = false;

		// Check object prerequisites
		if (ensure(Obj) && Obj->IsAsset() )
		{
			UPackage* ObjectPackage = Obj->GetOutermost();
			if( ObjectPackage != NULL )
			{
				if( ObjectPackage != GetTransientPackage()
					&& (ObjectPackage->HasAnyPackageFlags(PKG_PlayInEditor) == false)
					&& IsValidChecked(Obj) 
					&& !Obj->IsPackageExternal())
				{
					bIsSupported = true;
				}
			}
		}

		return bIsSupported;
	}

	bool IsNonGCObject(UObject* Object)
	{
		FUObjectItem* ObjectItem = GUObjectArray.ObjectToObjectItem(Object);
		return
			ObjectItem->IsRootSet() ||
			ObjectItem->HasAnyFlags(EInternalObjectFlags_GarbageCollectionKeepFlags) ||
			(GARBAGE_COLLECTION_KEEPFLAGS != RF_NoFlags && Object->HasAnyFlags(GARBAGE_COLLECTION_KEEPFLAGS));
	}

	TSet<UObject*> FindObjectsRoots(TSet<UObject*>& InObjects)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FindObjectsRoots)

		TSet<UObject*> Roots;
		UTransactor* Transactor = GEditor ? ToRawPtr(GEditor->Trans) : nullptr;

		// Handle the objects themselves if they can't be GCed
		for (UObject* Object : InObjects)
		{
			if (IsNonGCObject(Object))
			{
				Roots.Add(Object);
			}
		}

		// We recursively grow the cluster of objects we need to find referencers on until no more referencers are found
		int32 LastObjectCount = 0;
		while (InObjects.Num() != LastObjectCount)
		{
			LastObjectCount = InObjects.Num();
			for (UObject* NewReferencer : FReferencerFinder::GetAllReferencers(InObjects, &InObjects, EReferencerFinderFlags::SkipWeakReferences))
			{
				// Exclude any pendingkill or garbage object from counting as referencers
				if (IsValid(NewReferencer))
				{
					// Stop walking the dependency chain when the transactor is the referencer
					if (Transactor == NewReferencer)
					{
						Roots.Add(Transactor);
					}
					else if (IsNonGCObject(NewReferencer))
					{
						Roots.Add(NewReferencer);
					}
					else
					{
						InObjects.Add(NewReferencer);
					}
				}
			}
		}

		return MoveTemp(Roots);
	}

	void GatherObjectReferencersForDeletion(UObject* InObject, bool& bOutIsReferenced, bool& bOutIsReferencedInMemoryByUndo, FReferencerInformationList* OutMemoryReferences, bool bInRequireReferencingProperties)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GatherObjectReferencersForDeletion)

		if (OutMemoryReferences)
		{
			OutMemoryReferences->ExternalReferences.Reset();
			OutMemoryReferences->InternalReferences.Reset();
		}

		FReferencerInformationList LocalReferences;
		FReferencerInformationList& References = OutMemoryReferences ? *OutMemoryReferences : LocalReferences;

		bOutIsReferenced = false;
		bOutIsReferencedInMemoryByUndo = false;
		
		if (!CVarUseLegacyGetReferencersForDeletion.GetValueOnAnyThread())
		{
			UTransactor* Transactor = GEditor ? ToRawPtr(GEditor->Trans) : nullptr;
			bool bIsGatheringPackageRef = InObject->IsA<UPackage>();

			// Get the cluster of objects that are going to be deleted
			TArray<UObject*> ObjectsToDelete;
			GetObjectsWithOuter(InObject, ObjectsToDelete);

			bool bIsReferencedInternally = false;

			TSet<UObject*> InternalReferences;
			// The old behavior of GatherObjectReferencersForDeletion will find anything that prevents 
			// InObject from being garbage collected, including internal sub objects.
			for (UObject* ObjectToDelete : ObjectsToDelete)
			{
				if ((ObjectToDelete->HasAnyFlags(GARBAGE_COLLECTION_KEEPFLAGS) || ObjectToDelete->HasAnyInternalFlags(EInternalObjectFlags_GarbageCollectionKeepFlags)))
				{
					InternalReferences.Add(ObjectToDelete);
					bOutIsReferenced = true;
					bIsReferencedInternally = true;
				}
			}
			
			// Only add the main object to the list once we have finished checking sub-objects.
			ObjectsToDelete.Add(InObject);

			// If it's a blueprint, we also want to find anything with a reference to it's generated class
			UBlueprint* Blueprint = Cast<UBlueprint>(InObject);
			if (Blueprint && Blueprint->GeneratedClass)
			{
				ObjectsToDelete.Add(Blueprint->GeneratedClass);
			}

			// Check and see whether we are referenced by any objects that won't be garbage collected (*including* the undo buffer)
			for (UObject* Referencer : FReferencerFinder::GetAllReferencers(ObjectsToDelete, nullptr, EReferencerFinderFlags::SkipWeakReferences))
			{
				// Exclude any pendingkill or garbage object from counting as referencers
				if (IsValid(Referencer))
				{
					if (Referencer->IsIn(InObject))
					{
						InternalReferences.Add(Referencer);
					}
					else
					{
						if (Transactor == Referencer)
						{
							bOutIsReferencedInMemoryByUndo = true;
						}
						else
						{
							References.ExternalReferences.Emplace(Referencer);
							bOutIsReferenced = true;
						}
					}
				}
			}

			References.InternalReferences.Append(InternalReferences.Array());

			// If the object itself isn't in the transaction buffer, check to see if it's a Blueprint asset. We might have instances of the
			// Blueprint in the transaction buffer, in which case we also want to both alert the user and clear it prior to deleting the asset.
			if (!bOutIsReferencedInMemoryByUndo)
			{
				if (Blueprint && Blueprint->GeneratedClass)
				{
					TArray<UObject*> Objects;
					const TArray<FReferencerInformation>& ExternalMemoryReferences = References.ExternalReferences;
					for (auto RefIt = ExternalMemoryReferences.CreateConstIterator(); RefIt; ++RefIt)
					{
						const FReferencerInformation& RefInfo = *RefIt;
						if (RefInfo.Referencer->IsA(Blueprint->GeneratedClass))
						{
							Objects.Add(RefInfo.Referencer);
						}
					}

					if (FReferencerFinder::GetAllReferencers(Objects, nullptr, EReferencerFinderFlags::SkipWeakReferences).Contains(Transactor))
					{
						bOutIsReferencedInMemoryByUndo = true;
					}
				}
			}

			// Walk the ref chain to make sure external refs we found are actually reachable and can't be GCed
			if (bOutIsReferenced)
			{
				TSet<UObject*> Referencers;
				Referencers.Reserve(References.ExternalReferences.Num());

				for (FReferencerInformation& RefInfo : References.ExternalReferences)
				{
					Referencers.Add(RefInfo.Referencer);
				}

				TSet<UObject*> Roots = FindObjectsRoots(Referencers);

				// Remove the object we plan on deleting if it turns out to be a root because
				// the RF_Standalone flag is always removed later in the process.
				Roots.Remove(InObject);

				if (Roots.Contains(Transactor))
				{
					bOutIsReferencedInMemoryByUndo = true;
					bOutIsReferenced = Roots.Num() > 1 || bIsReferencedInternally;
				}
				else
				{
					bOutIsReferenced = Roots.Num() > 0 || bIsReferencedInternally;
				}
			}

			// For now, only IsReferenced can output which Property refers to an object and it is required
			// when showing the graph dialog of referencers. 
			// Only called when required and only when references are found, effect of this slower path is expected to be mostly negligible.
			// FReferencerFinder::GetAllReferencers could also be refactored a little bit to allow gathering of properties.
			if (bOutIsReferenced && bInRequireReferencingProperties && OutMemoryReferences)
			{
				// determine whether the transaction buffer is the only thing holding a reference to the object
				// and if so, offer the user the option to reset the transaction buffer.
				if (Transactor)
                {
					Transactor->DisableObjectSerialization();
				}
				bOutIsReferenced = IsReferenced(InObject, GARBAGE_COLLECTION_KEEPFLAGS, EInternalObjectFlags_GarbageCollectionKeepFlags, true, OutMemoryReferences);
				if (!bOutIsReferenced)
				{
					UE_LOGF(LogObjectTools, Warning, "Detected inconsistencies between reference gathering algorithms. Switching 'Editor.UseLegacyGetReferencersForDeletion' on for the remainder of this editor session.");
					CVarUseLegacyGetReferencersForDeletion->Set(1);
				}
				if (Transactor)
				{
					Transactor->EnableObjectSerialization();
				}
			}
		}
		// This is the old/slower behavior that is kept for debug/comparison and is going to be removed in a future release
		else
		{
			bOutIsReferenced = IsReferenced(InObject, GARBAGE_COLLECTION_KEEPFLAGS, EInternalObjectFlags_GarbageCollectionKeepFlags, true, OutMemoryReferences);
			if (bOutIsReferenced)
			{
				UTransactor* Transactor = GEditor ? ToRawPtr(GEditor->Trans) : nullptr;
				// determine whether the transaction buffer is the only thing holding a reference to the object
				// and if so, offer the user the option to reset the transaction buffer.
				if (Transactor)
				{
					Transactor->DisableObjectSerialization();
				}
				bOutIsReferenced = IsReferenced(InObject, GARBAGE_COLLECTION_KEEPFLAGS, EInternalObjectFlags_GarbageCollectionKeepFlags, true, OutMemoryReferences);
				if (Transactor)
				{
					Transactor->EnableObjectSerialization();
				}

				// If object is referenced both in undo and non-undo, we can't determine which one it is but
				// it doesn't matter since the undo stack is only cleared if objects are only referenced by it.
				if (!bOutIsReferenced)
				{
					bOutIsReferencedInMemoryByUndo = true;
				}
			}
		}
	}

	void GatherSubObjectsForReferenceReplacement(TSet<UObject*>& InObjects, TSet<UObject*>& ObjectsToExclude, TSet<UObject*>& OutObjectsAndSubObjects)
	{
		OutObjectsAndSubObjects = InObjects;
		if (InObjects.Num() > 0)
		{
			TArray<UObject*> AdditionalObjectsToExclude;
			for (UObject* ObjectToExclude : ObjectsToExclude)
			{
				GetObjectsWithOuter(ObjectToExclude, AdditionalObjectsToExclude);

				if (UBlueprint* BlueprintObject = Cast<UBlueprint>(ObjectToExclude))
				{
					if (BlueprintObject->GeneratedClass)
					{
						TArray<UObject*> ClassSubObjects;
						GetObjectsWithOuter(BlueprintObject->GeneratedClass, ClassSubObjects, EGetObjectsFlags::None);
						for (UObject* ClassSubObject : ClassSubObjects)
						{
							if (ClassSubObject->HasAnyFlags(RF_ArchetypeObject))
							{
								AdditionalObjectsToExclude.Add(ClassSubObject);
							}
						}
					}
				}
			}
			for (UObject* AdditionalObjectToExclude : AdditionalObjectsToExclude)
			{
				ObjectsToExclude.Add(AdditionalObjectToExclude);
			}

			for (UObject* InObject : InObjects)
			{
				TArray<UObject*> AdditionalObjects;
				{
					TArray<UObject*> SubObjects;
					GetObjectsWithOuter(InObject, SubObjects, EGetObjectsFlags::None);
					for (UObject* SubObject : SubObjects)
					{
						if (SubObject->HasAnyFlags(RF_ArchetypeObject)
							&& !ObjectsToExclude.Contains(SubObject)
							&& !InObjects.Contains(SubObject))
						{
							AdditionalObjects.Add(SubObject);
						}
					}
				}

				if (UBlueprint* BlueprintObject = Cast<UBlueprint>(InObject))
				{
					if (BlueprintObject->GeneratedClass)
					{
						if (AdditionalObjects.Contains(BlueprintObject->GeneratedClass))
						{
							// We don't want to replace within the generated class. 
							AdditionalObjects.Remove(BlueprintObject->GeneratedClass);
						}
						TArray<UObject*> ClassSubObjects;
						GetObjectsWithOuter(BlueprintObject->GeneratedClass, ClassSubObjects, EGetObjectsFlags::None);
						for (UObject* ClassSubObject : ClassSubObjects)
						{
							if (ClassSubObject->HasAnyFlags(RF_ArchetypeObject)
								&& !ObjectsToExclude.Contains(ClassSubObject)
								&& !InObjects.Contains(ClassSubObject))
							{
								AdditionalObjects.Add(ClassSubObject);
							}
						}
					}
				}
				OutObjectsAndSubObjects.Append(AdditionalObjects);
			}
		}

	}

	// recursive private function to determine the property paths that bring in a uobject
	void RecursiveBuildPropertyMap_Helper(TMap<const UObject*, int32>& CheckedObjects, TMap<FString, const UObject*>& PropertyToObject, const FString& InPropertyStr, const UObject* InObject, int32 Depth)
	{
		for (TPropertyValueIterator<FObjectProperty> PIter(InObject->GetClass(), InObject); PIter; ++PIter)
		{
			const FObjectProperty* Property = PIter.Key();
			const void* Value = PIter->Value;
			if (const UObject* ValueObject = Property->GetPropertyValue(Value))
			{
				if (int32* TimesEncountered = CheckedObjects.Find(ValueObject))
				{
					(*TimesEncountered)++;
					if (*TimesEncountered > MaxTimesToCheckSameObject)
					{
						continue;
					}
				}
				else
				{
					CheckedObjects.Add(ValueObject, 1);
				}

				const FString FullPropertyKey = InPropertyStr.IsEmpty() ? Property->GetName() : InPropertyStr + TEXT(" -> ") + Property->GetName();
				FString TestKey = FullPropertyKey;
				int32 ArrayIndex = 1;
				while (PropertyToObject.Contains(TestKey))
				{
					TestKey = FString::Printf(TEXT("%s[%d]"), *FullPropertyKey, ArrayIndex);
					ArrayIndex++;
				}
				PropertyToObject.Add(TestKey, ValueObject);
				if (Depth < MaxRecursionDepth)
				{
					RecursiveBuildPropertyMap_Helper(CheckedObjects, PropertyToObject, TestKey, ValueObject, Depth + 1);
				}
			}
		}
	}

	bool GatherPropertyChainsToObject(const UObject* SourceObject, const UObject* ObjectToSearchFor, TArray<FString>& OutFoundPropertyChains)
	{
		OutFoundPropertyChains.Empty();

		if (SourceObject && ObjectToSearchFor)
		{
			TMap<const UObject*, int32> CheckedObjects;
			TMap<FString, const UObject*> PropertyToObject;
			RecursiveBuildPropertyMap_Helper(CheckedObjects, PropertyToObject, TEXT(""), SourceObject, 0);

			for (const TPair<FString, const UObject*>& PropertyObjectPair : PropertyToObject)
			{
				const UObject* CurrentObject = PropertyObjectPair.Value;
				if (CurrentObject && CurrentObject == ObjectToSearchFor)
				{
					OutFoundPropertyChains.Add(PropertyObjectPair.Key);
				}
			}
		}
		return OutFoundPropertyChains.Num() > 0;
	}

	/**
	 * FArchiveTopLevelReferenceCollector constructor
	 * @todo: comment
	 */
	FArchiveTopLevelReferenceCollector::FArchiveTopLevelReferenceCollector(
		TArray<UObject*>* InObjectArray,
		const TArray<UObject*>& InIgnoreOuters,
		const TArray<UClass*>& InIgnoreClasses )
			:	ObjectArray( InObjectArray )
			,	IgnoreOuters( InIgnoreOuters )
			,	IgnoreClasses( InIgnoreClasses )
	{
		// Mark objects.
		for ( FThreadSafeObjectIterator It; It ; ++It )
		{
			if ( ShouldSearchForAssets(*It) )
			{
				It->Mark(OBJECTMARK_TagExp);
			}
			else
			{
				It->UnMark(OBJECTMARK_TagExp);
			}
		}
	}

	/**
	 * UObject serialize operator implementation
	 *
	 * @param Object	reference to Object reference
	 * @return reference to instance of this class
	 */
	FArchive& FArchiveTopLevelReferenceCollector::operator<<( UObject*& Obj )
	{
		if ( Obj != NULL && Obj->HasAnyMarks(OBJECTMARK_TagExp) )
		{
			// Clear the search flag so we don't revisit objects
			Obj->UnMark(OBJECTMARK_TagExp);
			if ( Obj->IsA(UField::StaticClass()) )
			{
				// skip all of the other stuff because the serialization of UFields will quickly overflow
				// our stack given the number of temporary variables we create in the below code
				Obj->Serialize(*this);
			}
			else
			{
				// Only report this object reference if it supports display in a browser.
				// this eliminates all of the random objects like functions, properties, etc.
				const bool bShouldReportAsset = ObjectTools::IsObjectBrowsable( Obj );
				if (Obj->IsValidLowLevel())
				{
					if ( bShouldReportAsset )
					{
						ObjectArray->Add( Obj );
					}
					// Check this object for any potential object references.
					Obj->Serialize(*this);
				}
			}
		}
		return *this;
	}


	void FMoveInfo::Set(const TCHAR* InFullPackageName, const TCHAR* InNewObjName)
	{
		FullPackageName = InFullPackageName;
		NewObjName = InNewObjName;
		check( IsValid() );
	}

	/** @return		true once valid (non-empty) move info exists. */
	bool FMoveInfo::IsValid() const
	{
		return ( FullPackageName.Len() > 0 && NewObjName.Len() > 0 );
	}

	/**
	 * Handles fully loading packages for a set of passed in objects.
	 *
	 * @param	Objects				Array of objects whose packages need to be fully loaded
	 * @param	OperationString		Localization key for a string describing the operation; appears in the warning string presented to the user.
	 *
	 * @return true if all packages where fully loaded, false otherwise
	 */
	bool HandleFullyLoadingPackages( const TArray<UObject*>& Objects, const FText& OperationText )
	{
		// Get list of outermost packages.
		TArray<UPackage*> TopLevelPackages;
		for( int32 ObjectIndex=0; ObjectIndex<Objects.Num(); ObjectIndex++ )
		{
			UObject* Object = Objects[ObjectIndex];
			if( Object )
			{
				TopLevelPackages.AddUnique( Object->GetOutermost() );
			}
		}

		return UPackageTools::HandleFullyLoadingPackages( TopLevelPackages, OperationText );
	}



	void DuplicateObjects( const TArray<UObject*>& SelectedObjects, const FString& SourcePath, const FString& DestinationPath, bool bOpenDialog, TArray<UObject*>* OutNewObjects )
	{
		if ( SelectedObjects.Num() < 1 )
		{
			return;
		}

		FMoveDialogInfo MoveDialogInfo;
		MoveDialogInfo.bOkToAll = !bOpenDialog;
		// The default value for save packages is true if SCC is enabled because the user can use SCC to revert a change
		MoveDialogInfo.bSavePackages = ISourceControlModule::Get().IsEnabled();

		bool bSawSuccessfulDuplicate = false;
		TSet<UPackage*> PackagesUserRefusedToFullyLoad;
		TArray<UPackage*> OutermostPackagesToSave;

		// If any objects are cooked, show just one dialog now so user is not flooded with a large number of dialogs
		for (UObject* Object : SelectedObjects)
		{
			if (Object && Object->RootPackageHasAnyFlags(PKG_FilterEditorOnly))
			{
				FMessageDialog::Open( EAppMsgType::Ok, FText::Format(NSLOCTEXT("UnrealEd", "CannotDuplicateCooked", "Cannot duplicate object: '{0}'\nPackage is cooked or missing editor data"), FText::FromString(Object->GetName())) );
				return;
			}
		}

		for( int32 ObjectIndex = 0 ; ObjectIndex < SelectedObjects.Num() ; ++ObjectIndex )
		{
			UObject* Object = SelectedObjects[ObjectIndex];
			if( !Object )
			{
				continue;
			}

			if ( !GetMoveDialogInfo(NSLOCTEXT("UnrealEd", "DuplicateObjects", "Copy Objects"), Object, /*bUniqueDefaultName=*/true, SourcePath, DestinationPath, MoveDialogInfo) )
			{
				// The user aborted the operation
				return;
			}

			UObject* NewObject = DuplicateSingleObject(Object, MoveDialogInfo.PGN, PackagesUserRefusedToFullyLoad);
			if ( NewObject != NULL )
			{
				if ( OutNewObjects != NULL )
				{
					OutNewObjects->Add(NewObject);
				}

				OutermostPackagesToSave.Add(NewObject->GetOutermost());
				bSawSuccessfulDuplicate = true;
			}
		}

		// Update the browser if something was actually moved.
		if ( bSawSuccessfulDuplicate )
		{
			bool bUpdateSCC = false;
			if ( MoveDialogInfo.bSavePackages )
			{
				const bool bCheckDirty = false;
				const bool bPromptToSave = false;
				FEditorFileUtils::PromptForCheckoutAndSave(OutermostPackagesToSave, bCheckDirty, bPromptToSave);
				bUpdateSCC = true;
			}

			if ( bUpdateSCC )
			{
				ISourceControlModule::Get().GetProvider().Execute(ISourceControlOperation::Create<FUpdateStatus>(), OutermostPackagesToSave);
			}
		}
	}

	UObject* DuplicateSingleObject(UObject* Object, const FPackageGroupName& PGN, TSet<UPackage*>& InOutPackagesUserRefusedToFullyLoad, bool bPromptToOverwrite, TMap<TSoftObjectPtr<UObject>, TSoftObjectPtr<UObject>>* DuplicatedObjects /*= nullptr*/)
	{
		UObject* ReturnObject = NULL;

		const FString& NewPackageName = PGN.PackageName;
		const FString& NewGroupName = PGN.GroupName;
		const FString& NewObjectName = PGN.ObjectName;

		const FScopedBusyCursor BusyCursor;

		// Check validity of each reference dup name.
		FString ErrorMessage;
		FText Reason;
		FString ObjectsToOverwriteName;
		FString ObjectsToOverwritePackage;
		FString ObjectsToOverwriteClass;
		TArray<UObject*> ObjectsToDelete;
		bool	bUserDeclinedToFullyLoadPackage = false;

		FMoveInfo MoveInfo;

		// Make sure that a target package exists.
		if ( !NewPackageName.Len() )
		{
			ErrorMessage += TEXT("Invalid package name supplied\n");
		}
		else if (Object->RootPackageHasAnyFlags(PKG_FilterEditorOnly))
		{
			ErrorMessage += TEXT("Package is cooked or missing editor data\n");
		}
		else
		{
			// Make a full path from the target package and group.
			const FString FullPackageName = NewGroupName.Len()
				? FString::Printf(TEXT("%s.%s"), *NewPackageName, *NewGroupName)
				: NewPackageName;

			// Make sure the packages being duplicated into are fully loaded.
			TArray<UPackage*> TopLevelPackages;
			UPackage* ExistingPackage = FindPackage(NULL, *FullPackageName);

			// If we did not find the package, it may not be loaded at all.
			if ( !ExistingPackage )
			{
				FString Filename;
				if ( FPackageName::DoesPackageExist(FullPackageName, &Filename) )
				{
					// There is an unloaded package file at the destination.
					ExistingPackage = LoadPackage(NULL, *FullPackageName, LOAD_None);
				}
			}

			if( ExistingPackage )
			{
				TopLevelPackages.Add( ExistingPackage->GetOutermost() );
			}

			if( (ExistingPackage && InOutPackagesUserRefusedToFullyLoad.Contains(ExistingPackage)) ||
				!UPackageTools::HandleFullyLoadingPackages( TopLevelPackages, NSLOCTEXT("UnrealEd", "Duplicate", "Duplicate") ) )
			{
				// HandleFullyLoadingPackages should never return false for empty input.
				check( ExistingPackage );
				InOutPackagesUserRefusedToFullyLoad.Add( ExistingPackage );
				bUserDeclinedToFullyLoadPackage = true;
			}
			else
			{
				UObject* ExistingObject = ExistingPackage ? StaticFindObject(UObject::StaticClass(), ExistingPackage, *NewObjectName) : NULL;

				if( !NewObjectName.Len() )
				{
					ErrorMessage += TEXT("Invalid object name\n");
				}
				else if(!FName(*NewObjectName).IsValidObjectName( Reason )
					||	!FPackageName::IsValidLongPackageName( NewPackageName, /*bIncludeReadOnlyRoots=*/false, &Reason )
					||	!FName(*NewGroupName).IsValidGroupName( Reason,true) )
				{
					// Make sure the object name is valid.
					ErrorMessage += FString::Printf(TEXT("    %s to %s.%s: %s\n"), *Object->GetPathName(), *FullPackageName, *NewObjectName, *Reason.ToString() );
				}
				else if (ExistingObject == Object)
				{
					ErrorMessage += TEXT("Can't duplicate an object onto itself!\n");
				}
				else
				{
					// If the object already exists in this package with the given name, give the user
					// the opportunity to overwrite the object. So, don't treat this as an error.
					if ( ExistingPackage && !IsUniqueObjectName(*NewObjectName, ExistingPackage, Reason) )
					{
						ObjectsToOverwriteName += *NewObjectName;
						ObjectsToOverwritePackage += *FullPackageName;
						ObjectsToOverwriteClass += *ExistingObject->GetClass()->GetName();

						ObjectsToDelete.Add(ExistingObject);
					}

					// NOTE: Set the move info if this object already exists in-case the user wants to
					// overwrite the existing asset. To overwrite the object, the move info is needed.

					// No errors!  Set asset move info.
					MoveInfo.Set( *FullPackageName, *NewObjectName );
				}
			}
		}

		// User declined to fully load the target package; no need to display message box.
		if( bUserDeclinedToFullyLoadPackage )
		{
			return NULL;
		}

		// If any errors are present, display them and abort this object.
		else if( ErrorMessage.Len() > 0 )
		{
			FMessageDialog::Open( EAppMsgType::Ok, FText::Format(NSLOCTEXT("UnrealEd", "CannotDuplicateList", "Cannot duplicate object: '{0}'\n{1}"), FText::FromString(Object->GetName()), FText::FromString(ErrorMessage)) );
			return NULL;
		}

		// If there are objects that already exist with the same name, give the user the option to overwrite the
		// object. This will delete the object so the new one can be created in its place.
		if(bPromptToOverwrite && ObjectsToOverwriteName.Len() > 0 )
		{
			TSharedRef<SMessageDialog> ConfirmDialog = SNew(SMessageDialog)
				.Icon(FAppStyle::Get().GetBrush("Icons.WarningWithColor.Large"))
				.Title(FText(NSLOCTEXT("UnrealEd", "ReplaceExistingObjectInPackageConfirmation_Title", "Overwrite Existing Object")))
				.Message(FText::Format(NSLOCTEXT("UnrealEd", "ReplaceExistingObjectInPackageConfirmation_Message", "An object already exists with this name.\n\n\tName: {0}\n\tClass: {1}\n\tAsset path: {2}\n\nOverwrite the existing object?"),
					FText::FromString(ObjectsToOverwriteName),
					FText::FromString(ObjectsToOverwriteClass),
					FText::FromString(ObjectsToOverwritePackage)))
				.Buttons({
					SCustomDialog::FButton(NSLOCTEXT("UnrealEd", "ReplaceExistingObjectInPackageConfirmation_ButtonOverwrite", "Overwrite")).SetPrimary(true),
					SCustomDialog::FButton(NSLOCTEXT("UnrealEd", "ReplaceExistingObjectInPackageConfirmation_ButtonCancel", "Cancel")),
					})
				.ContentMinWidth(300.0f);
			uint32 ConfirmationResult = ConfirmDialog->ShowModal();

			bool bOverwriteExistingObjects = ConfirmationResult == 0;

			// The user didn't want to overwrite the existing options, so bail out of the duplicate operation.
			if( !bOverwriteExistingObjects )
			{
				return NULL;
			}
		}

		// If some objects need to be deleted, delete them.
		if (ObjectsToDelete.Num() > 0)
		{
			TArray<UPackage*> DeletedObjectPackages;

			// Add all packages for deleted objects to the root set if they are not already so we can reuse them later.
			// This will prevent DeleteObjects from marking the file for delete in source control
			for ( auto ObjIt = ObjectsToDelete.CreateConstIterator(); ObjIt; ++ObjIt )
			{
				UPackage* Pkg = (*ObjIt)->GetOutermost();

				if ( Pkg && !Pkg->IsRooted() )
				{
					DeletedObjectPackages.AddUnique(Pkg);
					Pkg->AddToRoot();
				}
			}

			// Handle map built data
			const int32 OriginalNumToDelete = ObjectsToDelete.Num();
			ObjectTools::AddExtraObjectsToDelete(ObjectsToDelete);
			for (int32 i = OriginalNumToDelete; i < ObjectsToDelete.Num(); ++i)
			{
				UPackage* Pkg = ObjectsToDelete[i]->GetOutermost();
				if ( Pkg && !Pkg->IsRooted() )
				{
					DeletedObjectPackages.AddUnique(Pkg);
					Pkg->AddToRoot();
				}
			}

			const int32 NumObjectsDeleted = ObjectTools::DeleteObjects(ObjectsToDelete, bPromptToOverwrite, EAllowCancelDuringDelete::CancelNotAllowed);

			// Remove all packages that we added to the root set above.
			for ( auto PkgIt = DeletedObjectPackages.CreateConstIterator(); PkgIt; ++PkgIt )
			{
				(*PkgIt)->RemoveFromRoot();
			}

			if (NumObjectsDeleted != ObjectsToDelete.Num())
			{
				UE_LOGF(LogObjectTools, Warning,
					"Existing objects could not be deleted, unable to duplicate %ls",
					*Object->GetFullName());
				return NULL;
			}
		}

		// Create ReplacementMap for replacing references.
		TMap<UObject*, UObject*> ReplacementMap;

		check( MoveInfo.IsValid() );

		const FString& PkgName = MoveInfo.FullPackageName;
		const FString& ObjName = MoveInfo.NewObjName;

		// Make sure the referenced object is deselected before duplicating it.
		GEditor->GetSelectedObjects()->Deselect( Object );

		UObject* DupObject = NULL;

		UPackage* ExistingPackage = FindPackage(NULL, *PkgName);
		UObject* ExistingObject = ExistingPackage ? StaticFindObject(UObject::StaticClass(), ExistingPackage, *ObjName) : NULL;

		// Any existing objects should be deleted and garbage collected by now
		if ( ensure(ExistingObject == NULL) )
		{
			EDuplicateMode::Type DuplicateMode = Object->IsA(UWorld::StaticClass()) ? EDuplicateMode::World : EDuplicateMode::Normal;

			FObjectDuplicationParameters Params = InitStaticDuplicateObjectParams(Object, CreatePackage(*PkgName), *ObjName, RF_AllFlags, nullptr, DuplicateMode);

			TMap<UObject*, UObject*> CreatedObjects;
			if (DuplicatedObjects)
			{
				Params.CreatedObjects = &CreatedObjects;
			}

			DupObject = StaticDuplicateObjectEx(Params);

			if (DuplicatedObjects)
			{
				// Convert DuplicatedObjects map into an object paths map
				DuplicatedObjects->Reserve(CreatedObjects.Num());
				for (const auto& DuplicatedObjectPair : CreatedObjects)
				{
					DuplicatedObjects->Add(DuplicatedObjectPair.Key, DuplicatedObjectPair.Value);
				}
			}
		}

		if( DupObject )
		{
			ReplacementMap.Add( Object, DupObject );
			DupObject->MarkPackageDirty();

			// if the source object is in the MyLevel package and it's being duplicated into a content package, we need
			// to mark it RF_Standalone so that it will be saved (UWorld::CleanupWorld() clears this flag for all objects
			// inside the package)
			if (!Object->HasAnyFlags(RF_Standalone)
				&&	Object->GetOutermost()->ContainsMap()
				&&	!DupObject->GetOutermost()->ContainsMap() )
			{
				DupObject->SetFlags(RF_Standalone);
			}

			// Duplicating an asset should respect the export controls of the original.
			if (Object->GetOutermost()->HasAnyPackageFlags(PKG_DisallowExport))
			{
				DupObject->GetOutermost()->SetPackageFlags(PKG_DisallowExport);
			}

			// Duplicating an asset should respect the reference controls of the original.
			DupObject->GetOutermost()->SetAssetAccessSpecifier(Object->GetOutermost()->GetAssetAccessSpecifier());

			// When duplicating a World Composition map, make sure to properly initialize WorldTileInfo
			if (Object->GetOutermost()->GetWorldTileInfo())
			{
				DupObject->GetOutermost()->SetWorldTileInfo(MakeUnique<FWorldTileInfo>(*Object->GetOutermost()->GetWorldTileInfo()));
			}

			// Notify the asset registry
			FAssetRegistryModule::AssetCreated(DupObject);

			// Notify the asset registry of world's duplicated MapBuildData
			UWorld* DupWorld = Cast<UWorld>(DupObject);
			if (DupWorld && DupWorld->PersistentLevel && DupWorld->PersistentLevel->MapBuildData)
			{
				FAssetRegistryModule::AssetCreated(DupWorld->PersistentLevel->MapBuildData);
			}

			// if the duplicated object package has external packages, they were also duplicated. Mark them dirty as well
			for (UPackage* ExternalPackage : DupObject->GetPackage()->GetExternalPackages())
			{
				ExternalPackage->MarkPackageDirty();
			}

			ReturnObject = DupObject;
		}

		GEditor->GetSelectedObjects()->Select( Object );

		// Replace all references
		FArchiveReplaceObjectRef<UObject> ReplaceAr( DupObject, ReplacementMap, EArchiveReplaceObjectFlags::IgnoreOuterRef | EArchiveReplaceObjectFlags::IgnoreArchetypeRef );

		return ReturnObject;
	}

	/**
	 * Helper struct for passing multiple arrays to and from ForceReplaceReferences
	 */
	struct FForceReplaceInfo
	{
		// A list of packages which were dirtied as a result of a force replace
		TArray<UPackage*> DirtiedPackages;
		// Objects whose references were successfully replaced
		TArray<UObject*> ReplaceableObjects;
		// Objects whose references could not be successfully replaced
		TArray<UObject*> UnreplaceableObjects;

		void AppendUnique(const FForceReplaceInfo& ForceReplaceInfo)
		{
			const TArray<UPackage*>& ForceReplaceInfoDirtiedPackages = ForceReplaceInfo.DirtiedPackages;
			DirtiedPackages.Reserve(DirtiedPackages.Num() + ForceReplaceInfoDirtiedPackages.Num());
			for (UPackage* Package : ForceReplaceInfoDirtiedPackages)
			{
				DirtiedPackages.AddUnique(Package);
			}

			const TArray<UObject*>& ForceReplaceInfoReplaceableObjects = ForceReplaceInfo.ReplaceableObjects;
			ReplaceableObjects.Reserve(ReplaceableObjects.Num() + ForceReplaceInfoReplaceableObjects.Num());
			for (UObject* Object : ForceReplaceInfoReplaceableObjects)
			{
				ReplaceableObjects.Add(Object);
			}

			const TArray<UObject*>& ForceReplaceInfoUnreplaceableObjects = ForceReplaceInfo.UnreplaceableObjects;
			UnreplaceableObjects.Reserve(UnreplaceableObjects.Num() + ForceReplaceInfoUnreplaceableObjects.Num());
			for (UObject* Object : ForceReplaceInfoUnreplaceableObjects)
			{
				UnreplaceableObjects.Add(Object);
			}
		}
	};
	
	void ForceReplaceReferences(TArrayView<FReplaceRequest> Requests, TSet<UObject*>& ObjectsToReplaceWithin, FForceReplaceInfo& OutInfo, bool bWarnAboutRootSet = true)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ObjectTools::ForceReplaceReferences);

		bool bOnlyNullingOut = true;
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		TArray<UObject*> AllOld;
		for (FReplaceRequest Request : Requests)
		{
			AllOld.Append(Request.Old.GetData(), Request.Old.Num());
			bOnlyNullingOut &= !Request.New;
		}
		PropertyEditorModule.RemoveDeletedObjects(AllOld);

		TSet<UObject*> RootSetObjects;

		GWarn->StatusUpdate( 0, 0, NSLOCTEXT("UnrealEd", "ConsolidateAssetsUpdate_RootSetCheck", "Checking Assets for Root Set...") );

		// Iterate through all the objects to replace and see if they are in the root set.  If they are, offer to remove them from the root set.
		for (UObject* CurObjToReplace : AllOld)
		{
			checkf(CurObjToReplace != nullptr, TEXT("Cannot replace null references"));
			if (CurObjToReplace->IsRooted())
			{
				RootSetObjects.Add( CurObjToReplace );
			}
		}
		
		if ( RootSetObjects.Num() )
		{
			if( bWarnAboutRootSet )
			{
				// Collect names of root set assets
				FString RootSetObjectNames;
				for ( TSet<UObject*>::TConstIterator RootSetIter( RootSetObjects ); RootSetIter; ++RootSetIter )
				{
					UObject* CurRootSetObject = *RootSetIter;
					RootSetObjectNames += CurRootSetObject->GetName() + TEXT("\n");
				}

				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("Objects"), FText::FromString( RootSetObjectNames ));
				FText MessageFormatting = NSLOCTEXT("ObjectTools", "ConsolidateAssetsRootSetDlgMsgFormatting", "The assets below were in the root set and we must remove that flag in order to proceed.  Being in the root set means that this was loaded at startup and is meant to remain in memory during gameplay.  For most assets this should be fine.  If, for some reason, there is an error, you will be notified.  Would you like to remove this flag?\n\n{Objects}");
				FText Message = FText::Format( MessageFormatting, Arguments );
				FText Title = NSLOCTEXT("ObjectTools", "ConsolidateAssetsRootSetDlg_Title", "Failed to Consolidate Assets");

				// Prompt the user to see if they'd like to remove the root set flag from the assets and attempt to replace them
				EAppReturnType::Type UserResponse = FMessageDialog::Open( EAppMsgType::YesNo, EAppReturnType::No, Message, Title );

				// The user elected to not remove the root set flag, so cancel the replacement
				if (UserResponse == EAppReturnType::No )
				{
					return;
				}
			}

			for ( FThreadSafeObjectIterator ObjIter; ObjIter; ++ObjIter )
			{
				// Always clear the root set flags
				UObject* CurrentObject = *ObjIter;
				if ( CurrentObject )
				{
					// If the current object is one of the objects the user is attempting to replace but is marked RF_RootSet, strip the flag by removing it
					// from root
					if ( RootSetObjects.Find( CurrentObject ) )
					{
						CurrentObject->RemoveFromRoot();
					}
					// If the current object is inside one of the objects to replace but is marked RF_RootSet, strip the flag by removing it from root
					else
					{
						for( UObject* CurObjOuter = CurrentObject->GetOuter(); CurObjOuter; CurObjOuter = CurObjOuter->GetOuter() )
						{
							if ( RootSetObjects.Find( CurObjOuter ) )
							{
								CurrentObject->RemoveFromRoot();
								break;
							}
						}
					}
				}
			}
		}

		// @note FH: There shouldn't be a need to reset all loaders here to replace references. This actually currently causes all bulkdata to be force loaded since we are getting rid on the underlying loader archive
		// Although object references in linkers aren't tracked by the GC nor can't be replaced by the reference gathering archives, they will be properly cleaned out of the linker if the linker outlives the objects for one thing,
		// but moreover the linkers associated with the packages of the object we are replacing will also been cleaned up if we are deleting those packages after the force replace references.
		// For both of these reasons, a call to reset all linkers here seems entirely unnecessary
		//ResetLoaders(nullptr);

		TMap<UObject*, int32> ObjToNumRefsMap;
		if (!bOnlyNullingOut)
		{
			GWarn->StatusUpdate( 0, 0, NSLOCTEXT("UnrealEd", "ConsolidateAssetsUpdate_CheckAssetValidity", "Determining Validity of Assets...") );
			for (FReplaceRequest Request : Requests)
			{
				if (UObject* ObjectToReplaceWith = Request.New)
				{
					// Determine if the "object to replace with" has any references to any of the "objects to replace," if so, we don't
					// want to allow those objects to be replaced, as the object would end up referring to itself!
					// We can skip this check if "object to replace with" is NULL since it is not useful to check for null references
					FFindReferencersArchive FindRefsAr( ObjectToReplaceWith, Request.Old );
					FindRefsAr.AppendReferenceCounts( ObjToNumRefsMap );
				}
			}
		}

		// Objects already loaded and in memory have to have any of their references to the objects to replace swapped with a reference to
		// the "object to replace with". FArchiveReplaceObjectAndStructPropertyRef can serve this purpose, but it expects a TMap of object to replace : object to replace with.
		// Therefore, populate a map with all of the valid objects to replace as keys, with the object to replace with as the value for each one.
		TMap<UObject*, UObject*> ReplacementMap;
		for (FReplaceRequest Request : Requests)
		{
			UObject* ObjectToReplaceWith = Request.New;
			for (UObject* CurObjToReplace : Request.Old)
			{
				// If any of the objects to replace are marked RF_RootSet at this point, an error has occurred
				check( !CurObjToReplace->IsRooted() );

				// Exclude root packages from being replaced
				const bool bRootPackage = ( CurObjToReplace->GetClass() == UPackage::StaticClass() ) && !( CurObjToReplace->GetOuter() );

				// Additionally exclude any objects that the "object to replace with" contains references to, in order to prevent the "object to replace with" from
				// referring to itself
				int32 NumRefsInObjToReplaceWith = 0;
				int32* PtrToNumRefs = ObjToNumRefsMap.Find( CurObjToReplace );
				if ( PtrToNumRefs )
				{
					NumRefsInObjToReplaceWith = *PtrToNumRefs;
				}

				if ( !bRootPackage && NumRefsInObjToReplaceWith == 0 )
				{
					ReplacementMap.Add( CurObjToReplace, ObjectToReplaceWith );

					// Fully load the packages of objects to replace
					CurObjToReplace->GetOutermost()->FullyLoad();
				}
				// If an object is "unreplaceable" store it separately to warn the user about later
				else
				{
					OutInfo.UnreplaceableObjects.AddUnique(CurObjToReplace);
				}
			}
		}

		GWarn->StatusUpdate( 0, 0, NSLOCTEXT("UnrealEd", "ConsolidateAssetsUpdate_CollectingReferences", "Collecting Asset References...") );

		ReplacementMap.GenerateKeyArray( OutInfo.ReplaceableObjects );

		// Find all the properties (and their corresponding objects) that refer to any of the objects to be replaced
		using PropertyArrayType = TArray<FProperty*, TInlineAllocator<1>>;
		TArray<UObject*> ReferencingPropertiesMapKeys;
		TArray<PropertyArrayType> ReferencingPropertiesMapValues;

		{
			// Find the referencers of the objects to be replaced
			FFindReferencersArchive FindRefsArchive( nullptr, OutInfo.ReplaceableObjects );
			TMap<UObject*, int32> CurNumReferencesMap;
			TMultiMap<UObject*, FProperty*> CurReferencingPropertiesMMap;
			PropertyArrayType CurReferencedProperties;
			
			auto CollectObjectReferencers = [bOnlyNullingOut, &ReplacementMap, &ReferencingPropertiesMapKeys, &ReferencingPropertiesMapValues, &FindRefsArchive, &CurNumReferencesMap, &CurReferencingPropertiesMMap, &CurReferencedProperties](UObject* CurObject)
			{
				// Don't bother replacing in objects that are about to be garbage collected
				if (!IsValidChecked(CurObject) || CurObject->IsUnreachable())
				{
					return;
				}

				// Unless the "object to replace with" is null, ignore the objects being replaced
				UObject** ObjectToReplaceWithPtr = bOnlyNullingOut ? nullptr : ReplacementMap.Find(CurObject);
				if (ObjectToReplaceWithPtr == nullptr || *ObjectToReplaceWithPtr == nullptr)
				{
					FindRefsArchive.ResetPotentialReferencer(CurObject);

					// Inform the object referencing any of the objects to be replaced about the properties that are being forcefully
					// changed, and store both the object doing the referencing as well as the properties that were changed in a map (so that
					// we can correctly call PostEditChange later)

					if ( FindRefsArchive.GetReferenceCounts( CurNumReferencesMap, CurReferencingPropertiesMMap ) > 0  )
					{
						// TODO: FFindReferencersArchive is giving us the leaf property rather than the member property
						CurReferencedProperties.Reset(CurReferencingPropertiesMMap.Num());
						for (const TTuple<UObject*, FProperty*>& CurReferencingPropertiesPair : CurReferencingPropertiesMMap)
						{
							CurReferencedProperties.AddUnique(CurReferencingPropertiesPair.Value);
						}

						ReferencingPropertiesMapKeys.Add(CurObject);
						ReferencingPropertiesMapValues.Add(CurReferencedProperties);

						if ( CurReferencedProperties.Num() > 0)
						{
							for (FProperty* RefProp : CurReferencedProperties)
							{
								CurObject->PreEditChange(RefProp);
							}
						}
						else
						{
							CurObject->PreEditChange(nullptr);
						}
					}
				}
			};

			if (ObjectsToReplaceWithin.Num() > 0)
			{
				int32 NumObjsCollected = 0;
				TArray<UObject*> InnerObjects;
				for (UObject* CurObject : ObjectsToReplaceWithin)
				{
					++NumObjsCollected;
					GWarn->StatusUpdate(NumObjsCollected, ObjectsToReplaceWithin.Num(), NSLOCTEXT("UnrealEd", "ConsolidateAssetsUpdate_CollectingReferences", "Collecting Asset References..."));

					if (CurObject && CurObject->IsValidLowLevel())
					{
						CollectObjectReferencers(CurObject);

						// FArchiveReplaceObjectAndStructPropertyRef is recursive into sub-objects, but FFindReferencersArchive 
						// isn't so we need to handle that ourselves to build the complete set of references
						InnerObjects.Reset();
						GetObjectsWithOuter(CurObject, InnerObjects);
						for (UObject* InnerObject : InnerObjects)
						{
							CollectObjectReferencers(InnerObject);
						}
					}
				}
			}
			else
			{
				for (FThreadSafeObjectIterator ObjIter; ObjIter; ++ObjIter)
				{
					CollectObjectReferencers(*ObjIter);
				}
			}
		}

		// Shuffle dependents before the objects that they reference
		{
			TBitArray<> TouchedThisItteration(false, ReferencingPropertiesMapKeys.Num());
			for (int CurrentIndex = 0; CurrentIndex < ReferencingPropertiesMapKeys.Num(); CurrentIndex++)
			{
				GWarn->StatusUpdate(CurrentIndex + 1, ReferencingPropertiesMapKeys.Num(), NSLOCTEXT("UnrealEd", "ConsolidateAssetsUpdate_PreparingAssetReferences", "Preparing Asset References..."));

				TouchedThisItteration.Init(false, ReferencingPropertiesMapKeys.Num());
				FFindReferencersArchive FindDependentArchive(ReferencingPropertiesMapKeys[CurrentIndex], ReferencingPropertiesMapKeys);
				for (int DependentIndex = CurrentIndex + 1; DependentIndex < ReferencingPropertiesMapKeys.Num(); DependentIndex++)
				{
					if (!TouchedThisItteration[DependentIndex] && FindDependentArchive.GetReferenceCount(ReferencingPropertiesMapKeys[DependentIndex]) > 0)
					{
						Swap(ReferencingPropertiesMapKeys[CurrentIndex], ReferencingPropertiesMapKeys[DependentIndex]);
						Swap(ReferencingPropertiesMapValues[CurrentIndex], ReferencingPropertiesMapValues[DependentIndex]);

						FindDependentArchive.ResetPotentialReferencer(ReferencingPropertiesMapKeys[CurrentIndex]);
						TouchedThisItteration[DependentIndex] = true;
						DependentIndex = CurrentIndex;
					}
				}
			}
		}

		// Run the reference replacement
		if (ObjectsToReplaceWithin.Num() > 0)
		{
			int32 NumObjsReplaced = 0;
			for (UObject* CurObject : ObjectsToReplaceWithin)
			{
				++NumObjsReplaced;
				GWarn->StatusUpdate(NumObjsReplaced, ObjectsToReplaceWithin.Num(), NSLOCTEXT("UnrealEd", "ConsolidateAssetsUpdate_ReplacingReferences", "Replacing Asset References..."));

				if (CurObject && CurObject->IsValidLowLevel())
				{
					UBlueprint* BPObjectToUpdate = Cast<UBlueprint>(CurObject);
					if (BPObjectToUpdate)
					{
						FArchiveReplaceObjectAndStructPropertyRef<UObject> ReplaceInBPClassObject_Ar(BPObjectToUpdate->GeneratedClass, ReplacementMap, EArchiveReplaceObjectFlags::IncludeClassGeneratedByRef);
						FArchiveReplaceObjectAndStructPropertyRef<UObject> ReplaceInBPClassDefaultObject_Ar(BPObjectToUpdate->GeneratedClass->GetDefaultObject(false), ReplacementMap, EArchiveReplaceObjectFlags::IncludeClassGeneratedByRef);
					}

					UDataTable* DataTableObjectToUpdate = Cast<UDataTable>(CurObject);
					if (DataTableObjectToUpdate)
					{
						FArchiveReplaceObjectRefDataTableRows<UObject> ReplaceInDataTableRows_Ar(DataTableObjectToUpdate, ReplacementMap);
					}

					FArchiveReplaceObjectAndStructPropertyRef<UObject> ReplaceAr(CurObject, ReplacementMap, EArchiveReplaceObjectFlags::IncludeClassGeneratedByRef);
				}
			}
		}
		else
		{
			// Iterate over the map of referencing objects/changed properties, forcefully replacing the references and
			int32 NumObjsReplaced = 0;
			for (int32 Index = 0; Index < ReferencingPropertiesMapKeys.Num(); Index++)
			{
				++NumObjsReplaced;
				GWarn->StatusUpdate(NumObjsReplaced, ReferencingPropertiesMapKeys.Num(), NSLOCTEXT("UnrealEd", "ConsolidateAssetsUpdate_ReplacingReferences", "Replacing Asset References..."));

				UObject* CurReplaceObj = ReferencingPropertiesMapKeys[Index];
				FArchiveReplaceObjectAndStructPropertyRef<UObject> ReplaceAr(CurReplaceObj, ReplacementMap, EArchiveReplaceObjectFlags::IncludeClassGeneratedByRef);
			}
		}

		// Reset property bag associations after replacing references.
		UE::FPropertyBagRepository::Get().ReassociateObjects(ReplacementMap);

		// Now alter the referencing objects the change has completed via PostEditChange,
		// this is done in a separate loop to prevent reading of data that we want to overwrite
		int32 NumObjsPostEdited = 0;
		for (int32 Index = 0; Index < ReferencingPropertiesMapKeys.Num(); Index++)
		{
			++NumObjsPostEdited;
			GWarn->StatusUpdate( NumObjsPostEdited, ReferencingPropertiesMapKeys.Num(), NSLOCTEXT("UnrealEd", "ConsolidateAssetsUpdate_FinalizingReferences", "Finalizing Asset References...") );

			UObject* CurReplaceObj = ReferencingPropertiesMapKeys[Index];
			const PropertyArrayType& RefPropArray = ReferencingPropertiesMapValues[Index];

			if (RefPropArray.Num() > 0)
			{
				for (FProperty* RefProp : RefPropArray)
				{
					FPropertyChangedEvent PropertyEvent(RefProp, EPropertyChangeType::Redirected);
					CurReplaceObj->PostEditChangeProperty( PropertyEvent );
				}
			}
			else
			{
				FPropertyChangedEvent PropertyEvent(nullptr, EPropertyChangeType::Redirected);
				CurReplaceObj->PostEditChangeProperty(PropertyEvent);
			}

			if ( !CurReplaceObj->HasAnyFlags(RF_Transient) && CurReplaceObj->GetOutermost() != GetTransientPackage() )
			{
				if ( !CurReplaceObj->RootPackageHasAnyFlags(PKG_CompiledIn) )
				{
					CurReplaceObj->MarkPackageDirty();
					OutInfo.DirtiedPackages.AddUnique( CurReplaceObj->GetOutermost() );
				}
			}
		}
	}

	void ForceReplaceReferences( UObject* ObjectToReplaceWith, TArray<UObject*>& ObjectsToReplace, TSet<UObject*>& ObjectsToReplaceWithin, FForceReplaceInfo& OutInfo, bool bWarnAboutRootSet = true)
	{
		FReplaceRequest Request = {ObjectToReplaceWith, ObjectsToReplace};
		ForceReplaceReferences(MakeArrayView(&Request, 1), ObjectsToReplaceWithin, OutInfo, bWarnAboutRootSet);
	}

	/**
	 * Forcefully replaces references to passed in objects
	 *
	 * @param ObjectToReplaceWith	Any references found to 'ObjectsToReplace' will be replaced with this object.  If the object is NULL references will be nulled.
	 * @param ObjectsToReplace		An array of objects that should be replaced with 'ObjectToReplaceWith'
	 * @param OutInfo				FForceReplaceInfo struct containing useful information about the result of the call to this function
	 * @param bWarnAboutRootSet		If True a message will be displayed to a user asking them if they would like to remove the rootset flag from objects which have it set.
									If False, the message will not be displayed and rootset is automatically removed
	 */
	static void ForceReplaceReferences(UObject* ObjectToReplaceWith, TArray<UObject*>& ObjectsToReplace, FForceReplaceInfo& OutInfo, bool bWarnAboutRootSet = true)
	{
		TSet<UObject*> InObjectsToReplaceWithin;
		ForceReplaceReferences(ObjectToReplaceWith, ObjectsToReplace, InObjectsToReplaceWithin, OutInfo, bWarnAboutRootSet);
	}

	// ForceReplaceReferences version exposed to the public API
	void ForceReplaceReferences(UObject* ObjectToReplaceWith, TArray<UObject*>& ObjectsToReplace)
	{
		FForceReplaceInfo ReplaceInfo;
		ForceReplaceReferences(ObjectToReplaceWith, ObjectsToReplace, ReplaceInfo, false);
	}

	void ForceReplaceReferences(UObject* ObjectToReplaceWith, TArray<UObject*>& ObjectsToReplace, TSet<UObject*>& ObjectsToReplaceWithin)
	{
		FForceReplaceInfo ReplaceInfo;
		ForceReplaceReferences(ObjectToReplaceWith, ObjectsToReplace, ObjectsToReplaceWithin, ReplaceInfo, false);
	}

	void ForceReplaceReferences(TArrayView<FReplaceRequest> Requests, TSet<UObject*>& ObjectsToReplaceWithin)
	{
		FForceReplaceInfo ReplaceInfo;
		ForceReplaceReferences(Requests, ObjectsToReplaceWithin, ReplaceInfo, false);
	}

	FConsolidationResults ConsolidateObjects(TArrayView<FReplaceRequest> Requests, TSet<UObject*>& ObjectsToConsolidateWithin, TSet<UObject*>& ObjectsToNotConsolidateWithin, bool bShouldDeleteAfterConsolidate, bool bWarnAboutRootSet)
	{
		for (FReplaceRequest Request : Requests)
		{
			checkf(Request.New != nullptr, TEXT("Can't consolidate objects into null objects"));
		}
		
		FConsolidationResults ConsolidationResults;
		const bool bShouldShowDialogs = !IsRunningCommandlet();
		const bool bShouldHandleEditorUIChanges = !IsRunningCommandlet();

		if (bShouldHandleEditorUIChanges)
		{
			// Close all editors to avoid changing references to temporary objects used by the editor
			if (!GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllAssetEditors())
			{
				// Failed to close at least one editor. It is possible that this editor has in-memory object references
				// which are not prepared to be changed dynamically so it is not safe to continue
				return ConsolidationResults;
			}

			// Clear audio components to allow previewed sounds to be consolidated
			GEditor->ClearPreviewComponents();

			// Make sure none of the objects are referenced by the editor's USelection
			for (FReplaceRequest Request : Requests)
			{
				GEditor->GetSelectedObjects()->Deselect(Request.New);
				for (UObject* Old : Request.Old)
				{
					GEditor->GetSelectedObjects()->Deselect(Old);
				}
			}
		}

		GWarn->BeginSlowTask(NSLOCTEXT("UnrealEd", "ConsolidateAssetsUpdate_Consolidating", "Consolidating Assets..."), true);
		// Keep track of which objects, if any, cannot be consolidated, in order to notify the user later
		TArray<UObject*> UnconsolidatableObjects;

		// Keep track of objects which became partially consolidated but couldn't be deleted for some reason;
		// these are critical failures, and the user needs to be alerted
		TArray<UObject*> CriticalFailureObjects;

		// Keep track of which packages the consolidate operation has dirtied so the user can be alerted to them
		// during a critical failure
		TArray<UPackage*> DirtiedPackages;

		// Keep track of root set objects so the user can be prompted about stripping the flag from them
		TSet<UObject*> RootSetObjects;

		// List of objects successfully deleted
		TArray<UObject*> ConsolidatedObjects;

		// A list of names for object redirectors created during the delete process
		// This is needed because the redirectors may not have the same name as the
		// objects they are replacing until the objects are garbage collected
		TMap<UObjectRedirector*, FName> RedirectorToObjectNameMap;

		// Temporaries used after ReloadEditorWorldForReferenceReplacementIfNecessary
		TArray<FReplaceRequest, TInlineAllocator<1>> PostReloadRequests;
		PostReloadRequests.Reserve(Requests.Num());
		TArray<TArray<UObject*>> PostReloadUpdatedOld;

		{
			// Note reloading the world via ReloadEditorWorldForReferenceReplacementIfNecessary will cause a garbage collect and potentially cause entries in the ObjectsToConsolidate list to become invalid
			// We refresh the list here after reloading the editor world
			TArray< TWeakObjectPtr<UObject> > ObjectsToConsolidateWeakList;
			ObjectsToConsolidateWeakList.Reserve(Requests.Num());

			for (FReplaceRequest Request : Requests)
			{
				for (UObject* Old : Request.Old)
				{
					ObjectsToConsolidateWeakList.Add(Old);
				}
			}

			// If the current editor world is in this list, transition to a new map and reload the world to finish the delete
			ReloadEditorWorldForReferenceReplacementIfNecessary(ObjectsToConsolidateWeakList);

			// Make new PostReloadRequests where all Old objects are valid
			int32 WeakIndex = 0;
			for (FReplaceRequest Request : Requests)
			{
				int32 NumValid = 0;
				for (int32 OldIndex = 0; OldIndex < Request.Old.Num(); ++OldIndex)
				{
					NumValid += ObjectsToConsolidateWeakList[WeakIndex + OldIndex].IsValid();
				}

				if (NumValid == Request.Old.Num())
				{
					PostReloadRequests.Add(Request);
				}
				else if (NumValid > 0)
				{
					TArray<UObject*>& UpdatedOld = PostReloadUpdatedOld.AddDefaulted_GetRef();
					for (int32 OldIndex = 0; OldIndex < Request.Old.Num(); ++OldIndex)
					{
						if (UObject* ValidOld = ObjectsToConsolidateWeakList[WeakIndex + OldIndex].Get())
						{
							checkf(ValidOld == Request.Old[OldIndex], TEXT("Indexing bug?"));
							UpdatedOld.Add(ValidOld);
						}
					}
					checkf(NumValid == UpdatedOld.Num(), TEXT("Weak pointer validity changed"));

					PostReloadRequests.Add({Request.New, MakeArrayView(UpdatedOld)});
				}

				WeakIndex += Request.Old.Num();
			}
			checkf(WeakIndex == ObjectsToConsolidateWeakList.Num(), TEXT("Tested wrong number of weak pointers"));

			Requests = PostReloadRequests;
		}

		FForceReplaceInfo ReplaceInfo;

		bool bNeedsGarbageCollection = false;

		// Scope the reregister context below to complete after object deletion and before garbage collection
		{
			// Replacing references inside already loaded objects could cause rendering issues, so globally detach all components from their scenes for now
			FGlobalComponentRecreateRenderStateContext ReregisterContext;

			for (FReplaceRequest Request : Requests)
			{
				UObject* ObjectToConsolidateTo = Request.New;
				TArrayView<UObject*> ObjectsToConsolidate = Request.Old;

				// First, make sure that the class we're consolidating to has its hierarchy fixed so
				// that we don't create a cycle (e.g. directly or indirectly parent it to itself):
				UClass* ClassToConsolidateTo = nullptr;
				if (UBlueprint* BlueprintObject = Cast<UBlueprint>(ObjectToConsolidateTo))
				{
					ClassToConsolidateTo = BlueprintObject->GeneratedClass;
					if (!ClassToConsolidateTo)
					{
						ClassToConsolidateTo = UObject::StaticClass();
					}

					// Don't parent a blueprint to itself, instead fall back to the part of the
					// hierarchy that is not being consolidated. Worst case, fall back to
					// UObject::StaticClass():
					UClass* NewParent = BlueprintObject->ParentClass;
					UClass* OldParent = BlueprintObject->ParentClass;
					UClass* ParentIter = NewParent;
					while (ParentIter)
					{
						if (ObjectsToConsolidate.Contains(ParentIter->ClassGeneratedBy))
						{
							NewParent = ParentIter->GetSuperClass();
						}
						ParentIter = ParentIter->GetSuperClass();
					}

					if (!NewParent || ObjectsToConsolidate.Contains(NewParent->ClassGeneratedBy))
					{
						NewParent = UObject::StaticClass();
					}

					if (OldParent != NewParent)
					{
						BlueprintObject->ParentClass = NewParent;
						FKismetEditorUtilities::CompileBlueprint(BlueprintObject, EBlueprintCompileOptions::SkipGarbageCollection);
					}
				}
			
				// Then reparent any direct children to the class we're consolidating to:
				for (UObject* Object : ObjectsToConsolidate)
				{
					if (UBlueprint* BlueprintObject = Cast<UBlueprint>(Object))
					{
						if (BlueprintObject->ParentClass != nullptr && BlueprintObject->GeneratedClass)
						{
							TArray<UClass*> ChildClasses;
							GetDerivedClasses(BlueprintObject->GeneratedClass, ChildClasses, false);
							for(UClass* ChildClass : ChildClasses)
							{
								UBlueprint* ChildBlueprint = Cast<UBlueprint>(ChildClass->ClassGeneratedBy);
								if (ChildBlueprint != nullptr 
									&& !ChildClass->HasAnyClassFlags(CLASS_NewerVersionExists)
									&& (!ObjectsToNotConsolidateWithin.Contains(ChildBlueprint)))
								{
									// Do not reparent and recompile a Blueprint that is going to be deleted.
									if (ObjectsToConsolidate.Find(ChildBlueprint) == INDEX_NONE)
									{
										ChildBlueprint->Modify();

										UClass* NewParent = ClassToConsolidateTo;

										if (!NewParent)
										{
											NewParent = UObject::StaticClass();
										}

										ChildBlueprint->ParentClass = NewParent;
										FKismetEditorUtilities::CompileBlueprint(ChildBlueprint, EBlueprintCompileOptions::SkipGarbageCollection);

										// Defer garbage collection until after we're done processing the list of objects
										bNeedsGarbageCollection = true;
									}
								}
							}
						}
					}
				}
			
			}

			
			ForceReplaceReferences(Requests, ObjectsToConsolidateWithin, ReplaceInfo, bWarnAboutRootSet);

			for (FReplaceRequest Request : Requests)
			{
				TArrayView<UObject*> ObjectsToConsolidate = Request.Old;
				if (UBlueprint* ObjectToConsolidateTo_BP = Cast<UBlueprint>(Request.New))
				{
					// Replace all UClass/TSubClassOf properties of generated class.
					TArray<UObject*> ObjectsToConsolidate_BP;
					TArray<UClass*> OldGeneratedClasses;
					TMap<UClass*, UClass*> OldChildClassToOldParentClass;
					ObjectsToConsolidate_BP.Reserve(ObjectsToConsolidate.Num());
					OldGeneratedClasses.Reserve(ObjectsToConsolidate.Num());
					for (UObject* ObjectToConsolidate : ObjectsToConsolidate)
					{
						UClass* OldGeneratedClass = Cast<UBlueprint>(ObjectToConsolidate)->GeneratedClass;
						ObjectsToConsolidate_BP.Add(OldGeneratedClass);
						OldGeneratedClasses.Add(OldGeneratedClass);

						TArray<UClass*> OldChildClasses;
						GetDerivedClasses(OldGeneratedClass, OldChildClasses, false);
						for (UClass* OldChildClass : OldChildClasses)
						{
							OldChildClassToOldParentClass.Add(OldChildClass, OldGeneratedClass);
						}
					}
					
					FForceReplaceInfo GeneratedClassReplaceInfo;
					ForceReplaceReferences(ObjectToConsolidateTo_BP->GeneratedClass, ObjectsToConsolidate_BP, ObjectsToConsolidateWithin, GeneratedClassReplaceInfo, bWarnAboutRootSet);

					// Repair the references of GeneratedClass on the object being consolidated so they can be properly disposed of upon deletion.
					for (int32 Index = 0, MaxIndex = ObjectsToConsolidate.Num(); Index < MaxIndex; ++Index)
					{
						Cast<UBlueprint>(ObjectsToConsolidate[Index])->GeneratedClass = OldGeneratedClasses[Index];
					}

					// repair superstruct references:
					for (const TPair<UClass*,UClass*>& OldChild : OldChildClassToOldParentClass)
					{
						OldChild.Key->SetSuperStruct(OldChild.Value);
					}

					ReplaceInfo.AppendUnique(GeneratedClassReplaceInfo);

					// Find and cache all Blueprints that have a new dependency on the consolidation target after reference replacement.
					TArray<UBlueprint*> DependentBPs;
					FBlueprintEditorUtils::FindDependentBlueprints(ObjectToConsolidateTo_BP, DependentBPs);
					for (UBlueprint* DependentBP : DependentBPs)
					{
						ObjectToConsolidateTo_BP->CachedDependents.Add(DependentBP);
					}
				}
			}

			DirtiedPackages.Append( ReplaceInfo.DirtiedPackages );
			UnconsolidatableObjects.Append( ReplaceInfo.UnreplaceableObjects );
		}

		for (FReplaceRequest Request : Requests)
		{
			// See if this is a blueprint consolidate and replace instances of the generated class
			UBlueprint* BlueprintToConsolidateTo = Cast<UBlueprint>(Request.New);
			if (BlueprintToConsolidateTo && BlueprintToConsolidateTo->GeneratedClass)
			{
				for (UObject* Old : Request.Old)
				{
					UBlueprint* BlueprintToConsolidate = Cast<UBlueprint>(Old);
					if (BlueprintToConsolidate && BlueprintToConsolidate->GeneratedClass)
					{
						// Replace all instances of objects based on the old blueprint's class with objects based on the new class,
						// then repair the references on the object being consolidated so those objects can be properly disposed of upon deletion.
						UClass* OldClass = BlueprintToConsolidate->GeneratedClass;
						UClass* OldSkeletonClass = BlueprintToConsolidate->SkeletonGeneratedClass;

						FReplaceInstancesOfClassParameters ReplaceInstanceParams;
						ReplaceInstanceParams.ObjectsThatShouldUseOldStuff = &ObjectsToNotConsolidateWithin;
						ReplaceInstanceParams.bPreserveRootComponent = true;
						ReplaceInstanceParams.InstancesThatShouldUseOldClass = &ObjectsToNotConsolidateWithin;

						FBlueprintCompileReinstancer::ReplaceInstancesOfClass(OldClass, BlueprintToConsolidateTo->GeneratedClass, ReplaceInstanceParams);
						BlueprintToConsolidate->GeneratedClass = OldClass;
						BlueprintToConsolidate->SkeletonGeneratedClass = OldSkeletonClass;
					}
				}

				bNeedsGarbageCollection = true;
			}
		}

		if (bNeedsGarbageCollection)
		{
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
		}

		FEditorDelegates::OnAssetsPreDelete.Broadcast(ReplaceInfo.ReplaceableObjects);

		TSet<FString> AlreadyMappedObjectPaths;

		if (bShouldDeleteAfterConsolidate)
		{
			// Bit wasteful, rebuild same map that existed temporarily inside ForceReplaceReferences 
			TMap<UObject*, UObject*> ReplacementMap;
			ReplacementMap.Reserve(ReplaceInfo.ReplaceableObjects.Num());
			for (FReplaceRequest Request : Requests)
			{
				for (UObject* Old : Request.Old)
				{
					ReplacementMap.Add(Old, Request.New);
				}
			}

			// With all references to the objects to consolidate to eliminated from objects that are currently loaded, it should now be safe to delete
			// the objects to be consolidated themselves, leaving behind a redirector in their place to fix up objects that were not currently loaded at the time
			// of this operation.
			for ( TArray<UObject*>::TConstIterator ConsolIter( ReplaceInfo.ReplaceableObjects ); ConsolIter; ++ConsolIter )
			{
				GWarn->StatusUpdate( ConsolIter.GetIndex(), ReplaceInfo.ReplaceableObjects.Num(), NSLOCTEXT("UnrealEd", "ConsolidateAssetsUpdate_DeletingObjects", "Deleting Assets...") );

				UObject* CurObjToConsolidate = *ConsolIter;
				UObject* CurObjOuter = CurObjToConsolidate->GetOuter();
				UPackage* CurObjPackage = CurObjToConsolidate->GetOutermost();
				const FName CurObjName = CurObjToConsolidate->GetFName();
				const FString CurObjPath = CurObjToConsolidate->GetPathName();
				UBlueprint* BlueprintToConsolidate = Cast<UBlueprint>(CurObjToConsolidate);

				// Attempt to delete the object that was consolidated
				if ( DeleteSingleObject( CurObjToConsolidate ) )
				{
					// DONT GC YET!!! we still need these objects around to notify other tools that they are gone and to create redirectors
					ConsolidatedObjects.Add(CurObjToConsolidate);

					if ( AlreadyMappedObjectPaths.Contains(CurObjPath) )
					{
						continue;
					}

					UObject* ObjectToConsolidateTo = ReplacementMap.FindChecked(CurObjToConsolidate);
					UBlueprint* BlueprintToConsolidateTo = Cast<UBlueprint>(ObjectToConsolidateTo);

					// Create a redirector with a unique name
					// It will have the same name as the object that was consolidated after the garbage collect
					UObjectRedirector* Redirector = NewObject<UObjectRedirector>(CurObjOuter, NAME_None, RF_Standalone | RF_Public);
					check( Redirector );

					// Set the redirector to redirect to the object to consolidate to
					Redirector->DestinationObject = ObjectToConsolidateTo;

					// Keep track of the object name so we can rename the redirector later
					RedirectorToObjectNameMap.Add(Redirector, CurObjName);
					AlreadyMappedObjectPaths.Add(CurObjPath);

					// If consolidating blueprints, make sure redirectors are created for the consolidated blueprint class and CDO
					if ( BlueprintToConsolidateTo != NULL && BlueprintToConsolidate != NULL )
					{
						// One redirector for the class
						UObjectRedirector* ClassRedirector = NewObject<UObjectRedirector>(CurObjOuter, NAME_None, RF_Standalone | RF_Public);
						check( ClassRedirector );
						ClassRedirector->DestinationObject = BlueprintToConsolidateTo->GeneratedClass;
						RedirectorToObjectNameMap.Add(ClassRedirector, BlueprintToConsolidate->GeneratedClass->GetFName());
						AlreadyMappedObjectPaths.Add(BlueprintToConsolidate->GeneratedClass->GetPathName());

						// One redirector for the CDO
						UObjectRedirector* CDORedirector = NewObject<UObjectRedirector>(CurObjOuter, NAME_None, RF_Standalone | RF_Public);
						check( CDORedirector );
						CDORedirector->DestinationObject = BlueprintToConsolidateTo->GeneratedClass->GetDefaultObject();
						RedirectorToObjectNameMap.Add(CDORedirector, BlueprintToConsolidate->GeneratedClass->GetDefaultObject()->GetFName());
						AlreadyMappedObjectPaths.Add(BlueprintToConsolidate->GeneratedClass->GetDefaultObject()->GetPathName());
					}

					DirtiedPackages.AddUnique( CurObjPackage );
				}
				// If the object couldn't be deleted, store it in the array that will be used to show the user which objects had errors
				else
				{
					CriticalFailureObjects.Add( CurObjToConsolidate );
				}
			}

			// Prevent newly created redirectors from being GC'ed before we can rename them
			TArray<TStrongObjectPtr<UObjectRedirector>> Redirectors;
			Redirectors.Reserve(RedirectorToObjectNameMap.Num());
			for (TMap<UObjectRedirector*, FName>::TIterator RedirectIt(RedirectorToObjectNameMap); RedirectIt; ++RedirectIt)
			{
				UObjectRedirector* Redirector = RedirectIt.Key();
				Redirectors.Add(TStrongObjectPtr<UObjectRedirector>(Redirector));
			}

			TArray<UPackage*> PotentialPackagesToDelete;
			for ( int32 ObjIdx = 0; ObjIdx < ConsolidatedObjects.Num(); ++ObjIdx )
			{
				PotentialPackagesToDelete.AddUnique(ConsolidatedObjects[ObjIdx]->GetOutermost());
			}

			CleanupAfterSuccessfulDelete(PotentialPackagesToDelete);

			// Now that the old objects have been garbage collected, give the redirectors a proper name
			for (TMap<UObjectRedirector*, FName>::TIterator RedirectIt(RedirectorToObjectNameMap); RedirectIt; ++RedirectIt)
			{
				UObjectRedirector* Redirector = RedirectIt.Key();
				const FName ObjName = RedirectIt.Value();

				if ( Redirector->Rename(*ObjName.ToString(), NULL, REN_Test) )
				{
					Redirector->Rename(*ObjName.ToString(), NULL, REN_DontCreateRedirectors | REN_NonTransactional | REN_AllowPackageLinkerMismatch);
					FAssetRegistryModule::AssetCreated(Redirector);
				}
				else
				{
					// Could not rename the redirector back to the original object's name. This indicates the original
					// object could not be garbage collected even though DeleteSingleObject returned true.
					CriticalFailureObjects.AddUnique(Redirector);
				}
			}

			Redirectors.Empty();

		}

		ConsolidatedObjects.Empty();

		GWarn->EndSlowTask();

		ConsolidationResults.DirtiedPackages = ObjectPtrWrap(DirtiedPackages);
		ConsolidationResults.FailedConsolidationObjs = ObjectPtrWrap(CriticalFailureObjects);
		ConsolidationResults.InvalidConsolidationObjs = ObjectPtrWrap(UnconsolidatableObjects);

		// If some objects failed to consolidate, notify the user of the failed objects
		if ( UnconsolidatableObjects.Num() > 0 )
		{
			FString FailedObjectNames;
			for ( TArray<UObject*>::TConstIterator FailedIter( UnconsolidatableObjects ); FailedIter; ++FailedIter )
			{
				UObject* CurFailedObject = *FailedIter;
				FailedObjectNames += CurFailedObject->GetName() + TEXT("\n");
			}

			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("Objects"), FText::FromString( FailedObjectNames ));
			FText MessageFormatting = NSLOCTEXT("ObjectTools", "ConsolidateAssetsFailureDlgMFormattings", "The assets below were unable to be consolidated. This is likely because they are referenced by the object to consolidate to.\n\n{Objects}");
			FText Message = FText::Format( MessageFormatting, Arguments );
			FText Title = NSLOCTEXT("ObjectTools", "ConsolidateAssetsFailureDlg_Title", "Failed to Consolidate Assets");

			if (bShouldShowDialogs)
			{
				FMessageDialog::Open(EAppMsgType::Ok, Message, Title);
			}
			else
			{
				UE_LOGF(LogObjectTools, Warning, "Failed to consolidate assets: %ls", *Message.ToString());
			}
		}

		// Alert the user to critical object failure
		if ( CriticalFailureObjects.Num() > 0 )
		{
			FString CriticalFailedObjectNames;
			for ( TArray<UObject*>::TConstIterator FailedIter( CriticalFailureObjects ); FailedIter; ++FailedIter )
			{
				const UObject* CurFailedObject = *FailedIter;
				CriticalFailedObjectNames += CurFailedObject->GetName() + TEXT("\n");
			}

			FString DirtiedPackageNames;
			for (const FString& UserFacingPath : IAssetTools::Get().GetUserFacingLongPackageNames(DirtiedPackages))
			{
				DirtiedPackageNames += UserFacingPath + TEXT("\n");
			}

			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("Assets"), FText::FromString( CriticalFailedObjectNames ));
			Arguments.Add(TEXT("Packages"), FText::FromString( DirtiedPackageNames ));
			FText MessageFormatting = NSLOCTEXT("ObjectTools", "ConsolidateAssetsCriticalFailureDlgMsgFormatting", "CRITICAL FAILURE:\nOne or more assets were partially consolidated, yet still cannot be deleted for some reason. It is highly recommended that you restart the editor without saving any of the assets or packages.\n\nAffected Assets:\n{Assets}\n\nPotentially Affected Packages:\n{Packages}");
			FText Message = FText::Format( MessageFormatting, Arguments );
			FText Title = NSLOCTEXT("ObjectTools", "ConsolidateAssetsCriticalFailureDlg_Title", "Critical Failure to Consolidate Assets");

			if (bShouldShowDialogs)
			{
				FMessageDialog::Open(EAppMsgType::Ok, Message, Title);
			}
			else
			{
				UE_LOGF(LogObjectTools, Warning, "Failed to consolidate assets: %ls", *Message.ToString());
			}
		}

		return ConsolidationResults;
	}

	FConsolidationResults ConsolidateObjects(UObject* ObjectToConsolidateTo, TArray<UObject*>& ObjectsToConsolidate, TSet<UObject*>& ObjectsToConsolidateWithin, TSet<UObject*>& ObjectsToNotConsolidateWithin, bool bShouldDeleteAfterConsolidate, bool bWarnAboutRootSet)
	{
		if (!ObjectToConsolidateTo)
		{
			return FConsolidationResults();
		}

		// Empty the provided array so it's not full of pointers to deleted objects
		ON_SCOPE_EXIT{ ObjectsToConsolidate.Empty(); };

		FReplaceRequest Request = {ObjectToConsolidateTo, ObjectsToConsolidate};
		return ConsolidateObjects(MakeArrayView(&Request, 1), ObjectsToConsolidateWithin, ObjectsToNotConsolidateWithin, bShouldDeleteAfterConsolidate, bWarnAboutRootSet);
	}

	FConsolidationResults ConsolidateObjects(UObject* ObjectToConsolidateTo, TArray<UObject*>& ObjectsToConsolidate, bool bShowDeleteConfirmation)
	{
		FConsolidationResults ConsolidationResults;

		// Ensure the consolidation is headed toward a valid object and this isn't occurring in game
		if (ObjectToConsolidateTo)
		{
			// Confirm that the consolidate was intentional
			if (bShowDeleteConfirmation)
			{
				if (!ShowDeleteConfirmationDialog(ObjectsToConsolidate))
				{
					return ConsolidationResults;
				}
			}
			TSet<UObject*> ObjectsToConsolidateWithin;
			TSet<UObject*> ObjectsToNotConsolidateWithin;
			return ConsolidateObjects(ObjectToConsolidateTo, ObjectsToConsolidate, ObjectsToConsolidateWithin, ObjectsToNotConsolidateWithin, true);
		}

		return ConsolidationResults;
	}

	void CompileBlueprintsAfterRefUpdate(const TArray<UObject*>& ObjectsConsolidatedWithin)
	{
		for (UObject* CurObject : ObjectsConsolidatedWithin)
		{
			if (CurObject)
			{
				UBlueprint* BPObjectToUpdate = Cast<UBlueprint>(CurObject);
				if (BPObjectToUpdate)
				{
					FKismetEditorUtilities::CompileBlueprint(BPObjectToUpdate);
				}
			}
		}
	}

	/**
	 * Copies references for selected generic browser objects to the clipboard.
	 */
	void CopyReferences( const TArray< UObject* >& SelectedObjects ) // const
	{
		FString Ref;
		for ( int32 Index = 0 ; Index < SelectedObjects.Num() ; ++Index )
		{
			if( Ref.Len() )
			{
				Ref += LINE_TERMINATOR;
			}
			Ref += SelectedObjects[Index]->GetPathName();
		}

		FPlatformApplicationMisc::ClipboardCopy( *Ref );
	}

	/**
	 * Show the referencers of a selected object
	 *
	 * @param SelectedObjects	Array of the currently selected objects; the referencers of the first object are shown
	 */
	void ShowReferencers( const TArray< UObject* >& SelectedObjects ) // const
	{
		if( SelectedObjects.Num() > 0 )
		{
			UObject* Object = SelectedObjects[ 0 ];
			if ( Object )
			{
				GEditor->GetSelectedObjects()->Deselect( Object );

				CollectGarbage( GARBAGE_COLLECTION_KEEPFLAGS );

				FReferencerInformationList Refs;

				if (IsReferenced(Object, RF_Public, EInternalObjectFlags::Native, true, &Refs))
				{
					FStringOutputDevice Ar;
					Object->OutputReferencers(Ar, &Refs);
					UE_LOGF(LogObjectTools, Warning, "%ls", *Ar);  // also print the objects to the log so you can actually utilize the data

					// Display a dialog containing all referencers; the dialog is designed to destroy itself upon being closed, so this
					// allocation is ok and not a memory leak
					SGenericDialogWidget::OpenDialog(NSLOCTEXT("ObjectTools", "ShowReferencers", "Show Referencers"), SNew(SEditableTextBox).Text(FText::FromString(Ar)).IsReadOnly(true));
				}
				else
				{
					FMessageDialog::Open(EAppMsgType::Ok, FText::Format(NSLOCTEXT("UnrealEd", "ObjectNotReferenced", "Object '{0}' Is Not Referenced"), FText::FromString(Object->GetName())));
				}

				GEditor->GetSelectedObjects()->Select( Object );
			}
		}
	}

	/**
	 * Displays a tree(currently) of all assets which reference the passed in object.
	 *
	 * @param ObjectToGraph		The object to find references to.
	 */
	void ShowReferenceGraph( UObject* ObjectToGraph )
	{
		SReferenceTree::OpenDialog(ObjectToGraph);
	}

	/**
	 * Displays all of the objects the passed in object references
	 *
	 * @param	Object			Object whose references should be displayed
	 */
	void ShowReferencedObjs(UObject* Object)
	{
		ShowReferencedObjs(Object, nullptr, NAME_None, ECollectionShareType::CST_Private);
	}

	/**
	 * Displays all of the objects the passed in object references
	 *
	 * @param	Object			Object whose references should be displayed
	 * @param	CollectionName	Name of a collection that needs to be made with these referenced objects
	 * @param	ShareType		The share type of any created collection
	 */
	void ShowReferencedObjs(UObject* Object, const FString& CollectionName, ECollectionShareType::Type ShareType)
	{
		if (CollectionName.Len() == 0)
		{
			ShowReferencedObjs(Object, nullptr, NAME_None, ShareType);
		}
		else
		{
			ShowReferencedObjs(Object, &FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer().Get(), FName(CollectionName), ShareType);
		}
	}

	/**
	 * Displays all of the objects the passed in object references
	 *
	 * @param	Object				Object whose references should be displayed
	 * @param	CollectionContainer	The collection container for any created collection
	 * @param	CollectionName		Name of a collection that needs to be made with these referenced objects
	 * @param	ShareType			The share type of any created collection
	 */
	void ShowReferencedObjs( UObject* Object, ICollectionContainer* CollectionContainer, FName CollectionName, ECollectionShareType::Type ShareType )
	{
		if( Object )
		{
			GEditor->GetSelectedObjects()->Deselect( Object );

			// Find references.
			TSet<UObject*> ReferencedObjects;
			{
				const FScopedBusyCursor BusyCursor;
				TArray<UClass*> IgnoreClasses;
				TArray<FString> IgnorePackageNames;
				TArray<UPackage*> IgnorePackages;

				// Assemble an ignore list.
				IgnoreClasses.Add( ULevel::StaticClass() );
				IgnoreClasses.Add( UWorld::StaticClass() );
				IgnoreClasses.Add( UPhysicalMaterial::StaticClass() );

				// Load the asset registry module
				FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

				TArray<FAssetData> AssetData;
				FARFilter Filter;
				Filter.PackagePaths.Add(FName(TEXT("/Engine/EngineMaterials")));
				Filter.PackagePaths.Add(FName(TEXT("/Engine/EditorMeshes")));
				Filter.PackagePaths.Add(FName(TEXT("/Engine/EditorResources")));
				Filter.PackagePaths.Add(FName(TEXT("/Engine/EngineMaterials")));
				Filter.PackagePaths.Add(FName(TEXT("/Engine/EngineFonts")));
				Filter.PackagePaths.Add(FName(TEXT("/Engine/EngineResources")));

				AssetRegistryModule.Get().GetAssets(Filter, AssetData);

				for (int32 AssetIdx = 0; AssetIdx < AssetData.Num(); ++AssetIdx)
				{
					IgnorePackageNames.Add( AssetData[AssetIdx].PackageName.ToString() );
				}

				// Construct the ignore package list.
				for( int32 PackageNameItr = 0; PackageNameItr < IgnorePackageNames.Num(); ++PackageNameItr )
				{
					UPackage* PackageToIgnore = FindObject<UPackage>(NULL,*(IgnorePackageNames[PackageNameItr]),EFindObjectFlags::ExactClass);

					if( PackageToIgnore == NULL )
					{// An invalid package name was provided.
						UE_LOGF(LogObjectTools, Log,  "Package to ignore \"%ls\" in the list of referenced objects is NULL and should be removed from the list", *(IgnorePackageNames[PackageNameItr]) );
					}
					else
					{
						IgnorePackages.Add(PackageToIgnore);
					}
				}

				FFindReferencedAssets::BuildAssetList( Object, IgnoreClasses, IgnorePackages, ReferencedObjects );
			}

			const int32 NumReferencedObjects = ReferencedObjects.Num();

			// Make sure that the only referenced object (if there's only one) isn't the object itself before outputting object references
			if ( NumReferencedObjects > 1 || ( NumReferencedObjects == 1 && !ReferencedObjects.Contains( Object ) ) )
			{
				if (CollectionName == NAME_None)
				{
					FString OutString( FString::Printf( TEXT("\nObjects referenced by %s:\r\n"), *Object->GetFullName() ) );
					for(TSet<UObject*>::TConstIterator SetIt(ReferencedObjects); SetIt; ++SetIt)
					{
						const UObject *ReferencedObject = *SetIt;
						check(ReferencedObject);

						// Don't list an object as referring to itself.
						if ( ReferencedObject != Object )
						{
							OutString += FString::Printf( TEXT("\t%s:\r\n"), *ReferencedObject->GetFullName() );
						}
					}

					UE_LOGF(LogObjectTools, Warning, "%ls", *OutString );

					// Display the object references in a copy-friendly dialog; the dialog is designed to destroy itself upon being closed, so this
					// allocation is ok and not a memory leak
					SGenericDialogWidget::OpenDialog(NSLOCTEXT("ObjectTools", "ShowReferencedAssets", "Show Referenced Assets"), SNew(SEditableTextBox).Text(FText::FromString(OutString)).IsReadOnly(true));
				}
				else if (ensure(CollectionContainer))
				{
					TArray<FSoftObjectPath> ObjectsToAdd;
					for(TSet<UObject*>::TConstIterator SetIt(ReferencedObjects); SetIt; ++SetIt)
					{
						UObject* RefObj = *SetIt;
						if (RefObj != NULL)
						{
							if (RefObj != Object)
							{
								ObjectsToAdd.Emplace(RefObj);
							}
						}
					}

					if (ObjectsToAdd.Num() > 0)
					{
						FContentHelper ContentHelper(CollectionContainer->AsShared());
						if (ContentHelper.Initialize() == true)
						{
							ContentHelper.ClearCollection(CollectionName, ShareType);
							const bool CollectionCreated = ContentHelper.SetCollection(CollectionName, ShareType, ObjectsToAdd);

							// Notify the user whether the collection was successfully created
							FNotificationInfo Info( FText::Format( NSLOCTEXT("ObjectTools", "SuccessfulAddCollection", "{0} sucessfully added as a new collection."), FText::FromName(CollectionName)) );
							Info.ExpireDuration = 3.0f;
							Info.bUseLargeFont = false;

							if ( !CollectionCreated )
							{
								ISourceControlModule& SourceControlModule = ISourceControlModule::Get();
								if ( !SourceControlModule.IsEnabled() && ShareType != ECollectionShareType::CST_Local )
								{
									// Private and Shared collection types require a source control connection
									Info.Text = NSLOCTEXT("ObjectTools", "FailedToAddCollection_SCC", "Failed to create new collection, requires revision control connection");
								}
								else
								{
									Info.Text = NSLOCTEXT("ObjectTools", "FailedToAddCollection_Unknown", "Failed to create new collection");
								}
							}

							TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
							if ( Notification.IsValid() )
							{
								Notification->SetCompletionState( CollectionCreated ? SNotificationItem::CS_Success : SNotificationItem::CS_Fail );
							}
						}
					}
				}
			}
			else
			{
				FMessageDialog::Open( EAppMsgType::Ok, FText::Format( NSLOCTEXT("UnrealEd", "ObjectNoReferences", "Object '{0}' doesn't refer to any non-ignored objects."), FText::FromString(Object->GetName()) ) );
			}

			GEditor->GetSelectedObjects()->Select( Object );
		}
	}

	/**
	 * Select the object referencers in the level
	 *
	 * @param	Object			Object whose references are to be selected
	 *
	 */
	void SelectActorsInLevelDirectlyReferencingObject( UObject* RefObj )
	{
		UPackage* Package = RefObj->GetOutermost();
		if (Package && Package->ContainsMap())
		{
			// Walk the chain of outers to find the object that is 'in' the level...
			UObject* ObjToSelect = NULL;
			UObject* CurrObject = RefObj;
			UObject* Outer = RefObj->GetOuter();
			while ((ObjToSelect == NULL) && (Outer != NULL) && (Outer != Package))
			{
				ULevel* Level = Cast<ULevel>(Outer);
				if (Level)
				{
					// We found it!
					ObjToSelect = CurrObject;
				}
				else
				{
					UObject* TempObject = Outer;
					Outer = Outer->GetOuter();
					CurrObject = TempObject;
				}
			}

			if (ObjToSelect)
			{
				AActor* ActorToSelect = Cast<AActor>(ObjToSelect);
				if (ActorToSelect)
				{
					GEditor->SelectActor( ActorToSelect, true, true );
				}
			}
		}
	}

	/**
	 * Select the object and it's external referencers' referencers in the level.
	 * This function calls AccumulateObjectReferencersForObjectRecursive to
	 * recursively build a list of objects to check for referencers in the level
	 *
	 * @param	Object				Object whose references are to be selected
	 * @param	bRecurseMaterial	Whether or not we're allowed to recurse the material
	 *
	 */
	void SelectObjectAndExternalReferencersInLevel( UObject* Object, const bool bRecurseMaterial )
	{
		if(Object)
		{
			if(IsReferenced(Object, RF_Public, EInternalObjectFlags::Native))
			{
				TArray<UObject*> ObjectsToSelect;

				GEditor->SelectNone( true, true );

				// Generate the list of objects.  This function is necessary if the object
				//	in question is indirectly referenced by an actor.  For example, a
				//	material used on a static mesh that is instanced in the level
				AccumulateObjectReferencersForObjectRecursive( Object, ObjectsToSelect, bRecurseMaterial );

				// Select the objects in the world
				for ( TArray<UObject*>::TConstIterator ObjToSelectItr( ObjectsToSelect ); ObjToSelectItr; ++ObjToSelectItr )
				{
					UObject* ObjToSelect = *ObjToSelectItr;
					SelectActorsInLevelDirectlyReferencingObject(ObjToSelect);
				}

				GEditor->GetSelectedObjects()->Select( Object );
			}
			else
			{
				FMessageDialog::Open( EAppMsgType::Ok, FText::Format(NSLOCTEXT("UnrealEd", "ObjectNotReferenced", "Object '{0}' Is Not Referenced"), FText::FromString(Object->GetName())) );
			}
		}
	}


	/**
	 * Recursively add the objects referencers to a single array
	 *
	 * @param	Object				Object whose references are to be selected
	 * @param	Referencers			Array of objects being referenced in level
	 * @param	bRecurseMaterial	Whether or not we're allowed to recurse the material
	 *
	 */
	void AccumulateObjectReferencersForObjectRecursive( UObject* Object, TArray<UObject*>& Referencers, const bool bRecurseMaterial )
	{
		TArray<FReferencerInformation> OutInternalReferencers;
		TArray<FReferencerInformation> OutExternalReferencers;
		Object->RetrieveReferencers(&OutInternalReferencers, &OutExternalReferencers);

		// dump the referencers
		for (int32 ExtIndex = 0; ExtIndex < OutExternalReferencers.Num(); ExtIndex++)
		{
			UObject* RefdObject = OutExternalReferencers[ExtIndex].Referencer;
			if (RefdObject)
			{
				Referencers.Push( RefdObject );
				// Recursively search for static meshes and materials so that textures and materials will recurse back
				// to the meshes in which they are used
				if	( !(Object->IsA(UStaticMesh::StaticClass()) ) // Added this check for safety in case of a circular reference
					&& (	(RefdObject->IsA(UStaticMesh::StaticClass()))
						||	(RefdObject->IsA(UMaterialInterface::StaticClass()) && bRecurseMaterial)	// Only recurse the material if we're interested in it's children
						)
					)
				{
					AccumulateObjectReferencersForObjectRecursive( RefdObject, Referencers, bRecurseMaterial );
				}
			}
		}
	}

	bool ShowDeleteConfirmationDialog ( const TArray<UObject*>& ObjectsToDelete )
	{
		TArray<UPackage*> PackagesToDelete;

		// Gather a list of packages which may need to be deleted once the objects are deleted.
		for ( int32 ObjIdx = 0; ObjIdx < ObjectsToDelete.Num(); ++ObjIdx )
		{
			PackagesToDelete.AddUnique(ObjectsToDelete[ObjIdx]->GetOutermost());
		}

		// Cull out packages which cannot be found on disk or are not UAssets
		for ( int32 PackageIdx = PackagesToDelete.Num() - 1; PackageIdx >= 0; --PackageIdx )
		{
			UPackage* Package = PackagesToDelete[PackageIdx];

			FString PackageFilename;
			if( !FPackageName::DoesPackageExist( Package->GetName(), &PackageFilename ) )
			{
				// Could not determine filename for package so we can not delete
				PackagesToDelete.RemoveAt(PackageIdx);
			}
		}

		// If we found any packages that we may delete
		if ( PackagesToDelete.Num() )
		{
			// Set up the delete package dialog
			FPackagesDialogModule& PackagesDialogModule = FModuleManager::LoadModuleChecked<FPackagesDialogModule>( TEXT("PackagesDialog") );
			PackagesDialogModule.CreatePackagesDialog(NSLOCTEXT("PackagesDialogModule", "DeleteAssetsDialogTitle", "Delete Assets"), NSLOCTEXT("PackagesDialogModule", "DeleteAssetsDialogMessage", "The following assets will be deleted."), /*InReadOnly=*/true);
			PackagesDialogModule.AddButton(DRT_Save, NSLOCTEXT("PackagesDialogModule", "DeleteSelectedButton", "Delete"), NSLOCTEXT("PackagesDialogModule", "DeleteSelectedButtonTip", "Delete the listed assets"));
			if(!ISourceControlModule::Get().IsEnabled())
			{
				PackagesDialogModule.AddButton(DRT_MakeWritable, NSLOCTEXT("PackagesDialogModule", "MakeWritableAndDeleteSelectedButton", "Make Writable and Delete"), NSLOCTEXT("PackagesDialogModule", "MakeWritableAndDeleteSelectedButtonTip", "Makes the listed assets writable and deletes them"));
			}
			PackagesDialogModule.AddButton(DRT_Cancel, NSLOCTEXT("PackagesDialogModule", "CancelButton", "Cancel"), NSLOCTEXT("PackagesDialogModule", "CancelDeleteButtonTip", "Do not delete any assets and cancel the current operation"));

			for ( int32 PackageIdx = 0; PackageIdx < PackagesToDelete.Num(); ++PackageIdx )
			{
				UPackage* Package = PackagesToDelete[PackageIdx];
				PackagesDialogModule.AddPackageItem(Package, ECheckBoxState::Checked);
			}

			// Display the delete dialog
			const EDialogReturnType UserResponse = PackagesDialogModule.ShowPackagesDialog();

			if(UserResponse == DRT_MakeWritable)
			{
				// make each file writable before attempting to delete
				for ( int32 PackageIdx = 0; PackageIdx < PackagesToDelete.Num(); ++PackageIdx )
				{
					const UPackage* Package = PackagesToDelete[PackageIdx];
					FString PackageFilename;
					if(FPackageName::DoesPackageExist(Package->GetName(), &PackageFilename))
					{
						FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*PackageFilename, false);
					}
				}
			}

			// If the user selected a "Delete" option return true
			return UserResponse == DRT_Save || UserResponse == DRT_MakeWritable;
		}
		else
		{
			// There are no packages that are considered for deletion. Return true because this is a safe delete.
			return true;
		}
	}

	void CleanupAfterSuccessfulDelete (const TArray<UPackage*>& PotentialPackagesToDelete, bool bPerformReferenceCheck)
	{
		TArray<UPackage*> PackagesToDelete = PotentialPackagesToDelete;
		TArray<UPackage*> PackagesToUnload;
		TArray<FString> PackageFilesToDelete;
		TArray<TSharedRef<ISourceControlState, ESPMode::ThreadSafe> > PackageSCCStates;
		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

		GWarn->BeginSlowTask( NSLOCTEXT("ObjectTools", "OldPackageCleanupSlowTask", "Cleaning Up Old Assets"), true );
		const int32 OriginalNumPackagesToDelete = PackagesToDelete.Num();
		// Cull out packages which are still referenced, dont exist on disk, or are not UAssets
		// Record the filename and SCC state of any package which is not culled.
		for ( int32 PackageIdx = PackagesToDelete.Num() - 1; PackageIdx >= 0; --PackageIdx )
		{
			GWarn->StatusUpdate(OriginalNumPackagesToDelete - PackageIdx, OriginalNumPackagesToDelete, NSLOCTEXT("ObjectTools", "OldPackageCleanupSlowTask", "Cleaning Up Old Assets"));
			UPackage* Package = PackagesToDelete[PackageIdx];

			bool bIsReferenced = false;

			auto ShouldSkipReferenceCheck = [](const UPackage* InPackage)
			{
				auto IsExternalPackage = [](const UPackage* Package) -> bool
				{
					bool bIsExternalPackage = false;
					ForEachObjectWithPackage(Package, [Package, &bIsExternalPackage](UObject* InObject)
					{
						bIsExternalPackage = InObject->GetOutermostObject()->GetPackage() != Package;
						return !bIsExternalPackage;
					});
					return bIsExternalPackage;
				};

				// Skip external object packages when considering whether to clear the transaction buffer, as you should be able to undo deleting an actor, an actor folder, etc. (but not an asset)
				// If an external object package is kept alive by the transaction buffer then it will be re-marked as "newly created" further down this function
				return IsExternalPackage(InPackage);
			};

			if ( Package != nullptr && bPerformReferenceCheck && !ShouldSkipReferenceCheck(Package) )
			{
				bool bIsReferencedByUndo = false;
				GatherObjectReferencersForDeletion(Package, bIsReferenced, bIsReferencedByUndo);

				// only ref to this object is the transaction buffer, clear the transaction buffer
				if (!bIsReferenced && bIsReferencedByUndo && GEditor)
				{
					GEditor->ResetTransaction(NSLOCTEXT("UnrealEd", "DeleteSelectedItem", "Delete Selected Item"));
				}
			}

			if ( bIsReferenced )
			{
				PackagesToDelete.RemoveAt(PackageIdx);
			}
			else
			{
				UPackage* CurrentPackage = Cast<UPackage>(Package);

				FString PackageFilename;

				if( CurrentPackage == nullptr )
				{
					PackagesToDelete.RemoveAt(PackageIdx);
				}
				else
				{
					CurrentPackage->SetDirtyFlag(false);

					if (FPackageName::DoesPackageExist(Package->GetName(), &PackageFilename))
					{
						PackageFilesToDelete.Add(PackageFilename);
					}
					else
					{
						// Could not determine filename for package so we can not delete, but we should unload
						PackagesToDelete.RemoveAt(PackageIdx);
						PackagesToUnload.Add(CurrentPackage);
					}
				}
			}
		}

		// Get the current source control states of all the package files we're deleting at once.
		if ( PackagesToDelete.Num() && ISourceControlModule::Get().IsEnabled() )
		{
			SourceControlProvider.GetState(PackageFilesToDelete, PackageSCCStates, EStateCacheUsage::ForceUpdate);
		}

		GWarn->EndSlowTask();

		if (GUnrealEd)
		{
			// Let the package auto-saver know that it needs to ignore the deleted packages
			GUnrealEd->GetPackageAutoSaver().OnPackagesDeleted(PackagesToDelete);
		}

		// Let the asset registry know that these packages are being removed
		for (UPackage* PackageToDelete : PackagesToDelete)
		{
			FAssetRegistryModule::PackageDeleted(PackageToDelete);
			FEditorDelegates::OnPackageDeleted.Broadcast(PackageToDelete);
		}

		// Unload the packages and collect garbage.
		if ( PackagesToDelete.Num() > 0 || PackagesToUnload.Num() > 0 )
		{
			TArray<UPackage*> AllPackagesToUnload;
			AllPackagesToUnload.Reserve(PackagesToDelete.Num() + PackagesToUnload.Num());
			AllPackagesToUnload.Append(PackagesToDelete);
			AllPackagesToUnload.Append(PackagesToUnload);

			UPackageTools::FUnloadPackageParams UnloadParams(AllPackagesToUnload);
			UnloadParams.bResetTransBuffer = false; // Don't reset the transaction buffer, as we handled that above if needed
			UPackageTools::UnloadPackages(UnloadParams);
		}
		CollectGarbage( GARBAGE_COLLECTION_KEEPFLAGS );

		// Now delete all packages that have become empty
		bool bMakeWritable = false;
		bool bSilent = false;
		TArray<FString> SCCFilesToRevert;
		TArray<FString> SCCFilesToDelete;

		for ( int32 PackageFileIdx = 0; PackageFileIdx < PackageFilesToDelete.Num(); ++PackageFileIdx )
		{
			bool bDeletedFileLocallyWritable = false;

			const FString& PackageFilename = PackageFilesToDelete[PackageFileIdx];
			if ( ISourceControlModule::Get().IsEnabled() )
			{
				const FSourceControlStatePtr SourceControlState = PackageSCCStates.IsValidIndex(PackageFileIdx) ? PackageSCCStates[PackageFileIdx] : FSourceControlStatePtr();
				const bool bInDepot = SourceControlState.IsValid() && SourceControlState->IsSourceControlled();
				if ( bInDepot )
				{
					check(SourceControlState.IsValid());

					// The file is managed by source control. Open it for delete.
					FString FullPackageFilename = FPaths::ConvertRelativePathToFull(PackageFilename);

					// Revert the file if it is checked out
					const bool bIsAdded = SourceControlState->IsAdded();
					const bool bIsCheckedOut = SourceControlState->IsCheckedOut();
					if ( bIsCheckedOut || bIsAdded || SourceControlState->IsDeleted() )
					{
						// Batch the revert operation so that we only make one request to the source control module.
						SCCFilesToRevert.Add(FullPackageFilename);
					}

					if (bIsAdded)
					{
						// The file was open for add and reverted, this leaves the file on disk so here we delete it
						IFileManager::Get().Delete(*PackageFilename);
					}
					else if (SourceControlState->CanDelete())
					{
						// Batch this file for deletion so that we only send one deletion request to the source control module.
						SCCFilesToDelete.Add(FullPackageFilename);
					}
					else if (!bIsCheckedOut && !IFileManager::Get().IsReadOnly(*PackageFilename))
					{
						bDeletedFileLocallyWritable = true;
					}
					else
					{
						UE_LOGF(LogObjectTools, Warning, "SCC failed to open '%ls' for deletion.", *PackageFilename);
					}
				}
				else
				{
					// The file was never submitted to the depo, delete it locally
					IFileManager::Get().Delete(*PackageFilename);
				}
			}
			else
			{
				// Source control is compiled in, but is not enabled for some reason, delete the file locally
				if(IFileManager::Get().IsReadOnly(*PackageFilename))
				{
					EAppReturnType::Type ReturnType = EAppReturnType::No;
					if(!bMakeWritable && !bSilent)
					{
						FFormatNamedArguments Args;
						Args.Add(TEXT("Filename"), FText::FromString(PackageFilename));
						const FText Message = FText::Format(NSLOCTEXT("ObjectTools", "DeleteReadOnlyWarning", "This file is read-only on disk:\n\n{Filename}\n\nDelete it anyway?"), Args);

						ReturnType = FMessageDialog::Open(EAppMsgType::YesNoYesAllNoAll, Message);
						bMakeWritable = ReturnType == EAppReturnType::YesAll;
						bSilent = ReturnType == EAppReturnType::NoAll;
					}

					if(bMakeWritable || ReturnType == EAppReturnType::Yes)
					{
						FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*PackageFilename, false);
						bDeletedFileLocallyWritable = true;
					}
				}
				else
				{
					bDeletedFileLocallyWritable = true;
				}
			}
		
			if (bDeletedFileLocallyWritable)
			{
				FUncontrolledChangelistsModule& UncontrolledChangelistsModule = FUncontrolledChangelistsModule::Get();
				UncontrolledChangelistsModule.OnDeleteWritable(PackageFilename);

				IFileManager::Get().Delete(*PackageFilename);
			}
		}
				
		// Handle all source control revert and delete operations as a batched operation.
		if (ISourceControlModule::Get().IsEnabled())
		{
			if (SCCFilesToRevert.Num() > 0)
			{
				TSharedRef<FRevert, ESPMode::ThreadSafe> RevertOperation = ISourceControlOperation::Create<FRevert>();
				RevertOperation->SetSoftRevert(SourceControlProvider.UsesSoftRevertOnDelete());

				if (SourceControlProvider.Execute(RevertOperation, SCCFilesToRevert) == ECommandResult::Failed)
				{
					UE_LOG(LogObjectTools, Warning, TEXT("SCC failed to revert the selected files before deletion."));
				}
			}

			if (SCCFilesToDelete.Num() > 0)
			{
				TSharedRef<FDelete, ESPMode::ThreadSafe> DeleteOperation = ISourceControlOperation::Create<FDelete>();

				if (SourceControlProvider.Execute(DeleteOperation, SCCFilesToDelete) == ECommandResult::Failed)
				{
					UE_LOGF(LogObjectTools, Warning, "SCC failed to open the selected files for deletion.");
				}
			}
		}

		// Ensure that any packages that had their file deleted despite still existing in memory are marked "newly created" again, since they 
		// no longer have an associated file on disk. This typically happens for OFPA packages that are kept alive by the transaction buffer.
		for (const FString& PackageFilename : PackageFilesToDelete)
		{
			if (!FPaths::FileExists(PackageFilename))
			{
				FString PackageName;
				if (FPackageName::TryConvertFilenameToLongPackageName(PackageFilename, PackageName))
				{
					if (UPackage* Package = FindPackage(nullptr, *PackageName))
					{
						Package->MarkAsNewlyCreated();
					}
				}
			}
		}

		// Let the level browser that we deleted a level (must happen after physically deleting the package file as it will rescan the folders)
		FEditorDelegates::RefreshLevelBrowser.Broadcast();
	}

	int32 DeleteAssets( const TArray<FAssetData>& AssetsToDelete, bool bShowConfirmation )
	{
		TArray<FAssetData> AllAssetsToDelete;
		const UEditorProjectAssetSettings* Settings = GetDefault<UEditorProjectAssetSettings>();
		bool bAddExtraLocalizedVariants = (Settings == nullptr || Settings->bRenameLocalizedVariantsAlongsideSourceAsset);
		if (bAddExtraLocalizedVariants && bShowConfirmation)
		{
			if (!AddExtraLocalizedVariantsToDelete(AssetsToDelete, AllAssetsToDelete))
			{
				// Either the user requested to cancel the current operation or an error occurred that requires interrupting the current operation...
				return 0;
			}
		}
		else
		{
			AllAssetsToDelete = AssetsToDelete;
		}

		TArray<TWeakObjectPtr<UPackage>> PackageFilesToDelete;
		TArray<UObject*> ObjectsToDelete;
		for ( int i = 0; i < AllAssetsToDelete.Num(); i++ )
		{
			const FAssetData& AssetData = AllAssetsToDelete[i];
			UObject *ObjectToDelete = AssetData.GetAsset({ ULevel::LoadAllExternalObjectsTag });
			// Assets can be loaded even when their underlying type/class no longer exists...
			if ( ObjectToDelete!=nullptr )
			{
				ObjectsToDelete.Add( ObjectToDelete );
			}
			else if ( AssetData.IsUAsset() )
			{
				// ... In this cases there is no underlying asset or type so remove the package itself directly after confirming it's valid to do so.
				FString PackageFilename;
				if( !FPackageName::DoesPackageExist( AssetData.PackageName.ToString(), &PackageFilename ) )
				{
					// Could not determine filename for package so we can not delete
					continue;
				}

				UPackage* Package = FindPackage(nullptr, *AssetData.PackageName.ToString());
				if ( Package )
				{
					PackageFilesToDelete.Add(Package);
				}
			}
		}

		int32 NumObjectsToDelete = ObjectsToDelete.Num();
		if ( NumObjectsToDelete > 0 )
		{
			NumObjectsToDelete = DeleteObjects( ObjectsToDelete, bShowConfirmation );
		}

		const int32 NumPackagesToDelete = PackageFilesToDelete.Num();
		if (NumPackagesToDelete > 0)
		{
			TArray<UPackage*> PackagePointers;
			for ( const auto& PkgIt : PackageFilesToDelete )
			{
				UPackage* Package = PkgIt.Get();
				if ( Package )
				{
					PackagePointers.Add(Package);
				}
			}

			if ( PackagePointers.Num() > 0 )
			{
				const bool bPerformReferenceCheck = true;
				CleanupAfterSuccessfulDelete(PackagePointers, bPerformReferenceCheck);
			}
		}

		return NumPackagesToDelete + NumObjectsToDelete;
	}

	int32 PrivatizeAssets(const TArray<FAssetData>& AssetsToPrivatize, bool bShowConfirmation, const EAssetAccessSpecifier InAssetAccessSpecifier)
	{
		TArray<UObject*> ObjectsToPrivatize;
		for (const FAssetData& AssetToPrivatize : AssetsToPrivatize)
		{
			UObject* ObjectToPrivatize = AssetToPrivatize.GetAsset();

			if (ObjectToPrivatize)
			{
				ObjectsToPrivatize.Add(ObjectToPrivatize);
			}
		}

		if (!ObjectsToPrivatize.IsEmpty())
		{
			return PrivatizeObjects(ObjectsToPrivatize, bShowConfirmation, EAllowCancelDuringPrivatize::AllowCancel, InAssetAccessSpecifier);
		}

		return 0;
	}

	void AddExtraObjectsToDelete(TArray< UObject* >& ObjectsToDelete)
	{
		// Allows to inject extra assets to delete without modifying the engine source.
		TSet<UObject*> SecondaryObjects;
		FEditorDelegates::OnAddExtraObjectsToDelete.Broadcast(ObjectsToDelete, SecondaryObjects);
		for (UObject* Object : SecondaryObjects)
		{
			ObjectsToDelete.AddUnique(Object);
		}

		// Recursively include external packages
		const int32 OriginalNum = ObjectsToDelete.Num();
		TSet<const UPackage*> ProcessedOuterPackages;
		for (int32 Index=0; Index < OriginalNum; ++Index)
		{
			const UObject* ObjectToDelete = ObjectsToDelete[Index];
			const UPackage* OuterPackage = ObjectToDelete->GetPackage();
			if (!ProcessedOuterPackages.Contains(OuterPackage))
			{
				for (UPackage* Package : OuterPackage->GetExternalPackages())
				{
					// Don't include newly created packages
					if (!Package->HasAnyPackageFlags(PKG_NewlyCreated))
					{
						ObjectsToDelete.AddUnique(Package);
					}
				}
				
				ProcessedOuterPackages.Add(OuterPackage);
			}
			
		}
	}

	bool AddExtraLocalizedVariantsToDelete(const TArray<FAssetData>& InAssetsToDelete, TArray<FAssetData>& OutAssetsWithVariantsToDelete)
	{
		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
		TSharedPtr<ILocalizedAssetTools> LocalizedAssetTools = AssetTools.GetLocalizedAssetTools();

		TArray<FName> PackageNames;
		Algo::Transform(InAssetsToDelete, PackageNames, [](const FAssetData& InAssetData) { return InAssetData.PackageName; });

		TMap<FName, TArray<FName>> VariantsBySourcesOnDisk;
		TMap<FName, TArray<FName>> VariantsBySourcesOnlyInRevisionControl;
		TArray<FName> PackagesNotFound;
		bool bNeedToCheckInRevisionControl = USourceControlPreferences::RequiresRevisionControlToRenameLocalizableAssets();
		ELocalizedAssetsResult Result = LocalizedAssetTools->GetLocalizedVariants(PackageNames, VariantsBySourcesOnDisk, bNeedToCheckInRevisionControl, VariantsBySourcesOnlyInRevisionControl, &PackagesNotFound);
		bool bRevisionControlWasNeeded = (Result == ELocalizedAssetsResult::RevisionControlNotAvailable);

		// We have all the assets and variants that we need
		TArray<FName> PackagesAndVariants;
		for (const TPair<FName, TArray<FName>>& LocalizedVariantsWithSourceAsset : VariantsBySourcesOnDisk)
		{
			PackagesAndVariants.Add(LocalizedVariantsWithSourceAsset.Key); // Add Source Asset
			for (const FName& LocalizedVariant : LocalizedVariantsWithSourceAsset.Value)
			{
				PackagesAndVariants.Add(LocalizedVariant); // Add all Localized Variants
			}
		}
		for (const FName& OtherAsset : PackagesNotFound)
		{
			PackagesAndVariants.Add(OtherAsset); // Add the other assets
		}

		// Error processing
		if (!VariantsBySourcesOnlyInRevisionControl.IsEmpty())
		{
			// Files in Revision Control are needed
			TArray<FText> FilesNeeded;
			for (const TPair<FName, TArray<FName>>& LocalizedVariantsWithSourceAssetInSCC : VariantsBySourcesOnlyInRevisionControl)
			{
				for (const FName& LocalizedVariantInSCC : LocalizedVariantsWithSourceAssetInSCC.Value)
				{
					FilesNeeded.Add(FText::AsCultureInvariant(LocalizedVariantInSCC.ToString()));
				}
			}

			UE_LOGF(LogObjectTools, Error, "A file that needs to be renamed was detected in Revision Control but not on your disk.");
			LocalizedAssetTools->OpenFilesInRevisionControlRequiredDialog(FilesNeeded);

			if (&OutAssetsWithVariantsToDelete != &InAssetsToDelete)
			{
				OutAssetsWithVariantsToDelete = InAssetsToDelete;
			}
			return false;
		}
		else if (bRevisionControlWasNeeded)
		{
			// Revision Control is needed. Extraction must fail.
			UE_LOGF(LogObjectTools, Error, "%ls", *LocalizedAssetTools->GetRevisionControlIsNotAvailableWarningText().ToString());
			LocalizedAssetTools->OpenRevisionControlRequiredDialog();

			if (&OutAssetsWithVariantsToDelete != &InAssetsToDelete)
			{
				OutAssetsWithVariantsToDelete = InAssetsToDelete;
			}
			return false;
		}

		TArray<FText> NewAssetsAdded;
		for (const FName& Package : PackagesAndVariants)
		{
			if (!PackageNames.Contains(Package))
			{
				FString PackageNameStr = Package.ToString();
				UE_LOGF(LogObjectTools, Display, "Added asset to extraction (while checking for localized variants): %ls", *PackageNameStr);
				NewAssetsAdded.Add(FText::AsCultureInvariant(PackageNameStr));
			}
		}

		if (NewAssetsAdded.IsEmpty())
		{
			if (&OutAssetsWithVariantsToDelete != &InAssetsToDelete)
			{
				OutAssetsWithVariantsToDelete = InAssetsToDelete;
			}
			return true;
		}

		ELocalizedVariantsInclusion InclusionResult = LocalizedAssetTools->OpenIncludeLocalizedVariantsListDialog(NewAssetsAdded);
		switch (InclusionResult)
		{
		case ELocalizedVariantsInclusion::Exclude:
			if (&OutAssetsWithVariantsToDelete != &InAssetsToDelete)
			{
				OutAssetsWithVariantsToDelete = InAssetsToDelete;
			}
			return true;
		case ELocalizedVariantsInclusion::Cancel:
			return false;
		case ELocalizedVariantsInclusion::Include:
		default:
			// The inclusion is already done at this point. Only the exclude/cancel options need to do some revert work.
			break;
		}

		// Get the Asset Registry
		IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();

		// Don't add localized variants if the asset registry is not ready
		if (AssetRegistry.IsLoadingAssets())
		{
			FNotificationInfo Info(NSLOCTEXT("UnrealEd", "Warning_CantDeleteRebuildingAssetRegistry", "Unable To Delete While Discovering Assets"));
			Info.ExpireDuration = 3.0f;
			FSlateNotificationManager::Get().AddNotification(Info);
			return false;
		}

		// Iterate over each package path to convert to FAssetData
		for (const FName& PackagePath : PackagesAndVariants)
		{
			TArray<FAssetData> FoundAssets;
			AssetRegistry.GetAssetsByPackageName(PackagePath, FoundAssets);
			OutAssetsWithVariantsToDelete.Append(FoundAssets);
		}

		return true;
	}

	bool ContainsWorldInUse(const TArray< UObject* >& ObjectsToDelete)
	{
		TArray<const UWorld*> WorldsToDelete;

		for (const UObject* ObjectToDelete : ObjectsToDelete)
		{
			if (const UWorld* World = Cast<UWorld>(ObjectToDelete))
			{
				WorldsToDelete.AddUnique(World);
			}
		}

		if (WorldsToDelete.Num() == 0)
		{
			return false;
		}

		auto GetCombinedWorldNames = [](const TArray<const UWorld*>& Worlds) -> FString
		{
			return FString::JoinBy(Worlds, TEXT(", "),
				[](const UWorld* World) -> FString
				{
					return World->GetPathName();
				});
		};

		UE_LOGF(LogObjectTools, Log, "Deleting %d worlds: %ls", WorldsToDelete.Num(), *GetCombinedWorldNames(WorldsToDelete));

		TArray<const UWorld*> ActiveWorlds;

		for (const FWorldContext& WorldContext : GEditor->GetWorldContexts())
		{
			if (const UWorld* World = WorldContext.World())
			{
				ActiveWorlds.AddUnique(World);

				for (const ULevelStreaming* StreamingLevel : World->GetStreamingLevels())
				{
					if (StreamingLevel && StreamingLevel->GetLoadedLevel() && StreamingLevel->GetLoadedLevel()->GetOuter())
					{
						if (const UWorld* StreamingWorld = Cast<UWorld>(StreamingLevel->GetLoadedLevel()->GetOuter()))
						{
							ActiveWorlds.AddUnique(StreamingWorld);
						}
					}
				}
			}
		}

		UE_LOGF(LogObjectTools, Log, "Currently %d active worlds: %ls", ActiveWorlds.Num(), *GetCombinedWorldNames(ActiveWorlds));

		for (const UWorld* World : WorldsToDelete)
		{
			if (ActiveWorlds.Contains(World))
			{
				return true;
			}
		}

		return false;
	}

	int32 DeleteObjects( const TArray< UObject* >& InObjectsToDelete, bool bShowConfirmation, EAllowCancelDuringDelete AllowCancelDuringDelete )
	{
		FDeleteItemsParameters Options;
		Options.ObjectsToDelete = InObjectsToDelete;
		Options.bShowConfirmation = bShowConfirmation;
		Options.AllowCancelDuringDelete = AllowCancelDuringDelete;
		return DeleteItems(Options).ObjectsDeleted;
	}
	
	FDeleteItemsResult DeleteItems(const FDeleteItemsParameters& InOptions)
	{
		auto DeleteObjects = [&InOptions](TArray<UObject*> ObjectsToDelete, bool& bCancelled)
		{
			const FScopedBusyCursor BusyCursor;

			AddExtraObjectsToDelete(ObjectsToDelete);

			// Allows deleting of sounds after they have been previewed
			GEditor->ClearPreviewComponents();

			// Ensure the audio manager is not holding on to any sounds
			FAudioDeviceManager* AudioDeviceManager = GEditor->GetAudioDeviceManager();
			if (AudioDeviceManager != nullptr)
			{
				AudioDeviceManager->UpdateActiveAudioDevices(false);

				const int32 NumAudioDevices = AudioDeviceManager->GetNumActiveAudioDevices();
				for (int32 DeviceIndex = 0; DeviceIndex < NumAudioDevices; DeviceIndex++)
				{
					FAudioDevice* AudioDevice = AudioDeviceManager->GetAudioDeviceRaw(DeviceIndex);
					if (AudioDevice != nullptr)
					{
						AudioDevice->StopAllSounds();
					}
				}
			}

			// Query delegate hook to validate if the delete operation is available
			FCanDeleteAssetResult CanDeleteResult;
			FEditorDelegates::OnAssetsCanDelete.Broadcast(ObjectsToDelete, CanDeleteResult);
			if (!CanDeleteResult.Get())
			{
				FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "CannotDelete", "Cannot currently delete selected objects. See log for details."));
				return 0;
			}

			// Make sure packages being saved are fully loaded.
			if( !HandleFullyLoadingPackages( ObjectsToDelete, NSLOCTEXT("UnrealEd", "Delete", "Delete") ) )
			{
				return 0;
			}

			FOnPreDestructiveAssetActionArgs PreDestructiveArgs(ObjectsToDelete, EDestructiveAssetActions::AssetDelete);
			FEditorDelegates::OnPreDestructiveAssetActionWithArgs.Broadcast(PreDestructiveArgs);

			if (!PreDestructiveArgs.Result.bSuccess)
			{
				UE_LOGF(LogObjectTools, Warning, "%ls", *PreDestructiveArgs.Result.ErrorMessage);
				return 0;
			}

			// Load the asset registry module
			FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

			// Don't delete anything if we're still building the asset registry, warn the user and don't delete.
			if (AssetRegistryModule.Get().IsLoadingAssets())
			{
				FNotificationInfo Info( NSLOCTEXT("UnrealEd", "Warning_CantDeleteRebuildingAssetRegistry", "Unable To Delete While Discovering Assets") );
				Info.ExpireDuration = 3.0f;
				FSlateNotificationManager::Get().AddNotification(Info);
				return 0;
			}

			if (ContainsWorldInUse(ObjectsToDelete))
			{
				FMessageDialog::Open(
					EAppMsgType::Ok,
					NSLOCTEXT("UnrealEd", "DeleteFailedWorldInUse", "Unable to delete level while it is open"),
					NSLOCTEXT("UnrealEd", "DeleteFailedWorldInUseTitle", "Unable to delete level")
				);

				return 0;
			}

			// let systems clean up any unnecessary references that they may have
			// (so that they're not flagged in the dialog)
			FEditorDelegates::OnAssetsPreDelete.Broadcast(ObjectsToDelete);

			TSharedRef<FAssetDeleteModel> DeleteModel = MakeShared<FAssetDeleteModel>(ObjectsToDelete);

			if ( InOptions.bShowConfirmation )
			{
				const FVector2D WindowSize = FVector2D( 600, 700 );

				/** Create the window to host our package dialog widget */
				TSharedRef<SCustomDialog> DeleteAssetsWindow = SNew(SCustomDialog)
					.Title(FText::FromString("Delete Assets"))
					.ClientSize(WindowSize)
					.AutoCloseOnButtonPress(false);

				/** Set the content of the window to our package dialog widget */
				TSharedRef< SDeleteAssetsDialog > DeleteDialog =
					SNew(SDeleteAssetsDialog, DeleteModel)
					.FoldersToDelete(InOptions.FoldersToDelete)
					.OnDialogClosed(
						SDeleteAssetsDialog::FOnDialogClosed::CreateLambda(
						[&bCancelled](SDeleteAssetsDialog::EResult Result)
							{
								bCancelled = (Result == SDeleteAssetsDialog::EResult::Cancel);
							})
						)
					.ParentWindow(DeleteAssetsWindow);

				DeleteAssetsWindow->SetContent( DeleteDialog );
				DeleteAssetsWindow->SetOnCancelHotkeyPressed(FExecuteAction::CreateSP(DeleteDialog, &SDeleteAssetsDialog::OnCancelHotkeyPressed));
				DeleteAssetsWindow->SetOnConfirmHotkeyPressed(FExecuteAction::CreateSP(DeleteDialog, &SDeleteAssetsDialog::OnConfirmHotkeyPressed));
				DeleteAssetsWindow->SetGetWidgetToFocusOnActivate(FGetFocusWidget::CreateSP(DeleteDialog, &SDeleteAssetsDialog::GetWidgetToFocusOnActivate));

				/** Show the package dialog window as a modal window */
				GEditor->EditorAddModalWindow( DeleteAssetsWindow );

				return DeleteModel->GetDeletedObjectCount();
			}

			bool bUserCanceled = false;
			const bool bAllowCancelDuringDelete = (InOptions.AllowCancelDuringDelete == EAllowCancelDuringDelete::AllowCancel);
			GWarn->BeginSlowTask(NSLOCTEXT("UnrealEd", "VerifyingDelete", "Verifying Delete"), true, bAllowCancelDuringDelete);
			while ( !bUserCanceled && DeleteModel->GetState() != FAssetDeleteModel::Finished )
			{
				DeleteModel->Tick(0);
				GWarn->StatusUpdate((int32)( DeleteModel->GetProgress() * 100 ), 100, DeleteModel->GetProgressText());

				if (bAllowCancelDuringDelete)
				{
					bUserCanceled = GWarn->ReceivedUserCancel();
				}
			}
			GWarn->EndSlowTask();

			if ( bUserCanceled )
			{
				UE_LOGF(LogUObjectGlobals, Warning, "User canceled delete operation");
				return 0;
			}

			if ( !DeleteModel->DoDelete() )
			{
				UE_LOGF(LogUObjectGlobals, Warning, "Could not delete");

				//@todo ndarnell explain why the delete failed?  Maybe we should show the delete UI
				// when this fails?
			}

			return DeleteModel->GetDeletedObjectCount();
		};

		bool bCancelled = false;
		FDeleteItemsResult Result;
		Result.ObjectsDeleted = DeleteObjects(InOptions.ObjectsToDelete, bCancelled);

		if (!bCancelled)
		{
			// Look for files on disk in case the folder contains things not tracked by the asset registry
			auto DeleteEmptyFolderFromDisk = [](const FString& InPathToDelete)->bool
				{
					FString PathToDeleteOnDisk;
					if (FPackageName::TryConvertLongPackageNameToFilename(InPathToDelete, PathToDeleteOnDisk))
					{
						TArray<FString> FileNames;
						IFileManager::Get().FindFilesRecursive(FileNames, *PathToDeleteOnDisk, TEXT(""), /** Files */true, /** Directory */false);

						if (FileNames.IsEmpty())
						{
							return IFileManager::Get().DeleteDirectory(*PathToDeleteOnDisk, /** RequireExists */false, /** Tree */true);
						}
					}
			
					return false;
				};

			// Remove folders specified only if they are empty (no files)
			const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
			for (const FString& PathToDelete : InOptions.FoldersToDelete)
			{
				if (DeleteEmptyFolderFromDisk(PathToDelete))
				{
					AssetRegistryModule.Get().RemovePath(PathToDelete);
					Result.FoldersDeleted++;
				}
			}
		}

		return Result;
	}

	int32 PrivatizeObjects(const TArray<UObject*>& InObjectsToPrivatize, bool bShowConfirmation, EAllowCancelDuringPrivatize AllowCancelDuringPrivatize, const EAssetAccessSpecifier InAssetAccessSpecifier)
	{
		const FScopedBusyCursor BusyCursor;
		TArray<UObject*> ObjectsToPrivatize = InObjectsToPrivatize;

		if (!HandleFullyLoadingPackages(ObjectsToPrivatize, NSLOCTEXT("UnrealEd", "Privatize", "Privatize")))
		{
			return 0;
		}

		FOnPreDestructiveAssetActionArgs PreDestructiveArgs(ObjectsToPrivatize, EDestructiveAssetActions::AssetPrivatize);
		FEditorDelegates::OnPreDestructiveAssetActionWithArgs.Broadcast(PreDestructiveArgs);

		if (!PreDestructiveArgs.Result.bSuccess)
		{
			UE_LOGF(LogObjectTools, Warning, "%ls", *PreDestructiveArgs.Result.ErrorMessage);
			return 0;
		}

		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

		if (AssetRegistryModule.Get().IsLoadingAssets())
		{
			FNotificationInfo Info(NSLOCTEXT("UnrealEd", "Warning_CantPrivatizeRebuildingAssetRegistry", "Unable To Mark Private While Discovering Assets"));
			Info.ExpireDuration = 3.0f;
			FSlateNotificationManager::Get().AddNotification(Info);
			return 0;
		}

		TSharedRef<FAssetPrivatizeModel> PrivatizeModel = MakeShared<FAssetPrivatizeModel>(ObjectsToPrivatize, InAssetAccessSpecifier);

		if (bShowConfirmation)
		{
			const FVector2D DEFAULT_WINDOW_SIZE = FVector2D(600, 700);

			TSharedRef<SWindow> PrivatizeAssetsWindow = SNew(SWindow)
				.Title(NSLOCTEXT("UnrealED", "Privatize Assets", "Make Assets Private"))
				.ClientSize(DEFAULT_WINDOW_SIZE);

			TSharedRef<SPrivateAssetsDialog> PrivatizeDialog =
				SNew(SPrivateAssetsDialog, PrivatizeModel)
				.ParentWindow(PrivatizeAssetsWindow);

			PrivatizeAssetsWindow->SetContent(PrivatizeDialog);

			GEditor->EditorAddModalWindow(PrivatizeAssetsWindow);

			return PrivatizeModel->GetObjectsPrivatizedCount();
		}

		bool bUserCanceled = false;
		const bool bAllowCancelDuringPrivatize = (AllowCancelDuringPrivatize == EAllowCancelDuringPrivatize::AllowCancel);
		GWarn->BeginSlowTask(NSLOCTEXT("UnrealEd", "VerifyingPrivatize", "Verifying Privatize"), true, bAllowCancelDuringPrivatize);
		while (!bUserCanceled && PrivatizeModel->GetState() != FAssetPrivatizeModel::Finished)
		{
			PrivatizeModel->Tick(0);
			GWarn->StatusUpdate((int32)(PrivatizeModel->GetProgress() * 100), 100, PrivatizeModel->GetProgressText());

			if (bAllowCancelDuringPrivatize)
			{
				bUserCanceled = GWarn->ReceivedUserCancel();
			}
		}
		GWarn->EndSlowTask();

		if (bUserCanceled)
		{
			UE_LOGF(LogUObjectGlobals, Warning, "User cancelled privatize operation");
			return 0;
		}

		if (!PrivatizeModel->DoPrivatize())
		{
			UE_LOGF(LogUObjectGlobals, Warning, "Could not mark private");
		}

		return PrivatizeModel->GetObjectsPrivatizedCount();
	}

	static bool MakeReadOnlyPackageWritable(UObject* ObjectToDelete, bool& bMakeWritable, bool& bSilent)
	{
		// If an object's package is read only, and source control is not enabled, ask the user whether they wish
		// to make it writable.
		if (!ISourceControlModule::Get().IsEnabled())
		{
			UPackage* ObjectPackage = ObjectToDelete->GetOutermost();
			check(ObjectPackage != nullptr);

			FString PackageFilename;
			if (FPackageName::DoesPackageExist(ObjectPackage->GetName(), &PackageFilename))
			{
				if (IFileManager::Get().IsReadOnly(*PackageFilename))
				{
					EAppReturnType::Type ReturnType = EAppReturnType::No;
					if (!bMakeWritable && !bSilent)
					{
						FFormatNamedArguments Args;
						Args.Add(TEXT("Filename"), FText::FromString(PackageFilename));
						const FText Message = FText::Format(NSLOCTEXT("ObjectTools", "DeleteReadOnlyWarning", "This file is read-only on disk:\n\n{Filename}\n\nDelete it anyway?"), Args);

						ReturnType = FMessageDialog::Open(EAppMsgType::YesNoYesAllNoAll, EAppReturnType::No, Message);
						bMakeWritable = ReturnType == EAppReturnType::YesAll;
						bSilent = ReturnType == EAppReturnType::NoAll;
					}

					if (bMakeWritable || ReturnType == EAppReturnType::Yes)
					{
						FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*PackageFilename, false);
					}
					else
					{
						return false;
					}
				}
			}
		}

		return true;
	}

	int32 DeleteObjectsUnchecked(const TArray< UObject* >& InObjectsToDelete)
	{
		GWarn->BeginSlowTask( NSLOCTEXT( "UnrealEd", "Deleting", "Deleting" ), true );

		TArray<UObject*> ObjectsDeletedSuccessfully;
		TArray<UObject*> ObjectsToDelete = InObjectsToDelete;
		AddExtraObjectsToDelete(ObjectsToDelete);

		bool bSawSuccessfulDelete = false;
		bool bMakeWritable = false;
		bool bSilent = false;

		for ( int32 Index = 0; Index < ObjectsToDelete.Num(); Index++ )
		{
			GWarn->StatusUpdate( Index, ObjectsToDelete.Num(), FText::Format( NSLOCTEXT( "UnrealEd", "Deletingf", "Deleting ({0} of {1})" ), FText::AsNumber( Index ), FText::AsNumber( ObjectsToDelete.Num() ) ) );
			UObject* ObjectToDelete = ObjectsToDelete[Index];

			if ( !ensure( ObjectToDelete != NULL ) )
			{
				continue;
			}

			// Early exclusion for assets contained in read-only packages if the user chooses not to write enable them
			if (!MakeReadOnlyPackageWritable(ObjectToDelete, bMakeWritable, bSilent))
			{
				continue;
			}

			// We already know it's not referenced or we wouldn't be performing the safe delete, so don't repeat the reference check.
			bool bPerformReferenceCheck = false;
			if ( DeleteSingleObject( ObjectToDelete, bPerformReferenceCheck ) )
			{
				ObjectsDeletedSuccessfully.Push( ObjectToDelete );
				bSawSuccessfulDelete = true;
			}
		}

		GWarn->EndSlowTask();

		// Record the number of objects deleted successfully so we can clear the list (once it is just full of pointers to deleted objects)
		const int32 NumObjectsDeletedSuccessfully = ObjectsDeletedSuccessfully.Num();

		// Update the browser if something was actually deleted.
		if ( bSawSuccessfulDelete )
		{
			TArray<UClass*> DeletedObjectClasses;
			TArray<UPackage*> PotentialPackagesToDelete;
			for ( int32 ObjIdx = 0; ObjIdx < ObjectsDeletedSuccessfully.Num(); ++ObjIdx )
			{
				DeletedObjectClasses.AddUnique(ObjectsDeletedSuccessfully[ObjIdx]->GetClass());
				PotentialPackagesToDelete.AddUnique( ObjectsDeletedSuccessfully[ObjIdx]->GetOutermost() );
			}
			// Broadcast the classes of the successfully deleted objects (before cleanup)
			FEditorDelegates::OnAssetsDeleted.Broadcast(DeletedObjectClasses);

			bool bPerformReferenceCheck = false;
			CleanupAfterSuccessfulDelete( PotentialPackagesToDelete, bPerformReferenceCheck );
			ObjectsDeletedSuccessfully.Empty();
		}

		return NumObjectsDeletedSuccessfully;
	}

	bool DeleteSingleObject( UObject* ObjectToDelete, bool bPerformReferenceCheck )
	{
		// Query delegate hook to validate if the delete operation is available
		FCanDeleteAssetResult CanDeleteResult;
		FEditorDelegates::OnAssetsCanDelete.Broadcast(TArray<UObject*>{ ObjectToDelete }, CanDeleteResult);
		if (!CanDeleteResult.Get())
		{
			FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "CannotDelete", "Cannot currently delete selected objects. See log for details."));
			return false;
		}

		if (GEditor)
		{
			GEditor->GetSelectedObjects()->Deselect(ObjectToDelete);
		
			if (ObjectToDelete->IsAsset())
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(ObjectToDelete);
			}
		}

		{
			// @todo Animation temporary HACK to allow deleting of UMorphTargets. This will be removed when UMorphTargets are subobjects of USkeleton.
			// Get the base skeleton and unregister this morphtarget
			UMorphTarget* MorphTarget = Cast<UMorphTarget>(ObjectToDelete);
			if (MorphTarget && MorphTarget->BaseSkelMesh)
			{
				MorphTarget->BaseSkelMesh->UnregisterMorphTarget(MorphTarget);
			}

			// @todo FH: Temporary Hack for world to clean up references until `ForceReplaceReferences` can be made consistent with `IsReferenced`
			// Worlds get hooked on by a lot of external non-uobject system through GCObject, call World cleanup to fire delegates to tell them to unhook and release reference
			if (UWorld* World = Cast<UWorld>(ObjectToDelete))
			{
				World->CleanupWorld();
			}
		}

		if ( bPerformReferenceCheck )
		{
			FReferencerInformationList Refs;

			bool bIsReferenced = false;
			bool bIsReferencedByUndo = false;
			const bool bRequireReferencedProperties = true;
			GatherObjectReferencersForDeletion(ObjectToDelete, bIsReferenced, bIsReferencedByUndo, &Refs, bRequireReferencedProperties);

			// only ref to this object is the transaction buffer, clear the transaction buffer
			if (!bIsReferenced && bIsReferencedByUndo && GEditor)
			{
				GEditor->ResetTransaction( NSLOCTEXT( "UnrealEd", "DeleteSelectedItem", "Delete Selected Item" ) );
			}

			if ( bIsReferenced )
			{
				// We cannot safely delete this object. Print out a list of objects referencing this one
				// that prevent us from being able to delete it.
				FStringOutputDevice Ar;
				ObjectToDelete->OutputReferencers( Ar, &Refs );
				FMessageDialog::Open( EAppMsgType::Ok,
					FText::Format( NSLOCTEXT( "UnrealEd", "Error_InUse", "{0} is in use.\n\n---\nRunning the editor with '-NoLoadStartupPackages' may help if the object is loaded at startup.\n---\n\n{1}" ),
					FText::FromString( ObjectToDelete->GetFullName() ), FText::FromString( *Ar ) ) );

				// Reselect the object as it failed to be deleted
				if (GEditor)
				{
					GEditor->GetSelectedObjects()->Select(ObjectToDelete);
				}

				return false;
			}
		}

		// Mark its package as dirty as we're going to delete it.
		ObjectToDelete->MarkPackageDirty();

		// Notify the asset registry. This done before the removal of the flags otherwise the content browser will ignore the update.
		FAssetRegistryModule::AssetDeleted( ObjectToDelete );

		// Remove standalone flag so garbage collection can delete the object and public flag so that the object is no longer considered to be an asset
		ObjectToDelete->ClearFlags(RF_Standalone | RF_Public);

		return true;
	}

	/**
	 * Inspects all objects in memory and returns the set of all objects that transitively refer to the given InInterestSet
	 * Objects in the original InInterestSet are included in the output ReferencingObjects set
	 * Inner Objects that only have a path to the InterestSet through their outers are excluded.
	 */
	static void RecursiveRetrieveReferencers(const TArray<UObject*>& InInterestSet, TSet<FWeakObjectPtr>& OutReferencingObjects)
	{
		if (!CVarUseLegacyGetReferencersForDeletion.GetValueOnAnyThread())
		{
			// Use the fast reference collector to recursively find referencers until no more are found
			TSet<UObject*> InterestSet;
			InterestSet.Append(InInterestSet);

			// Continue until we're not adding any more referencers to the set
			for (int32 LastCount = 0; LastCount != InterestSet.Num(); )
			{
				LastCount = InterestSet.Num();
				InterestSet.Append(FReferencerFinder::GetAllReferencers(InterestSet, nullptr, EReferencerFinderFlags::SkipInnerReferences));
			}

			for (UObject* Referencer : InterestSet)
			{
				OutReferencingObjects.Add(Referencer);
			}
		}
		else
		{
			const int32 ExpectedArraySize = 100;
			const int32 ExpectedReferencesPerObject = 5;
			TArray<UObject*> InterestSetAdditions(InInterestSet, FMath::Max(0, ExpectedArraySize - InInterestSet.Num()));

			TMap<UObject*, int32> References;
			TArray<UObject*> InterestSet;
			InterestSet.Reserve(InterestSetAdditions.Max() * 2);
			References.Reserve(ExpectedReferencesPerObject);

			// It would be faster to run a single TObjectIterator+Serialize loop and capture the complete graph of object references, and then do operations
			// on the resultant graph, but that would require memory equal to sizeof(pointer)*num objects*(average references per object+3) to hold the graph.
			// The extra cost of the current solution is that the TObjectIterator will be executed a number of times equal to 
			// the length of the maximum (minimum reference chain length) from any object to the original interest set
			// TODO: Worth the memory cost?
			while (InterestSetAdditions.Num() > 0)
			{
				InterestSet.Append(InterestSetAdditions);
				Algo::Sort(InterestSet, TLess<UObject*>());
				InterestSetAdditions.Reset();

				for (FThreadSafeObjectIterator It; It; ++It)
				{
					UObject* Object = *It;
					if (Algo::BinarySearch(InterestSet, Object, TLess<UObject*>()) != INDEX_NONE)
					{
						continue;
					}

					const bool bAlsoFindWeakReferences = false;
					FFindReferencersArchive ArFind(Object, InterestSet, bAlsoFindWeakReferences);
					ArFind.GetReferenceCounts(References);
					if (References.Num() > 0)
					{
						// Ignore internal references; only add the searched object if it refers to a member of the interest set but is not inside that member
						for (const TPair<UObject*, int32>& kvpair : References)
						{
							if (!Object->IsIn(kvpair.Key))
							{
								InterestSetAdditions.Add(Object);
								break;
							}
						}
						References.Reset();
					}
				}
			}

			for (UObject* Referencer : InterestSet)
			{
				OutReferencingObjects.Add(Referencer);
			}
		}
	}

	int32 ForceDeleteObjects(const TArray< UObject* >& InObjectsToDelete, bool ShowConfirmation)
	{
		int32 NumDeletedObjects = 0;

		TArray<UObject*> ShownObjectsToDelete = InObjectsToDelete;
		AddExtraObjectsToDelete(ShownObjectsToDelete);

		// Query delegate hook to validate if the delete operation is available
		FCanDeleteAssetResult CanDeleteResult;
		FEditorDelegates::OnAssetsCanDelete.Broadcast(ShownObjectsToDelete, CanDeleteResult);
		if (!CanDeleteResult.Get())
		{
			FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "CannotDelete", "Cannot currently delete selected objects. See log for details."));
			return 0;
		}

		// Confirm that the delete was intentional
		if (ShowConfirmation && !ShowDeleteConfirmationDialog(ShownObjectsToDelete))
		{
			return 0;
		}

		// Recursively find all references to objects being deleted
		TSet<FWeakObjectPtr> ReferencingObjects;
		RecursiveRetrieveReferencers(InObjectsToDelete, ReferencingObjects);

		// Attempt to close all editors referencing any of the deleted objects
		bool bClosedAllEditors = true;
		for (const FWeakObjectPtr& ObjectPtr : ReferencingObjects)
		{
			UObject* Object = ObjectPtr.Get();
			if (Object != nullptr && Object->IsAsset())
			{
				TArray<IAssetEditorInstance*> ObjectEditors = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorsForAssetAndSubObjects(Object);
				for (IAssetEditorInstance* ObjectEditorInstance : ObjectEditors)
				{
					if (!ObjectEditorInstance->CloseWindow(EAssetEditorCloseReason::AssetForceDeleted))
					{
						bClosedAllEditors = false;
					}
				}
			}
		}

		// Failed to close at least one editor. It is possible that this editor has in-memory object references
		// which are not prepared to be changed dynamically so it is not safe to continue
		if (!bClosedAllEditors)
		{
			return 0;
		}

		{
			// Force delete is a dangerous operation, add some fingerprints to the log:
			FString Msg;
			Msg.Append(FString::Printf(TEXT("Force Deleting %d Package(s):"), ShownObjectsToDelete.Num()));
			const int32 MAX_PACKAGES_TO_LOG = 10;
			for(int32 I = 0; I < ShownObjectsToDelete.Num() && I < MAX_PACKAGES_TO_LOG; ++I)
			{
				Msg.Append(TEXT("\n"));
				Msg.Append(FString::Printf(TEXT("\tAsset Name: %s\n"), *GetPathNameSafe(ShownObjectsToDelete[I])));
				Msg.Append(FString::Printf(TEXT("\tAsset Type: %s"), *(ShownObjectsToDelete[I]->GetClass()->GetName())));
			}
			UE_LOGF(LogUObjectGlobals, Log, "%ls", *Msg);
		}

		GWarn->BeginSlowTask( NSLOCTEXT("UnrealEd", "Deleting", "Deleting"), true );

		FEditorDelegates::OnPreForceDeleteObjects.Broadcast(ShownObjectsToDelete);
		
		struct FSCSNodeToDelete
		{
			USimpleConstructionScript* SimpleConstructionScript;
			USCS_Node* SCS_Node;
		};

		TArray<FSCSNodeToDelete> SCSNodesToDelete;
		TArray<UActorComponent*> ComponentsToDelete;
		TArray<AActor*> ActorsToDelete;
		TArray<TWeakObjectPtr<UObject>> ObjectsToDelete;
		bool bNeedsGarbageCollection = false;
		bool bMakeWritable = false;
		bool bSilent = false;

		// Clear audio components to allow previewed sounds to be consolidated
		GEditor->ClearPreviewComponents();

		for ( TArray<UObject*>::TConstIterator ObjectItr(ShownObjectsToDelete); ObjectItr; ++ObjectItr )
		{
			UObject* CurrentObject = *ObjectItr;

			GEditor->GetSelectedObjects()->Deselect( CurrentObject );

			// Early exclusion for assets contained in read-only packages if the user chooses not to write enable them
			if (!MakeReadOnlyPackageWritable(CurrentObject, bMakeWritable, bSilent))
			{
				continue;
			}

			ObjectsToDelete.Add( CurrentObject );

			// If the object about to be deleted is a Blueprint asset, make sure that any instances of the Blueprint class get deleted as well
			UBlueprint* BlueprintObject = Cast<UBlueprint>(CurrentObject);
			if ( BlueprintObject && BlueprintObject->GeneratedClass && BlueprintObject->GeneratedClass->GetDefaultObject(false) )
			{
				TArray<UObject*> InstancesToDelete;
				BlueprintObject->GeneratedClass->GetDefaultObject(false)->GetArchetypeInstances( InstancesToDelete );

				for ( TArray<UObject*>::TConstIterator InstanceItr( InstancesToDelete ); InstanceItr; ++InstanceItr )
				{
					UObject* CurrentInstance = *InstanceItr;

					// Don't include derived class CDOs.
					if(CurrentInstance->HasAnyFlags(RF_ClassDefaultObject))
					{
						continue;
					}

					AActor* CurrentInstanceAsActor = Cast<AActor>( CurrentInstance );
					UActorComponent* CurrentInstanceAsComponent = Cast<UActorComponent>(CurrentInstance);
					if ( CurrentInstanceAsActor )
					{
						ActorsToDelete.Add( CurrentInstanceAsActor );
					}
					else if ( CurrentInstanceAsComponent )
					{
						ComponentsToDelete.Add( CurrentInstanceAsComponent );

						// Find all the SCS_Node references that need to be destroyed before this component is destroyed.
						UBlueprintGeneratedClass* UBGC = CurrentInstanceAsComponent->GetTypedOuter<UBlueprintGeneratedClass>();
						if (UBGC && UBGC->SimpleConstructionScript)
						{
							for (USCS_Node* SCS_Node : UBGC->SimpleConstructionScript->GetAllNodes())
							{
								if (SCS_Node && SCS_Node->ComponentTemplate == CurrentInstanceAsComponent)
								{
									FSCSNodeToDelete DeleteNode;
									DeleteNode.SimpleConstructionScript = UBGC->SimpleConstructionScript;
									DeleteNode.SCS_Node = SCS_Node;
									SCSNodesToDelete.Add(DeleteNode);
								}
							}
						}
					}
					else
					{
						ObjectsToDelete.Add( CurrentInstance );
					}
				}
			}
		}

		// Destroy all SCSNodes
		if (SCSNodesToDelete.Num() > 0)
		{
			for (TArray<FSCSNodeToDelete>::TConstIterator SCSNodeItr(SCSNodesToDelete); SCSNodeItr; ++SCSNodeItr)
			{
				FSCSNodeToDelete SCSNodeToDelete = *SCSNodeItr;

				SCSNodeToDelete.SimpleConstructionScript->RemoveNodeAndPromoteChildren(SCSNodeToDelete.SCS_Node);

				GWarn->StatusUpdate(SCSNodeItr.GetIndex(), SCSNodesToDelete.Num(), NSLOCTEXT("UnrealEd", "ConsolidateAssetsUpdate_DeletingSCSNodes", "Deleting Blueprint Component references..."));
			}
		}

		bool bSelectionChanged = false;

		TArray<UObject*> ObjectsToReplace;
		ObjectsToReplace.Reserve(ObjectsToDelete.Num());

		// Destroy all Components
		if (ComponentsToDelete.Num() > 0)
		{
			for (TArray<UActorComponent*>::TConstIterator ComponentItr(ComponentsToDelete); ComponentItr; ++ComponentItr)
			{
				UActorComponent* CurComponent = *ComponentItr;

				// Skip if already pending GC
				if (IsValid(CurComponent))
				{
					// Deselect if active
					USelection* SelectedComponents = GEditor->GetSelectedComponents();
					if (SelectedComponents && CurComponent->IsSelected())
					{
						SelectedComponents->Deselect(CurComponent);

						bSelectionChanged = true;
					}

					// Destroy the Component Instance
					CurComponent->DestroyComponent(true);

					bNeedsGarbageCollection = true;
				}

				GWarn->StatusUpdate(ComponentItr.GetIndex(), ComponentsToDelete.Num(), NSLOCTEXT("UnrealEd", "ConsolidateAssetsUpdate_DeletingComponentInstances", "Deleting Component Instances..."));
			}
		}

		// Destroy all Actor instances
		if ( ActorsToDelete.Num() > 0 )
		{
			ULayersSubsystem* Layers = GEditor->GetEditorSubsystem<ULayersSubsystem>();
			for ( TArray<AActor*>::TConstIterator ActorItr( ActorsToDelete ); ActorItr; ++ActorItr )
			{
				AActor* CurActor = *ActorItr;

				// Skip if already pending GC
				if ( IsValid(CurActor) )
				{
					// Deselect if active
					USelection* SelectedActors = GEditor->GetSelectedActors();
					if ( SelectedActors && CurActor->IsSelected() )
					{
						SelectedActors->Deselect( CurActor );

						bSelectionChanged = true;
					}

					// Destroy the Actor instance. This is similar to edactDeleteSelected(), but we don't request user confirmation here.
					Layers->DisassociateActorFromLayers( CurActor );
					if( CurActor->GetWorld() )
					{
						CurActor->GetWorld()->EditorDestroyActor( CurActor, false );
					}
					// Ensure that we replace any generated actors who don't have worlds that are left such as the template
					// from Child Actor Components
					else
					{
						ObjectsToReplace.Add(CurActor);
					}

					bNeedsGarbageCollection = true;
				}

				GWarn->StatusUpdate( ActorItr.GetIndex(), ActorsToDelete.Num(), NSLOCTEXT( "UnrealEd", "ConsolidateAssetsUpdate_DeletingActorInstances", "Deleting Actor Instances..." ) );
			}
		}

		GEditor->NoteSelectionChange();


		{
			// If the current editor world is in this list, transition to a new map and reload the world to finish the delete
			ReloadEditorWorldForReferenceReplacementIfNecessary(ObjectsToDelete);
		}

		TArray<UPackage*> PackagesFailedToDelete;
		{
			{
				for(TWeakObjectPtr<UObject>& Object : ObjectsToDelete)
				{
					if(Object.IsValid())
					{
						ObjectsToReplace.Add(Object.Get());

						UBlueprint* BlueprintObject = Cast<UBlueprint>(Object.Get());
						if (BlueprintObject)
						{
							// If we're a blueprint add our generated class as well
							if (BlueprintObject->GeneratedClass)
							{
								ObjectsToReplace.AddUnique(BlueprintObject->GeneratedClass);
							}

							// Also include the skeleton class, in case it is referenced
							if (BlueprintObject->SkeletonGeneratedClass)
							{
								ObjectsToReplace.AddUnique(BlueprintObject->SkeletonGeneratedClass);
							}

							// Reparent any direct children to the parent class of the blueprint that's about to be deleted
							if (BlueprintObject->ParentClass != nullptr)
							{
								for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
								{
									UClass* ChildClass = *ClassIt;
									if (ChildClass->GetSuperStruct() == BlueprintObject->GeneratedClass)
									{
										UBlueprint* ChildBlueprint = Cast<UBlueprint>(ChildClass->ClassGeneratedBy);
										if (ChildBlueprint != nullptr)
										{
											// Do not reparent and recompile a Blueprint that is going to be deleted.
											if (ObjectsToDelete.Find(ChildBlueprint) == INDEX_NONE)
											{
												ChildBlueprint->Modify();
												ChildBlueprint->ParentClass = BlueprintObject->ParentClass;

												// Recompile the child blueprint to fix up the generated class
												FKismetEditorUtilities::CompileBlueprint(ChildBlueprint, EBlueprintCompileOptions::SkipGarbageCollection);

												// Defer garbage collection until after we're done processing the list of objects
												bNeedsGarbageCollection = true;
											}
										}
									}
								}
							}

							BlueprintObject->RemoveChildRedirectors();
							BlueprintObject->RemoveGeneratedClasses();
						}
					}
				}

				// Replacing references inside already loaded objects could cause rendering issues, so globally detach all components from their scenes for now
				FGlobalComponentRecreateRenderStateContext ReregisterContext;

				// UserDefinedStructs (probably all ScriptStructs) should be replaced with the FallbackStruct - but they must be reinstanced properly
				// this is very similar to the "Reparent any direct children".. logic above, but for UScriptStruct instead of UClass.. Likely 
				// we also need similar logic for UEnum
				{
					TArray<UObject*> UDStructToReplace;
					for (int32 Iter = 0; Iter < ObjectsToReplace.Num(); )
					{
						if (UUserDefinedStruct* UDStruct = Cast<UUserDefinedStruct>(ObjectsToReplace[Iter]))
						{
							ObjectsToReplace.RemoveAtSwap(Iter);
							
							UDStructToReplace.Add(UDStruct);
							FBlueprintCompilationManager::ReinstanceStruct(UDStruct);
						}
						else
						{
							Iter++;
						}
					}

					if (UDStructToReplace.Num())
					{
						FForceReplaceInfo ReplaceInfo;
						ForceReplaceReferences(GetFallbackStruct(), UDStructToReplace, ReplaceInfo, false);
					}
				}

				{
					FForceReplaceInfo ReplaceInfo;
					ForceReplaceReferences(nullptr, ObjectsToReplace, ReplaceInfo, false);
				}
			}

			// Handle deferred garbage collection
			if (bNeedsGarbageCollection)
			{
				CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
				bNeedsGarbageCollection = false;
				// The GC above may have deleted some of the ObjectsToDelete (e.g. preview instances of blueprints) - 
				// remove them from ObjectsToDelete:
				ObjectsToDelete.SetNum(Algo::RemoveIf(ObjectsToDelete, [](const TWeakObjectPtr<UObject>& ObjectWeakPtr)
				{
					return ObjectWeakPtr.Get() == nullptr;
				}));
			}
			
			// Give systems opportunity to clean up references to the objects being deleted
			{
				TArray<UObject*> ObjectsToPreDelete;
				Algo::Transform(ObjectsToDelete, ObjectsToPreDelete, [](const TWeakObjectPtr<UObject>& ObjectWeakPtr) { return ObjectWeakPtr.Get(); });
				FEditorDelegates::OnAssetsPreDelete.Broadcast(ObjectsToPreDelete);
			}

			// Load the asset tools module to get access to the browser type maps
			FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));

			int32 Count = 0;
			const int32 OriginalNumObjectsToDelete = ObjectsToDelete.Num();
			for(auto It = ObjectsToDelete.CreateIterator(); It; ++It)
			{
				UObject* CurObject = It->Get();
				if ( !ensure(CurObject != nullptr) )
				{
					continue;
				}

				if( DeleteSingleObject( CurObject ) )
				{
					// Only count the objects we were given to delete, as this function may have added more (eg, BP instances)
					if (InObjectsToDelete.Contains(CurObject))
					{
						// Update return val
						++NumDeletedObjects;
					}
				}
				// if the delete fails at this point, it means the object won't be able to be purged and might be left in a weird state, as a last resort queue its package for reload
				else
				{
					UE_LOGF(LogObjectTools, Warning, "ForceDeleteObject failed to delete %ls, this package is now potentially corrupt", *CurObject->GetName());
					PackagesFailedToDelete.AddUnique(CurObject->GetOutermost());
					It.RemoveCurrent();
				}

				GWarn->StatusUpdate(Count, OriginalNumObjectsToDelete, NSLOCTEXT("UnrealEd", "ConsolidateAssetsUpdate_DeletingObjects", "Deleting Assets..."));
				++Count;

			}
		}

		TArray<UClass*> DeletedObjectClasses;
		TArray<UPackage*> PotentialPackagesToDelete;
		for(TWeakObjectPtr<UObject>& Object : ObjectsToDelete)
		{
			if(Object.IsValid())
			{
				DeletedObjectClasses.AddUnique(Object->GetClass());
				PotentialPackagesToDelete.AddUnique(Object->GetOutermost());
			}
		}

		if (PotentialPackagesToDelete.Num() > 0)
		{
			FEditorDelegates::OnAssetsDeleted.Broadcast(DeletedObjectClasses);
			CleanupAfterSuccessfulDelete(PotentialPackagesToDelete);
		}
		ObjectsToDelete.Empty();

		// Final report of packages we failed to delete. This is mostly for crash reporter. To fix this ensure you need
		// to fix the reference replacement/isreferenced mismatch. The former does not find subobjects, the latter does:
		if(PackagesFailedToDelete.Num())
		{
			FString FailedPackageNames;
			for(UPackage* Package : PackagesFailedToDelete)
			{
				FailedPackageNames.Append(FString::Printf(TEXT("\r\n%s"), *Package->GetName()));
			}

			ensureMsgf( false, 
				TEXT(R"(
					Failed to unload all packages during ForceDeleteObjects - 
					these packages are likely corrupt. Consider restarting the 
					editor, noting which assets remain and then deleting them 
					from the file system manually: %s)"), *FailedPackageNames
			);
		}

		GWarn->EndSlowTask();

		if (GUnrealEd)
		{
			// Redraw viewports
			GUnrealEd->RedrawAllViewports();
		}

		return NumDeletedObjects;
	}


	/**
	 * Utility function to compose a string list of referencing objects
	 *
	 * @param References			Array of references to the relevant object
	 * @param RefObjNames			String list of all objects
	 * @param DefObjNames			String list of all objects referenced in default properties
	 *
	 * @return Whether or not any objects are in default properties
	 */
	bool ComposeStringOfReferencingObjects( TArray<FReferencerInformation>& References, FString& RefObjNames, FString& DefObjNames )
	{
		bool bInDefaultProperties = false;

		for ( TArray<FReferencerInformation>::TConstIterator ReferenceInfoItr( References ); ReferenceInfoItr; ++ReferenceInfoItr )
		{
			FReferencerInformation RefInfo = *ReferenceInfoItr;
			UObject* ReferencingObject = RefInfo.Referencer;
			RefObjNames = RefObjNames + TEXT("\n") + ReferencingObject->GetPathName();

			if( ReferencingObject->GetPathName().Contains( FString(DEFAULT_OBJECT_PREFIX)) )
			{
				DefObjNames = DefObjNames + TEXT("\n") + ReferencingObject->GetName();
				bInDefaultProperties = true;
			}
		}

		return bInDefaultProperties;
	}

	void DeleteRedirector (UObjectRedirector* Redirector)
	{
		// We can't actually delete the redirector. We will just send it to the transient package where it will get cleaned up later
		if (Redirector)
		{
			FAssetRegistryModule::AssetDeleted(Redirector);

			// Remove public flag if set and set transient flag to ensure below rename doesn't create a redirect.
			Redirector->ClearFlags( RF_Public );
			Redirector->SetFlags( RF_Transient );

			// Instead of deleting we rename the redirector into a dummy package where it will be GCed later.
			Redirector->Rename(NULL, GetTransientPackage(), REN_DontCreateRedirectors);
			Redirector->DestinationObject = NULL;
		}
	}

	bool GetMoveDialogInfo(const FText& DialogTitle, UObject* Object, bool bUniqueDefaultName, const FString& SourcePath, const FString& DestinationPath, FMoveDialogInfo& InOutInfo)
	{
		if ( !ensure(Object) )
		{
			return false;
		}

		const FString CurrentPackageName = Object->GetOutermost()->GetName();

		FString PreviousPackage = InOutInfo.PGN.PackageName;
		FString PreviousGroup = InOutInfo.PGN.GroupName;

		FString PackageName;
		FString GroupName;
		FString ObjectName;

		ObjectName = Object->GetName();

		const bool bIsRelativeOperation = SourcePath.Len() && DestinationPath.Len() && CurrentPackageName.StartsWith(SourcePath);
		if ( bIsRelativeOperation )
		{
			// Folder copy/move.

			// Collect the relative path then use it to determine the new location
			// For example, if SourcePath = /Game/MyPath and CurrentPackageName = /Game/MyPath/MySubPath/MyAsset
			//     /Game/MyPath/MySubPath/MyAsset -> /MySubPath/

			const int32 ShortPackageNameLen = FPackageName::GetLongPackageAssetName(CurrentPackageName).Len();
			const int32 RelativePathLen = CurrentPackageName.Len() - ShortPackageNameLen - SourcePath.Len();
			const FString RelativeDestPath = CurrentPackageName.Mid(SourcePath.Len(), RelativePathLen);

			PackageName = DestinationPath + RelativeDestPath + ObjectName;
			GroupName = TEXT("");

			// Folder copies dont need a dialog
			InOutInfo.bOkToAll = true;
		}
		else if ( PreviousPackage.Len() )
		{
			// Use the last supplied path
			// Non-relative move/copy, use the location from the previous operation
			PackageName = FPackageName::GetLongPackagePath(PreviousPackage) + "/" + ObjectName;
			GroupName = TEXT("");
		}
		else if ( DestinationPath.Len() )
		{
			// Use the passed in default path
			// Normal path
			PackageName = DestinationPath + "/" + ObjectName;
			GroupName = TEXT("");
		}
		else
		{
			// Use the path from the old package
			PackageName = Object->GetOutermost()->GetName();

			GroupName = TEXT("");
		}

		// If the target package already exists, check for name clashes and find a unique name
		if ( bUniqueDefaultName )
		{
			FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
			AssetToolsModule.Get().CreateUniqueAssetName(*PackageName, TEXT(""), PackageName, ObjectName);
		}

		if( !InOutInfo.bOkToAll && InOutInfo.bPromptForRenameOnConflict )
		{
			// Present the user with a rename dialog for each asset.
			FDlgMoveAsset MoveDialog(/*bIsLegacyOrMapPackage*/ false, PackageName, GroupName, ObjectName, DialogTitle);

			const FDlgMoveAsset::EResult MoveDialogResult = MoveDialog.ShowModal();

			// Abort if the user cancelled.
			if( MoveDialogResult == FDlgMoveAsset::Cancel)
			{
				return false;
			}

			// Don't show the dialog again if "Ok to All" was selected.
			if( MoveDialogResult == FDlgMoveAsset::OKToAll )
			{
				InOutInfo.bOkToAll = true;
			}

			// Store the entered package/group/name for later retrieval.
			PackageName = MoveDialog.GetNewPackage();
			GroupName = MoveDialog.GetNewGroup();
			ObjectName = MoveDialog.GetNewName();

			// @todo asset: Should we interactively add localized packages
			//bSawOKToAll |= bLocPackages;
		}

		InOutInfo.PGN.PackageName = PackageName;
		InOutInfo.PGN.GroupName = GroupName;
		InOutInfo.PGN.ObjectName = ObjectName;

		return true;
	}


	bool RenameObjectsInternal( const TArray<UObject*>& Objects, bool bLocPackages, const TMap< UObject*, FString >* ObjectToLanguageExtMap, const FString& SourcePath, const FString& DestinationPath, bool bOpenDialog )
	{
		TSet<UPackage*> PackagesUserRefusedToFullyLoad;
		TArray<UPackage*> OutermostPackagesToSave;
		FText ErrorMessage;

		bool bSawSuccessfulRename = false;

		FMoveDialogInfo MoveDialogInfo;
		MoveDialogInfo.bOkToAll = !bOpenDialog;

		// The default value for save packages is true if SCC is enabled because the user can use SCC to revert a change
		MoveDialogInfo.bSavePackages = ISourceControlModule::Get().IsEnabled();

		for( int32 Index = 0; Index < Objects.Num(); Index++ )
		{
			UObject* Object = Objects[ Index ];
			if( !Object )
			{
				continue;
			}

			if ( !GetMoveDialogInfo(NSLOCTEXT("UnrealEd", "RenameObjects", "Move/Rename Objects" ), Object, /*bUniqueDefaultName=*/false, SourcePath, DestinationPath, MoveDialogInfo) )
			{
				// The user aborted the operation
				return false;
			}

			UPackage* OldPackage = Object->GetOutermost();
			if ( RenameSingleObject(Object, MoveDialogInfo.PGN, PackagesUserRefusedToFullyLoad, ErrorMessage, ObjectToLanguageExtMap) )
			{
				OutermostPackagesToSave.AddUnique( OldPackage );
				OutermostPackagesToSave.AddUnique( Object->GetOutermost() );
				bSawSuccessfulRename = true;
			}
		} // Selected objects.

		// Display any error messages that accumulated.
		if ( !ErrorMessage.IsEmpty() )
		{
			FMessageDialog::Open( EAppMsgType::Ok, ErrorMessage );
		}

		// Update the browser if something was actually renamed.
		if ( bSawSuccessfulRename )
		{
			bool bUpdateSCC = false;
			if ( MoveDialogInfo.bSavePackages )
			{
				const bool bCheckDirty = false;
				const bool bPromptToSave = false;
				FEditorFileUtils::PromptForCheckoutAndSave(OutermostPackagesToSave, bCheckDirty, bPromptToSave);
				bUpdateSCC = true;
			}

			if ( bUpdateSCC )
			{
				ISourceControlModule::Get().QueueStatusUpdate(OutermostPackagesToSave);
			}
		}

		return ErrorMessage.IsEmpty();
	}

	bool RenameSingleObject(UObject* Object, FPackageGroupName& PGN, TSet<UPackage*>& InOutPackagesUserRefusedToFullyLoad, FText& InOutErrorMessage, const TMap< UObject*, FString >* ObjectToLanguageExtMap, bool bLeaveRedirector)
	{
		FString ErrorMessage;

		if( !Object )
		{
			// Can not rename NULL objects.
			return false;
		}

		// @todo asset: Find an appropriate place for localized sounds
		bool bLocPackages = false;

		const FString& NewPackageName = PGN.PackageName;
		const FString& NewGroupName = PGN.GroupName;
		const FString& NewObjectName = PGN.ObjectName;

		const FScopedBusyCursor BusyCursor;

		bool bMoveFailed = false;
		bool bMoveRedirectorFailed = false;
		FMoveInfo MoveInfo;

		// The language extension for localized packages. Defaults to int32
		FString LanguageExt = TEXT("INT");

		// If the package the object is being moved to is new
		bool bPackageIsNew = false;

		if( bLocPackages && NewPackageName != Object->GetOutermost()->GetName() )
		{
			// If localized sounds are being moved to a different package
			// make sure the package they are being moved to is valid
			if( ObjectToLanguageExtMap )
			{
				// Language extension package this object is in
				const FString* FoundLanguageExt = ObjectToLanguageExtMap->Find( Object );

				if( FoundLanguageExt && *FoundLanguageExt != TEXT("INT") )
				{
					// A language extension has been found for this object.
					// Append the package name with the language extension.
					// Do not append int32 packages as they have no extension
					LanguageExt = *FoundLanguageExt->ToUpper();
					PGN.PackageName += FString::Printf( TEXT("_%s"), *LanguageExt );
					PGN.GroupName += FString::Printf( TEXT("_%s"), *LanguageExt );
				}

			}

			// Check to see if the language specific path is the same as the path in the filename
			const FString LanguageSpecificPath = FString::Printf( TEXT("%s/%s"), TEXT("Sounds"), *LanguageExt );

			// Filename of the package we are moving from
			FString OriginPackageFilename;
			// If the object was is in a localized directory.  SoundWaves in non localized package file paths should  be able to move anywhere.
			bool bOriginPackageInLocalizedDir = false;
			if ( FPackageName::DoesPackageExist( Object->GetOutermost()->GetName(), &OriginPackageFilename ) )
			{
				// if the language specific path cant be found in the origin package filename, this package is not in a directory for only localized packages
				bOriginPackageInLocalizedDir = (OriginPackageFilename.Contains( LanguageSpecificPath ) );
			}

			// Filename of the package we are moving to
			FString DestPackageName;
			// Find the package filename of the package we are moving to.
			bPackageIsNew = !FPackageName::DoesPackageExist( NewPackageName, &DestPackageName );
			if( !bPackageIsNew && bOriginPackageInLocalizedDir && !DestPackageName.Contains( LanguageSpecificPath ) )
			{
				// Skip new packages or packages not in localized dirs (objects in these can move anywhere)
				// If the the language specific path cannot be found in the destination package filename
				// This package is being moved to an invalid location.
				bMoveFailed = true;
				ErrorMessage += FText::Format( NSLOCTEXT("UnrealEd", "Error_InvalidMoveOfLocalizedObject", "Attempting to move localized sound {0} into non localized package or package with different localization.\n" ),
					FText::FromString(Object->GetName()) ).ToString();
			}
		}

		if ( !bMoveFailed )
		{
			// Make sure that a target package exists.
			if ( !NewPackageName.Len() )
			{
				ErrorMessage += TEXT("Invalid package name supplied\n");
				bMoveFailed = true;
			}
			else
			{
				// Make a full path from the target package and group.
				const FString FullPackageName = NewGroupName.Len()
					? FString::Printf(TEXT("%s.%s"), *NewPackageName, *NewGroupName)
					: NewPackageName;

				// Make sure the target package is fully loaded.
				TArray<UPackage*> TopLevelPackages;
				UPackage* ExistingPackage = FindPackage(NULL, *FullPackageName);
				UPackage* ExistingOutermostPackage = NewGroupName.Len() ? FindPackage(NULL, *NewPackageName) : ExistingPackage;

				if( ExistingPackage )
				{
					TopLevelPackages.Add( ExistingPackage->GetOutermost() );
				}

				// If there's an existing outermost package, try to find its filename
				FString ExistingOutermostPackageFilename;
				if ( ExistingOutermostPackage )
				{
					FPackageName::DoesPackageExist( ExistingOutermostPackage->GetName(), &ExistingOutermostPackageFilename );
				}

				// Fully load the ref objects package
				TopLevelPackages.Add( Object->GetOutermost() );

				// Used in the IsValidObjectName checks below
				FText Reason;

				if( ExistingPackage && ( InOutPackagesUserRefusedToFullyLoad.Contains(ExistingPackage) || !UPackageTools::HandleFullyLoadingPackages( TopLevelPackages, NSLOCTEXT("UnrealEd", "Rename", "Rename") ) ) )
				{
					// HandleFullyLoadingPackages should never return false for empty input.
					check( ExistingPackage );
					InOutPackagesUserRefusedToFullyLoad.Add( ExistingPackage );
					bMoveFailed = true;
				}
				// Don't allow a move/rename to occur into a package that has a filename invalid for saving. This is a rare case
				// that should not happen often, but could occur using packages created before the editor checked against file name length
				else if ( ExistingOutermostPackage && ExistingOutermostPackageFilename.Len() > 0 && !FFileHelper::IsFilenameValidForSaving( ExistingOutermostPackageFilename, Reason ) )
				{
					bMoveFailed = true;
				}
				else if( !NewObjectName.Len() )
				{
					ErrorMessage += TEXT("Invalid object name\n");
					bMoveFailed = true;
				}
				else if(!FName(*NewObjectName).IsValidObjectName( Reason )
					||	!FPackageName::IsValidLongPackageName( NewPackageName, /*bIncludeReadOnlyRoots=*/false, &Reason )
					||	!FName(*NewGroupName).IsValidGroupName(Reason,true) )
				{
					// Make sure the object name is valid.
					ErrorMessage += FString::Printf(TEXT("    %s to %s.%s: %s\n"), *Object->GetPathName(), *FullPackageName, *NewObjectName, *Reason.ToString() );
					bMoveFailed = true;
				}
				else
				{
					// We can rename on top of an object redirection (basically destroy the redirection and put us in its place).
					UPackage* NewPackage = CreatePackage( *FullPackageName );

					bool bFoundCompatibleRedirector = false;
					UObjectRedirector* Redirector = nullptr;

					UPackage* OldPackage = Object->GetPackage();
					if (NewPackage != OldPackage)
					{
						NewPackage->GetOutermost()->FullyLoad();

						// Make sure we copy all the cooked package flags if the asset was already cooked.
						if (OldPackage->HasAnyPackageFlags(PKG_FilterEditorOnly))
						{
							NewPackage->SetPackageFlags(PKG_FilterEditorOnly);
						}
						NewPackage->bIsCookedForEditor = OldPackage->bIsCookedForEditor;

						// Renaming an asset should respect the export controls of the original.
						if (OldPackage->HasAnyPackageFlags(PKG_DisallowExport))
						{
							NewPackage->SetPackageFlags(PKG_DisallowExport);
						}

						if (OldPackage->HasAnyPackageFlags(PKG_NewlyCreated))
						{
							NewPackage->SetPackageFlags(PKG_NewlyCreated);
						}

						NewPackage->SetAssetAccessSpecifier(OldPackage->GetAssetAccessSpecifier());

						// When renaming a World Composition map, make sure to properly initialize WorldTileInfo
						if (OldPackage->GetWorldTileInfo())
						{
							NewPackage->SetWorldTileInfo(MakeUnique<FWorldTileInfo>(*OldPackage->GetWorldTileInfo()));
						}

						Redirector = Cast<UObjectRedirector>(StaticFindObject(UObjectRedirector::StaticClass(), NewPackage, *NewObjectName));
						// If we found a redirector, check that the object it points to is of the same class.
						if (Redirector
							&& Redirector->DestinationObject
							&& Redirector->DestinationObject->GetClass() == Object->GetClass())
						{
							// Test renaming the redirector into a dummy package.
							if (Redirector->Rename(*Redirector->GetName(), CreatePackage(TEXT("/Temp/TempRedirectors")), REN_Test))
							{
								// Actually rename the redirector here so it doesn't get in the way of the rename below.
								Redirector->Rename(*Redirector->GetName(), CreatePackage(TEXT("/Temp/TempRedirectors")), REN_DontCreateRedirectors);

								bFoundCompatibleRedirector = true;
							}
							else
							{
								bMoveFailed = true;
								bMoveRedirectorFailed = true;
							}
						}
					}
					else
					{
						bMoveFailed = true;
						ErrorMessage += (NSLOCTEXT("UnrealEd", "Error_ObjectNameCaseChange", "Cannot change the case of an object name.\n")).ToString();
					}


					if ( !bMoveFailed )
					{
						// Test to see if the rename will succeed.
						if ( Object->Rename(*NewObjectName, NewPackage, REN_Test) )
						{
							// No errors!  Set asset move info.
							MoveInfo.Set( *FullPackageName, *NewObjectName );

							// @todo asset: Find an appropriate place for localized sounds
							bLocPackages = false;
							if( bLocPackages && bPackageIsNew )
							{
								// Setup the path this localized package should be saved to.
								FString Path;

								// Newly renamed objects must have the single asset package extension
								Path = FPaths::Combine(*FPaths::ProjectDir(), TEXT("Content"), TEXT("Sounds"), *LanguageExt, *(FPackageName::GetLongPackageAssetName(NewPackageName) + FPackageName::GetAssetPackageExtension()));

								// Move the package into the correct file location by saving it
								GEditor->Exec( NULL, *FString::Printf(TEXT("OBJ SAVEPACKAGE PACKAGE=\"%s\" FILE=\"%s\""), *NewPackageName, *Path) );
							}
						}
						else
						{
							const FString FullObjectPath = FString::Printf(TEXT("%s.%s"), *FullPackageName, *NewObjectName);
							ErrorMessage += FText::Format( NSLOCTEXT("UnrealEd", "Error_ObjectNameAlreadyExists", "An object named '{0}' already exists.\n"), FText::FromString(FullObjectPath) ).ToString();
							bMoveFailed = true;
						}
					}

					if (bFoundCompatibleRedirector)
					{
						// Rename the redirector back since we are just testing
						UPackage* DestinationPackage = FindPackage(NULL, *FullPackageName);

						if ( ensure(DestinationPackage) )
						{
							if ( Redirector->Rename(*Redirector->GetName(), DestinationPackage, REN_Test) )
							{
								Redirector->Rename(*Redirector->GetName(), DestinationPackage, REN_DontCreateRedirectors);
							}
							else
							{
								UE_LOGF(LogObjectTools, Warning, "RenameObjectsInternal failed to return a redirector '%ls' to its original location. This was because there was already an asset in the way. Deleting redirector.", *Redirector->GetName());
								DeleteRedirector(Redirector);
								Redirector = NULL;
							}
						}
					}
				}
			} // NewPackageName valid?
		}

		if ( !bMoveFailed )
		{
			// Actually perform the move!
			check( MoveInfo.IsValid() );

			const FString& PkgName = MoveInfo.FullPackageName;
			const FString& ObjName = MoveInfo.NewObjName;
			const FString FullObjectPath = FString::Printf(TEXT("%s.%s"), *PkgName, *ObjName);

			// We can rename on top of an object redirection (basically destroy the redirection and put us in its place).
			UObjectRedirector* Redirector = Cast<UObjectRedirector>( StaticFindObject(UObjectRedirector::StaticClass(), NULL, *FullObjectPath) );
			// If we found a redirector, check that the object it points to is of the same class.
			if ( Redirector
				&& Redirector->DestinationObject
				&& Redirector->DestinationObject->GetClass() == Object->GetClass() )
			{
				DeleteRedirector(Redirector);
				Redirector = NULL;
			}

			UPackage* OldPackage = Object->GetOutermost();
			UPackage* NewPackage = CreatePackage( *PkgName );

			// if this object is being renamed out of the MyLevel package into a content package, we need to mark it RF_Standalone
			// so that it will be saved (UWorld::CleanupWorld() clears this flag for all objects inside the package)
			if (!Object->HasAnyFlags(RF_Standalone)
				&&	(OldPackage && OldPackage->ContainsMap())
				&&	!NewPackage->GetOutermost()->ContainsMap() )
			{
				Object->SetFlags(RF_Standalone);
			}

			// The object must be fully loaded to realize latent thumbnail data
			EnsureLoadingComplete(Object->GetOutermost());

			// Look for a thumbnail for this asset before we rename it
			FObjectThumbnail* Thumbnail = ThumbnailTools::GetThumbnailForObject(Object);

			// Make sure there is no async compilation outstanding in the package we're going to unload
			UPackageTools::FlushAsyncCompilation( { Object->GetPackage() });

			FString OldObjectPathName = Object->GetPathName();
			GEditor->RenameObject( Object, NewPackage, *ObjName, bLeaveRedirector ? REN_None : REN_DontCreateRedirectors );

			if (OldPackage)
			{
#if USE_STABLE_LOCALIZATION_KEYS
				// Migrate the localization ID to the new package
				TextNamespaceUtil::ForcePackageNamespace(NewPackage, TextNamespaceUtil::GetPackageNamespace(OldPackage));
				TextNamespaceUtil::ClearPackageNamespace(OldPackage);
#endif

				// Remove any metadata from old package pointing to moved objects
				OldPackage->GetMetaData().RemoveMetaDataOutsidePackage(OldPackage);

				// Allow external systems to deal with metadata
				UPackage::OnMetaDataTransferRequestedDelegate.Broadcast(OldPackage->GetMetaData(), NewPackage->GetMetaData(), OldPackage, NewPackage);
			}

			// Migrate any thumbnail from the old package to the new one
			if (Thumbnail)
			{
				ThumbnailTools::CacheThumbnail(Object->GetFullName(), Thumbnail, NewPackage);
			}

			// Notify the asset registry of the rename
			FAssetRegistryModule::AssetRenamed(Object, OldObjectPathName);

			// If a redirector was created, notify the asset registry
			UObjectRedirector* NewRedirector = FindObject<UObjectRedirector>(NULL, *OldObjectPathName);
			if ( NewRedirector )
			{
				// If we created a redirector to a map asset, ensure the redirector package is flagged as containing a map for it to have the correct file extension.
				if (NewPackage->ContainsMap())
				{
					NewRedirector->GetOutermost()->ThisContainsMap();
				}

				FAssetRegistryModule::AssetCreated(NewRedirector);
			}

			// Saw Successful Rename
			InOutErrorMessage = FText::FromString( ErrorMessage );
			return true;
		}
		else
		{
			if(bMoveRedirectorFailed)
			{
				ErrorMessage += FText::Format( NSLOCTEXT("UnrealEd", "Error_CouldntRenameObjectRedirectorF", "Couldn't rename '{0}' object because there is an object redirector of the same name, please fixup redirect from editor by enabling Show Redirects in content browser.\n"),
					FText::FromString(IAssetTools::Get().GetUserFacingFullName(*Object)) ).ToString();
			}
			else
			{
				ErrorMessage += FText::Format( NSLOCTEXT("UnrealEd", "Error_CouldntRenameObjectF", "Couldn't rename '{0}'.\n"), FText::FromString(IAssetTools::Get().GetUserFacingFullName(*Object)) ).ToString();
			}

			// @todo asset: Find an appropriate place for localized sounds
			bLocPackages = false;
			if( bLocPackages )
			{
				// Inform the user that no localized objects will be moved or renamed
				ErrorMessage += FString::Printf( TEXT("No localized objects could be moved"));
				// break out of the main loop,
				//break;
			}
		}

		InOutErrorMessage = FText::FromString( ErrorMessage );
		return false;
	}

	/**
	 * Finds all language variants for the passed in sound wave
	 *
	 * @param OutObjects	A list of found localized sound wave objects
	 * @param OutObjectToLanguageExtMap	A mapping of sound wave objects to their language extension
	 * @param Wave	The sound wave to search for
	 */
	void AddLanguageVariants( TArray<UObject*>& OutObjects, TMap< UObject*, FString >& OutObjectToLanguageExtMap, USoundWave* Wave )
	{
		//@todo-packageloc Handle sound localization packages.
	}

	bool RenameObjects( const TArray< UObject* >& SelectedObjects, bool bIncludeLocInstances, const FString& SourcePath, const FString& DestinationPath, bool bOpenDialog )
	{
		return RenameObjectsInternal( SelectedObjects, bIncludeLocInstances, NULL, SourcePath, DestinationPath, bOpenDialog );
	}

	FString SanitizeObjectName(const FString& InObjectName)
	{
		return SanitizeInvalidChars(InObjectName, INVALID_OBJECTNAME_CHARACTERS);
	}

	FString SanitizeObjectPath(const FString& InObjectPath)
	{
		return SanitizeInvalidChars(InObjectPath, INVALID_OBJECTPATH_CHARACTERS);
	}

	FString SanitizeInvalidChars(const FString& InText, const FString& InvalidChars)
	{
		return SanitizeInvalidChars(InText, *InvalidChars);
	}

	FString SanitizeInvalidChars(const FString& InText, const TCHAR* InvalidChars)
	{
		FString SanitizedText = InText;
		SanitizeInvalidCharsInline(SanitizedText, InvalidChars);
		return SanitizedText;
	}

	void SanitizeInvalidCharsInline(FString& InText, const TCHAR* InvalidChars)
	{
		const TCHAR* InvalidChar = InvalidChars ? InvalidChars : TEXT("");
		while (*InvalidChar)
		{
			InText.ReplaceCharInline(*InvalidChar, TCHAR('_'), ESearchCase::CaseSensitive);
			++InvalidChar;
		}
	}

	/**
	 * Internal helper function to obtain format descriptions and extensions of formats list
	 *
	 * @param	Formats				List of formats who should be retrieved
	 * @param	out_Descriptions	Array of format descriptions associated with the current factory; should equal the number of extensions
	 * @param	out_Extensions		Array of format extensions associated with the current factory; should equal the number of descriptions
	 */
	void InternalGetFormatInfo(const TArray<FString>& Formats
		, TArray<FString>& out_Descriptions
		, TArray<FString>& out_Extensions)
	{
		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools")).Get();
		// Iterate over each formats.
		for ( TArray<FString>::TConstIterator FormatIter( Formats ); FormatIter; ++FormatIter )
		{
			const FString& CurFormat = *FormatIter;

			// Parse the format into its extension and description parts
			TArray<FString> FormatComponents;
			CurFormat.ParseIntoArray( FormatComponents, TEXT(";"), false );

			for ( int32 ComponentIndex = 0; ComponentIndex < FormatComponents.Num(); ComponentIndex += 2 )
			{
				check( FormatComponents.IsValidIndex( ComponentIndex + 1 ) );

				FString& RefExtension = FormatComponents[ComponentIndex];
				if (!AssetTools.IsImportExtensionAllowed(RefExtension))
				{
					//Skip this extension
					continue;
				}

				out_Extensions.Add( FormatComponents[ComponentIndex] );
				out_Descriptions.Add( FormatComponents[ComponentIndex + 1] );
			}
		}
	}

	/**
	 * Populates two strings with all of the file types and extensions the provided factory supports.
	 *
	 * @param	InFactory		Factory whose supported file types and extensions should be retrieved
	 * @param	out_Filetypes	File types supported by the provided factory, concatenated into a string
	 * @param	out_Extensions	Extensions supported by the provided factory, concatenated into a string
	 */
	void GenerateFactoryFileExtensions( UFactory* InFactory
		, FString& out_Filetypes
		, FString& out_Extensions
		, TMultiMap<uint32, UFactory*>& out_FilterIndexToFactory)
	{
		// Place the factory in an array and call the overloaded version of this function
		TArray<UFactory*> FactoryArray;
		FactoryArray.Add( InFactory );
		GenerateFactoryFileExtensions( FactoryArray, out_Filetypes, out_Extensions, out_FilterIndexToFactory);
	}

	/**
	 * Populates two strings with all of the file types and extensions the provided factories support.
	 *
	 * @param	InFactories		Factories whose supported file types and extensions should be retrieved
	 * @param	out_Filetypes	File types supported by the provided factory, concatenated into a string
	 * @param	out_Extensions	Extensions supported by the provided factory, concatenated into a string
	 */
	void GenerateFactoryFileExtensions( const TArray<UFactory*>& InFactories
		, FString& out_Filetypes
		, FString& out_Extensions
		, TMultiMap<uint32, UFactory*>& out_FilterIndexToFactory)
	{
		// Store all the descriptions and their corresponding extensions in a map
		TMultiMap<FString, FString> DescToExtensionMap;
		TMultiMap<FString, UFactory*> DescToFactory;

		// Iterate over each factory, retrieving their supported file descriptions and extensions, and storing them into the map
		for ( TArray<UFactory*>::TConstIterator FactoryIter(InFactories); FactoryIter; ++FactoryIter )
		{
			const UFactory* CurFactory = *FactoryIter;
			check(CurFactory);

			TArray<FString> Descriptions;
			TArray<FString> Extensions;
			InternalGetFormatInfo( CurFactory->GetFormats(), Descriptions, Extensions);
			check( Descriptions.Num() == Extensions.Num() );

			// Make sure to only store each key, value pair once
			for ( int32 FormatIndex = 0; FormatIndex < Descriptions.Num() && FormatIndex < Extensions.Num(); ++FormatIndex )
			{
				DescToExtensionMap.AddUnique( Descriptions[FormatIndex], Extensions[FormatIndex ] );
				DescToFactory.AddUnique( Descriptions[FormatIndex], *FactoryIter );
			}
		}

		// Zero out the output strings in case they came in with data already
		out_Filetypes = "";
		out_Extensions = "";

		// Sort the map's keys alphabetically
		DescToExtensionMap.KeySort( TLess<FString>() );

		// Retrieve an array of all of the unique keys within the map
		TArray<FString> DescriptionKeyMap;
		DescToExtensionMap.GetKeys( DescriptionKeyMap );
		const TArray<FString>& DescriptionKeys = DescriptionKeyMap;

		uint32 IdxFilter = 1; // the type list will start by an all supported files wildcard value

		// Keep track of added extensions to prevent duplicates
		TArray<FString> AddedExtensions;

		// Iterate over each unique map key, retrieving all of each key's associated values in order to populate the strings
		for ( TArray<FString>::TConstIterator DescIter( DescriptionKeys ); DescIter; ++DescIter )
		{
			const FString& CurDescription = *DescIter;

			// Retrieve each value associated with the current key
			TArray<FString> Extensions;
			DescToExtensionMap.MultiFind( CurDescription, Extensions );
			if ( Extensions.Num() > 0 )
			{
				// Sort each extension alphabetically, so that the output is alphabetical by description, and in the event of
				// a description with multiple extensions, alphabetical by extension as well
				Extensions.Sort();

				for ( TArray<FString>::TConstIterator ExtIter( Extensions ); ExtIter; ++ExtIter )
				{
					const FString& CurExtension = *ExtIter;
					const FString& CurLine = FString::Printf( TEXT("%s (*.%s)|*.%s"), *CurDescription, *CurExtension, *CurExtension );

					// The same extension could be used for multiple types (like with t3d), so ensure any given extension is only added to the string once
					if ( !AddedExtensions.Contains(CurExtension))
					{
						if ( out_Extensions.Len() > 0 )
						{
							out_Extensions += TEXT(";");
						}
						out_Extensions += FString::Printf(TEXT("*.%s"), *CurExtension);
                       				AddedExtensions.Add(CurExtension);
					}

					// Each description-extension pair can only appear once in the map, so no need to check the string for duplicates
					if ( out_Filetypes.Len() > 0 )
					{
						out_Filetypes += TEXT("|");
					}
					out_Filetypes += CurLine;

					// save the order in which descriptions are added to be able to identify
					// factories using filter index
					TArray<UFactory*> Factories;
					DescToFactory.MultiFind( CurDescription, Factories );
					TArray<UFactory*>::TIterator FactIt(Factories);
					for (;FactIt;++FactIt)
					{
						out_FilterIndexToFactory.Add( IdxFilter, *FactIt );
					}
					++IdxFilter;
				}
			}
		}
	}

	void InternalAppendFileExtensions(const TArray<FString>& InDescriptions, const TArray<FString>& InExtensions, FString& out_Filetypes, FString& out_Extensions)
	{
		check(InDescriptions.Num() == InExtensions.Num());

		for (int32 FormatIndex = 0; FormatIndex < InDescriptions.Num() && FormatIndex < InExtensions.Num(); ++FormatIndex)
		{
			const FString& CurDescription = InDescriptions[FormatIndex];
			const FString& CurExtension = InExtensions[FormatIndex];
			const FString& CurLine = FString::Printf( TEXT("%s (*.%s)|*.%s"), *CurDescription, *CurExtension, *CurExtension );

			// Only append the extension if it's not already one of the found extensions
			if ( !out_Extensions.Contains( CurExtension) )
			{
				if ( out_Extensions.Len() > 0 )
				{
					out_Extensions += TEXT(";");
				}
				out_Extensions += FString::Printf(TEXT("*.%s"), *CurExtension);
			}

			// Only append the line if it's not already one of the found filetypes
			if ( !out_Filetypes.Contains( CurLine) )
			{
				if ( out_Filetypes.Len() > 0 )
				{
					out_Filetypes += TEXT("|");
				}
				out_Filetypes += CurLine;
			}
		}
	}

	void AppendFormatsFileExtensions(const TArray<FString>& InFormats
		, FString& out_FileTypes
		, FString& out_Extensions)
	{
		TArray<FString> Descriptions;
		TArray<FString> Extensions;
		InternalGetFormatInfo(InFormats, Descriptions, Extensions);
		InternalAppendFileExtensions(Descriptions, Extensions, out_FileTypes, out_Extensions);
	}

	void AppendFormatsFileExtensions(const TArray<FString>& InFormats
		, FString& out_FileTypes
		, FString& out_Extensions
		, TMultiMap<uint32, UFactory*>& out_FilterIndexToFactory)
	{
		TArray<FString> Descriptions;
		TArray<FString> Extensions;
		InternalGetFormatInfo(InFormats, Descriptions, Extensions);
		InternalAppendFileExtensions(Descriptions, Extensions, out_FileTypes, out_Extensions);
		uint32 MaxKeyNumber = 0;
		TSet<uint32> Keys;
		out_FilterIndexToFactory.GetKeys(Keys);
		for (uint32 Key : Keys)
		{
			MaxKeyNumber = FMath::Max(MaxKeyNumber, Key);
		}
		if (Keys.Num() > 0)
		{
			MaxKeyNumber++;
		}
		for (const FString& Extension : Extensions)
		{
			out_FilterIndexToFactory.Add(MaxKeyNumber++, nullptr);
		}
	}

	/**
	 * Generates a list of file types for a given class.
	 */
	void AppendFactoryFileExtensions ( UFactory* InFactory
		, FString& out_Filetypes
		, FString& out_Extensions)
	{
		check(InFactory);
		TArray<FString> Descriptions;
		TArray<FString> Extensions;
		InternalGetFormatInfo( InFactory->GetFormats(), Descriptions, Extensions);
		InternalAppendFileExtensions( Descriptions, Extensions, out_Filetypes, out_Extensions);
	}

	/**
	 * Iterates over all classes and assembles a list of non-abstract UExport-derived type instances.
	 */
	void AssembleListOfExporters(TArray<UExporter*>& OutExporters)
	{
		auto TransientPackage = GetTransientPackage();

		// @todo DB: Assemble this set once.
		OutExporters.Empty();
		for( TObjectIterator<UClass> It ; It ; ++It )
		{
			if( It->IsChildOf(UExporter::StaticClass()) && !It->HasAnyClassFlags(CLASS_Abstract) )
			{
				UExporter* Exporter = NewObject<UExporter>(TransientPackage, *It);
				OutExporters.Add( Exporter );
			}
		}
	}

	/**
	 * Assembles a path from the outer chain of the specified object.
	 */
	void GetDirectoryFromObjectPath(const UObject* Obj, FString& OutResult)
	{
		if( Obj )
		{
			GetDirectoryFromObjectPath( Obj->GetOuter(), OutResult );
			OutResult /= Obj->GetName();
		}
	}

	/**
	 * Tags objects which are in use by levels specified by the search option
	 *
	 * @param SearchOption	 The search option for finding in use objects
	 */
	void TagInUseObjects( EInUseSearchOption SearchOption, EInUseSearchFlags InUseSearchFlags )
	{
		UWorld* World = GWorld;
		TSet<UObject*> LevelPackages;
		TSet<UObject*> Levels;

		if( !World )
		{
			// Don't do anything if there is no World.  This could be called during a level load transition
			return;
		}

		switch( SearchOption )
		{
		case SO_CurrentLevel:
			LevelPackages.Add( World->GetCurrentLevel()->GetOutermost() );
			Levels.Add( World->GetCurrentLevel() );
			break;
		case SO_VisibleLevels:
			// Add the persistent level if its visible
			if( FLevelUtils::IsLevelVisible( World->PersistentLevel ) )
			{
				LevelPackages.Add( World->PersistentLevel->GetOutermost() );
				Levels.Add( World->PersistentLevel );
			}
			// Add all other levels if they are visible
			for (ULevelStreaming* StreamingLevel  : World->GetStreamingLevels())
			{
				if (StreamingLevel && FLevelUtils::IsStreamingLevelVisibleInEditor( StreamingLevel ) )
				{
					if (ULevel* Level = StreamingLevel->GetLoadedLevel())
					{
						LevelPackages.Add( Level->GetOutermost() );
						Levels.Add( Level );
					}
				}
			}
			break;
		case SO_LoadedLevels:
			// Add the persistent level as its always loaded
			LevelPackages.Add( World->PersistentLevel->GetOutermost() );
			Levels.Add( World->PersistentLevel );

			// Add all other levels
			for (ULevelStreaming* StreamingLevel : World->GetStreamingLevels())
			{
				if (StreamingLevel)
				{
					if (ULevel* Level = StreamingLevel->GetLoadedLevel())
					{
						LevelPackages.Add( Level->GetOutermost() );
						Levels.Add( Level );
					}
				}
			}
			break;
		default:
			// A bad option was passed in.
			check(0);
		}

		TArray<UObject*> ObjectsInLevels;

		for( FThreadSafeObjectIterator It; It; ++It )
		{
			UObject* Obj = *It;

			// Clear all marked flags that could have been tagged in a previous search or by another system.
			Obj->UnMark(EObjectMark(OBJECTMARK_TagImp | OBJECTMARK_TagExp));

			// If the object is not flagged for GC and it is in one of the level packages do an indepth search to see what references it.
			if( IsValidChecked(Obj) && !Obj->IsUnreachable())
			{
				// Get Object's Outermost package which isn't the same as the object's package for external objects/actors
				const UObject* OuterPackage = Obj->IsA<UPackage>() ? Obj : Obj->GetOutermostObject()->GetPackage();
				if (LevelPackages.Find(OuterPackage) != NULL)
				{
					if (ULevel* OuterLevel = Obj->GetTypedOuter<ULevel>(); OuterLevel && Levels.Contains(OuterLevel))
					{
						// this object was contained within one of our ReferenceRoots
						ObjectsInLevels.Add(Obj);

						// If the object is using a blueprint generated class, also add the blueprint as a reference
						if (UBlueprint* const Blueprint = Cast<UBlueprint>(Obj->GetClass()->ClassGeneratedBy))
						{
							ObjectsInLevels.Add(Blueprint);
						}
					}
				}
			}
			else if( Obj->IsA( AWorldSettings::StaticClass() ) )
			{
				// If a skipped object is a world info ensure it is not serialized because it may contain
				// references to levels (and by extension, their actors) that we are not searching for references to.
				Obj->Mark(OBJECTMARK_TagImp);
			}
		}

		EArchiveReferenceMarkerFlags MarkerFlags = EArchiveReferenceMarkerFlags::None;
		if ((InUseSearchFlags & EInUseSearchFlags::SkipCompilingAssets) != EInUseSearchFlags::None)
		{
			MarkerFlags |= EArchiveReferenceMarkerFlags::SkipCompilingAssets;
		
}
		// Tag all objects that are referenced by objects in the levels we are searching.
		FArchiveReferenceMarker Marker( ObjectsInLevels, MarkerFlags);
	}
	
	TSharedPtr<SWindow> OpenPropertiesForSelectedObjects( const TArray<UObject*>& SelectedObjects )
	{
		TSharedPtr<SWindow> FloatingDetailsView;
		if ( SelectedObjects.Num() > 0 )
		{
			FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>( "PropertyEditor" );

			FloatingDetailsView = PropertyEditorModule.CreateFloatingDetailsView( SelectedObjects, false );
		}

		return FloatingDetailsView;
	}

	void RemoveDeletedObjectsFromPropertyWindows( TArray<UObject*>& DeletedObjects )
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyEditorModule.RemoveDeletedObjects( DeletedObjects );
	}

	bool IsAssetValidForPlacing(UWorld* InWorld, const FString& ObjectPath)
	{
		bool bResult = ObjectPath.Len() > 0;
		if ( bResult )
		{
			bResult = !FEditorFileUtils::IsMapPackageAsset(ObjectPath);
			if ( !bResult )
			{
				// if this map is loaded, allow the asset to be placed
				FString AssetPackageName = FEditorFileUtils::ExtractPackageName(ObjectPath);
				if ( AssetPackageName.Len() > 0 )
				{
					if ( UPackage* AssetPackage = FindObjectSafe<UPackage>(nullptr, *AssetPackageName, EFindObjectFlags::ExactClass) )
					{
						// so it's loaded - make sure it is the current map
						TArray<UWorld*> CurrentMapWorlds;
						EditorLevelUtils::GetWorlds(InWorld, CurrentMapWorlds, true);
						for ( int32 WorldIndex = 0; WorldIndex < CurrentMapWorlds.Num(); WorldIndex++ )
						{
							UWorld* World = CurrentMapWorlds[WorldIndex];
							if ( World && World->GetOutermost() == AssetPackage )
							{
								bResult = true;
								break;
							}
						}
					}
				}
			}
		}

		return bResult;
	}

	bool IsClassValidForPlacing(const UClass* InClass)
	{
		check(InClass);

		const bool bIsPlaceable = !InClass->HasAllClassFlags(CLASS_NotPlaceable);
		const bool bIsAbstractOrDeprecated = InClass->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists);
		const bool bIsSkeletonClass = FKismetEditorUtilities::IsClassABlueprintSkeleton(InClass);

		return bIsPlaceable && !bIsAbstractOrDeprecated && !bIsSkeletonClass;
	}

	bool AreObjectsOfEquivalantType( const TArray<UObject*>& InProposedObjects )
	{
		if ( InProposedObjects.Num() > 0 )
		{
			// Use the first proposed object as the basis for the compatible check.
			const UObject* ComparisonObject = InProposedObjects[0];
			check( ComparisonObject );

			const UClass* ComparisonClass = ComparisonObject->GetClass();
			check( ComparisonClass );

			// Iterate over each proposed consolidation object, checking if each shares a common class with the consolidation objects, or at least, a common base that
			// is allowed as an exception (currently only exceptions made for textures and materials).
			for ( TArray<UObject*>::TConstIterator ProposedObjIter( InProposedObjects ); ProposedObjIter; ++ProposedObjIter )
			{
				UObject* CurProposedObj = *ProposedObjIter;
				check( CurProposedObj );

				const UClass* CurProposedClass = CurProposedObj->GetClass();

				if (ComparisonClass->IsChildOf(UBlueprint::StaticClass()) && CurProposedClass->IsChildOf(UBlueprint::StaticClass()))
				{
					if (*CastChecked<UBlueprint>(ComparisonObject)->ParentClass != *CastChecked<UBlueprint>(CurProposedObj)->ParentClass)
					{
						return false;
					}
				}

				if ( !AreClassesInterchangeable( ComparisonClass, CurProposedClass ) )
				{
					return false;
				}
			}
		}

		return true;
	}

	bool IsClassRedirector( const UClass* Class )
	{
		if ( Class == nullptr )
		{
			return false;
		}

		// You may not consolidate object redirectors
		if ( Class->IsChildOf( UObjectRedirector::StaticClass() ) )
		{
			return true;
		}

		return false;
	}

	bool AreClassesInterchangeable( const UClass* ClassA, const UClass* ClassB )
	{
		// You may not consolidate object redirectors
		if ( IsClassRedirector( ClassB ) )
		{
			return false;
		}

		if ( ClassB != ClassA )
		{
			const UClass* NearestCommonBase = ClassB->FindNearestCommonBaseClass( ClassA );

			// If the proposed object doesn't share a common class or a common base that is allowed as an exception, it is not a compatible object
			if ( !( NearestCommonBase->IsChildOf( UTexture::StaticClass() ) ) && !( NearestCommonBase->IsChildOf( UMaterialInterface::StaticClass() ) ) )
			{
				return false;
			}
		}

		return true;
	}

	void BatchGetArchetypeInstances(TArrayView<UObject*> InObjects, TArray<TArray<UObject*>>& OutInstances)
	{
		// Mapping from object pointer to index in InObjects array, for archetype objects.  If there are repeated objects,
		// only the first is added to the map, and we'll go back later and copy the final results from the first to the repeats.
		TMap<UObject*, int32> ArchetypeObjectToIndexMap;

		// Unique list of classes we need to search.  If there is a class default object, the value of the map will contain the
		// index of it in the array, and all instances of that class are added to that index's list (otherwise, INDEX_NONE).
		TMap<UClass*, int32> ClassToDefaultMap;

		// Tracks if there were any repeats found, indicating we need a final pass to copy results to those repeats.
		bool bHasRepeats = false;

		// Start off by clearing our input array, and generating our maps
		OutInstances.SetNum(InObjects.Num());

		for (int32 ObjectIndex = 0; ObjectIndex < InObjects.Num(); ObjectIndex++)
		{
			OutInstances[ObjectIndex].Empty();

			// Determine if we need to consider this object at all
			UObject* Object = InObjects[ObjectIndex];

			// Allow NULL to be passed in, which may be useful to callers, say if they have a mixed array of objects, only some
			// of which need an archetype search done.  They can pass in NULL for those items, and get back an array with the
			// the same indexing, rather than needing to remap the results.  Doesn't cost anything extra in the inner loop,
			// since we filter those objects out up front.
			if (Object && Object->HasAnyFlags(RF_ArchetypeObject | RF_ClassDefaultObject))
			{
				// Add class as one we need to search, defaulting to INDEX_NONE if it hasn't yet been added
				int32& ClassMapValue = ClassToDefaultMap.FindOrAdd(Object->GetClass(), INDEX_NONE);

				// if this object is the class default object, any object of the same class (or derived classes) could potentially be affected
				if (!Object->HasAnyFlags(RF_ArchetypeObject))
				{
					if (ClassMapValue == INDEX_NONE)
					{
						// Set the index where we will accumulate objects for class defaults
						ClassMapValue = ObjectIndex;
					}
					else
					{
						bHasRepeats = true;
					}
				}
				else
				{
					int32& ObjectMapValue = ArchetypeObjectToIndexMap.FindOrAdd(Object, ObjectIndex);
					if (ObjectMapValue != ObjectIndex)
					{
						bHasRepeats = true;
					}
				}
			}
		}

		// Now do our pass through all the classes
		for (auto ClassIt = ClassToDefaultMap.CreateConstIterator(); ClassIt; ++ClassIt)
		{
			const bool bIncludeNestedObjects = true;
			ForEachObjectOfClass(ClassIt.Key(), [DefaultIndex = ClassIt.Value(), &ArchetypeObjectToIndexMap, &InObjects, &OutInstances](UObject* Obj)
			{
				// Check if we need to add this object as an instance of a default object
				if (DefaultIndex != INDEX_NONE)
				{
					if (Obj != InObjects[DefaultIndex])
					{
						OutInstances[DefaultIndex].Add(Obj);
					}
				}

				// Check if we need to add this object as an instance of an archetype object.  This logic mirrors "UObject::IsBasedOnArchetype",
				// except instead of testing a single "SomeObject" to see if it matches "Template", it searchs a map of potential "SomeObjects".
				for (UObject* Template = Obj->GetArchetype(); Template; Template = Template->GetArchetype())
				{
					int32* ObjectIndex = ArchetypeObjectToIndexMap.Find(Template);
					if (ObjectIndex && Obj != InObjects[*ObjectIndex])
					{
						OutInstances[*ObjectIndex].Add(Obj);
					}
				}

			}, bIncludeNestedObjects, RF_NoFlags, EInternalObjectFlags::Garbage); // we need to evaluate CDOs as well, but nothing pending kill
		}

		if (bHasRepeats)
		{
			// Iterate over objects like the original object loop, checking which ones are repeats.  A repeat will have an array index
			// that doesn't match the index in the map for that object.
			for (int32 ObjectIndex = 0; ObjectIndex < InObjects.Num(); ObjectIndex++)
			{
				UObject* Object = InObjects[ObjectIndex];
				if (Object && Object->HasAnyFlags(RF_ArchetypeObject | RF_ClassDefaultObject))
				{
					if (!Object->HasAnyFlags(RF_ArchetypeObject))
					{
						// Class default, check if this is a repeat, and copy it from the first index of the same object
						int32* ClassMapValue = ClassToDefaultMap.Find(Object->GetClass());
						if (*ClassMapValue != ObjectIndex)
						{
							OutInstances[ObjectIndex] = OutInstances[*ClassMapValue];
						}
					}
					else
					{
						// Archetype, check if this is a repeat, and copy it from the first index of the same object
						int32* ObjectMapValue = ArchetypeObjectToIndexMap.Find(Object);
						if (*ObjectMapValue != ObjectIndex)
						{
							OutInstances[ObjectIndex] = OutInstances[*ObjectMapValue];
						}
					}
				}
			}
		}
	}

	FText GetUserFacingFunctionName(const UFunction* Function, bool bAllowFriendlyNames)
	{
		if (Function != nullptr)
		{
			// Function->GetDisplayNameText() and Function->GetMetaDataText(NAME_DisplayName,...) are equivalent.
			// However, we cannot simply use those functions since they enforce friendly names and do not work on SKEL classes.
			// We implement the same functionality a third time.
			static const FName NAME_DisplayName{ TEXT("DisplayName") };

			FString FunctionDisplayName = Function->GetName();
			FString FunctionFriendlyName;
			if (const FString* FoundMetaData = Function->FindMetaData(NAME_DisplayName))
			{
				FunctionDisplayName = *FoundMetaData;
			}
			else
			{
				FunctionFriendlyName = FName::NameToDisplayString(FunctionDisplayName, false);
			}

			// There is a long-term Localization vision that no code is localized, but we are choosing to always localize in UE5.
			if (constexpr bool bUseLocalization = true)
			{
				// We are sometimes calling this function on a SKEL_ClassName_C Actor rather than the instance
				// So we should keep traversing the hierarchy finding the right function name
				const UFunction* BaseMostFunction = Function;
				for (; BaseMostFunction->GetSuperFunction(); BaseMostFunction = BaseMostFunction->GetSuperFunction());

				static const FTextKey LocalizationNamespace = TEXT("UObjectDisplayNames");
				FString LocalizationKey = BaseMostFunction->GetFullGroupName(false);
				FText OutLocalizedText;
				if (FText::FindTextInLiveTable_Advanced(LocalizationNamespace, LocalizationKey, OutLocalizedText, FunctionFriendlyName.IsEmpty() ? &FunctionDisplayName : &FunctionFriendlyName))
				{
					return OutLocalizedText;
				}
			}

			// Functions can opt to not use friendly names because they can be manually input by a user in the editor (and it would otherwise not adhere to their name)
			// There is a long-term goal of removing friendly names from the Engine.  However, we keep them in some cases as it helps the nodes
			// be decipherable in a zoomed-out view.
			const bool bUseFriendlyNames = bAllowFriendlyNames && GEditor && GetDefault<UEditorStyleSettings>()->bShowFriendlyNames;
			if (bUseFriendlyNames && !FunctionFriendlyName.IsEmpty())
			{
				return FText::FromString(FunctionFriendlyName);
			}
			else
			{
				return FText::FromString(FunctionDisplayName);
			}
		}

		return FText::GetEmpty();
	}

	FString GetDefaultTooltipForFunction(const UFunction* Function, bool bIncludeTarget)
	{
		if (!ensure(Function))
		{
			return FString();
		}

		FString Tooltip;
		if (Function != nullptr)
		{
			Tooltip = Function->GetToolTipText().ToString();
		}

		if (!Tooltip.IsEmpty())
		{
			// Strip off the doxygen nastiness
			static const FString DoxygenParam(TEXT("@param"));
			static const FString DoxygenReturn(TEXT("@return"));
			static const FString DoxygenSee(TEXT("@see"));
			static const FString TooltipSee(TEXT("See:"));
			static const FString DoxygenNote(TEXT("@note"));
			static const FString TooltipNote(TEXT("Note:"));

			Tooltip.Split(DoxygenParam, &Tooltip, nullptr, ESearchCase::IgnoreCase, ESearchDir::FromStart);
			Tooltip.Split(DoxygenReturn, &Tooltip, nullptr, ESearchCase::IgnoreCase, ESearchDir::FromStart);

			Tooltip.ReplaceInline(*DoxygenSee, *TooltipSee);
			Tooltip.ReplaceInline(*DoxygenNote, *TooltipNote);

			Tooltip.TrimStartAndEndInline();
			
			if (bIncludeTarget)
			{
				UClass* CurrentSelfClass = (Function != nullptr) ? Function->GetOwnerClass() : nullptr;
				UClass const* TrueSelfClass = CurrentSelfClass;
				if (CurrentSelfClass && CurrentSelfClass->ClassGeneratedBy)
				{
					TrueSelfClass = CurrentSelfClass->GetAuthoritativeClass();
				}
	
				FText TargetDisplayText = (TrueSelfClass != nullptr) ? TrueSelfClass->GetDisplayNameText() : LOCTEXT("None", "None");
	
				FFormatNamedArguments Args;
				Args.Add(TEXT("TargetName"), TargetDisplayText);
				Args.Add(TEXT("Tooltip"), FText::FromString(Tooltip));
				Tooltip = FText::Format(LOCTEXT("CallFunction_Tooltip", "{Tooltip}\n\nTarget is {TargetName}"), Args).ToString();
			}
		}
		else if (bIncludeTarget)
		{
			Tooltip = GetUserFacingFunctionName(Function).ToString();
		}
		
		return Tooltip;
	}

	FString GetDefaultTooltipForFunctionParamOrReturn(const UFunction* Function, const FString& ParamName, const bool bReturnValue)
	{
		if (!ensure(Function))
		{
			return FString();
		}
		
		const FString TagStr = bReturnValue ? TEXT("@return") : TEXT("@param");
		
		// grab the function's comment block for us to parse
		FString FunctionToolTipText = Function->GetToolTipText().ToString();
		
		int32 CurStrPos = INDEX_NONE;
		int32 FullToolTipLen = FunctionToolTipText.Len();
		// parse the full function tooltip text, looking for tag lines
		do 
		{
			CurStrPos = FunctionToolTipText.Find(TagStr, ESearchCase::IgnoreCase, ESearchDir::FromStart, CurStrPos);
			if (CurStrPos == INDEX_NONE) // if the tag wasn't found
			{
				break;
			}

			// advance past the tag
			CurStrPos += TagStr.Len();

			// handle people having done @returns instead of @return
			if (bReturnValue && CurStrPos < FullToolTipLen && FunctionToolTipText[CurStrPos] == TEXT('s'))
			{
				++CurStrPos;
			}

			// advance past whitespace
			while(CurStrPos < FullToolTipLen && FChar::IsWhitespace(FunctionToolTipText[CurStrPos]))
			{
				++CurStrPos;
			}

			// if this is a parameter pin
			if (!ParamName.IsEmpty())
			{
				FString TagParamName;

				// copy the parameter name
				while (CurStrPos < FullToolTipLen && !FChar::IsWhitespace(FunctionToolTipText[CurStrPos]))
				{
					TagParamName.AppendChar(FunctionToolTipText[CurStrPos++]);
				}

				// if this @param tag doesn't match the param we're looking for
				if (TagParamName != ParamName)
				{
					continue;
				}
			}

			// advance past whitespace (get to the meat of the comment)
			// since many doxygen style @param use the format "@param <param name> - <comment>" we also strip - if it is before we get to any other non-whitespace
			while(CurStrPos < FullToolTipLen && (FChar::IsWhitespace(FunctionToolTipText[CurStrPos]) || FunctionToolTipText[CurStrPos] == '-'))
			{
				++CurStrPos;
			}


			FString ParamDesc;
			// collect the param/return-val description
			while (CurStrPos < FullToolTipLen && FunctionToolTipText[CurStrPos] != TEXT('@'))
			{
				// advance past newline
				while(CurStrPos < FullToolTipLen && FChar::IsLinebreak(FunctionToolTipText[CurStrPos]))
				{
					++CurStrPos;

					// advance past whitespace at the start of a new line
					while(CurStrPos < FullToolTipLen && FChar::IsWhitespace(FunctionToolTipText[CurStrPos]))
					{
						++CurStrPos;
					}

					// replace the newline with a single space
					if(CurStrPos < FullToolTipLen && !FChar::IsLinebreak(FunctionToolTipText[CurStrPos]))
					{
						ParamDesc.AppendChar(TEXT(' '));
					}
				}

				if (CurStrPos < FullToolTipLen && FunctionToolTipText[CurStrPos] != TEXT('@'))
				{
					ParamDesc.AppendChar(FunctionToolTipText[CurStrPos++]);
				}
			}

			// trim any trailing whitespace from the descriptive text
			ParamDesc.TrimEndInline();

			// if we came up with a valid description for the param/return-val
			if (!ParamDesc.IsEmpty())
			{
				// we found a match, so there's no need to continue
				return ParamDesc;
			}

		} while (CurStrPos < FullToolTipLen);

		return FString();
	}

	FString GetDefaultTooltipForFunctionParam(const UFunction* Function, const FString& ParamName)
	{
		return GetDefaultTooltipForFunctionParamOrReturn(Function, ParamName, /*bReturnValue*/false);
	}
	
	FString GetDefaultTooltipForFunctionReturn(const UFunction* Function)
	{
		return GetDefaultTooltipForFunctionParamOrReturn(Function, FString(), /*bReturnValue*/true);
	}
}


UE_TRACE_EVENT_BEGIN(Cpu, RenderThumbnail, NoSync)
UE_TRACE_EVENT_FIELD(UE::Trace::WideString, ObjectPath)
UE_TRACE_EVENT_END()

namespace ThumbnailTools
{
	FAsyncObjectThumbnail::FAsyncObjectThumbnail(int32 InWidth, int32 InHeight, TSharedRef<FRHIGPUTextureReadback> InTextureReadback)
	: ImageWidth(InWidth)
	, ImageHeight(InHeight)
	, TextureReadback(InTextureReadback)
	{}

	bool FAsyncObjectThumbnail::IsTextureReadbackReady() const { return TextureReadback->IsReady() && !ObjectThumbnailFuture.IsSet(); }
	bool FAsyncObjectThumbnail::IsObjectThumbnailReady() const { return ObjectThumbnailFuture.IsSet() && ObjectThumbnailFuture->IsReady(); }

	void FAsyncObjectThumbnail::EnqueueReadbackDataCopy()
	{
		check(TextureReadback->IsReady());
		{
			ObjectThumbnailPromise = MakeShared<TPromise<FObjectThumbnail>>();
			ObjectThumbnailFuture = ObjectThumbnailPromise->GetFuture();

			ENQUEUE_RENDER_COMMAND(CopyReadbackDataToThumbnailRTCommand)(
			[ObjectThumbnailPromise = this->ObjectThumbnailPromise, ImageWidth = this->ImageWidth, ImageHeight = this->ImageHeight, TextureReadback = this->TextureReadback]
			(FRHICommandListImmediate& RHICmdList)
			{
				FObjectThumbnail ObjectThumbnail;
				ObjectThumbnail.SetImageSize(ImageWidth, ImageHeight);

				int32 ReadbackWidth = 0, ReadbackHeight = 0;
				void* ReadbackBuffer = TextureReadback->Lock(ReadbackWidth, &ReadbackHeight);

				if (ReadbackBuffer)
				{
					TArray<uint8>& ImageData = ObjectThumbnail.AccessImageData();
					ImageData.Empty();
					ImageData.AddUninitialized(ObjectThumbnail.GetImageWidth() * ObjectThumbnail.GetImageHeight() * sizeof(FColor));

					check(ReadbackBuffer && ObjectThumbnail.GetImageWidth() <= ReadbackWidth && ObjectThumbnail.GetImageHeight() <= ReadbackHeight);

					if (ObjectThumbnail.GetImageWidth() == ReadbackWidth && ObjectThumbnail.GetImageHeight() == ReadbackHeight)
					{
						FMemory::Memcpy( (void*)ImageData.GetData(), ReadbackBuffer, ObjectThumbnail.GetImageWidth() * ObjectThumbnail.GetImageHeight() * sizeof(FColor) );
					}
					else // Copy one line at a time
					{
						int32 SrcPitchElem = ReadbackWidth;
						int32 DstPitchElem = ObjectThumbnail.GetImageWidth();

						const uint8* SrcColorData = reinterpret_cast<const uint8*>(ReadbackBuffer);

						for (int32 RowIndex = 0; RowIndex < ObjectThumbnail.GetImageHeight(); RowIndex++)
						{
							const void* SrcPtr = SrcColorData + RowIndex * SrcPitchElem * sizeof(FColor);
							void* DstPtr = ImageData.GetData() + RowIndex * DstPitchElem * sizeof(FColor);
							FMemory::Memcpy(DstPtr, SrcPtr, DstPitchElem * sizeof(FColor));
						}
					}
				}

				TextureReadback->Unlock();

				ObjectThumbnailPromise->SetValue(MoveTemp(ObjectThumbnail));
			});
		}
	}

	void ReadbackThumbnail(FTextureRenderTargetResource* RenderTargetResource, FObjectThumbnail& OutThumbnail)
	{
		const FIntRect InSrcRect(0,	0, OutThumbnail.GetImageWidth(), OutThumbnail.GetImageHeight());

		TArray<uint8>& OutData = OutThumbnail.AccessImageData();

		OutData.Empty();
		OutData.AddUninitialized(OutThumbnail.GetImageWidth() * OutThumbnail.GetImageHeight() * sizeof(FColor));

		// Copy the contents of the remote texture to system memory
		// prefer GetRenderTargetImage()
		RenderTargetResource->ReadPixelsPtr((FColor*)OutData.GetData(), FReadSurfaceDataFlags(), InSrcRect);
	}

	FAsyncObjectThumbnail ReadbackThumbnailAsync(FTextureRenderTargetResource* RenderTargetResource, const int32 ImageWidth, const int32 ImageHeight)
	{
		TSharedRef<FRHIGPUTextureReadback> TextureReadback = MakeShared<FRHIGPUTextureReadback>("ThumbnailReadback");
		FAsyncObjectThumbnail AsyncObjectThumbnail(ImageWidth, ImageHeight, TextureReadback);

		ENQUEUE_RENDER_COMMAND(ReadbackThumbnailRTCommand)(
		[RenderTargetResource, ImageWidth, ImageHeight, TextureReadback]
		(FRHICommandListImmediate& RHICmdList) mutable
		{
			RHICmdList.Transition(FRHITransitionInfo(RenderTargetResource->GetRenderTargetTexture(), ERHIAccess::SRVMask, ERHIAccess::CopySrc));

			const FResolveRect SrcRect(0, 0, ImageWidth, ImageHeight);
			TextureReadback->EnqueueCopy(RHICmdList, RenderTargetResource->GetRenderTargetTexture(), SrcRect);

			RHICmdList.Transition(FRHITransitionInfo(RenderTargetResource->GetRenderTargetTexture(), ERHIAccess::CopySrc, ERHIAccess::SRVMask));
		});

		return AsyncObjectThumbnail;
	}

	bool RenderThumbnailInternal( UObject* InObject, uint32& InOutImageWidth, uint32& InOutImageHeight, EThumbnailTextureFlushMode::Type InFlushMode,
								  FTextureRenderTargetResource*& InOutTextureRenderTargetResource )
	{
		if (!FApp::CanEverRender())
		{
			return {};
		}
		
#if CPUPROFILERTRACE_ENABLED
		UE_TRACE_LOG_SCOPED_T(Cpu, RenderThumbnail, CpuChannel)
			<< RenderThumbnail.ObjectPath(*InObject->GetPathName());
#endif // CPUPROFILERTRACE_ENABLED

		// Renderer must be initialized before generating thumbnails
		check( GIsRHIInitialized );

		const bool bIsRenderingToSpecificRenderTarget = (InOutTextureRenderTargetResource != nullptr);

		// Grab the actual render target resource from the texture.  Note that we're absolutely NOT ALLOWED to
		// dereference this pointer.  We're just passing it along to other functions that will use it on the render
		// thread.  The only thing we're allowed to do is check to see if it's NULL or not.
		FTextureRenderTargetResource* RenderTargetResource = InOutTextureRenderTargetResource;
		if ( RenderTargetResource == NULL )
		{
			// No render target was supplied, just use a scratch texture render target
			const uint32 MinRenderTargetSize = FMath::Max( InOutImageWidth, InOutImageHeight );
			UTextureRenderTarget2D* RenderTargetTexture = GEditor->GetScratchRenderTarget( MinRenderTargetSize );
			check( RenderTargetTexture != NULL );

			// Make sure the input dimensions are OK.  The requested dimensions must be less than or equal to
			// our scratch render target size.
			check( InOutImageWidth <= RenderTargetTexture->GetSurfaceWidth() );
			check( InOutImageHeight <= RenderTargetTexture->GetSurfaceHeight() );

			RenderTargetResource = RenderTargetTexture->GameThread_GetRenderTargetResource();
			InOutTextureRenderTargetResource = RenderTargetResource;
		}
		check( RenderTargetResource != NULL );

		// Create a canvas for the render target and clear it to black
		FCanvas Canvas( RenderTargetResource, NULL, FGameTime::GetTimeSinceAppStart(), GMaxRHIFeatureLevel );
		Canvas.Clear( FLinearColor::Black );

		// Get the rendering info for this object
		FThumbnailRenderingInfo* RenderInfo = GUnrealEd ? GUnrealEd->GetThumbnailManager()->GetRenderingInfo( InObject ) : nullptr;

		UObject* ObjectToRender = InObject;
		if ((RenderInfo != nullptr) && RenderInfo->bUseClassDefaultObject)
		{
			if (UBlueprint* Blueprint = Cast<UBlueprint>(InObject))
			{
				if (Blueprint->GeneratedClass != nullptr)
				{
					ObjectToRender = Blueprint->GeneratedClass->GetDefaultObject(false);
				}
			}
		}

		if( InFlushMode == EThumbnailTextureFlushMode::AlwaysFlush )
		{
			// Wait for pending load requests.
			FlushAsyncLoading();

			// Wait for shader and other asset compilation to finish.
			FAssetCompilingManager::Get().FinishAllCompilation();

			// Force all mips to load.
			UTexture::ForceUpdateTextureStreaming();

			// Force all streamed resources to finish.
			IStreamingManager::Get().StreamAllResources();
		}

		// If this object's thumbnail will be rendered to a texture on the GPU.
		bool bUseGPUGeneratedThumbnail = true;

		if( RenderInfo != NULL && RenderInfo->Renderer != NULL )
		{
			// Make sure we suppress any message dialogs that might result from constructing
			// or initializing any of the renderable objects.
			TGuardValue<bool> Unattended(GIsRunningUnattendedScript, true);

			const float ZoomFactor = 1.0f;

			uint32 DrawWidth = InOutImageWidth;
			uint32 DrawHeight = InOutImageHeight;

			// If we're not rendering to a specific render target, then allow the thumbnail renderer to override the thumbnail size.
			// This happens when we render the thumbnail that will be saved in the asset package.
			if (!bIsRenderingToSpecificRenderTarget)
			{
				// Find how big the thumbnail WANTS to be
				uint32 DesiredWidth = 0;
				uint32 DesiredHeight = 0;
				{
					// Currently we only allow textures/icons (and derived classes) to override our desired size
					// @todo CB: Some thumbnail renderers (like particles and lens flares) hard code their own
					//	   arbitrary thumbnail size even though they derive from TextureThumbnailRenderer
					if( RenderInfo->Renderer->IsA( UTextureThumbnailRenderer::StaticClass() ) )
					{
						RenderInfo->Renderer->GetThumbnailSize(
							ObjectToRender,
							ZoomFactor,
							DesiredWidth,		// Out
							DesiredHeight );	// Out
					}
				}

				// Does this thumbnail have a size associated with it?  Materials and textures often do!
				if( DesiredWidth > 0 && DesiredHeight > 0 )
				{
					// Scale the desired size down if it's too big, preserving aspect ratio
					if( DesiredWidth > InOutImageWidth )
					{
						DesiredHeight = ( DesiredHeight * InOutImageWidth ) / DesiredWidth;
						DesiredWidth = InOutImageWidth;
					}
					if( DesiredHeight > InOutImageHeight )
					{
						DesiredWidth = ( DesiredWidth * InOutImageHeight ) / DesiredHeight;
						DesiredHeight = InOutImageHeight;
					}

					// Update dimensions
					DrawWidth = FMath::Max<uint32>(1, DesiredWidth);
					DrawHeight = FMath::Max<uint32>(1, DesiredHeight);

					InOutImageWidth = DrawWidth;
					InOutImageHeight = DrawHeight;
				}
			}

			// Draw the thumbnail
			const int32 XPos = 0;
			const int32 YPos = 0;
			const bool bAdditionalViewFamily = false;
			RenderInfo->Renderer->Draw(
				ObjectToRender,
				XPos,
				YPos,
				DrawWidth,
				DrawHeight,
				RenderTargetResource,
				&Canvas, 
				bAdditionalViewFamily
				);
		}

		// GPU based thumbnail rendering only
		if( bUseGPUGeneratedThumbnail )
		{
			// Tell the rendering thread to draw any remaining batched elements
			Canvas.Flush_GameThread();

			ENQUEUE_RENDER_COMMAND(UpdateThumbnailRTCommand)(
				[RenderTargetResource] (FRHICommandListImmediate& RHICmdList)
				{
					TransitionAndCopyTexture(RHICmdList, RenderTargetResource->GetRenderTargetTexture(), RenderTargetResource->TextureRHI, {});
				});
		}

		return true;
	}

	/** Renders a thumbnail for the specified object */
	void RenderThumbnail( UObject* InObject, const uint32 InImageWidth, const uint32 InImageHeight, EThumbnailTextureFlushMode::Type InFlushMode, FTextureRenderTargetResource* InTextureRenderTargetResource, FObjectThumbnail* OutThumbnail )
	{
		uint32 ImageWidth = InImageWidth;
		uint32 ImageHeight = InImageHeight;

		const bool bThumbnailRendered = RenderThumbnailInternal(InObject, ImageWidth, ImageHeight, InFlushMode, InTextureRenderTargetResource);

		if (OutThumbnail && bThumbnailRendered)
		{
			OutThumbnail->SetImageSize(ImageWidth, ImageHeight);
			ReadbackThumbnail(InTextureRenderTargetResource, *OutThumbnail);
		}
	}

	TOptional<FAsyncObjectThumbnail> RenderThumbnailAsync( UObject* InObject, const uint32 InImageWidth, const uint32 InImageHeight, EThumbnailTextureFlushMode::Type InFlushMode,
															   FTextureRenderTargetResource* InTextureRenderTargetResource )
	{
		uint32 ImageWidth = InImageWidth;
		uint32 ImageHeight = InImageHeight;

		const bool bThumbnailRendered = RenderThumbnailInternal(InObject, ImageWidth, ImageHeight, InFlushMode, InTextureRenderTargetResource);

		if (bThumbnailRendered)
		{
			return ReadbackThumbnailAsync(InTextureRenderTargetResource, ImageWidth, ImageHeight);
		}
		else
		{
			return {};
		}
	}

	/** Generates a thumbnail for the specified object and caches it */
	FObjectThumbnail* GenerateThumbnailForObjectToSaveToDisk( UObject* InObject )
	{
		// Does the object support thumbnails?
		FThumbnailRenderingInfo* RenderInfo = GUnrealEd ? GUnrealEd->GetThumbnailManager()->GetRenderingInfo( InObject ) : nullptr;
		if( RenderInfo != NULL && RenderInfo->Renderer != NULL )
		{
			// Set the size of cached thumbnails
			const int32 ImageWidth = ThumbnailTools::DefaultThumbnailSize;
			const int32 ImageHeight = ThumbnailTools::DefaultThumbnailSize;

			// For cached thumbnails we want to make sure that textures are fully streamed in so that the thumbnail we're saving won't have artifacts
			// However, this can add 30s - 100s to editor load
			//@todo - come up with a cleaner solution for this, preferably not blocking on texture streaming at all but updating when textures are fully streamed in
			const ThumbnailTools::EThumbnailTextureFlushMode::Type TextureFlushMode = ThumbnailTools::EThumbnailTextureFlushMode::NeverFlush;

			if ( UTexture* Texture = Cast<UTexture>(InObject) )
			{
				// SetForceMipLevelsToBeResident ?
				Texture->BlockOnAnyAsyncBuild();
				Texture->WaitForStreaming();
			}

			// When generating a material thumbnail to save in a package, make sure we finish compilation on the material first
			if ( UMaterial* InMaterial = Cast<UMaterial>(InObject) )
			{
				FScopedSlowTask SlowTask(0, NSLOCTEXT( "ObjectTools", "FinishingCompilationStatus", "Finishing Shader Compilation..." ) );
				SlowTask.MakeDialog();

				// Block until the shader maps that we will save have finished being compiled
				FMaterialResource* CurrentResource = InMaterial->GetMaterialResource(GMaxRHIShaderPlatform);
				if (CurrentResource)
				{
					if (!CurrentResource->IsGameThreadShaderMapComplete())
					{
						CurrentResource->SubmitCompileJobs_GameThread(EShaderCompileJobPriority::High);
					}
					CurrentResource->FinishCompilation();
				}
			}

			// Generate the thumbnail
			FObjectThumbnail NewThumbnail;
			RenderThumbnail(
				InObject, ImageWidth, ImageHeight, TextureFlushMode, NULL,
				&NewThumbnail );		// Out

			UPackage* MyOutermostPackage = InObject->GetOutermost();
			return CacheThumbnail( InObject->GetFullName(), &NewThumbnail, MyOutermostPackage );
		}

		return NULL;
	}

	/**
	 * Caches a thumbnail into a package's thumbnail map.
	 *
	 * @param	ObjectFullName	the full name for the object to associate with the thumbnail
	 * @param	Thumbnail		the thumbnail to cache; specify NULL to remove the current cached thumbnail
	 * @param	DestPackage		the package that will hold the cached thumbnail
	 *
	 * @return	pointer to the thumbnail data that was cached into the package
	 */
	FObjectThumbnail* CacheThumbnail( const FString& ObjectFullName, FObjectThumbnail* Thumbnail, UPackage* DestPackage )
	{
		FObjectThumbnail* Result = nullptr;

		if ( ObjectFullName.Len() > 0 && DestPackage != nullptr)
		{
			// Create a new thumbnail map if we don't have one already
			if( !DestPackage->HasThumbnailMap() )
			{
				DestPackage->SetThumbnailMap(MakeUnique<FThumbnailMap>());
			}

			FName InObjectShortClassFullName( *UClass::ConvertFullNameToShortTypeFullName( ObjectFullName ) );
			const FObjectThumbnail* CachedThumbnail = DestPackage->GetThumbnailMap().Find( InObjectShortClassFullName );
			if ( Thumbnail != nullptr )
			{
				// Cache the thumbnail (possibly replacing an existing thumb!)
				Result = &DestPackage->AccessThumbnailMap().Add( InObjectShortClassFullName, *Thumbnail );
			}
			//only let thumbnails loaded from disk to be removed.
			//When capturing thumbnails from the content browser, it will only exist in memory until it is saved out to a package.
			//Don't let the recycling purge them
			else if ((CachedThumbnail != nullptr) && (CachedThumbnail->IsLoadedFromDisk()))
			{
				DestPackage->AccessThumbnailMap().Remove( InObjectShortClassFullName );
			}

		}

		return Result;
	}



	/**
	 * Caches an empty thumbnail entry
	 *
	 * @param	ObjectFullName	the full name for the object to associate with the thumbnail
	 * @param	DestPackage		the package that will hold the cached thumbnail
	 */
	void CacheEmptyThumbnail( const FString& ObjectFullName, UPackage* DestPackage )
	{
		FObjectThumbnail EmptyThumbnail;
		CacheThumbnail( ObjectFullName, &EmptyThumbnail, DestPackage );
	}

	/** Returns the long path name of the package from InFullName */
	FString GetPackageNameForObject( const FString& InFullName )
	{
		// First strip off the class name
		int32 FirstSpaceIndex = InFullName.Find( TEXT( " " ) );
		if( FirstSpaceIndex == INDEX_NONE || FirstSpaceIndex <= 0 )
		{
			// Malformed full name
			return FString();
		}

		// Determine the package file path/name for the specified object
		FString ObjectPathName = InFullName.Mid( FirstSpaceIndex + 1 );

		// Pull the package out of the fully qualified object path
		int32 FirstDotIndex = ObjectPathName.Find( TEXT( "." ) );
		if( FirstDotIndex == INDEX_NONE || FirstDotIndex <= 0 )
		{
			// Malformed object path
			return FString();
		}

		return ObjectPathName.Left( FirstDotIndex );
	}

	/** Returns the package file name on disk from InFullName */
	bool QueryPackageFileNameForObject( const FString& InFullName, FString& OutPackageFileName )
	{
		FString PackageName = GetPackageNameForObject( InFullName );

		// Ask the package file cache for the full path to this package
		if( PackageName.IsEmpty() || !FPackageName::DoesPackageExist( PackageName, &OutPackageFileName ) )
		{
			// Couldn't find the package
			return false;
		}

		return true;
	}

	namespace Private
	{
		FObjectThumbnail* FindCachedThumbnailInPackage(UPackage* InPackage, const FName InObjectShortClassFullName)
		{
			FObjectThumbnail* FoundThumbnail = NULL;

			// We're expecting this to be an outermost package!
			check(InPackage->GetOutermost() == InPackage);

			// Does the package have any thumbnails?
			if (InPackage->HasThumbnailMap())
			{
				// @todo thumbnails: Backwards compat
				FThumbnailMap& PackageThumbnailMap = InPackage->AccessThumbnailMap();
				FoundThumbnail = PackageThumbnailMap.Find(InObjectShortClassFullName);
			}

			return FoundThumbnail;
		}
	}
	/** Searches for an object's thumbnail in memory and returns it if found */
	FObjectThumbnail* FindCachedThumbnailInPackage(UPackage* InPackage, FName InObjectFullName)
	{
		return Private::FindCachedThumbnailInPackage(InPackage,
			FName(*UClass::ConvertFullNameToShortTypeFullName(WriteToString<256>(InObjectFullName).ToView())));
	}
	/** Searches for an object's thumbnail in memory and returns it if found */
	FObjectThumbnail* FindCachedThumbnailInPackage(UPackage* InPackage, FStringView InObjectFullName)
	{
		return Private::FindCachedThumbnailInPackage(InPackage,
			FName(*UClass::ConvertFullNameToShortTypeFullName(InObjectFullName)));
	}
	/** Searches for an object's thumbnail in memory and returns it if found */
	FObjectThumbnail* FindCachedThumbnailInPackage(UPackage* InPackage, const TCHAR* InObjectFullName)
	{
		return Private::FindCachedThumbnailInPackage(InPackage,
			FName(*UClass::ConvertFullNameToShortTypeFullName(InObjectFullName)));
	}

	namespace Private
	{
		/** Searches for an object's thumbnail in memory and returns it if found */
		FObjectThumbnail* FindCachedThumbnailInPackage(const FString& InPackageFileName, const FName InObjectShortClassFullName)
		{
			FObjectThumbnail* FoundThumbnail = nullptr;

			FString PackageName = InPackageFileName;
			if (FPackageName::TryConvertFilenameToLongPackageName(PackageName, PackageName))
			{
				if (PackageName == TEXT("None"))
				{
					UE_LOGF(LogUObjectGlobals, Warning, "Attempted to FindCachedThumbnailInPackage named 'None' - PackageName: %ls InPackageFileName: %ls", *PackageName, *InPackageFileName);
					return nullptr;
				}

				// First check to see if the package is already in memory.  If it is, some or all of the thumbnails
				// may already be loaded and ready.
				UObject* PackageOuter = nullptr;
				UPackage* Package = FindPackage(PackageOuter, *PackageName);
				if (Package != nullptr)
				{
					FoundThumbnail = Private::FindCachedThumbnailInPackage(Package, InObjectShortClassFullName);
				}
			}
			return FoundThumbnail;
		}
	}

	/** Searches for an object's thumbnail in memory and returns it if found */
	FObjectThumbnail* FindCachedThumbnailInPackage(const FString& InPackageFileName, FName InObjectFullName)
	{
		return Private::FindCachedThumbnailInPackage(InPackageFileName,
			FName(*UClass::ConvertFullNameToShortTypeFullName(WriteToString<256>(InObjectFullName).ToView())));
	}
	/** Searches for an object's thumbnail in memory and returns it if found */
	FObjectThumbnail* FindCachedThumbnailInPackage(const FString& InPackageFileName, FStringView InObjectFullName)
	{
		return Private::FindCachedThumbnailInPackage(InPackageFileName,
			FName(*UClass::ConvertFullNameToShortTypeFullName(InObjectFullName)));
	}
	/** Searches for an object's thumbnail in memory and returns it if found */
	FObjectThumbnail* FindCachedThumbnailInPackage(const FString& InPackageFileName, const TCHAR* InObjectFullName)
	{
		return Private::FindCachedThumbnailInPackage(InPackageFileName,
			FName(*UClass::ConvertFullNameToShortTypeFullName(InObjectFullName)));
	}

	/** Searches for an object's thumbnail in memory and returns it if found */
	const FObjectThumbnail* FindCachedThumbnail( const FString& InFullName )
	{
		// Determine the package file path/name for the specified object
		const FString PackageFileName = GetPackageNameForObject(InFullName);

		if (PackageFileName.IsEmpty())
		{
			// Couldn't find the package
			return nullptr;
		}

		return FindCachedThumbnailInPackage( PackageFileName, *InFullName );
	}



	/** Returns the thumbnail for the specified object or NULL if one doesn't exist yet */
	FObjectThumbnail* GetThumbnailForObject( UObject* InObject )
	{
		UPackage* ObjectPackage = InObject->GetOutermost();
		return FindCachedThumbnailInPackage( ObjectPackage, *InObject->GetFullName() );
	}



	/** Loads thumbnails from the specified package file name */
	bool LoadThumbnailsFromPackageDirectly(const FString& InPackageFileName, const TSet<FName>& InObjectFullNames, FThumbnailMap& InOutThumbnails, const FString& InPackageFileNameToOpen)
	{
		// Create a file reader to load the file
		TUniquePtr<FArchive> FileReader(IFileManager::Get().CreateFileReader(*InPackageFileNameToOpen));
		if( FileReader == nullptr )
		{
			// Couldn't open the file
			return false;
		}


		// Read package file summary from the file
		FPackageFileSummary FileSummary;
		(*FileReader) << FileSummary;

		// Make sure this is indeed a package
		if( FileSummary.Tag != PACKAGE_FILE_TAG || FileReader->IsError() )
		{
			// Unrecognized or malformed package file
			return false;
		}


		// Does the package contains a thumbnail table?
		if( FileSummary.ThumbnailTableOffset == 0 )
		{
			// No thumbnails to be loaded
			return false;
		}

		// Seek the the part of the file where the thumbnail table lives
		FileReader->Seek( FileSummary.ThumbnailTableOffset );

		int32 LastFileOffset = -1;
		// Load the thumbnail table of contents
		TMap< FName, int32 > ObjectNameToFileOffsetMap;
		{
			// Load the thumbnail count
			int32 ThumbnailCount = 0;
			*FileReader << ThumbnailCount;

			// Load the names and file offsets for the thumbnails in this package
			for( int32 CurThumbnailIndex = 0; CurThumbnailIndex < ThumbnailCount; ++CurThumbnailIndex )
			{
				bool bHaveValidClassName = false;
				FString ObjectShortClassName;
				*FileReader << ObjectShortClassName;

				// Object path
				FString ObjectPathWithoutPackageName;
				*FileReader << ObjectPathWithoutPackageName;

				FString ObjectPath;

				// handle UPackage thumbnails differently from usual assets
				if (ObjectShortClassName == UPackage::StaticClass()->GetName())
				{
					ObjectPath = ObjectPathWithoutPackageName;
				}
				else
				{
					ObjectPath = ( FPackageName::FilenameToLongPackageName(InPackageFileName) + TEXT( "." ) + ObjectPathWithoutPackageName );
				}

				// If the thumbnail was stored with a missing class name ("???") when we'll catch that here
				if(ObjectShortClassName.Len() > 0 && ObjectShortClassName != TEXT( "???" ) )
				{
					bHaveValidClassName = true;
				}
				else
				{
					// Class name isn't valid.  Probably legacy data.  We'll try to fix it up below.
				}


				if( !bHaveValidClassName )
				{
					// Try to figure out a class name based on input assets.  This should really only be needed
					// for packages saved by older versions of the editor (VER_CONTENT_BROWSER_FULL_NAMES)
					for ( TSet<FName>::TConstIterator It(InObjectFullNames); It; ++It )
					{
						const FName& CurObjectFullNameFName = *It;

						FString CurObjectFullName;
						CurObjectFullNameFName.ToString( CurObjectFullName );

						if( CurObjectFullName.EndsWith( ObjectPath ) )
						{
							// Great, we found a path that matches -- we just need to add that class name
							const int32 FirstSpaceIndex = CurObjectFullName.Find( TEXT( " " ) );
							check( FirstSpaceIndex != -1 );
							ObjectShortClassName = CurObjectFullName.Left( FirstSpaceIndex );
							ObjectShortClassName = UClass::ConvertPathNameToShortTypeName(ObjectShortClassName);

							// We have a useful class name now!
							bHaveValidClassName = true;
							break;
						}
					}
				}


				// File offset to image data
				int32 FileOffset = 0;
				*FileReader << FileOffset;

				if ( FileOffset != -1 && FileOffset < LastFileOffset )
				{
					UE_LOGF(LogObjectTools, Warning, "Loaded thumbnail '%ls' out of order!: FileOffset:%i    LastFileOffset:%i", *ObjectPath, FileOffset, LastFileOffset);
				}


				if( bHaveValidClassName )
				{
					// Create a full name string with the object's class and fully qualified path
					const FString ObjectFullName(ObjectShortClassName + TEXT( " " ) + ObjectPath);


					// Add to our map
					ObjectNameToFileOffsetMap.Add( FName( *ObjectFullName ), FileOffset );
				}
				else
				{
					// Oh well, we weren't able to fix the class name up.  We won't bother making this
					// thumbnail available to load
				}
			}
		}


		// @todo CB: Should sort the thumbnails to load by file offset to reduce seeks [reviewed; pre-qa release]
		for ( TSet<FName>::TConstIterator It(InObjectFullNames); It; ++It )
		{
			FName CurObjectFullName = *It;
			FName CurObjectShortClassFullName = FName(*UClass::ConvertFullNameToShortTypeFullName(WriteToString<256>(CurObjectFullName).ToView()));

			// Do we have this thumbnail in the file?
			// @todo thumbnails: Backwards compat
			const int32* pFileOffset = ObjectNameToFileOffsetMap.Find(CurObjectShortClassFullName);
			if ( pFileOffset != NULL )
			{
				// Seek to the location in the file with the image data
				FileReader->Seek( *pFileOffset );

				// Load the image data
				FObjectThumbnail LoadedThumbnail;
				LoadedThumbnail.Serialize( *FileReader );

				// Store the data!
				InOutThumbnails.Add( CurObjectFullName, LoadedThumbnail );
			}
			else
			{
				// Couldn't find the requested thumbnail in the file!
			}
		}


		return true;
	}



	bool GetPackageFilePathAndAssetFullName(const FAssetData& AssetData, FString& OutPackageFilePath, FName& OutAssetFullName)
	{
		// Determine package file path
		if (FPackageName::DoesPackageExist(AssetData.PackageName.ToString(), &OutPackageFilePath))
		{
			// Determine asset fullname
			FNameBuilder FullNameBuilder;
			AssetData.GetFullName(FullNameBuilder);
			OutAssetFullName = FName(FullNameBuilder);
			return true;
		}
		return false;
	}



	bool LoadThumbnailFromPackage(const FAssetData& AssetData, FObjectThumbnail& OutThumbnail)
	{
		FString PackageFilePath;
		FName AssetFullName;
		if (GetPackageFilePathAndAssetFullName(AssetData, PackageFilePath, AssetFullName))
		{
			TSet<FName> AssetFullNames;
			AssetFullNames.Add(AssetFullName);

			FThumbnailMap ThumbnailMap;
			LoadThumbnailsFromPackage(PackageFilePath, AssetFullNames, ThumbnailMap);
			if (FObjectThumbnail* Found = ThumbnailMap.Find(AssetFullName))
			{
				OutThumbnail = MoveTemp(*Found);
				return true;
			}
		}
		return false;
	}


	TArray<FLoadThumbnailsFromPackage>& GetLoadThumbnailsFromPackageDelegate()
	{
		static TArray<FLoadThumbnailsFromPackage> LoadThumbnailsFromPackageDelegates;
		return LoadThumbnailsFromPackageDelegates;
	}

	FDelegateHandle AddLoadThumbnailsFromPackageDelegate(FLoadThumbnailsFromPackage InDelegate)
	{
		return GetLoadThumbnailsFromPackageDelegate().Add_GetRef(InDelegate).GetHandle();
	}

	void RemoveLoadThumbnailsFromPackageDelegate(const FDelegateHandle InHandle)
	{
		GetLoadThumbnailsFromPackageDelegate().RemoveAll([InHandle](const FLoadThumbnailsFromPackage& Entry)
		{
			return Entry.GetHandle() == InHandle || !Entry.IsBound();
		});
	}

	/** Loads thumbnails from the specified package file name, try loading from external cache file if not found in package file */
	bool LoadThumbnailsFromPackage( const FString& InPackageFileName, const TSet< FName >& InObjectFullNames, FThumbnailMap& InOutThumbnails )
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("ThumbnailTools::LoadThumbnailsFromPackage");
		if (LoadThumbnailsFromPackageDirectly(InPackageFileName, InObjectFullNames, InOutThumbnails, InPackageFileName))
		{
			return true;
		}
		else if (FThumbnailExternalCache::Get().LoadThumbnailsFromExternalCache(InObjectFullNames, InOutThumbnails))
		{
			return true;
		}
		else
		{
			// Delegates
			bool bDelegatesNeedCleanup = false;
			for (const FLoadThumbnailsFromPackage& DelegateEntry : GetLoadThumbnailsFromPackageDelegate())
			{
				if (DelegateEntry.IsBound())
				{
					if (DelegateEntry.Execute(InPackageFileName, InObjectFullNames, InOutThumbnails))
					{
						return true;
					}
				}
				else
				{
					bDelegatesNeedCleanup = true;
				}
			}

			if (bDelegatesNeedCleanup)
			{
				RemoveLoadThumbnailsFromPackageDelegate(FDelegateHandle());
			}
		}

		return false;
	}



	/** Loads thumbnails from a package unless they're already cached in that package's thumbnail map */
	bool ConditionallyLoadThumbnailsFromPackage( const FString& InPackageFileName, const TSet< FName >& InObjectFullNames, FThumbnailMap& InOutThumbnails )
	{
		// First check to see if any of the requested thumbnails are already in memory
		TSet< FName > ObjectFullNamesToLoad;
		ObjectFullNamesToLoad.Empty(InObjectFullNames.Num());
		for ( TSet<FName>::TConstIterator It(InObjectFullNames); It; ++It )
		{
			const FName& CurObjectFullName = *It;

			// Do we have this thumbnail in our cache already?
			// @todo thumbnails: Backwards compat
			const FObjectThumbnail* FoundThumbnail = FindCachedThumbnailInPackage( InPackageFileName, CurObjectFullName );
			if( FoundThumbnail != NULL )
			{
				// Great, we already have this thumbnail in memory!  Copy it to our output map.
				InOutThumbnails.Add( CurObjectFullName, *FoundThumbnail );
			}
			else
			{
				ObjectFullNamesToLoad.Add(CurObjectFullName);
			}
		}


		// Did we find all of the requested thumbnails in our cache?
		if( ObjectFullNamesToLoad.Num() == 0 )
		{
			// Done!
			return true;
		}

		// OK, go ahead and load the remaining thumbnails!
		return LoadThumbnailsFromPackage( InPackageFileName, ObjectFullNamesToLoad, InOutThumbnails );
	}



	/** Loads thumbnails for the specified objects (or copies them from a cache, if they're already loaded.) */
	bool ConditionallyLoadThumbnailsForObjects( const TArray< FName >& InObjectFullNames, FThumbnailMap& InOutThumbnails )
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ConditionallyLoadThumbnailsForObjects);

		// Create a list of unique package file names that we'll need to interrogate
		struct FObjectFullNamesForPackage
		{
			TSet< FName > ObjectFullNames;
		};

		typedef TMap< FString, FObjectFullNamesForPackage > PackageFileNameToObjectPathsMap;
		PackageFileNameToObjectPathsMap PackagesToProcess;
		for( int32 CurObjectIndex = 0; CurObjectIndex < InObjectFullNames.Num(); ++CurObjectIndex )
		{
			const FName ObjectFullName = InObjectFullNames[ CurObjectIndex ];


			// Determine the package file path/name for the specified object
			FString PackageFilePathName;
			if( !QueryPackageFileNameForObject( ObjectFullName.ToString(), PackageFilePathName ) )
			{
				// Couldn't find the package in our cache
				return false;
			}


			// Do we know about this package yet?
			FObjectFullNamesForPackage* ObjectFullNamesForPackage = PackagesToProcess.Find( PackageFilePathName );
			if( ObjectFullNamesForPackage == NULL )
			{
				ObjectFullNamesForPackage = &PackagesToProcess.Add( PackageFilePathName, FObjectFullNamesForPackage() );
			}

			if ( ObjectFullNamesForPackage->ObjectFullNames.Find(ObjectFullName) == NULL )
			{
				ObjectFullNamesForPackage->ObjectFullNames.Add(ObjectFullName);
			}
		}


		// Load thumbnails, one package at a time
		for( PackageFileNameToObjectPathsMap::TConstIterator PackageIt( PackagesToProcess ); PackageIt; ++PackageIt )
		{
			const FString& CurPackageFileName = PackageIt.Key();
			const FObjectFullNamesForPackage& CurPackageObjectPaths = PackageIt.Value();

			if( !ConditionallyLoadThumbnailsFromPackage( CurPackageFileName, CurPackageObjectPaths.ObjectFullNames, InOutThumbnails ) )
			{
				// Failed to load thumbnail data
				return false;
			}
		}


		return true;
	}

	bool AssetHasCustomThumbnail(const FString& InAssetDataFullName)
	{
		FObjectThumbnail Thumbnail;
		return AssetHasCustomThumbnail(InAssetDataFullName, Thumbnail);
	}

	bool AssetHasCustomThumbnail(const FString& InAssetDataFullName, FObjectThumbnail& OutThumbnail)
	{
		const FObjectThumbnail* CachedThumbnail = FindCachedThumbnail(InAssetDataFullName);
		if (CachedThumbnail != NULL && !CachedThumbnail->IsEmpty())
		{
			OutThumbnail = *CachedThumbnail;
			return true;
		}

		// If we don't yet have a thumbnail map, check the disk
		FName ObjectFullName = FName(*InAssetDataFullName);
		TArray<FName> ObjectFullNames;
		FThumbnailMap LoadedThumbnails;
		ObjectFullNames.Add(ObjectFullName);
		if (ConditionallyLoadThumbnailsForObjects(ObjectFullNames, LoadedThumbnails))
		{
			const FObjectThumbnail* Thumbnail = LoadedThumbnails.Find(ObjectFullName);

			if (Thumbnail != NULL && !Thumbnail->IsEmpty())
			{
				OutThumbnail = *Thumbnail;
				return true;
			}
		}
		return false;
	}

	bool AssetHasCustomCreatedThumbnail(const FString& InAssetDataFullName)
	{
		FObjectThumbnail Thumbnail;
		if (AssetHasCustomThumbnail(InAssetDataFullName, Thumbnail))
		{
			return Thumbnail.IsCreatedAfterCustomThumbsEnabled();
		}

		return false;
	}
}

#undef LOCTEXT_NAMESPACE
