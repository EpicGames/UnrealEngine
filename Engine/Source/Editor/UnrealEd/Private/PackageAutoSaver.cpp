// Copyright Epic Games, Inc. All Rights Reserved.

#include "PackageAutoSaver.h"

#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/App.h"
#include "Dom/JsonValue.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonReader.h"
#include "Policies/PrettyJsonPrintPolicy.h"
#include "Serialization/JsonSerializer.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/AppStyle.h"
#include "Editor/UnrealEdEngine.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "UnrealEdMisc.h"
#include "FileHelpers.h"
#include "FileHelpersInternal.h"
#include "InterchangeManager.h"
#include "UncontrolledChangelistsModule.h"
#include "UnrealEdGlobals.h"
#include "UObject/Linker.h"
#include "PackageRestore.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "AutoSaveUtils.h"
#include "ShaderCompiler.h"
#include "EditorLevelUtils.h"
#include "ISourceControlModule.h"
#include "SourceControlOperations.h"
#include "IVREditorModule.h"
#include "LevelEditorViewport.h"
#include "AssetCompilingManager.h"
#include "AssetDefinition.h"
#include "AssetDefinitionRegistry.h"

namespace PackageAutoSaver
{
	static bool bUseUncontrolledChangelists = true;
	static FAutoConsoleVariableRef CVarUseUncontrolledChangelists(TEXT("Editor.AutoSave.UseUncontrolledChangelists"), bUseUncontrolledChangelists, TEXT("True to use uncontrolled changelists (if possible) when using the BackupAndOverwrite auto-save method."));

	static bool bInteractiveSaveManagesUncontrolledChangelist = true;
	static FAutoConsoleVariableRef CVarInteractiveSaveManagesUncontrolledChangelist(TEXT("Editor.AutoSave.InteractiveSaveManagesUncontrolledChangelist"), bInteractiveSaveManagesUncontrolledChangelist, TEXT("True if an interactive save should manage which entries stay in the auto-save uncontrolled changelist (ie, promoting them to a controlled save, or moving them to the default uncontrolled changelist)."));

	static const FUncontrolledChangelist AutoSaveUncontrolledChangelist(FGuid(0x0D227D90, 0x2AC34632, 0xB1A35533, 0xF3CC134E));

	bool CanUseUncontrolledChangelist()
	{
		return bUseUncontrolledChangelists && FUncontrolledChangelistsModule::Get().IsEnabled() && ISourceControlModule::Get().GetProvider().UsesLocalReadOnlyState();
	}

	bool ShouldInteractiveSaveManageUncontrolledChangelist(const bool bRequiresSourceControl = true)
	{
		return bInteractiveSaveManagesUncontrolledChangelist && (!bRequiresSourceControl || (ISourceControlModule::Get().IsEnabled() && ISourceControlModule::Get().GetProvider().IsAvailable()));
	}

	TMap<FName, FString> GetPackagesInUncontrolledChangelist(const bool bUpdateStatus = false)
	{
		TMap<FName, FString> UncontrolledPackages;

		if (CanUseUncontrolledChangelist())
		{
			if (FUncontrolledChangelistStatePtr AutoSaveUncontrolledChangelistState = FUncontrolledChangelistsModule::Get().GetChangelistState(AutoSaveUncontrolledChangelist))
			{
				if (bUpdateStatus && AutoSaveUncontrolledChangelistState->UpdateStatus())
				{
					FUncontrolledChangelistsModule::Get().HandleChangelistStateModified();
				}

				for (const FString& UncontrolledFilename : AutoSaveUncontrolledChangelistState->GetFilenames())
				{
					FNameBuilder PackageName;
					if (FPackageName::TryConvertFilenameToLongPackageName(UncontrolledFilename, PackageName))
					{
						UncontrolledPackages.Add(FName(PackageName), UncontrolledFilename);
					}
				}
			}
		}

		return UncontrolledPackages;
	}
}

namespace PackageAutoSaverJson
{
	typedef TCHAR CharType;

	typedef TJsonWriter< CharType, TPrettyJsonPrintPolicy<CharType> > FStringWriter;
	typedef TJsonWriterFactory< CharType, TPrettyJsonPrintPolicy<CharType> > FStringWriterFactory;

	typedef TJsonReader<CharType> FJsonReader;
	typedef TJsonReaderFactory<CharType> FJsonReaderFactory;

	static const FString TagRestoreEnabled	 = TEXT("RestoreEnabled");
	static const FString TagPackages		 = TEXT("Packages");
	static const FString TagPackagePathName  = TEXT("PackagePathName");
	static const FString TagPackageAssetName = TEXT("PackageAssetName");
	static const FString TagAutoSavePath	 = TEXT("AutoSavePath");
	static const FString RestoreFilename	 = TEXT("PackageRestoreData.json");

	/**
	 * @param bEnsurePath True to ensure that the directory for the restore file exists

	 * @return Returns the full path to the restore file
	 */
	FString GetRestoreFilename(const bool bEnsurePath);

	/**
	 * Load the restore file from disk (if present)
	 *
	 * @return The packages that have auto-saves that they can be restored from
	 */
	TMap<FString, TPair<FString, FString>> LoadRestoreFile();

	/**
	 * Save the file on disk that's used to restore auto-saved packages in the event of a crash
	 * 
	 * @param bRestoreEnabled	Is the restore enabled, or is it disabled because we've shut-down cleanly, or are running under the debugger?
	 * @param DirtyPackages		Packages that may have auto-saves that they could be restored from
	 */
	void SaveRestoreFile(const bool bRestoreEnabled, const TMap< TWeakObjectPtr<UPackage>, FString, FDefaultSetAllocator, TWeakObjectPtrMapKeyFuncs<TWeakObjectPtr<UPackage>, FString> >& DirtyPackages);

	/** @return whether the auto-save restore should be enabled (you can force this to true when testing with a debugger attached) */
	bool IsRestoreEnabled()
	{
		// Restore is disabled unless using the BackupAndRestore auto-save method
		if (GetDefault<UEditorLoadingSavingSettings>()->AutoSaveMethod != EAutoSaveMethod::BackupAndRestore)
		{
			return false;
		}

		// Note: Restore is disabled when running under the debugger, as programmers
		// like to just kill applications and we don't want this to count as a crash
		return !FPlatformMisc::IsDebuggerPresent();
	}
}


/************************************************************************/
/* FPackageAutoSaver                                                    */
/************************************************************************/

DEFINE_LOG_CATEGORY_STATIC(LogPackageAutoSaver, Log, All);

FPackageAutoSaver::FPackageAutoSaver()
	: AutoSaveIndex(0)
	, AutoSaveCount(0.0f)
	, bDelayingDueToFailedSave(false)
	, bAutoDeclineRecovery(FParse::Param(FCommandLine::Get(), TEXT("AutoDeclinePackageRecovery")))
{
	// Register for the package dirty state updated callback to catch packages that have been cleaned without being saved
	UPackage::PackageDirtyStateChangedEvent.AddRaw(this, &FPackageAutoSaver::OnPackageDirtyStateUpdated);

	// Register for the "MarkPackageDirty" callback to catch packages that have been modified and need to be saved
	UPackage::PackageMarkedDirtyEvent.AddRaw(this, &FPackageAutoSaver::OnMarkPackageDirty);

	// Register for the package modified callback to catch packages that have been saved
	UPackage::PackageSavedWithContextEvent.AddRaw(this, &FPackageAutoSaver::OnPackageSaved);

	// Register to detect when an Undo/Redo changes the dirty state of a package
	FEditorDelegates::PostUndoRedo.AddRaw(this, &FPackageAutoSaver::OnUndoRedo);

	// Register for the dirty sync handling
	FCoreDelegates::OnEndFrame.AddRaw(this, &FPackageAutoSaver::OnEndFrame);
	FCoreUObjectDelegates::OnEndLoadPackage.AddRaw(this, &FPackageAutoSaver::OnPackageLoaded);
	FUncontrolledChangelistsModule::Get().OnUncontrolledChangelistModuleChanged.AddRaw(this, &FPackageAutoSaver::OnUncontrolledChangelistChanged);

	// Register for the interactive checkout handlers
	UE::FileHelpers::Internal::GetAdditionalInteractiveSavePackageCandidates.AddRaw(this, &FPackageAutoSaver::GetAdditionalInteractiveSavePackageCandidates);
	UE::FileHelpers::Internal::OnPreInteractiveCheckoutPackages.AddRaw(this, &FPackageAutoSaver::OnPreInteractiveCheckoutPackages);
	UE::FileHelpers::Internal::OnPostInteractiveCheckoutPackages.AddRaw(this, &FPackageAutoSaver::OnPostInteractiveCheckoutPackages);
	UE::FileHelpers::Internal::OnPackagesInteractivelyCheckedOut.AddRaw(this, &FPackageAutoSaver::OnPackagesInteractivelyCheckedOut);
	UE::FileHelpers::Internal::OnPackagesInteractivelyMadeWritable.AddRaw(this, &FPackageAutoSaver::OnPackagesInteractivelyMadeWritable);
	UE::FileHelpers::Internal::OnPackagesInteractivelyDiscarded.AddRaw(this, &FPackageAutoSaver::OnPackagesInteractivelyDiscarded);
}

FPackageAutoSaver::~FPackageAutoSaver()
{
	UPackage::PackageDirtyStateChangedEvent.RemoveAll(this);
	UPackage::PackageMarkedDirtyEvent.RemoveAll(this);
	UPackage::PackageSavedWithContextEvent.RemoveAll(this);
	FEditorDelegates::PostUndoRedo.RemoveAll(this);

	FCoreDelegates::OnEndFrame.RemoveAll(this);
	FCoreUObjectDelegates::OnEndLoadPackage.RemoveAll(this);
	if (FUncontrolledChangelistsModule* UncontrolledChangelistsModule = FUncontrolledChangelistsModule::GetPtr())
	{
		UncontrolledChangelistsModule->OnUncontrolledChangelistModuleChanged.RemoveAll(this);
	}

	UE::FileHelpers::Internal::GetAdditionalInteractiveSavePackageCandidates.RemoveAll(this);
	UE::FileHelpers::Internal::OnPreInteractiveCheckoutPackages.RemoveAll(this);
	UE::FileHelpers::Internal::OnPostInteractiveCheckoutPackages.RemoveAll(this);
	UE::FileHelpers::Internal::OnPackagesInteractivelyCheckedOut.RemoveAll(this);
	UE::FileHelpers::Internal::OnPackagesInteractivelyMadeWritable.RemoveAll(this);
	UE::FileHelpers::Internal::OnPackagesInteractivelyDiscarded.RemoveAll(this);
}

void FPackageAutoSaver::UpdateAutoSaveCount(const float DeltaSeconds)
{
	const UEditorLoadingSavingSettings* LoadingSavingSettings = GetDefault<UEditorLoadingSavingSettings>();

	const float AutoSaveWarningTime = FMath::Max(0.0f, static_cast<float>(LoadingSavingSettings->AutoSaveTimeMinutes * 60 - LoadingSavingSettings->AutoSaveWarningInSeconds));

	// Make sure we don't skip the auto-save warning when debugging the editor. 
	if (AutoSaveCount < AutoSaveWarningTime && (AutoSaveCount + DeltaSeconds) > AutoSaveWarningTime)
	{
		AutoSaveCount = AutoSaveWarningTime;
	}
	else
	{
		AutoSaveCount += DeltaSeconds;
	}
}

void FPackageAutoSaver::ResetAutoSaveTimer()
{
	// Reset the "seconds since last auto-save" counter.
	AutoSaveCount = 0.0f;
}

void FPackageAutoSaver::ForceAutoSaveTimer()
{
	AutoSaveCount = GetDefault<UEditorLoadingSavingSettings>()->AutoSaveTimeMinutes * 60.0f;
}

void FPackageAutoSaver::ForceMinimumTimeTillAutoSave(const float TimeTillAutoSave)
{
	const float MinumumTime = (GetDefault<UEditorLoadingSavingSettings>()->AutoSaveTimeMinutes * 60.0f) - TimeTillAutoSave;
	AutoSaveCount = (MinumumTime < AutoSaveCount) ? MinumumTime : AutoSaveCount;
}

void FPackageAutoSaver::ResumeAutoSave()
{
	--AutoSaveSuspendedCounter;
	if (!ensureMsgf(AutoSaveSuspendedCounter >= 0, TEXT("FPackageAutoSaver::ResumeAutoSave() called more times than FPackageAutoSaver::SuspendAutoSave(), those calls should come in pair.")))
	{
		AutoSaveSuspendedCounter = 0;
	}
}

bool FPackageAutoSaver::AttemptAutoSave(const bool bForceAutoSave)
{
	const UEditorLoadingSavingSettings* LoadingSavingSettings = GetDefault<UEditorLoadingSavingSettings>();
	FUnrealEdMisc& UnrealEdMisc = FUnrealEdMisc::Get();

	// Re-sync if needed
	if (bSyncWithDirtyPackageList)
	{
		bSyncWithDirtyPackageList = false;
		PackagesPendingUpdate.Reset();

		DirtyMapsForAutoSave.Reset();
		DirtyContentForAutoSave.Reset();

		// The the list of dirty packages tracked by the engine (considered source of truth)
		TArray<UPackage*> DirtyPackages;
		FEditorFileUtils::GetDirtyPackages(DirtyPackages);
		for (UPackage* Pkg : DirtyPackages)
		{
			UpdateDirtyListsForPackage(Pkg);
		}

		// Remove any clean package from the user-restore list
		for (auto It = DirtyPackagesForUserSave.CreateIterator(); It; ++It)
		{
			UPackage* Pkg = It->Key.Get();
			if (!Pkg || !Pkg->IsDirty() || (PackagesToIgnoreIfEmpty.Contains(Pkg->GetFName()) && UPackage::IsEmptyPackage(Pkg)))
			{
				bNeedRestoreFileUpdate = true;
				It.RemoveCurrent();
			}
		}
	}

	// Process any packages that are pending an update
	if (PackagesPendingUpdate.Num() > 0)
	{
		for (const TWeakObjectPtr<UPackage>& WeakPkg : PackagesPendingUpdate)
		{
			if (UPackage* Pkg = WeakPkg.Get())
			{
				UpdateDirtyListsForPackage(Pkg);
			}
		}
		PackagesPendingUpdate.Reset();
	}

	// Update the restore information too, if needed
	if (bNeedRestoreFileUpdate)
	{
		UpdateRestoreFile(PackageAutoSaverJson::IsRestoreEnabled());
	}

	// Don't auto-save if disabled or if it is not yet time to auto-save.
	const bool bTimeToAutosave = bForceAutoSave || (!IsAutoSaveSuspended() && LoadingSavingSettings->bAutoSaveEnable && AutoSaveCount >= LoadingSavingSettings->AutoSaveTimeMinutes * 60.0f);
	bool bAutosaveHandled = false;
	bool bDidAutoSaveSuccessfully = false;

	if (bTimeToAutosave)
	{
		ClearStalePointers();

		// If we don't need to perform an auto-save, then just reset the timer and bail
		const bool bNeedsAutoSave = DoPackagesNeedAutoSave();
		if(!bNeedsAutoSave)
		{
			ResetAutoSaveTimer();
			return false;
		}

		// Don't auto-save during interpolation editing, if there's another slow task
		// already in progress, or while a PIE world is playing or when doing automated tests.
		const bool bCanAutosave = CanAutoSave(bForceAutoSave);
		if (bCanAutosave)
		{
			FScopedSlowTask SlowTask(100.f, NSLOCTEXT("AutoSaveNotify", "PerformingAutoSave_Caption", "Auto-saving out of date packages..."));
			SlowTask.MakeDialog();

			GUnrealEd->SaveConfig();

			bAutosaveHandled = true;

			// Make sure the auto-save directory exists before attempting to write the file
			const FString AutoSaveDir = AutoSaveUtils::GetAutoSaveDir();
			IFileManager::Get().MakeDirectory(*AutoSaveDir, true);

			const int32 AutoSaveMaxBackups = LoadingSavingSettings->AutoSaveMaxBackups > 0 ? LoadingSavingSettings->AutoSaveMaxBackups : 10;
			// Auto-save maps and/or content packages based on user settings.
			const int32 NewAutoSaveIndex = (AutoSaveIndex + 1) % AutoSaveMaxBackups;

			EAutosaveContentPackagesResult::Type MapsSaveResults = EAutosaveContentPackagesResult::NothingToDo;
			EAutosaveContentPackagesResult::Type AssetsSaveResults = EAutosaveContentPackagesResult::NothingToDo;

			if (LoadingSavingSettings->AutoSaveMethod == EAutoSaveMethod::BackupAndRestore)
			{
				TGuardValue<EPackageAutoSaveType> IsAutoSavingScope(CurrentAutoSaveType, EPackageAutoSaveType::Transient);

				SlowTask.EnterProgressFrame(50);

				if (LoadingSavingSettings->bAutoSaveMaps)
				{
					MapsSaveResults = FEditorFileUtils::AutosaveMapEx(AutoSaveDir, NewAutoSaveIndex, false, DirtyMapsForAutoSave);
					if (MapsSaveResults == EAutosaveContentPackagesResult::Success)
					{
						DirtyMapsForAutoSave.Empty();
					}
				}

				SlowTask.EnterProgressFrame(50);

				if (LoadingSavingSettings->bAutoSaveContent)
				{
					AssetsSaveResults = FEditorFileUtils::AutosaveContentPackagesEx(AutoSaveDir, NewAutoSaveIndex, false, DirtyContentForAutoSave);
					if (AssetsSaveResults == EAutosaveContentPackagesResult::Success)
					{
						DirtyContentForAutoSave.Empty();
					}
				}
			}
			else if (LoadingSavingSettings->AutoSaveMethod == EAutoSaveMethod::BackupAndOverwrite)
			{
				// Make a backup copy of any packages we may be about to overwrite
				{
					auto BackupExistingPackages = [&AutoSaveDir, NewAutoSaveIndex](const TSet<TWeakObjectPtr<UPackage>, TWeakObjectPtrSetKeyFuncs<TWeakObjectPtr<UPackage>>>& PackagesToBackup)
					{
						FString PackageFilename;
						for (const TWeakObjectPtr<UPackage>& PackageToBackup : PackagesToBackup)
						{
							if (UPackage* Pkg = PackageToBackup.Get())
							{
								if (FPackageName::DoesPackageExist(PackageToBackup->GetPathName(), &PackageFilename))
								{
									const FString PackageAutoSaveFilename = FEditorFileUtils::GetAutoSaveFilename(Pkg, AutoSaveDir, NewAutoSaveIndex, FPaths::GetExtension(PackageFilename, /*bIncludeDot*/true));
									IFileManager::Get().Copy(*PackageAutoSaveFilename, *PackageFilename, /*bReplace*/true);
								}
							}
						}
					};
				
					if (LoadingSavingSettings->bAutoSaveMaps)
					{
						BackupExistingPackages(DirtyMapsForAutoSave);
					}
					if (LoadingSavingSettings->bAutoSaveContent)
					{
						BackupExistingPackages(DirtyContentForAutoSave);
					}
				}

				// If possible, auto-save will use an uncontrolled changelist rather than actually checkout the files
				FUncontrolledChangelistsModule& UncontrolledChangelistsModule = FUncontrolledChangelistsModule::Get();
				const bool bUseUncontrolledChangelist = PackageAutoSaver::CanUseUncontrolledChangelist();

				// Build the complete list of packages to save
				TArray<UPackage*> PackagesToSave;
				{
					auto AppendPackagesToSave = [&PackagesToSave, &UncontrolledChangelistsModule, bUseUncontrolledChangelist](const TSet<TWeakObjectPtr<UPackage>, TWeakObjectPtrSetKeyFuncs<TWeakObjectPtr<UPackage>>>& PotentialPackagesToSave)
					{
						FNameBuilder PkgName;
						FString PackageFilename;
						for (const TWeakObjectPtr<UPackage>& PotentialPackageToSave : PotentialPackagesToSave)
						{
							if (UPackage* Pkg = PotentialPackageToSave.Get())
							{
								PkgName.Reset();
								Pkg->GetFName().AppendString(PkgName);

								// Skip packages in read-only roots (like /Temp)
								if (FPackageName::IsValidLongPackageName(PkgName.ToView(), /*bIncludeReadOnlyRoots*/false))
								{
									PackagesToSave.Add(Pkg);

									// If using an uncontrolled changelist to track writable files, then make the file writable now so that we can overwrite it during save
									if (bUseUncontrolledChangelist)
									{
										if (FPackageName::DoesPackageExist(Pkg->GetPathName(), &PackageFilename) && FPlatformFileManager::Get().GetPlatformFile().IsReadOnly(*PackageFilename))
										{
											if (FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*PackageFilename, false))
											{
												UncontrolledChangelistsModule.OnMakeWritable(PackageFilename);
											}
										}
									}
								}
							}
						}
					};

					if (LoadingSavingSettings->bAutoSaveMaps)
					{
						AppendPackagesToSave(DirtyMapsForAutoSave);
					}
					if (LoadingSavingSettings->bAutoSaveContent)
					{
						AppendPackagesToSave(DirtyContentForAutoSave);
					}
				}

				if (PackagesToSave.Num() > 0)
				{
					TGuardValue<EPackageAutoSaveType> IsAutoSavingScope(CurrentAutoSaveType, EPackageAutoSaveType::Persistent);

					bool bSuccess = false;
					{
						ISourceControlProvider& Provider = ISourceControlModule::Get().GetProvider();
						FUncontrolledChangelistStatePtr DefaultUncontrolledChangelistState = UncontrolledChangelistsModule.GetDefaultChangelistState();

						TArray<FString> SavedPackageFilenames;
						TSet<FString> PreSaveFilesInDefaultUncontrolledChangelist;

						if (bUseUncontrolledChangelist)
						{
							// Track which files were in the default uncontrolled changelist prior to the auto-save
							if (DefaultUncontrolledChangelistState)
							{
								PreSaveFilesInDefaultUncontrolledChangelist.Append(DefaultUncontrolledChangelistState->GetFilenames());
							}

							// Since we disable source control around UEditorLoadingAndSavingUtils::SavePackages, we need to handle reverting and checking out any source controlled files that are in a deleted state prior to the save
							// Source controlled files won't be managed by the uncontrolled changelist, so we must keep their source control status correct to avoid work being lost
							if (Provider.IsEnabled())
							{
								TSet<FString> EmptyPackageFilenames;
								for (UPackage* Pkg : PackagesToSave)
								{
									// This is processed at the end of the frame, so we can populate it before we run the save
									PackagesToMakeDirtyIfManagedByAutoSave.Add(Pkg->GetFName());

									FString PackageFilename;
									if (FPackageName::TryConvertLongPackageNameToFilename(FNameBuilder(Pkg->GetFName()).ToView(), PackageFilename, Pkg->ContainsMap() ? FPackageName::GetMapPackageExtension() : FPackageName::GetAssetPackageExtension()))
									{
										PackageFilename = FPaths::ConvertRelativePathToFull(MoveTemp(PackageFilename));

										if (UPackage::IsEmptyPackage(Pkg))
										{
											EmptyPackageFilenames.Add(PackageFilename);
										}

										SavedPackageFilenames.Add(MoveTemp(PackageFilename));
									}
								}

								TArray<FSourceControlStateRef> FileStates;
								if (Provider.GetState(SavedPackageFilenames, FileStates, EStateCacheUsage::ForceUpdate) == ECommandResult::Succeeded)
								{
									TArray<FString> FilesToRevertAndCheckout;
									for (const FSourceControlStateRef& FileState : FileStates)
									{
										if (FileState->IsDeleted() && !EmptyPackageFilenames.Contains(FileState->GetFilename()))
										{
											FilesToRevertAndCheckout.Add(FileState->GetFilename());
										}
									}

									if (FilesToRevertAndCheckout.Num() > 0)
									{
										Provider.Execute(ISourceControlOperation::Create<FRevert>(), FilesToRevertAndCheckout);
										Provider.Execute(ISourceControlOperation::Create<FCheckOut>(), FilesToRevertAndCheckout);

										// If any files failed to checkout then make them writable so they go into the uncontrolled changelist instead
										for (const FString& PackageFilename : FilesToRevertAndCheckout)
										{
											if (FPlatformFileManager::Get().GetPlatformFile().IsReadOnly(*PackageFilename))
											{
												if (FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*PackageFilename, false))
												{
													UncontrolledChangelistsModule.OnMakeWritable(PackageFilename);
												}
											}
										}
									}
								}
							}
						}

						// Disable source control around UEditorLoadingAndSavingUtils::SavePackages when using uncontrolled changelists avoid it still checking writable files out
						{
							FScopedDisableSourceControl DisableSourceControl(bUseUncontrolledChangelist);
							bSuccess = UEditorLoadingAndSavingUtils::SavePackages(PackagesToSave, false);
						}

						// Note: We don't check bSuccess here, as a subset of files might still have been saved
						if (bUseUncontrolledChangelist)
						{
							// Since we disable source control around UEditorLoadingAndSavingUtils::SavePackages, we need to handle reverting and deleting any source controlled files that are added or checked out, but no longer available on disk
							// Source controlled files won't be managed by the uncontrolled changelist, so we must keep their source control status correct to avoid work being lost
							if (Provider.IsEnabled())
							{
								TArray<FSourceControlStateRef> FileStates;
								if (Provider.GetState(SavedPackageFilenames, FileStates, EStateCacheUsage::ForceUpdate) == ECommandResult::Succeeded)
								{
									TArray<FString> FilesToRevertAndDelete;
									for (const FSourceControlStateRef& FileState : FileStates)
									{
										if ((FileState->IsAdded() || FileState->IsCheckedOut()) && !FPaths::FileExists(FileState->GetFilename()))
										{
											FilesToRevertAndDelete.Add(FileState->GetFilename());
										}
									}

									if (FilesToRevertAndDelete.Num() > 0)
									{
										TSharedRef<FRevert, ESPMode::ThreadSafe> RevertOperation = ISourceControlOperation::Create<FRevert>();
										RevertOperation->SetSoftRevert(Provider.UsesSoftRevertOnDelete());
										Provider.Execute(RevertOperation, FilesToRevertAndDelete);

										TSharedRef<FDelete, ESPMode::ThreadSafe> DeleteOperation = ISourceControlOperation::Create<FDelete>();
										Provider.Execute(DeleteOperation, FilesToRevertAndDelete);

										// Make sure the source control operations didn't leave any files on disk
										for (const FString& PackageFilename : FilesToRevertAndDelete)
										{
											IFileManager::Get().Delete(*PackageFilename);
										}
									}
								}
							}

							// Move any files that the auto-save added to the default uncontrolled changelist to the auto-save specific uncontrolled changelist
							if (DefaultUncontrolledChangelistState && Provider.IsAvailable())
							{
								TSet<FString> PostSaveFilesInDefaultUncontrolledChangelist = TSet<FString>(DefaultUncontrolledChangelistState->GetFilenames());
								TSet<FString> NewFilesInDefaultUncontrolledChangelist = PostSaveFilesInDefaultUncontrolledChangelist.Difference(PreSaveFilesInDefaultUncontrolledChangelist);

								if (NewFilesInDefaultUncontrolledChangelist.Num() > 0)
								{
									if (UncontrolledChangelistsModule.CreateUncontrolledChangelist(NSLOCTEXT("AutoSave", "UncontrolledChangelistTitle", "Auto Save Uncontrolled Changelist"), PackageAutoSaver::AutoSaveUncontrolledChangelist))
									{
										TArray<FSourceControlStateRef> FileStates;
										if (Provider.GetState(NewFilesInDefaultUncontrolledChangelist.Array(), FileStates, EStateCacheUsage::Use) == ECommandResult::Succeeded)
										{
											UncontrolledChangelistsModule.MoveFilesToUncontrolledChangelist(TArray<FSourceControlStateRef>(), FileStates, PackageAutoSaver::AutoSaveUncontrolledChangelist);
										}
									}
								}
							}
						}
					}

					// Note: We don't update DirtyMapsForAutoSave/DirtyContentForAutoSave manually post-save, as any packages that were actually saved will have been removed from the list via the save callback
					if (bSuccess)
					{
						MapsSaveResults = EAutosaveContentPackagesResult::Success;
						AssetsSaveResults = EAutosaveContentPackagesResult::Success;
					}
					else
					{
						MapsSaveResults = EAutosaveContentPackagesResult::Failure;
						AssetsSaveResults = EAutosaveContentPackagesResult::Failure;
					}
				}
			}
			else
			{
				checkf(false, TEXT("Unknown AutoSaveMethod!"));
			}

			const bool bNothingToDo = (MapsSaveResults == EAutosaveContentPackagesResult::NothingToDo && AssetsSaveResults == EAutosaveContentPackagesResult::NothingToDo);
			const bool bSuccess = (MapsSaveResults != EAutosaveContentPackagesResult::Failure && AssetsSaveResults != EAutosaveContentPackagesResult::Failure && !bNothingToDo);
			const bool bFailure = (MapsSaveResults == EAutosaveContentPackagesResult::Failure || AssetsSaveResults == EAutosaveContentPackagesResult::Failure);

			bDidAutoSaveSuccessfully = bSuccess;

			// Auto-saved, so close any warning notifications.
			CloseAutoSaveNotification(
				bSuccess ?		ECloseNotification::Success		:
				bFailure ?		ECloseNotification::Failed		:
				bNothingToDo ?	ECloseNotification::NothingToDo	:
								ECloseNotification::Postponed);

			if (bSuccess)
			{
				// If a level was actually saved, update the auto-save index
				AutoSaveIndex = NewAutoSaveIndex;

				// Update the restore information
				UpdateRestoreFile(PackageAutoSaverJson::IsRestoreEnabled());
			}

			ResetAutoSaveTimer();
			bDelayingDueToFailedSave = false;
		}
		else
		{
			bDelayingDueToFailedSave = true;

			// Extend the time by 3 seconds if we failed to save because the user was interacting.  
			// We do this to avoid cases where they are rapidly clicking and are interrupted by autosaves
			AutoSaveCount = (LoadingSavingSettings->AutoSaveTimeMinutes*60.0f) - 3.0f;

			TSharedPtr<SNotificationItem> NotificationItem = AutoSaveNotificationPtr.Pin();

			// ensure the notification exists
			if (NotificationItem.IsValid())
			{
				// update notification
				NotificationItem->SetText(NSLOCTEXT("AutoSaveNotify", "WaitingToPerformAutoSave", "Waiting to perform Auto-save..."));
			}
		}
	}


	// The auto save notification must always be ticked, 
	// so as to correctly handle pausing and resetting.
	if (!bAutosaveHandled)
	{
		UpdateAutoSaveNotification();
	}

	return bDidAutoSaveSuccessfully;
}

void FPackageAutoSaver::LoadRestoreFile()
{
	PackagesThatCanBeRestored = PackageAutoSaverJson::LoadRestoreFile();
}

void FPackageAutoSaver::UpdateRestoreFile(const bool bRestoreEnabled)
{
	PackageAutoSaverJson::SaveRestoreFile(bRestoreEnabled, DirtyPackagesForUserSave);
	bNeedRestoreFileUpdate = false;
}

bool FPackageAutoSaver::HasPackagesToRestore() const
{
	// Don't offer to restore packages during automation testing or when unattended; the dlg is modal and blocks
	return !GIsAutomationTesting && !FApp::IsUnattended() && PackagesThatCanBeRestored.Num() > 0;
}

void FPackageAutoSaver::OfferToRestorePackages()
{
	bool bRemoveRestoreFile = true;

	if(HasPackagesToRestore() && !bAutoDeclineRecovery && !FApp::IsUnattended()) // if bAutoDeclineRecovery is true, do like the user selected to decline. (then remove the restore files)
	{
		// If we failed to restore, keep the restore information around
		if(PackageRestore::PromptToRestorePackages(PackagesThatCanBeRestored) == FEditorFileUtils::PR_Failure)
		{
			bRemoveRestoreFile = false;
		}
	}

	if(bRemoveRestoreFile)
	{
		// We've finished restoring, so remove this file to avoid being prompted about it again
		UpdateRestoreFile(false);
	}
}

void FPackageAutoSaver::OnPackagesDeleted(const TArray<UPackage*>& DeletedPackages)
{
	for (UPackage* DeletedPackage : DeletedPackages)
	{
		PackagesPendingUpdate.Remove(DeletedPackage);
		PackagesToIgnoreIfEmpty.Add(DeletedPackage->GetFName());

		// We remove the package immediately as it may not survive to the next tick if queued for update via PackagesPendingUpdate
		DirtyMapsForAutoSave.Remove(DeletedPackage);
		DirtyContentForAutoSave.Remove(DeletedPackage);
		if (DirtyPackagesForUserSave.Remove(DeletedPackage) > 0)
		{
			bNeedRestoreFileUpdate = true;
		}
	}
}

void FPackageAutoSaver::OnPackageDirtyStateUpdated(UPackage* Pkg)
{
	if (!IsAutoSaving(EPackageAutoSaveType::Transient))
	{
		PackagesPendingUpdate.Add(Pkg);
	}
}

void FPackageAutoSaver::OnMarkPackageDirty(UPackage* Pkg, bool bWasDirty)
{
	if (!IsAutoSaving(EPackageAutoSaveType::Transient))
	{
		PackagesPendingUpdate.Add(Pkg);
	}
}

void FPackageAutoSaver::OnPackageSaved(const FString& Filename, UPackage* Pkg, FObjectPostSaveContext ObjectSaveContext)
{
	// If this has come from an auto-save, update the last known filename in the user dirty list so that we can offer is up as a restore file later
	if (IsAutoSaving(EPackageAutoSaveType::Transient))
	{
		if (FString* const AutoSaveFilename = DirtyPackagesForUserSave.Find(Pkg))
		{
			// Make the filename relative to the auto-save directory
			// Note: MakePathRelativeTo modifies in-place, hence the copy of Filename
			const FString AutoSaveDir = AutoSaveUtils::GetAutoSaveDir() / "";
			FString RelativeFilename = Filename;
			FPaths::MakePathRelativeTo(RelativeFilename, *AutoSaveDir);

			(*AutoSaveFilename) = RelativeFilename;
			bNeedRestoreFileUpdate = true;
		}
	}
	else
	{
		// If the package was previously deleted, then it's certainly back after being saved!
		PackagesToIgnoreIfEmpty.Remove(Pkg->GetFName());

		// Remove the saved package from the user-restore list when this was a full save
		if (DirtyPackagesForUserSave.Remove(Pkg) > 0)
		{
			bNeedRestoreFileUpdate = true;
		}
	}

	// Always remove a saved package from the auto-save lists
	DirtyMapsForAutoSave.Remove(Pkg);
	DirtyContentForAutoSave.Remove(Pkg);

	// Discard any pending update since the save has already handled it
	PackagesPendingUpdate.Remove(Pkg);
}

void FPackageAutoSaver::OnUndoRedo()
{
	bSyncWithDirtyPackageList = true;
}

void FPackageAutoSaver::OnUncontrolledChangelistChanged()
{
	// Any loaded packages in the auto-save changelist should be dirty so that they can be considered for an interactive save
	if (const TMap<FName, FString> UncontrolledPackages = PackageAutoSaver::GetPackagesInUncontrolledChangelist();
		UncontrolledPackages.Num() > 0)
	{
		for (const TTuple<FName, FString>& UncontrolledPackagePair : UncontrolledPackages)
		{
			PackagesToMakeDirtyIfManagedByAutoSave.Add(UncontrolledPackagePair.Key);
		}
	}
}

void FPackageAutoSaver::OnPackageLoaded(const FEndLoadPackageContext& Context)
{
	// Packages in the auto-save uncontrolled changelist should become dirty post-load so that they can be considered for an interactive save
	for (UPackage* Pkg : Context.LoadedPackages)
	{
		PackagesToMakeDirtyIfManagedByAutoSave.Add(Pkg->GetFName());
	}
}

void FPackageAutoSaver::OnEndFrame()
{
	if (PackagesToMakeDirtyIfManagedByAutoSave.Num() > 0 && PackageAutoSaver::ShouldInteractiveSaveManageUncontrolledChangelist())
	{
		if (const TMap<FName, FString> UncontrolledPackages = PackageAutoSaver::GetPackagesInUncontrolledChangelist();
			UncontrolledPackages.Num() > 0)
		{
			auto FindAndDirtyPackage = [this](const FName PackageName)
			{
				if (UPackage* Pkg = FindPackage(nullptr, FNameBuilder(PackageName).ToString());
					IsValid(Pkg) && !Pkg->IsDirty())
				{
					TGuardValue<EPackageAutoSaveType> IsAutoSavingScope(CurrentAutoSaveType, EPackageAutoSaveType::Transient); // Set EPackageAutoSaveType::Transient so this doesn't get re-added to the auto-save queue
					Pkg->MarkPackageDirty();
					UE_LOGF(LogPackageAutoSaver, Log, "Package '%ls' was marked dirty by the auto-save uncontrolled changelist", FNameBuilder(Pkg->GetFName()).ToString());
				}
			};

			// Iterate whichever is smaller
			if (PackagesToMakeDirtyIfManagedByAutoSave.Num() <= UncontrolledPackages.Num())
			{
				for (const FName PackageName : PackagesToMakeDirtyIfManagedByAutoSave)
				{
					if (UncontrolledPackages.Contains(PackageName))
					{
						FindAndDirtyPackage(PackageName);
					}
				}
			}
			else
			{
				for (const TTuple<FName, FString>& UncontrolledPackagePair : UncontrolledPackages)
				{
					if (PackagesToMakeDirtyIfManagedByAutoSave.Contains(UncontrolledPackagePair.Key))
					{
						FindAndDirtyPackage(UncontrolledPackagePair.Key);
					}
				}
			}
		}
	}

	PackagesToMakeDirtyIfManagedByAutoSave.Reset();
}

void FPackageAutoSaver::GetAdditionalInteractiveSavePackageCandidates(TSet<UPackage*>& OutPackages)
{
	if (!PackageAutoSaver::ShouldInteractiveSaveManageUncontrolledChangelist())
	{
		return;
	}

	// Strongly hint that any loaded packages in the auto-save changelist should be considered for an interactive save
	if (const TMap<FName, FString> UncontrolledPackages = PackageAutoSaver::GetPackagesInUncontrolledChangelist(/*bUpdateStatus*/true);
		UncontrolledPackages.Num() > 0)
	{
		for (const TTuple<FName, FString>& UncontrolledPackagePair : UncontrolledPackages)
		{
			if (UPackage* Pkg = FindPackage(nullptr, FNameBuilder(UncontrolledPackagePair.Key).ToString());
				IsValid(Pkg))
			{
				OutPackages.Add(Pkg);
			}
		}
	}
}

void FPackageAutoSaver::OnPreInteractiveCheckoutPackages(const TArray<UPackage*>& Packages, TSet<FName>& OutReadOnlyPackages)
{
	if (!PackageAutoSaver::ShouldInteractiveSaveManageUncontrolledChangelist())
	{
		return;
	}

	// Report that packages in the auto-save uncontrolled changelist are read-only so that PromptToCheckoutPackagesInternal will offer to check them out or make them writable
	if (const TMap<FName, FString> UncontrolledPackages = PackageAutoSaver::GetPackagesInUncontrolledChangelist(/*bUpdateStatus*/true);
		UncontrolledPackages.Num() > 0)
	{
		for (UPackage* Pkg : Packages)
		{
			if (UncontrolledPackages.Contains(Pkg->GetFName()))
			{
				OutReadOnlyPackages.Add(Pkg->GetFName());
			}
		}
	}
}

void FPackageAutoSaver::OnPostInteractiveCheckoutPackages(const TArray<UPackage*>& Packages, bool bUserResponse)
{
	if (!bUserResponse || !PackageAutoSaver::ShouldInteractiveSaveManageUncontrolledChangelist())
	{
		return;
	}

	// Handle any source control adds, as newly added files don't appear in the checkout list since adds usually happen automatically on save
	TArray<FString> FilesToPotentiallyAdd;
	if (const TMap<FName, FString> UncontrolledPackages = PackageAutoSaver::GetPackagesInUncontrolledChangelist();
		UncontrolledPackages.Num() > 0)
	{
		for (UPackage* Pkg : Packages)
		{
			if (const FString* PackageFilename = UncontrolledPackages.Find(Pkg->GetFName()))
			{
				if (!UPackage::IsEmptyPackage(Pkg) && FPaths::FileExists(*PackageFilename))
				{
					FilesToPotentiallyAdd.Add(*PackageFilename);
				}
			}
		}
	}

	if (FilesToPotentiallyAdd.Num() > 0)
	{
		FScopedSlowTask SlowTask(100.0f, NSLOCTEXT("AutoSave", "AddingAutoSavedChanges", "Adding auto-saved changes..."));
		SlowTask.MakeDialog();
		SlowTask.EnterProgressFrame(25.0f);

		ISourceControlProvider& Provider = ISourceControlModule::Get().GetProvider();

		TArray<FSourceControlStateRef> FileStates;
		if (Provider.GetState(FilesToPotentiallyAdd, FileStates, EStateCacheUsage::ForceUpdate) == ECommandResult::Succeeded)
		{
			TArray<FString> FilesToAdd;
			TArray<FSourceControlStateRef> FileStatesToAdd;
			for (const FSourceControlStateRef& FileState : FileStates)
			{
				if (!FileState->IsSourceControlled() && !FileState->IsAdded())
				{
					FilesToAdd.Add(FileState->GetFilename());
					FileStatesToAdd.Add(FileState);
				}
			}
			if (FilesToAdd.Num() > 0)
			{
				SlowTask.EnterProgressFrame(25.0f);

				if (Provider.Execute(ISourceControlOperation::Create<FMarkForAdd>(), FilesToAdd) == ECommandResult::Succeeded)
				{
					FUncontrolledChangelistStatePtr AutoSaveUncontrolledChangelistState = FUncontrolledChangelistsModule::Get().GetChangelistState(PackageAutoSaver::AutoSaveUncontrolledChangelist);
					check(AutoSaveUncontrolledChangelistState);

					if (AutoSaveUncontrolledChangelistState->RemoveFiles(FileStatesToAdd))
					{
						FUncontrolledChangelistsModule::Get().HandleChangelistStateModified();
					}
				}
			}
		}
	}
}

void FPackageAutoSaver::OnPackagesInteractivelyCheckedOut(const TArray<UPackage*>& Packages)
{
	if (!PackageAutoSaver::ShouldInteractiveSaveManageUncontrolledChangelist())
	{
		return;
	}

	// Remove any now checked out packages from the auto-save uncontrolled changelist
	TArray<FString> FilesToRemove;
	if (const TMap<FName, FString> UncontrolledPackages = PackageAutoSaver::GetPackagesInUncontrolledChangelist();
		UncontrolledPackages.Num() > 0)
	{
		for (UPackage* Pkg : Packages)
		{
			if (const FString* PackageFilename = UncontrolledPackages.Find(Pkg->GetFName()))
			{
				FilesToRemove.Add(*PackageFilename);
			}
		}
	}

	if (FilesToRemove.Num() > 0)
	{
		TArray<FSourceControlStateRef> FileStates;
		if (ISourceControlModule::Get().GetProvider().GetState(FilesToRemove, FileStates, EStateCacheUsage::Use) == ECommandResult::Succeeded)
		{
			FUncontrolledChangelistStatePtr AutoSaveUncontrolledChangelistState = FUncontrolledChangelistsModule::Get().GetChangelistState(PackageAutoSaver::AutoSaveUncontrolledChangelist);
			check(AutoSaveUncontrolledChangelistState);

			if (AutoSaveUncontrolledChangelistState->RemoveFiles(FileStates))
			{
				FUncontrolledChangelistsModule::Get().HandleChangelistStateModified();
			}
		}
	}
}

void FPackageAutoSaver::OnPackagesInteractivelyMadeWritable(const TArray<UPackage*>& Packages)
{
	if (!PackageAutoSaver::ShouldInteractiveSaveManageUncontrolledChangelist(/*bRequiresSourceControl*/false))
	{
		return;
	}

	// Move any packages the user made writable to the default uncontrolled changelist
	TArray<FString> FilesToMove;
	if (const TMap<FName, FString> UncontrolledPackages = PackageAutoSaver::GetPackagesInUncontrolledChangelist();
		UncontrolledPackages.Num() > 0)
	{
		for (UPackage* Pkg : Packages)
		{
			if (const FString* PackageFilename = UncontrolledPackages.Find(Pkg->GetFName()))
			{
				FilesToMove.Add(*PackageFilename);
			}
		}
	}

	if (FilesToMove.Num() > 0)
	{
		TArray<FSourceControlStateRef> FileStates;
		if (ISourceControlModule::Get().GetProvider().GetState(FilesToMove, FileStates, EStateCacheUsage::Use) == ECommandResult::Succeeded)
		{
			FUncontrolledChangelistsModule::Get().MoveFilesToUncontrolledChangelist(TArray<FSourceControlStateRef>(), FileStates, FUncontrolledChangelist(FUncontrolledChangelist::DEFAULT_UNCONTROLLED_CHANGELIST_GUID));
		}
	}
}

void FPackageAutoSaver::OnPackagesInteractivelyDiscarded(const TArray<UPackage*>& Packages)
{
	if (!PackageAutoSaver::ShouldInteractiveSaveManageUncontrolledChangelist())
	{
		return;
	}

	// Revert any auto-save uncontrolled changes as the user has asked to discard saving them to disk
	{
		FScopedSlowTask SlowTask(100.0f, NSLOCTEXT("AutoSave", "DiscardingAutoSavedChanges", "Discarding auto-saved changes..."));
		SlowTask.MakeDialog();

		TArray<FString> FilesToDiscard;
		if (const TMap<FName, FString> UncontrolledPackages = PackageAutoSaver::GetPackagesInUncontrolledChangelist(/*bUpdateStatus*/true);
			UncontrolledPackages.Num() > 0)
		{
			auto DetachLinker = [](UPackage* Package)
			{
				if (!Package->IsFullyLoaded())
				{
					FlushAsyncLoading();
					Package->FullyLoad();
				}
				ResetLoaders(Package);
			};

			for (UPackage* Pkg : Packages)
			{
				if (const FString* PackageFilename = UncontrolledPackages.Find(Pkg->GetFName()))
				{
					DetachLinker(Pkg);
					FilesToDiscard.Add(*PackageFilename);
				}
			}
		}

		if (FilesToDiscard.Num() > 0)
		{
			SlowTask.EnterProgressFrame(50.0f);

			ISourceControlProvider& Provider = ISourceControlModule::Get().GetProvider();

			TArray<FString> FilesToDelete;
			TArray<FString> FilesToRevert;
			TArray<FSourceControlStateRef> FileStates;
			if (Provider.GetState(FilesToDiscard, FileStates, EStateCacheUsage::ForceUpdate) == ECommandResult::Succeeded)
			{
				for (const FSourceControlStateRef& FileState : FileStates)
				{
					if (FileState->IsSourceControlled())
					{
						FilesToRevert.Add(FileState->GetFilename());
					}
					else
					{
						FilesToDelete.Add(FileState->GetFilename());
					}
				}
			}

			SlowTask.EnterProgressFrame(25.0f);

			bool bUpdatedChangelistState = false;

			if (FilesToRevert.Num() > 0)
			{
				TSharedRef<FSync> ForceSyncOperation = ISourceControlOperation::Create<FSync>();
				ForceSyncOperation->SetForce(true);
				ForceSyncOperation->SetLastSyncedFlag(true);

				bUpdatedChangelistState |= Provider.Execute(ForceSyncOperation, FilesToRevert) == ECommandResult::Succeeded;
			}

			if (FilesToDelete.Num() > 0)
			{
				IFileManager& FileManager = IFileManager::Get();

				for (const FString& FileToDelete : FilesToDelete)
				{
					const bool bRequireExists = true;
					const bool bEvenReadOnly = false;
					const bool bQuiet = false;

					bUpdatedChangelistState |= FileManager.Delete(*FileToDelete, bRequireExists, bEvenReadOnly, bQuiet);
				}

				ISourceControlModule::Get().GetOnFilesDeleted().Broadcast(FilesToDelete);
			}

			if (bUpdatedChangelistState)
			{
				FUncontrolledChangelistStatePtr AutoSaveUncontrolledChangelistState = FUncontrolledChangelistsModule::Get().GetChangelistState(PackageAutoSaver::AutoSaveUncontrolledChangelist);
				check(AutoSaveUncontrolledChangelistState);

				if (AutoSaveUncontrolledChangelistState->UpdateStatus())
				{
					FUncontrolledChangelistsModule::Get().HandleChangelistStateModified();
				}
			}
		}
	}
}

void FPackageAutoSaver::UpdateDirtyListsForPackage(UPackage* Pkg)
{
	const UPackage* TransientPackage = GetTransientPackage();

	// Don't auto-save the transient package or packages with the transient flag.
	if ( Pkg == TransientPackage || Pkg->HasAnyFlags(RF_Transient) || Pkg->HasAnyPackageFlags(PKG_CompiledIn) )
	{
		return;
	}

	// Should this package be ignored because it was previously deleted and is still empty?
	if (PackagesToIgnoreIfEmpty.Contains(Pkg->GetFName()) && UPackage::IsEmptyPackage(Pkg))
	{
		return;
	}

	if ( Pkg->IsDirty() )
	{
		// Add the package to the user-restore list
		if (!DirtyPackagesForUserSave.Contains(Pkg))
		{
			DirtyPackagesForUserSave.Add(Pkg);
			bNeedRestoreFileUpdate = true;
		}

		// Add package into the appropriate list (map or content)
		if (UWorld::IsWorldOrWorldExternalPackage(Pkg))
		{
			DirtyMapsForAutoSave.Add(Pkg);
		}
		else
		{
			DirtyContentForAutoSave.Add(Pkg);
		}
	}
	else
	{
		// Always remove a clean package from the auto-save and user-restore lists
		DirtyMapsForAutoSave.Remove(Pkg);
		DirtyContentForAutoSave.Remove(Pkg);
		if (DirtyPackagesForUserSave.Remove(Pkg) > 0)
		{
			bNeedRestoreFileUpdate = true;
		}
	}
}

bool FPackageAutoSaver::CanAutoSave(const bool bForceAutoSave) const
{
	// Don't allow auto-saving if the auto-save wouldn't save anything
	const bool bPackagesNeedAutoSave = DoPackagesNeedAutoSave();

	const UEditorLoadingSavingSettings* LoadingSavingSettings = GetDefault<UEditorLoadingSavingSettings>();
	const bool bAutosaveEnabled = LoadingSavingSettings->bAutoSaveEnable && bPackagesNeedAutoSave;

	if (!bAutosaveEnabled)
	{
		return false;
	}

	if (bForceAutoSave)
	{
		return true;
	}

	const double LastInteractionTime = FSlateApplication::Get().GetLastUserInteractionTime();
	const float InteractionDelay = float(LoadingSavingSettings->AutoSaveInteractionDelayInSeconds);

	const bool bDidInteractRecently = (FApp::GetCurrentTime() - LastInteractionTime) < InteractionDelay;
	const bool bSlowTask = GIsSlowTask;
	const bool bPlayWorldValid = GUnrealEd->PlayWorld != nullptr;
	const bool bAnyMenusVisible	= FSlateApplication::Get().AnyMenusVisible();
	const bool bAutomationTesting = GIsAutomationTesting;
	const bool bIsInteracting = FSlateApplication::Get().HasAnyMouseCaptor() || FSlateApplication::Get().IsDragDropping() || GUnrealEd->IsUserInteracting() || (bDidInteractRecently && !bAutoSaveNotificationLaunched && !bDelayingDueToFailedSave);
	const bool bHasGameOrProjectLoaded = FApp::HasProjectName();
	const bool bAreShadersCompiling = GShaderCompilingManager->IsCompiling();
	const bool bAreAssetsCompiling = FAssetCompilingManager::Get().GetNumRemainingAssets() > 0;
	const bool bIsVREditorActive = IVREditorModule::Get().IsVREditorEnabled();	// @todo vreditor: Eventually we should support this while in VR (modal VR progress, with sufficient early warning)
	const bool bIsInterchangeActive = UInterchangeManager::GetInterchangeManager().IsInterchangeActive();

	bool bIsSequencerPlaying = false;
	for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		if (LevelVC && LevelVC->AllowsCinematicControl() && LevelVC->ViewState.GetReference()->GetSequencerState() == ESS_Playing)
		{
			bIsSequencerPlaying = true;
			break;
		}
	}

	// query any active editor modes and allow them to prevent autosave
	const bool bActiveModesAllowAutoSave = GLevelEditorModeTools().CanAutoSave();

	return !bSlowTask
		&& !bPlayWorldValid
		&& !bAnyMenusVisible
		&& !bAutomationTesting
		&& !bIsInteracting
		&& !GIsDemoMode
		&& bHasGameOrProjectLoaded
		&& !bAreShadersCompiling
		&& !bAreAssetsCompiling
		&& !bIsVREditorActive
		&& !bIsSequencerPlaying
		&& !bIsInterchangeActive
		&& bActiveModesAllowAutoSave;
}

bool FPackageAutoSaver::DoPackagesNeedAutoSave() const
{
	const UEditorLoadingSavingSettings* LoadingSavingSettings = GetDefault<UEditorLoadingSavingSettings>();

	const bool bHasDirtyMapsForAutoSave = DirtyMapsForAutoSave.Num() != 0;
	const bool bHasDirtyContentForAutoSave = DirtyContentForAutoSave.Num() != 0;
	const bool bWorldsMightBeDirty = LoadingSavingSettings->bAutoSaveMaps && bHasDirtyMapsForAutoSave;
	const bool bContentPackagesMightBeDirty = LoadingSavingSettings->bAutoSaveContent && bHasDirtyContentForAutoSave;
	const bool bPackagesNeedAutoSave = bWorldsMightBeDirty || bContentPackagesMightBeDirty;

	return bPackagesNeedAutoSave;
}

FText FPackageAutoSaver::GetAutoSaveNotificationText(const int32 TimeInSecondsUntilAutosave)
{
	// Don't switch to pending text unless auto-save really is overdue
	if(!bDelayingDueToFailedSave && TimeInSecondsUntilAutosave > -1)
	{
		const UEditorLoadingSavingSettings* LoadingSavingSettings = GetDefault<UEditorLoadingSavingSettings>();
		int32 NumPackagesToAutoSave = 0;
		if (DirtyMapsForAutoSave.Num() != 0 && LoadingSavingSettings->bAutoSaveMaps)
		{
			NumPackagesToAutoSave += DirtyMapsForAutoSave.Num();
		}
		if (DirtyContentForAutoSave.Num() != 0 && LoadingSavingSettings->bAutoSaveContent)
		{
			NumPackagesToAutoSave += DirtyContentForAutoSave.Num();
		}

		// Count down the time
		FFormatNamedArguments Args;
		Args.Add(TEXT("TimeInSecondsUntilAutosave"), TimeInSecondsUntilAutosave);
		Args.Add(TEXT("DirtyPackagesCount"), NumPackagesToAutoSave);
		return (NumPackagesToAutoSave == 1)
			? FText::Format(NSLOCTEXT("AutoSaveNotify", "AutoSaveIn", "Autosave in {TimeInSecondsUntilAutosave} seconds"), Args)
			: FText::Format(NSLOCTEXT("AutoSaveNotify", "AutoSaveXPackagesIn", "Autosave in {TimeInSecondsUntilAutosave} seconds for {DirtyPackagesCount} items"), Args);
	}

	// Auto-save is imminent 
	return NSLOCTEXT("AutoSaveNotify", "AutoSavePending", "Autosave pending");
}

int32 FPackageAutoSaver::GetTimeTillAutoSave(const bool bIgnoreCanAutoSave) const
{
	int32 Result = -1;
	if (bIgnoreCanAutoSave || CanAutoSave())
	{
		Result = FMath::CeilToInt(GetDefault<UEditorLoadingSavingSettings>()->AutoSaveTimeMinutes * 60.0f - AutoSaveCount);
	}

	return Result;
}

void FPackageAutoSaver::UpdateAutoSaveNotification()
{
	const UEditorLoadingSavingSettings* LoadingSavingSettings = GetDefault<UEditorLoadingSavingSettings>();

	const bool bIgnoreCanAutoSave = true;
	const int32 TimeInSecondsUntilAutosave = GetTimeTillAutoSave(bIgnoreCanAutoSave);

	const bool UserAllowsAutosave = LoadingSavingSettings->bAutoSaveEnable && !GIsDemoMode;
	const bool InGame = (GUnrealEd->PlayWorld != nullptr);

	if (UserAllowsAutosave && // The user has set to allow auto-save in preferences
		TimeInSecondsUntilAutosave < LoadingSavingSettings->AutoSaveWarningInSeconds && 
		!InGame // we want to hide auto-save if we are simulating/playing
		)
	{		
		if (!bAutoSaveNotificationLaunched && !bDelayingDueToFailedSave)
		{
			if (CanAutoSave())
			{
				ClearStalePointers();

				// Starting a new request! Notify the UI.
				if (AutoSaveNotificationPtr.IsValid())
				{
					AutoSaveNotificationPtr.Pin()->ExpireAndFadeout();
				}

				// Setup button localized strings
				static FText AutoSaveCancelButtonText			= NSLOCTEXT("AutoSaveNotify", "AutoSaveCancel", "Cancel");
				static FText AutoSaveCancelButtonToolTipText	= NSLOCTEXT("AutoSaveNotify", "AutoSaveCancelToolTip", "Postpone Autosave");
				static FText AutoSaveSaveButtonText				= NSLOCTEXT("AutoSaveNotify", "AutoSaveSave", "Save Now");
				static FText AutoSaveSaveButtonToolTipText		= NSLOCTEXT("AutoSaveNotify", "AutoSaveSaveToolTip", "Force Autosave");

				FNotificationInfo Info( GetAutoSaveNotificationText(TimeInSecondsUntilAutosave) );
				Info.Image = FAppStyle::Get().GetBrush("Icons.Save");

				// Add the buttons with text, tooltip and callback
				Info.ButtonDetails.Add(FNotificationButtonInfo(AutoSaveCancelButtonText, AutoSaveCancelButtonToolTipText, FSimpleDelegate::CreateRaw(this, &FPackageAutoSaver::OnAutoSaveCancel)));
				Info.ButtonDetails.Add(FNotificationButtonInfo(AutoSaveSaveButtonText, AutoSaveSaveButtonToolTipText, FSimpleDelegate::CreateRaw(this, &FPackageAutoSaver::OnAutoSaveSave)));

				// Force the width so that any text changes don't resize the notification
				Info.WidthOverride = 240.0f;

				// We will be keeping track of this ourselves
				Info.bFireAndForget = false;

				// We want the auto-save to be subtle
				Info.bUseLargeFont = false;
				Info.bUseThrobber = false;
				Info.bUseSuccessFailIcons = false;

				// Launch notification
				AutoSaveNotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);

				if(AutoSaveNotificationPtr.IsValid())
				{
					AutoSaveNotificationPtr.Pin()->SetCompletionState(SNotificationItem::CS_Pending);
				}

				// Update launched flag
				bAutoSaveNotificationLaunched = true;
			}
			else // defer until the user finishes using pop-up menus or the notification will dismiss them...
			{
				ForceMinimumTimeTillAutoSave(static_cast<float>(LoadingSavingSettings->AutoSaveWarningInSeconds));
			}
		}
		else
		{
			// Update the remaining time on the notification
			TSharedPtr<SNotificationItem> NotificationItem = AutoSaveNotificationPtr.Pin();
			if (NotificationItem.IsValid())
			{
				// update text
				NotificationItem->SetText(GetAutoSaveNotificationText(TimeInSecondsUntilAutosave));
			}
		}
	}
	else
	{
		// Ensures notifications are cleaned up
		CloseAutoSaveNotification(false);
	}
}

void FPackageAutoSaver::CloseAutoSaveNotification(const bool Success)
{
	CloseAutoSaveNotification(Success ? ECloseNotification::Success : ECloseNotification::Postponed);
}

void FPackageAutoSaver::CloseAutoSaveNotification(ECloseNotification::Type Type)
{
	// If a notification is open close it
	if(bAutoSaveNotificationLaunched)
	{
		TSharedPtr<SNotificationItem> NotificationItem = AutoSaveNotificationPtr.Pin();

		// ensure the notification exists
		if(NotificationItem.IsValid())
		{
			SNotificationItem::ECompletionState CloseState;
			FText CloseMessage;

			// Set the test on the notification based on whether it was a successful launch
			if (Type == ECloseNotification::Success)
			{
				CloseMessage = NSLOCTEXT("AutoSaveNotify", "AutoSaving", "Saving");
				CloseState = SNotificationItem::CS_Success;
			}
			else if (Type == ECloseNotification::Postponed)
			{
				CloseMessage = NSLOCTEXT("AutoSaveNotify", "AutoSavePostponed", "Autosave postponed");
				CloseState = SNotificationItem::CS_None; // Set back to none rather than failed, as this is too harsh
			}
			else if (Type == ECloseNotification::Failed)
			{
				CloseMessage = NSLOCTEXT("AutoSaveNotify", "AutoSaveFailed", "Auto-save failed. Please check the log for the details.");
				CloseState = SNotificationItem::CS_Fail;
			}
			else
			{
				CloseMessage = NSLOCTEXT("AutoSaveNotify", "AutoSaveNothingToDo", "Already auto-saved.");
				CloseState = SNotificationItem::CS_None;
			}

			// update notification
			NotificationItem->SetText(CloseMessage);
			NotificationItem->SetCompletionState(CloseState);
			NotificationItem->ExpireAndFadeout();

			// clear reference
			AutoSaveNotificationPtr.Reset();
		}

		// Auto-save has been closed
		bAutoSaveNotificationLaunched = false;
	}
}

void FPackageAutoSaver::OnAutoSaveSave()
{
	ForceAutoSaveTimer();
	CloseAutoSaveNotification(true);
}

void FPackageAutoSaver::OnAutoSaveCancel()
{
	ResetAutoSaveTimer();
	CloseAutoSaveNotification(false);
}

void FPackageAutoSaver::ClearStalePointers()
{
	for(auto It = DirtyPackagesForUserSave.CreateIterator(); It; ++It)
	{
		const TWeakObjectPtr<UPackage>& Package = It->Key;
		if(!Package.IsValid())
		{
			bNeedRestoreFileUpdate = true;
			It.RemoveCurrent();
		}
	}

	for(auto It = DirtyMapsForAutoSave.CreateIterator(); It; ++It)
	{
		const TWeakObjectPtr<UPackage>& Package = *It;
		if(!Package.IsValid())
		{
			It.RemoveCurrent();
		}
	}

	for (auto It = DirtyContentForAutoSave.CreateIterator(); It; ++It)
	{
		const TWeakObjectPtr<UPackage>& Package = *It;
		if (!Package.IsValid())
		{
			It.RemoveCurrent();
		}
	}
}


/************************************************************************/
/* PackageAutoSaverJson                                                 */
/************************************************************************/

FString PackageAutoSaverJson::GetRestoreFilename(const bool bEnsurePath)
{
	const FString AutoSaveDir = AutoSaveUtils::GetAutoSaveDir();
	if(bEnsurePath)
	{
		// Make sure the auto-save directory exists before attempting to write the file
		IFileManager::Get().MakeDirectory(*AutoSaveDir, true);
	}

	const FString Filename = AutoSaveDir / RestoreFilename;
	return Filename;
}

TMap<FString, TPair<FString, FString>> PackageAutoSaverJson::LoadRestoreFile()
{
	TMap<FString, TPair<FString, FString>> PackagesThatCanBeRestored;

	const FString Filename = GetRestoreFilename(false);
	FArchive* const FileAr = IFileManager::Get().CreateFileReader(*Filename);
	if(!FileAr)
	{
		// File doesn't exist; nothing to restore
		return PackagesThatCanBeRestored;
	}

	bool bJsonLoaded = false;
	TSharedPtr<FJsonObject> RootObject = MakeShareable(new FJsonObject);
	{
		TSharedRef<FJsonReader> Reader = FJsonReaderFactory::Create(FileAr);
		bJsonLoaded = FJsonSerializer::Deserialize(Reader, RootObject);		
		FileAr->Close();
	}

	if(!bJsonLoaded || !RootObject->GetBoolField(TagRestoreEnabled))
	{
		// File failed to load, or the restore is disabled; nothing to restore
		return PackagesThatCanBeRestored;
	}

	TArray< TSharedPtr<FJsonValue> > PackagesThatCanBeRestoredArray = RootObject->GetArrayField(TagPackages);
	for(auto It = PackagesThatCanBeRestoredArray.CreateConstIterator(); It; ++It)
	{
		TSharedPtr<FJsonObject> EntryObject = (*It)->AsObject();

		const FString PackagePathName = EntryObject->GetStringField(TagPackagePathName);
		const FString AutoSavePath = EntryObject->GetStringField(TagAutoSavePath);

		FString PackageAssetName;
		EntryObject->TryGetStringField(TagPackageAssetName, PackageAssetName);

		PackagesThatCanBeRestored.Add(PackagePathName, TPair<FString, FString>(PackageAssetName, AutoSavePath));
	}

	return PackagesThatCanBeRestored;
}

void PackageAutoSaverJson::SaveRestoreFile(const bool bRestoreEnabled, const TMap< TWeakObjectPtr<UPackage>, FString, FDefaultSetAllocator, TWeakObjectPtrMapKeyFuncs<TWeakObjectPtr<UPackage>, FString> >& DirtyPackages)
{
	TSharedPtr<FJsonObject> RootObject = MakeShareable(new FJsonObject);

	RootObject->SetBoolField(TagRestoreEnabled, bRestoreEnabled);

	TArray< TSharedPtr<FJsonValue> > PackagesThatCanBeRestored;

	// Only bother populating the list of packages if the restore is enabled
	if(bRestoreEnabled)
	{
		PackagesThatCanBeRestored.Reserve(DirtyPackages.Num());

		// Build up the array of package names with auto-saves that can be restored
		for(auto It = DirtyPackages.CreateConstIterator(); It; ++It)
		{
			const TWeakObjectPtr<UPackage>& Package = It.Key();
			const FString& AutoSavePath = It.Value();

			UPackage* const PackagePtr = Package.Get();
			if(PackagePtr && !AutoSavePath.IsEmpty())
			{
				const FString& PackagePathName = PackagePtr->GetPathName();
				TSharedPtr<FJsonObject> EntryObject = MakeShareable(new FJsonObject);
				EntryObject->SetStringField(TagPackagePathName, PackagePathName);

				if (UObject* PackageAsset = PackagePtr->FindAssetInPackage())
				{
					if (UAssetDefinitionRegistry* AssetDefinitionRegistry = UAssetDefinitionRegistry::Get())
					{
						const UAssetDefinition* AssetDefinition = AssetDefinitionRegistry->GetAssetDefinitionForClass(PackageAsset->GetClass());
						if (AssetDefinition)
						{
							const FString PackagAssetName = AssetDefinition->GetObjectDisplayNameText(PackageAsset).ToString();
							EntryObject->SetStringField(TagPackageAssetName, PackagAssetName);
						}
					}					
				}				
				
				EntryObject->SetStringField(TagAutoSavePath, AutoSavePath);

				TSharedPtr<FJsonValue> EntryValue = MakeShareable(new FJsonValueObject(EntryObject));
				PackagesThatCanBeRestored.Add(EntryValue);
			}
		}
	}

	RootObject->SetArrayField(TagPackages, PackagesThatCanBeRestored);

	const FString Filename = GetRestoreFilename(true);
	FArchive* const FileAr = IFileManager::Get().CreateFileWriter(*Filename, FILEWRITE_EvenIfReadOnly);
	if(FileAr)
	{
		TSharedRef<FStringWriter> Writer = FStringWriterFactory::Create(FileAr);
		FJsonSerializer::Serialize(RootObject.ToSharedRef(), Writer);		
		FileAr->Close();
	}
}

FPackageAutoSaverScopeGuard::FPackageAutoSaverScopeGuard()
{
	if (GUnrealEd)
	{
		GUnrealEd->GetPackageAutoSaver().SuspendAutoSave();
		bSuspended = true;
	}
}

FPackageAutoSaverScopeGuard::FPackageAutoSaverScopeGuard(FPackageAutoSaverScopeGuard&& Other)
	: bSuspended(Other.bSuspended)
{
	Other.bSuspended = false;
}

FPackageAutoSaverScopeGuard::~FPackageAutoSaverScopeGuard()
{
	if (bSuspended && GUnrealEd)
	{
		GUnrealEd->GetPackageAutoSaver().ResumeAutoSave();
	}
}

FPackageAutoSaverScopeGuard& FPackageAutoSaverScopeGuard::operator=(FPackageAutoSaverScopeGuard&& Other)
{
	bSuspended = Other.bSuspended;
	Other.bSuspended = false;
	return *this;
}
