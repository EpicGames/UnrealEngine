// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "UObject/WeakObjectPtr.h"
#include "IPackageAutoSaver.h"

class FObjectPostSaveContext;
struct FEndLoadPackageContext;
struct FTransactionContext;

namespace ECloseNotification
{
	enum Type
	{
		NothingToDo,
		Success,
		Postponed,
		Failed
	};
}

/** A class to handle the creation, destruction, and restoration of auto-saved packages */
class FPackageAutoSaver final : public IPackageAutoSaver
{
public:
	FPackageAutoSaver();
	virtual ~FPackageAutoSaver();

	/** IPackageAutoSaver */
	virtual void UpdateAutoSaveCount(const float DeltaSeconds) override;
	virtual void ResetAutoSaveTimer() override;
	virtual void ForceAutoSaveTimer() override;
	virtual void SuspendAutoSave() override { AutoSaveSuspendedCounter++; }
	virtual void ResumeAutoSave() override;
	virtual bool IsAutoSaveSuspended() const override { return AutoSaveSuspendedCounter > 0; }
	virtual void ForceMinimumTimeTillAutoSave(const float TimeTillAutoSave = 10.0f) override;
	virtual bool AttemptAutoSave(const bool bForceAutoSave = false) override;
	virtual void LoadRestoreFile() override;
	virtual void UpdateRestoreFile(const bool bRestoreEnabled) override;
	virtual bool HasPackagesToRestore() const override;
	virtual void OfferToRestorePackages() override;
	virtual void OnPackagesDeleted(const TArray<UPackage*>& DeletedPackages) override;
	virtual bool IsAutoSaving(const EPackageAutoSaveType AutoSaveType = EPackageAutoSaveType::Transient) const override { return EnumHasAnyFlags(CurrentAutoSaveType, AutoSaveType); }
	virtual void DisableRestorePromptAndDeclinePackageRecovery() override { bAutoDeclineRecovery = true; }

private:
	/**
	 * Called when a package dirty state has been updated
	 *
	 * @param Pkg The package that was changed
	 */
	void OnPackageDirtyStateUpdated(UPackage* Pkg);

	/**
	 * Called when a package is marked as dirty
	 *
	 * @param Pkg The package that was marked as dirty
	 * @param bWasDirty Whether the package was previously dirty before the call to MarkPackageDirty
	 */
	void OnMarkPackageDirty(UPackage* Pkg, bool bWasDirty);

	/**
	 * Called when a package has been saved
	 *
	 * @param Filename The filename the package was saved to
	 * @param Obj The package that was saved
	 */
	void OnPackageSaved(const FString& Filename, UPackage* Pkg, FObjectPostSaveContext ObjectSaveContext);

	/**
	 * Called when an undo/redo operation happens.
	 */
	void OnUndoRedo();

	/**
	 * Dirty sync handling for the auto-save uncontrolled changelist.
	 */
	void OnUncontrolledChangelistChanged();
	void OnPackageLoaded(const FEndLoadPackageContext& Context);
	void OnEndFrame();

	/**
	 * Interactive checkout handling for the auto-save uncontrolled changelist.
	 */
	void GetAdditionalInteractiveSavePackageCandidates(TSet<UPackage*>& OutPackages);
	void OnPreInteractiveCheckoutPackages(const TArray<UPackage*>& Packages, TSet<FName>& OutReadOnlyPackages);
	void OnPostInteractiveCheckoutPackages(const TArray<UPackage*>& Packages, bool bUserResponse);
	void OnPackagesInteractivelyCheckedOut(const TArray<UPackage*>& Packages);
	void OnPackagesInteractivelyMadeWritable(const TArray<UPackage*>& Packages);
	void OnPackagesInteractivelyDiscarded(const TArray<UPackage*>& Packages);

	/**
	 * Update the dirty lists for the current package
	 *
	 * @param Pkg The package to update the lists for
	 */
	void UpdateDirtyListsForPackage(UPackage* Pkg);

	/** @return Returns whether or not the user is able to auto-save. */
	bool CanAutoSave(const bool bForceAutoSave = false) const;

	/** @return Returns whether or not we would need to perform an auto-save (note: does not check if we can perform an auto-save, only that we should if we could). */
	bool DoPackagesNeedAutoSave() const;

	/** @return The notification text to be displayed during auto-saving */
	FText GetAutoSaveNotificationText(const int32 TimeInSecondsUntilAutosave);

	/** 
	 * @param bIgnoreCanAutoSave if True this function returns the time regardless of whether auto-save is enabled.
	 * 
	 * @return Returns the amount of time until the next auto-save in seconds. Or -1 if auto-save is disabled.
	 */
	int32 GetTimeTillAutoSave(const bool bIgnoreCanAutoSave = false) const;

	/**
	 * Attempts to launch a auto-save warning notification if auto-save is imminent, if this is already the case
	 * it will update the time remaining. 
	*/
	void UpdateAutoSaveNotification();

	/** Closes the auto-save warning notification if open, with an appropriate message based on whether it was successful */
	void CloseAutoSaveNotification(const bool Success);

	/** Closes the auto-save warning notification if open, with an appropriate message based on type */
	void CloseAutoSaveNotification(ECloseNotification::Type Type);

	/** Used as a callback for auto-save warning buttons, called when auto-save is forced early */
	void OnAutoSaveSave();

	/** Used as a callback for auto-save warning buttons, called when auto-save is postponed */
	void OnAutoSaveCancel();

	/** Clear out any stale pointers in the dirty packages containers */
	void ClearStalePointers();

	/** The current auto-save number, appended to the auto-save map name, wraps after 10 */
	int32 AutoSaveIndex;

	/** The number of 10-sec intervals that have passed since last auto-save. */
	float AutoSaveCount;

	/** When >0, the auto-saver will not trigger timer-based auto-saves.  */
	int32 AutoSaveSuspendedCounter = 0;

	/** If we are currently auto-saving */
	EPackageAutoSaveType CurrentAutoSaveType = EPackageAutoSaveType::None;

	/** Flag for whether auto-save warning notification has been launched */
	bool bAutoSaveNotificationLaunched;

	/** If we are delaying the time a little bit because we failed to save */
	bool bDelayingDueToFailedSave;

	/** Flag as true if recovery prompt is disabled. Should be true if another system is restoring, replacing the auto-saver recovery, like the Disaster Recovery plugin. */
	bool bAutoDeclineRecovery;

	/** Indicate that we need to update the restore file. */
	bool bNeedRestoreFileUpdate = false;

	/** True if we need to invalidate and re-cache our list of dirty files. */
	bool bSyncWithDirtyPackageList = false;

	/** Set of packages that are pending an update in the auto-save lists */
	TSet<TWeakObjectPtr<UPackage>, TWeakObjectPtrSetKeyFuncs<TWeakObjectPtr<UPackage>>> PackagesPendingUpdate;

	/** Set of package names that have been deleted and should be ignored while in an empty state */
	TSet<FName> PackagesToIgnoreIfEmpty;

	/** Used to reference to the active auto-save warning notification */
	TWeakPtr<SNotificationItem> AutoSaveNotificationPtr;

	/** Packages that have been dirtied and not saved by the user, mapped to their latest auto-save file */
	TMap<TWeakObjectPtr<UPackage>, FString, FDefaultSetAllocator, TWeakObjectPtrMapKeyFuncs<TWeakObjectPtr<UPackage>, FString>> DirtyPackagesForUserSave;

	/** Maps that have been dirtied and not saved by the auto-saver */
	TSet<TWeakObjectPtr<UPackage>, TWeakObjectPtrSetKeyFuncs<TWeakObjectPtr<UPackage>>> DirtyMapsForAutoSave;

	/** Content that has been dirtied and not saved by the auto-saver */
	TSet<TWeakObjectPtr<UPackage>, TWeakObjectPtrSetKeyFuncs<TWeakObjectPtr<UPackage>>> DirtyContentForAutoSave;

	/** Restore information that was loaded following a crash */
	TMap<FString, TPair<FString, FString>> PackagesThatCanBeRestored;

	/** Set of packages to make dirty if they're being managed by the auto-save uncontrolled changelist */
	TSet<FName> PackagesToMakeDirtyIfManagedByAutoSave;
};
