// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PackageBackup.cpp: Utility class for backing up a package.
=============================================================================*/

#include "PackageBackup.h"

#include "CoreGlobals.h"
#include "HAL/FileManager.h"
#include "Internationalization/Internationalization.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/DateTime.h"
#include "Misc/FeedbackContext.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Misc/Timespan.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"

/**
 * Helper struct to hold information on backup files to prevent redundant checks
 */
struct FBackupFileInfo
{
	int64 FileSize;						/** Size of the file */
	FString FileName;					/** Name of the file */
	FDateTime FileTimeStamp;			/** Timestamp of the file */
};


/**
 * Create a backup of the specified package. A backup is only created if the specified
 * package meets specific criteria, as outlined in the comments for ShouldBackupPackage().
 *
 * @param	InPackage	Package which should be backed up
 *
 * @see		FAutoPackageBackup::ShouldBackupPackage()
 *
 * @return	true if the package was successfully backed up; false if it was not
 */
bool FAutoPackageBackup::BackupPackage( const UPackage& InPackage )
{
	bool bPackageBackedUp = false;
	FString OriginalFileName;
	
	// Check if the package is valid for being backed up
	if ( ShouldBackupPackage( InPackage, OriginalFileName ) )
	{
		GWarn->StatusUpdate( -1, -1, NSLOCTEXT("UnrealEd", "PackageBackup_Warning", "Backing up asset...") );

		// Construct the backup file name by appending a timestamp in between the base file name and extension
		FString DestinationFileName = GetBackupDirectory() / FPaths::GetBaseFilename(OriginalFileName);

		DestinationFileName += TEXT("_");
		DestinationFileName += FDateTime::Now().ToString(TEXT("%Y-%m-%d-%H-%M-%S"));
		DestinationFileName += FPaths::GetExtension( OriginalFileName, true );

		// Copy the file to the backup file name
		IFileManager::Get().Copy( *DestinationFileName, *OriginalFileName );

		bPackageBackedUp = true;
	}
	
	return bPackageBackedUp;
}

/**
 * Create a backup of the specified packages. A backup is only created if a specified
 * package meets specific criteria, as outlined in the comments for ShouldBackupPackage().
 *
 * @param	InPackage	Package which should be backed up
 *
 * @see		FAutoPackageBackup::ShouldBackupPackage()
 *
 * @return	true if all provided packages were successfully backed up; false if one or more
 *			were not
 */
bool FAutoPackageBackup::BackupPackages( const TArray<UPackage*>& InPackages )
{
	bool bAllPackagesBackedUp = true;
	for ( TArray<UPackage*>::TConstIterator PackageIter( InPackages ); PackageIter; ++PackageIter )
	{
		UPackage* CurPackage = *PackageIter;
		check( CurPackage );

		if ( !BackupPackage( *CurPackage ) )
		{
			bAllPackagesBackedUp = false;
		}
	}
	return bAllPackagesBackedUp;
}

/**
 * Helper function designed to determine if the provided package should be backed up or not.
 * The function checks for many conditions, such as if the package is too large to backup,
 * if the package has a particular attribute that should prevent it from being backed up (such
 * as being marked for PIE-use), if cooking is in progress, etc.
 *
 * @param	InPackage		Package which should be checked to see if its valid for backing-up
 * @param	OutFileName		File name of the package on disk if the function determines the package
 *							already existed
 *
 * @return	true if the package is valid for backing-up; false otherwise
 */
bool FAutoPackageBackup::ShouldBackupPackage( const UPackage& InPackage, FString& OutFilename )
{
	// Check various conditions to see if the package is a valid candidate for backing up
	bool bShouldBackup =
		GIsEditor																			// Backing up packages only makes sense in the editor
		&& !IsRunningCommandlet()															// Don't backup saves resulting from commandlets
		&& IsPackageBackupEnabled()															// Ensure that the package backup is enabled in the first place
		&& (InPackage.HasAnyPackageFlags(PKG_PlayInEditor) == false)						// Don't back up PIE packages
		&& (InPackage.HasAnyPackageFlags(PKG_ContainsScript) == false);						// Don't back up script packages

	if (!bShouldBackup)
	{
		// Early out here to avoid the call to FileSize below which can be expensive on slower hard drives
		return false;
	}

	if( bShouldBackup )
	{
		GWarn->StatusUpdate( -1, -1, NSLOCTEXT("UnrealEd", "PackageBackup_ValidityWarning", "Determining asset backup validity...") );

		bShouldBackup =	FPackageName::DoesPackageExist( InPackage.GetName(), &OutFilename );	// Make sure the file already exists (no sense in backing up a new package)
	}
	
	// If the package passed the initial backup checks, proceed to check more specific conditions
	// that might disqualify the package from being backed up
	const int64 FileSizeOfBackup = IFileManager::Get().FileSize( *OutFilename );
	if ( bShouldBackup )
	{
		// Ensure that the size the backup would require is less than that of the maximum allowed
		// space for backups
		bShouldBackup = FileSizeOfBackup <= GetMaxAllowedBackupSpace();
	}

	// If all of the prior checks have passed, now see if the package has been backed up
	// too recently to be considered for an additional backup
	if ( bShouldBackup )
	{
		// Ensure that the autosave/backup directory exists
		const FString& BackupSaveDir = GetBackupDirectory();
		IFileManager::Get().MakeDirectory( *BackupSaveDir, 1 );

		// Find all of the files in the backup directory
		TArray<FString> FilesInBackupDir;
		IFileManager::Get().FindFilesRecursive(FilesInBackupDir, *BackupSaveDir, TEXT("*.*"), true, false);

		// Extract the base file name and extension from the passed-in package file name
		FString ExistingBaseFileName = FPaths::GetBaseFilename(OutFilename);
		FString ExistingFileNameExtension = FPaths::GetExtension(OutFilename);

		bool bFoundExistingBackup = false;
		int64 DirectorySize = 0;
		FDateTime LastBackupTimeStamp = FDateTime::MinValue();

		TArray<FBackupFileInfo> BackupFileInfoArray;
		
		// Check every file in the backup directory for matches against the passed-in package
		// (Additionally keep statistics on all backup files for potential maintenance)
		for ( TArray<FString>::TConstIterator FileIter( FilesInBackupDir ); FileIter; ++FileIter )
		{
			const FString CurBackupFileName = FString( *FileIter );
			
			// Create a new backup file info struct for keeping information about each backup file
			const int32 FileInfoIndex = BackupFileInfoArray.AddZeroed();
			FBackupFileInfo& CurBackupFileInfo = BackupFileInfoArray[ FileInfoIndex ];
			
			// Record the backup file's name, size, and timestamp
			CurBackupFileInfo.FileName = CurBackupFileName;
			CurBackupFileInfo.FileSize = IFileManager::Get().FileSize( *CurBackupFileName );
			
			// If we failed to get a timestamp or a valid size, something has happened to the file and it shouldn't be considered
			CurBackupFileInfo.FileTimeStamp = IFileManager::Get().GetTimeStamp(*CurBackupFileName);
			if (CurBackupFileInfo.FileTimeStamp == FDateTime::MinValue() || CurBackupFileInfo.FileSize == -1)
			{
				BackupFileInfoArray.RemoveAt( BackupFileInfoArray.Num() - 1 );
				continue;
			}

			// Calculate total directory size by adding the size of this backup file
			DirectorySize += CurBackupFileInfo.FileSize;

			FString CurBackupBaseFileName =  FPaths::GetBaseFilename(CurBackupFileName);
			FString CurBackupFileNameExtension = FPaths::GetExtension(CurBackupFileName);

			// The base file name of the backup file is going to include an underscore followed by a timestamp, so they must be removed for comparison's sake
			CurBackupBaseFileName.LeftInline( CurBackupBaseFileName.Find( TEXT("_"), ESearchCase::CaseSensitive, ESearchDir::FromEnd ), EAllowShrinking::No);
					
			// If the base file names and extensions match, we've found a backup
			if ( CurBackupBaseFileName == ExistingBaseFileName &&  CurBackupFileNameExtension == ExistingFileNameExtension )
			{
				bFoundExistingBackup = true;

				// Keep track of the most recent matching time stamp so we can check if the passed-in package
				// has been backed up too recently
				if ( CurBackupFileInfo.FileTimeStamp > LastBackupTimeStamp )
				{
					LastBackupTimeStamp = CurBackupFileInfo.FileTimeStamp;
				}
			}
		}

		// If there was an existing backup, check to see if it was created too recently to allow another backup
		if ( bFoundExistingBackup )
		{
			// Check the difference in timestamp seconds against the backup interval; if not enough time has elapsed since
			// the last backup, we don't want to make another one
			if ((FDateTime::UtcNow() - LastBackupTimeStamp).GetTotalSeconds() < GetBackupInterval())
			{
				bShouldBackup = false;
			}
		}

		// If every other check against the package has succeeded for backup purposes, ensure there is enough directory space
		// available in the backup directory, as adding the new backup might use more space than the user allowed for backups.
		// If the backup file size + the current directory size exceeds the max allowed space, deleted old backups until there
		// is sufficient space. If enough space can't be freed for whatever reason, then no back-up will be created.
		if ( bShouldBackup && ( FileSizeOfBackup + DirectorySize > GetMaxAllowedBackupSpace() ) )
		{
			bShouldBackup = PerformBackupSpaceMaintenance( BackupFileInfoArray, DirectorySize, FileSizeOfBackup );
		}
	}
	
	return bShouldBackup;
}

/**
 * Helper function that returns whether the user has package backups enabled or not. The value
 * is determined by a configuration INI setting.
 *
 * @return	true if package backups are enabled; false otherwise
 */
bool FAutoPackageBackup::IsPackageBackupEnabled()
{
	bool bEnabled = false;
	GConfig->GetBool( TEXT("FAutoPackageBackup"), TEXT("Enabled"), bEnabled, GEditorPerProjectIni );
	return bEnabled;
}

/**
 * Helper function that returns the maximum amount of space the user has designated to allow for
 * package backups. This value is determined by a configuration INI setting.
 *
 * @return	The maximum amount of space allowed, in bytes
 */
int32 FAutoPackageBackup::GetMaxAllowedBackupSpace()
{
	int32 MaxSpaceAllowed = 0;
	if ( GConfig->GetInt( TEXT("FAutoPackageBackup"), TEXT("MaxAllowedSpaceInMB"), MaxSpaceAllowed, GEditorPerProjectIni ) )
	{
		// Convert the user stored value from megabytes to bytes; <<= 20 is the same as *= 1024 * 1024
		MaxSpaceAllowed <<= 20;
	}
	return MaxSpaceAllowed;
}

/**
 * Helper function that returns the time in between backups of a package before another backup of
 * the same package should be considered valid. This value is determined by a configuration INI setting,
 * and prevents a package from being backed-up over and over again in a small time frame.
 *
 * @return	The interval to wait before allowing another backup of the same package, in seconds
 */
int32 FAutoPackageBackup::GetBackupInterval()
{
	int32 BackupInterval = 0;
	if ( GConfig->GetInt( TEXT("FAutoPackageBackup"), TEXT("BackupIntervalInMinutes"), BackupInterval, GEditorPerProjectIni ) )
	{
		// Convert the user stored value from minutes to seconds
		BackupInterval *= 60;
	}
	return BackupInterval;
}

/**
 * Helper function that returns the directory to store package backups in.
 *
 * @return	String containing the directory to store package backups in.
 */
FString FAutoPackageBackup::GetBackupDirectory()
{
	FString Directory = FPaths::ProjectSavedDir() / TEXT("Backup");
	return Directory;
}

/**
 * Deletes old backed-up package files until the provided amount of space (in bytes)
 * is available to use in the backup directory. Fails if the provided amount of space
 * is more than the amount of space the user has allowed for backups or if enough space
 * could not be made.
 *
 * @param	InBackupFiles		File info of the files in the backup directory
 * @param	InSpaceUsed			The amount of space, in bytes, the files in the provided array take up
 * @param	InSpaceRequired		The amount of space, in bytes, to assure is available
 *								for the purposes of package backups
 *
 * @return	true if the space was successfully provided, false if not or if the requested space was
 *			greater than the max allowed space by the user
 */
bool FAutoPackageBackup::PerformBackupSpaceMaintenance( TArray<FBackupFileInfo>& InBackupFiles, int64 InSpaceUsed, int64 InSpaceRequired )
{
	bool bSpaceFreed = false;

	const int32 MaxAllowedSpace = GetMaxAllowedBackupSpace();
	
	// We can only free up enough space if the required space is less than the maximum allowed space to begin with
	if ( InSpaceRequired < MaxAllowedSpace )
	{
		GWarn->StatusUpdate( -1, -1, NSLOCTEXT("UnrealEd", "PackageBackup_MaintenanceWarning", "Performing maintenance on asset backup folder...") );
		
		// Sort the backup files in order of their timestamps; we want to naively delete the oldest files first
		struct FCompareFBackupFileInfo
		{
			FORCEINLINE bool operator()( const FBackupFileInfo& A, const FBackupFileInfo& B ) const
			{
				return A.FileTimeStamp < B.FileTimeStamp;
			}
		};
		InBackupFiles.Sort( FCompareFBackupFileInfo() );
		
		int64 CurSpaceUsed = InSpaceUsed;
		TArray<FBackupFileInfo>::TConstIterator BackupFileIter( InBackupFiles );
		
		// Iterate through the backup files until all of the files have been deleted or enough space has been freed
		while ( ( InSpaceRequired + CurSpaceUsed > MaxAllowedSpace ) && BackupFileIter )
		{
			const FBackupFileInfo& CurBackupFileInfo = *BackupFileIter;
			
			// Delete the file; this could potentially fail, but not because of a read-only flag, so if it fails
			// it's likely because the file was removed by the user
			IFileManager::Get().Delete( *CurBackupFileInfo.FileName, true, true );
			CurSpaceUsed -= CurBackupFileInfo.FileSize;
			++BackupFileIter;
		}
		if ( InSpaceRequired + CurSpaceUsed <= MaxAllowedSpace )
		{
			bSpaceFreed = true;
		}
	}

	return bSpaceFreed;
}

