// Copyright Epic Games, Inc. All Rights Reserved.

#include "PackageRestore.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Layout/Margin.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SWindow.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SCheckBox.h"
#include "Styling/AppStyle.h"
#include "Editor.h"
#include "Misc/MessageDialog.h"
#include "PackageTools.h"
#include "AutoSaveUtils.h"
#include "SWarningOrErrorBox.h"

#define LOCTEXT_NAMESPACE "PackageRestore"

namespace PackageRestore
{
	/** An item in the SPackageRestoreDialog package list */
	class FPackageRestoreItem : public TSharedFromThis<FPackageRestoreItem>
	{
	public:
		FPackageRestoreItem(const FString& InPackageName, const FString& InPackageLabel, const FString& InPackageFilename, const FString& InAutoSaveFilename, const bool bInIsExistingPackage)
			: PackageName(InPackageName)
			, PackageLabel(InPackageLabel)
			, PackageFilename(InPackageFilename)			
			, AutoSaveFilename(InAutoSaveFilename)
			, bIsExistingPackage(bInIsExistingPackage)
			, State(ECheckBoxState::Unchecked)
		{
		}

		/** @return The package name for this item */
		const FString& GetPackageName() const
		{
			return PackageName;
		}

		/** @return The package label for this item */
		const FString& GetPackageLabel() const
		{
			return PackageLabel.IsEmpty() ? PackageName : PackageLabel;
		}

		/** @return The package filename for this item */
		const FString& GetPackageFilename() const
		{
			return PackageFilename;
		}

		/** @return The package auto-save filename for this item */
		const FString& GetAutoSaveFilename() const
		{
			return AutoSaveFilename;
		}

		/** @return True if this item is to replace an existing package, or false if it is to add a new package */
		bool IsExistingPackage() const
		{
			return bIsExistingPackage;
		}

		/** @return The state of this item (checked, unchecked) */
		ECheckBoxState GetState() const
		{
			return State;
		}

		/** Set the state of this item (checked, unchecked)  */
		void SetState(const ECheckBoxState InState)
		{
			State = InState;
		}

		/** @return The tooltip text for this item  */
		FText GetToolTip() const
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("PackageName"), LOCTEXT("PackageName", "Package Name"));
			Args.Add(TEXT("PackageFile"), LOCTEXT("PackageFile", "Package File"));
			Args.Add(TEXT("AutoSaveFile"), LOCTEXT("AutoSaveFile", "Autosave File"));

			Args.Add(TEXT("PackageNameStr"), FText::FromString(PackageName));
			Args.Add(TEXT("PackageFileStr"), FText::FromString(PackageFilename));
			Args.Add(TEXT("AutoSaveFileStr"), FText::FromString(AutoSaveFilename));

			return FText::Format(FText::FromString("{PackageName}: {PackageNameStr}\n\n{PackageFile}: {PackageFileStr}\n\n{AutoSaveFile}: {AutoSaveFileStr}"), Args);
		}

		/** @return Process a request to navigate to the package location */
		FReply OnExploreToPackage(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) const
		{
			const FString AbsoluteFilename = FPaths::ConvertRelativePathToFull(PackageFilename);
			const FString AbsolutePath = FPaths::GetPath(AbsoluteFilename);
			FPlatformProcess::ExploreFolder(*AbsolutePath);

			return FReply::Handled();
		}

		/** @return Process a request to navigate to the auto-save location */
		FReply OnExploreToAutoSave(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) const
		{
			const FString AbsoluteFilename = FPaths::ConvertRelativePathToFull(AutoSaveFilename);
			const FString AbsolutePath = FPaths::GetPath(AbsoluteFilename);
			FPlatformProcess::ExploreFolder(*AbsolutePath);

			return FReply::Handled();
		}

		EVisibility GetRestoreOverMoreRecentPackageWarningVisibility() const
		{
			FString RestoreDstPathname = *FPaths::ConvertRelativePathToFull(PackageFilename);
			if (IFileManager::Get().FileExists(*RestoreDstPathname))
			{
				FDateTime AutoSavedSrcModificationTime = IFileManager::Get().GetStatData(*FPaths::ConvertRelativePathToFull(AutoSaveFilename)).ModificationTime;
				FDateTime RestoredDstModificationTime = IFileManager::Get().GetStatData(*RestoreDstPathname).ModificationTime;
				return AutoSavedSrcModificationTime > RestoredDstModificationTime ? EVisibility::Collapsed : EVisibility::Visible;
			}
			return EVisibility::Collapsed; // The destination file doesn't exist, restoring the auto-saved file cannot overwrite anything.
		}

	private:
		FString PackageName;
		FString PackageLabel;
		FString PackageFilename;
		FString AutoSaveFilename;
		bool bIsExistingPackage;
		ECheckBoxState State;
	};

	typedef TSharedPtr<FPackageRestoreItem> FPackageRestoreItemPtr;
	typedef TArray<FPackageRestoreItemPtr> FPackageRestoreItems;
	const FName ColumnID_CheckBoxLabel("PackageCheckboxLabel");
	const FName ColumnID_PackageLabel("PackageNameLabel");
	const FName ColumnID_FileLabel("PackageFileLabel");
	const FName ColumnID_SaveLabel("PackageAutosaveLabel");

	/** Widget that represents a row in the PackageRestoreDialog's list view.  Generates widgets for each column on demand. */
	class SPackageRestoreItemsListRow
		: public SMultiColumnTableRow< FPackageRestoreItemPtr >
	{

	public:

		SLATE_BEGIN_ARGS(SPackageRestoreItemsListRow) {}

		/** The list item for this row */
		SLATE_ARGUMENT(FPackageRestoreItemPtr, Item)

		SLATE_END_ARGS()

		/** Construct function for this widget */
		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
		{
			Item = InArgs._Item;

			SMultiColumnTableRow< FPackageRestoreItemPtr >::Construct(
				FSuperRowType::FArguments().Padding(FMargin(0, 3))
				, InOwnerTableView);
		}

		/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the list row. */
		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
		{
			check(Item.IsValid());

			const FSlateBrush* FolderOpenBrush = FAppStyle::Get().GetBrush("PackageRestore.FolderOpen");
			TSharedPtr<SWidget> ItemContentWidget;

			if (ColumnName == ColumnID_CheckBoxLabel)
			{
				ItemContentWidget = SNew(SHorizontalBox)
					.ToolTipText(Item->GetToolTip())
					+ SHorizontalBox::Slot()
					.Padding(7, 0, 2, 0)
					.VAlign(VAlign_Center)
					[
						SNew(SCheckBox)
						.IsChecked(Item.Get(), &FPackageRestoreItem::GetState)
						.OnCheckStateChanged(Item.Get(), &FPackageRestoreItem::SetState)
					];
					
			}
			else if (ColumnName == ColumnID_PackageLabel)
			{
				ItemContentWidget = SNew(SHorizontalBox)
					.ToolTipText(Item->GetToolTip())
					+ SHorizontalBox::Slot()
					.Padding(FMargin(2, 0, 4, 0))
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.WarningWithColor"))
						.DesiredSizeOverride(FVector2D(16, 16))
						.ToolTipText(LOCTEXT("OverwritingMoreRecentPackageFile", "The auto-saved file is older than the file it restores. You could lose work if the file was updated or modified after the crash but before the Editor was restarted."))
						.Visibility(Item.Get(), &FPackageRestoreItem::GetRestoreOverMoreRecentPackageWarningVisibility)
					]
					+ SHorizontalBox::Slot()
					.Padding(FMargin(6, 0, 20, 0))
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					.FillWidth(1)
					[
						SNew(STextBlock)
						.Text(FText::FromString(Item->GetPackageLabel()))
					];
			}
			else if (ColumnName == ColumnID_FileLabel)
			{
				ItemContentWidget = SNew(SHorizontalBox)
					.ToolTipText(Item->GetToolTip())
					+ SHorizontalBox::Slot()
					.Padding(FMargin(4, 0, 0, 0))
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					.FillWidth(1)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.FillWidth(1)
						[
							SNew(STextBlock)
							.Text(FText::FromString(Item->GetPackageFilename()))
						]
						+ SHorizontalBox::Slot()
						.Padding(FMargin(11, 0, 20, 0))
						.AutoWidth()
						[
							SNew(SImage)
							.Image(FolderOpenBrush)
							.OnMouseButtonDown(Item.Get(), &FPackageRestoreItem::OnExploreToPackage)
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					];
			}
			else if (ColumnName == ColumnID_SaveLabel)
			{
				ItemContentWidget = SNew(SHorizontalBox)
					.ToolTipText(Item->GetToolTip())
					+ SHorizontalBox::Slot()
					.Padding(FMargin(4, 0, 0, 0))
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					.FillWidth(1)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.FillWidth(1)
						[
							SNew(STextBlock)
							.Text(FText::FromString(Item->GetAutoSaveFilename()))
						]
						+ SHorizontalBox::Slot()
						.Padding(FMargin(11, 0, 40, 0))
						.AutoWidth()
						[
							SNew(SImage)
							.Image(FolderOpenBrush)
							.OnMouseButtonDown(Item.Get(), &FPackageRestoreItem::OnExploreToAutoSave)
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					];
			}
			
			return ItemContentWidget.ToSharedRef();
		}

	private:

		/** The item associated with this row of data */
		FPackageRestoreItemPtr Item;
	};

	/** Dialog for letting the user choose which packages they want to restore */
	class SPackageRestoreDialog : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SPackageRestoreDialog)
			{}

		/** Information about which packages to offer restoration for */
		SLATE_ATTRIBUTE(FPackageRestoreItems*, PackageRestoreItems)

		SLATE_END_ARGS()
		
		/**
		 * Construct this widget
		 *
		 * @param	InArgs	The declaration data for this widget
		 */
		void Construct(const FArguments& InArgs)
		{
			PackageRestoreItems = InArgs._PackageRestoreItems.Get();
			ReturnCode = false;

			TSharedRef< SHeaderRow > HeaderRowWidget = SNew(SHeaderRow);

			HeaderRowWidget->AddColumn(
				SHeaderRow::Column(ColumnID_CheckBoxLabel)
				[
					SNew(SCheckBox)
					.IsChecked(this, &SPackageRestoreDialog::GetToggleSelectedState)
					.OnCheckStateChanged(this, &SPackageRestoreDialog::OnToggleSelectedCheckBox)
				]
				.FixedWidth(34.f)
				.HAlignHeader(HAlign_Center)
				);

			HeaderRowWidget->AddColumn(
				SHeaderRow::Column(ColumnID_PackageLabel)
				.DefaultLabel(LOCTEXT("PackageName", "Package Name"))
				.HeaderContentPadding(FMargin(8, 0, 0, 0))
			);

			HeaderRowWidget->AddColumn(
				SHeaderRow::Column(ColumnID_FileLabel)
				.DefaultLabel(LOCTEXT("PackageFile", "Package File"))
				.HeaderContentPadding(FMargin(8, 0, 0, 0))
			);

			HeaderRowWidget->AddColumn(
				SHeaderRow::Column(ColumnID_SaveLabel)
				.DefaultLabel(LOCTEXT("AutoSaveFile", "Autosave File"))
				.HeaderContentPadding(FMargin(8, 0, 0, 0))
			);


			this->ChildSlot
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.Padding(9, 13, 7, 16)
					.AutoHeight()
					[
						SNew(SWarningOrErrorBox)
						.MessageStyle(EMessageStyle::Warning)
						.Message(LOCTEXT("RestoreInfo", "Unreal Editor detected that it did not shut-down cleanly and that the following packages have auto-saves associated with them.\nWould you like to restore from these auto-saves?"))
					]
					+SVerticalBox::Slot() 
					.FillHeight(1)
					.Padding(8, 0)
					[
						SAssignNew(ItemListView, SListView<FPackageRestoreItemPtr>)
						.ListItemsSource(PackageRestoreItems)
						.OnGenerateRow(this, &SPackageRestoreDialog::MakePackageRestoreListItemWidget)
						.HeaderRow(HeaderRowWidget)
					]
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2, 17, 26, 17)
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Bottom)
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.Padding(2)
						.AutoWidth()
						[
							SNew(SButton)
							.Text(LOCTEXT("RestoreSelectedPackages", "Restore Selected"))
							.OnClicked(this, &SPackageRestoreDialog::OnRestoreSelectedButtonClicked)
							.IsEnabled(this, &SPackageRestoreDialog::IsRestoreSelectedButtonEnabled)
						]
						+SHorizontalBox::Slot()
						.Padding(7, 2, 2, 2)
						.AutoWidth()
						[
							SNew(SButton)
							.Text(LOCTEXT("SkipRestorePackages", "Skip Restore"))
							.OnClicked(this, &SPackageRestoreDialog::OnSkipRestoreButtonClicked)
						]
					]
				]
			];
		}

		/** 
		 * Set the window which owns us (we'll close it when we're finished)
		 */
		void SetWindow(TSharedRef<SWindow> InWindow)
		{
			ParentWindowPtr = InWindow;
		}

		/** 
		 * Makes the widget for the checkbox items in the list view 
		 */
		TSharedRef<ITableRow> MakePackageRestoreListItemWidget(FPackageRestoreItemPtr Item, const TSharedRef<STableViewBase>& OwnerTable)
		{
			return SNew(SPackageRestoreItemsListRow, OwnerTable)
				.Item(Item);
		}

		/** 
		 * @return	the desired toggle state for the ToggleSelectedCheckBox.
		 * Returns Unchecked, unless all of the selected packages are Checked.
		 */
		ECheckBoxState GetToggleSelectedState() const
		{
			// Default to a Checked state
			ECheckBoxState PendingState = ECheckBoxState::Checked;

			for(auto It = PackageRestoreItems->CreateConstIterator(); It; ++It)
			{
				const FPackageRestoreItemPtr& ListItem = *It;

				if(ListItem->GetState() == ECheckBoxState::Unchecked)
				{
					// If any package in the selection is Unchecked, then represent the entire set of highlighted packages as Unchecked,
					// so that the first (user) toggle of ToggleSelectedCheckBox consistently Checks all highlighted packages
					PendingState = ECheckBoxState::Unchecked;
				}
			}

			return PendingState;
		}

		/** 
		 * Toggles the highlighted packages.
		 * If no packages are explicitly highlighted, toggles all packages in the list.
		 */
		void OnToggleSelectedCheckBox(ECheckBoxState InNewState)
		{
			for(auto It = PackageRestoreItems->CreateIterator(); It; ++It)
			{
				FPackageRestoreItemPtr& ListItem = *It;
				ListItem->SetState(InNewState);
			}

			ItemListView->RequestListRefresh();
		}

		/** 
		 * Check to see if the "Restore Selected" button should be enabled
		 */
		bool IsRestoreSelectedButtonEnabled() const
		{
			// Enabled if anything is selected
			for(auto It = PackageRestoreItems->CreateConstIterator(); It; ++It)
			{
				const FPackageRestoreItemPtr& ListItem = *It;

				if(ListItem->GetState() == ECheckBoxState::Checked)
				{
					return true;
				}
			}

			return false;
		}

		/**
		 * Called when the "Restore Selected" button is clicked
		 */
		FReply OnRestoreSelectedButtonClicked()
		{
			ReturnCode = true;

			if(ParentWindowPtr.IsValid())
			{
				TSharedPtr<SWindow> ParentWindowPin = ParentWindowPtr.Pin();
				ParentWindowPin->RequestDestroyWindow();
			}

			return FReply::Handled();
		}

		/**
		 * Called when the "Skip Restore" button is clicked
		 */
		FReply OnSkipRestoreButtonClicked()
		{
			if(ParentWindowPtr.IsValid())
			{
				TSharedPtr<SWindow> ParentWindowPin = ParentWindowPtr.Pin();
				ParentWindowPin->RequestDestroyWindow();
			}

			return FReply::Handled();
		}

		/**
		 * Get the return code for this dlg, as well as some useful information about what was selected
		 *
		 * @param SelectedPackageItems Array to fill with the list items the user wants to restore
		 *
		 * @return true if we should perform an import, false if the user cancelled
		 */
		bool GetReturnType(FPackageRestoreItems& SelectedPackageItems) const
		{
			SelectedPackageItems.Empty();
			SelectedPackageItems.Reserve(PackageRestoreItems->Num());

			// Get the list of packages selected to be restored
			for(auto It = PackageRestoreItems->CreateConstIterator(); It; ++It)
			{
				const FPackageRestoreItemPtr& ListItem = *It;

				if(ListItem->GetState() == ECheckBoxState::Checked)
				{
					SelectedPackageItems.Add(ListItem);
				}
			}

			return ReturnCode;
		}

	private:
		FPackageRestoreItems* PackageRestoreItems;
		TWeakPtr<SWindow> ParentWindowPtr;
		TSharedPtr< SListView<FPackageRestoreItemPtr> > ItemListView;
		bool ReturnCode;
	};

	void UnloadPackagesBeforeRestore(const FPackageRestoreItems& SelectedPackageItems, FPackageRestoreItems& OutContentPackagesToReload, FPackageRestoreItemPtr& OutWorldPackageToReload)
	{
		// Get the package for the currently loaded world; if we need to restore this package then we also need to unload the current world
		UPackage* const CurrentWorldPackage = CastChecked<UPackage>(GWorld->GetOuter());

		// Work out a list of content packages that need unloading, also work out if we need to unload the current world
		TArray<UPackage*> PackagesToUnload;
		FPackageRestoreItemPtr CurrentWorldRestoreItem;
		for(auto It = SelectedPackageItems.CreateConstIterator(); It; ++It)
		{
			const FPackageRestoreItemPtr& RestoreItem = *It;

			if(!RestoreItem->IsExistingPackage())
			{
				continue;
			}

			UPackage* const Package = FindPackage(nullptr, *RestoreItem->GetPackageName());
			if(Package)
			{
				const bool bIsContentPackage = RestoreItem->GetPackageFilename().EndsWith(FPackageName::GetAssetPackageExtension());
				if(bIsContentPackage)
				{
					// Add this package to the list to be reloaded once we've restored everything
					PackagesToUnload.Add(Package);
					OutContentPackagesToReload.Add(RestoreItem);
				}
				else if(Package == CurrentWorldPackage)
				{
					// If this is the current world, we also need to unload it
					CurrentWorldRestoreItem = RestoreItem;
				}
			}
		}

		if(CurrentWorldRestoreItem.IsValid())
		{
			// Replace the current world with an empty world (this may fail)
			GEditor->CreateNewMapForEditing();

			// See if our world package has been unloaded
			UPackage* const EmptyWorldPackage = CastChecked<UPackage>(GWorld->GetOuter());
			if(CurrentWorldPackage != EmptyWorldPackage)
			{
				OutWorldPackageToReload = CurrentWorldRestoreItem;

				// If we can still find the package for the old world, forcibly unload it too
				UPackage* const Package = FindPackage(nullptr, *CurrentWorldRestoreItem->GetPackageName());
				if(Package)
				{
					PackagesToUnload.Add(Package);
				}
			}
		}

		UPackageTools::UnloadPackages(PackagesToUnload);
	}

	void ReloadPackagesAfterRestore(const FPackageRestoreItems& ContentPackagesToReload, const FPackageRestoreItemPtr& WorldPackageToReload)
	{
		// Reload any content packages that we unloaded to perform the restore
		for(auto It = ContentPackagesToReload.CreateConstIterator(); It; ++It)
		{
			const FPackageRestoreItemPtr& RestoreItem = *It;
			UPackageTools::LoadPackage(*RestoreItem->GetPackageName());
		}

		// Also reload the current world if we caused it to be unloaded
		if(WorldPackageToReload.IsValid())
		{
			FEditorFileUtils::LoadMap(WorldPackageToReload->GetPackageFilename());
		}
	}
}

FEditorFileUtils::EPromptReturnCode PackageRestore::PromptToRestorePackages(const TMap<FString, TPair<FString, FString>>& PackagesToRestore, TArray<FString>* OutFailedPackages)
{
	const FString AutoSaveDir = AutoSaveUtils::GetAutoSaveDir();

	FPackageRestoreItems PackageRestoreItems;
	PackageRestoreItems.Reserve(PackagesToRestore.Num());

	for(auto It = PackagesToRestore.CreateConstIterator(); It; ++It)
	{
		const FString& PackageFullPath = It.Key();
		const FString& PackageAssetLabel = It.Value().Key;
		const FString& AutoSavePath = It.Value().Value;

		FString PackageFilename;
		if(FPackageName::DoesPackageExist(PackageFullPath, &PackageFilename))
		{
			FPackageRestoreItemPtr PackageItemPtr = MakeShared<FPackageRestoreItem>(PackageFullPath, PackageAssetLabel, PackageFilename, AutoSaveDir / AutoSavePath, true/*bIsExistingPackage*/);
			PackageRestoreItems.Add(PackageItemPtr);
		}
		else
		{
			// A package may not exist on disk if it was for a newly added or imported asset, which hasn't yet had SaveDirtyPackages called for it
			if (FPackageName::TryConvertLongPackageNameToFilename(PackageFullPath, PackageFilename)) // no extension yet
			{
				PackageFilename += FPaths::GetExtension(AutoSavePath, true/*bIncludeDot*/);

				FPackageRestoreItemPtr PackageItemPtr = MakeShared<FPackageRestoreItem>(PackageFullPath, PackageAssetLabel, PackageFilename, AutoSaveDir / AutoSavePath, false/*bIsExistingPackage*/);
				PackageRestoreItems.Add(PackageItemPtr);
			}
		}
	}

	if(!PackageRestoreItems.Num())
	{
		// Nothing to restore
		return FEditorFileUtils::PR_Success;
	}

	// Create the dlg to ask the user which packages to restore
	TSharedRef<SPackageRestoreDialog> PackageRestoreDlgRef = SNew(SPackageRestoreDialog)
		.PackageRestoreItems(&PackageRestoreItems);

	// Create the window to host our dlg
	TSharedRef<SWindow> PackageRestoreWindowRef = SNew(SWindow)
		.Title(LOCTEXT("RestorePackages", "Restore Packages"))
		.ClientSize(FVector2D(1000, 550));
	PackageRestoreWindowRef->SetContent(PackageRestoreDlgRef);
	PackageRestoreDlgRef->SetWindow(PackageRestoreWindowRef);

	// Show the dlg in a modal window so we can wait for the result in this function
	GEditor->EditorAddModalWindow(PackageRestoreWindowRef);

	// Get the return code, and work out what we need to restore
	FPackageRestoreItems SelectedPackageItems;
	if(!PackageRestoreDlgRef->GetReturnType(SelectedPackageItems))
	{
		return FEditorFileUtils::PR_Declined;
	}

	// Try and ensure that these packages are checked-out by the source control system
	{
		TArray<FString> SelectedPackageNames;
		SelectedPackageNames.Reserve(SelectedPackageItems.Num());

		// Get an array of selected package names to check out
		for(auto It = SelectedPackageItems.CreateConstIterator(); It; ++It)
		{
			const FPackageRestoreItemPtr& SelectedPackageItem = *It;

			if(SelectedPackageItem->IsExistingPackage())
			{
				SelectedPackageNames.Add(SelectedPackageItem->GetPackageName());
			}
		}

		// Note: This may fail and present the user with an error message, however we still 
		// want to continue as they may have checked out some packages that could now be restored
		const bool bErrorIfAlreadyCheckedOut = false; // some of the packages might already be checked out; that isn't an error
		FEditorFileUtils::CheckoutPackages(SelectedPackageNames, nullptr, bErrorIfAlreadyCheckedOut);
	}

	// It's possible that some packages may have already been loaded by the editor
	// If they have, we need to forcibly unload them so that we can overwrite their files
	FPackageRestoreItems ContentPackagesToReload;
	FPackageRestoreItemPtr WorldPackageToReload;
	UnloadPackagesBeforeRestore(SelectedPackageItems, ContentPackagesToReload, WorldPackageToReload);

	// Copy the auto-save files over the originals
	TArray<FString> FailedPackages;
	for(auto It = SelectedPackageItems.CreateConstIterator(); It; ++It)
	{
		const FPackageRestoreItemPtr& SelectedItem = *It;

		if(IFileManager::Get().Copy(*SelectedItem->GetPackageFilename(), *SelectedItem->GetAutoSaveFilename()) != COPY_OK)
		{
			FailedPackages.Add(SelectedItem->GetPackageName());
		}
	}

	// Reload any packages that we unloaded above
	ReloadPackagesAfterRestore(ContentPackagesToReload, WorldPackageToReload);

	if(FailedPackages.Num())
	{
		if(OutFailedPackages)
		{
			*OutFailedPackages = FailedPackages;
		}

		FString FailedPackagesStr;
		for(auto It = FailedPackages.CreateConstIterator(); It; ++It)
		{
			const FString& PackageName = *It;

			if(It.GetIndex() > 0)
			{
				FailedPackagesStr += "\n";
			}

			FailedPackagesStr += PackageName;
		}

		FFormatNamedArguments Args;
		Args.Add(TEXT("FailedRestoreMessage"), LOCTEXT("FailedRestoreMessage", "The following packages could not be restored"));
		Args.Add(TEXT("FailedPackages"), FText::FromString(FailedPackagesStr));

		const FText Message = FText::Format(FText::FromString("{FailedRestoreMessage}:\n{FailedPackages}"), Args);
		const FText Title = LOCTEXT("FailedRestoreDlgTitle", "Failed to restore packages!");

		FMessageDialog::Open(EAppMsgType::Ok, Message, Title);

		return FEditorFileUtils::PR_Failure;
	}

	return FEditorFileUtils::PR_Success;
}

#undef LOCTEXT_NAMESPACE
