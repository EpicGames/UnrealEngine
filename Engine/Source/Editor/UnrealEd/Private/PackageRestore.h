// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "FileHelpers.h"

namespace PackageRestore
{
	/**
	 * Prompt the user for which of the provided packages should be restored from an auto-save. If the user cancels the dialog, no packages are restored.
	 * Any packages the user selects to restore will attempt to be checked out via source control.
	 * After all packages are restored (or not), the user is provided with a warning about which packages failed to restore.
	 *
	 * @param		PackagesRestore		A map of package path names to their most up-to-date auto-save file. Both map and content packages are supported.
	 * @param		OutFailedPackages	If specified, will be filled in with all of the packages that failed to save successfully
	 *
	 * @return		An enum value signifying success, failure, user declined, or cancellation. If any packages at all failed to restore during execution, the return code will be 
	 *				failure, even if other packages successfully restored. If the user cancels at any point during any prompt, the return code will be cancellation, even though it
	 *				is possible some packages have been successfully restored. If the user opts the "Skip Restore" option on the dialog, the return code will indicate the user has 
	 *				declined out of the prompt. This way calling code can distinguish between a decline and a cancel and then proceed as planned, or abort its operation accordingly.
	 */
	FEditorFileUtils::EPromptReturnCode PromptToRestorePackages(const  TMap<FString, TPair<FString, FString>>& PackagesToRestore, TArray<FString>* OutFailedPackages = nullptr);
}
