// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================================
 CommandletPackageHelper.cpp: Utility class that provides tools to handle packages & source control operations.
=============================================================================================================*/

#include "PackageSourceControlHelper.h"
#include "Logging/LogMacros.h"
#include "UObject/Linker.h"
#include "UObject/Package.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "PackageTools.h"
#include "ISourceControlOperation.h"
#include "ISourceControlModule.h"
#include "ISourceControlState.h"
#include "ISourceControlProvider.h"
#include "SourceControlOperations.h"
#include "Misc/PackageName.h"

DEFINE_LOG_CATEGORY_STATIC(LogCommandletPackageHelper, Log, All);

namespace FPackageSourceControlHelperLog
{
	void Error(bool bErrorsAsWarnings, const FString& Msg)
	{
		if (bErrorsAsWarnings)
		{
			UE_LOGF(LogCommandletPackageHelper, Warning, "%ls", *Msg);
		}
		else
		{
			UE_LOGF(LogCommandletPackageHelper, Error, "%ls", *Msg);
		}
	}
}

bool FPackageSourceControlHelper::UseSourceControl() const
{
	return GetSourceControlProvider().IsEnabled();
}

ISourceControlProvider& FPackageSourceControlHelper::GetSourceControlProvider() const
{ 
	return ISourceControlModule::Get().GetProvider();
}

bool FPackageSourceControlHelper::Delete(const FString& PackageName) const
{
	TArray<FString> PackageNames = { PackageName };
	return Delete(PackageNames);
}

bool FPackageSourceControlHelper::Delete(const TArray<FString>& PackageNames, bool bErrorsAsWarnings) const
{
	if (PackageNames.IsEmpty())
	{
		return true;
	}

	bool bSuccess = true;
	
	// Early out when not using source control
	if (!UseSourceControl())
	{
		for (const FString& PackageName : PackageNames)
		{
			FString Filename = SourceControlHelpers::PackageFilename(PackageName);

			if (!IPlatformFile::GetPlatformPhysical().SetReadOnly(*Filename, false) ||
				!IPlatformFile::GetPlatformPhysical().DeleteFile(*Filename))
			{
				bSuccess = false;
				FPackageSourceControlHelperLog::Error(bErrorsAsWarnings, FString::Printf(TEXT("Error deleting %s"), *Filename));
				if (!bErrorsAsWarnings)
				{
					return false;
				}
			}
		}

		return bSuccess;
	}

	TArray<FString> FilesToRevert;
	TArray<FString> FilesToDeleteFromDisk;
	TArray<FString> FilesToDeleteFromSCC;
		
	// First: get latest state from source control
	TArray<FString> Filenames = SourceControlHelpers::PackageFilenames(PackageNames);
	TArray<FSourceControlStateRef> SourceControlStates;
	
	if (GetSourceControlProvider().GetState(Filenames, SourceControlStates, EStateCacheUsage::ForceUpdate) != ECommandResult::Succeeded)
	{
		// Nothing we can do if SCCStates fail
		FPackageSourceControlHelperLog::Error(false, TEXT("Could not get revision control state for packages"));
		return false;
	}
		
	for(FSourceControlStateRef& SourceControlState : SourceControlStates)
	{
		const FString& Filename = SourceControlState->GetFilename();

		UE_LOGF(LogCommandletPackageHelper, Verbose, "Deleting %ls", *Filename);

		if (SourceControlState->IsSourceControlled())
		{
			FString OtherCheckedOutUser;
			if (SourceControlState->IsCheckedOutOther(&OtherCheckedOutUser))
			{
				FPackageSourceControlHelperLog::Error(bErrorsAsWarnings, FString::Printf(TEXT("File %s already checked out by %s, will not delete"), *Filename, *OtherCheckedOutUser));
				bSuccess = false;
			}
			else if (!SourceControlState->IsCurrent())
			{
				FPackageSourceControlHelperLog::Error(bErrorsAsWarnings, FString::Printf(TEXT("File %s (not at head revision), will not delete"), *Filename));
				bSuccess = false;
			}
			else if (SourceControlState->IsAdded())
			{
				FilesToRevert.Add(Filename);
				FilesToDeleteFromDisk.Add(Filename);
			}
			else
			{
				if (SourceControlState->IsCheckedOut())
				{
					FilesToRevert.Add(Filename);
				}

				FilesToDeleteFromSCC.Add(Filename);
			}
		}
		else
		{
			FilesToDeleteFromDisk.Add(Filename);
		}
	}

	if (!bSuccess && !bErrorsAsWarnings)
	{
		// Errors were found, we'll cancel everything
		return false;
	}

	// It's possible that not all files were in the source control cache, in which case we should still add them to the
	// files to delete on disk.
	if (Filenames.Num() != SourceControlStates.Num())
	{
		for (FSourceControlStateRef& SourceControlState : SourceControlStates)
		{
			Filenames.Remove(SourceControlState->GetFilename());
		}

		FilesToDeleteFromDisk.Append(Filenames);
	}

	// First, revert files from SCC
	if (FilesToRevert.Num() > 0)
	{
		TSharedRef<FRevert, ESPMode::ThreadSafe> RevertOperation = ISourceControlOperation::Create<FRevert>();
		RevertOperation->SetSoftRevert(GetSourceControlProvider().UsesSoftRevertOnDelete());

		if (GetSourceControlProvider().Execute(RevertOperation, FilesToRevert) != ECommandResult::Succeeded)
		{
			bSuccess = false;
			FPackageSourceControlHelperLog::Error(bErrorsAsWarnings, TEXT("Error reverting packages from revision control"));
			if (!bErrorsAsWarnings)
			{
				return false;
			}
		}
	}

	// Then delete files from SCC
	if (FilesToDeleteFromSCC.Num() > 0)
	{
		TSharedRef<FDelete, ESPMode::ThreadSafe> DeleteOperation = ISourceControlOperation::Create<FDelete>();

		if (GetSourceControlProvider().Execute(DeleteOperation, FilesToDeleteFromSCC) != ECommandResult::Succeeded)
		{
			bSuccess = false;
			FPackageSourceControlHelperLog::Error(bErrorsAsWarnings, TEXT("Error deleting packages from revision control"));
			if(!bErrorsAsWarnings)
			{
				return false;
			}
		}
	}

	// Then delete files on disk
	for (const FString& Filename : FilesToDeleteFromDisk)
	{
		if (!IFileManager::Get().Delete(*Filename, false, true))
		{
			bSuccess = false;
			FPackageSourceControlHelperLog::Error(bErrorsAsWarnings, FString::Printf(TEXT("Error deleting %s locally"), *Filename));
			if(!bErrorsAsWarnings)
			{
				return false;
			}
		}
	}

	return bSuccess;
}

bool FPackageSourceControlHelper::Delete(UPackage* Package) const
{
	TArray<UPackage*> Packages = { Package };
	return Delete(Packages);
}

bool FPackageSourceControlHelper::Delete(const TArray<UPackage*>& Packages) const
{
	if (Packages.IsEmpty())
	{
		return true;
	}

	TArray<FString> PackageNames;
	PackageNames.Reserve(Packages.Num());
	
	for (UPackage* Package : Packages)
	{
		PackageNames.Add(Package->GetName());
		ResetLoaders(Package);
	}

	return Delete(PackageNames);
}

bool FPackageSourceControlHelper::AddToSourceControl(UPackage* Package) const
{
	if (UseSourceControl())
	{
		const FString PackageFilename = SourceControlHelpers::PackageFilename(Package);
		return AddToSourceControl({ PackageFilename });
	}

	return true;
}

bool FPackageSourceControlHelper::AddToSourceControl(const TArray<UPackage*>& Packages, bool bErrorsAsWarnings) const
{
	if (!UseSourceControl() || Packages.IsEmpty())
	{
		return true;
	}

	TArray<FString> PackageFilenames;
	PackageFilenames.Reserve(Packages.Num());

	for (const UPackage* Package : Packages)
	{
		FString PackageFilename = SourceControlHelpers::PackageFilename(Package);
		PackageFilenames.Emplace(MoveTemp(PackageFilename));
	}
	return AddToSourceControl(PackageFilenames, bErrorsAsWarnings);
}

bool FPackageSourceControlHelper::AddToSourceControl(const TArray<FString>& PackageNames, bool bErrorsAsWarnings) const
{
	if (!UseSourceControl() || PackageNames.IsEmpty())
	{
		return true;
	}

	// Convert package names to package filenames
	TArray<FString> PackageFilenames = SourceControlHelpers::PackageFilenames(PackageNames);

	// Two-pass checkout mechanism
	TArray<FString> PackagesToAdd;
	PackagesToAdd.Reserve(PackageFilenames.Num());
	bool bSuccess = true;
	
	TArray<FSourceControlStateRef> SourceControlStates;
	if (GetSourceControlProvider().GetState(PackageFilenames, SourceControlStates, EStateCacheUsage::ForceUpdate) != ECommandResult::Succeeded)
	{
		// Nothing we can do if SCCStates fail
		FPackageSourceControlHelperLog::Error(false, TEXT("Could not get revision control state for packages"));
		return false;
	}
		
	for (FSourceControlStateRef& SourceControlState : SourceControlStates)
	{
		const FString& PackageFilename = SourceControlState->GetFilename();

		FString OtherCheckedOutUser;
		if (SourceControlState->IsCheckedOutOther(&OtherCheckedOutUser))
		{
			FPackageSourceControlHelperLog::Error(bErrorsAsWarnings, FString::Printf(TEXT("File %s already checked out by %s, will not add"), *PackageFilename, *OtherCheckedOutUser));
			bSuccess = false;
		}
		else if (!SourceControlState->IsCurrent())
		{
			FPackageSourceControlHelperLog::Error(bErrorsAsWarnings, FString::Printf(TEXT("File %s (not at head revision), will not add"), *PackageFilename));
			bSuccess = false;
		}
		else if (SourceControlState->IsAdded())
		{
			// Nothing to do
		}
		else if (!SourceControlState->IsSourceControlled())
		{
			PackagesToAdd.Add(PackageFilename);
		}
	}

	// Any error up to here will be an early out
	if (!bSuccess && !bErrorsAsWarnings)
	{
		return false;
	}

	if (PackagesToAdd.Num())
	{
		if (GetSourceControlProvider().Execute(ISourceControlOperation::Create<FMarkForAdd>(), PackagesToAdd) != ECommandResult::Succeeded)
		{
			FPackageSourceControlHelperLog::Error(bErrorsAsWarnings, TEXT("Error adding packages to revision control"));
			return false;
		}
	}

	return bSuccess;
}

bool FPackageSourceControlHelper::Checkout(UPackage* Package) const
{
	return !Package || Checkout({ Package->GetName() });
}

bool FPackageSourceControlHelper::Checkout(const TArray<FString>& PackageNames, bool bErrorsAsWarnings) const
{
	if (PackageNames.IsEmpty())
	{
		return true;
	}

	const bool bUseSourceControl = UseSourceControl();

	// Convert package names to package filenames
	TArray<FString> PackageFilenames = SourceControlHelpers::PackageFilenames(PackageNames);

	// Two-pass checkout mechanism
	TArray<FString> PackagesToCheckout;
	TArray<FString> PackagesToRevert;
	PackagesToCheckout.Reserve(PackageFilenames.Num());
	bool bSuccess = true;

	// In the first pass, we will gather the packages to be checked out, or flag errors and return if we've found any
	if (bUseSourceControl)
	{
		TArray<FSourceControlStateRef> SourceControlStates;
		ECommandResult::Type UpdateState = GetSourceControlProvider().GetState(PackageFilenames, SourceControlStates, EStateCacheUsage::ForceUpdate);

		if (UpdateState != ECommandResult::Succeeded)
		{
			// Nothing we can do if SCCStates fail
			FPackageSourceControlHelperLog::Error(false, TEXT("Could not get revision control state for packages"));
			return false;
		}

		for(FSourceControlStateRef& SourceControlState : SourceControlStates)
		{
			const FString& PackageFilename = SourceControlState->GetFilename();

			FString OtherCheckedOutUser;
			if (SourceControlState->IsCheckedOutOther(&OtherCheckedOutUser))
			{
				FPackageSourceControlHelperLog::Error(bErrorsAsWarnings, FString::Printf(TEXT("File %s already checked out by %s, will not checkout"), *PackageFilename, *OtherCheckedOutUser));
				bSuccess = false;
			}
			else if (!SourceControlState->IsCurrent())
			{
				FPackageSourceControlHelperLog::Error(bErrorsAsWarnings, FString::Printf(TEXT("File %s (not at head revision), will not checkout"), *PackageFilename));
				bSuccess = false;
			}
			else if (SourceControlState->IsCheckedOut() || SourceControlState->IsAdded())
			{
				// Nothing to do
			}
			else if (SourceControlState->IsSourceControlled())
			{
				// Package needs to be checked out but if it is currently opened for delete, it needs to be reverted first.
				if (SourceControlState->IsDeleted())
				{
					PackagesToRevert.Add(PackageFilename);
				}

				PackagesToCheckout.Add(PackageFilename);
			}
		}
	}
	else
	{
		for (const FString& PackageFilename : PackageFilenames)
		{
			if (!IPlatformFile::GetPlatformPhysical().FileExists(*PackageFilename))
			{
				FPackageSourceControlHelperLog::Error(bErrorsAsWarnings, FString::Printf(TEXT("File %s cannot be checked out as it does not exist"), *PackageFilename));
				bSuccess = false;
			}
			else if (IPlatformFile::GetPlatformPhysical().IsReadOnly(*PackageFilename))
			{
				PackagesToCheckout.Add(PackageFilename);
			}
		}
	}

	// Any error up to here will be an early out
	if (!bSuccess && !bErrorsAsWarnings)
	{
		return false;
	}

	// In the second pass, we will perform the checkout operation
	if (PackagesToCheckout.Num() == 0)
	{
		return bSuccess;
	}
	else if (bUseSourceControl)
	{
		if (!PackagesToRevert.IsEmpty())
		{
			TSharedRef<FRevert, ESPMode::ThreadSafe> RevertOperation = ISourceControlOperation::Create<FRevert>();
			
			// Avoid losing local changes
			RevertOperation->SetSoftRevert(true);

			if (GetSourceControlProvider().Execute(RevertOperation, PackagesToRevert) != ECommandResult::Succeeded)
			{
				FPackageSourceControlHelperLog::Error(bErrorsAsWarnings, TEXT("Error reverting opened for delete file packages prior to checking out"));
				return false;
			}
		}

		if (GetSourceControlProvider().Execute(ISourceControlOperation::Create<FCheckOut>(), PackagesToCheckout) != ECommandResult::Succeeded)
		{
			// If operation didn't succeed. Get the list of invalid states when provided with a OutFailedPackages
			FPackageSourceControlHelperLog::Error(bErrorsAsWarnings, TEXT("Error checking out packages from revision control"));
			return false;
		}
	}
	else
	{
		int PackageIndex = 0;

		for (; PackageIndex < PackagesToCheckout.Num(); ++PackageIndex)
		{
			if (!IPlatformFile::GetPlatformPhysical().SetReadOnly(*PackagesToCheckout[PackageIndex], false))
			{
				FPackageSourceControlHelperLog::Error(bErrorsAsWarnings, FString::Printf(TEXT("Error setting %s writable"), *PackagesToCheckout[PackageIndex]));
				bSuccess = false;
				--PackageIndex;
				break;
			}
		}

		// If a file couldn't be made writeable, put back the files to their original state
		if (!bSuccess && !bErrorsAsWarnings)
		{
			for (; PackageIndex >= 0; --PackageIndex)
			{
				IPlatformFile::GetPlatformPhysical().SetReadOnly(*PackagesToCheckout[PackageIndex], true);
			}
		}
	}

	return bSuccess;
}

bool FPackageSourceControlHelper::GetDesiredStatesForModification(const TArray<FString>& PackageNames, TArray<FString>& OutPackagesToCheckout, TArray<FString>& OutPackagesToAdd, bool bErrorsAsWarnings) const
{
	// Convert package names to package filenames
	TMap<FString, FString> PackageFilenamesToPackageName;
	TArray<FString> PackageFilenames;
	PackageFilenames.Reserve(PackageNames.Num());

	for (const FString& PackageName : PackageNames)
	{
		FString PackageFilename = SourceControlHelpers::PackageFilename(PackageName);
		PackageFilenamesToPackageName.Add(PackageFilename, PackageName);
		PackageFilenames.Add(PackageFilename);
	}

	if (!UseSourceControl())
	{
		for (const FString& PackageFilename : PackageFilenames)
		{
			if (IPlatformFile::GetPlatformPhysical().FileExists(*PackageFilename) && IPlatformFile::GetPlatformPhysical().IsReadOnly(*PackageFilename))
			{
				OutPackagesToCheckout.Add(PackageFilenamesToPackageName.FindChecked(PackageFilename));
			}
		}
		return true;
	}
		
	TArray<FSourceControlStateRef> SourceControlStates;
	if (GetSourceControlProvider().GetState(PackageFilenames, SourceControlStates, EStateCacheUsage::ForceUpdate) != ECommandResult::Succeeded)
	{
		// Nothing we can do if SCCStates fail
		FPackageSourceControlHelperLog::Error(false, TEXT("Could not get revision control state for packages"));
		return false;
	}

	bool bSuccess = true;


	for (FSourceControlStateRef& SourceControlState : SourceControlStates)
	{
		const FString& PackageFilename = SourceControlState->GetFilename();

		FString OtherCheckedOutUser;
		if (SourceControlState->IsCheckedOutOther(&OtherCheckedOutUser))
		{
			FPackageSourceControlHelperLog::Error(bErrorsAsWarnings, FString::Printf(TEXT("File %s already checked out by %s, will not checkout"), *PackageFilename, *OtherCheckedOutUser));
			bSuccess = false;
		}
		else if (!SourceControlState->IsCurrent())
		{
			FPackageSourceControlHelperLog::Error(bErrorsAsWarnings, FString::Printf(TEXT("File %s (not at head revision), will not checkout"), *PackageFilename));
			bSuccess = false;
		}
		else if (SourceControlState->IsCheckedOut() || SourceControlState->IsAdded())
		{
			// Nothing to do
		}
		else if (SourceControlState->IsSourceControlled())
		{
			OutPackagesToCheckout.Add(PackageFilenamesToPackageName.FindChecked(PackageFilename));
		}
		else
		{
			OutPackagesToAdd.Add(PackageFilenamesToPackageName.FindChecked(PackageFilename));
		}
	}

	return bSuccess;
}

bool FPackageSourceControlHelper::GetMarkedForDeleteFiles(const TArray<FString>& Filenames, TArray<FString>& OutFilenames, bool bErrorsAsWarnings) const
{
	if (!UseSourceControl())
	{
		return true;
	}

	TArray<FSourceControlStateRef> SourceControlStates;
	if (GetSourceControlProvider().GetState(Filenames, SourceControlStates, EStateCacheUsage::ForceUpdate) != ECommandResult::Succeeded)
	{
		// Nothing we can do if SCCStates fail
		FPackageSourceControlHelperLog::Error(false, TEXT("Could not get revision control state for packages"));
		return false;
	}

	for (const FSourceControlStateRef& SourceControlState : SourceControlStates)
	{
		const FString& PackageFilename = SourceControlState->GetFilename();

		if (SourceControlState->IsDeleted())
		{
			OutFilenames.Add(PackageFilename);
		}
	}

	return true;
}

