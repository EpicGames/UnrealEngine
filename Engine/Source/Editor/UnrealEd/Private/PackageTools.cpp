// Copyright Epic Games, Inc. All Rights Reserved.


#include "PackageTools.h"
#include "PackageTools_Private.h"
#include "Algo/Transform.h"
#include "BlueprintCompilationManager.h"
#include "UObject/PackageReload.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "Misc/Guid.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/FeedbackContext.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/GarbageCollection.h"
#include "UObject/Package.h"
#include "UObject/MetaData.h"
#include "UObject/UObjectHash.h"
#include "UObject/GCObjectScopeGuard.h"
#include "Serialization/ArchiveFindCulprit.h"
#include "Misc/PackageName.h"
#include "Editor/Transactor.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "IAssetTools.h"
#include "ISourceControlOperation.h"
#include "SourceControlOperations.h"
#include "ISourceControlProvider.h"
#include "ISourceControlModule.h"
#include "SourceControlHelpers.h"
#include "Editor.h"
#include "Dialogs/Dialogs.h"

#include "ObjectTools.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/KismetReinstanceUtilities.h"
#include "BusyCursor.h"

#include "FileHelpers.h"

#include "Framework/Notifications/NotificationManager.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Notifications/SNotificationList.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ComponentReregisterContext.h"
#include "Engine/AssetManager.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/GameEngine.h"
#include "Engine/LevelStreaming.h"
#include "Templates/GuardValueAccessors.h"
#include "Engine/Selection.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Logging/MessageLog.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectIterator.h"

#include "AssetCompilingManager.h"

#include "ShaderCompiler.h"
#include "DistanceFieldAtlas.h"
#include "MeshCardBuild.h"
#include "MeshCardRepresentation.h"
#include "AssetToolsModule.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Misc/AutomationTest.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PackageTools)


#define LOCTEXT_NAMESPACE "PackageTools"

DEFINE_LOG_CATEGORY_STATIC(LogPackageTools, Log, All);

namespace UE::UnrealEd
{
namespace PackageTools_Private
{
static bool bUnloadPackagesUnloadsPrimaryAssets = true;
FAutoConsoleVariableRef CVarUnloadPackagesUnloadsPrimaryAssets(TEXT("PackageTools.UnloadPackagesUnloadsPrimaryAssets"), bUnloadPackagesUnloadsPrimaryAssets, TEXT("During unload packages, also unload primary assets"), ECVF_Default);

FOnUnloadPackagesDelegate OnUnloadPackagesDelegate;
}	 // namespace PackageTools_Private
}	 // namespace UE::UnrealEd

/** State passed to RestoreStandaloneOnReachableObjects. */
TSet<UPackage*>* UPackageTools::PackagesBeingUnloaded = nullptr;
TSet<UObject*> UPackageTools::ObjectsThatHadFlagsCleared;
FDelegateHandle UPackageTools::ReachabilityCallbackHandle;

void UPackageTools::FlushAsyncCompilation(TArrayView<UPackage* const> InPackages)
{
	TArray<UObject*> ObjectsToFinish;

	for (const UPackage* Package : InPackages)
	{
		ForEachObjectWithPackage(Package, [&ObjectsToFinish](UObject* Object)
		{
			if (const IInterface_AsyncCompilation* AsyncCompilationIF = Cast<IInterface_AsyncCompilation>(Object))
			{
				ObjectsToFinish.Add(Object);
			}

			return true;
		});
	}

	if (ObjectsToFinish.Num())
	{
		FAssetCompilingManager::Get().FinishCompilationForObjects(ObjectsToFinish);
	}
}

struct FPackageToolsCommands
{
	FPackageToolsCommands()
		: ReloadConsoleCommand(TEXT("PackageTools.ReloadPackage"),
							   TEXT("Force a reload of the named package, e.g. PackageTools.ReloadPackage /Game/MyAsset"),
							   FConsoleCommandWithArgsDelegate::CreateRaw(this, &FPackageToolsCommands::ReloadPackageCommand))
	{
	}

	void ReloadPackageCommand(const TArray<FString>& Params)
	{
		TArray<UPackage*> Packages;
		for (const FString& Param : Params)
		{
			UPackage* ExistingPackage = FindPackage(nullptr, *Param);
			if (ExistingPackage)
			{
				Packages.Add(ExistingPackage);
			}
		}
		if (Packages.Num() > 0)
		{
			UPackageTools::ReloadPackages(Packages);
		}
	}

	FAutoConsoleCommand ReloadConsoleCommand;
};

static FPackageToolsCommands PackageCommands;

UPackageTools::UPackageTools(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		FCoreUObjectDelegates::OnPackageReloaded.AddStatic(&UPackageTools::HandlePackageReloaded);
	}
}

	/**
	 * Called during GC, after reachability analysis is performed but before garbage is purged.
	 * Restores RF_Standalone to objects in the package-to-be-unloaded that are still reachable.
	 */
	void UPackageTools::RestoreStandaloneOnReachableObjects()
	{
		check(GIsEditor);
		TRACE_CPUPROFILER_EVENT_SCOPE(UPackageTools::RestoreStandaloneOnReachableObjects);

		if (PackagesBeingUnloaded && ObjectsThatHadFlagsCleared.Num() > 0)
		{
			for (UPackage* PackageBeingUnloaded : *PackagesBeingUnloaded)
			{
				ForEachObjectWithPackage(PackageBeingUnloaded, [](UObject* Object)
				{
					if (ObjectsThatHadFlagsCleared.Contains(Object))
					{
						Object->SetFlags(RF_Standalone);
					}
					return true;
				}, EGetObjectsFlags::IncludeNestedObjects, RF_NoFlags, EInternalObjectFlags::Unreachable);
			}
		}
	}

	/**
	 * Filters the global set of packages.
	 *
	 * @param	OutGroupPackages			The map that receives the filtered list of group packages.
	 * @param	OutPackageList				The array that will contain the list of filtered packages.
	 */
	void UPackageTools::GetFilteredPackageList(TSet<UPackage*>& OutFilteredPackageMap)
	{
		// The UObject list is iterated rather than the UPackage list because we need to be sure we are only adding
		// group packages that contain things the generic browser cares about.  The packages are derived by walking
		// the outer chain of each object.

		// Assemble a list of packages.  Only show packages that match the current resource type filter.
		for (UObject* Obj : TObjectRange<UObject>())
		{
			// This is here to hopefully catch a bit more info about a spurious in-the-wild problem which ultimately
			// crashes inside UObjectBaseUtility::GetOutermost(), which is called inside IsObjectBrowsable().
			checkf(Obj->IsValidLowLevel(), TEXT("GetFilteredPackageList: bad object found, address: %p, name: %s"), Obj, *Obj->GetName());

			// Make sure that we support displaying this object type
			bool bIsSupported = ObjectTools::IsObjectBrowsable( Obj );
			if( bIsSupported )
			{
				UPackage* ObjectPackage = Obj->GetOutermost();
				if( ObjectPackage != NULL )
				{
					OutFilteredPackageMap.Add( ObjectPackage );
				}
			}
		}
	}

	/**
	 * Fills the OutObjects list with all valid objects that are supported by the current
	 * browser settings and that reside withing the set of specified packages.
	 *
	 * @param	InPackages			Filters objects based on package.
	 * @param	OutObjects			[out] Receives the list of objects
	 * @param	bMustBeBrowsable	If specified, does a check to see if object is browsable. Defaults to true.
	 */
	void UPackageTools::GetObjectsInPackages( const TArray<UPackage*>* InPackages, TArray<UObject*>& OutObjects )
	{
		if (InPackages)
		{
			for (UPackage* Package : *InPackages)
			{
				ForEachObjectWithPackage(Package,[&OutObjects](UObject* Obj)
					{
						if (ObjectTools::IsObjectBrowsable(Obj))
						{
							OutObjects.Add(Obj);
						}
						return true;
					});
			}
		}
		else
		{
			for (TObjectIterator<UObject> It; It; ++It)
			{
				UObject* Obj = *It;

				if (ObjectTools::IsObjectBrowsable(Obj))
				{
					OutObjects.Add(Obj);
				}
			}
		}
	}

	bool UPackageTools::HandleFullyLoadingPackages( const TArray<UPackage*>& TopLevelPackages, const FText& OperationText )
	{
		bool bSuccessfullyCompleted = true;

		// whether or not to suppress the ask to fully load message
		bool bSuppress = GetDefault<UEditorPerProjectUserSettings>()->bSuppressFullyLoadPrompt;

		// Make sure they are all fully loaded.
		bool bNeedsUpdate = false;
		for( int32 PackageIndex=0; PackageIndex<TopLevelPackages.Num(); PackageIndex++ )
		{
			UPackage* TopLevelPackage = TopLevelPackages[PackageIndex];
			check( TopLevelPackage );
			check( TopLevelPackage->GetOuter() == NULL );

			// Calling IsFullyLoaded() will mark a package as fully loaded if it does not exist on disk
			if( !TopLevelPackage->IsFullyLoaded() )
			{	
				// Ask user to fully load or suppress the message and just fully load
				if(bSuppress || EAppReturnType::Yes == FMessageDialog::Open( EAppMsgType::YesNo, EAppReturnType::Yes, FText::Format(
					NSLOCTEXT("UnrealEd", "NeedsToFullyLoadPackageF", "Asset {0} is not fully loaded. Do you want to fully load it? Not doing so will abort the '{1}' operation."),
					FText::FromString(IAssetTools::Get().GetUserFacingLongPackageName(*TopLevelPackage)), OperationText)))
				{
					// Fully load package.
					const FScopedBusyCursor BusyCursor;
					GWarn->BeginSlowTask( NSLOCTEXT("UnrealEd", "FullyLoadingPackages", "Fully loading assets"), true );
					TopLevelPackage->FullyLoad();
					GWarn->EndSlowTask();
					bNeedsUpdate = true;
				}
				// User declined abort operation.
				else
				{
					bSuccessfullyCompleted = false;
					UE_LOGF(LogPackageTools, Log, "Aborting operation as %ls was not fully loaded.",*TopLevelPackage->GetName());
					break;
				}
			}
		}

		// no need to refresh content browser here as UPackage::FullyLoad() already does this
		return bSuccessfullyCompleted;
	}
	
	/**
	 * Loads the specified package file (or returns an existing package if it's already loaded.)
	 *
	 * @param	InFilename	File name of package to load
	 * @param	InLoadFlags	Flags controlling loading behavior, from the ELoadFlags enum
	 *
	 * @return	The loaded package (or NULL if something went wrong.)
	 */
	UPackage* UPackageTools::LoadPackage( FString InFilename, uint32 InLoadFlags )
	{
		// Detach all components while loading a package.
		// This is necessary for the cases where the load replaces existing objects which may be referenced by the attached components.
		FGlobalComponentReregisterContext ReregisterContext;

		// record the name of this file to make sure we load objects in this package on top of in-memory objects in this package
		GEditor->UserOpenedFile = InFilename;

		// clear any previous load errors
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("PackageName"), FText::FromString(InFilename));
		FMessageLog("LoadErrors").NewPage(FText::Format(LOCTEXT("LoadPackageLogPage", "Loading package: {PackageName}"), Arguments));

		UPackage* Package = ::LoadPackage( NULL, *InFilename, InLoadFlags );

		// display any load errors that happened while loading the package
		FEditorDelegates::DisplayLoadErrors.Broadcast();

		// reset the opened package to nothing
		GEditor->UserOpenedFile = FString();

		// If a script package was loaded, update the
		// actor browser in case a script package was loaded
		if ( Package != NULL )
		{
			if (Package->HasAnyPackageFlags(PKG_ContainsScript))
			{
				GEditor->BroadcastClassPackageLoadedOrUnloaded();
			}
		}

		return Package;
	}


	bool UPackageTools::UnloadPackages( const TArray<UPackage*>& TopLevelPackages )
	{
		FText ErrorMessage;
		const bool bResult = UnloadPackages(TopLevelPackages, ErrorMessage);
		if(!ErrorMessage.IsEmpty())
		{
			FMessageDialog::Open( EAppMsgType::Ok, ErrorMessage );
		}

		return bResult;
	}

	bool UPackageTools::UnloadPackages(const TArray<UPackage*>& TopLevelPackages, FText& OutErrorMessage, bool bUnloadDirtyPackages)
	{
		FUnloadPackageParams Params(TopLevelPackages);
		Params.bUnloadDirtyPackages = bUnloadDirtyPackages;
		const bool bResult = UnloadPackages(Params);
		OutErrorMessage = Params.OutErrorMessage;
		return bResult;
	}

	bool UPackageTools::UnloadPackages(FUnloadPackageParams& Params)
	{
		// Early out if no package is provided
		if (Params.Packages.IsEmpty())
		{
			return true;
		}

		bool bResult = false;

		// Get outermost packages, in case groups were selected.
		TSet<UPackage*> PackagesToUnload;

		// Split the set of selected top level packages into packages which are dirty (and thus cannot be unloaded)
		// and packages that are not dirty (and thus can be unloaded).
		TArray<UPackage*> DirtyPackages;
		for (UPackage* TopLevelPackage : Params.Packages)
		{
			if (TopLevelPackage)
			{
				if (!Params.bUnloadDirtyPackages && TopLevelPackage->IsDirty())
				{
					DirtyPackages.Add(TopLevelPackage);
				}
				else
				{
					UPackage* PackageToUnload = TopLevelPackage->GetOutermost();
					if (!PackageToUnload)
					{
						PackageToUnload = TopLevelPackage;
					}
					PackagesToUnload.Add(PackageToUnload);
				}
			}
		}

		// Inform the user that dirty packages won't be unloaded.
		if ( DirtyPackages.Num() > 0 )
		{
			FString DirtyPackagesList;
			for (const FString& UserFacingPath : IAssetTools::Get().GetUserFacingLongPackageNames(DirtyPackages))
			{
				DirtyPackagesList += FString::Printf(TEXT("\n    %s"), *UserFacingPath);
			}

			FFormatNamedArguments Args;
			Args.Add(TEXT("DirtyPackages"), FText::FromString(DirtyPackagesList));

			Params.OutErrorMessage = FText::Format( NSLOCTEXT("UnrealEd", "UnloadDirtyPackagesList", "The following assets have been modified and cannot be unloaded:{DirtyPackages}\nSaving these assets will allow them to be unloaded."), Args );
		}
		if (GEditor)
		{
			if (UWorld* EditorWorld = GEditor->GetEditorWorldContext().World())
			{
				// Is the current world being unloaded?
				if (PackagesToUnload.Contains(EditorWorld->GetPackage()))
				{
					TArray<TWeakObjectPtr<UPackage>> WeakPackages;
					WeakPackages.Reserve(PackagesToUnload.Num());
					for (UPackage* Package : PackagesToUnload)
					{
						WeakPackages.Add(Package);
					}

					// Unload the current world
					GEditor->CreateNewMapForEditing();

					// Remove stale entries in PackagesToUnload (unloaded world, level build data, streaming levels, external actors, etc)
					PackagesToUnload.Reset();
					for (const TWeakObjectPtr<UPackage>& WeakPackage : WeakPackages)
					{
						if (UPackage* Package = WeakPackage.Get())
						{
							PackagesToUnload.Add(Package);
						}
					}
				}
			}
		}

		if (PackagesToUnload.Num() > 0 && GEditor)
		{
			const FScopedBusyCursor BusyCursor;

			// Complete any load/streaming requests, then lock IO.
			FlushAsyncLoading();
			(*GFlushStreamingFunc)();

			GWarn->BeginSlowTask(NSLOCTEXT("UnrealEd", "Unloading", "Unloading"), true);

			// Remove potential references to to-be deleted objects from the GB selection set.
			GEditor->GetSelectedObjects()->GetElementSelectionSet()->ClearSelection(FTypedElementSelectionOptions());

			// Clear undo history because transaction records can hold onto assets we want to unload
			if (GEditor->Trans && Params.bResetTransBuffer)
			{
				GEditor->Trans->Reset(NSLOCTEXT("UnrealEd", "UnloadPackagesResetUndo", "Unload Assets"));
			}

			// First add all packages to unload to the root set so they don't get garbage collected while we are operating on them
			TArray<UPackage*> PackagesAddedToRoot;
			for (UPackage* PackageToUnload : PackagesToUnload)
			{
				if (!PackageToUnload->IsRooted())
				{
					PackageToUnload->AddToRoot();
					PackagesAddedToRoot.Add(PackageToUnload);
				}
			}

			// We need to make sure that there is no async compilation work running for the packages that we are about to unload
			// so that it is safe to call ::ResetLoaders
			FlushAsyncCompilation(PackagesToUnload.Array());

			// Before we clean up the assets in the packages that are about to be unloaded, we give a chance for any registered observers
			// to clean up any state that they need to for the assets in those packages.
			UE::UnrealEd::PackageTools_Private::OnUnloadPackagesDelegate.Broadcast(PackagesToUnload);

			// Now try to clean up assets in all packages to unload.
			bool bScriptPackageWasUnloaded = false;
			int32 PackageIndex = 0;
			for (UPackage* PackageBeingUnloaded : PackagesToUnload)
			{
				GWarn->StatusUpdate(PackageIndex++, PackagesToUnload.Num(), FText::Format(NSLOCTEXT("UnrealEd", "Unloadingf", "Unloading {0}..."), FText::FromString(IAssetTools::Get().GetUserFacingLongPackageName(*PackageBeingUnloaded)) ) );

				// Flush all pending render commands, as unloading the package may invalidate render resources.
				FlushRenderingCommands();

				TArray<UObject*> ObjectsInPackage;

				// Can't use ForEachObjectWithPackage here as closing the editor may modify UObject hash tables (known case: renaming objects)
				GetObjectsWithPackage(PackageBeingUnloaded, ObjectsInPackage, EGetObjectsFlags::None);
				// Close any open asset editors.
				for (UObject* Obj : ObjectsInPackage)
				{
					if (Obj->IsAsset())
					{
						GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(Obj);
					}
				}
				ObjectsInPackage.Reset();

				PackageBeingUnloaded->MarkAsUnloaded();
				if ( PackageBeingUnloaded->HasAnyPackageFlags(PKG_ContainsScript) )
				{
					bScriptPackageWasUnloaded = true;
				}

				GetObjectsWithPackage(PackageBeingUnloaded, ObjectsInPackage, EGetObjectsFlags::IncludeNestedObjects, RF_Transient, EInternalObjectFlags::Garbage);
				// Notify any Blueprints and other systems that are about to be unloaded, also destroy any leftover worlds.
				FCoreDelegates::CleanupUnloadingObjects.Broadcast(ObjectsInPackage);
				for (UObject* Obj : ObjectsInPackage)
				{
					// Asset manager can hold hard references to this object and prevent GC
					if (UE::UnrealEd::PackageTools_Private::bUnloadPackagesUnloadsPrimaryAssets)
					{
						const FPrimaryAssetId PrimaryAssetId = UAssetManager::Get().GetPrimaryAssetIdForObject(Obj);
						if (PrimaryAssetId.IsValid())
						{
							UAssetManager::Get().UnloadPrimaryAsset(PrimaryAssetId);
						}
					}

					if (UBlueprint* BP = Cast<UBlueprint>(Obj))
					{
						BP->ClearEditorReferences();

						// Remove from cached dependent lists.
						for (const TWeakObjectPtr<UBlueprint> Dependency : BP->CachedDependencies)
						{
							if (UBlueprint* ResolvedDependency = Dependency.Get())
							{
								ResolvedDependency->CachedDependents.Remove(BP);
							}
						}

						BP->CachedDependencies.Reset();

						// Remove from cached dependency lists.
						for (const TWeakObjectPtr<UBlueprint> Dependent : BP->CachedDependents)
						{
							if (UBlueprint* ResolvedDependent = Dependent.Get())
							{
								ResolvedDependent->CachedDependencies.Remove(BP);
							}
						}

						BP->CachedDependents.Reset();
					}
					else if (UBlueprintGeneratedClass* BPGC = Cast<UBlueprintGeneratedClass>(Obj))
					{
						FKismetEditorUtilities::OnBlueprintGeneratedClassUnloaded.Broadcast(BPGC);
					}
					else if (UWorld* World = Cast<UWorld>(Obj))
					{
						if (World->bIsWorldInitialized)
						{
							World->CleanupWorld();
						}
					}
				}
				ObjectsInPackage.Reset();

				// Clear RF_Standalone flag from objects in the package to be unloaded so they get GC'd.
				{
					GetObjectsWithPackage(PackageBeingUnloaded, ObjectsInPackage);
					for ( UObject* Object : ObjectsInPackage )
					{
						if (Object->HasAnyFlags(RF_Standalone))
						{
							Object->ClearFlags(RF_Standalone);
							ObjectsThatHadFlagsCleared.Add(Object);
						}
					}
					ObjectsInPackage.Reset();
				}

				// Cleanup.
				bResult = true;
			}

			// Calling ::ResetLoaders now will force any bulkdata objects still attached to the FLinkerLoad to load
			// their payloads into memory. If we don't call this now, then the version that will be called during
			// garbage collection will cause the bulkdata objects to be invalidated rather than loading the payloads 
			// into memory.
			// This might seem odd, but if the package we are unloading is being renamed, then the inner UObjects will
			// be moved to the newly named package rather than being garbage collected and so we need to make sure that
			// their bulkdata objects remain valid, otherwise renamed packages will not save correctly and cease to function.
			ResetLoaders(TArray<UObject*>(PackagesToUnload.Array()));

			for (UPackage* PackageBeingUnloaded : PackagesToUnload)
			{
				if (PackageBeingUnloaded->IsDirty())
				{
					// The package was marked dirty as a result of something that happened above (e.g callbacks in CollectGarbage).  
					// Dirty packages we actually care about unloading were filtered above so if the package becomes dirty here it should still be unloaded
					PackageBeingUnloaded->SetDirtyFlag(false);
				}
			}

			// Set the callback for restoring RF_Standalone post reachability analysis.
			// GC will call this function before purging objects, allowing us to restore RF_Standalone
			// to any objects that have not been marked RF_Unreachable.
			ReachabilityCallbackHandle = FCoreUObjectDelegates::PostReachabilityAnalysis.AddStatic(RestoreStandaloneOnReachableObjects);

			PackagesBeingUnloaded = &PackagesToUnload;

			// Collect garbage.
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

			ObjectsThatHadFlagsCleared.Empty();
			PackagesBeingUnloaded = nullptr;
			
			// Now remove from root all the packages we added earlier so they may be GCed if possible
			for (UPackage* PackageAddedToRoot : PackagesAddedToRoot)
			{
				PackageAddedToRoot->RemoveFromRoot();
			}
			PackagesAddedToRoot.Empty();

			GWarn->EndSlowTask();

			// Remove the post reachability callback.
			FCoreUObjectDelegates::PostReachabilityAnalysis.Remove(ReachabilityCallbackHandle);

			CollectGarbage( GARBAGE_COLLECTION_KEEPFLAGS );

			// Update the actor browser if a script package was unloaded
			if ( bScriptPackageWasUnloaded )
			{
				GEditor->BroadcastClassPackageLoadedOrUnloaded();
			}
		}
		return bResult;
	}


	bool UPackageTools::ReloadPackages( const TArray<UPackage*>& TopLevelPackages )
	{
		FText ErrorMessage;
		const bool bResult = ReloadPackages(TopLevelPackages, ErrorMessage, EReloadPackagesInteractionMode::Interactive);

		if (!ErrorMessage.IsEmpty())
		{
			FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage);
		}

		return bResult;
	}


	bool UPackageTools::ReloadPackages( const TArray<UPackage*>& TopLevelPackages, FText& OutErrorMessage, const bool bInteractive )
	{
		return ReloadPackages(TopLevelPackages, OutErrorMessage, bInteractive ? EReloadPackagesInteractionMode::Interactive : EReloadPackagesInteractionMode::AssumeNegative);
	}


	namespace UE::PackageTools::Private
	{
	struct FFilteredPackages
	{
		void Add(UPackage* RealPackage)
		{
			if (RealPackage->HasAnyPackageFlags(PKG_NewlyCreated))
			{
				// Newly created packages *may* exist on disk (eg, deleting an actor, submitting, and then reverting the delete)
				// so in that case we want to allow them to reload via PackagesToReload rather than block them via InMemoryPackages
				if (FPackageName::DoesPackageExist(RealPackage->GetName()))
				{
					PackagesToReload.AddUnique(RealPackage);
				}
				else
				{
					InMemoryPackages.AddUnique(RealPackage);
				}
			}
			else if (RealPackage->HasAnyPackageFlags(PKG_InMemoryOnly))
			{
				InMemoryPackages.AddUnique(RealPackage);
			}
			else if (RealPackage->IsDirty())
			{
				DirtyPackages.AddUnique(RealPackage);
			}
			else
			{
				PackagesToReload.AddUnique(RealPackage);
			}
		}

		void Remove(UPackage* InPackage)
		{
			PackagesToReload.Remove(InPackage);
		}

		bool Contains(UPackage* InPackage) const
		{
			return PackagesToReload.Contains(InPackage);
		}

		TArray<UPackage*> PackagesToReload;
		TArray<UPackage*> DirtyPackages;
		TArray<UPackage*> InMemoryPackages;
	};

	FFilteredPackages GetPackagesToReload(const TArray<UPackage*>& TopLevelPackages)
	{
		FFilteredPackages Filtered;
		Algo::TransformIf(TopLevelPackages, Filtered,
						  [](UPackage* TopLevelPackage) {return TopLevelPackage;},
						  [](UPackage* TopLevelPackage)
						  {
							  return TopLevelPackage;
						  });
		return Filtered;
	}

	void PromptUserForDirtyPackages(FFilteredPackages& Filtered, const EReloadPackagesInteractionMode InteractionMode)
	{
		// How should we handle locally dirty packages?
		if (Filtered.DirtyPackages.Num() == 0)
		{
			return;
		}

		EAppReturnType::Type ReloadDirtyPackagesResult = EAppReturnType::No;

		// Ask the user whether dirty packages should be reloaded.
		if (InteractionMode == EReloadPackagesInteractionMode::Interactive)
		{
			FTextBuilder ReloadDirtyPackagesMsgBuilder;
			ReloadDirtyPackagesMsgBuilder.AppendLine(NSLOCTEXT("UnrealEd", "ShouldReloadDirtyPackagesHeader", "The following assets have been modified:"));
			{
				ReloadDirtyPackagesMsgBuilder.Indent();
				for (const FString& UserFacingPath : IAssetTools::Get().GetUserFacingLongPackageNames(Filtered.DirtyPackages))
				{
					ReloadDirtyPackagesMsgBuilder.AppendLine(UserFacingPath);
				}
				ReloadDirtyPackagesMsgBuilder.Unindent();
			}
			ReloadDirtyPackagesMsgBuilder.AppendLine(NSLOCTEXT("UnrealEd", "ShouldReloadDirtyPackagesFooter", "Would you like to reload these assets? This will revert any changes you have made."));

			ReloadDirtyPackagesResult = FMessageDialog::Open(EAppMsgType::YesNo, ReloadDirtyPackagesMsgBuilder.ToText());
		}
		else if (InteractionMode == EReloadPackagesInteractionMode::AssumePositive)
		{
			ReloadDirtyPackagesResult = EAppReturnType::Yes;
		}

		if (ReloadDirtyPackagesResult == EAppReturnType::Yes)
		{
			for (UPackage* DirtyPackage : Filtered.DirtyPackages)
			{
				DirtyPackage->SetDirtyFlag(false);
				Filtered.PackagesToReload.AddUnique(DirtyPackage);
			}
			Filtered.DirtyPackages.Reset();
		}
	}

	void InformUserAboutDirtyPackages(const FFilteredPackages& Filtered, FTextBuilder& ErrorMessageBuilder)
	{
		if (Filtered.DirtyPackages.Num() == 0)
		{
			return;
		}
		// Inform the user that dirty packages won't be reloaded.
		if (!ErrorMessageBuilder.IsEmpty())
		{
			ErrorMessageBuilder.AppendLine();
		}

		ErrorMessageBuilder.AppendLine(NSLOCTEXT("UnrealEd", "Error_ReloadDirtyPackagesHeader", "The following assets have been modified and cannot be reloaded:"));
		{
			ErrorMessageBuilder.Indent();
			for (const FString& UserFacingPath : IAssetTools::Get().GetUserFacingLongPackageNames(Filtered.DirtyPackages))
			{
				ErrorMessageBuilder.AppendLine(UserFacingPath);
			}
			ErrorMessageBuilder.Unindent();
		}
		ErrorMessageBuilder.AppendLine(NSLOCTEXT("UnrealEd", "Error_ReloadDirtyPackagesFooter", "Saving these assets will allow them to be reloaded."));
	}

	void InformUsersAboutInMemoryPackages(const FFilteredPackages& Filtered, FTextBuilder& ErrorMessageBuilder)
	{
		if (Filtered.InMemoryPackages.Num() == 0)
		{
			return;
		}
		if (!ErrorMessageBuilder.IsEmpty())
		{
			ErrorMessageBuilder.AppendLine();
		}
		ErrorMessageBuilder.AppendLine(NSLOCTEXT("UnrealEd", "Error_ReloadInMemoryPackagesHeader", "The following assets are in-memory only and cannot be reloaded:"));
		{
			ErrorMessageBuilder.Indent();
			for (const FString& UserFacingPath : IAssetTools::Get().GetUserFacingLongPackageNames(Filtered.InMemoryPackages))
			{
				ErrorMessageBuilder.AppendLine(UserFacingPath);
			}
			ErrorMessageBuilder.Unindent();
		}
	}

	struct FScopedTrackFilteredPackages
	{
		FScopedTrackFilteredPackages(FFilteredPackages& InFiltered) :
			Filtered(InFiltered)
		{
			Algo::Transform(Filtered.PackagesToReload, WeakPackagesToReload, [](UPackage* InPackage) -> TWeakObjectPtr<UPackage> { return InPackage; });
		}

		~FScopedTrackFilteredPackages()
		{
			Filtered.PackagesToReload.Reset();
			Algo::TransformIf(WeakPackagesToReload, Filtered.PackagesToReload,
							  [](TWeakObjectPtr<UPackage> InPackage) {return InPackage.IsValid();},
							  [](TWeakObjectPtr<UPackage> InPackage) -> UPackage* {return InPackage.Get();});

		}

		TArray<TWeakObjectPtr<UPackage>> WeakPackagesToReload;
		FFilteredPackages& Filtered;
	};

	TWeakObjectPtr<UWorld> GetCurrentWorld()
	{
		if (GIsEditor)
		{
			if (UWorld* EditorWorld = GEditor->GetEditorWorldContext().World())
			{
				return EditorWorld;
			}
		}
		else if (UGameEngine* GameEngine = Cast<UGameEngine>(GEngine))
		{
			if (UWorld* GameWorld = GameEngine->GetGameWorld())
			{
				return GameWorld;
			}
		}
		return nullptr;
	}

	}

	bool UPackageTools::ReloadPackages( const TArray<UPackage*>& TopLevelPackages, FText& OutErrorMessage, const EReloadPackagesInteractionMode InteractionMode )
	{
		bool bResult = false;

		TGuardValueAccessors<bool> IsEditorLoadingPackageGuard(UE::GetIsEditorLoadingPackage, UE::SetIsEditorLoadingPackage, true);

		FTextBuilder ErrorMessageBuilder;

		using namespace UE::PackageTools::Private;
		FFilteredPackages Filtered = GetPackagesToReload(TopLevelPackages);

		PromptUserForDirtyPackages(Filtered, InteractionMode);
		InformUserAboutDirtyPackages(Filtered, ErrorMessageBuilder);
		InformUsersAboutInMemoryPackages(Filtered, ErrorMessageBuilder);

		TWeakObjectPtr<UWorld> CurrentWorld = GetCurrentWorld();

		// Check to see if we need to reload the current world.
		FName WorldNameToReload;
		FName CurrentWorldPackageName;
		TArray<ULevelStreaming*> RemovedStreamingLevels;
		if (UWorld* CurrentWorldPtr = CurrentWorld.Get())
		{
			FScopedSlowTask UnloadingWorldSlowTask(1.f, NSLOCTEXT("UnrealEd", "UnloadingWorld", "Unloading World"));
			UnloadingWorldSlowTask.ForceRefresh();
		
			// Is the current world being reloaded? If so, we just reset the current world and load it again at the end rather than let it go through ReloadPackage 
			// (which doesn't work for the current world due to some assumptions about worlds, and their lifetimes).
			// We also need to skip the build data package as that will also be destroyed by the transition.
			if (Filtered.Contains(CurrentWorldPtr->GetOutermost()))
			{
				// Cache this so we can reload the world later
				WorldNameToReload = *CurrentWorldPtr->GetPathName();
				CurrentWorldPackageName = CurrentWorldPtr->GetPackage()->GetFName();

				// Remove the world package from the reload list
				Filtered.Remove(CurrentWorldPtr->GetOutermost());

				// Unload the current world
				if (GIsEditor)
				{
					FScopedTrackFilteredPackages TrackPackages(Filtered);
					const bool bPromptForSave = InteractionMode == UPackageTools::EReloadPackagesInteractionMode::Interactive;
					GEditor->CreateNewMapForEditing(bPromptForSave);
				}
				else if (UGameEngine* GameEngine = Cast<UGameEngine>(GEngine))
				{
					// Outside of the editor we need to keep the packages alive to stop the world transition from GC'ing them
					TGCObjectsScopeGuard<UPackage> KeepPackagesAlive(Filtered.PackagesToReload);

					FString LoadMapError;
					GameEngine->LoadMap(GameEngine->GetWorldContextFromWorldChecked(CurrentWorldPtr), FURL(TEXT("/Engine/Maps/Templates/Template_Default")), nullptr, LoadMapError);
				}
			}
			// Cache the current map build data for the levels of the current world so we can see if they change due to a reload (we can skip this if reloading the current world).
			else
			{
				const TArray<ULevel*>& Levels = CurrentWorldPtr->GetLevels();
				for (int32 i = Levels.Num() - 1; i >= 0; --i)
				{
					ULevel* Level = Levels[i];

					Level->ReleaseRenderingResources();

					if (Filtered.Contains(Level->GetOutermost()))
					{
						for (ULevelStreaming* StreamingLevel : CurrentWorldPtr->GetStreamingLevels())
						{
							if (StreamingLevel->GetLoadedLevel() == Level)
							{
								CurrentWorldPtr->RemoveFromWorld(Level);
								StreamingLevel->RemoveLevelFromCollectionForReload();
								ULevelStreaming::RemoveLevelAnnotation(Level);
								RemovedStreamingLevels.Add(StreamingLevel);
								break;
							}
						}
					}
				}
			}
		}

		TArray<UPackage*>& PackagesToReload = Filtered.PackagesToReload;
		if (PackagesToReload.Num() > 0)
		{
			const FScopedBusyCursor BusyCursor;

			// We need to sort the packages to reload so that dependencies are reloaded before the assets that depend on them
			::SortPackagesForReload(PackagesToReload);

			// Remove potential references to to-be deleted objects from the global selection sets.
			if (GIsEditor)
			{
				GEditor->ResetAllSelectionSets();
			}
			// Detach all components while loading a package.
			// This is necessary for the cases where the load replaces existing objects which may be referenced by the attached components.
			FGlobalComponentReregisterContext ReregisterContext;

			bool bScriptPackageWasReloaded = false;
			TArray<FReloadPackageData> PackagesToReloadData;
			PackagesToReloadData.Reserve(PackagesToReload.Num());
			for (UPackage* PackageToReload : PackagesToReload)
			{
				bScriptPackageWasReloaded |= PackageToReload->HasAnyPackageFlags(PKG_ContainsScript);
				PackagesToReloadData.Emplace(PackageToReload, LOAD_None);
			}

			TArray<UPackage*> ReloadedPackages;
			::ReloadPackages(PackagesToReloadData, ReloadedPackages, 500);

			TArray<UPackage*> FailedPackages;
			for (int32 PackageIndex = 0; PackageIndex < PackagesToReload.Num(); ++PackageIndex)
			{
				UPackage* ExistingPackage = PackagesToReload[PackageIndex];
				UPackage* ReloadedPackage = ReloadedPackages[PackageIndex];

				if (ReloadedPackage)
				{
					bScriptPackageWasReloaded |= ReloadedPackage->HasAnyPackageFlags(PKG_ContainsScript);
					bResult = true;
				}
				else
				{
					FailedPackages.Add(ExistingPackage);
				}
			}

			// Inform the user of any packages that failed to reload.
			if (FailedPackages.Num() > 0)
			{
				if (!ErrorMessageBuilder.IsEmpty())
				{
					ErrorMessageBuilder.AppendLine();
				}

				ErrorMessageBuilder.AppendLine(NSLOCTEXT("UnrealEd", "Error_ReloadFailedPackagesHeader", "The following assets failed to reload:"));
				{
					ErrorMessageBuilder.Indent();
					for (const FString& UserFacingPath : IAssetTools::Get().GetUserFacingLongPackageNames(FailedPackages))
					{
						ErrorMessageBuilder.AppendLine(UserFacingPath);
					}
					ErrorMessageBuilder.Unindent();
				}
			}

			// Update the actor browser if a script package was reloaded.
			if (GIsEditor && bScriptPackageWasReloaded)
			{
				GEditor->BroadcastClassPackageLoadedOrUnloaded();
			}
		}

		// Load the previous world (if needed).
		if (!WorldNameToReload.IsNone())
		{
			FScopedSlowTask ReloadingWorldSlowTask(1.f, NSLOCTEXT("UnrealEd", "ReloadingWorld", "Reloading World"));
			ReloadingWorldSlowTask.ForceRefresh();

			if (GIsEditor)
			{
				UWorld::WorldTypePreLoadMap.FindOrAdd(CurrentWorldPackageName) = EWorldType::Editor;

				TArray<FName> WorldNamesToReload;
				WorldNamesToReload.Add(WorldNameToReload);
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorsForAssets(WorldNamesToReload);

				UWorld::WorldTypePreLoadMap.Remove(CurrentWorldPackageName);
			}
			else if (UGameEngine* GameEngine = Cast<UGameEngine>(GEngine))
			{
				FString LoadMapError;
				// FURL requires a package name and not an asset path
				FString WorldPackage = FPackageName::ObjectPathToPackageName(WorldNameToReload.ToString());
				GameEngine->LoadMap(GameEngine->GetWorldContextFromWorldChecked(GameEngine->GetGameWorld()), FURL(*WorldPackage), nullptr, LoadMapError);
			}
		}
		// Update the rendering resources for the levels of the current world if their map build data has changed (we skip this if reloading the current world).
		else
		{
			UWorld* CurrentWorldPtr = CurrentWorld.Get();
			check(CurrentWorldPtr);

			if (RemovedStreamingLevels.Num() > 0)
			{
				for (ULevelStreaming* StreamingLevel : RemovedStreamingLevels)
				{
					ULevel* NewLevel = StreamingLevel->GetLoadedLevel();
					ULevelStreaming::LevelAnnotations.AddAnnotation(
						NewLevel,
						ULevelStreaming::FLevelAnnotation(StreamingLevel));
					CurrentWorldPtr->AddToWorld(NewLevel, StreamingLevel->LevelTransform, false);
					StreamingLevel->AddLevelToCollectionAfterReload();
				}
			}

			CurrentWorldPtr->InvalidateModelGeometry(nullptr);

			const TArray<ULevel*>& Levels = CurrentWorldPtr->GetLevels();
			for (ULevel* Level : Levels)
			{
				Level->UpdateModelComponents();
			}
			
			CurrentWorldPtr->PropagateLightingScenarioChange();
		}

		OutErrorMessage = ErrorMessageBuilder.ToText();

		return bResult;
	}


	void UPackageTools::HandlePackageReloaded(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent)
	{
		static TSet<UBlueprint*> BlueprintsToRecompileThisBatch;

		if (InPackageReloadPhase == EPackageReloadPhase::PrePackageFixup)
		{
			GEngine->NotifyToolsOfObjectReplacement(InPackageReloadedEvent->GetRepointedObjects());

			// Notify any Blueprints that are about to be unloaded, and destroy any leftover worlds.
			TArray<UObject*> Objects;
			GetObjectsWithPackage(InPackageReloadedEvent->GetOldPackage(), Objects, EGetObjectsFlags::IncludeNestedObjects, RF_Transient, EInternalObjectFlags::Garbage);
			for (UObject* InObject : Objects)
			{
				if (UBlueprint* BP = Cast<UBlueprint>(InObject))
				{
					BP->ClearEditorReferences();

					// Remove from cached dependent lists; this will be repopulated on reload, but we
					// don't wish to consider every one of our dependencies as a potential referencer,
					// and this set will be serialized by the archiver that's used to find those. Any
					// references will instead be collected from other fields that reference the asset.
					for (const TWeakObjectPtr<UBlueprint> Dependency : BP->CachedDependencies)
					{
						if (UBlueprint* ResolvedDependency = Dependency.Get())
						{
							ResolvedDependency->CachedDependents.Remove(BP);
						}
					}
				}
				if (UWorld* World = Cast<UWorld>(InObject))
				{
					if (World->bIsWorldInitialized)
					{
						World->CleanupWorld();
					}
				}
			}
		}

		if (InPackageReloadPhase == EPackageReloadPhase::OnPackageFixup)
		{
			TMap<UClass*, UClass*> OldClassToNewClass;
			for (const auto& RepointedObjectPair : InPackageReloadedEvent->GetRepointedObjects())
			{
				UObject* OldObject = RepointedObjectPair.Key;
				UObject* NewObject = RepointedObjectPair.Value;

				if(OldObject && NewObject)
				{
					// Only the blueprint generated class are supported by the FBlueprintCompilationManager so we only reparent those
					UClass* OldObjectAsClass = Cast<UBlueprintGeneratedClass>(OldObject);
					if(OldObjectAsClass)
					{
						UClass* NewObjectAsClass = Cast<UClass>(NewObject);
						if(ensureMsgf(NewObjectAsClass, TEXT("Class object replaced with non-class object: %s %s"), *(OldObject->GetName()), *(NewObject->GetName())))
						{
							OldClassToNewClass.Add(OldObjectAsClass, NewObjectAsClass);
						}
					}
				}
			}

			FBlueprintCompilationManager::ReparentHierarchies(OldClassToNewClass);

			for (const auto& RepointedObjectPair : InPackageReloadedEvent->GetRepointedObjects())
			{
				UObject* OldObject = RepointedObjectPair.Key;
				UObject* NewObject = RepointedObjectPair.Value;

				if (OldObject && OldObject->IsAsset())
				{
					if (const UBlueprint* OldBlueprint = Cast<UBlueprint>(OldObject))
					{
						if (NewObject && CastChecked<UBlueprint>(NewObject)->GeneratedClass && OldBlueprint->GeneratedClass)
						{
							// Don't change the class on instances that are being thrown away by the reload code. If we update
							// the class and recompile the old class ::ReplaceInstancesOfClass will experience some crosstalk 
							// with the compiler (both trying to create objects of the same class in the same location):
							TArray<UObject*> OldInstances;
							GetObjectsOfClass(OldBlueprint->GeneratedClass, OldInstances, false);
							OldInstances.RemoveAllSwap(
								[](UObject* Obj){ return !Obj->HasAnyFlags(RF_NewerVersionExists); }
							);

							TSet<UObject*> InstancesToLeaveAlone(OldInstances);
							FReplaceInstancesOfClassParameters ReplaceInstancesParameters;
							ReplaceInstancesParameters.InstancesThatShouldUseOldClass = &InstancesToLeaveAlone;
							FBlueprintCompileReinstancer::ReplaceInstancesOfClass(OldBlueprint->GeneratedClass, CastChecked<UBlueprint>(NewObject)->GeneratedClass, ReplaceInstancesParameters);
						}
						else
						{
							// we failed to load the UBlueprint and/or it's GeneratedClass. Show a notification indicating that maps may need to be reloaded:
							FNotificationInfo Warning(
								FText::Format(
									NSLOCTEXT("UnrealEd", "Warning_FailedToLoadParentClass", "Failed to load ParentClass for {0}"),
									FText::FromName(OldObject->GetFName())
								)
							);
							Warning.ExpireDuration = 3.0f;
							FSlateNotificationManager::Get().AddNotification(Warning);
						}
					}
				}
			}
		}

		if (InPackageReloadPhase == EPackageReloadPhase::PostPackageFixup)
		{
			for (TWeakObjectPtr<UObject> ObjectReferencer : InPackageReloadedEvent->GetObjectReferencers())
			{
				UObject* ObjectReferencerPtr = ObjectReferencer.Get();
				if (!ObjectReferencerPtr)
				{
					continue;
				}

				if (!ObjectReferencerPtr->GetClass()->HasAnyClassFlags(CLASS_NewerVersionExists))
				{
					// Calling PostEditChangeProperty on an actor with an outdated class will trigger a check() during construction scripts.
					FPropertyChangedEvent PropertyEvent(nullptr, EPropertyChangeType::Redirected);
					ObjectReferencerPtr->PostEditChangeProperty(PropertyEvent);
				}
				
				// We need to recompile any Blueprints that had properties changed to make sure their generated class is up-to-date and has no lingering references to the old objects
				UBlueprint* BlueprintToRecompile = nullptr;
				if (UBlueprint* BlueprintReferencer = Cast<UBlueprint>(ObjectReferencerPtr))
				{
					BlueprintToRecompile = BlueprintReferencer;
				}
				else if (UClass* ClassReferencer = Cast<UClass>(ObjectReferencerPtr))
				{
					BlueprintToRecompile = Cast<UBlueprint>(ClassReferencer->ClassGeneratedBy);
				}
				else
				{
					BlueprintToRecompile = ObjectReferencerPtr->GetTypedOuter<UBlueprint>();
				}

				if (BlueprintToRecompile)
				{
					BlueprintsToRecompileThisBatch.Add(BlueprintToRecompile);
				}
			}

			// @todo FH: we should eventually have a specific api for hot reloading single objects or external packages' objects
			// Call post edit change property on the reloaded objects in the package if they are external 
			FPropertyChangedEvent PropertyEvent(nullptr, EPropertyChangeType::Redirected);
			for (const auto& ObjectPair : InPackageReloadedEvent->GetRepointedObjects())
			{
				// An object is external, if it has a directly assigned package
				if (ObjectPair.Value && ObjectPair.Value->GetExternalPackage())
				{
					ObjectPair.Value->PostEditChangeProperty(PropertyEvent);
				}
			}
		}

		if (InPackageReloadPhase == EPackageReloadPhase::PreBatch)
		{
			// If this fires then ReloadPackages has probably bee called recursively :(
			check(BlueprintsToRecompileThisBatch.Num() == 0);

			// Flush all pending render commands, as reloading the package may invalidate render resources.
			FlushRenderingCommands();
		}

		if (InPackageReloadPhase == EPackageReloadPhase::PostBatchPreGC)
		{
			if (GEditor)
			{
				// Make sure we don't have any lingering transaction buffer references.
				GEditor->ResetTransaction(NSLOCTEXT("UnrealEd", "ReloadedPackage", "Reloaded Package"));
			}

			// Recompile any BPs that had their references updated
			if (BlueprintsToRecompileThisBatch.Num() > 0)
			{
				FScopedSlowTask CompilingBlueprintsSlowTask(static_cast<float>(BlueprintsToRecompileThisBatch.Num()), NSLOCTEXT("UnrealEd", "CompilingBlueprints", "Compiling Blueprints"));

				// Gather up all loaded BP assets.
				TArray<UObject*> BPs;
				GetObjectsOfClass(UBlueprint::StaticClass(), BPs);

				// Keeps track of BPs with a dependent cache set that's ready to be repopulated.
				TSet<UBlueprint*> BPsWithResetDependentCache;
				BPsWithResetDependentCache.Reserve(BPs.Num());

				// Rebuild the dependency/dependent cache sets for each loaded BP.
				for (UObject* BP : BPs)
				{
					UBlueprint* AsBP = CastChecked<UBlueprint>(BP);

					// If this BP has been replaced, clear its dependency cache, but don't rebuild it.
					if (AsBP->HasAnyFlags(RF_NewerVersionExists))
					{
						AsBP->CachedDependencies.Empty();
						AsBP->CachedUDSDependencies.Empty();
						AsBP->bCachedDependenciesUpToDate = true;

						// Also clear out its dependent cache, as there will no longer be any dependencies on it.
						AsBP->CachedDependents.Empty();
						BPsWithResetDependentCache.Add(AsBP);
					}
					else
					{
						// Rebuild the dependency cache for the this BP.
						AsBP->bCachedDependenciesUpToDate = false;
						FBlueprintEditorUtils::EnsureCachedDependenciesUpToDate(AsBP);
					}

					// For each cached dependency, add this BP into its dependent cache set. Note that replaced
					// BPs won't have any dependencies, and so will not be included in any dependent cache sets.
					TSet<TWeakObjectPtr<UBlueprint>> LocalCopy_CachedDependencies = AsBP->CachedDependencies;
					for (const TWeakObjectPtr<UBlueprint>& DependencyPtr : LocalCopy_CachedDependencies)
					{
						if (UBlueprint* Dependency = DependencyPtr.Get())
						{
							// Ensure that the dependency's cached dependent set has been cleared before we start adding to it.
							if (!BPsWithResetDependentCache.Contains(Dependency))
							{
								Dependency->CachedDependents.Empty();
								BPsWithResetDependentCache.Add(Dependency);
							}

							Dependency->CachedDependents.Add(AsBP);
						}
						else
						{
							// Remove any entries that cannot be resolved. Not likely, but just in case.
							AsBP->CachedDependencies.Remove(DependencyPtr);
						}
					}
				}

				// Clear out remaining dependent cache sets that weren't already covered above (those not considered as a dependency and/or any replaced BPs).
				if (BPs.Num() > BPsWithResetDependentCache.Num())
				{
					for (UObject* BP : BPs)
					{
						UBlueprint* AsBP = CastChecked<UBlueprint>(BP);
						if (!BPsWithResetDependentCache.Contains(AsBP))
						{
							AsBP->CachedDependents.Empty();
							BPsWithResetDependentCache.Add(AsBP);
						}
					}
				}

				// Sanity check that all BPs have reset their dependent cache sets, to assert that there are no stale references left behind.
				check(BPs.Num() == BPsWithResetDependentCache.Num());

				for (UBlueprint* BlueprintToRecompile : BlueprintsToRecompileThisBatch)
				{
					CompilingBlueprintsSlowTask.EnterProgressFrame(1.0f);

					FKismetEditorUtilities::CompileBlueprint(BlueprintToRecompile, EBlueprintCompileOptions::SkipGarbageCollection);
				}
			}
			BlueprintsToRecompileThisBatch.Reset();
		}

		if (InPackageReloadPhase == EPackageReloadPhase::PostBatchPostGC)
		{
			// Tick some things that aren't processed while we're reloading packages and can result in excessive memory usage if not periodically updated.
			if (GShaderCompilingManager)
			{
				GShaderCompilingManager->ProcessAsyncResults(true, false);
			}
			if (GDistanceFieldAsyncQueue)
			{
				GDistanceFieldAsyncQueue->ProcessAsyncTasks();
			}
			if (GCardRepresentationAsyncQueue)
			{
				GCardRepresentationAsyncQueue->ProcessAsyncTasks();
			}
		}
	}


	/**
	 * Wrapper method for multiple objects at once.
	 *
	 * @param	TopLevelPackages		the packages to be export
	 * @param	LastExportPath			the path that the user last exported assets to
	 * @param	FilteredClasses			if specified, set of classes that should be the only types exported if not exporting to single file
	 * @param	bUseProvidedExportPath	If true, use LastExportPath as the user's export path w/o prompting for a directory, where applicable
	 *
	 * @return	the path that the user chose for the export.
	 */
	FString UPackageTools::DoBulkExport(const TArray<UPackage*>& TopLevelPackages, FString LastExportPath, const TSet<UClass*>* FilteredClasses /* = NULL */, bool bUseProvidedExportPath/* = false*/ )
	{
		// Disallow export if any packages are cooked.
		if (HandleFullyLoadingPackages( TopLevelPackages, NSLOCTEXT("UnrealEd", "BulkExportE", "Bulk Export...") ) )
		{
			TArray<UObject*> ObjectsInPackages;
			GetObjectsInPackages(&TopLevelPackages, ObjectsInPackages);

			// See if any filtering has been requested. Objects can be filtered by class and/or localization filter.
			TArray<UObject*> FilteredObjects;
			if ( FilteredClasses )
			{
				// Present the user with a warning that only the filtered types are being exported
				FSuppressableWarningDialog::FSetupInfo Info( NSLOCTEXT("UnrealEd", "BulkExport_FilteredWarning", "Asset types are currently filtered within the Content Browser. Only objects of the filtered types will be exported."),
					LOCTEXT("BulkExport_FilteredWarning_Title", "Asset Filter in Effect"), "BulkExportFilterWarning" );
				Info.ConfirmText = NSLOCTEXT("ModalDialogs", "BulkExport_FilteredWarningConfirm", "Close");

				FSuppressableWarningDialog PromptAboutFiltering( Info );
				PromptAboutFiltering.ShowModal();
				
				for ( TArray<UObject*>::TConstIterator ObjIter(ObjectsInPackages); ObjIter; ++ObjIter )
				{
					UObject* CurObj = *ObjIter;

					// Only add the object if it passes all of the specified filters
					if ( CurObj && FilteredClasses->Contains( CurObj->GetClass() ) )
					{
						FilteredObjects.Add( CurObj );
					}
				}
			}

			// If a filtered set was provided, export the filtered objects array; otherwise, export all objects in the packages
			TArray<UObject*>& ObjectsToExport = FilteredClasses ? FilteredObjects : ObjectsInPackages;

			// Prompt the user about how many objects will be exported before proceeding.
			const bool bProceed = EAppReturnType::Yes == FMessageDialog::Open( EAppMsgType::YesNo, EAppReturnType::Yes, FText::Format(
				NSLOCTEXT("UnrealEd", "Prompt_AboutToBulkExportNItems_F", "About to bulk export {0} items.  Proceed?"), FText::AsNumber(ObjectsToExport.Num()) ) );
			if ( bProceed )
			{
				FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");

				AssetToolsModule.Get().ExportAssets(ObjectsToExport, LastExportPath);
			}
		}

		return LastExportPath;
	}

	void UPackageTools::CheckOutRootPackages( const TArray<UPackage*>& Packages )
	{
		if (ISourceControlModule::Get().IsEnabled())
		{
			ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

			// Update to the latest source control state.
			SourceControlProvider.Execute(ISourceControlOperation::Create<FUpdateStatus>(), Packages);

			TArray<FString> TouchedPackageNames;
			TArray<FString> FailedPackageNames;
			for( int32 PackageIndex = 0 ; PackageIndex < Packages.Num() ; ++PackageIndex )
			{
				UPackage* Package = Packages[PackageIndex];
				FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(Package, EStateCacheUsage::Use);
				if( SourceControlState.IsValid() && SourceControlState->CanCheckout() )
				{
					// The package is still available, so do the check out.
					TouchedPackageNames.Add(Package->GetName());
				}
				else
				{
					// The status on the package has changed to something inaccessible, so we have to disallow the check out.
					// Don't warn if the file isn't in the depot.
					if (SourceControlState.IsValid() && SourceControlState->IsSourceControlled())
					{			
						FailedPackageNames.Add(Package->GetName());
					}
				}
			}

			if (FailedPackageNames.Num() > 0)
			{
				FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Error_PackageStatusChanged", "Some package(s) can't be checked out - status has changed!"));
			}

			if (TouchedPackageNames.Num() > 0)
			{
				// Synchronize source control state if something was checked out.
				SourceControlProvider.Execute(ISourceControlOperation::Create<FCheckOut>(), SourceControlHelpers::PackageFilenames(TouchedPackageNames));
			}
		}
	}

	/**
	 * Checks if the passed in path is in an external directory. I.E Ones not found automatically in the content directory
	 *
	 * @param	PackagePath	Path of the package to check, relative or absolute
	 * @return	true if PackagePath points to an external location
	 */
	bool UPackageTools::IsPackagePathExternal( const FString& PackagePath )
	{
		bool bIsExternal = true;
		TArray< FString > Paths;
		GConfig->GetArray( TEXT("Core.System"), TEXT("Paths"), Paths, GEngineIni );
	
		FString PackageFilename = FPaths::ConvertRelativePathToFull(PackagePath);

		// absolute path of the package that was passed in, without the actual name of the package
		FString PackageFullPath = FPaths::GetPath(PackageFilename);

		for(int32 pathIdx = 0; pathIdx < Paths.Num(); ++pathIdx)
		{ 
			FString AbsolutePathName = FPaths::ConvertRelativePathToFull(Paths[ pathIdx ]);

			// check if the package path is within the list of paths the engine searches.
			if( PackageFullPath.Contains( AbsolutePathName ) )
			{
				bIsExternal = false;
				break;
			}
		}

		return bIsExternal;
	}

	/**
	 * Checks if the passed in package's filename is in an external directory. I.E Ones not found automatically in the content directory
	 *
	 * @param	Package	The package to check
	 * @return	true if the package points to an external filename
	 */
	bool UPackageTools::IsPackageExternal(const UPackage& Package)
	{
		FString FileString;
		FPackageName::DoesPackageExist(Package.GetName(), &FileString);

		return IsPackagePathExternal( FileString );
	}

	bool UPackageTools::SavePackagesForObjects(const TArray<UObject*>& ObjectsToSave)
	{
		// Retrieve all dirty packages for the objects 
		TArray<UPackage*> PackagesToSave;
		for (UObject* Object : ObjectsToSave)
		{
			if (Object->GetOutermost()->IsDirty())
			{
				PackagesToSave.AddUnique(Object->GetOutermost());
			}
		}

		const bool bCheckDirty = false;
		const bool bPromptToSave = false;
		const FEditorFileUtils::EPromptReturnCode Return = FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, bCheckDirty, bPromptToSave);
		
		return (PackagesToSave.Num() > 0) && Return == FEditorFileUtils::EPromptReturnCode::PR_Success;
	}

	bool UPackageTools::IsSingleAssetPackage(const FString& PackageName)
	{
		FString PackageFileName;
		if ( FPackageName::DoesPackageExist(PackageName, &PackageFileName) )
		{
			return FPaths::GetExtension(PackageFileName, /*bIncludeDot=*/true) == FPackageName::GetAssetPackageExtension();
		}

		// If it wasn't found in the package file cache, this package does not yet
		// exist so it is assumed to be saved as a UAsset file.
		return true;
	}

	FString UPackageTools::SanitizePackageName(const FString& InPackageName)
	{
		FString SanitizedName = ObjectTools::SanitizeInvalidChars(InPackageName, INVALID_LONGPACKAGE_CHARACTERS);

		// Coalesce multiple contiguous slashes into a single slash
		int32 CharIndex = 0;
		while (CharIndex < SanitizedName.Len())
		{
			if (SanitizedName[CharIndex] == TEXT('/'))
			{
				int32 SlashCount = 1;
				while (CharIndex + SlashCount < SanitizedName.Len() &&
					   SanitizedName[CharIndex + SlashCount] == TEXT('/'))
				{
					SlashCount++;
				}

				if (SlashCount > 1)
				{
					SanitizedName.RemoveAt(CharIndex + 1, SlashCount - 1, EAllowShrinking::No);
				}
			}

			CharIndex++;
		}

		return SanitizedName;
	}

	FString UPackageTools::PackageNameToFilename(const FString& InPackageName, const FString& Extension)
	{
		FString Result;
		FPackageName::TryConvertLongPackageNameToFilename(InPackageName, Result, Extension);
		return Result;
	}

	FString UPackageTools::FilenameToPackageName(const FString& InFilename)
	{
		FString Result;
    	FPackageName::TryConvertFilenameToLongPackageName(InFilename, Result);
    	return Result;
	}

	UPackage* UPackageTools::FindOrCreatePackageForAssetType(const FName LongPackageName, UClass* AssetClass)
	{
		if (AssetClass)
		{
			// Test the asset registry first
			// Only scan the disk cache since it's fast O(1) (otherwise the asset registry will iterate on all the objects in memory)
			constexpr bool bIncludeOnlyOnDiskAssets = true;
			TArray<FAssetData> OutAssets;
			IAssetRegistry& AssetRegistry = FAssetRegistryModule::GetRegistry();
			AssetRegistry.GetAssetsByPackageName(LongPackageName, OutAssets, bIncludeOnlyOnDiskAssets);
			bool bShouldGenerateUniquePackageName = false;
			int32 NumberOfAssets = OutAssets.Num();
			if (NumberOfAssets == 1)
			{
				const FAssetData& AssetData = OutAssets[0];
				if (AssetData.AssetClassPath != AssetClass->GetClassPathName())
				{
					bShouldGenerateUniquePackageName = true;
				}
			}
			else if (NumberOfAssets > 1)
			{
				// this shouldn't happen 
				bShouldGenerateUniquePackageName = true;
			}

			UPackage* Package = nullptr;
			if (!bShouldGenerateUniquePackageName)
			{
				// Create or return the existing package
				FString PackageNameAsString = LongPackageName.ToString();
				Package = LoadPackage(PackageNameAsString, LOAD_NoWarn);

				if (Package)
				{
					UObject* Object = FindObjectFast<UObject>(Package, Package->GetFName());
					if (Object && Object->GetClass() != AssetClass)
					{
						bShouldGenerateUniquePackageName = true;
					}
				}
				else
				{
					Package = NewObject<UPackage>(nullptr, LongPackageName, RF_Public);
				}
			}

			if (bShouldGenerateUniquePackageName)
			{
				FString NewPackageName = MakeUniqueObjectName(nullptr, UPackage::StaticClass(), LongPackageName).ToString();
				Package = NewObject<UPackage>(nullptr, *NewPackageName, RF_Public);
			}

			return Package;
		}

		return nullptr;
	}


#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPackageToolsAutomationTest, "Editor.PackageTools.UnitTests", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FPackageToolsAutomationTest::RunTest(const FString& Parameters)
{
	struct FTest
	{
		FString Input;
		FString Output;
	};

	TArray<FTest> TestStrings =
	{
		{ TEXT("/Game/Blah/Boo"), TEXT("/Game/Blah/Boo") },
		{ TEXT("/Game/Blah//Double"), TEXT("/Game/Blah/Double") },
		{ TEXT(""), TEXT("") },
		{ TEXT("/Game/Trailing/"), TEXT("/Game/Trailing/") },
		{ TEXT("/Game/TrailingDouble//"), TEXT("/Game/TrailingDouble/") },
		{ TEXT("/Game/Blah///Multiple"), TEXT("/Game/Blah/Multiple") },
		{ TEXT("/Game/Blah///Multiple///"), TEXT("/Game/Blah/Multiple/") }
	};

	bool bSuccess = true;
	for (const FTest& TestString : TestStrings)
	{
		FString Result = UPackageTools::SanitizePackageName(TestString.Input);
		if (Result != TestString.Output)
		{
			AddError(FString::Printf(TEXT("SanitizePackageName failed (result = %s, expected = %s)"), *Result, *TestString.Output));
			bSuccess = false;
		}
	}

	return bSuccess;
}

#endif // WITH_DEV_AUTOMATION_TESTS


#undef LOCTEXT_NAMESPACE

// EOF
