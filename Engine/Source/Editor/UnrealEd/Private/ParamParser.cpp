// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParamParser.cpp: Functions to help parse commands.

	What's happening: When the Visual Basic level editor is being used,
	this code exchanges messages with Visual Basic.  This lets Visual Basic
	affect the world, and it gives us a way of sending world information back
	to Visual Basic.
=============================================================================*/

#include "CoreTypes.h"
#include "Logging/LogMacros.h"
#include "Math/Rotator.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Misc/CString.h"
#include "Misc/Parse.h"

/*-----------------------------------------------------------------------------
	Getters.
	All of these functions return 1 if the appropriate item was
	fetched, or 0 if not.
-----------------------------------------------------------------------------*/
DEFINE_LOG_CATEGORY_STATIC(LogParamParser, Log, All);

//
// Get a floating-point vector (X=, Y=, Z=).
//
bool GetFVECTOR( const TCHAR* Stream, FVector& Value )
{
	int32 NumVects = 0;

	Value = FVector::ZeroVector;

	// Support for old format.
	NumVects += FParse::Value( Stream, TEXT("X="), Value.X );
	NumVects += FParse::Value( Stream, TEXT("Y="), Value.Y );
	NumVects += FParse::Value( Stream, TEXT("Z="), Value.Z );

	// New format.
	if( NumVects == 0 )
	{
		Value.X = (FVector::FReal)FCString::Atof(Stream);
		Stream = FCString::Strchr(Stream,',');
		if( !Stream )
		{
			return 0;
		}

		Stream++;
		Value.Y = (FVector::FReal)FCString::Atof(Stream);
		Stream = FCString::Strchr(Stream,',');
		if( !Stream )
		{
			return 0;
		}

		Stream++;
		Value.Z = (FVector::FReal)FCString::Atof(Stream);

		NumVects=3;
	}

	return NumVects==3;
}

/**
 * Get a floating-point vector (X Y Z)
 *
 * @param The stream which has the vector in it
 * @param this is an out param which will have the FVector
 *
 * @return this will return the current location in the stream after having processed the Vector out of it
 **/
const TCHAR* GetFVECTORSpaceDelimited( const TCHAR* Stream, FVector& Value )
{
	if( Stream == NULL )
	{
		return NULL;
	}

	Value = FVector::ZeroVector;


	Value.X = FCString::Atof(Stream);
	//UE_LOGF(LogParamParser, Warning, "Value.X %f", Value.X );
	Stream = FCString::Strchr(Stream,' ');
	if( !Stream )
	{
		return Stream;
	}

	Stream++;
	Value.Y = FCString::Atof(Stream);
	//UE_LOGF(LogParamParser, Warning, "Value.Y %f", Value.Y );
	Stream = FCString::Strchr(Stream,' ');
	if( !Stream )
	{
		return Stream;
	}

	Stream++;
	Value.Z = FCString::Atof(Stream);
	//UE_LOGF(LogParamParser, Warning, "Value.Z %f", Value.Z );
	
	return Stream;
}


//
// Get a string enclosed in parenthesis.
//
bool GetSUBSTRING
(
	const TCHAR*	Stream, 
	const TCHAR*	Match,
	TCHAR*			Value,
	int32				MaxLen
)
{
	const TCHAR* Found = FCString::Strifind(Stream,Match);
	const TCHAR* Start;

	if( Found == NULL ) return false; // didn't match.

	Start = Found + FCString::Strlen(Match);
	if( *Start != '(' )
		return false;

	FCString::Strncpy( Value, Start+1, MaxLen );
	TCHAR* Temp=FCString::Strchr( Value, ')' );
	if( Temp )
		*Temp=0;

	return true;
}

//
// Get a floating-point vector (X=, Y=, Z=).
//
bool GetFVECTOR
(
	const TCHAR*	Stream, 
	const TCHAR*	Match, 
	FVector&		Value
)
{
	TCHAR Temp[80];
	if (!GetSUBSTRING(Stream,Match,Temp,80)) return false;
	return GetFVECTOR(Temp,Value);

}

//
// Get a set of rotations (PITCH=, YAW=, ROLL=), return whether anything got parsed.
//
bool GetFROTATOR
(
	const TCHAR*	Stream, 
	FRotator&		Rotation,
	int32				ScaleFactor
)
{
	float	Temp=0.0;
	int32 	N = 0;

	Rotation = FRotator::ZeroRotator;

	// Old format.
	if( FParse::Value(Stream,TEXT("PITCH="),Temp) ) {Rotation.Pitch = Temp * ScaleFactor; N++;}
	if( FParse::Value(Stream,TEXT("YAW="),  Temp) ) {Rotation.Yaw   = Temp * ScaleFactor; N++;}
	if( FParse::Value(Stream,TEXT("ROLL="), Temp) ) {Rotation.Roll  = Temp * ScaleFactor; N++;}

	// New format.
	if( N == 0 )
	{
		Rotation.Pitch = FCString::Atof(Stream) * ScaleFactor;
		Stream = FCString::Strchr(Stream,',');
		if( !Stream )
		{
			return false;
		}

		Rotation.Yaw = FCString::Atof(++Stream) * ScaleFactor;
		Stream = FCString::Strchr(Stream,',');
		if( !Stream )
		{
			return false;
		}

		Rotation.Roll = FCString::Atof(++Stream) * ScaleFactor;
		return true;
	}

	return true;
}


/**
 * Get an int based FRotator (X Y Z)
 *
 * @param The stream which has the rotator in it
 * @param this is an out param which will have the FRotator
 *
 * @return this will return the current location in the stream after having processed the rotator out of it
 **/
const TCHAR* GetFROTATORSpaceDelimited
(
	const TCHAR*	Stream, 
	FRotator&		Rotation,
	int32				ScaleFactor
)
{
	if( Stream == NULL )
	{
		return NULL;
	}

	Rotation = FRotator::ZeroRotator;
	Rotation.Pitch = FCString::Atof(Stream) * ScaleFactor;
	//UE_LOGF(LogParamParser, Warning, "Rotation.Pitch %d", Rotation.Pitch );
	Stream = FCString::Strchr(Stream,' ');
	if( !Stream )
	{
		return Stream;
	}

	Rotation.Yaw = FCString::Atof(++Stream) * ScaleFactor;
	//UE_LOGF(LogParamParser, Warning, "Rotation.Yaw %d", Rotation.Yaw );
	Stream = FCString::Strchr(Stream,' ');
	if( !Stream )
	{
		return Stream;
	}

	Rotation.Roll = FCString::Atof(++Stream) * ScaleFactor;
	//UE_LOGF(LogParamParser, Warning, "Rotation.Roll %d", Rotation.Roll );


	return Stream;
}


//
// Get a rotation value, return whether anything got parsed.
//
bool GetFROTATOR
(
	const TCHAR*	Stream, 
	const TCHAR*	Match, 
	FRotator&		Value,
	int32				ScaleFactor
)
{
	TCHAR Temp[80];
	if (!GetSUBSTRING(Stream,Match,Temp,80)) return false;
	return GetFROTATOR(Temp,Value,ScaleFactor);

}

//
// Gets a "BEGIN" string.  Returns 1 if gotten, 0 if not.
// If not gotten, doesn't affect anything.
//
bool GetBEGIN( const TCHAR** Stream, const TCHAR* Match )
{
	const TCHAR* Original = *Stream;
	if( FParse::Command( Stream, TEXT("BEGIN") ) && FParse::Command( Stream, Match ) )
		return true;
	*Stream = Original;
	return false;

}

//
// Gets an "END" string.  Returns 1 if gotten, 0 if not.
// If not gotten, doesn't affect anything.
//
bool GetEND( const TCHAR** Stream, const TCHAR* Match )
{
	const TCHAR* Original = *Stream;
	if (FParse::Command (Stream,TEXT("END")) && FParse::Command (Stream,Match)) return 1; // Gotten.
	*Stream = Original;
	return false;

}

//
// Gets an "REMOVE" string.  Returns true if gotten, false if not.
// If not gotten, doesn't affect anything.
//
bool GetREMOVE( const TCHAR** Stream, const TCHAR* Match )
{
	const TCHAR* Original = *Stream;
	if (FParse::Command (Stream,TEXT("REMOVE")) && FParse::Command (Stream,Match)) 
		return true; // Gotten.
	*Stream = Original;
	return false;
}

//
// Output a vector.
//
TCHAR* SetFVECTOR( TCHAR* Dest, const FVector* FVector )
{
	FCString::Sprintf( Dest, TEXT("%+013.6f,%+013.6f,%+013.6f"), FVector->X, FVector->Y, FVector->Z );
	return Dest;
}

