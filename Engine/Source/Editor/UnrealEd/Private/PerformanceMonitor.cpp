// Copyright Epic Games, Inc. All Rights Reserved.


#include "PerformanceMonitor.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Engine/GameViewportClient.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SToolTip.h"
#include "Framework/Docking/TabManager.h"
#include "Styling/AppStyle.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "EngineGlobals.h"
#include "Editor.h"
#include "SScalabilitySettings.h"
#include "ShaderCompiler.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#include "Settings/EditorSettings.h"
#include "Editor/EditorPerformanceSettings.h"
#include "EditorViewportClient.h"

#define LOCTEXT_NAMESPACE "PerformanceMonitor"

const double AutoApplyScalabilityTimeout = 10;

/** Scalability dialog widget */
class SScalabilitySettingsDialog : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SScalabilitySettingsDialog) {}
		SLATE_ARGUMENT(FOnClicked, OnDoneClicked)
	SLATE_END_ARGS()

public:

	/** Construct this widget */
	void Construct(const FArguments& InArgs)
	{
		ChildSlot
		[
			SNew(SBorder)
			.HAlign(HAlign_Fill)
			.BorderImage(FAppStyle::GetBrush("ChildWindow.Background"))
			[
				SNew(SBox)
				.WidthOverride(500.0f)
				[
					SNew(SVerticalBox)
					
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(5.f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("PerformanceWarningDescription", "The current performance of the editor seems to be low.\nUse the options below to reduce the amount of detail and increase performance."))
					]
					
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(5.f)
					[
						SNew(SScalabilitySettings)
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(5.f)
					[
						SNew(STextBlock)
						.ToolTip(
							SNew(SToolTip)
							[
								SNew(SImage)
								.Image(FAppStyle::GetBrush("Scalability.ScalabilitySettings"))
							]
						)
						.AutoWrapText(true)
						.Text(LOCTEXT("PerformanceWarningChangeLater", "You can modify these settings in future via \"Quick Settings\" button on the level editor toolbar and choosing \"Engine Scalability Settings\"."))
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(5.f)
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.FillWidth(1.0)
						.HAlign(HAlign_Right)
						[
							SNew(SButton)
							.Text(LOCTEXT("ScalabilityDone", "Done"))
							.OnClicked(InArgs._OnDoneClicked)
						]
					]
				]
			]
		];
	}
};

static TAutoConsoleVariable<int32> CVarFineSampleTime(
	TEXT("PerfWarn.FineSampleTime"), 30, TEXT("How many seconds we sample the percentage for the fine-grained minimum FPS."), ECVF_Default);
static TAutoConsoleVariable<int32> CVarCoarseSampleTime(
	TEXT("PerfWarn.CoarseSampleTime"), 600, TEXT("How many seconds we sample the percentage for the coarse-grained minimum FPS."), ECVF_Default);
static TAutoConsoleVariable<int32> CVarFineMinFPS(
	TEXT("PerfWarn.FineMinFPS"), 10, TEXT("The FPS threshold below which we warn about for fine-grained sampling."), ECVF_Default);
static TAutoConsoleVariable<int32> CVarCoarseMinFPS(
	TEXT("PerfWarn.CoarseMinFPS"), 20, TEXT("The FPS threshold below which we warn about for coarse-grained sampling."), ECVF_Default);
static TAutoConsoleVariable<int32> CVarFinePercentThreshold(
	TEXT("PerfWarn.FinePercentThreshold"), 80, TEXT("The percentage of samples that fall below min FPS above which we warn for."), ECVF_Default);
static TAutoConsoleVariable<int32> CVarCoarsePercentThreshold(
	TEXT("PerfWarn.CoarsePercentThreshold"), 80, TEXT("The percentage of samples that fall below min FPS above which we warn for."), ECVF_Default);

FPerformanceMonitor::FPerformanceMonitor()
{
	LastEnableTime = 0;
	NotificationTimeout = AutoApplyScalabilityTimeout;
	bIsNotificationAllowed = true;

	CVarDelegate = FConsoleCommandDelegate::CreateLambda([this]{
		static const auto CVarFine = IConsoleManager::Get().FindConsoleVariable(TEXT("PerfWarn.FineSampleTime"));
		static const auto CVarCoarse = IConsoleManager::Get().FindConsoleVariable(TEXT("PerfWarn.CoarseSampleTime"));

		static int32 LastTickFine = -1, LastTickCoarse = -1;

		if (CVarFine->GetInt() != LastTickFine || CVarCoarse->GetInt() != LastTickCoarse)
		{
			LastTickFine = CVarFine->GetInt();
			LastTickCoarse = CVarCoarse->GetInt();
			
			UpdateMovingAverageSamplers();
		}
	});

	CVarDelegateHandle = IConsoleManager::Get().RegisterConsoleVariableSink_Handle(CVarDelegate);
}

FPerformanceMonitor::~FPerformanceMonitor()
{
	IConsoleManager::Get().UnregisterConsoleVariableSink_Handle(CVarDelegateHandle);
}

bool FPerformanceMonitor::WillAutoScalabilityHelp() const
{
	Scalability::FQualityLevels CurrentLevels = Scalability::GetQualityLevels();
	Scalability::FQualityLevels NewLevels = GetAutoScalabilityQualityLevels();

	bool IsAutoScaleLower = false;
	IsAutoScaleLower |= NewLevels.ResolutionQuality < CurrentLevels.ResolutionQuality;
	IsAutoScaleLower |= NewLevels.ViewDistanceQuality < CurrentLevels.ViewDistanceQuality;
	IsAutoScaleLower |= NewLevels.AntiAliasingQuality < CurrentLevels.AntiAliasingQuality;
	IsAutoScaleLower |= NewLevels.ShadowQuality < CurrentLevels.ShadowQuality;
	IsAutoScaleLower |= NewLevels.GlobalIlluminationQuality < CurrentLevels.GlobalIlluminationQuality;
	IsAutoScaleLower |= NewLevels.ReflectionQuality < CurrentLevels.ReflectionQuality;
	IsAutoScaleLower |= NewLevels.PostProcessQuality < CurrentLevels.PostProcessQuality;
	IsAutoScaleLower |= NewLevels.TextureQuality < CurrentLevels.TextureQuality;
	IsAutoScaleLower |= NewLevels.EffectsQuality < CurrentLevels.EffectsQuality;
	IsAutoScaleLower |= NewLevels.FoliageQuality < CurrentLevels.FoliageQuality;
	IsAutoScaleLower |= NewLevels.ShadingQuality < CurrentLevels.ShadingQuality;

	// We don't check things like real-time, because the user may have enabled it temporarily.

	return IsAutoScaleLower;
}

Scalability::FQualityLevels FPerformanceMonitor::GetAutoScalabilityQualityLevels() const
{
	const Scalability::FQualityLevels ExistingLevels = Scalability::GetQualityLevels();
	Scalability::FQualityLevels NewLevels = GetDefault<UEditorSettings>()->EngineBenchmarkResult;

	// Make sure we don't turn settings up if the user has turned them down for any reason
	NewLevels.ResolutionQuality		= FMath::Min(NewLevels.ResolutionQuality, ExistingLevels.ResolutionQuality);
	NewLevels.ViewDistanceQuality	= FMath::Min(NewLevels.ViewDistanceQuality, ExistingLevels.ViewDistanceQuality);
	NewLevels.AntiAliasingQuality	= FMath::Min(NewLevels.AntiAliasingQuality, ExistingLevels.AntiAliasingQuality);
	NewLevels.ShadowQuality			= FMath::Min(NewLevels.ShadowQuality, ExistingLevels.ShadowQuality);
	NewLevels.GlobalIlluminationQuality			= FMath::Min(NewLevels.GlobalIlluminationQuality, ExistingLevels.GlobalIlluminationQuality);
	NewLevels.ReflectionQuality			= FMath::Min(NewLevels.ReflectionQuality, ExistingLevels.ReflectionQuality);
	NewLevels.PostProcessQuality	= FMath::Min(NewLevels.PostProcessQuality, ExistingLevels.PostProcessQuality);
	NewLevels.TextureQuality		= FMath::Min(NewLevels.TextureQuality, ExistingLevels.TextureQuality);
	NewLevels.EffectsQuality		= FMath::Min(NewLevels.EffectsQuality, ExistingLevels.EffectsQuality);
	NewLevels.FoliageQuality		= FMath::Min(NewLevels.FoliageQuality, ExistingLevels.FoliageQuality);
	NewLevels.ShadingQuality		= FMath::Min(NewLevels.ShadingQuality, ExistingLevels.ShadingQuality);

	return NewLevels;
}

void FPerformanceMonitor::AutoApplyScalability()
{
	GetMutableDefault<UEditorSettings>()->AutoApplyScalabilityBenchmark();

	Scalability::FQualityLevels NewLevels = FPerformanceMonitor::GetAutoScalabilityQualityLevels();

	Scalability::SetQualityLevels(NewLevels);
	Scalability::SaveState(GEditorSettingsIni);
	GEditor->RedrawAllViewports();

	const bool bAutoApplied = false;
	Scalability::RecordQualityLevelsAnalytics(bAutoApplied);

	for (FEditorViewportClient* VC : GEditor->GetAllViewportClients())
	{
		if (VC)
		{
			VC->SetRealtime(false);
			VC->Invalidate();
		}
	}

	// Reset the timers so as not to skew the data with the time it took to do the benchmark	
	FineMovingAverage.Reset();
	CoarseMovingAverage.Reset();
}

void FPerformanceMonitor::ShowPerformanceWarning(FText MessageText)
{
	static double MinNotifyTime = 30;
	if ((FPlatformTime::Seconds() - LastEnableTime) > MinNotifyTime)
	{
		// Only show a new one if we've not shown one for a while
		LastEnableTime = FPlatformTime::Seconds();
		NotificationTimeout = AutoApplyScalabilityTimeout;

		// Create notification item
		FNotificationInfo Info(MessageText);
		Info.bFireAndForget = false;
		Info.FadeOutDuration = 3.0f;
		Info.ExpireDuration = 0.0f;
		Info.bUseLargeFont = false;

		Info.ButtonDetails.Add( FNotificationButtonInfo( LOCTEXT("ApplyNow", "Apply Now"), FText::GetEmpty(), FSimpleDelegate::CreateRaw( this, &FPerformanceMonitor::AutoApplyScalability ) ) );
		Info.ButtonDetails.Add( FNotificationButtonInfo( LOCTEXT("TweakManually", "Tweak Manually"), FText::GetEmpty(), FSimpleDelegate::CreateRaw( this, &FPerformanceMonitor::ShowScalabilityDialog ) ) );
		Info.ButtonDetails.Add( FNotificationButtonInfo( LOCTEXT("DontRemindMe", "Cancel & Ignore"), FText::GetEmpty(), FSimpleDelegate::CreateRaw( this, &FPerformanceMonitor::CancelPerformanceNotification ) ) );

		PerformanceWarningNotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
		PerformanceWarningNotificationPtr.Pin()->SetCompletionState(SNotificationItem::CS_Pending);
	}
}

void FPerformanceMonitor::CancelPerformanceNotification()
{
	UEditorPerformanceSettings* EditorUserSettings = GetMutableDefault<UEditorPerformanceSettings>();
	EditorUserSettings->bMonitorEditorPerformance = false;
	EditorUserSettings->PostEditChange();
	EditorUserSettings->SaveConfig();

	Reset();
}

void FPerformanceMonitor::HidePerformanceWarning()
{
	// Finished! Notify the UI.
	TSharedPtr<SNotificationItem> NotificationItem = PerformanceWarningNotificationPtr.Pin();

	if (NotificationItem.IsValid())
	{
		NotificationItem->SetCompletionState(SNotificationItem::CS_Success);
		NotificationItem->Fadeout();

		PerformanceWarningNotificationPtr.Reset();
	}
}

void FPerformanceMonitor::Tick(float DeltaTime)
{
	if (GEngine->ShouldThrottleCPUUsage() && (!GShaderCompilingManager || !GShaderCompilingManager->IsCompiling() ) )
	{
		return;
	}

	extern ENGINE_API float GAverageFPS;
	FineMovingAverage.Tick(FPlatformTime::Seconds(), GAverageFPS);
	CoarseMovingAverage.Tick(FPlatformTime::Seconds(), GAverageFPS);

	bool bMonitorEditorPerformance = 
		GetDefault<UEditorPerformanceSettings>()->bMonitorEditorPerformance && FApp::IsBenchmarking() == false;

	if( !bMonitorEditorPerformance || !bIsNotificationAllowed )
	{
		return;
	}

	FFormatNamedArguments Arguments;
	bool bLowFramerate = false;
	int32 SampleTime = 0;
	float PercentUnderTarget = 0.f;

	if (FineMovingAverage.IsReliable())
	{
		static IConsoleVariable* CVarMinFPS = IConsoleManager::Get().FindConsoleVariable(TEXT("PerfWarn.FineMinFPS"));
		static IConsoleVariable* CVarPercentThreshold = IConsoleManager::Get().FindConsoleVariable(TEXT("PerfWarn.FinePercentThreshold"));
		static IConsoleVariable* CVarSampleTime = IConsoleManager::Get().FindConsoleVariable(TEXT("PerfWarn.FineSampleTime"));

		PercentUnderTarget = FineMovingAverage.PercentageBelowThreshold(CVarMinFPS->GetFloat());

		if (PercentUnderTarget >= CVarPercentThreshold->GetFloat())
		{
			Arguments.Add(TEXT("Framerate"), CVarMinFPS->GetInt());
			Arguments.Add(TEXT("Percentage"), FMath::FloorToFloat(PercentUnderTarget));
			SampleTime = CVarSampleTime->GetInt();

			bLowFramerate = true;
		}
	}

	if (!bLowFramerate && CoarseMovingAverage.IsReliable())
	{
		static IConsoleVariable* CVarMinFPS = IConsoleManager::Get().FindConsoleVariable(TEXT("PerfWarn.CoarseMinFPS"));
		static IConsoleVariable* CVarPercentThreshold = IConsoleManager::Get().FindConsoleVariable(TEXT("PerfWarn.CoarsePercentThreshold"));

		PercentUnderTarget = CoarseMovingAverage.PercentageBelowThreshold(CVarMinFPS->GetFloat());

		if (PercentUnderTarget >= CVarPercentThreshold->GetFloat())
		{
			static IConsoleVariable* CoarseSampleTimeCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("PerfWarn.CoarseSampleTime"));

			Arguments.Add(TEXT("Framerate"), CVarMinFPS->GetInt());
			Arguments.Add(TEXT("Percentage"), FMath::FloorToFloat(PercentUnderTarget));
			SampleTime = CoarseSampleTimeCVar->GetInt();

			bLowFramerate = true;
		}
	}

	auto AlreadyOpenItem = PerformanceWarningNotificationPtr.Pin();

	if (!bLowFramerate)
	{
		// Framerate is back up again - just reset everything and hide the timeout
		if (AlreadyOpenItem.IsValid())
		{
			Reset();
		}
	}
	else
	{
		if( GetDefault<UEditorSettings>()->IsScalabilityBenchmarkValid() )
		{
			return;
		}

		// Choose an appropriate message
		enum MessagesEnum { Seconds, SecondsPercent, Minute, MinutePercent, Minutes, MinutesPercent };
		const FText Messages[] = {
			LOCTEXT("PerformanceWarningInProgress_Seconds",			"Your framerate has been under {Framerate} FPS for the past {SampleTime} seconds.\n\nDo you want to apply reduced quality settings? {TimeRemaining}s"),
			LOCTEXT("PerformanceWarningInProgress_Seconds_Percent", "Your framerate has been under {Framerate} FPS for {Percentage}% of the past {SampleTime} seconds.\n\nDo you want to apply reduced quality settings? {TimeRemaining}s"),

			LOCTEXT("PerformanceWarningInProgress_Minute",			"Your framerate has been under {Framerate} FPS for the past minute.\n\nDo you want to apply reduced quality settings? {TimeRemaining}s"),
			LOCTEXT("PerformanceWarningInProgress_Minute_Percent",	"Your framerate has been under {Framerate} FPS for {Percentage}% of the last minute.\n\nDo you want to apply reduced quality settings? {TimeRemaining}s"),

			LOCTEXT("PerformanceWarningInProgress_Minutes",			"Your framerate has been below {Framerate} FPS for the past {SampleTime} minutes.\n\nDo you want to apply reduced quality settings? {TimeRemaining}s"),
			LOCTEXT("PerformanceWarningInProgress_Minutes_Percent", "Your framerate has been below {Framerate} FPS for {Percentage}% of the past {SampleTime} minutes.\n\nDo you want to apply reduced quality settings? {TimeRemaining}s"),
		};

		int32 Message;
		if (SampleTime < 60)
		{
			Arguments.Add(TEXT("SampleTime"), SampleTime);
			Message = Seconds;
		}
		else if (SampleTime == 60)
		{
			Message = Minute;
		}
		else
		{
			Arguments.Add(TEXT("SampleTime"), SampleTime / 60);
			Message = Minutes;
		}

		// Use the message with the specific percentage on if applicable
		if (PercentUnderTarget <= 95.f)
		{
			++Message;
		}


		// Now update the notification or create a new one
		if (AlreadyOpenItem.IsValid())
		{
			NotificationTimeout -= DeltaTime;
			Arguments.Add(TEXT("TimeRemaining"), FMath::Max(1, FMath::CeilToInt(NotificationTimeout)));

			if (NotificationTimeout <= 0)
			{
				// Timed-out. Apply the settings.
				Reset();
				bIsNotificationAllowed = false;
			}
			else
			{
				AlreadyOpenItem->SetText(FText::Format(Messages[Message], Arguments));
			}
		}
		else
		{
			NotificationTimeout = AutoApplyScalabilityTimeout;
			Arguments.Add(TEXT("TimeRemaining"), int(NotificationTimeout));

			ShowPerformanceWarning(FText::Format(Messages[Message], Arguments));
		}
	}
}

void FPerformanceMonitor::Reset()
{
	FineMovingAverage.Reset();
	CoarseMovingAverage.Reset();

	HidePerformanceWarning();
	bIsNotificationAllowed = true;
}

void FPerformanceMonitor::UpdateMovingAverageSamplers()
{
	static const int NumberOfSamples = 50;
	IConsoleManager& Console = IConsoleManager::Get();

	float SampleTime = Console.FindConsoleVariable(TEXT("PerfWarn.FineSampleTime"))->GetFloat();
	FineMovingAverage = FMovingAverage(NumberOfSamples, SampleTime / NumberOfSamples);

	SampleTime = Console.FindConsoleVariable(TEXT("PerfWarn.CoarseSampleTime"))->GetFloat();
	CoarseMovingAverage = FMovingAverage(NumberOfSamples, SampleTime / NumberOfSamples);
}

void FPerformanceMonitor::ShowScalabilityDialog()
{
	Reset();
	bIsNotificationAllowed = false;

	auto ExistingWindow = ScalabilitySettingsWindowPtr.Pin();

	if (ExistingWindow.IsValid())
	{
		ExistingWindow->BringToFront();
	}
	else
	{
		// Create the window
		ScalabilitySettingsWindowPtr = ExistingWindow = SNew(SWindow)
			.Title(LOCTEXT("PerformanceWarningDialogTitle", "Scalability Options"))
			.SupportsMaximize(false)
			.SupportsMinimize(false)
			.CreateTitleBar(true)
			.SizingRule(ESizingRule::Autosized);

		ExistingWindow->SetOnWindowClosed(FOnWindowClosed::CreateStatic([](const TSharedRef<SWindow>&, FPerformanceMonitor* PerfWarn){
			PerfWarn->Reset();
		}, this));

		ExistingWindow->SetContent(
			SNew(SScalabilitySettingsDialog)
			.OnDoneClicked(
				FOnClicked::CreateStatic([](TWeakPtr<SWindow> Window, FPerformanceMonitor* PerfWarn){
					auto WindowPin = Window.Pin();
					if (WindowPin.IsValid())
					{
						PerfWarn->bIsNotificationAllowed = true;
						WindowPin->RequestDestroyWindow();
					}
					return FReply::Handled();
				}, ScalabilitySettingsWindowPtr, this)
			)
		);


		TSharedPtr<SWindow> RootWindow = FGlobalTabmanager::Get()->GetRootWindow();
		if (RootWindow.IsValid())
		{
			FSlateApplication::Get().AddModalWindow(ExistingWindow.ToSharedRef(), RootWindow);
		}
		else
		{
			FSlateApplication::Get().AddWindow(ExistingWindow.ToSharedRef());
		}
	}
}

#undef LOCTEXT_NAMESPACE
