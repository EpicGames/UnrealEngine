// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "TickableEditorObject.h"
#include "Scalability.h"

class SWindow;

/**
 * Helper class for a calculating a moving average. This works by maintaining an accumulator which is then sampled at regular intervals into the "Samples" array.
 */
struct FMovingAverage
{
	/** Constructor from a sample size, and sample rate */
	FMovingAverage(const int32 InSampleSize = 0, const float InSampleRateSeconds = 1.f)
		: CurrentSampleCount(0)
		, CurrentSampleAccumulator(0)
		, CurrentSampleStartTime(0)
		, SampleRateSeconds(InSampleRateSeconds)
		, SampleSize(InSampleSize)
		, SampleAverage(0)
		, NextSampleIndex(0)
	{
		Samples.Reserve(SampleSize);
	}
	
	/** Check if this data is reliable or not. Returns false if the sampler is not populated with enough data */
	inline bool IsReliable() const
	{
		return SampleSize != 0 && Samples.Num() == SampleSize;
	}

	/** Reset this sampler to its default (unpopulated) state */
	inline void Reset()
	{
		*this = FMovingAverage(SampleSize, SampleRateSeconds);
	}

	/** Get the average of all the samples contained in this sampler */
	inline float GetAverage() const
	{
		return SampleAverage;
	}

	/** Get the current number of available samples */
	inline int32 GetSampleCount() const
	{
		return Samples.Num();
	}

	/** Allow this sampler to tick the frame time, and potentially save a new sample */
	void Tick(double CurrentTime, float Value)
	{
		if (SampleSize == 0)
		{
			return;
		}

		if (CurrentSampleStartTime == 0)
		{
			CurrentSampleStartTime = CurrentTime;
		}

		++CurrentSampleCount;
		CurrentSampleAccumulator += Value;

		if (CurrentTime - CurrentSampleStartTime > SampleRateSeconds)
		{
			// Limit to a minimum of 5 frames per second
			const float AccumulatorAverage = FMath::Max(CurrentSampleAccumulator / CurrentSampleCount, 5.f);

			if (Samples.Num() == SampleSize)
			{
				Samples[NextSampleIndex] = AccumulatorAverage;
			}
			else
			{
				Samples.Add(AccumulatorAverage);
			}

			NextSampleIndex = (NextSampleIndex + 1) % SampleSize;
			ensure(NextSampleIndex >= 0 && NextSampleIndex < SampleSize);

			// calculate the average
			float Sum = 0.0;
			for (const auto& Sample : Samples)
			{
				Sum += Sample;
			}
			SampleAverage = Sum / Samples.Num();

			// reset the accumulator and counter
			CurrentSampleAccumulator = 0.0f;
			CurrentSampleCount = 0;
			CurrentSampleStartTime = CurrentTime;
		}
	}

	/** Return the percentage of samples that fall below the specified threshold */
	float PercentageBelowThreshold(const float Threshold) const
	{
		check(IsReliable());

		float NumBelowThreshold = 0;
		for (const auto& Sample : Samples)
		{
			if (Sample < Threshold)
			{
				++NumBelowThreshold;
			}
		}

		return (NumBelowThreshold / Samples.Num()) * 100.0f;
	}
private:
	/** The number of samples in the current accumulator */
	int32 CurrentSampleCount;
	/** The cumulative sum of samples for the current time period */
	float CurrentSampleAccumulator;
	/** The time at which we started accumulating the current sample */
	double CurrentSampleStartTime;

	/** The rate at which to store accumulated samples of FPS in seconds */
	float SampleRateSeconds;
	/** The maximum number of accumulated samples to store */
	int32 SampleSize;
	/** The actual average stored across all samples */
	float SampleAverage;

	/** The actual FPS samples */
	TArray<float> Samples;
	/** The index of the next sample to write to */
	int32 NextSampleIndex;
};

/** Notification class for performance warnings. */
class FPerformanceMonitor
	: public FTickableEditorObject
{

public:

	/** Constructor */
	FPerformanceMonitor();

	/** Destructor */
	~FPerformanceMonitor();

protected:

	/** FTickableEditorObject interface */
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
	virtual void Tick(float DeltaTime) override;
	virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(FPerformanceMonitor, STATGROUP_Tickables); }

private:

	/** Starts the notification. */
	void ShowPerformanceWarning(FText MessageText);

	/** Gets the quality levels that would be applied with auto-scalability. */
	Scalability::FQualityLevels GetAutoScalabilityQualityLevels() const;

	/** Returns true if the quality settings match the ones that would be applied with auto scalability. */
	bool WillAutoScalabilityHelp() const;

	/** Run a benchmark and auto apply scalability settings */
	void AutoApplyScalability();

	/** Adjusts the performance monitor settings to stop monitoring. */
	void CancelPerformanceNotification();

	/** Ends the notification. */
	void HidePerformanceWarning();

	/** Display the scalability dialog. */
	void ShowScalabilityDialog();

	/** Resets the performance warning data and hides the warning */
	void Reset();

	/** Update the moving average samplers to match the settings specified in the console variables */
	void UpdateMovingAverageSamplers();

	/** Moving average data for the fine and coarse-grained moving averages */
	FMovingAverage FineMovingAverage, CoarseMovingAverage;

	/** Tracks the last time the notification was started, used to avoid spamming. */
	double LastEnableTime;

	/** The time remaining before the auto scalability settings are automatically applied. */
	double NotificationTimeout;

	/** The notification window ptr */
	TWeakPtr<SNotificationItem> PerformanceWarningNotificationPtr;

	/** The Scalability Setting window we may be using */
	TWeakPtr<SWindow> ScalabilitySettingsWindowPtr;

	/** Whether the notification is allowed to pop up this session */
	bool bIsNotificationAllowed;

	/** Console variable delegate for checking when the console variables have changed */
	FConsoleCommandDelegate CVarDelegate;

	FConsoleVariableSinkHandle CVarDelegateHandle;
};
