// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicalMaterialMaskImport.h"
#include "PhysicalMaterials/PhysicalMaterialMask.h"
#include "CoreMinimal.h"
#include "Engine/Texture2D.h"
#include "EngineDefines.h"
#include "Misc/MessageDialog.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/FileManager.h"
#include "Misc/CoreMisc.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "Misc/PackageName.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "UObject/UObjectIterator.h"
#include "UObject/Package.h"
#include "UObject/Interface.h"
#include "Misc/PackageName.h"
#include "Fonts/FontBulkData.h"
#include "Fonts/CompositeFont.h"
#include "Input/Reply.h"
#include "Engine/Texture.h"
#include "Editor.h"
#include "EditorDirectories.h"
#include "EditorReimportHandler.h"
#include "EditorFramework/AssetImportData.h"
#include "Framework/Application/SlateApplication.h"
#include "ObjectTools.h"
#include "PackageTools.h"
#include "Modules/ModuleManager.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "ClassViewerModule.h"
#include "ClassViewerFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "IAssetTools.h"
#include "DesktopPlatformModule.h"
#include "IDesktopPlatform.h"
#include "Engine/Texture.h"
#include "Factories/TextureFactory.h"
#include "ScopedTransaction.h"

DEFINE_LOG_CATEGORY_STATIC(LogPhysicalMaterialMaskImport, Log, All);

#define LOCTEXT_NAMESPACE "PhysicalMaterialMaskImport"

void FPhysicalMaterialMaskImport::ImportMaskTexture(UPhysicalMaterialMask* PhysMatMask)
{
	check(PhysMatMask);

	const FString TextureFilename = OpenMaskFileDialog();

	if (TextureFilename.IsEmpty())
	{
		UE_LOGF(LogPhysicalMaterialMaskImport, Error, "No texture mask selected.");
	}

	if (UTexture* MaskTexture = ImportMaskTextureFile(PhysMatMask, TextureFilename))
	{
		const FScopedTransaction Transaction(LOCTEXT("PhysicalMaterialMask_ImportTexture", "Import Mask Texture"));

		PhysMatMask->SetMaskTexture(MaskTexture, TextureFilename);
	}
}

EReimportResult::Type FPhysicalMaterialMaskImport::ReimportMaskTexture(UPhysicalMaterialMask* PhysMatMask)
{
	check(PhysMatMask);
	check(PhysMatMask->AssetImportData);
	
	const FString TextureFilename = PhysMatMask->AssetImportData->GetFirstFilename();

	if (TextureFilename.IsEmpty())
	{
		UE_LOGF(LogPhysicalMaterialMaskImport, Error, "PhysicalMaterialMask does not have asset import data filename for reimport.");
		return EReimportResult::Failed;
	}

	if (UTexture* MaskTexture = ImportMaskTextureFile(PhysMatMask, TextureFilename))
	{
		const FScopedTransaction Transaction(LOCTEXT("PhysicalMaterialMask_ReimportTexture", "Reimport Mask Texture"));

		PhysMatMask->SetMaskTexture(MaskTexture, TextureFilename);
		return EReimportResult::Succeeded;
	}

	return EReimportResult::Failed;
}

EReimportResult::Type FPhysicalMaterialMaskImport::ReimportMaskTextureWithNewFile(UPhysicalMaterialMask* PhysMatMask)
{
	check(PhysMatMask);

	const FString TextureFilename = OpenMaskFileDialog();

	if (TextureFilename.IsEmpty())
	{
		UE_LOGF(LogPhysicalMaterialMaskImport, Error, "No texture mask selected.");
		return EReimportResult::Failed;
	}

	if (UTexture* MaskTexture = ImportMaskTextureFile(PhysMatMask, TextureFilename))
	{
		const FScopedTransaction Transaction(LOCTEXT("PhysicalMaterialMask_ReimportTextureNewFile", "Reimport Mask Texture With New File"));

		PhysMatMask->SetMaskTexture(MaskTexture, TextureFilename);
		return EReimportResult::Succeeded;
	}

	return EReimportResult::Failed;
}

FString FPhysicalMaterialMaskImport::OpenMaskFileDialog()
{
	TArray<FString> OutFiles;

	bool bOpen = false;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		FString Filter;
		GetFileTypeFilterString(Filter);
		const FString DefaultPath = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_IMPORT);

		bOpen = DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			NSLOCTEXT("Import", "Import", "Import from...").ToString(),
			FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_IMPORT),
			TEXT(""),
			*Filter,
			EFileDialogFlags::None,
			OutFiles
		);
	}
	if (!bOpen || OutFiles.Num() == 0)
	{
		return TEXT("");
	}

	return OutFiles[0];
}

UTexture* FPhysicalMaterialMaskImport::ImportMaskTextureFile(UPhysicalMaterialMask* PhysMatMask, const FString& TextureFilename)
{
	if (TextureFilename.IsEmpty())
	{
		return nullptr;
	}

	FString Extension = FPaths::GetExtension(TextureFilename).ToLower();
	FString TextureName = FPaths::GetBaseFilename(TextureFilename) + TEXT("_Mask");
	TextureName = ObjectTools::SanitizeObjectName(TextureName);

	UTexture* Texture = nullptr;

	// try opening from absolute path
	TArray<uint8> TextureData;
	if (!(FFileHelper::LoadFileToArray(TextureData, *TextureFilename) && TextureData.Num() > 0))
	{
		UE_LOGF(LogPhysicalMaterialMaskImport, Error, "Unable to find texture file %ls.", *TextureFilename);
	}
	else
	{
		UTextureFactory* TextureFactory = NewObject<UTextureFactory>();
		TextureFactory->AddToRoot();
		TextureFactory->SuppressImportOverwriteDialog();

		TextureFactory->CompressionSettings = TC_Masks;

		const uint8* PtrTexture = TextureData.GetData();
		Texture = (UTexture*)TextureFactory->FactoryCreateBinary(UTexture2D::StaticClass(), PhysMatMask->GetOuter(), *TextureName, RF_NoFlags, NULL, *Extension, PtrTexture, PtrTexture + TextureData.Num(), GWarn);
		if (Texture != NULL)
		{
			Texture->PreEditChange(nullptr);
			Texture->SRGB = false;
			Texture->CompressionNone = true;
			Texture->CompressionSettings = TC_Masks;
			Texture->MipGenSettings = TMGS_NoMipmaps;
			Texture->Filter = TF_Nearest;
			Texture->AssetImportData->Update(TextureFilename);
			Texture->PostEditChange();
		}

		TextureFactory->RemoveFromRoot();
	}

	return Texture;
}

void FPhysicalMaterialMaskImport::GetSupportedTextureFileTypes(TArray<FString>& OutFileTypes)
{
	OutFileTypes.Empty();
	OutFileTypes.Emplace(TEXT("png"));
	OutFileTypes.Emplace(TEXT("jpg"));
}

void FPhysicalMaterialMaskImport::GetSupportedTextureSourceFormats(TArray<ETextureSourceFormat>& OutSourceFormats)
{
	OutSourceFormats.Empty();
	OutSourceFormats.Emplace(ETextureSourceFormat::TSF_BGRA8);
	OutSourceFormats.Emplace(ETextureSourceFormat::TSF_RGBA16);
}

void FPhysicalMaterialMaskImport::GetFileTypeFilterString(FString& OutFileTypeFilter)
{
	TArray<FString> SupportedTypes;
	GetSupportedTextureFileTypes(SupportedTypes);

	bool bJoin = false;
	OutFileTypeFilter = FString(TEXT("Image file ("));
	for (const FString& FileType : SupportedTypes)
	{
		if (bJoin)
		{
			OutFileTypeFilter += TEXT(',');
		}
		const FString Type = FString::Printf(TEXT("*.%s"), *FileType);
		OutFileTypeFilter += Type;
		bJoin = true;
	}
	OutFileTypeFilter += FString(TEXT(")|"));
	bJoin = false;
	for (const FString& FileType : SupportedTypes)
	{
		if (bJoin)
		{
			OutFileTypeFilter += TEXT(';');
		}
		const FString Type = FString::Printf(TEXT("*.%s"), *FileType);
		OutFileTypeFilter += Type;
		bJoin = true;
	}
}

#undef LOCTEXT_NAMESPACE

