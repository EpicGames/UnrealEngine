// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsManipulationMode.h"
#include "Components/PrimitiveComponent.h"
#include "EditorViewportClient.h"
#include "PhysicsEngine/PhysicsHandleComponent.h"
#include "Engine/Selection.h"
#include "EditorModeManager.h"
#include "EditorModes.h"

DEFINE_LOG_CATEGORY_STATIC(LogEditorPhysMode, Log, All);

// Disable drag handles for PIE manipulation and just teleport (handy for some internal physics debugging modes)
bool bDisableEditorPhysicsHandle = false;
FAutoConsoleVariableRef CVarHackMaxVelocity(TEXT("p.DisableEditorPhysicsHandle"), bDisableEditorPhysicsHandle, TEXT("When true, disable the physics spring for dragging objects in PIE. Use a teleport instead."));


void FPhysicsManipulationEdModeFactory::OnSelectionChanged(FEditorModeTools& Tools, UObject* ItemUndergoingChange) const
{
	USelection* Selection = GEditor->GetSelectedActors();

	if (ItemUndergoingChange != NULL && ItemUndergoingChange->IsSelected())
	{
		AActor* SelectedActor = Cast<AActor>(ItemUndergoingChange);
		if (SelectedActor != NULL)
		{
			UPrimitiveComponent* PC = Cast<UPrimitiveComponent>(SelectedActor->GetRootComponent());
			// Note the physics handle to be grabbed will be accessed via the GetBodyInstance() function, not via direct access to the BodyInstance member
			if (PC != NULL && PC->GetBodyInstance() != nullptr && PC->BodyInstance.bSimulatePhysics)
			{
				Tools.ActivateMode(FBuiltinEditorModes::EM_Physics);
				return;
			}
		}
	}
	else if (ItemUndergoingChange != NULL && !ItemUndergoingChange->IsA(USelection::StaticClass()))
	{
		Tools.DeactivateMode(FBuiltinEditorModes::EM_Physics);
	}
}

FEditorModeInfo FPhysicsManipulationEdModeFactory::GetModeInfo() const
{
	return FEditorModeInfo(FBuiltinEditorModes::EM_Physics, NSLOCTEXT("EditorModes", "PhysicsMode", "Physics Mode"));
}

TSharedRef<FEdMode> FPhysicsManipulationEdModeFactory::CreateMode() const
{
	return MakeShareable( new FPhysicsManipulationEdMode );
}

FPhysicsManipulationEdMode::FPhysicsManipulationEdMode()
{
	HandleComp = NewObject<UPhysicsHandleComponent>();
}

FPhysicsManipulationEdMode::~FPhysicsManipulationEdMode()
{
	HandleComp = NULL;
}

void FPhysicsManipulationEdMode::Enter()
{
	FWorldContext* PIEWorldContext = GEditor->GetPIEWorldContext();
	check(PIEWorldContext && PIEWorldContext->World());
	HandleComp->RegisterComponentWithWorld(PIEWorldContext->World());
}

void FPhysicsManipulationEdMode::Exit()
{
	HandleComp->UnregisterComponent();
}

bool FPhysicsManipulationEdMode::InputDelta( FEditorViewportClient* InViewportClient,FViewport* InViewport,FVector& InDrag,FRotator& InRot,FVector& InScale )
{
	//UE_LOGF(LogEditorPhysMode, Warning, "Mouse: %ls InDrag: %ls  InRot: %ls", *GEditor->MouseMovement.ToString(), *InDrag.ToString(), *InRot.ToString());

	const float GrabMoveSpeed = 1.0f;
	const float GrabRotateSpeed = 1.0f;

	if (InViewportClient->GetCurrentWidgetAxis() != EAxisList::None && HandleComp->GrabbedComponent)
	{
		HandleTargetLocation += InDrag * GrabMoveSpeed;
		HandleTargetRotation += InRot;

		HandleComp->SetTargetLocation(HandleTargetLocation);
		HandleComp->SetTargetRotation(HandleTargetRotation);

		// We cannot use a spring to move the object when the world is paused. Teleport instead
		const bool bTeleportObject = (GetWorld() && GetWorld()->IsPaused()) || bDisableEditorPhysicsHandle;
		if (bTeleportObject)
		{
			HandleComp->GrabbedComponent->SetWorldLocationAndRotation(HandleTargetLocation, HandleTargetRotation, false, nullptr, ETeleportType::TeleportPhysics);
			HandleComp->GrabbedComponent->SetPhysicsLinearVelocity(FVector::Zero());
			HandleComp->GrabbedComponent->SetPhysicsAngularVelocityInRadians(FVector::Zero());
		}

		return true;
	}
	else
	{
		return FEdMode::InputDelta(InViewportClient, InViewport, InDrag, InRot, InScale);
	}
}

bool FPhysicsManipulationEdMode::StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	//UE_LOGF(LogEditorPhysMode, Warning, "Start Tracking");

	if (InViewportClient->GetCurrentWidgetAxis() != EAxisList::None)
	{
		FVector GrabLocation(0,0,0);
		UPrimitiveComponent* ComponentToGrab = NULL;
	
		USelection* Selection = GEditor->GetSelectedActors();

		for (int32 i=0; i<Selection->Num(); ++i)
		{
			AActor* SelectedActor = Cast<AActor>(Selection->GetSelectedObject(i));

			if (SelectedActor != NULL)
			{
				UPrimitiveComponent* PC = Cast<UPrimitiveComponent>(SelectedActor->GetRootComponent());

				if (PC != NULL && PC->BodyInstance.bSimulatePhysics)
				{
					ComponentToGrab = PC;
				
					HandleTargetLocation = SelectedActor->GetActorLocation();
					HandleTargetRotation = SelectedActor->GetActorRotation();
					break;
				}

				if (ComponentToGrab != NULL) { break; }
			}
		}

		if (ComponentToGrab != NULL)
		{
			HandleComp->GrabComponentAtLocationWithRotation(ComponentToGrab, NAME_None, ComponentToGrab->GetOwner()->GetActorLocation(), ComponentToGrab->GetOwner()->GetActorRotation());
		}
	}

	return FEdMode::StartTracking(InViewportClient, InViewport);
}

bool FPhysicsManipulationEdMode::EndTracking( FEditorViewportClient* InViewportClient, FViewport* InViewport )
{
	//UE_LOGF(LogEditorPhysMode, Warning, "End Tracking");
	HandleComp->ReleaseComponent();

	return FEdMode::EndTracking(InViewportClient, InViewport);
}

void FPhysicsManipulationEdMode::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObject(HandleComp);
}
