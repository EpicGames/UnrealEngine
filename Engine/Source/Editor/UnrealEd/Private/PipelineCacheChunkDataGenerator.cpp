// Copyright Epic Games, Inc. All Rights Reserved.

#include "PipelineCacheChunkDataGenerator.h"

#include "IPlatformFileSandboxWrapper.h"
#include "Interfaces/ITargetPlatform.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ConfigCacheIni.h"
#include "PipelineCacheUtilities.h"

class FName;

FPipelineCacheChunkDataGenerator::FPipelineCacheChunkDataGenerator(const ITargetPlatform* TargetPlatform, const FString& InShaderLibraryName)
{
	// Find out if this platform requires stable shader keys, by reading the platform setting file.
	bOptedOut = false;
	PlatformNameUsedForIni = TargetPlatform->IniPlatformName();
	ShaderLibraryName = InShaderLibraryName;

	FConfigFile PlatformIniFile;
	FConfigCacheIni::LoadLocalIniFile(PlatformIniFile, TEXT("Engine"), true, *PlatformNameUsedForIni);
	PlatformIniFile.GetBool(TEXT("DevOptions.Shaders"), TEXT("bDoNotChunkPSOCache"), bOptedOut);
}


void FPipelineCacheChunkDataGenerator::GenerateChunkDataFiles(const int32 InChunkId, const TSet<FName>& InPackagesInChunk, const ITargetPlatform* TargetPlatform, FSandboxPlatformFile* InSandboxFile, TArray<FString>& OutChunkFilenames)
{
	if (!bOptedOut && InPackagesInChunk.Num() > 0)
	{
		checkf(PlatformNameUsedForIni == TargetPlatform->IniPlatformName(), TEXT("Mismatch between platform names in PSO cache chunk generator. Ini settings might have been applied incorrectly."));

		// get the sandbox content directory here, to relieve shaderlib from including the wrapper
		const FString InPlatformName = TargetPlatform->PlatformName();
		const FString ContentSandboxRoot = (InSandboxFile->GetSandboxDirectory() / InSandboxFile->GetGameSandboxDirectoryName() / TEXT("Content")).Replace(TEXT("[Platform]"), *InPlatformName);

		UE::PipelineCacheUtilities::SaveChunkInfo(ShaderLibraryName, InChunkId, InPackagesInChunk, TargetPlatform, ContentSandboxRoot, OutChunkFilenames);
	}
}
