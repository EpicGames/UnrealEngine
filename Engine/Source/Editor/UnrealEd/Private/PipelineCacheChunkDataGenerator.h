// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PipelineCacheChunkDataGenerator.h: parts of PSO cache that are only used in the editor
=============================================================================*/

#pragma once

#include "Commandlets/IChunkDataGenerator.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"

class FName;
class FSandboxPlatformFile;
class ITargetPlatform;

/**
 * Implementation for splitting shader library into chunks when creating streaming install manifests.
 */
class FPipelineCacheChunkDataGenerator : public IChunkDataGenerator
{
	/** As a temporary/transitional feature, allow opting out from chunking per platform. */
	bool bOptedOut;

	/** Temporary/transitional - this holds the platform name whose ini we checked for the opt out. */
	FString PlatformNameUsedForIni;

	/** Shader code library this pipeline cache is associated with. */
	FString ShaderLibraryName;

public:

	/** Target platform passed in MUST match target platform to generate chunks for. This is checked.*/
	FPipelineCacheChunkDataGenerator(const ITargetPlatform* TargetPlatform, const FString& InShaderLibraryName);
	virtual ~FPipelineCacheChunkDataGenerator() = default;

	//~ IChunkDataGenerator
	virtual void GenerateChunkDataFiles(const int32 InChunkId, const TSet<FName>& InPackagesInChunk, const ITargetPlatform* TargetPlatform, FSandboxPlatformFile* InSandboxFile, TArray<FString>& OutChunkFilenames) override;
};
