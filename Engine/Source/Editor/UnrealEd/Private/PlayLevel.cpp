// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayLevel.h"
#include "CoreMinimal.h"
#include "AssetCompilingManager.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Misc/MessageDialog.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "Misc/Guid.h"
#include "Stats/Stats.h"
#include "GenericPlatform/GenericApplication.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "UObject/ObjectMacros.h"
#include "UObject/GarbageCollection.h"
#include "UObject/Class.h"
#include "UObject/UObjectIterator.h"
#include "UObject/Package.h"
#include "UObject/LazyObjectPtr.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/ReferenceChainSearch.h"
#include "Misc/PackageName.h"
#include "InputCoreTypes.h"
#include "Layout/Margin.h"
#include "Layout/SlateRect.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SWindow.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SViewport.h"
#include "Framework/Docking/TabManager.h"
#include "Styling/AppStyle.h"
#include "Engine/EngineTypes.h"
#include "Async/TaskGraphInterfaces.h"
#include "GameFramework/Actor.h"
#include "Engine/Blueprint.h"
#include "Engine/CoreSettings.h"
#include "Engine/GameViewportClient.h"
#include "Engine/GameInstance.h"
#include "Engine/RendererSettings.h"
#include "Engine/World.h"
#include "Settings/LevelEditorPlaySettings.h"
#include "AI/NavigationSystemBase.h"
#include "Editor/EditorEngine.h"
#include "Editor/UnrealEdEngine.h"
#include "Settings/ProjectPackagingSettings.h"
#include "GameMapsSettings.h"
#include "GeneralProjectSettings.h"
#include "Engine/NavigationObjectBase.h"
#include "GameFramework/InputSettings.h"
#include "GameFramework/PlayerStart.h"
#include "GameFramework/GameModeBase.h"
#include "Components/AudioComponent.h"
#include "Engine/Note.h"
#include "Engine/Selection.h"
#include "Engine/NetDriver.h"
#include "UnrealEngine.h"
#include "EngineUtils.h"
#include "Editor.h"
#include "Editor/Transactor.h"
#include "LevelEditorViewport.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "UnrealEdMisc.h"
#include "FileHelpers.h"
#include "UnrealEdGlobals.h"
#include "EditorAnalytics.h"
#include "AudioDevice.h"
#include "BusyCursor.h"
#include "ScopedTransaction.h"
#include "PackageTools.h"
#include "Slate/SceneViewport.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "DataDrivenShaderPlatformInfo.h"


#include "LevelEditor.h"
#include "IAssetViewport.h"
#include "BlueprintEditorModule.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Interfaces/IMainFrameModule.h"
#include "Logging/TokenizedMessage.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "Misc/MapErrors.h"
#include "GameProjectGenerationModule.h"
#include "SourceCodeNavigation.h"
#include "Physics/PhysicsInterfaceCore.h"
#include "Physics/PhysicsInterfaceScene.h"
#include "AnalyticsEventAttribute.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "EngineAnalytics.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Engine/LocalPlayer.h"
#include "Slate/SGameLayerManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Widgets/Input/SHyperlink.h"
#include "Dialog/SCustomDialog.h"

#include "IHeadMountedDisplay.h"
#include "IXRTrackingSystem.h"
#include "Engine/LevelStreaming.h"
#include "Components/ModelComponent.h"
#include "GameDelegates.h"
#include "Net/OnlineEngineInterface.h"
#include "Kismet2/DebuggerCommands.h"
#include "Misc/ScopeExit.h"
#include "IVREditorModule.h"
#include "EditorModeRegistry.h"
#include "PhysicsManipulationMode.h"
#include "CookerSettings.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Async/Async.h"
#include "UObject/SoftObjectPath.h"
#include "IAssetViewport.h"
#include "IPIEAuthorizer.h"
#include "Features/IModularFeatures.h"
#include "Containers/ConsumeAllMpmcQueue.h"
#include "TickableEditorObject.h"
#include "IOutOfDateAssetCompilation.h"

DEFINE_LOG_CATEGORY(LogPlayLevel);

#define LOCTEXT_NAMESPACE "PlayLevel"

const static FName NAME_CategoryPIE("PIE");

// Forward declare local utility functions
FText GeneratePIEViewportWindowTitle(const EPlayNetMode InNetMode, const FRequestPlaySessionParams& InSessionParams, const int32 ClientIndex, const int32 InViewportIndex, const float FixedTick, const bool bVRPreview);
bool IsPrimaryPIEClient(const FRequestPlaySessionParams& InPlaySessionParams, const int32 InClientIndex);

// This class listens to output log messages, and forwards warnings and errors to the message log
class FOutputLogErrorsToMessageLogProxy final : public FOutputDevice, public FTickableEditorObject
{
public:
	FOutputLogErrorsToMessageLogProxy()
	{
		GLog->AddOutputDevice(this);
	}

	~FOutputLogErrorsToMessageLogProxy()
	{
		GLog->FlushThreadedLogs();
		GLog->RemoveOutputDevice(this);
		Tick(0.0f);
	}

	// FOutputDevice Interface

	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
	{
		if (Verbosity <= ELogVerbosity::Warning)
		{
			FLine Line;
			Line.Message = FText::Format(LOCTEXT("OutputLogToMessageLog", "{0}: {1}"), FText::FromName(Category), FText::AsCultureInvariant(FString(V)));
			Line.Verbosity = Verbosity;
			if (IsInGameThread())
			{
				LogLine(Line);
			}
			else
			{
				QueuedLines.ProduceItem(MoveTemp(Line));
			}
		}
	}

	virtual bool CanBeUsedOnMultipleThreads() const override
	{
		return true;
	}

	// FTickableEditorObject Interface

	virtual void Tick(float DeltaTime) override
	{
		QueuedLines.ConsumeAllFifo(LogLine);
	}

	virtual ETickableTickType GetTickableTickType() const override
	{
		return ETickableTickType::Always;
	}

	virtual TStatId GetStatId() const override
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FOutputLogErrorsToMessageLogProxy, STATGROUP_Tickables);
	}

private:
	struct FLine
	{
		FText Message;
		ELogVerbosity::Type Verbosity;
	};

	static void LogLine(const FLine& Line)
	{
		switch (Line.Verbosity)
		{
		case ELogVerbosity::Warning:
			FMessageLog(NAME_CategoryPIE).SuppressLoggingToOutputLog(true).Warning(Line.Message);
			break;
		case ELogVerbosity::Error:
			FMessageLog(NAME_CategoryPIE).SuppressLoggingToOutputLog(true).Error(Line.Message);
			break;
		case ELogVerbosity::Fatal:
			checkf(false, TEXT("%s"), *Line.Message.ToString());
			break;
		}
	}

	UE::TConsumeAllMpmcQueue<FLine> QueuedLines;
};

void UEditorEngine::EndPlayMap()
{
	if ( bIsEndingPlay )
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UEditorEngine::EndPlayMap);

	TGuardValue<bool> GuardIsEndingPlay(bIsEndingPlay, true);

	FEditorDelegates::PrePIEEnded.Broadcast( bIsSimulatingInEditor );

	// Restore optionally minimized windows.
	// We always restore no matter what the setting is since it could be toggled during PIE.
	for (TWeakPtr<SWindow>& Window : MinimizedWindowsDuringPIE)
	{
		if (Window.IsValid())
		{
			Window.Pin()->Restore();
		}
	}
	MinimizedWindowsDuringPIE.Empty();
		
	// Clean up Soft Object Path remaps
	FSoftObjectPath::ClearPIEPackageNames();

	FlushAsyncLoading();

	if (GEngine->XRSystem.IsValid() && !bIsSimulatingInEditor)
	{
		GEngine->XRSystem->OnEndPlay(*GEngine->GetWorldContextFromWorld(PlayWorld));
	}

	EndPlayOnLocalPc();

	const FScopedBusyCursor BusyCursor;
	check(PlayWorld);

	// Enable screensavers when ending PIE.
	EnableScreenSaver( true );

	// Make a list of all the actors that should be selected
	TArray<TWeakObjectPtr<AActor>> SelectedActors;
	if ( ActorsThatWereSelected.Num() > 0 )
	{
		for ( int32 ActorIndex = 0; ActorIndex < ActorsThatWereSelected.Num(); ++ActorIndex )
		{
			TWeakObjectPtr<AActor> Actor = ActorsThatWereSelected[ ActorIndex ].Get();
			if (Actor.IsValid())
			{
				SelectedActors.Add( Actor );
			}
		}
		ActorsThatWereSelected.Empty();
	}
	else
	{
		for ( FSelectionIterator It( GetSelectedActorIterator() ); It; ++It )
		{
			AActor* Actor = static_cast<AActor*>( *It );
			if (Actor)
			{
				checkSlow( Actor->IsA(AActor::StaticClass()) );

				AActor* EditorActor = EditorUtilities::GetEditorWorldCounterpartActor(Actor);
				if (EditorActor)
				{
					SelectedActors.Add( EditorActor );
				}
			}
		}
	}

	// Deselect all objects, to avoid problems caused by property windows still displaying
	// properties for an object that gets garbage collected during the PIE clean-up phase.
	GEditor->SelectNone( true, true, false );
	GetSelectedActors()->DeselectAll();
	GetSelectedObjects()->DeselectAll();
	GetSelectedComponents()->DeselectAll();

	// let the editor and the world know
	FEditorDelegates::EndPIE.Broadcast(bIsSimulatingInEditor);
	FWorldDelegates::OnPIEEnded.Broadcast(PlayWorld->GetGameInstance());

	// clean up any previous Play From Here sessions
	if ( GameViewport != NULL && GameViewport->Viewport != NULL )
	{
		// Remove debugger commands handler binding
		GameViewport->OnGameViewportInputKey().Unbind();

		// Remove close handler binding
		GameViewport->OnCloseRequested().Remove(ViewportCloseRequestedDelegateHandle);

		GameViewport->CloseRequested(GameViewport->Viewport);
	}
	CleanupGameViewport();

	// Clean up each world individually
	TArray<FName> OnlineIdentifiers;
	TArray<UWorld*> WorldsBeingCleanedUp;
	bool bSeamlessTravelActive = false;

	for (int32 WorldIdx = WorldList.Num()-1; WorldIdx >= 0; --WorldIdx)
	{
		FWorldContext &ThisContext = WorldList[WorldIdx];
		if (ThisContext.WorldType == EWorldType::PIE)
		{
			if (ThisContext.World())
			{
				WorldsBeingCleanedUp.Add(ThisContext.World());
			}

			if (ThisContext.SeamlessTravelHandler.IsInTransition())
			{
				bSeamlessTravelActive = true;
			}

			if (ThisContext.World())
			{
				TeardownPlaySession(ThisContext);

				// Destroy the PendingNetGame netdriver if it never finished connecting.
				if (ThisContext.PendingNetGame && ThisContext.PendingNetGame->NetDriver)
				{
					if (ThisContext.PendingNetGame->NetDriver->ServerConnection)
					{
						CancelPending(ThisContext);
					}
					else
					{
						DestroyNamedNetDriver(ThisContext.PendingNetGame, ThisContext.PendingNetGame->NetDriver->NetDriverName);
					}
				}

				ShutdownWorldNetDriver(ThisContext.World());
			}

			// Cleanup online subsystems instantiated during PIE
			FName OnlineIdentifier = UOnlineEngineInterface::Get()->GetOnlineIdentifier(ThisContext);
			if (UOnlineEngineInterface::Get()->DoesInstanceExist(OnlineIdentifier))
			{
				OnlineIdentifiers.Add(OnlineIdentifier);
			}
		}
	}

	// We perform subsystem cleanup before removing the world contexts
	for (const FName& OnlineIdentifier : OnlineIdentifiers)
	{
		UOnlineEngineInterface::Get()->PIEEndCleanup(OnlineIdentifier);	
	}

	// Only remove from list after all PIE worlds have been torn down, to prevent callbacks from trying to access removed worldcontexts
	for (int32 WorldIdx = WorldList.Num() - 1; WorldIdx >= 0; --WorldIdx)
	{
		FWorldContext& ThisContext = WorldList[WorldIdx];
		if (ThisContext.WorldType == EWorldType::PIE)
		{
			WorldList.RemoveAt(WorldIdx);
		}
	}

	// If seamless travel is happening then there is likely additional PIE worlds that need tearing down so seek them out
	if (bSeamlessTravelActive)
	{
		for (TObjectIterator<UWorld> WorldIt; WorldIt; ++WorldIt)
		{
			if (WorldIt->IsPlayInEditor())
			{
				WorldsBeingCleanedUp.AddUnique(*WorldIt);
			}
		}
	}

	if (OnlineIdentifiers.Num())
	{
		UE_LOGF(LogPlayLevel, Display, "Shutting down PIE online subsystems");
		// Cleanup online subsystem shortly as we might be in a failed delegate 
		// have to do this in batch because timer delegate doesn't recognize bound data 
		// as a different delegate
		FTimerDelegate DestroyTimer;
		DestroyTimer.BindUObject(this, &UEditorEngine::CleanupPIEOnlineSessions, OnlineIdentifiers);
		GetTimerManager()->SetTimer(CleanupPIEOnlineSessionsTimerHandle, DestroyTimer, 0.1f, false);
	}
	
	{
		// We could have been toggling back and forth between simulate and pie before ending the play map
		// Make sure the property windows are cleared of any pie actors
		GUnrealEd->UpdateFloatingPropertyWindows();

		// Clean up any pie actors being referenced 
		GEngine->BroadcastLevelActorListChanged();
	}

	// Lose the EditorWorld pointer (this is only maintained while PIEing)
	FNavigationSystem::OnPIEEnd(*EditorWorld);

	FGameDelegates::Get().GetEndPlayMapDelegate().Broadcast();
	
	// find objects like Textures in the playworld levels that won't get garbage collected as they are marked RF_Standalone
	for (FThreadSafeObjectIterator It; It; ++It)
	{
		UObject* Object = *It;

		if (Object->GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor))
		{
			if (Object->HasAnyFlags(RF_Standalone))
			{
				// Clear RF_Standalone flag from objects in the levels used for PIE so they get cleaned up.
				Object->ClearFlags(RF_Standalone);
			}
			// Close any asset editors that are currently editing this object
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(Object);
		}
	}

	EditorWorld->bAllowAudioPlayback = true;
	EditorWorld = nullptr;

	// mark everything contained in the PIE worlds to be deleted
	for (UWorld* World : WorldsBeingCleanedUp)
	{
		// Because of the seamless travel the world might still be in the root set, so clear that
		World->RemoveFromRoot();

		// Occasionally during seamless travel the Levels array won't yet be populated so mark this world first
		// then pick up the sub-levels via the level iterator
		World->MarkObjectsPendingKill();

		for (auto LevelIt(World->GetLevelIterator()); LevelIt; ++LevelIt)
		{
			if (const ULevel* Level = *LevelIt)
			{
				// We already picked up the persistent level with the top level mark objects
				if (Level->GetOuter() != World)
				{
					CastChecked<UWorld>(Level->GetOuter())->MarkObjectsPendingKill();
				}
			}
		}

		for (ULevelStreaming* LevelStreaming : World->GetStreamingLevels())
		{
			// If an unloaded levelstreaming still has a loaded level we need to mark its objects to be deleted as well
			if (LevelStreaming->GetLoadedLevel() && (!LevelStreaming->ShouldBeLoaded() || !LevelStreaming->ShouldBeVisible()))
			{
				CastChecked<UWorld>(LevelStreaming->GetLoadedLevel()->GetOuter())->MarkObjectsPendingKill();
			}
		}
	}

	// mark all objects contained within the PIE game instances to be deleted
	for (TObjectIterator<UGameInstance> It; It; ++It)
	{
		auto MarkObjectPendingKill = [](UObject* Object)
		{
			Object->MarkAsGarbage();
		};
		ForEachObjectWithOuter(*It, MarkObjectPendingKill, EGetObjectsFlags::IncludeNestedObjects, RF_NoFlags, EInternalObjectFlags::Garbage);
		It->MarkAsGarbage();
	}

	// Flush any render commands and released accessed UTextures and materials to give them a chance to be collected.
	if ( FSlateApplication::IsInitialized() )
	{
		FSlateApplication::Get().FlushRenderState();
	}

	// Clean up any PIE world objects
	{
		// The trans buffer should never have a PIE object in it.  If it does though, reset it, which may happen sometimes with selection objects
		if( GEditor->Trans->ContainsPieObjects() )
		{
			GEditor->ResetTransaction( NSLOCTEXT("UnrealEd", "TransactionContainedPIEObject", "A PIE object was in the transaction buffer and had to be destroyed") );
		}

		// Garbage Collect
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}

	// Make sure that all objects in the temp levels were entirely garbage collected.
	TSet<UObject*> LeakedObjectsSet;
	TSet<UPackage*> LeakedPackages;
	for(FThreadSafeObjectIterator ObjectIt; ObjectIt; ++ObjectIt )
	{
		UObject* Object = *ObjectIt;
		UPackage* ObjectPackage = Object->GetOutermost();
		if (ObjectPackage->HasAnyPackageFlags(PKG_PlayInEditor))
		{
			LeakedPackages.Add(ObjectPackage);
			if (UWorld* TheWorld = UWorld::FindWorldInPackage(ObjectPackage))
			{
				LeakedObjectsSet.Add(TheWorld);
			}
			else
			{
				LeakedObjectsSet.Add(ObjectPackage);
			}
		}
	}

	for (TObjectIterator<UGameInstance> It; It; ++It)
	{
		if (It->HasAnyInternalFlags(EInternalObjectFlags::Garbage))
		{
			LeakedObjectsSet.Add(*It);
		}
	}

	TArray<UObject*> LeakedObjects = LeakedObjectsSet.Array();
	TArray<FString> Paths;
	if (LeakedObjects.Num() > 0)
	{
		Paths = FReferenceChainSearch::FindAndPrintStaleReferencesToObjects(LeakedObjects, EPrintStaleReferencesOptions::Ensure);
	}
	for (int32 i = 0; i < LeakedObjects.Num(); ++i)
	{
		const FString& Path = Paths[i];
		UObject* Object = LeakedObjects[i];

		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("Path"), FText::FromString(Paths[i]));
		Arguments.Add(TEXT("Object"), FText::FromString(Object->GetFullName()));

		// We cannot safely recover from this.
		if (UObjectBaseUtility::IsGarbageEliminationEnabled())
		{
			checkf(false, TEXT("%s"), *FText::Format(
				LOCTEXT("PIEObjectStillReferenced", "Object '{Object}' from PIE level still referenced. Shortest path from root: {Path}"), Arguments).ToString());
		}
		else
		{
			// Nonfatal error, we will rename objects to try and recover. 
			FText ErrorMessage = FText::Format(
				LOCTEXT("PIEAnObjectStillReferenced", "Object '{Object}' from PIE level still referenced. Shortest path from root: {Path}"), Arguments);
			FMessageLog(NAME_CategoryPIE).Error()
				->AddToken(FUObjectToken::Create(Object, FText::FromString(Object->GetFullName())))
				->AddToken(FTextToken::Create(ErrorMessage));
		}
	}

	// Try and recover by renaming leaked packages 
	if (!UObjectBaseUtility::IsGarbageEliminationEnabled())
	{
		for (UPackage* ObjectPackage : LeakedPackages)
		{
			ObjectPackage->ClearPackageFlags(PKG_PlayInEditor);
			ObjectPackage->ClearFlags(RF_Standalone);
			// We let it leak but it needs to be renamed to not collide with future attempts at creating the same object(s)
			FName NewName = MakeUniqueObjectName(nullptr, UPackage::StaticClass());
			UE_LOGF(LogTemp, Log, "Renaming PIE package from '%ls' to '%ls' to prevent future name collisions.",
				*ObjectPackage->GetName(), *NewName.ToString());
			ObjectPackage->Rename(*NewName.ToString(), nullptr, REN_DontCreateRedirectors | REN_DoNotDirty | REN_NonTransactional | REN_AllowPackageLinkerMismatch);
		}
	}

	// Final cleanup/reseting
	FWorldContext& EditorWorldContext = GEditor->GetEditorWorldContext();
	UPackage* Package = EditorWorldContext.World()->GetOutermost();

	// Spawn note actors dropped in PIE.
	if(GEngine->PendingDroppedNotes.Num() > 0)
	{
		const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "CreatePIENoteActors", "Create PIE Notes") );

		for(int32 i=0; i<GEngine->PendingDroppedNotes.Num(); i++)
		{
			FDropNoteInfo& NoteInfo = GEngine->PendingDroppedNotes[i];
			ANote* NewNote = EditorWorldContext.World()->SpawnActor<ANote>(NoteInfo.Location, NoteInfo.Rotation);
			if(NewNote)
			{
				NewNote->Text = NoteInfo.Comment;
				if( NewNote->GetRootComponent() != NULL )
				{
					NewNote->GetRootComponent()->SetRelativeScale3D( FVector(2.f) );
				}
			}
		}
		Package->MarkPackageDirty();
		GEngine->PendingDroppedNotes.Empty();
	}

	//ensure stereo rendering is disabled in case we need to re-enable next PIE run 
	if (GEngine->StereoRenderingDevice)
	{
		GEngine->StereoRenderingDevice->EnableStereo(false);
	}

	// Restores realtime viewports that have been disabled for PIE.
	const FText SystemDisplayName = LOCTEXT("RealtimeOverrideMessage_PIE", "Play in Editor");
	RemoveViewportsRealtimeOverride(SystemDisplayName);

	EnableWorldSwitchCallbacks(false);

	// Set the autosave timer to have at least 10 seconds remaining before autosave
	const static float SecondsWarningTillAutosave = 10.0f;
	GUnrealEd->GetPackageAutoSaver().ForceMinimumTimeTillAutoSave(SecondsWarningTillAutosave);

	for(TObjectIterator<UAudioComponent> It; It; ++It)
	{
		UAudioComponent* AudioComp = *It;
		if (AudioComp->GetWorld() == EditorWorldContext.World())
		{
			AudioComp->ReregisterComponent();
		}
	}

	if (PlayInEditorSessionInfo.IsSet())
	{
		// Save the window positions to the CDO object.
		ULevelEditorPlaySettings* PlaySettingsConfig = GetMutableDefault<ULevelEditorPlaySettings>();

		for (int32 WindowIndex = 0; WindowIndex < PlayInEditorSessionInfo->CachedWindowInfo.Num(); WindowIndex++)
		{
			if (WindowIndex < PlaySettingsConfig->MultipleInstancePositions.Num())
			{
				PlaySettingsConfig->MultipleInstancePositions[WindowIndex] = PlayInEditorSessionInfo->CachedWindowInfo[WindowIndex].Position;
			}
			else
			{
				PlaySettingsConfig->MultipleInstancePositions.Add(PlayInEditorSessionInfo->CachedWindowInfo[WindowIndex].Position);
			}

			// Update the position where the primary PIE window will be opened (this also updates its displayed value in "Editor Preferences" --> "Level Editor" --> "Play" --> "New Window Position")
			if (IsPrimaryPIEClient(PlayInEditorSessionInfo->OriginalRequestParams, WindowIndex))
			{
				// Remember last known size
				PlaySettingsConfig->LastSize = PlayInEditorSessionInfo->CachedWindowInfo[WindowIndex].Size;

				// Only update it if "Always center window to screen" is disabled, and the size was not 0 (which means it is attached to the editor rather than being an standalone window)
				if (!PlaySettingsConfig->CenterNewWindow && PlaySettingsConfig->LastSize.X > 0 && PlaySettingsConfig->LastSize.Y > 0)
				{
					PlaySettingsConfig->NewWindowPosition = PlaySettingsConfig->MultipleInstancePositions[WindowIndex];
					PlaySettingsConfig->NewWindowWidth = PlaySettingsConfig->LastSize.X;
					PlaySettingsConfig->NewWindowHeight = PlaySettingsConfig->LastSize.Y;
				}
			}
		}

		PlaySettingsConfig->PostEditChange();
		PlaySettingsConfig->SaveConfig();
	}
	PlayInEditorSessionInfo.Reset();

	// no longer queued
	CancelRequestPlaySession();
	bRequestEndPlayMapQueued = false;

	// Tear down the output log to message log thunker
	OutputLogErrorsToMessageLogProxyPtr.Reset();

	// Remove undo barrier
	GUnrealEd->Trans->RemoveUndoBarrier();

	// display any info if required.
	FMessageLog(NAME_CategoryPIE).Notify(LOCTEXT("PIEErrorsPresent", "Errors/warnings reported while playing in editor."));

	FMessageLog(NAME_CategoryPIE).Open(EMessageSeverity::Warning);

	{
		// Temporary until the deprecated variable is removed
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bIsSimulatingInEditor = false;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * For every actor that was selected previously, make sure it's editor equivalent is selected. We do that after the cleanup in case some actor where removed.
	 * This must be done at after bIsSimulatingInEditor is set to false because some of the selection logic can have a dependency on the state of this variable.
	 */
	GEditor->GetSelectedActors()->BeginBatchSelectOperation();
	for (int32 ActorIndex = 0; ActorIndex < SelectedActors.Num(); ++ActorIndex)
	{
		AActor* Actor = SelectedActors[ActorIndex].Get();
		if (Actor)
		{
			// We need to notify or else the manipulation transform widget won't appear, but only notify once at the end because OnEditorSelectionChanged is expensive for large groups. 
			SelectActor(Actor, true, false);
		}
	}
	GEditor->GetSelectedActors()->EndBatchSelectOperation(true);
	
	FEditorDelegates::ShutdownPIE.Broadcast(bIsSimulatingInEditor);
}

void UEditorEngine::CleanupPIEOnlineSessions(TArray<FName> OnlineIdentifiers)
{
	for (FName& OnlineIdentifier : OnlineIdentifiers)
	{
		UOnlineEngineInterface::Get()->ShutdownOnlineSubsystem(OnlineIdentifier);

		UE_LOGF(LogPlayLevel, Display, "Destroying online subsystem %ls", *OnlineIdentifier.ToString());
		UOnlineEngineInterface::Get()->DestroyOnlineSubsystem(OnlineIdentifier);
	}
}

void UEditorEngine::TeardownPlaySession(FWorldContext& PieWorldContext)
{
	check(PieWorldContext.WorldType == EWorldType::PIE);
	PlayWorld = PieWorldContext.World();
	PlayWorld->BeginTearingDown();

	if (!PieWorldContext.RunAsDedicated)
	{
		// Slate data for this pie world
		FSlatePlayInEditorInfo* const SlatePlayInEditorSession = SlatePlayInEditorMap.Find(PieWorldContext.ContextHandle);

		// Destroy Viewport
		if ( PieWorldContext.GameViewport != NULL && PieWorldContext.GameViewport->Viewport != NULL )
		{
			PieWorldContext.GameViewport->CloseRequested(PieWorldContext.GameViewport->Viewport);
		}
		CleanupGameViewport();
	
		// Clean up the slate PIE viewport if we have one
		if (SlatePlayInEditorSession)
		{
			if (SlatePlayInEditorSession->DestinationSlateViewport.IsValid())
			{
				TSharedPtr<IAssetViewport> Viewport = SlatePlayInEditorSession->DestinationSlateViewport.Pin();

				if(!bIsSimulatingInEditor)
				{
					// Set the editor viewport location to match that of Play in Viewport if we aren't simulating in the editor, we have a valid player to get the location from (unless we're going back to VR Editor, in which case we won't teleport the user.)
					if (bLastViewAndLocationValid == true && !GEngine->IsStereoscopic3D( Viewport->GetActiveViewport() ) )
					{
						bLastViewAndLocationValid = false;
						Viewport->GetAssetViewportClient().SetViewLocation( LastViewLocation );

						if( Viewport->GetAssetViewportClient().IsPerspective() )
						{
							// Rotation only matters for perspective viewports not orthographic
							Viewport->GetAssetViewportClient().SetViewRotation( LastViewRotation );
						}
					}
				}

				// No longer simulating in the viewport
				Viewport->GetAssetViewportClient().SetIsSimulateInEditorViewport( false );

				
				FEditorModeRegistry::Get().UnregisterMode(FBuiltinEditorModes::EM_Physics);
				

				// Clear out the hit proxies before GC'ing
				Viewport->GetAssetViewportClient().Viewport->InvalidateHitProxy();
			}
			else if (SlatePlayInEditorSession->SlatePlayInEditorWindow.IsValid())
			{
				// Unregister the game viewport from slate.  This sends a final message to the viewport
				// so it can have a chance to release mouse capture, mouse lock, etc.		
				FSlateApplication::Get().UnregisterGameViewport();

				// Viewport client is cleaned up.  Make sure it's not being accessed.
				SlatePlayInEditorSession->SlatePlayInEditorWindowViewport->SetViewportClient(nullptr);
				SlatePlayInEditorSession->SlatePlayInEditorWindowViewport.Reset();

				// The window may have already been destroyed in the case that the PIE window close box was pressed 
				if (SlatePlayInEditorSession->SlatePlayInEditorWindow.IsValid())
				{
					// Destroy the SWindow
					FSlateApplication::Get().DestroyWindowImmediately(SlatePlayInEditorSession->SlatePlayInEditorWindow.Pin().ToSharedRef());
				}
			}
		}
	
		// Disassociate the players from their PlayerControllers.
		// This is done in the GameEngine path in UEngine::LoadMap.
		// But since PIE is just shutting down, and not loading a 
		// new map, we need to do it manually here for now.
		//for (auto It = GEngine->GetLocalPlayerIterator(PlayWorld); It; ++It)
		for (FLocalPlayerIterator It(GEngine, PlayWorld); It; ++It)
		{
			if(It->PlayerController)
			{
				if(It->PlayerController->GetPawn())
				{
					PlayWorld->DestroyActor(It->PlayerController->GetPawn(), true);
				}
				PlayWorld->DestroyActor(It->PlayerController, true);
				It->PlayerController = NULL;
			}
		}

	}

	// Change GWorld to be the play in editor world during cleanup.
	ensureMsgf( EditorWorld == GWorld, TEXT("TearDownPlaySession current world: %s"), GWorld ? *GWorld->GetName() : TEXT("No World"));
	GWorld = PlayWorld;
	GIsPlayInEditorWorld = true;
	
	// Remember Simulating flag so that we know if OnSimulateSessionFinished is required after everything has been cleaned up. 
	bool bWasSimulatingInEditor = PlayInEditorSessionInfo.IsSet() ? PlayInEditorSessionInfo->OriginalRequestParams.WorldType == EPlaySessionWorldType::SimulateInEditor : false;
	
	// Stop all audio and remove references to temp level.
	if (FAudioDevice* AudioDevice = PlayWorld->GetAudioDeviceRaw())
	{
		AudioDevice->Flush(PlayWorld);
		AudioDevice->ResetInterpolation();
		AudioDevice->OnEndPIE(false); // TODO: Should this have been bWasSimulatingInEditor?
		AudioDevice->SetTransientPrimaryVolume(1.0f);
		// Reset solo audio
		if (PlayInEditorSessionInfo.IsSet())
		{
			ULevelEditorPlaySettings* EditorPlaySettings = PlayInEditorSessionInfo->OriginalRequestParams.EditorPlaySettings;
			if (EditorPlaySettings && EditorPlaySettings->SoloAudioInFirstPIEClient && GEngine)
			{
				if (FWorldContext* WorldContext = GEngine->GetWorldContextFromWorld(PlayWorld))
				{
					if (WorldContext->bIsPrimaryPIEInstance)
					{
						if (FAudioDeviceManager* DeviceManager = AudioDevice->GetAudioDeviceManager())
						{
							DeviceManager->SetSoloDevice(INDEX_NONE);
						}
					}
				}
			}
		}
	}

	// Clean up all streaming levels
	PlayWorld->bIsLevelStreamingFrozen = false;
	PlayWorld->SetShouldForceUnloadStreamingLevels(true);
	PlayWorld->FlushLevelStreaming();

	// cleanup refs to any duplicated streaming levels
	for ( int32 LevelIndex=0; LevelIndex<PlayWorld->GetStreamingLevels().Num(); LevelIndex++ )
	{
		ULevelStreaming* StreamingLevel = PlayWorld->GetStreamingLevels()[LevelIndex];
		if( StreamingLevel != NULL )
		{
			const ULevel* PlayWorldLevel = StreamingLevel->GetLoadedLevel();
			if ( PlayWorldLevel != NULL )
			{
				UWorld* World = Cast<UWorld>( PlayWorldLevel->GetOuter() );
				if( World != NULL )
				{
					// Attempt to move blueprint debugging references back to the editor world
					if( EditorWorld != NULL && EditorWorld->GetStreamingLevels().IsValidIndex(LevelIndex) )
					{
						const ULevel* EditorWorldLevel = EditorWorld->GetStreamingLevels()[LevelIndex]->GetLoadedLevel();
						if ( EditorWorldLevel != NULL )
						{
							UWorld* SublevelEditorWorld  = Cast<UWorld>(EditorWorldLevel->GetOuter());
							if( SublevelEditorWorld != NULL )
							{
								World->TransferBlueprintDebugReferences(SublevelEditorWorld);
							}	
						}
					}
				}
			}
		}
	}

	// Construct a list of editors that are active for objects being debugged. We will refresh these when we have cleaned up to ensure no invalid objects exist in them
	TArray< IBlueprintEditor* > Editors;
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	const UWorld::FBlueprintToDebuggedObjectMap& EditDebugObjectsPre = PlayWorld->GetBlueprintObjectsBeingDebugged();
	for (UWorld::FBlueprintToDebuggedObjectMap::TConstIterator EditIt(EditDebugObjectsPre); EditIt; ++EditIt)
	{
		if (UBlueprint* TargetBP = EditIt.Key().Get())
		{
			if(IBlueprintEditor* EachEditor = static_cast<IBlueprintEditor*>(AssetEditorSubsystem->FindEditorForAsset(TargetBP, false)))
			{
				Editors.AddUnique( EachEditor );
			}
		}
	}
	
	// End play for all actors in the world
	PlayWorld->EndPlay(EEndPlayReason::EndPlayInEditor);
	
	PieWorldContext.OwningGameInstance->Shutdown();

	// Move blueprint debugging pointers back to the objects in the editor world
	PlayWorld->TransferBlueprintDebugReferences(EditorWorld);

	FPhysScene* PhysScene = PlayWorld->GetPhysicsScene();
	if (PhysScene)
	{
		PhysScene->WaitPhysScenes();
		PhysScene->KillVisualDebugger();
	}

	// Clean up the temporary play level.
	PlayWorld->CleanupWorld();

	// Remove from root (Seamless travel may have done this)
	PlayWorld->RemoveFromRoot();
		
	PlayWorld = NULL;

	// Refresh any editors we had open in case they referenced objects that no longer exist.
	for (int32 iEditors = 0; iEditors <  Editors.Num(); iEditors++)
	{
		Editors[ iEditors ]->RefreshEditors();
	}
	
	// Restore GWorld.
	GWorld = EditorWorld;
	GIsPlayInEditorWorld = false;

	// Restore the previously purged scene interface for the editor world to its original glory.
	GWorld->RestoreScene();

	FWorldContext& EditorWorldContext = GEditor->GetEditorWorldContext();

	// Let the viewport know about leaving PIE/Simulate session. Do it after everything's been cleaned up
	// as the viewport will play exit sound here and this has to be done after GetAudioDevice()->Flush
	// otherwise all sounds will be immediately stopped.
	if (!PieWorldContext.RunAsDedicated)
	{
		// Slate data for this pie world
		FSlatePlayInEditorInfo* const SlatePlayInEditorSession = SlatePlayInEditorMap.Find(PieWorldContext.ContextHandle);
		if (SlatePlayInEditorSession && SlatePlayInEditorSession->DestinationSlateViewport.IsValid())
		{
			TSharedPtr<IAssetViewport> Viewport = SlatePlayInEditorSession->DestinationSlateViewport.Pin();

			if( Viewport->HasPlayInEditorViewport() )
			{
				Viewport->EndPlayInEditorSession();
			}

			// Let the Slate viewport know that we're leaving Simulate mode
			if( bWasSimulatingInEditor )
			{
				Viewport->OnSimulateSessionFinished();
			}

			StaticCast<FLevelEditorViewportClient&>(Viewport->GetAssetViewportClient()).SetReferenceToWorldContext(EditorWorldContext);
		}

		// Remove the slate info from the map (note that the UWorld* is long gone at this point, but the WorldContext still exists. It will be removed outside of this function)
		SlatePlayInEditorMap.Remove(PieWorldContext.ContextHandle);
	}
}

// Deprecated, just format to match our new style.
void UEditorEngine::PlayMap(const FVector* StartLocation, const FRotator* StartRotation, int32 Destination, int32 InPlayInViewportIndex, bool bUseMobilePreview)
{
	// Deprecated, replaced with RequestPlaySession.
	FRequestPlaySessionParams Params;

	if (bUseMobilePreview)
	{
		Params.SessionDestination = EPlaySessionDestinationType::NewProcess;
		Params.SessionPreviewTypeOverride = EPlaySessionPreviewType::MobilePreview;
	}

	if (StartLocation)
	{
		Params.StartLocation = *StartLocation;
		Params.StartRotation = StartRotation ? *StartRotation : FRotator::ZeroRotator;
	}

	RequestPlaySession(Params);
}

void UEditorEngine::RequestPlaySession(const FRequestPlaySessionParams& InParams)
{
	// Store our Request to be operated on next Tick.
	PlaySessionRequest = InParams;

	// If they don't want to use a specific set of Editor Play Settings, fall back to the CDO.
	if (!PlaySessionRequest->EditorPlaySettings)
	{
		PlaySessionRequest->EditorPlaySettings = GetMutableDefault<ULevelEditorPlaySettings>();
	}

	// Now we duplicate their Editor Play Settings so that we can mutate it as part of startup
	// to help rule out invalid configuration combinations.
	FObjectDuplicationParameters DuplicationParams(PlaySessionRequest->EditorPlaySettings, GetTransientPackage());
	// Kept alive by AddReferencedObjects
	PlaySessionRequest->EditorPlaySettings = CastChecked<ULevelEditorPlaySettings>(StaticDuplicateObjectEx(DuplicationParams));


	// ToDo: Allow the CDO for the Game Instance to modify the settings after we copy them
	// so that they can validate user settings before attempting a launch.

	// Play Sessions can use the Game Mode to determine the default Player Start position
	// or the start position can be overridden by the incoming launch arguments
	if (InParams.SessionDestination == EPlaySessionDestinationType::Launcher)
	{
		check(InParams.LauncherTargetDevice.IsSet());
	}
}

void UEditorEngine::CancelRequestPlaySession()
{
	FEditorDelegates::CancelPIE.Broadcast();
	PlaySessionRequest.Reset();
	PlayInEditorSessionInfo.Reset();
}

bool UEditorEngine::SaveMapsForPlaySession()
{
	// Detect if we're dealing with an unsaved newly created world
	const bool bNewlyCreatedWorld = (FEditorFileUtils::GetFilename(GWorld).Len() == 0);

	// Save dirty packages, this is required because we're going to be launching a session outside of our normal process
	const bool bPromptUserToSave      = true;
	const bool bSaveMapPackages       = true;
	const bool bSaveContentPackages   = true;
	const bool bFastSave              = false;
	const bool bNotifyNoPackagesSaved = false;
	const bool bCanBeDeclined         = false;
	if (!FEditorFileUtils::SaveDirtyPackages(bPromptUserToSave, bSaveMapPackages, bSaveContentPackages, bFastSave, bNotifyNoPackagesSaved, bCanBeDeclined))
	{
		return false;
	}

	// Prompt the user to save the level if it has not been saved before. 
	// An unmodified but unsaved blank template level does not appear in the dirty packages check below.
	if (FEditorFileUtils::GetFilename(GWorld).Len() == 0)
	{		
		bool bSaved = FEditorFileUtils::SaveCurrentLevel();
		
		if (!bSaved)
		{
			return false;
		}
	}

	// In the case we're dealing with a newly created world it must be reloaded after being saved
	if (bNewlyCreatedWorld)
	{
		FEditorFileUtils::LoadMap(FEditorFileUtils::GetFilename(GWorld));
	}

	return true;
}

bool UEditorEngine::SetPIEWorldsPaused(bool Paused)
{
	bool WasPausedOrUnpaused = false;
	for (const FWorldContext& PieContext : GetWorldContexts())
	{
		UWorld * PieContextWorld = PieContext.World();
		if (PieContextWorld && PieContextWorld->IsGameWorld() && PieContextWorld->bDebugPauseExecution != Paused)
		{
			PieContextWorld->bDebugPauseExecution = Paused;
			WasPausedOrUnpaused = true;
		}
	}
	return WasPausedOrUnpaused;
}

void UEditorEngine::PlaySessionPaused()
{
	FEditorDelegates::PausePIE.Broadcast(PlayInEditorSessionInfo->OriginalRequestParams.WorldType == EPlaySessionWorldType::SimulateInEditor);
}

void UEditorEngine::PlaySessionResumed()
{
	FEditorDelegates::ResumePIE.Broadcast(PlayInEditorSessionInfo->OriginalRequestParams.WorldType == EPlaySessionWorldType::SimulateInEditor);
}

void UEditorEngine::PlaySessionSingleStepped()
{
	FEditorDelegates::SingleStepPIE.Broadcast(PlayInEditorSessionInfo->OriginalRequestParams.WorldType == EPlaySessionWorldType::SimulateInEditor);
}

bool UEditorEngine::ProcessDebuggerCommands(const FKey InKey, const FModifierKeysState ModifierKeyState, EInputEvent EventType )
{
	if( EventType == IE_Pressed )
	{
		return FPlayWorldCommands::GlobalPlayWorldActions->ProcessCommandBindings(InKey, ModifierKeyState, false);
	}
	
	return false;
}

void UEditorEngine::StartQueuedPlaySessionRequest()
{
	if (!PlaySessionRequest.IsSet())
	{
		UE_LOGF(LogPlayLevel, Warning, "StartQueuedPlaySessionRequest() called whith no request queued. Ignoring...");
		return;
	}

	StartQueuedPlaySessionRequestImpl();

	// Ensure the request is always reset after an attempt (which may fail)
	// so that we don't get stuck in an infinite loop of start attempts.
	PlaySessionRequest.Reset();
}

void UEditorEngine::StartQueuedPlaySessionRequestImpl()
{
	if (!ensureAlwaysMsgf(PlaySessionRequest.IsSet(), TEXT("StartQueuedPlaySessionRequest should not be called without a request set!")))
	{
		return;
	}

	// End any previous sessions running in separate processes.
	EndPlayOnLocalPc();

	// If there's level already being played, close it. (This may change GWorld). 
	if (PlayWorld && PlaySessionRequest->SessionDestination == EPlaySessionDestinationType::InProcess)
	{
		// Cache our Play Session Request, as EndPlayMap will clear it. When this function exits the request will be reset anyways.
		FRequestPlaySessionParams OriginalRequest = PlaySessionRequest.GetValue();
		// Immediately end the current play world.
		EndPlayMap(); 
		// Restore the request as we're now processing it.
		PlaySessionRequest = OriginalRequest;
	}

	// We want to use the ULevelEditorPlaySettings that come from the Play Session Request.
	// By the time this function gets called, these settings are a copy of either the CDO, 
	// or a user provided instance. The settings may have been modified by the game instance
	// after the request was made, to allow game instances to pre-validate settings.
	const ULevelEditorPlaySettings* EditorPlaySettings = PlaySessionRequest->EditorPlaySettings;
	check(EditorPlaySettings);

	PlayInEditorSessionInfo = FPlayInEditorSessionInfo();
	PlayInEditorSessionInfo->PlayRequestStartTime = FPlatformTime::Seconds();
	PlayInEditorSessionInfo->PlayRequestStartTime_StudioAnalytics = FPlatformTime::Seconds();;

	// Keep a copy of their original request settings for any late
	// joiners or async processes that need access to the settings after launch.
	PlayInEditorSessionInfo->OriginalRequestParams = PlaySessionRequest.GetValue();

	// Load the saved window positions from the EditorPlaySettings object.
	for (const FIntPoint& Position : EditorPlaySettings->MultipleInstancePositions)
	{
		FPlayInEditorSessionInfo::FWindowSizeAndPos& NewPos = PlayInEditorSessionInfo->CachedWindowInfo.Add_GetRef(FPlayInEditorSessionInfo::FWindowSizeAndPos());
		NewPos.Position = Position;
	}

	// If our settings require us to launch a separate process in any form, we require the user to save
	// their content so that when the new process reads the data from disk it will match what we have in-editor.
	bool bUserWantsInProcess;
	EditorPlaySettings->GetRunUnderOneProcess(bUserWantsInProcess);

	bool bIsSeparateProcess = PlaySessionRequest->SessionDestination != EPlaySessionDestinationType::InProcess;
	if (!bUserWantsInProcess)
	{
		int32 NumClients;
		EditorPlaySettings->GetPlayNumberOfClients(NumClients);

		EPlayNetMode NetMode;
		EditorPlaySettings->GetPlayNetMode(NetMode);

		// More than one client will spawn a second process.		
		bIsSeparateProcess |= NumClients > 1;

		// If they want to run anyone as a client, a dedicated server is started in a separate process.
		bIsSeparateProcess |= NetMode == EPlayNetMode::PIE_Client;
	}

	if (bIsSeparateProcess && !SaveMapsForPlaySession())
	{
		// Maps did not save, print a warning
		FText ErrorMsg = LOCTEXT("PIEWorldSaveFail", "PIE failed because map save was canceled");
		UE_LOGF(LogPlayLevel, Warning, "%ls", *ErrorMsg.ToString());
		FMessageLog(NAME_CategoryPIE).Warning(ErrorMsg);
		FMessageLog(NAME_CategoryPIE).Open();
		
		CancelRequestPlaySession();
		return;
	}

	FEditorDelegates::StartPIE.Broadcast(PlayInEditorSessionInfo->OriginalRequestParams.WorldType == EPlaySessionWorldType::SimulateInEditor);

	// We'll branch primarily based on the Session Destination, because it affects which settings we apply and how.
	switch (PlaySessionRequest->SessionDestination)
	{
	case EPlaySessionDestinationType::InProcess:
		// Create one-or-more PIE/SIE sessions inside of the current process.
		StartPlayInEditorSession(PlaySessionRequest.GetValue());
		break;
	case EPlaySessionDestinationType::NewProcess:
		// Create one-or-more PIE session by launching a new process on the local machine.
		StartPlayInNewProcessSession(PlaySessionRequest.GetValue());
		break;
	case EPlaySessionDestinationType::Launcher:
		// Create a Play Session via the Launcher which may be on a local or remote device.
		StartPlayUsingLauncherSession(PlaySessionRequest.GetValue());
		break;
	default:
		check(false);
	}
}

void UEditorEngine::EndPlayOnLocalPc( )
{
	for (int32 i=0; i < PlayOnLocalPCSessions.Num(); ++i)
	{
		if (PlayOnLocalPCSessions[i].ProcessHandle.IsValid())
		{
			if ( FPlatformProcess::IsProcRunning(PlayOnLocalPCSessions[i].ProcessHandle) )
			{
				FPlatformProcess::TerminateProc(PlayOnLocalPCSessions[i].ProcessHandle);
			}
			PlayOnLocalPCSessions[i].ProcessHandle.Reset();
		}
	}

	PlayOnLocalPCSessions.Empty();
}


int32 FInternalPlayLevelUtils::ResolveDirtyBlueprints(const bool bPromptForCompile, TArray<UBlueprint*>& ErroredBlueprints, const bool bForceLevelScriptRecompile)
{
	struct FLocal
	{
		static void OnMessageLogLinkActivated(const class TSharedRef<IMessageToken>& Token)
		{
			if (Token->GetType() == EMessageToken::Object)
			{
				const TSharedRef<FUObjectToken> UObjectToken = StaticCastSharedRef<FUObjectToken>(Token);
				if (UObjectToken->GetObject().IsValid())
				{
					FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(UObjectToken->GetObject().Get());
				}
			}
		}

		static void AddCompileErrorToLog(UBlueprint* ErroredBlueprint, FMessageLog& BlueprintLog)
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("Name"), FText::FromString(ErroredBlueprint->GetName()));

			TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Warning);
			Message->AddToken(FTextToken::Create(LOCTEXT("BlueprintCompileFailed", "Blueprint failed to compile: ")));
			Message->AddToken(FUObjectToken::Create(ErroredBlueprint, FText::FromString(ErroredBlueprint->GetName()))
				->OnMessageTokenActivated(FOnMessageTokenActivated::CreateStatic(&FLocal::OnMessageLogLinkActivated))
			);

			BlueprintLog.AddMessage(Message);
		}
	};

	const bool bAutoCompile = !bPromptForCompile;
	FString PromptDirtyList;

	TArray<UBlueprint*> InNeedOfRecompile;
	ErroredBlueprints.Empty();

	FMessageLog BlueprintLog("BlueprintLog");

	double BPRegenStartTime = FPlatformTime::Seconds();
	for (TObjectIterator<UBlueprint> BlueprintIt; BlueprintIt; ++BlueprintIt)
	{
		UBlueprint* Blueprint = *BlueprintIt;

		// ignore garbage and up-to-date BPs
		if (!IsValid(Blueprint) || Blueprint->IsUpToDate())
		{
			continue;
		}

		UPackage* Package = Blueprint->GetPackage();
		if(Package && Package->HasAnyPackageFlags(PKG_ForDiffing))
		{
			continue;
		}

		// do not try to recompile BPs that have not changed since they last failed to compile, so don't check Blueprint->IsUpToDate()
		const bool bIsDirtyAndShouldBeRecompiled = Blueprint->IsPossiblyDirty();
		if (!FBlueprintEditorUtils::IsDataOnlyBlueprint(Blueprint)
			&& (bIsDirtyAndShouldBeRecompiled || (FBlueprintEditorUtils::IsLevelScriptBlueprint(Blueprint) && bForceLevelScriptRecompile))
			&& (Blueprint->Status != BS_Unknown))
		{
			InNeedOfRecompile.Add(Blueprint);

			if (bPromptForCompile)
			{
				PromptDirtyList += FString::Printf(TEXT("\n   %s"), *Blueprint->GetName());
			}
		}
		else if (BS_Error == Blueprint->Status && Blueprint->bDisplayCompilePIEWarning)
		{
			ErroredBlueprints.Add(Blueprint);
			FLocal::AddCompileErrorToLog(Blueprint, BlueprintLog);
		}
	}

	bool bRunCompilation = bAutoCompile;
	if (bPromptForCompile && (InNeedOfRecompile.Num() > 0))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("DirtyBlueprints"), FText::FromString(PromptDirtyList));
		const FText PromptMsg = FText::Format(NSLOCTEXT("PlayInEditor", "PrePIE_BlueprintsDirty", "One or more blueprints have been modified without being recompiled. Do you want to compile them now? \n{DirtyBlueprints}"), Args);

		EAppReturnType::Type PromptResponse = FMessageDialog::Open(EAppMsgType::YesNo, PromptMsg);
		bRunCompilation = (PromptResponse == EAppReturnType::Yes);
	}
	int32 RecompiledCount = 0;

	if (bRunCompilation && (InNeedOfRecompile.Num() > 0))
	{
		const FText LogPageLabel = (bAutoCompile) ? LOCTEXT("BlueprintAutoCompilationPageLabel", "Pre-Play auto-recompile") :
			LOCTEXT("BlueprintCompilationPageLabel", "Pre-Play recompile");
		BlueprintLog.NewPage(LogPageLabel);

		TArray<UBlueprint*> CompiledBlueprints;
		auto OnBlueprintPreCompileLambda = [&CompiledBlueprints](UBlueprint* InBlueprint)
		{
			check(InBlueprint != nullptr);

			if (CompiledBlueprints.Num() == 0)
			{
				UE_LOGF(LogPlayLevel, Log, "[PlayLevel] Compiling %ls before play...", *InBlueprint->GetName());
			}
			else
			{
				UE_LOGF(LogPlayLevel, Log, "[PlayLevel]   Compiling %ls as a dependent...", *InBlueprint->GetName());
			}

			CompiledBlueprints.Add(InBlueprint);
		};

		// Register compile callback
		FDelegateHandle PreCompileDelegateHandle = GEditor->OnBlueprintPreCompile().AddLambda(OnBlueprintPreCompileLambda);

		// Recompile all necessary blueprints in a single loop, saving GC until the end
		for (auto BlueprintIt = InNeedOfRecompile.CreateIterator(); BlueprintIt; ++BlueprintIt)
		{
			UBlueprint* Blueprint = *BlueprintIt;

			int32 CurrItIndex = BlueprintIt.GetIndex();

			// Compile the Blueprint (note: re-instancing may trigger additional compiles for child/dependent Blueprints; see callback above)
			FKismetEditorUtilities::CompileBlueprint(Blueprint, EBlueprintCompileOptions::SkipGarbageCollection);

			// Check for errors after compiling
			for (UBlueprint* CompiledBlueprint : CompiledBlueprints)
			{
				if (CompiledBlueprint != Blueprint)
				{
					int32 ExistingIndex = InNeedOfRecompile.Find(CompiledBlueprint);
					// if this dependent blueprint is already set up to compile 
					// later in this loop, then there is no need to add it to be recompiled again
					if (ExistingIndex > CurrItIndex)
					{
						InNeedOfRecompile.RemoveAt(ExistingIndex);
					}
				}

				const bool bHadError = (!CompiledBlueprint->IsUpToDate() && CompiledBlueprint->Status != BS_Unknown);

				// Check if the Blueprint has already been added to the error list to prevent it from being added again
				if (bHadError && ErroredBlueprints.Find(CompiledBlueprint) == INDEX_NONE)
				{
					ErroredBlueprints.Add(CompiledBlueprint);
					FLocal::AddCompileErrorToLog(CompiledBlueprint, BlueprintLog);
				}

				++RecompiledCount;
			}

			// Reset for next pass
			CompiledBlueprints.Empty();
		}

		// Now that all Blueprints have been compiled, run a single GC pass to clean up artifacts
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

		// Unregister compile callback
		GEditor->OnBlueprintPreCompile().Remove(PreCompileDelegateHandle);

		UE_LOGF(LogPlayLevel, Log, "PlayLevel: Blueprint regeneration took %d ms (%i blueprints)", (int32)((FPlatformTime::Seconds() - BPRegenStartTime) * 1000), RecompiledCount);
	}
	else if (bAutoCompile)
	{
		UE_LOGF(LogPlayLevel, Log, "PlayLevel: No blueprints needed recompiling");
	}

	return RecompiledCount;
}

void UEditorEngine::RequestEndPlayMap()
{
	if( PlayWorld )
	{
		bRequestEndPlayMapQueued = true;

		// Cache the position and rotation of the camera (the controller may be destroyed before we end the pie session and we need them to preserve the camera position)
		if (bLastViewAndLocationValid == false)
		{
			for (int32 WorldIdx = WorldList.Num() - 1; WorldIdx >= 0; --WorldIdx)
			{
				FWorldContext &ThisContext = WorldList[WorldIdx];
				if (ThisContext.WorldType == EWorldType::PIE)
				{
					FSlatePlayInEditorInfo* const SlatePlayInEditorSession = SlatePlayInEditorMap.Find(ThisContext.ContextHandle);
					if ((SlatePlayInEditorSession != nullptr) && (SlatePlayInEditorSession->EditorPlayer.IsValid() == true) )
					{
						if( SlatePlayInEditorSession->EditorPlayer.Get()->PlayerController != nullptr )
						{
							SlatePlayInEditorSession->EditorPlayer.Get()->PlayerController->GetPlayerViewPoint( LastViewLocation, LastViewRotation );
							bLastViewAndLocationValid = true;
							break;
						}
					}
				}
			}
		}
	}
}

FString UEditorEngine::BuildPlayWorldURL(const TCHAR* MapName, bool bSpectatorMode, FString AdditionalURLOptions)
{
	// the URL we are building up
	FString URL(MapName);

	// If we hold down control, start in spectating mode
	if (bSpectatorMode)
	{
		// Start in spectator mode
		URL += TEXT("?SpectatorOnly=1");
	}

	// Add any game-specific options set in the INI file
	URL += InEditorGameURLOptions;

	// Add any additional options that were specified for this call
	URL += AdditionalURLOptions;

	return URL;
}

bool UEditorEngine::SpawnPlayFromHereStart(UWorld* World, AActor*& PlayerStart)
{
	if (PlayInEditorSessionInfo.IsSet() && PlayInEditorSessionInfo->OriginalRequestParams.HasPlayWorldPlacement())
	{
		// Rotation may be optional in original request.
		return SpawnPlayFromHereStart(World, PlayerStart, PlayInEditorSessionInfo->OriginalRequestParams.StartLocation.GetValue(), PlayInEditorSessionInfo->OriginalRequestParams.StartRotation.Get(FRotator::ZeroRotator));
	}

	// Not having a location set is still considered a success.
	return true;
}

bool UEditorEngine::SpawnPlayFromHereStart( UWorld* World, AActor*& PlayerStart, const FVector& StartLocation, const FRotator& StartRotation )
{
	// null it out in case we don't need to spawn one, and the caller relies on us setting it
	PlayerStart = NULL;

	// spawn the PlayerStartPIE in the given world
	FActorSpawnParameters SpawnParameters;
	SpawnParameters.OverrideLevel = World->PersistentLevel;
	PlayerStart = World->SpawnActor<AActor>(PlayFromHerePlayerStartClass, StartLocation, StartRotation, SpawnParameters);

	// make sure we were able to spawn the PlayerStartPIE there
	if(!PlayerStart)
	{
		FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Prompt_22", "Failed to create entry point. Try another location, or you may have to rebuild your level."));
		return false;
	}

	// tag the start
	ANavigationObjectBase* NavPlayerStart = Cast<ANavigationObjectBase>(PlayerStart);
	if (NavPlayerStart)
	{
		NavPlayerStart->bIsPIEPlayerStart = true;
	}

	// If PlayFromHere originated from a specific actor
	if (World->PersistentLevel->PlayFromHereActor)
	{
		World->PersistentLevel->PlayFromHereActor->OnPlayFromHere();
	}

	return true;
}

bool UE::PIE::IOutOfDateAssetCompilation::ShowCompilationErrorsDialog(TConstArrayView<UObject*> InAssetsWithErrors, const FText& AssetTypeText, TFunction<void(const TWeakObjectPtr<UObject>&)> InOpenAssetFunc)
{
	if (FApp::IsUnattended() || GIsRunningUnattendedScript || !InAssetsWithErrors.Num())
	{
		// App is running in unattended mode, so we should avoid modal dialogs and proceed
		return true;
	}
	
	TSharedRef<SVerticalBox> DialogContents = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(0.f, 0.f, 0.f, 16.f)
		[
			SNew(STextBlock)
			.Text(FText::FormatOrdered(NSLOCTEXT("PlayInEditor", "PrePIE_AssetErrors", "Are you sure you want to Play in Editor? The following {0} {1}|plural(one=Asset,other=Assets) {1}|plural(one=has,other=have) unresolved compiler errors."), AssetTypeText, InAssetsWithErrors.Num()))
		];

	TSharedPtr<SCustomDialog> CustomDialog;

	for (UObject* Asset : InAssetsWithErrors)
	{
		TWeakObjectPtr<UObject> WeakAssetPtr = Asset;

		DialogContents->AddSlot()
			.AutoHeight()
			.HAlign(HAlign_Left)
			[
				SNew(SHyperlink)
				.Style(FAppStyle::Get(), "Common.GotoBlueprintHyperlink")
				.OnNavigate(FSimpleDelegate::CreateLambda([WeakAssetPtr, &CustomDialog, InOpenAssetFunc]() 
				{
					InOpenAssetFunc(WeakAssetPtr);

					if (CustomDialog.IsValid())
					{
						CustomDialog->RequestDestroyWindow();
					}
				 }))
				.Text(FText::FromString(Asset->GetName()))
				.ToolTipText(FText::FormatOrdered(NSLOCTEXT("SourceHyperlink", "EditAsset_ToolTip", "Click to edit the {0}}"), Asset->GetClass()->GetDisplayNameText()))
			];
	}

	// Add an option to open all errored asset
	if(InAssetsWithErrors.Num() > 1)
	{
		DialogContents->AddSlot()
			.Padding(0.f, 16.f, 0.f, 0.f)
			.HAlign(HAlign_Left)
			[
				SNew(SHyperlink)
				.Style(FAppStyle::Get(), "Common.GotoBlueprintHyperlink")
				.OnNavigate(FSimpleDelegate::CreateLambda([&InAssetsWithErrors, &CustomDialog, InOpenAssetFunc]()
				{
					for(UObject* Asset : InAssetsWithErrors)
					{
						InOpenAssetFunc(Asset);
					}

					if (CustomDialog.IsValid())
					{
						CustomDialog->RequestDestroyWindow();
					}
				}))
				.Text(NSLOCTEXT("SourceHyperlink", "EditAllErroredAssets", "Open all assets with compilation errors"))
				.ToolTipText(FText::FormatOrdered(NSLOCTEXT("SourceHyperlink", "EditAllErroredAssets_ToolTip", "Opens all the {0} assets with compilation errors in their respective asset editor"), AssetTypeText))
			];
	}

	const FText DialogTitle = FText::FormatOrdered(NSLOCTEXT("PlayInEditor", "PrePIE_AssetErrorsTitle", "{0} Asset Compilation {1}|plural(one=Error,other=Errors)"), AssetTypeText, InAssetsWithErrors.Num());
	static const FText OKText = NSLOCTEXT("PlayInEditor", "PrePIE_OkText", "Play in Editor");
	static const FText CancelText = NSLOCTEXT("Dialogs", "EAppReturnTypeCancel", "Cancel");

	CustomDialog = SNew(SCustomDialog)
		.Title(DialogTitle)
		.Icon(FAppStyle::Get().GetBrush("NotificationList.DefaultMessage"))
		.Content()
		[
			DialogContents
		]
		.Buttons( { SCustomDialog::FButton(OKText), SCustomDialog::FButton(CancelText) } );

	const int32 ButtonPressed = CustomDialog->ShowModal();
	return ButtonPressed == 0;
}

FGameInstancePIEResult UEditorEngine::PreCreatePIEInstances(const bool bAnyBlueprintErrors, const bool bStartInSpectatorMode, const float PIEStartTime, const bool bSupportsOnlinePIE, int32& InNumOnlinePIEInstances)
{
	return FGameInstancePIEResult::Success();
}

FGameInstancePIEResult UEditorEngine::PreCreatePIEServerInstance(const bool bAnyBlueprintErrors, const bool bStartInSpectatorMode, const float PIEStartTime, const bool bSupportsOnlinePIE, int32& InNumOnlinePIEInstances)
{
	return FGameInstancePIEResult::Success();
}

bool UEditorEngine::SupportsOnlinePIE() const
{
	return UOnlineEngineInterface::Get()->SupportsOnlinePIE();
}

void UEditorEngine::OnLoginPIEComplete(int32 LocalUserNum, bool bWasSuccessful, const FString& ErrorString, FPieLoginStruct DataStruct)
{
	// This is needed because pie login may change the state of the online objects that called this function. This also enqueues the
	// callback back onto the main thread at an appropriate time instead of mid-networking callback.
	GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateUObject(this, &UEditorEngine::OnLoginPIEComplete_Deferred, LocalUserNum, bWasSuccessful, ErrorString, DataStruct));
}

void UEditorEngine::OnLoginPIEComplete_Deferred(int32 LocalUserNum, bool bWasSuccessful, FString ErrorString, FPieLoginStruct DataStruct)
{
	// This function will get called for both Async Online Service Log-in and for local non-async client creation.
	// This is called once per client that will be created, and may be called many frames after the initial PIE request
	// was started.

	UE_LOGF(LogPlayLevel, Verbose, "OnLoginPIEComplete LocalUserNum: %d bSuccess: %d %ls", LocalUserNum, bWasSuccessful, *ErrorString);
	FWorldContext* PieWorldContext = GetWorldContextFromHandle(DataStruct.WorldContextHandle);

	if (!PieWorldContext)
	{
		// This will fail if PIE was ended before this callback happened, silently return
		return;
	}

	// Detect success based on override
	bool bPIELoginSucceeded = IsLoginPIESuccessful(LocalUserNum, bWasSuccessful, ErrorString, DataStruct);

	// Create a new Game Instance for this.
	UGameInstance* GameInstance = CreateInnerProcessPIEGameInstance(PlayInEditorSessionInfo->OriginalRequestParams, DataStruct.GameInstancePIEParameters, DataStruct.PIEInstanceIndex);	
	if (GameInstance)
	{
		GameInstance->GetWorldContext()->bWaitingOnOnlineSubsystem = false;

		// Logging after the create so a new MessageLog Page is created
		if (bPIELoginSucceeded)
		{
			if (DataStruct.GameInstancePIEParameters.NetMode != EPlayNetMode::PIE_Client )
			{
				FMessageLog(NAME_CategoryPIE).Info(LOCTEXT("LoggedInServer", "Server logged in"));
			}
			else
			{
				FMessageLog(NAME_CategoryPIE).Info(LOCTEXT("LoggedInClient", "Client logged in"));
			}
		}
		else
		{
			if (DataStruct.GameInstancePIEParameters.NetMode != EPlayNetMode::PIE_Client)
			{
				FMessageLog(NAME_CategoryPIE).Error(FText::Format(LOCTEXT("LoggedInServerFailure", "Server failed to login. {0}"), FText::FromString(ErrorString)));
			}
			else
			{
				FMessageLog(NAME_CategoryPIE).Error(FText::Format(LOCTEXT("LoggedInClientFailure", "Client failed to login. {0}"), FText::FromString(ErrorString)));
			}
		}
	}

	// If there was a startup failure EndPlayMap might have already been called and unsetting this.
	if (PlayInEditorSessionInfo.IsSet())
	{
		PlayInEditorSessionInfo->NumOutstandingPIELogins--;
		if (!ensureAlwaysMsgf(PlayInEditorSessionInfo->NumOutstandingPIELogins >= 0, TEXT("PIEInstancesToLogInCount was not properly reset at some point.")))
		{
			PlayInEditorSessionInfo->NumOutstandingPIELogins = 0;
		}

		// If there are no more instances waiting to log-in then we can do post-launch notifications.
		if (PlayInEditorSessionInfo->NumOutstandingPIELogins == 0)
		{
			if (!bPIELoginSucceeded)
			{
				UE_LOGF(LogPlayLevel, Warning, "At least one of the requested PIE instances failed to start, ending PIE session.");
				EndPlayMap();
			}
			else
			{
				OnAllPIEInstancesStarted();
			}
		}
	}
}

void UEditorEngine::OnAllPIEInstancesStarted()
{
	GiveFocusToLastClientPIEViewport();

	// Print out a log message stating the overall startup time.
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("StartTime"), FPlatformTime::Seconds() - PlayInEditorSessionInfo->PlayRequestStartTime);
		FMessageLog(NAME_CategoryPIE).Info(FText::Format(LOCTEXT("PIETotalStartTime", "Play in editor total start time {StartTime} seconds."), Arguments));
	}
}

void UEditorEngine::GiveFocusToLastClientPIEViewport()
{
	// Find the non-dedicated server or last client window to give focus to. We choose the last one
	// because when launching additional instances, by applying focus to the first one it immediately
	// pushes the PINW instances behind the editor and you can't tell that they were launched.

	int32 HighestPIEInstance = TNumericLimits<int32>::Min();
	UGameViewportClient* ViewportClient = nullptr;
	for (const FWorldContext& WorldContext : WorldList)
	{
		if (WorldContext.WorldType == EWorldType::PIE && !WorldContext.RunAsDedicated)
		{
			if (WorldContext.PIEInstance > HighestPIEInstance)
			{
				HighestPIEInstance = WorldContext.PIEInstance;
				ViewportClient = WorldContext.GameViewport;
			}
		}
	}

	if (ViewportClient && ViewportClient->GetGameViewportWidget().IsValid())
	{
		FSlateApplication::Get().RegisterGameViewport(ViewportClient->GetGameViewportWidget().ToSharedRef());
	}

	// Make sure to focus the game viewport.
	{
		const bool bPreviewTypeIsVR = PlayInEditorSessionInfo->OriginalRequestParams.SessionPreviewTypeOverride.Get(EPlaySessionPreviewType::NoPreview) == EPlaySessionPreviewType::VRPreview;
		const bool bIsVR = IVREditorModule::Get().IsVREditorEnabled() || (bPreviewTypeIsVR && GEngine->XRSystem.IsValid());
		const bool bGameGetsMouseControl = PlayInEditorSessionInfo->OriginalRequestParams.EditorPlaySettings->GameGetsMouseControl;

		if (PlayInEditorSessionInfo->OriginalRequestParams.WorldType == EPlaySessionWorldType::PlayInEditor && (bGameGetsMouseControl || bIsVR))
		{
			FSlateApplication::Get().SetAllUserFocusToGameViewport();
		}
	}
}

void UEditorEngine::RequestLateJoin()
{
	if (PlayInEditorSessionInfo.IsSet())
	{
		PlayInEditorSessionInfo->bLateJoinRequested = true;
	}
}

void UEditorEngine::AddPendingLateJoinClient()
{
	if (!ensureMsgf(PlayInEditorSessionInfo.IsSet(), TEXT("RequestLateJoin shouldn't be called if no session is in progress!")))
	{
		return;
	}
	
	if(!ensureMsgf(PlayInEditorSessionInfo->bLateJoinRequested, TEXT("AddPendingLateJoinClient() shouldn't be called directly, use RequestLateJoin() instead!")))
	{
		return;
	}

	PlayInEditorSessionInfo->bLateJoinRequested = false;

	if (!ensureMsgf(PlayInEditorSessionInfo->OriginalRequestParams.WorldType != EPlaySessionWorldType::SimulateInEditor, TEXT("RequestLateJoin shouldn't be called for SIE!")))
	{
		return;
	}

	bool bUsePIEOnlineAuthentication = false;
	if (PlayInEditorSessionInfo->bUsingOnlinePlatform)
	{
		int32 TotalNumDesiredClients = PlayInEditorSessionInfo->NumClientInstancesCreated + 1;
		bool bHasRequiredLogins = TotalNumDesiredClients <= UOnlineEngineInterface::Get()->GetNumPIELogins();
		if (bHasRequiredLogins)
		{
			bUsePIEOnlineAuthentication = true;
		}
		else
		{
			FText ErrorMsg = LOCTEXT("PIELateJoinLoginFailure", "Not enough login credentials to add additional PIE instance, change editor settings.");
			UE_LOGF(LogPlayLevel, Warning, "%ls", *ErrorMsg.ToString());
			FMessageLog(NAME_CategoryPIE).Warning(ErrorMsg);
			return;
		}
	}

	// By the time of Late Join, a server should already be running if needed. So instead we only need
	// to launch a new instance following the same flow as all of the clients that already joined.
	EPlayNetMode NetMode;
	PlayInEditorSessionInfo->OriginalRequestParams.EditorPlaySettings->GetPlayNetMode(NetMode);

	// Adding another instance to a Listen Server will create clients, not more servers.
	if (NetMode == EPlayNetMode::PIE_ListenServer)
	{
		NetMode = EPlayNetMode::PIE_Client;
	}

	CreateNewPlayInEditorInstance(PlayInEditorSessionInfo->OriginalRequestParams, false, NetMode);
}

void UEditorEngine::CreateNewPlayInEditorInstance(FRequestPlaySessionParams &InRequestParams, const bool bInDedicatedInstance, const EPlayNetMode InNetMode)
{
	bool bUserWantsSingleProcess;
	InRequestParams.EditorPlaySettings->GetRunUnderOneProcess(bUserWantsSingleProcess);

	// If they don't want to use a single process, we can only launch one client inside the editor.
	if (!bUserWantsSingleProcess && PlayInEditorSessionInfo->NumClientInstancesCreated > 0)
	{
		if (bInDedicatedInstance)
		{
			// If they needed a server and they need multiple processes, it should
			// have already been launched by this time (by the block above). This means
			// this code should only ever be launching clients (but the code below can
			// launch either clients, listen servers, or dedicated servers.)
			check(PlayInEditorSessionInfo->bServerWasLaunched);
		}

		const bool bIsDedicatedServer = false;
		LaunchNewProcess(InRequestParams, PlayInEditorSessionInfo->NumClientInstancesCreated, InNetMode, bIsDedicatedServer);
		PlayInEditorSessionInfo->NumClientInstancesCreated++;
	}
	else
	{
		FPieLoginStruct PIELoginInfo;

		FGameInstancePIEParameters GameInstancePIEParameters;
		GameInstancePIEParameters.PIEStartTime = PlayInEditorSessionInfo->PlayRequestStartTime_StudioAnalytics;
		GameInstancePIEParameters.bAnyBlueprintErrors = PlayInEditorSessionInfo->bAnyBlueprintErrors;
		// If they require a server and one hasn't been launched then it is dedicated. If they're a client or listen server
		// then it doesn't count as a dedicated server so this can be false (NetMode will handle ListenServer).
		GameInstancePIEParameters.bRunAsDedicated = bInDedicatedInstance;
		GameInstancePIEParameters.bSimulateInEditor = InRequestParams.WorldType == EPlaySessionWorldType::SimulateInEditor;
		GameInstancePIEParameters.bStartInSpectatorMode = PlayInEditorSessionInfo->bStartedInSpectatorMode;
		GameInstancePIEParameters.EditorPlaySettings = PlayInEditorSessionInfo->OriginalRequestParams.EditorPlaySettings;
		GameInstancePIEParameters.WorldFeatureLevel = PreviewPlatform.GetEffectivePreviewFeatureLevel();
		GameInstancePIEParameters.NetMode = InNetMode;
		GameInstancePIEParameters.OverrideMapURL = InRequestParams.GlobalMapOverride;
		GameInstancePIEParameters.bIsPrimaryPIEClient = !bInDedicatedInstance && IsPrimaryPIEClient(InRequestParams, PlayInEditorSessionInfo->NumClientInstancesCreated);

		PIELoginInfo.GameInstancePIEParameters = GameInstancePIEParameters;

		// Create a World Context for our client.
		FWorldContext &PieWorldContext = CreateNewWorldContext(EWorldType::PIE);
		PieWorldContext.PIEInstance = PlayInEditorSessionInfo->PIEInstanceCount++;
		PieWorldContext.bWaitingOnOnlineSubsystem = true;

		PIELoginInfo.WorldContextHandle = PieWorldContext.ContextHandle;
		PIELoginInfo.PIEInstanceIndex = PieWorldContext.PIEInstance;

		// Fixed tick setting
		if (InRequestParams.EditorPlaySettings->ServerFixedFPS > 0 && bInDedicatedInstance)
		{
			PieWorldContext.PIEFixedTickSeconds = 1.f / (float)InRequestParams.EditorPlaySettings->ServerFixedFPS;
		}
		if (InRequestParams.EditorPlaySettings->ClientFixedFPS.Num() > 0 && !bInDedicatedInstance)
		{
			const int32 ClientFixedIdx = PlayInEditorSessionInfo->NumClientInstancesCreated % InRequestParams.EditorPlaySettings->ClientFixedFPS.Num();
			const int32 DesiredFPS = InRequestParams.EditorPlaySettings->ClientFixedFPS[ClientFixedIdx];
			if (DesiredFPS > 0)
			{
				PieWorldContext.PIEFixedTickSeconds = 1.f / (float)DesiredFPS;
			}
		}

		// We increment how many PIE instances we think are going to boot up as it is decremented
		// in OnLoginPIEComplete_Deferred and used to check if all instances have booted up.
		PlayInEditorSessionInfo->NumOutstandingPIELogins++;

		// If they are using an Online Subsystem that requires log-in then they require async
		// log-in so we use a deferred log-in approach.
		if (PlayInEditorSessionInfo->bUsingOnlinePlatform)
		{
			FName OnlineIdentifier = UOnlineEngineInterface::Get()->GetOnlineIdentifier(PieWorldContext);
			UE_LOGF(LogPlayLevel, Display, "Creating online subsystem for client %ls", *OnlineIdentifier.ToString());

			if (GameInstancePIEParameters.bRunAsDedicated)
			{
				// Dedicated servers don't use a login
				UOnlineEngineInterface::Get()->SetForceDedicated(OnlineIdentifier, true);

				OnLoginPIEComplete_Deferred(0, true, FString(), PIELoginInfo);
			}
			else
			{
				// Login to Online platform before creating world
				FOnPIELoginComplete OnPIELoginCompleteDelegate;
				OnPIELoginCompleteDelegate.BindUObject(this, &UEditorEngine::OnLoginPIEComplete, PIELoginInfo);

				// Kick off an async request to the Online Interface to log-in and, on completion, finish client creation.
				UOnlineEngineInterface::Get()->LoginPIEInstance(OnlineIdentifier, 0, PlayInEditorSessionInfo->NumClientInstancesCreated, OnPIELoginCompleteDelegate);
			}
		}
		else
		{
			// Otherwise, we can create the client immediately. We'll emulate the PIE login structure so we can just
			// use the same flow as the deferred log-in, but skipping the deferred part.
			OnLoginPIEComplete_Deferred(0, true, FString(), PIELoginInfo);
		}

		// Only count non-dedicated instances as clients so that our indexes line up for log-ins.
		if (!GameInstancePIEParameters.bRunAsDedicated && PlayInEditorSessionInfo.IsSet())
		{
			PlayInEditorSessionInfo->NumClientInstancesCreated++;
		}
	}
}

class SPIEViewport : public SViewport
{
	SLATE_BEGIN_ARGS(SPIEViewport)
		: _Content()
		, _RenderDirectlyToWindow(false)
		, _EnableStereoRendering(false)
		, _IgnoreTextureAlpha(true)
	{
		_Clipping = EWidgetClipping::ClipToBoundsAlways;
	}

		SLATE_DEFAULT_SLOT(FArguments, Content)

		/**
		 * Whether or not to render directly to the window's backbuffer or an offscreen render target that is applied to the window later
		 * Rendering to an offscreen target is the most common option in the editor where there may be many frames which this viewport's interface may wish to not re-render but use a cached buffer instead
		 * Rendering directly to the backbuffer is the most common option in the game where you want to update each frame without the cost of writing to an intermediate target first.
		 */
		SLATE_ARGUMENT(bool, RenderDirectlyToWindow)

		/** Whether or not to enable stereo rendering. */
		SLATE_ARGUMENT(bool, EnableStereoRendering )

		/**
		 * If true, the viewport's texture alpha is ignored when performing blending.  In this case only the viewport tint opacity is used
		 * If false, the texture alpha is used during blending
		 */
		SLATE_ARGUMENT( bool, IgnoreTextureAlpha )

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		SViewport::Construct(
			SViewport::FArguments()
			.EnableGammaCorrection(false) // Gamma correction in the game is handled in post processing in the scene renderer
			.RenderDirectlyToWindow(InArgs._RenderDirectlyToWindow)
			.EnableStereoRendering(InArgs._EnableStereoRendering)
			.IgnoreTextureAlpha(InArgs._IgnoreTextureAlpha)
			[
				InArgs._Content.Widget
			]
		);
	}

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
	{
		SViewport::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

		// Rather than binding the attribute we're going to poll it in tick, otherwise we will make this widget volatile, and it therefore
		// wont be possible to cache it or its children in GSlateEnableGlobalInvalidation mode.
		SetEnabled(FSlateApplication::Get().GetNormalExecutionAttribute().Get());
	}
};

void UEditorEngine::OnViewportCloseRequested(FViewport* InViewport)
{
	RequestEndPlayMap();
}

FSceneViewport* UEditorEngine::GetGameSceneViewport(UGameViewportClient* ViewportClient) const
{
	return ViewportClient->GetGameViewport();
}

FViewport* UEditorEngine::GetActiveViewport()
{
	// Get the Level editor module and request the Active Viewport.
	FLevelEditorModule& LevelEditorModule = FModuleManager::Get().GetModuleChecked<FLevelEditorModule>( TEXT("LevelEditor") );

	TSharedPtr<IAssetViewport> ActiveLevelViewport = LevelEditorModule.GetFirstActiveViewport();

	if ( ActiveLevelViewport.IsValid() )
	{
		return ActiveLevelViewport->GetActiveViewport();
	}
	
	return nullptr;
}

FViewport* UEditorEngine::GetPIEViewport()
{
	// Check both cases where the PIE viewport may be, otherwise return NULL if none are found.
	if( GameViewport )
	{
		return GameViewport->Viewport;
	}
	else
	{
		for (const FWorldContext& WorldContext : WorldList)
		{
			if (WorldContext.WorldType == EWorldType::PIE)
			{
				// We can't use FindChecked here because when using the dedicated server option we don't initialize this map 
				//	(we don't use a viewport for the PIE context in this case)
				FSlatePlayInEditorInfo * SlatePlayInEditorSessionPtr = SlatePlayInEditorMap.Find(WorldContext.ContextHandle);
				if ( SlatePlayInEditorSessionPtr != nullptr && SlatePlayInEditorSessionPtr->SlatePlayInEditorWindowViewport.IsValid() )
				{
					return SlatePlayInEditorSessionPtr->SlatePlayInEditorWindowViewport.Get();
				}
			}
		}
	}

	return nullptr;
}

bool UEditorEngine::GetSimulateInEditorViewTransform(FTransform& OutViewTransform) const
{
	if (PlayInEditorSessionInfo->OriginalRequestParams.WorldType == EPlaySessionWorldType::SimulateInEditor)
	{
		// The first PIE world context is the one that can toggle between PIE and SIE
		for (const FWorldContext& WorldContext : WorldList)
		{
			if (WorldContext.WorldType == EWorldType::PIE && !WorldContext.RunAsDedicated)
			{
				const FSlatePlayInEditorInfo* SlateInfoPtr = SlatePlayInEditorMap.Find(WorldContext.ContextHandle);
				if (SlateInfoPtr)
				{
					// This is only supported inside SLevelEditor viewports currently
					TSharedPtr<IAssetViewport> LevelViewport = SlateInfoPtr->DestinationSlateViewport.Pin();
					if (LevelViewport.IsValid())
					{
						FEditorViewportClient& EditorViewportClient = LevelViewport->GetAssetViewportClient();
						OutViewTransform = FTransform(EditorViewportClient.GetViewRotation(), EditorViewportClient.GetViewLocation());
						return true;
					}
				}
				break;
			}
		}
	}
	return false;
}

void UEditorEngine::ToggleBetweenPIEandSIE( bool bNewSession )
{
	bIsToggleBetweenPIEandSIEQueued = false;

	// The first PIE world context is the one that can toggle between PIE and SIE
	// Network PIE/SIE toggling is not really meant to be supported.
	FSlatePlayInEditorInfo * SlateInfoPtr = nullptr;
	for (const FWorldContext& WorldContext : WorldList)
	{
		if (WorldContext.WorldType == EWorldType::PIE && !WorldContext.RunAsDedicated)
		{
			SlateInfoPtr = SlatePlayInEditorMap.Find(WorldContext.ContextHandle);
			break;
		}
	}

	// This is only supported inside SLevelEditor viewports currently
	if (!SlateInfoPtr || !SlateInfoPtr->DestinationSlateViewport.IsValid())
	{
		UE_LOGF(LogPlayLevel, Warning, "Failed to find valid viewport when switching between PIE and SIE.");
		return;
	}

	FEditorDelegates::OnPreSwitchBeginPIEAndSIE.Broadcast(bIsSimulatingInEditor);

	if( FEngineAnalytics::IsAvailable() && !bNewSession )
	{
		FString ToggleType = bIsSimulatingInEditor ? TEXT("SIEtoPIE") : TEXT("PIEtoSIE");

		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.PIE"), TEXT("ToggleBetweenPIEandSIE"), ToggleType );
	}

	FSlatePlayInEditorInfo& SlatePlayInEditorSession = *SlateInfoPtr;
	TSharedPtr<IAssetViewport> LevelViewport = SlatePlayInEditorSession.DestinationSlateViewport.Pin();
	FEditorViewportClient& EditorViewportClient = LevelViewport->GetAssetViewportClient();

	// Toggle to pie if currently simulating
	if( bIsSimulatingInEditor )
	{
		// The undo system may have a reference to a SIE object that is about to be destroyed, so clear the transactions
		ResetTransaction( NSLOCTEXT("UnrealEd", "ToggleBetweenPIEandSIE", "Toggle Between PIE and SIE") );

		// The Game's viewport needs to know about the change away from simluate before the PC is (potentially) created
		GameViewport->GetGameViewport()->SetPlayInEditorIsSimulate(false);

		// The editor viewport client wont be visible so temporarily disable it being realtime
		const bool bShouldBeRealtime = false;
		const FText SystemDisplayName = LOCTEXT("RealtimeOverrideMessage_PIE", "Play in Editor");
		// Remove any previous override since we already applied a override when entering PIE
		EditorViewportClient.RemoveRealtimeOverride(SystemDisplayName);
		EditorViewportClient.AddRealtimeOverride(bShouldBeRealtime, SystemDisplayName);

		if (!SlatePlayInEditorSession.EditorPlayer.IsValid())
		{
			OnSwitchWorldsForPIE(true);

			UWorld* World = GameViewport->GetWorld();
			AGameModeBase* AuthGameMode = World->GetAuthGameMode();
			if (AuthGameMode && GameViewport->GetGameInstance())	// If there is no GameMode, we are probably the client and cannot RestartPlayer.
			{
				AuthGameMode->SpawnPlayerFromSimulate(EditorViewportClient.GetViewLocation(), EditorViewportClient.GetViewRotation());
			}

			OnSwitchWorldsForPIE(false);
		}

		// A game viewport already exists, tell the level viewport its in to swap to it
		LevelViewport->SwapViewportsForPlayInEditor();

		// No longer simulating
		GameViewport->SetIsSimulateInEditorViewport(false);
		EditorViewportClient.SetIsSimulateInEditorViewport(false);

		FEditorModeRegistry::Get().UnregisterMode(FBuiltinEditorModes::EM_Physics);
			
		bIsSimulatingInEditor = false;
	}
	else
	{
		// Swap to simulate from PIE
		LevelViewport->SwapViewportsForSimulateInEditor();
	
		GameViewport->SetIsSimulateInEditorViewport(true);
		GameViewport->GetGameViewport()->SetPlayInEditorIsSimulate(true);
		EditorViewportClient.SetIsSimulateInEditorViewport(true);

		
		TSharedRef<FPhysicsManipulationEdModeFactory> Factory = MakeShareable(new FPhysicsManipulationEdModeFactory);
		FEditorModeRegistry::Get().RegisterMode(FBuiltinEditorModes::EM_Physics, Factory);
			
		bIsSimulatingInEditor = true;

		// Make sure the viewport is in real-time mode
		const bool bShouldBeRealtime = true;
		const FText SystemDisplayName = LOCTEXT("RealtimeOverrideMessage_PIE", "Play in Editor");
		// Remove any previous override since we already applied a override when entering PIE
		EditorViewportClient.RemoveRealtimeOverride(SystemDisplayName);
		EditorViewportClient.AddRealtimeOverride(bShouldBeRealtime, SystemDisplayName);

		// The Simulate window should show stats
		EditorViewportClient.SetShowStats( true );

		if ( SlatePlayInEditorSession.EditorPlayer.IsValid() && SlatePlayInEditorSession.EditorPlayer.Get()->PlayerController )
		{
			// Move the editor camera to where the player was.  
			FVector ViewLocation;
			FRotator ViewRotation;
			SlatePlayInEditorSession.EditorPlayer.Get()->PlayerController->GetPlayerViewPoint( ViewLocation, ViewRotation );
			ViewRotation.Roll = 0.0; // Clear roll since editor camera controls has no way to adjust roll
			EditorViewportClient.SetViewLocation( ViewLocation );

			if( EditorViewportClient.IsPerspective() )
			{
				// Rotation only matters for perspective viewports not orthographic
				EditorViewportClient.SetViewRotation( ViewRotation );
			}
		}
	}

	// Backup ActorsThatWereSelected as this will be cleared whilst deselecting
	TArray<TWeakObjectPtr<class AActor> > BackupOfActorsThatWereSelected(ActorsThatWereSelected);

	// Unselect everything
	GEditor->SelectNone( true, true, false );
	GetSelectedActors()->DeselectAll();
	GetSelectedObjects()->DeselectAll();

	// restore the backup
	ActorsThatWereSelected = BackupOfActorsThatWereSelected;

	// make sure each selected actors sim equivalent is selected if we're Simulating but not if we're Playing
	for ( int32 ActorIndex = 0; ActorIndex < ActorsThatWereSelected.Num(); ++ActorIndex )
	{
		TWeakObjectPtr<AActor> Actor = ActorsThatWereSelected[ ActorIndex ].Get();
		if (Actor.IsValid())
		{
			AActor* SimActor = EditorUtilities::GetSimWorldCounterpartActor(Actor.Get());
			if (SimActor && !SimActor->IsHidden())
			{
				SelectActor( SimActor, bIsSimulatingInEditor, false );
			}
		}
	}

	FEditorDelegates::OnSwitchBeginPIEAndSIE.Broadcast( bIsSimulatingInEditor );
}

int32 UEditorEngine::OnSwitchWorldForSlatePieWindow(int32 WorldID, int32 WorldPIEInstance)
{
	static const int32 EditorWorldID = 0;
	static const int32 PieWorldID = 1;

	// PlayWorld cannot be depended on as it only points to the first instance
	int32 RestoreID = -1;
	if (WorldID == -1 && !GIsPlayInEditorWorld)
	{
		// When we have an invalid world id we always switch to the pie world in the PIE window
		OnSwitchWorldsForPIEInstance(WorldPIEInstance);

		// Make sure the switch to the PIE world was successful
		if (GIsPlayInEditorWorld)
		{
			// The editor world was active restore it later
			RestoreID = EditorWorldID;
		}
	}
	else if(WorldID == PieWorldID && !GIsPlayInEditorWorld)
	{
		// Want to restore the PIE world and the current world is not already the pie world
		OnSwitchWorldsForPIEInstance(WorldPIEInstance);
	}
	else if(WorldID == EditorWorldID && GWorld != EditorWorld)
	{
		// Want to restore the editor world and the current world is not already the editor world
		OnSwitchWorldsForPIEInstance(-1);
	}
	else
	{
		// Current world is already the same as the world being switched to (nested calls to this for example)
	}

	return RestoreID;
}

void UEditorEngine::OnSwitchWorldsForPIE( bool bSwitchToPieWorld, UWorld* OverrideWorld )
{
	if( bSwitchToPieWorld )
	{
		SetPlayInEditorWorld( OverrideWorld ? OverrideWorld : ToRawPtr(PlayWorld) );
	}
	else
	{
		RestoreEditorWorld( OverrideWorld ? OverrideWorld : ToRawPtr(EditorWorld) );
	}
}

void UEditorEngine::OnSwitchWorldsForPIEInstance(int32 WorldPIEInstance)
{
	if (WorldPIEInstance < 0)
	{
		RestoreEditorWorld(EditorWorld);
	}
	else
	{
		FWorldContext* PIEContext = GetPIEWorldContext(WorldPIEInstance);
		if (PIEContext && PIEContext->World())
		{
			SetPlayInEditorWorld(PIEContext->World());
		}
	}
}

void UEditorEngine::EnableWorldSwitchCallbacks(bool bEnable)
{
	if (bEnable)
	{
		// Set up a delegate to be called in Slate when GWorld needs to change.  Slate does not have direct access to the playworld to switch itself
		FScopedConditionalWorldSwitcher::SwitchWorldForPIEDelegate = FOnSwitchWorldForPIE::CreateUObject(this, &UEditorEngine::OnSwitchWorldsForPIE);

		if (!ScriptExecutionStartHandle.IsValid())
		{
			// This function can get called multiple times in multiplayer PIE
			ScriptExecutionStartHandle = FBlueprintContextTracker::OnEnterScriptContext.AddUObject(this, &UEditorEngine::OnScriptExecutionStart);
			ScriptExecutionEndHandle = FBlueprintContextTracker::OnExitScriptContext.AddUObject(this, &UEditorEngine::OnScriptExecutionEnd);
		}
	}
	else
	{
		// Don't actually need to reset this delegate but doing so allows is to check invalid attempts to execute the delegate
		FScopedConditionalWorldSwitcher::SwitchWorldForPIEDelegate = FOnSwitchWorldForPIE();

		// There should never be an active function context when pie is ending!
		check(!FunctionStackWorldSwitcher);

		if (ScriptExecutionStartHandle.IsValid())
		{
			FBlueprintContextTracker::OnEnterScriptContext.Remove(ScriptExecutionStartHandle);
			ScriptExecutionStartHandle.Reset();
			FBlueprintContextTracker::OnExitScriptContext.Remove(ScriptExecutionEndHandle);
			ScriptExecutionEndHandle.Reset();
		}
	}
}

void UEditorEngine::OnScriptExecutionStart(const FBlueprintContextTracker& ContextTracker, const UObject* ContextObject, const UFunction* ContextFunction)
{
	// Only do world switching for game thread callbacks when current world is set, this is only bound at all in PIE so no need to check GIsEditor
	if (IsInGameThread() && GWorld)
	{
		// See if we should create a world switcher, which is true if we don't have one and our PIE info is missing
		if (!FunctionStackWorldSwitcher && (!GIsPlayInEditorWorld || UE::GetPlayInEditorID() == -1))
		{
			check(FunctionStackWorldSwitcherTag == -1);
			UWorld* ContextWorld = GetWorldFromContextObject(ContextObject, EGetWorldErrorMode::ReturnNull);

			if (ContextWorld && ContextWorld->WorldType == EWorldType::PIE)
			{
				FunctionStackWorldSwitcher = new FScopedConditionalWorldSwitcher(ContextWorld);
				FunctionStackWorldSwitcherTag = ContextTracker.GetScriptEntryTag();
			}
		}
	}
}

void UEditorEngine::OnScriptExecutionEnd(const struct FBlueprintContextTracker& ContextTracker)
{
	if (IsInGameThread())
	{
		if (FunctionStackWorldSwitcher)
		{
			int32 CurrentScriptEntryTag = ContextTracker.GetScriptEntryTag();

			// Tag starts at 1 for first function on stack
			check(CurrentScriptEntryTag >= 1 && FunctionStackWorldSwitcherTag >= 1);

			if (CurrentScriptEntryTag == FunctionStackWorldSwitcherTag)
			{
				FunctionStackWorldSwitcherTag = -1;
				delete FunctionStackWorldSwitcher;
				FunctionStackWorldSwitcher = nullptr;
			}
		}
	}
}

UWorld* UEditorEngine::CreatePIEWorldByDuplication(FWorldContext &WorldContext, UWorld* InWorld, FString &PlayWorldMapName)
{
	double StartTime = FPlatformTime::Seconds();
	UPackage* InPackage = InWorld->GetOutermost();
	UWorld* NewPIEWorld = NULL;
	
	const FString WorldPackageName = InPackage->GetName();

	// Preserve the old path keeping EditorWorld name the same
	PlayWorldMapName = UWorld::ConvertToPIEPackageName(WorldPackageName, WorldContext.PIEInstance);

	// Display a busy cursor while we prepare the PIE world
	const FScopedBusyCursor BusyCursor;

	// Before loading the map, we need to set these flags to true so that postload will work properly
	TGuardValue<bool> OverrideIsPlayWorld(GIsPlayInEditorWorld, true);

	const FName PlayWorldMapFName = FName(*PlayWorldMapName);
	UWorld::WorldTypePreLoadMap.FindOrAdd(PlayWorldMapFName) = EWorldType::PIE;

	// Create a package for the PIE world
	UE_LOGF( LogPlayLevel, Log, "Creating play world package: %ls",  *PlayWorldMapName );	

	UPackage* PlayWorldPackage = CreatePackage(*PlayWorldMapName);
	// Add PKG_NewlyCreated flag to this package so we don't try to resolve its linker as it is unsaved duplicated world package
	PlayWorldPackage->SetPackageFlags(PKG_PlayInEditor | PKG_NewlyCreated);
	PlayWorldPackage->SetPIEInstanceID(WorldContext.PIEInstance);
	PlayWorldPackage->SetLoadedPath(InPackage->GetLoadedPath());
	PlayWorldPackage->SetSavedHash( InPackage->GetSavedHash() );
	PlayWorldPackage->MarkAsFullyLoaded();

	// check(UE::GetPlayInEditorID() == -1 || UE::GetPlayInEditorID() == WorldContext.PIEInstance);
	// Currently GPlayInEditorID is not correctly reset after map loading, so it's not safe to assert here
	FTemporaryPlayInEditorIDOverride IDHelper(WorldContext.PIEInstance);

	{
		double SDOStart = FPlatformTime::Seconds();

		// Reset any GUID fixups with lazy pointers
		FLazyObjectPtr::ResetPIEFixups();

		// Prepare soft object paths for fixup
		FSoftObjectPath::AddPIEPackageName(FName(*PlayWorldMapName));
		for (ULevelStreaming* StreamingLevel : InWorld->GetStreamingLevels())
		{
			if (StreamingLevel && !StreamingLevel->HasAllFlags(RF_DuplicateTransient))
			{
				FString StreamingLevelPIEName = UWorld::ConvertToPIEPackageName(StreamingLevel->GetWorldAssetPackageName(), WorldContext.PIEInstance);
				FSoftObjectPath::AddPIEPackageName(FName(*StreamingLevelPIEName));
			}
		}

		// NULL GWorld before various PostLoad functions are called, this makes it easier to debug invalid GWorld accesses
		GWorld = NULL;
		
		// Gives a chance to any assets being used for PIE/game to complete
		FAssetCompilingManager::Get().ProcessAsyncTasks();

		// Duplicate the editor world to create the PIE world
		NewPIEWorld = UWorld::GetDuplicatedWorldForPIE(InWorld, PlayWorldPackage, WorldContext.PIEInstance);

		// Fixup model components. The index buffers have been created for the components in the source world and the order
		// in which components were post-loaded matters. So don't try to guarantee a particular order here, just copy the
		// elements over.
		if ( NewPIEWorld->PersistentLevel->Model != NULL
			&& NewPIEWorld->PersistentLevel->Model == InWorld->PersistentLevel->Model
			&& NewPIEWorld->PersistentLevel->ModelComponents.Num() == InWorld->PersistentLevel->ModelComponents.Num() )
		{
			NewPIEWorld->PersistentLevel->Model->ClearLocalMaterialIndexBuffersData();
			for (int32 ComponentIndex = 0; ComponentIndex < NewPIEWorld->PersistentLevel->ModelComponents.Num(); ++ComponentIndex)
			{
				UModelComponent* SrcComponent = InWorld->PersistentLevel->ModelComponents[ComponentIndex];
				UModelComponent* DestComponent = NewPIEWorld->PersistentLevel->ModelComponents[ComponentIndex];
				DestComponent->CopyElementsFrom(SrcComponent);
			}
		}

		UE_LOGF(LogPlayLevel, Log, "PIE: StaticDuplicateObject took: (%fs)",  float(FPlatformTime::Seconds() - SDOStart));		
	}

	// Clean up the world type list now that PostLoad has occurred
	UWorld::WorldTypePreLoadMap.Remove(PlayWorldMapFName);

	check( NewPIEWorld );
	NewPIEWorld->SetFeatureLevel(InWorld->GetFeatureLevel());
	NewPIEWorld->WorldType = EWorldType::PIE;
	
	UE_LOGF(LogPlayLevel, Log, "PIE: Created PIE world by copying editor world from %ls to %ls (%fs)", *InWorld->GetPathName(), *NewPIEWorld->GetPathName(), float(FPlatformTime::Seconds() - StartTime));
	return NewPIEWorld;
}

void UEditorEngine::PostCreatePIEWorld(UWorld *NewPIEWorld)
{
	double WorldInitStart = FPlatformTime::Seconds();
	
	ensure(!NewPIEWorld->bIsWorldInitialized);
	
	// make sure we can clean up this world!
	NewPIEWorld->ClearFlags(RF_Standalone);

	// Force the new world to use a dedicated server net mode if needed
	// The other types will correctly derive it from the URL as it changes during play
	FWorldContext* const Context = GetWorldContextFromWorld(NewPIEWorld);
	if (Context && Context->RunAsDedicated)
	{
		NewPIEWorld->SetPlayInEditorInitialNetMode(NM_DedicatedServer);
	}
	
	// Init the PIE world
	NewPIEWorld->InitWorld();
	UE_LOGF(LogPlayLevel, Log, "PIE: World Init took: (%fs)",  float(FPlatformTime::Seconds() - WorldInitStart));

	// Tag PlayWorld Actors that also exist in EditorWorld.  At this point, no temporary/run-time actors exist in PlayWorld
	for( FActorIterator PlayActorIt(NewPIEWorld); PlayActorIt; ++PlayActorIt )
	{
		GEditor->ObjectsThatExistInEditorWorld.Set(*PlayActorIt);
	}
}

UWorld* UEditorEngine::CreatePIEWorldFromEntry(FWorldContext &WorldContext, UWorld* InWorld, FString &PlayWorldMapName)
{
	double StartTime = FPlatformTime::Seconds();

	// Create the world but do not initialize yet
	UWorld* LoadedWorld = UWorld::CreateWorld(EWorldType::PIE, false, NAME_None, nullptr, false, ERHIFeatureLevel::Num, nullptr, true);
	check(LoadedWorld);
	if (LoadedWorld->GetOutermost() != GetTransientPackage())
	{
		LoadedWorld->GetOutermost()->SetPIEInstanceID(WorldContext.PIEInstance);
	}
	// Force default GameMode class so project specific code doesn't fire off. 
	// We want this world to truly remain empty while we wait for connect!
	check(LoadedWorld->GetWorldSettings());
	LoadedWorld->GetWorldSettings()->DefaultGameMode = AGameModeBase::StaticClass();

	PlayWorldMapName = UGameMapsSettings::GetGameDefaultMap();
	return LoadedWorld;
}

bool UEditorEngine::WorldIsPIEInNewViewport(UWorld *InWorld)
{
	FWorldContext &WorldContext = GetWorldContextFromWorldChecked(InWorld);
	if (WorldContext.WorldType == EWorldType::PIE)
	{
		FSlatePlayInEditorInfo * SlateInfoPtr = SlatePlayInEditorMap.Find(WorldContext.ContextHandle);
		if (SlateInfoPtr)
		{
			return SlateInfoPtr->SlatePlayInEditorWindow.IsValid();
		}
	}
	
	return false;
}

void UEditorEngine::SetPIEInstanceWindowSwitchDelegate(FPIEInstanceWindowSwitch InSwitchDelegate)
{
	PIEInstanceWindowSwitchDelegate = InSwitchDelegate;
}

void UEditorEngine::FocusNextPIEWorld(UWorld *CurrentPieWorld, bool previous)
{
	// Get the current world's idx
	int32 CurrentIdx = 0;
	for (CurrentIdx = 0; CurrentPieWorld && CurrentIdx < WorldList.Num(); ++CurrentIdx)
	{
		if (WorldList[CurrentIdx].World() == CurrentPieWorld)
		{
			break;
		}
	}

	// Step through the list to find the next or previous
	int32 step = previous? -1 : 1;
	CurrentIdx += (WorldList.Num() + step);
	
	while ( CurrentPieWorld && WorldList[ CurrentIdx % WorldList.Num() ].World() != CurrentPieWorld )
	{
		FWorldContext &Context = WorldList[CurrentIdx % WorldList.Num()];
		if (Context.World() && Context.WorldType == EWorldType::PIE && Context.GameViewport != NULL)
		{
			break;
		}

		CurrentIdx += step;
	}
	
	if (WorldList[CurrentIdx % WorldList.Num()].World())
	{
		// Bring new window to front and activate new viewport
		FSlatePlayInEditorInfo* SlateInfoPtr = SlatePlayInEditorMap.Find(WorldList[CurrentIdx % WorldList.Num()].ContextHandle);
		if (SlateInfoPtr && SlateInfoPtr->SlatePlayInEditorWindowViewport.IsValid())
		{
			FSceneViewport* SceneViewport = SlateInfoPtr->SlatePlayInEditorWindowViewport.Get();

			FSlateApplication& SlateApp = FSlateApplication::Get();
			TSharedRef<SViewport> ViewportWidget = SceneViewport->GetViewportWidget().Pin().ToSharedRef();

			TSharedPtr<SWindow> ViewportWindow = SlateApp.FindWidgetWindow(ViewportWidget);
			check(ViewportWindow.IsValid());

			// Force window to front
			ViewportWindow->BringToFront();

			// Execute notification delegate in case game code has to do anything else
			PIEInstanceWindowSwitchDelegate.ExecuteIfBound();
		}
	}
}
void UEditorEngine::ResetPIEAudioSetting(UWorld *CurrentPieWorld)
{
	ULevelEditorPlaySettings* PlayInSettings = GetMutableDefault<ULevelEditorPlaySettings>();
	if (!PlayInSettings->EnableGameSound)
	{
		if (FAudioDevice* AudioDevice = CurrentPieWorld->GetAudioDeviceRaw())
		{
			AudioDevice->SetTransientPrimaryVolume(0.0f);
		}
	}
}
UGameViewportClient * UEditorEngine::GetNextPIEViewport(UGameViewportClient * CurrentViewport)
{
	// Get the current world's idx
	int32 CurrentIdx = 0;
	for (CurrentIdx = 0; CurrentViewport && CurrentIdx < WorldList.Num(); ++CurrentIdx)
	{
		if (WorldList[CurrentIdx].GameViewport == CurrentViewport)
		{
			break;
		}
	}

	// Step through the list to find the next or previous
	int32 step = 1;
	CurrentIdx += (WorldList.Num() + step);

	while ( CurrentViewport && WorldList[ CurrentIdx % WorldList.Num() ].GameViewport != CurrentViewport )
	{
		FWorldContext &Context = WorldList[CurrentIdx % WorldList.Num()];
		if (Context.GameViewport && Context.WorldType == EWorldType::PIE)
		{
			return Context.GameViewport;
		}

		CurrentIdx += step;
	}

	return NULL;
}

void UEditorEngine::RemapGamepadControllerIdForPIE(class UGameViewportClient* InGameViewport, int32 &ControllerId)
{
	// Increment the controller id if we are the focused window, and RouteGamepadToSecondWindow is true (and we are running multiple clients).
	// This cause the focused window to NOT handle the input, decrement controllerID, and pass it to the next window.
	const ULevelEditorPlaySettings* PlayInSettings = GetDefault<ULevelEditorPlaySettings>();
	const bool CanRouteGamepadToSecondWindow = [&PlayInSettings]{ bool RouteGamepadToSecondWindow(false); return (PlayInSettings->GetRouteGamepadToSecondWindow(RouteGamepadToSecondWindow) && RouteGamepadToSecondWindow); }();
	const bool CanRunUnderOneProcess = [&PlayInSettings]{ bool RunUnderOneProcess(false); return (PlayInSettings->GetRunUnderOneProcess(RunUnderOneProcess) && RunUnderOneProcess); }();
	if ( CanRouteGamepadToSecondWindow && CanRunUnderOneProcess && InGameViewport->GetWindow().IsValid() && InGameViewport->GetWindow()->HasFocusedDescendants())
	{
		ControllerId++;
	}
}

void UEditorEngine::StartPlayInEditorSession(FRequestPlaySessionParams& InRequestParams)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UEditorEngine::StartPlayInEditorSession);

	// This reflects that the user has tried to launch a PIE session, but it may still
	// create one-or-more new processes depending on multiplayer settings.
	check(InRequestParams.SessionDestination == EPlaySessionDestinationType::InProcess);

	// Broadcast PreBeginPIE before checks that might block PIE below (BeginPIE is broadcast below after the checks)
	FEditorDelegates::PreBeginPIE.Broadcast(InRequestParams.WorldType == EPlaySessionWorldType::SimulateInEditor);
	const double PIEStartTime = FPlatformTime::Seconds();;
	const FScopedBusyCursor BusyCursor;

	// Cancel the transaction if one is opened when PIE is requested. This is generally avoided
	// because we buffer the request for the PIE session until the start of the next frame, but sometimes
	// transactions can get stuck open due to implementation errors so it's important that we check.
	if (GEditor->IsTransactionActive())
	{
		FFormatNamedArguments Args;
		FText TransactionName = GEditor->GetTransactionName();
		Args.Add(TEXT("TransactionName"), TransactionName);
		if (InRequestParams.WorldType == EPlaySessionWorldType::SimulateInEditor)
		{
			Args.Add(TEXT("PlaySession"), NSLOCTEXT("UnrealEd", "SimulatePlaySession", "Simulate"));
		}
		else
		{
			Args.Add(TEXT("PlaySession"), NSLOCTEXT("UnrealEd", "PIEPlaySession", "Play In Editor"));
		}

		FText NotificationText;
		NotificationText = FText::Format(NSLOCTEXT("UnrealEd", "CancellingTransactionForPIE", "Cancelling open '{TransactionName}' operation to start {PlaySession}"), Args);

		FNotificationInfo Info(NotificationText);
		Info.ExpireDuration = 5.0f;
		Info.bUseLargeFont = true;
		FSlateNotificationManager::Get().AddNotification(Info);
		GEditor->CancelTransaction(0);
		UE_LOGF(LogPlayLevel, Warning, "Cancelling Open Transaction '%ls' to start PIE session.", *TransactionName.ToString());
	}

	TArray<IPIEAuthorizer*> PlayAuthorizers = IModularFeatures::Get().GetModularFeatureImplementations<IPIEAuthorizer>(IPIEAuthorizer::GetModularFeatureName());
	for (const IPIEAuthorizer* Authority : PlayAuthorizers)
	{
		TValueOrError<bool, FText> AuthorizationResult = Authority->RequestPIEPermission(InRequestParams.WorldType == EPlaySessionWorldType::SimulateInEditor);
		if (AuthorizationResult.HasError() || !AuthorizationResult.GetValue())
		{
			if (AuthorizationResult.HasError())
			{
				UE_LOGF(LogPlayLevel, Warning, "Play-In-Editor canceled by plugin: %ls", *AuthorizationResult.GetError().ToString());
				FSlateNotificationManager::Get().AddNotification(FNotificationInfo(AuthorizationResult.GetError()));
			}

			CancelRequestPlaySession();
			return;
		}
	}

	// Make sure there's no outstanding load requests
	FlushAsyncLoading();

	// Gameplay relies on asset registry to be fully constructed, wait for completion before starting PIE
	IAssetRegistry::GetChecked().WaitForCompletion();

	// Update the Blueprint Debugger 
	FBlueprintEditorUtils::FindAndSetDebuggableBlueprintInstances();

	// Broadcast BeginPIE after checks that might block PIE above (PreBeginPIE is broadcast above before the checks)
	// ToDo: Shouldn't this move below the early-out for Error'd Blueprints?
	FEditorDelegates::BeginPIE.Broadcast(InRequestParams.WorldType == EPlaySessionWorldType::SimulateInEditor);

	UWorld* InWorld = GetEditorWorldContext().World();

	// Let navigation know PIE is starting so it can avoid any blueprint creation/deletion/instantiation affect editor map's navmesh changes
	FNavigationSystem::OnPIEStart(*InWorld);

	ULevelEditorPlaySettings* EditorPlaySettings = InRequestParams.EditorPlaySettings;
	check(EditorPlaySettings);

	// Auto-compile dirty blueprints (if needed) and let the user cancel PIE if there are 
	TArray<UBlueprint*> ErroredBlueprints;
	FInternalPlayLevelUtils::ResolveDirtyBlueprints(!EditorPlaySettings->AutoRecompileBlueprints, ErroredBlueprints);

	// Don't show the dialog if we're in the middle of a demo, just assume they'll work.
	if (ErroredBlueprints.Num() && !GIsDemoMode)
	{
		// There was at least one blueprint with an error, make sure the user is OK with that.
		bool bContinuePIE = UE::PIE::IOutOfDateAssetCompilation::ShowCompilationErrorsDialog(MakeArrayView<UObject*>((UObject**)ErroredBlueprints.GetData(),ErroredBlueprints.Num()), UBlueprint::StaticClass()->GetDisplayNameText(), [](const TWeakObjectPtr<UObject>& InAssetToOpen)
		{
			check(InAssetToOpen.IsValid());
			GEditor->EditObject(InAssetToOpen.Get());
		});

		if (!bContinuePIE)
		{
			FMessageLog("BlueprintLog").Open(EMessageSeverity::Warning);

			FEditorDelegates::EndPIE.Broadcast(InRequestParams.WorldType == EPlaySessionWorldType::SimulateInEditor);
			FNavigationSystem::OnPIEEnd(*InWorld);
			CancelRequestPlaySession();
			return;
		}
		else
		{
			// The user wants to ignore the compiler errors, mark the Blueprints and do not warn them again unless the Blueprint attempts to compile
			for (UBlueprint* Blueprint : ErroredBlueprints)
			{
				Blueprint->bDisplayCompilePIEWarning = false;
			}
		}
	}

	// Auto-compile other dirty assets if needed, and let the user cancel PIE if there are any errors
	TArray<UE::PIE::IOutOfDateAssetCompilation*> AssetCompilers = IModularFeatures::Get().GetModularFeatureImplementations<UE::PIE::IOutOfDateAssetCompilation>(UE::PIE::IOutOfDateAssetCompilation::GetModularFeatureName());
	for (UE::PIE::IOutOfDateAssetCompilation* AssetCompiler : AssetCompilers)
	{
		TArray<UObject*> AssetsWithErrors;
		const bool bContinuePIE = AssetCompiler->HandleOutOfDateAssets(InRequestParams, AssetsWithErrors);
		if (!bContinuePIE && !GIsDemoMode)
		{
			FEditorDelegates::EndPIE.Broadcast(InRequestParams.WorldType == EPlaySessionWorldType::SimulateInEditor);
			FNavigationSystem::OnPIEEnd(*InWorld);
			CancelRequestPlaySession();
			return;
		}
	}

	// Register for log processing so we can promote errors/warnings to the message log
	if (GetDefault<ULevelEditorPlaySettings>()->bPromoteOutputLogWarningsDuringPIE)
	{
		OutputLogErrorsToMessageLogProxyPtr = MakeShared<FOutputLogErrorsToMessageLogProxy>();
	}

	// Notify the XRSystem that it needs to BeginPlay.
	if (GEngine->XRSystem.IsValid() && InRequestParams.WorldType == EPlaySessionWorldType::PlayInEditor)
	{
		GEngine->XRSystem->OnBeginPlay(*GEngine->GetWorldContextFromWorld(InWorld));
	}

	// Remember old GWorld
	EditorWorld = InWorld;

	// Clear any messages from last time
	GEngine->ClearOnScreenDebugMessages();

	// Start a new PIE log page
	{
		const FString WorldPackageName = EditorWorld->GetOutermost()->GetName();

		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("Package"), FText::FromString(FPackageName::GetLongPackageAssetName(WorldPackageName)));
		Arguments.Add(TEXT("TimeStamp"), FText::AsDateTime(FDateTime::UtcNow()));

		FText PIESessionLabel = InRequestParams.WorldType == EPlaySessionWorldType::SimulateInEditor ?
			FText::Format(LOCTEXT("SIESessionLabel", "SIE session: {Package} ({TimeStamp})"), Arguments) :
			FText::Format(LOCTEXT("PIESessionLabel", "PIE session: {Package} ({TimeStamp})"), Arguments);

		FMessageLog(NAME_CategoryPIE).NewPage(PIESessionLabel);
	}

	// Flush all audio sources from the editor world
	if (FAudioDeviceHandle AudioDevice = EditorWorld->GetAudioDevice())
	{
		AudioDevice->Flush(EditorWorld);
		AudioDevice->ResetInterpolation();
		AudioDevice->OnBeginPIE(InRequestParams.WorldType == EPlaySessionWorldType::SimulateInEditor);
	}

	// Mute the editor world so no sounds come from it.
	EditorWorld->bAllowAudioPlayback = false;

	// Can't allow realtime viewports whilst in PIE so disable it for ALL viewports here.
	const bool bShouldBeRealtime = false;
	const FText SystemDisplayName = LOCTEXT("RealtimeOverrideMessage_PIE", "Play in Editor");
	SetViewportsRealtimeOverride(bShouldBeRealtime, SystemDisplayName);

	// Allow the global config to override our ability to create multiple PIE worlds.
	if (!GEditor->bAllowMultiplePIEWorlds)
	{
		EditorPlaySettings->SetRunUnderOneProcess(false);
	}

	// If they're Simulating in Editor, we just override them to only one instance (as it'll use the editor world and not duplicate)
	// We also override them to Offline play mode as networking isn't supported with SIE anyways.
	if (InRequestParams.WorldType == EPlaySessionWorldType::SimulateInEditor)
	{
		EditorPlaySettings->SetPlayNetMode(EPlayNetMode::PIE_Standalone);
		EditorPlaySettings->SetPlayNumberOfClients(1);
	}

	// If they need to use the online services, validate that they have provided 
	// enough pie credentials to launch the desired number of clients.
	bool bUseOnlineSubsystemForLogin = false;
	bool bRequestAllowsOnlineSubsystem = InRequestParams.bAllowOnlineSubsystem;
	if(bRequestAllowsOnlineSubsystem)
	{
		int32 DesiredNumberOfClients;
		EditorPlaySettings->GetPlayNumberOfClients(DesiredNumberOfClients);
		if (SupportsOnlinePIE() && InRequestParams.WorldType != EPlaySessionWorldType::SimulateInEditor)
		{
			bool bHasRequiredLogins = DesiredNumberOfClients <= UOnlineEngineInterface::Get()->GetNumPIELogins();
			if (bHasRequiredLogins)
			{
				// If we support online PIE use it even if we're standalone
				bUseOnlineSubsystemForLogin = true;
			}
			else
			{
				FText ErrorMsg = LOCTEXT("PIELoginFailure", "Not enough login credentials to launch all PIE instances, change editor settings");
				UE_LOGF(LogPlayLevel, Verbose, "%ls", *ErrorMsg.ToString());
				FMessageLog(NAME_CategoryPIE).Warning(ErrorMsg);
			}
		}
	}
	UOnlineEngineInterface::Get()->SetShouldTryOnlinePIE(bUseOnlineSubsystemForLogin);

	PlayInEditorSessionInfo->bUsingOnlinePlatform = bUseOnlineSubsystemForLogin;
	PlayInEditorSessionInfo->bAnyBlueprintErrors = ErroredBlueprints.Num() > 0;

	// Optionally minimize all extra windows for precious framerate.
	if (InRequestParams.EditorPlaySettings->bShouldMinimizeEditorOnNonVRPIE
		&& !bIsSimulatingInEditor
		&& InRequestParams.DestinationSlateViewport.IsSet())
	{
		TArray<TSharedRef<SWindow>> Windows;
		FSlateApplication::Get().GetAllVisibleWindowsOrdered(Windows);
		if (Windows.Num() > 0)
		{
			TSharedRef<SWindow> RootWindow = Windows[0];
			if (TSharedPtr<IAssetViewport> DestinationViewport = InRequestParams.DestinationSlateViewport.GetValue().Pin())
			{
				TSharedPtr<SWindow> DestinationWindow = FSlateApplication::Get().FindWidgetWindow(DestinationViewport->AsWidget());
			
				for (TSharedRef<SWindow>& Window : Windows)
				{
					// Don't minimize the root,
					// Don't minimize any free floating windows like other PIE windows.
					// Don't minimize the viewport that is being targeted.
					// Don't minimize already minimized windows.
					if (Window == RootWindow
						|| Window == DestinationWindow
						|| Window->GetParentWindow() != RootWindow
						|| Window->IsWindowMinimized())
					{
						continue;
					}
					
					Window->Minimize();
					MinimizedWindowsDuringPIE.Add(Window);
				}
			}
		}
	}
	
	// Now that we've gotten all of the editor house-keeping out of the way we can finally
	// start creating world instances and multi player clients!
	{
		// Allow the engine to cancel the PIE request if needed.
		FGameInstancePIEResult PreCreateResult = PreCreatePIEInstances(
			ErroredBlueprints.Num() > 0, false /*bStartInSpectorMode*/, PIEStartTime, SupportsOnlinePIE(), PlayInEditorSessionInfo->NumOutstandingPIELogins);
		if (!PreCreateResult.IsSuccess())
		{
			UE_LOGF(LogPlayLevel, Warning, "PlayInEditor Session failed (%ls::PreCreatePIEInstances) and will not be started.", *GetClass()->GetName());
			return;
		}
		
		// First, we handle starting a dedicated server. This can exist as either a separate 
		// process, or as an internal world.
		bool bUserWantsSingleProcess;
		InRequestParams.EditorPlaySettings->GetRunUnderOneProcess(bUserWantsSingleProcess);

		EPlayNetMode NetMode;
		InRequestParams.EditorPlaySettings->GetPlayNetMode(NetMode);

		// Standalone requires no server, and ListenServer doesn't require a separate server.
		const bool bNetModeRequiresSeparateServer = NetMode == EPlayNetMode::PIE_Client;
		const bool bLaunchExtraServerAnyways = InRequestParams.EditorPlaySettings->bLaunchSeparateServer;
		const bool bNeedsServer = bNetModeRequiresSeparateServer || bLaunchExtraServerAnyways;

		// If they require a separate server we'll give the EditorEngine a chance to handle any additional prep-work.
		if (bNeedsServer)
		{
			// Allow the engine to cancel the server request if needed.
			FGameInstancePIEResult ServerPreCreateResult = PreCreatePIEServerInstance(
				ErroredBlueprints.Num() > 0, false /*bStartInSpectorMode*/, PIEStartTime, true, PlayInEditorSessionInfo->NumOutstandingPIELogins);
			if (!ServerPreCreateResult.IsSuccess())
			{
				// ToDo: This will skip client creation as well right now. Probably OK though.
				UE_LOGF(LogPlayLevel, Warning, "PlayInEditor Session Server failed Pre-Create and will not be started.");
				return;
			}

			// If they don't want single process we launch the server as a separate process. If they do
			// want single process, it will get handled below as part of client startup.
			if (!bUserWantsSingleProcess)
			{
				const bool bIsDedicatedServer = true;
				const bool bIsHost = true;
				const int32 InstanceIndex = 0;
				LaunchNewProcess(InRequestParams, InstanceIndex, EPlayNetMode::PIE_ListenServer, bIsDedicatedServer);

				PlayInEditorSessionInfo->bServerWasLaunched = true;
			}
		}

		// If control is pressed, start in spectator mode
		FModifierKeysState KeysState = FSlateApplication::Get().GetModifierKeys();
		PlayInEditorSessionInfo->bStartedInSpectatorMode = InRequestParams.WorldType == EPlaySessionWorldType::SimulateInEditor || KeysState.IsControlDown();

		// Now that the dedicated server was (optionally) started, we'll start as many requested clients as we can.
		// Because the user indicated they wanted PIE/PINW we'll put the first client in the editor respecting that
		// setting. Any additional clients will either be in-process new windows, or separate processes based on settings.
		int32 NumClients;
		InRequestParams.EditorPlaySettings->GetPlayNumberOfClients(NumClients);

		// If the have a net mode that requires a server but they didn't create (or couldn't create due to single-process
		// limitations) a dedicated one, then we launch an extra world context acting as a server in-process.
		const bool bRequiresExtraListenServer = bNeedsServer && !PlayInEditorSessionInfo->bServerWasLaunched;
		int32 NumRequestedInstances = FMath::Max(NumClients, 1);
		if (bRequiresExtraListenServer)
		{
			NumRequestedInstances++;
		}

		for (int32 InstanceIndex = 0; InstanceIndex < NumRequestedInstances; InstanceIndex++)
		{
			// If they are running single-process and they need a server, the first instance  will be the server.
			const bool bClientIsServer = (InstanceIndex == 0) && (NetMode == EPlayNetMode::PIE_ListenServer || bRequiresExtraListenServer);

			EPlayNetMode LocalNetMode = NetMode;

			// If they're the server, we want to override them to be a ListenServer. This will get ignored if they're secretly a dedicated
			// server so it's okay.
			if (bClientIsServer)
			{
				LocalNetMode = EPlayNetMode::PIE_ListenServer;
			}

			// If they want to launch a Listen Server and have multiple clients, the subsequent clients need to be
			// treated as Clients so they connect to the listen server instead of launching multiple Listen Servers.
			if (NetMode == EPlayNetMode::PIE_ListenServer && InstanceIndex > 0)
			{
				LocalNetMode = EPlayNetMode::PIE_Client;
			}

			bool bRunAsDedicated = bClientIsServer && bRequiresExtraListenServer;

			// Create the instance. This can end up creating separate processes if needed based on settings.
			// This code is separated out of here so it can be re-used by the Late Join flow.
			CreateNewPlayInEditorInstance(InRequestParams, bRunAsDedicated, LocalNetMode);

			// If there was an error creating an instance it will call EndPlay which invalidates the session info.
			// This also broadcasts the EndPIE event so no need to do that here (to make a matching call to our BeginPIE)
			if (!PlayInEditorSessionInfo.IsSet())
			{
				return;
			}

			if (bClientIsServer)
			{
				// Grab New Created PIE Server Instance and set PlaySettings to Server's actual Port so Clients Connect To Correct Server
				const FWorldContext *const PIEServerWorldContext = GetWorldContextFromPIEInstance(PlayInEditorSessionInfo->PIEInstanceCount - 1);
				const UWorld *const PIEServerWorld = PIEServerWorldContext->World();
				if (PIEServerWorld)
				{
					UNetDriver *const NetDriver = PIEServerWorld->GetNetDriver();
					if (NetDriver && NetDriver->GetLocalAddr().IsValid())
					{
						EditorPlaySettings->SetServerPort(NetDriver->GetLocalAddr()->GetPort());
					}
				}
			}
		}

		// This needs to be run before ToggleBetweenPIEandSIE as it creates the list of selected actors.
		// It only re-selects the actors if you are entering SIE. 
		TransferEditorSelectionToPlayInstances(InRequestParams.WorldType == EPlaySessionWorldType::SimulateInEditor);

		// If they requested a SIE, we immediately convert the PIE into a SIE. This finds and executes on the first
		// world context, and doesn't support networking.
		if (!bUseOnlineSubsystemForLogin && InRequestParams.WorldType == EPlaySessionWorldType::SimulateInEditor)
		{
			const bool bNewSession = true;
			ToggleBetweenPIEandSIE(bNewSession);
		}
	}

	// Disable the screen saver when PIE is running
	EnableScreenSaver(false);

	// Update the details window with the actors we have just selected
	GUnrealEd->UpdateFloatingPropertyWindows();

	// Clean up any editor actors being referenced 
	GEngine->BroadcastLevelActorListChanged();

	// Set an undo barrier so that transactions prior to PIE can't be undone
	GUnrealEd->Trans->SetUndoBarrier();

	// Notify that we've changed safe zone ratios for visualization.
	{
		FMargin SafeZoneRatio = EditorPlaySettings->PIESafeZoneOverride;
		SafeZoneRatio.Left /= (EditorPlaySettings->NewWindowWidth / 2.0f);
		SafeZoneRatio.Right /= (EditorPlaySettings->NewWindowWidth / 2.0f);
		SafeZoneRatio.Bottom /= (EditorPlaySettings->NewWindowHeight / 2.0f);
		SafeZoneRatio.Top /= (EditorPlaySettings->NewWindowHeight / 2.0f);
		FSlateApplication::Get().OnDebugSafeZoneChanged.Broadcast(SafeZoneRatio, false);
	}

	{
		// Temporarily set this information until the deprecated variables are removed.
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bIsSimulatingInEditor = InRequestParams.WorldType == EPlaySessionWorldType::SimulateInEditor;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	FEditorDelegates::PostPIEStarted.Broadcast(InRequestParams.WorldType == EPlaySessionWorldType::SimulateInEditor);
}

/** Creates an GameInstance with the given settings. A window is created if this isn't a server. */
UGameInstance* UEditorEngine::CreateInnerProcessPIEGameInstance(FRequestPlaySessionParams& InParams, const FGameInstancePIEParameters& InPIEParameters, int32 InPIEInstanceIndex)
{
	// Create a GameInstance for this new instance.
	FSoftClassPath GameInstanceClassName = GetDefault<UGameMapsSettings>()->GameInstanceClass;
	UClass* GameInstanceClass = GameInstanceClassName.TryLoadClass<UGameInstance>();

	// If an invalid class type was specified we fall back to the default.
	if (!GameInstanceClass)
	{
		GameInstanceClass = UGameInstance::StaticClass();
	}

	UGameInstance* GameInstance = NewObject<UGameInstance>(this, GameInstanceClass);

	// We need to temporarily add the GameInstance to the root because the InitializeForPlayInEditor
	// call can do garbage collection wiping out the GameInstance
	GameInstance->AddToRoot();

	// Attempt to initialize the GameInstance. This will construct the world.
	const bool bFirstWorld = !PlayWorld;
	const FGameInstancePIEResult InitializeResult = GameInstance->InitializeForPlayInEditor(InPIEInstanceIndex, InPIEParameters);
	if (!InitializeResult.IsSuccess())
	{
		FMessageDialog::Open(EAppMsgType::Ok, InitializeResult.FailureReason);
		FEditorDelegates::EndPIE.Broadcast(InPIEParameters.bSimulateInEditor);
		FNavigationSystem::OnPIEEnd(*EditorWorld);

		GameInstance->RemoveFromRoot();
		return nullptr;
	}

	// Our game instance was successfully created
	FWorldContext* const PieWorldContext = GameInstance->GetWorldContext();
	check(PieWorldContext);
	PlayWorld = PieWorldContext->World();

	// Temporarily set GWorld to our newly created world. This utility function
	// also sets GIsPlayInEditorWorld so that users can know if GWorld is actually
	// a PIE world or not.
	SetPlayInEditorWorld(PlayWorld);

	// Initialize a local player and viewport client for non-dedicated server instances.
	UGameViewportClient* ViewportClient = nullptr;
	ULocalPlayer *NewLocalPlayer = nullptr;
	TSharedPtr<SPIEViewport> PIEViewport = nullptr;

	if (!InPIEParameters.bRunAsDedicated)
	{
		// Create an instance of the Game Viewport Client, with the class specified by the Engine.
		ViewportClient = NewObject<UGameViewportClient>(this, GameViewportClientClass);
		ViewportClient->Init(*PieWorldContext, GameInstance);

		ULevelEditorPlaySettings* PlayInSettings = GetMutableDefault<ULevelEditorPlaySettings>();
		ViewportClient->EngineShowFlags.SetServerDrawDebug(PlayInSettings->ShowServerDebugDrawingByDefault());

		const bool bUseMouseForTouch = InParams.EditorPlaySettings->UseMouseForTouch || GetDefault<UInputSettings>()->bUseMouseForTouch; 
		FSlateApplication::Get().SetGameIsFakingTouchEvents(bUseMouseForTouch);

		if (!InParams.EditorPlaySettings->EnableGameSound)
		{
			if (FAudioDeviceHandle GameInstanceAudioDevice = GameInstance->GetWorld()->GetAudioDevice())
			{
				GameInstanceAudioDevice->SetTransientPrimaryVolume(0.0f);
			}
		}
		if (InParams.EditorPlaySettings->SoloAudioInFirstPIEClient)
		{
			if (FAudioDeviceHandle GameInstanceAudioDevice = PlayWorld->GetAudioDevice())
			{
				if (GEngine)
				{
					if (FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager())
					{
						EPlayNetMode NetMode; 
						InParams.EditorPlaySettings->GetPlayNetMode(NetMode);
						if ((NetMode == PIE_Client && InPIEInstanceIndex == 1) || 
							((NetMode == PIE_Standalone || NetMode == PIE_ListenServer) && InPIEInstanceIndex == 0))
						{
							DeviceManager->SetSoloDevice(GameInstanceAudioDevice->DeviceID);
						}
						else
						{
							GameInstanceAudioDevice->SetDeviceMuted(true);
						}
					}
				}
			}
		}

		GameViewport = ViewportClient;
		GameViewport->bIsPlayInEditorViewport = true;

		// Update our World Context to know which Viewport Client is associated.
		PieWorldContext->GameViewport = ViewportClient;

		// Add a callback for Game Input that isn't absorbed by the Game Viewport. This allows us to
		// make editor commands work (such as Shift F1, etc.) from within PIE.
		ViewportClient->OnGameViewportInputKey().BindUObject(this, &UEditorEngine::ProcessDebuggerCommands);

		// Listen for when the viewport is closed, so we can see about shutting down PIE.
		ViewportCloseRequestedDelegateHandle = ViewportClient->OnCloseRequested().AddUObject(this, &UEditorEngine::OnViewportCloseRequested);
		FSlatePlayInEditorInfo& SlatePlayInEditorSession = SlatePlayInEditorMap.Add(PieWorldContext->ContextHandle, FSlatePlayInEditorInfo());

		// Might be invalid depending how pie was launched. Code below handles this
		if (InParams.DestinationSlateViewport.Get(nullptr).IsValid())
		{
			SlatePlayInEditorSession.DestinationSlateViewport = InParams.DestinationSlateViewport.GetValue();
			
			// Only one PIE Instance can live in a given viewport, so we'll null it out so that we create
			// windows instead for the remaining clients.
			InParams.DestinationSlateViewport = nullptr;
		}

		// Attempt to initialize a Local Player.
		FString Error;
		NewLocalPlayer = ViewportClient->SetupInitialLocalPlayer(Error);
		if (!NewLocalPlayer)
		{
			FMessageDialog::Open(EAppMsgType::Ok, FText::Format(NSLOCTEXT("UnrealEd", "Error_CouldntSpawnPlayer", "Couldn't spawn player: {0}"), FText::FromString(Error)));
			// go back to using the real world as GWorld
			RestoreEditorWorld(EditorWorld);
			EndPlayMap();
			GameInstance->RemoveFromRoot();
			return nullptr;
		}

		// A Local Player gets created even in SIE (which is different than a Player Controller), but we only
		// store a reference if we're PIE for the UI to know where to restore our viewport location after PIE closes.
		if (!InPIEParameters.bSimulateInEditor)
		{
			SlatePlayInEditorSession.EditorPlayer = NewLocalPlayer;
		}

		// Note: For K2 debugging purposes this MUST be created before beginplay is called because beginplay can trigger breakpoints
		// and we need to be able to refocus the pie viewport afterwards so it must be created first in order for us to find it
		{
			// If the original request provided a Slate Viewport, we'll use that for our output.
			if (SlatePlayInEditorSession.DestinationSlateViewport.IsValid())
			{
				TSharedPtr<IAssetViewport> LevelViewportRef = SlatePlayInEditorSession.DestinationSlateViewport.Pin();
				LevelViewportRef->StartPlayInEditorSession(ViewportClient, InParams.WorldType == EPlaySessionWorldType::SimulateInEditor);

				// We count this as a viewport being created so that subsequent clients won't think they're the 'first' and use the wrong setting.
				PlayInEditorSessionInfo->NumViewportInstancesCreated++;
			}
			else
			{
				// Generate a new Window to put this instance in.
				PIEViewport = GeneratePIEViewportWindow(InParams, PlayInEditorSessionInfo->NumViewportInstancesCreated, *PieWorldContext, InPIEParameters.NetMode, ViewportClient, SlatePlayInEditorSession);

				// Increment for each viewport so that the window titles get correct numbers and it uses the right save/load setting. Non-visible
				// servers won't be bumping this number as it's used for saving/restoring window positions.
				PlayInEditorSessionInfo->NumViewportInstancesCreated++;
			}


			// Broadcast that the Viewport has been successfully created.
			UGameViewportClient::OnViewportCreated().Broadcast();
		}

		// Mark the Viewport as a PIE Viewport
		if (GameViewport && GameViewport->Viewport)
		{
			GameViewport->Viewport->SetPlayInEditorViewport(true);
		}

		if (InParams.EditorPlaySettings->bUseNonRealtimeAudioDevice && AudioDeviceManager)
		{
			UE_LOGF(LogPlayLevel, Log, "Creating new non-realtime audio mixer");
			FAudioDeviceParams DeviceParams = AudioDeviceManager->GetDefaultParamsForNewWorld();
			DeviceParams.Scope = EAudioDeviceScope::Unique;
			DeviceParams.AssociatedWorld = PlayWorld;
			DeviceParams.bIsNonRealtime = true;
			// For NRT rendering, don't need a large buffer and don't need to double buffer
			DeviceParams.BufferSizeOverride = 32;
			DeviceParams.NumBuffersOverride = 2;
			FAudioDeviceHandle AudioDevice = AudioDeviceManager->RequestAudioDevice(DeviceParams);
			check(AudioDevice.IsValid());
			if (PlayWorld)
			{
				PlayWorld->SetAudioDevice(AudioDevice);
			}
		}
	}

	// By this point it is safe to remove the GameInstance from the root and allow it to garbage collected as per usual
	GameInstance->RemoveFromRoot();

	// If the request wanted to override the game mode we have to do that here while we still have specifics about
	// the request. This will allow
	if (InParams.GameModeOverride)
	{
		GameInstance->GetWorld()->GetWorldSettings()->DefaultGameMode = InParams.GameModeOverride;
	}

	// Transfer the Blueprint Debug references to the first client world that is created. This needs to be called before 
	// GameInstance->StartPlayInEditorGameInstance so that references are transfered by the time BeginPlay is called.
	if (bFirstWorld && PlayWorld)
	{
		EditorWorld->TransferBlueprintDebugReferences(PlayWorld);
	}

	FGameInstancePIEResult StartResult = FGameInstancePIEResult::Success();
	{
		FTemporaryPlayInEditorIDOverride OverrideIDHelper(InPIEInstanceIndex);
		StartResult = GameInstance->StartPlayInEditorGameInstance(NewLocalPlayer, InPIEParameters);
	}

	if (!StartResult.IsSuccess())
	{
		FMessageDialog::Open(EAppMsgType::Ok, StartResult.FailureReason);
		RestoreEditorWorld(EditorWorld);
		EndPlayMap();
		return nullptr;
	}

	EnableWorldSwitchCallbacks(true);

	if (PIEViewport.IsValid())
	{
		// Register the new viewport widget with Slate for viewport specific message routing.
		FSlateApplication::Get().RegisterGameViewport(PIEViewport.ToSharedRef());
	}

	// Go back to using the editor world as GWorld.
	RestoreEditorWorld(EditorWorld);

	return GameInstance;
}

FText GeneratePIEViewportWindowTitle(const EPlayNetMode InNetMode, const FRequestPlaySessionParams& InSessionParams, const int32 ClientIndex, const float FixedTick, const bool bVRPreview)
{
	const FText WindowTitleOverride = GetDefault<UGeneralProjectSettings>()->ProjectDisplayedTitle;

	FFormatNamedArguments Args;
	Args.Add(TEXT("GameName"), FText::FromString(FString(WindowTitleOverride.IsEmpty() ? FApp::GetProjectName() : WindowTitleOverride.ToString())));
	Args.Add(TEXT("PlatformBits"), FText::FromString(TEXT("64")));
	Args.Add(TEXT("RHIName"), GEditor ? GEditor->GetActiveShaderPlatformFriendlyName(): FDataDrivenShaderPlatformInfo::GetFriendlyName(GMaxRHIShaderPlatform));
	
	if (InNetMode == PIE_Client)
	{
		Args.Add(TEXT("NetMode"), FText::FromString(FString::Printf(TEXT("Client %d"), ClientIndex)));
	}
	else if (InNetMode == PIE_ListenServer)
	{
		Args.Add(TEXT("NetMode"), FText::FromString(FString::Printf(TEXT("Server %d"), ClientIndex)));
	}
	else
	{
		Args.Add(TEXT("NetMode"), FText::FromString(FString::Printf(TEXT("Standalone %d"), ClientIndex)));
	}

	if (bVRPreview)
	{
		Args.Add(TEXT("XRSystemName"), FText::FromName(GEngine->XRSystem->GetSystemName()));
		Args.Add(TEXT("XRRuntimeVersion"), FText::FromString(GEngine->XRSystem->GetVersionString()));
	}
	else
	{
		Args.Add(TEXT("XRSystemName"), FText::GetEmpty());
		Args.Add(TEXT("XRRuntimeVersion"), FText::GetEmpty());
	}

	if (FixedTick > 0.f)
	{
		int32 FixedFPS = (int32)(1.f / FixedTick);
		Args.Add(TEXT("FixedFPS"), FText::FromString(FString::Printf(TEXT("Fixed %dfps"), FixedFPS)));
	}
	else
	{
		Args.Add(TEXT("FixedFPS"), FText::GetEmpty());
	}

	return FText::TrimTrailing(FText::Format(NSLOCTEXT("UnrealEd", "PlayInEditor_WindowTitleFormat", "{GameName} Preview [NetMode: {NetMode}] {FixedFPS} ({PlatformBits}-bit/{RHIName}) {XRSystemName} {XRRuntimeVersion}"), Args));
}

void UEditorEngine::TransferEditorSelectionToPlayInstances(const bool bInSelectInstances)
{
	// Make a list of all the selected actors
	TArray<UObject *> SelectedActors;
	TArray<UObject*> SelectedComponents;
	for (FSelectionIterator It(GetSelectedActorIterator()); It; ++It)
	{
		AActor* Actor = static_cast<AActor*>(*It);
		if (Actor)
		{
			checkSlow(Actor->IsA(AActor::StaticClass()));

			SelectedActors.Add(Actor);
		}
	}


	// Unselect everything
	GEditor->SelectNone(true, true, false);
	GetSelectedActors()->DeselectAll();
	GetSelectedObjects()->DeselectAll();
	GetSelectedComponents()->DeselectAll();

	ActorsThatWereSelected.Empty();

	// For every actor that was selected previously, make sure it's sim equivalent is selected
	for (int32 ActorIndex = 0; ActorIndex < SelectedActors.Num(); ++ActorIndex)
	{
		AActor* Actor = Cast<AActor>(SelectedActors[ActorIndex]);
		if (Actor)
		{
			ActorsThatWereSelected.Add(Actor);

			AActor* SimActor = EditorUtilities::GetSimWorldCounterpartActor(Actor);
			if (SimActor && !SimActor->IsHidden())
			{
				SelectActor(SimActor, bInSelectInstances, false);
			}
		}
	}
}

TSharedRef<SPIEViewport> UEditorEngine::GeneratePIEViewportWindow(const FRequestPlaySessionParams& InSessionParams, int32 InViewportIndex, const FWorldContext& InWorldContext, EPlayNetMode InNetMode, UGameViewportClient* InViewportClient, FSlatePlayInEditorInfo& InSlateInfo)
{
	FIntPoint WindowSize, WindowPosition;
	GetWindowSizeAndPositionForInstanceIndex(*InSessionParams.EditorPlaySettings, InViewportIndex, InWorldContext, WindowSize, WindowPosition);
	bool bCenterNewWindowOverride = false;
	
	// VR Preview overrides window location.
	const bool bVRPreview = InWorldContext.bIsPrimaryPIEInstance && InSessionParams.SessionPreviewTypeOverride.Get(EPlaySessionPreviewType::NoPreview) == EPlaySessionPreviewType::VRPreview;

	// Because we could switch primary PIE on the fly, we should make all PIE windows with the same UI style
	bool bUseOSWndBorder = InSessionParams.SessionPreviewTypeOverride.Get(EPlaySessionPreviewType::NoPreview) == EPlaySessionPreviewType::VRPreview;

	if (bVRPreview)
	{
		bCenterNewWindowOverride = true;
	}

	// Check to see if they've provided a custom SWindow for us to place our Session in.
	TSharedPtr<SWindow> PieWindow = InSessionParams.CustomPIEWindow.Pin();
	const bool bHasCustomWindow = PieWindow.IsValid();

	// If they haven't provided a Slate Window (common), we will create one.
	if (!bHasCustomWindow)
	{
		FText ViewportName = GeneratePIEViewportWindowTitle(InNetMode, InSessionParams, InWorldContext.PIEInstance, InWorldContext.PIEFixedTickSeconds, bVRPreview);
		PieWindow = SNew(SWindow)
			.Title(ViewportName)
			.Tag("PIEWindow")
			.ScreenPosition(FVector2D(WindowPosition.X, WindowPosition.Y))
			.ClientSize(FVector2D(WindowSize.X, WindowSize.Y))	
			.AutoCenter(bCenterNewWindowOverride ? EAutoCenter::PreferredWorkArea : EAutoCenter::None)
			.UseOSWindowBorder(bUseOSWndBorder)
			.SaneWindowPlacement(!bCenterNewWindowOverride)
			.SizingRule(ESizingRule::UserSized)
			.AdjustInitialSizeAndPositionForDPIScale(false);

		PieWindow->SetAllowFastUpdate(true);
	}

	// Setup a delegate for switching to the play world on slate input events, drawing and ticking
	FOnSwitchWorldHack OnWorldSwitch = FOnSwitchWorldHack::CreateUObject(this, &UEditorEngine::OnSwitchWorldForSlatePieWindow, InWorldContext.PIEInstance);
	PieWindow->SetOnWorldSwitchHack(OnWorldSwitch);

	if (!bHasCustomWindow)
	{
		// Mac does not support parenting, do not keep on top
#if PLATFORM_MAC
		FSlateApplication::Get().AddWindow(PieWindow.ToSharedRef());
#else
		TSharedRef<SWindow> MainWindow = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame")).GetParentWindow().ToSharedRef();
		if (InSessionParams.EditorPlaySettings->PIEAlwaysOnTop)
		{
			FSlateApplication::Get().AddWindowAsNativeChild(PieWindow.ToSharedRef(), MainWindow, true);
		}
		else
		{
			FSlateApplication::Get().AddWindow(PieWindow.ToSharedRef());
		}
#endif
	}

	bool bRenderDirectlyToWindow = bVRPreview;
	bool bEnableStereoRendering = bVRPreview;

	static const auto CVarPropagateAlpha = IConsoleManager::Get().FindConsoleVariable(TEXT("r.PostProcessing.PropagateAlpha"));
	const bool bIgnoreTextureAlpha = (CVarPropagateAlpha->GetBool() != true);

	TSharedRef<SPIEViewport> PieViewportWidget =
		SNew(SPIEViewport)
		.RenderDirectlyToWindow(bRenderDirectlyToWindow)
		.EnableStereoRendering(bEnableStereoRendering)
		.IgnoreTextureAlpha(bIgnoreTextureAlpha);

	// Create a wrapper widget for PIE viewport to process play world actions
	TSharedRef<SGlobalPlayWorldActions> GlobalPlayWorldActionsWidgetRef =
		SNew(SGlobalPlayWorldActions)
		[
			PieViewportWidget
		];

	PieWindow->SetContent(GlobalPlayWorldActionsWidgetRef);

	if (!bHasCustomWindow)
	{
		// Ensure the PIE window appears does not appear behind other windows.
		PieWindow->BringToFront();
	}

	const bool bShouldMinimizeRootWindowForVRPreview = bVRPreview && GEngine->XRSystem.IsValid() && InSessionParams.EditorPlaySettings->ShouldMinimizeEditorOnVRPIE;
	const bool bShouldMinimizeRootWindowForNonVRPreview = !bVRPreview && InSessionParams.EditorPlaySettings->bShouldMinimizeEditorOnNonVRPIE;
	// Set up a notification when the window is closed so we can clean up PIE
	{
		struct FLocal
		{
			static void RequestDestroyPIEWindowOverride(const TSharedRef<SWindow>& WindowBeingClosed, TWeakObjectPtr<UEditorEngine> OwningEditorEngine)
			{
				if (OwningEditorEngine.IsValid())
				{
					OwningEditorEngine->RequestEndPlayMap();
					FSlateApplication::Get().LeaveDebuggingMode();
				}
				else
				{
					FSlateApplication::Get().RequestDestroyWindow(WindowBeingClosed);
				}
			}

			static void OnPIEWindowClosed(const TSharedRef<SWindow>& WindowBeingClosed, TWeakPtr<SViewport> PIEViewportWidget, TWeakObjectPtr<UEditorEngine> OwningEditorEngine, int32 ViewportIndex, bool bRestoreRootWindow, FDelegateHandle PreviewPlatformChangedHandle)
			{
				// Save off the window position
				const FVector2D PIEWindowPos = WindowBeingClosed->GetLocalToScreenTransform().GetTranslation();

				FIntPoint WindowSize = FIntPoint(WindowBeingClosed->GetClientSizeInScreen().X, WindowBeingClosed->GetClientSizeInScreen().Y);
				FIntPoint WindowPosition = FIntPoint(PIEWindowPos.X, PIEWindowPos.Y);

				if (OwningEditorEngine.IsValid())
				{
					OwningEditorEngine->StoreWindowSizeAndPositionForInstanceIndex(ViewportIndex, WindowSize, WindowPosition);

					if (PreviewPlatformChangedHandle.IsValid())
					{
						OwningEditorEngine->OnPreviewShaderPlatformChanged().Remove(PreviewPlatformChangedHandle);
					}
				}

				// Route the callback
				PIEViewportWidget.Pin()->OnWindowClosed(WindowBeingClosed);

				if (bRestoreRootWindow)
				{
					// restore previously minimized root window.
					TSharedPtr<SWindow> RootWindow = FGlobalTabmanager::Get()->GetRootWindow();
					if (RootWindow.IsValid() && RootWindow->IsWindowMinimized())
					{
						RootWindow->Restore();
					}
				}
			}
		};

		FDelegateHandle PreviewPlatformChangedHandle = OnPreviewShaderPlatformChanged().AddLambda([PieWindow, bHasCustomWindow, InViewportClient, InNetMode, InSessionParams, InWorldContext, bVRPreview](EShaderPlatform NewShaderPlatform)
			{
				if (!bHasCustomWindow)
				{
					FText ViewportName = GeneratePIEViewportWindowTitle(InNetMode, InSessionParams, InWorldContext.PIEInstance, InWorldContext.PIEFixedTickSeconds, bVRPreview);
					PieWindow->SetTitle(ViewportName);
				}
				InViewportClient->GetWorld()->ShaderPlatformChanged(NewShaderPlatform);
			});

		PieWindow->SetRequestDestroyWindowOverride(FRequestDestroyWindowOverride::CreateStatic(&FLocal::RequestDestroyPIEWindowOverride, TWeakObjectPtr<UEditorEngine>(this)));
		PieWindow->SetOnWindowClosed(FOnWindowClosed::CreateStatic(&FLocal::OnPIEWindowClosed, TWeakPtr<SViewport>(PieViewportWidget), TWeakObjectPtr<UEditorEngine>(this),
			InViewportIndex, bShouldMinimizeRootWindowForVRPreview || bShouldMinimizeRootWindowForNonVRPreview, PreviewPlatformChangedHandle));
	}

	InSlateInfo.SlatePlayInEditorWindowViewport = FSceneViewport::Create(InViewportClient, PieViewportWidget);

	const bool bShouldGameGetMouseControl = InSessionParams.EditorPlaySettings->GameGetsMouseControl || (bEnableStereoRendering && GEngine && GEngine->XRSystem.IsValid());
	InSlateInfo.SlatePlayInEditorWindowViewport->SetPlayInEditorGetsMouseControl(bShouldGameGetMouseControl);

	FSlateApplication::Get().RegisterViewport(PieViewportWidget);

	InSlateInfo.SlatePlayInEditorWindow = PieWindow;

	// Mark the viewport as PIE viewport
	InViewportClient->Viewport->SetPlayInEditorViewport(InViewportClient->bIsPlayInEditorViewport);

	// Change the system resolution to match our window, to make sure game and slate window are kept synchronized
	FSystemResolution::RequestResolutionChange(WindowSize.X, WindowSize.Y, EWindowMode::Windowed);
	
	const bool bHMDIsReady = (GEngine && GEngine->XRSystem.IsValid() && GEngine->XRSystem->GetHMDDevice() && GEngine->XRSystem->GetHMDDevice()->IsHMDConnected());
	if (bVRPreview && bHMDIsReady)
	{
		GEngine->StereoRenderingDevice->EnableStereo(true);
	}

	// minimize the root window to provide max performance for the preview.
	TSharedPtr<SWindow> RootWindow = FGlobalTabmanager::Get()->GetRootWindow();
	if (RootWindow.IsValid() && (bShouldMinimizeRootWindowForVRPreview || bShouldMinimizeRootWindowForNonVRPreview))
	{
		RootWindow->Minimize();
	}

	return PieViewportWidget;
}


/* fits the window position to make sure it falls within the confines of the desktop */
void FitWindowPositionToWorkArea(FIntPoint &WinPos, FIntPoint &WinSize, const FMargin &WinPadding)
{
	const int32 HorzPad = WinPadding.GetTotalSpaceAlong<Orient_Horizontal>();
	const int32 VertPad = WinPadding.GetTotalSpaceAlong<Orient_Vertical>();
	FIntPoint TotalSize(WinSize.X + HorzPad, WinSize.Y + VertPad);

	FDisplayMetrics DisplayMetrics;
	FSlateApplication::Get().GetCachedDisplayMetrics(DisplayMetrics);

	// Limit the size, to make sure it fits within the desktop area
	{
		FIntPoint NewWinSize;
		NewWinSize.X = FMath::Min(TotalSize.X, DisplayMetrics.VirtualDisplayRect.Right - DisplayMetrics.VirtualDisplayRect.Left);
		NewWinSize.Y = FMath::Min(TotalSize.Y, DisplayMetrics.VirtualDisplayRect.Bottom - DisplayMetrics.VirtualDisplayRect.Top);
		if (NewWinSize != TotalSize)
		{
			TotalSize = NewWinSize;
			WinSize.X = NewWinSize.X - HorzPad;
			WinSize.Y = NewWinSize.Y - VertPad;
		}
	}

	const FSlateRect PreferredWorkArea(DisplayMetrics.VirtualDisplayRect.Left,
		DisplayMetrics.VirtualDisplayRect.Top,
		DisplayMetrics.VirtualDisplayRect.Right - TotalSize.X,
		DisplayMetrics.VirtualDisplayRect.Bottom - TotalSize.Y);

	// if no more windows fit horizontally, place them in a new row
	if (WinPos.X > PreferredWorkArea.Right)
	{
		WinPos.X = PreferredWorkArea.Left;
		WinPos.Y += TotalSize.Y;
		if (WinPos.Y > PreferredWorkArea.Bottom)
		{
			WinPos.Y = PreferredWorkArea.Top;
		}
	}

	// if no more rows fit vertically, stack windows on top of each other
	else if (WinPos.Y > PreferredWorkArea.Bottom)
	{
		WinPos.Y = PreferredWorkArea.Top;
		WinPos.X += TotalSize.X;
		if (WinPos.X > PreferredWorkArea.Right)
		{
			WinPos.X = PreferredWorkArea.Left;
		}
	}

	// Clamp values to make sure they fall within the desktop area
	WinPos.X = FMath::Clamp(WinPos.X, (int32)PreferredWorkArea.Left, (int32)PreferredWorkArea.Right);
	WinPos.Y = FMath::Clamp(WinPos.Y, (int32)PreferredWorkArea.Top, (int32)PreferredWorkArea.Bottom);
}

bool IsPrimaryPIEClient(const FRequestPlaySessionParams& InPlaySessionParams, const int32 InClientIndex)
{
	// Note: the InClientIndex here is the index of the pie clinet instances, meaning instances with pie windows, created not the pie instance index. These may differ, for example it is possible that pie instance 0 is a windowless dedicated server in netmode 'play as client'.

	const ULevelEditorPlaySettings& EditorPlaySettings = InPlaySessionParams.EditorPlaySettings ? *InPlaySessionParams.EditorPlaySettings : *GetDefault<ULevelEditorPlaySettings>();

	if (InClientIndex == EditorPlaySettings.GetPrimaryPIEClientIndex())
	{
		return true;
	}

	int32 ClientCount;
	if (!EditorPlaySettings.GetPlayNumberOfClients(ClientCount))
	{
		return InClientIndex == 0;  // If we aren't doing 'number of clients' just primary the first client.
	}

	if ((EditorPlaySettings.GetPrimaryPIEClientIndex() >= ClientCount) && (InClientIndex == ClientCount - 1))
	{
		// If the number is set too high use the last client.  This could easily happen if the user reduces the number of clients and forgets to update this setting. We won't assume they are intentionally avoiding having a 'primary' in this way.
		return true;
	}

	// Note any negative value returned by GetPrimaryPIEClientIndex() would result in no primary client at all.
	return false;

}

void UEditorEngine::GetWindowSizeAndPositionForInstanceIndex(ULevelEditorPlaySettings& InEditorPlaySettings, const int32 InViewportIndex, const FWorldContext& InWorldContext, FIntPoint& OutSize, FIntPoint& OutPosition)
{
	if (!ensureMsgf(PlayInEditorSessionInfo.IsSet(), TEXT("Cannot get saved Window Size/Position if a session has not been started.")))
	{
		OutSize = FIntPoint(1280, 720);
		OutPosition = FIntPoint(0, 0);
		return;
	}

	FMargin WindowBorderSize(8.0f, 30.0f, 8.0f, 8.0f);
	TSharedPtr<SWindow> TopLevelWindow = FSlateApplication::Get().GetActiveTopLevelWindow();

	if (TopLevelWindow.IsValid())
	{
		WindowBorderSize = TopLevelWindow->GetWindowBorderSize(true);
	}

	// Alright, they don't have a saved position or don't want to load from the saved position. First, figure out
	// how big the window should be. If it is the primary client, it uses a different resolution source than additional.
	if (InWorldContext.bIsPrimaryPIEInstance)
	{
		OutSize = FIntPoint(InEditorPlaySettings.NewWindowWidth, InEditorPlaySettings.NewWindowHeight);
	}
	else
	{
		// Use the size for additional client windows.
		InEditorPlaySettings.GetClientWindowSize(OutSize);
	}

	// Figure out how big the users resolution is
	FDisplayMetrics DisplayMetrics;
	FSlateApplication::Get().GetCachedDisplayMetrics(DisplayMetrics);

	const FVector2D DisplaySize = FVector2D(
		DisplayMetrics.PrimaryDisplayWorkAreaRect.Right - DisplayMetrics.PrimaryDisplayWorkAreaRect.Left,
		DisplayMetrics.PrimaryDisplayWorkAreaRect.Bottom - DisplayMetrics.PrimaryDisplayWorkAreaRect.Top
	);

	// If the size is zero then they want us to auto-detect the resolution.
	if (OutSize.X <= 0 || OutSize.Y <= 0)
	{
		OutSize.X = FMath::RoundToInt(0.75f * DisplaySize.X);
		OutSize.Y = FMath::RoundToInt(0.75f * DisplaySize.Y);
	}


	// Now we can position the window. If it is the first window, we can respect the center window flag.
	if (InWorldContext.bIsPrimaryPIEInstance)
	{
		// Center window if CenterNewWindow checked or if NewWindowPosition is FIntPoint::NoneValue (-1,-1)
		if (InEditorPlaySettings.CenterNewWindow || InEditorPlaySettings.NewWindowPosition == FIntPoint::NoneValue)
		{
			// We don't store the last window position in this case, because we want additional windows
			// to open starting at the top left of the monitor.
			OutPosition.X = FMath::RoundToInt((DisplaySize.X / 2.f) - (OutSize.X / 2));
			OutPosition.Y = FMath::RoundToInt((DisplaySize.Y / 2.f) - (OutSize.Y / 2));
		}
		else
		{
			OutPosition = InEditorPlaySettings.NewWindowPosition;
		}
	}
	else
	{
		if (InViewportIndex < PlayInEditorSessionInfo->CachedWindowInfo.Num())
		{
			OutPosition = PlayInEditorSessionInfo->CachedWindowInfo[InViewportIndex].Position;
			FitWindowPositionToWorkArea(OutPosition, OutSize, WindowBorderSize);
		}
		// Add a new entry.
		else
		{
			// We bump the position to go to the right, and the clamp will auto-wrap it for us if it falls off screen.
			OutPosition = PlayInEditorSessionInfo->LastOpenedWindowInfo.Position + FIntPoint(PlayInEditorSessionInfo->LastOpenedWindowInfo.Size.X, 0);
			// We're opening multiple windows. We're going to calculate a new position (opening them
			FitWindowPositionToWorkArea(OutPosition, OutSize, WindowBorderSize);

			// Store this position as the 'last opened' position. This means additional windows will start to
			// the right of this, unless they run out of room at which point they'll start over on the next row
			PlayInEditorSessionInfo->LastOpenedWindowInfo.Size = OutSize;
			PlayInEditorSessionInfo->LastOpenedWindowInfo.Position = OutPosition;
		}
	}
	// Store this Size/Position for this duration
	StoreWindowSizeAndPositionForInstanceIndex(InViewportIndex, OutSize, OutPosition);
}
void UEditorEngine::StoreWindowSizeAndPositionForInstanceIndex(const int32 InViewportIndex, const FIntPoint& InSize, const FIntPoint& InPosition)
{
	// Overwrite an existing one if we have it
	if (InViewportIndex < PlayInEditorSessionInfo->CachedWindowInfo.Num())
	{
		PlayInEditorSessionInfo->CachedWindowInfo[InViewportIndex].Size = InSize;
		PlayInEditorSessionInfo->CachedWindowInfo[InViewportIndex].Position = InPosition;
	}
	else
	{
		// It is possible for PIE to play in viewports which are not independent and for which we do not want to cache window size/position information.  For example "Selected Viewport" will use a docked editor viewport for PIE.
		// Appending as many as we need to get to our proper index.  These cache entries may be overwritten later, or they may be left with what must be at least a plausible size/pos.
		while (PlayInEditorSessionInfo->CachedWindowInfo.Num() < InViewportIndex + 1)
		{
			FPlayInEditorSessionInfo::FWindowSizeAndPos& NewInfo = PlayInEditorSessionInfo->CachedWindowInfo.Add_GetRef(FPlayInEditorSessionInfo::FWindowSizeAndPos());
			NewInfo.Size = InSize;
			NewInfo.Position = InPosition;
		}
		check(PlayInEditorSessionInfo->CachedWindowInfo.Num() == InViewportIndex + 1); 
	}
}

#undef LOCTEXT_NAMESPACE
