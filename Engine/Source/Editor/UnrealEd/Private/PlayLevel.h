// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Containers/Array.h"

class UBlueprint;
class ULevelEditorPlaySettings;

struct FInternalPlayLevelUtils
{
	static int32 ResolveDirtyBlueprints(const bool bPromptForCompile, TArray<UBlueprint*>& ErroredBlueprints, const bool bForceLevelScriptRecompile = true);
};

DECLARE_LOG_CATEGORY_EXTERN(LogPlayLevel, Log, All);