// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/EditorEngine.h"
#include "PlayLevel.h"
#include "HeadMountedDisplayTypes.h"
#include "Editor.h"
#include "GameFramework/GameModeBase.h"
#include "GameFramework/InputSettings.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Settings/LevelEditorPlaySettings.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Net/OnlineEngineInterface.h"
#include "Misc/StringBuilder.h"

void UEditorEngine::StartPlayInNewProcessSession(FRequestPlaySessionParams& InRequestParams)
{
	check(InRequestParams.SessionDestination == EPlaySessionDestinationType::NewProcess);

	EPlayNetMode NetMode;
	InRequestParams.EditorPlaySettings->GetPlayNetMode(NetMode);

	// Standalone requires no server, and ListenServer doesn't require a separate server.
	const bool bNetModeRequiresSeparateServer = NetMode == EPlayNetMode::PIE_Client;
	const bool bLaunchExtraServerAnyways = InRequestParams.EditorPlaySettings->bLaunchSeparateServer;
	const bool bNeedsServer = bNetModeRequiresSeparateServer || bLaunchExtraServerAnyways;
	
	bool bServerWasLaunched = false;

	if (bNeedsServer)
	{
		const bool bIsDedicatedServer = true;
		LaunchNewProcess(InRequestParams, 0, EPlayNetMode::PIE_ListenServer, bIsDedicatedServer);
		
		bServerWasLaunched = true;
	}
	
	int32 NumClients;
	InRequestParams.EditorPlaySettings->GetPlayNumberOfClients(NumClients);

	// If the have a net mode that requires a server but they didn't create (or couldn't create due to single-process
	// limitations) a dedicated one, then we launch an extra world context acting as a server in-process.
	int32 NumRequestedInstances = FMath::Max(NumClients, 1);
	for (int32 InstanceIndex = 0; InstanceIndex < NumRequestedInstances; InstanceIndex++)
	{
		EPlayNetMode LocalNetMode = NetMode;

		// If they want to launch a Listen Server and have multiple clients, the subsequent clients need to be
		// treated as Clients so they connect to the listen server instead of launching multiple Listen Servers.
		if (NetMode == EPlayNetMode::PIE_ListenServer && InstanceIndex > 0)
		{
			LocalNetMode = EPlayNetMode::PIE_Client;
		}

		// Dedicated servers should have been launched above, so this is only clients + listen servers.
		const bool bIsDedicatedServer = false;

		LaunchNewProcess(InRequestParams, InstanceIndex, LocalNetMode, bIsDedicatedServer);
	}

	// Now that we've launched the new process, we'll cancel the request so that the UI lets us go into PIE.
	// This doesn't clear our tracked sessions, so next time PIE is started it will close any standalone instances.
	CancelRequestPlaySession();
}

void UEditorEngine::LaunchNewProcess(const FRequestPlaySessionParams& InParams, const int32 InInstanceNum, EPlayNetMode NetMode, bool bIsDedicatedServer)
{
	// All dedicated servers should be considered hosts as well.
	if (bIsDedicatedServer)
	{
		NetMode = EPlayNetMode::PIE_ListenServer;
	}

	// Apply various launch arguments based on their settings.
	FString CommandLine;
	FString UnrealURLParams;

	if (InParams.GameModeOverride)
	{
		UnrealURLParams += FString::Printf(TEXT("?game=%s"), *InParams.GameModeOverride->GetPathName());
	}

	if (bIsDedicatedServer)
	{
		CommandLine += TEXT("-server -log");
	}
	else if (NetMode == EPlayNetMode::PIE_ListenServer)
	{
		UnrealURLParams += TEXT("?Listen");
	}

	if (NetMode == EPlayNetMode::PIE_ListenServer)
	{
		// Add any additional url parameters the user might have specified, for both listen and dedicated servers
		FString AdditionalServerGameOptions;
		InParams.EditorPlaySettings->GetAdditionalServerGameOptions(AdditionalServerGameOptions);

		if (AdditionalServerGameOptions.Len() > 0)
		{
			UnrealURLParams += AdditionalServerGameOptions;
		}
	}

	// Allow loading specific GameUserSettings from the ini which differ per-process.
	FString GameUserSettingsOverride = GGameUserSettingsIni.Replace(TEXT("GameUserSettings"), *FString::Printf(TEXT("PIEGameUserSettings%d"), InInstanceNum));

	// Construct parms:
	//	-Override GameUserSettings.ini
	//	-Allow saving of config files (since we are giving them an override INI)
	//	-Force the OSS (Steam is the only thing that implements this right now) to use passthrough sockets instead of connecting to the platform session int.
	CommandLine += FString::Printf(TEXT(" GameUserSettingsINI=\"%s\" -MultiprocessSaveConfig -forcepassthrough"), *GameUserSettingsOverride);

	if (bIsDedicatedServer)
	{
		// Allow server specific launch parameters. Only works with separate process standalone servers.
		CommandLine += FString::Printf(TEXT(" %s"), *InParams.EditorPlaySettings->AdditionalServerLaunchParameters);
	}

	// If they're not a host, configure the URL Params to connect to the server (instead of a specifying a map later)
	if (NetMode == EPlayNetMode::PIE_Client)
	{
		FString ServerIP = TEXT("127.0.0.1");
		uint16 ServerPort;
		InParams.EditorPlaySettings->GetServerPort(ServerPort);
		UnrealURLParams += FString::Printf(TEXT(" %s:%hu"), *ServerIP, ServerPort);
	}

	// Add Messaging and a SessionName for the Unreal Front End
	CommandLine += TEXT(" -messaging -SessionName=\"Play in Standalone Game\"");

	// Allow overriding the localization for testing other languages.
	const FString& PreviewGameLanguage = FTextLocalizationManager::Get().GetConfiguredGameLocalizationPreviewLanguage();
	if (!PreviewGameLanguage.IsEmpty())
	{
		CommandLine += TEXT(" -culture=");
		CommandLine += PreviewGameLanguage;
	}

	if (InParams.SessionPreviewTypeOverride.Get(EPlaySessionPreviewType::NoPreview) == EPlaySessionPreviewType::MobilePreview)
	{
		CommandLine += TEXT(" -featureleveles31");

		// If we're currently running in OpenGL mode, pass that onto our newly spawned processes.
		if (IsOpenGLPlatform(GShaderPlatformForFeatureLevel[GMaxRHIFeatureLevel]))
		{
			CommandLine += TEXT(" -opengl");
		}

		// Fake touch events since we're on a desktop and not a mobile device.
		CommandLine += TEXT(" -faketouches");

		// Ensure the executable writes out a differently named config file to avoid multiple instances overwriting each other.
		// ToDo: Should this be on all multi-client launches?
		CommandLine += TEXT(" -MultiprocessSaveConfig");

		if(FDataDrivenShaderPlatformInfo::IsValid(PreviewPlatform.GetShaderPlatform()))
		{
			// Override the ShaderPlatform if the PreviewPlatform is set in the main editor viewport.
			CommandLine += FString::Printf(TEXT(" OVERRIDESP=%s"), *PreviewPlatform.PreviewShaderPlatformName.ToString());
			if (PreviewPlatform.DeviceProfileName != NAME_None)
			{
				// Override the DeviceProfile if the DeviceProfile is set in the main editor viewport for preview.
				CommandLine += FString::Printf(TEXT(" dp.Override=%s"), *PreviewPlatform.DeviceProfileName.ToString());
			}
		}
	}
	// If we are not launching a mobile preview, them we need to check the editor/project settings for using mouse touch as well.
	else
	{
#if WITH_EDITOR
		const bool bUseMouseForTouch = GetDefault<ULevelEditorPlaySettings>()->UseMouseForTouch || GetDefault<UInputSettings>()->bUseMouseForTouch;
#else
		const bool bUseMouseForTouch = GetDefault<UInputSettings>()->bUseMouseForTouch;
#endif

		if (bUseMouseForTouch)
		{
			CommandLine += TEXT(" -faketouches");
		}
	}

	// In order for the previewer to adjust its safe zone according to the device profile specified in the editor play settings,
	// we need to pass the PIESafeZoneOverride's values as command line variables to the new process that we are about to launch.
	FMargin PIESafeZoneOverride = InParams.EditorPlaySettings->PIESafeZoneOverride;
	if (!PIESafeZoneOverride.GetDesiredSize().IsZero())
	{
		CommandLine += FString::Printf(TEXT(" -SafeZonePaddingLeft=%f -SafeZonePaddingRight=%f -SafeZonePaddingTop=%f -SafeZonePaddingBottom=%f"),
			PIESafeZoneOverride.Left,
			PIESafeZoneOverride.Right,
			PIESafeZoneOverride.Top,
			PIESafeZoneOverride.Bottom
		);
	}

	// VRPreview handling
	if (InParams.SessionPreviewTypeOverride.Get(EPlaySessionPreviewType::NoPreview) == EPlaySessionPreviewType::VRPreview)
	{
		if (InParams.EditorPlaySettings->IsHMDForPrimaryProcessOnly())
		{
			// If they're trying to launch a new process (from the editor) in VR, this will fail because the editor
			// owns the HMD resource, so we warn, and then fall back. They will need to use single-process for VR preview.
			CommandLine += TEXT(" -nohmd");
			UE_LOGF(LogPlayLevel, Warning, "Standalone Game VR not supported, please use VR Preview. Launching separate process PIE with -nohmd.");
		}
		else if (InInstanceNum != 0) // PIE instance 0 is normally run in the editor process, so we may not see it here. That instance get the real HMD, so no simulator argument is passed.
		{
			CommandLine += TEXT(" -HMDSimulator");
			UE_LOGF(LogPlayLevel, Log, "Launching separate process PIE with -HMDSimulator. See bOneHeadsetEachProcess editor preference tooltip for more information about this.");
		}
	}

	// if we had -emulatestereo on the commandline, also pass it to the new process
	if (InParams.EditorPlaySettings->bEmulateStereo || FParse::Param(FCommandLine::Get(), TEXT("emulatestereo")))
	{
		CommandLine += TEXT(" -emulatestereo");
	}

	// Allow disabling the sound in the new clients.
	if (InParams.EditorPlaySettings->DisableStandaloneSound)
	{
		CommandLine += TEXT(" -nosound");
	}

	// Allow the user to specify their own additional launch parameters to be set.
	if (InParams.EditorPlaySettings->AdditionalLaunchParameters.Len() > 0)
	{
		CommandLine += FString::Printf(TEXT(" %s"), *InParams.EditorPlaySettings->AdditionalLaunchParameters);
	}

	// The Play in Editor request may have had its own parameters as well.
	if (InParams.AdditionalStandaloneCommandLineParameters.IsSet())
	{
		CommandLine += FString::Printf(TEXT(" %s"), *InParams.AdditionalStandaloneCommandLineParameters.GetValue());
	}

	// Allow servers to override which port they are launched on.
	if (NetMode == EPlayNetMode::PIE_ListenServer)
	{
		uint16 ServerPort;
		InParams.EditorPlaySettings->GetServerPort(ServerPort);

		CommandLine += FString::Printf(TEXT(" -port=%hu"), ServerPort);
	}

	// Allow emulating adverse network conditions.
	if (InParams.EditorPlaySettings->IsNetworkEmulationEnabled())
	{
		NetworkEmulationTarget CurrentTarget = (NetMode == EPlayNetMode::PIE_ListenServer) ? NetworkEmulationTarget::Server : NetworkEmulationTarget::Client;
		if (InParams.EditorPlaySettings->NetworkEmulationSettings.IsEmulationEnabledForTarget(CurrentTarget))
		{
			CommandLine += InParams.EditorPlaySettings->NetworkEmulationSettings.BuildPacketSettingsForCmdLine();
		}
	}

	// Decide if fullscreen or windowed based on what is specified in the params
	if (!CommandLine.Contains(TEXT("-fullscreen")) && !CommandLine.Contains(TEXT("-windowed")))
	{
		// Nothing specified fallback to window otherwise keep what is specified
		CommandLine += TEXT(" -windowed");
	}

	if (!bIsDedicatedServer)
	{
		// Get desktop metrics
		FDisplayMetrics DisplayMetrics;
		FSlateApplication::Get().GetCachedDisplayMetrics(DisplayMetrics);

		// We don't use GetWindowSizeAndPositionForInstanceIndex here because that is for PIE windows and uses a separate system for saving window positions,
		// so we'll just respect the settings object for viewport size. If you're in standlone (non multiplayer) we respect viewport resolution, while
		// networked modes respect the multiplayer version.
		FIntPoint WindowSize;
		if (NetMode == EPlayNetMode::PIE_Standalone)
		{
			WindowSize.X = InParams.EditorPlaySettings->NewWindowWidth;
			WindowSize.Y = InParams.EditorPlaySettings->NewWindowHeight;
		}
		else
		{
			InParams.EditorPlaySettings->GetClientWindowSize(WindowSize);
		}

		// If not center window nor NewWindowPosition is FIntPoint::NoneValue (-1,-1)
		if (!InParams.EditorPlaySettings->CenterNewWindow && InParams.EditorPlaySettings->NewWindowPosition != FIntPoint::NoneValue)
		{
			FIntPoint WindowPosition = InParams.EditorPlaySettings->NewWindowPosition;
			
			WindowPosition.X += FMath::Max(InInstanceNum - 1, 0) * WindowSize.X;
			WindowPosition.Y += static_cast<int32>(SWindowDefs::DefaultTitleBarSize * FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(0, 0));

			// If they don't want to center the new window, we add a specific location. This will get saved to user settings
			// via SAVEWINPOS and not end up reflected in our PlayInEditor settings.
			CommandLine += FString::Printf(TEXT(" -WinX=%d -WinY=%d SAVEWINPOS=1"), WindowPosition.X, WindowPosition.Y);
		}

		// If the user didn't specify a resolution in the settings, default to full resolution.
		if (WindowSize.X <= 0)
		{
			WindowSize.X = DisplayMetrics.PrimaryDisplayWidth;
		}

		if (WindowSize.Y <= 0)
		{
			WindowSize.Y = DisplayMetrics.PrimaryDisplayHeight;
		}

		// Update the WindowSize to the Constrained Aspect Ratio if its set in the main editor viewport for preview.
		if (InParams.SessionPreviewTypeOverride.Get(EPlaySessionPreviewType::NoPreview) == EPlaySessionPreviewType::MobilePreview)
		{
			if (FDataDrivenShaderPlatformInfo::IsValid(PreviewPlatform.GetShaderPlatform()))
			{
				float PreviewConstrainedAspectRatio = PreviewPlatform.GetConstrainedAspectRatio();
				if (PreviewConstrainedAspectRatio != 0.f)
				{
					if (GEditor->GetFlipConstrainedAspectRatioInPreview())
					{
						PreviewConstrainedAspectRatio = 1/ PreviewConstrainedAspectRatio;
					}
					float CurrentAspectRatio = float(WindowSize.X) / float(WindowSize.Y);

					if (CurrentAspectRatio > PreviewConstrainedAspectRatio)
					{
						WindowSize.X = WindowSize.Y * PreviewConstrainedAspectRatio;
					}
					else
					{
						WindowSize.Y = WindowSize.X / PreviewConstrainedAspectRatio;
					}
				}
			}
		}

		CommandLine += FString::Printf(TEXT(" -ResX=%d -ResY=%d"), WindowSize.X, WindowSize.Y);

		// If they request a size larger than their display, add -ForceRes to prevent the engine
		// from automatically resizing the new instance to fit within the bounds of the screen.
		if ((WindowSize.X <= 0 || WindowSize.X > DisplayMetrics.PrimaryDisplayWidth) || (WindowSize.Y <= 0 || WindowSize.Y > DisplayMetrics.PrimaryDisplayHeight))
		{
			CommandLine += TEXT(" -ForceRes");
		}

		// Pass through `-AUTH_LOGIN` etc arguments for external processes using the PIE credentials, if applicable.
		const int32 NumLogins = UOnlineEngineInterface::Get()->GetNumPIELogins();
		if (SupportsOnlinePIE() && InInstanceNum >= 0 && InInstanceNum < NumLogins)
		{
			CommandLine += UOnlineEngineInterface::Get()->GetPIELoginCommandLineArgs(InInstanceNum);
		}
	}

	FString GameNameOrProjectFile;
	if (FPaths::IsProjectFilePathSet())
	{
		GameNameOrProjectFile = FString::Printf(TEXT("\"%s\""), *FPaths::GetProjectFilePath());
	}
	else
	{
		GameNameOrProjectFile = FApp::GetProjectName();
	}

	// Build the final command line
	FWorldContext & EditorContext = GetEditorWorldContext();
	FString MapName = EditorContext.World()->GetOutermost()->GetName();

	// Launch a new process.
	TMap<FString, FStringFormatArg> NamedArguments;
	NamedArguments.Add(TEXT("GameNameOrProjectFile"), GameNameOrProjectFile);
	if (NetMode != EPlayNetMode::PIE_Client)
	{
		// If we're not a client, build a PlayWorld URL to load to.
		if (NetMode != EPlayNetMode::PIE_Standalone)
		{
			FString ServerMapNameOverride;
			InParams.EditorPlaySettings->GetServerMapNameOverride(ServerMapNameOverride);
		
			// Allow the user to override which map the server should load.
			if (ServerMapNameOverride.Len() > 0)
			{
				UE_LOGF(LogPlayLevel, Log, "Map Override specified in configuration, using %ls instead of current map (%ls)", *ServerMapNameOverride, *MapName);
				MapName = ServerMapNameOverride;
			}
		}

		NamedArguments.Add(TEXT("PlayWorldURL"), BuildPlayWorldURL(*MapName, false, UnrealURLParams));
	}
	else
	{
		// Otherwise hosts just connect and accept whatever the server's settings are.
		NamedArguments.Add(TEXT("PlayWorldURL"), UnrealURLParams);
	}
	TStringBuilder<64> SubprocessCommandLine;
	ECommandLineArgumentFlags CommandLineArgFlags = ECommandLineArgumentFlags::ServerContext;
	if (!bIsDedicatedServer)
	{
		CommandLineArgFlags = ECommandLineArgumentFlags::ClientContext;
	}
	FCommandLine::BuildSubprocessCommandLine(CommandLineArgFlags, false /*bOnlyInherited*/, SubprocessCommandLine);
	NamedArguments.Add(TEXT("SubprocessCommandLine"), *SubprocessCommandLine);
	NamedArguments.Add(TEXT("CommandLineParams"), CommandLine);

	FString FinalCommandLine = FString::Format(
		TEXT("{GameNameOrProjectFile} {PlayWorldURL} -game -PIEVIACONSOLE {SubprocessCommandLine} {CommandLineParams}"), NamedArguments);

	// Create a handle that we can keep track of for later killing.
	FPlayOnPCInfo& NewSession = PlayOnLocalPCSessions.Add_GetRef(FPlayOnPCInfo());
	const TCHAR* ExecutablePath = FPlatformProcess::ExecutablePath();
	uint32 ProcessID = 0;
	const bool bLaunchDetatched = true;
	const bool bLaunchMinimized = false;
	const bool bLaunchWindowHidden = false;
	const uint32 PriorityModifier = 0;
	NewSession.ProcessHandle = FPlatformProcess::CreateProc(
		FPlatformProcess::ExecutablePath(), *FinalCommandLine, bLaunchDetatched,
		bLaunchMinimized, bLaunchWindowHidden, &ProcessID,
		PriorityModifier, nullptr, nullptr, nullptr);

	if (!NewSession.ProcessHandle.IsValid())
	{
		UE_LOGF(LogPlayLevel, Error, "Failed to run a copy of the game on this PC.");
	}

	// Notify anyone listening that we started a new Standalone Process.
	FEditorDelegates::BeginStandaloneLocalPlay.Broadcast(ProcessID);
}