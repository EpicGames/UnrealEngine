// Copyright Epic Games, Inc. All Rights Reserved.

#include "Algo/AllOf.h"
#include "AnalyticsEventAttribute.h"
#include "Async/Async.h"
#include "CookerSettings.h"
#include "DesktopPlatformModule.h"
#include "Editor/EditorEngine.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "EditorAnalytics.h"
#include "Experimental/ZenServerInterface.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GameProjectGenerationModule.h"
#include "ILauncherServicesModule.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "ITargetDeviceServicesModule.h"
#include "Logging/MessageLog.h"
#include "Misc/CoreMisc.h"
#include "Misc/ConfigCacheIni.h"
#include "PlatformInfo.h"
#include "PlayLevel.h"
#include "Settings/EditorExperimentalSettings.h"
#include "Settings/LevelEditorPlaySettings.h"
#include "Settings/ProjectPackagingSettings.h"
#include "Settings/PlatformsMenuSettings.h"
#include "TargetReceipt.h"
#include "UnrealEdMisc.h"
#include "Features/IModularFeatures.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "IOutOfDateAssetCompilation.h"

#define LOCTEXT_NAMESPACE "PlayLevel"

static void HandleHyperlinkNavigate()
{
	FGlobalTabmanager::Get()->TryInvokeTab(FName("OutputLog"));
}

static void HandleCancelButtonClicked(ILauncherWorkerPtr LauncherWorker)
{
	if (LauncherWorker.IsValid())
	{
		LauncherWorker->Cancel();
	}
}

static void HandleOutputReceived(const FString& InMessage)
{
	if (InMessage.Contains(TEXT("Error:")))
	{
		UE_LOGF(LogPlayLevel, Error, "UAT: %ls", *InMessage);
	}
	else if (InMessage.Contains(TEXT("Warning:")))
	{
		UE_LOGF(LogPlayLevel, Warning, "UAT: %ls", *InMessage);
	}
	else
	{
		UE_LOGF(LogPlayLevel, Log, "UAT: %ls", *InMessage);
	}
}

void UEditorEngine::StartPlayUsingLauncherSession(FRequestPlaySessionParams& InRequestParams)
{
	check(InRequestParams.SessionDestination == EPlaySessionDestinationType::Launcher);

	// Cache the DeviceId we've been asked to run on. This is used by the UI to know which device
	// clicking the button (without choosing from the dropdown) should use.
	LastPlayUsingLauncherDeviceId = InRequestParams.LauncherTargetDevice->DeviceId;
	LauncherSessionInfo = FLauncherCachedInfo();

	LauncherSessionInfo->PlayUsingLauncherDeviceName = PlaySessionRequest->LauncherTargetDevice->DeviceName;

	if (!ensureAlwaysMsgf(PlaySessionRequest->LauncherTargetDevice.IsSet(), TEXT("PlayUsingLauncher should not be called without a target device set!")))
	{
		CancelRequestPlaySession();
		return;
	}

	if (!ensureAlwaysMsgf(LastPlayUsingLauncherDeviceId.Len() > 0, TEXT("PlayUsingLauncher should not be called without a target device id set!")))
	{
		CancelRequestPlaySession();
		return;
	}

	ILauncherServicesModule& LauncherServicesModule = FModuleManager::LoadModuleChecked<ILauncherServicesModule>(TEXT("LauncherServices"));
	ITargetDeviceServicesModule& TargetDeviceServicesModule = FModuleManager::LoadModuleChecked<ITargetDeviceServicesModule>("TargetDeviceServices");

	//if the device is not authorized to be launched to, we need to pop an error instead of trying to launch
	FString LaunchPlatformName = LastPlayUsingLauncherDeviceId.Left(LastPlayUsingLauncherDeviceId.Find(TEXT("@")));
	FString LaunchPlatformNameFromID = LastPlayUsingLauncherDeviceId.Right(LastPlayUsingLauncherDeviceId.Find(TEXT("@")));
	ITargetPlatform* LaunchPlatform = GetTargetPlatformManagerRef().FindTargetPlatform(LaunchPlatformName);
	FString IniPlatformName = LaunchPlatformName;

	// create a temporary device group and launcher profile
	ILauncherDeviceGroupRef DeviceGroup = LauncherServicesModule.CreateDeviceGroup(FGuid::NewGuid(), TEXT("PlayOnDevices"));
	if (LaunchPlatform != nullptr)
	{
		IniPlatformName = LaunchPlatform->IniPlatformName();
		if (LaunchPlatformNameFromID.Equals(LaunchPlatformName))
		{
			// create a temporary list of devices for the target platform
			TArray<ITargetDevicePtr> TargetDevices;
			LaunchPlatform->GetAllDevices(TargetDevices);

			for (const ITargetDevicePtr& PlayDevice : TargetDevices)
			{
				// compose the device id
				FString PlayDeviceId = LaunchPlatformName + TEXT("@") + PlayDevice.Get()->GetId().GetDeviceName();
				if (PlayDevice.IsValid() && !PlayDevice->IsAuthorized())
				{
					CancelPlayUsingLauncher();
				}
				else
				{
					DeviceGroup->AddDevice(PlayDeviceId);
					UE_LOGF(LogPlayLevel, Log, "Launcher Device ID: %ls", *PlayDeviceId);
				}
			}
		}
		else
		{
			ITargetDevicePtr PlayDevice = LaunchPlatform->GetDefaultDevice();
			if (PlayDevice.IsValid() && !PlayDevice->IsAuthorized())
			{
				CancelPlayUsingLauncher();
			}
			else
			{

				DeviceGroup->AddDevice(LastPlayUsingLauncherDeviceId);
				UE_LOGF(LogPlayLevel, Log, "Launcher Device ID: %ls", *LastPlayUsingLauncherDeviceId);
			}
		}

		if (DeviceGroup.Get().GetNumDevices() == 0)
		{
			return;
		}
	}

	// set the build/launch configuration 
	EBuildConfiguration BuildConfiguration = EBuildConfiguration::Development;
	const ULevelEditorPlaySettings* EditorPlaySettings = PlaySessionRequest->EditorPlaySettings;
	switch (EditorPlaySettings->LaunchConfiguration)
	{
	case LaunchConfig_Debug:
		BuildConfiguration = EBuildConfiguration::Debug;
		break;
	case LaunchConfig_Development:
		BuildConfiguration = EBuildConfiguration::Development;
		break;
	case LaunchConfig_Test:
		BuildConfiguration = EBuildConfiguration::Test;
		break;
	case LaunchConfig_Shipping:
		BuildConfiguration = EBuildConfiguration::Shipping;
		break;
	default:
	{
		const UProjectPackagingSettings* AllPlatformPackagingSettings = GetDefault<UProjectPackagingSettings>();
		const UPlatformsMenuSettings* PlatformsSettings = GetDefault<UPlatformsMenuSettings>();

		EProjectPackagingBuildConfigurations BuildConfig = PlatformsSettings->GetLaunchOnBuildConfiguration();
		// if PPBC_MAX is set, then the project default should be used instead of the per platform build config
		if (BuildConfig == EProjectPackagingBuildConfigurations::PPBC_MAX)
		{
			BuildConfig = AllPlatformPackagingSettings->BuildConfiguration;
		}

		switch (BuildConfig)
		{
		case EProjectPackagingBuildConfigurations::PPBC_Debug:
		case EProjectPackagingBuildConfigurations::PPBC_DebugGame:
			BuildConfiguration = EBuildConfiguration::Debug;
			break;
		case EProjectPackagingBuildConfigurations::PPBC_Development:
			BuildConfiguration = EBuildConfiguration::Development;
			break;
		case EProjectPackagingBuildConfigurations::PPBC_Test:
			BuildConfiguration = EBuildConfiguration::Test;
			break;
		case EProjectPackagingBuildConfigurations::PPBC_Shipping:
			BuildConfiguration = EBuildConfiguration::Shipping;
			break;
		}

		break;
	}
	}

	// does the project have any code?
	FGameProjectGenerationModule& GameProjectModule = FModuleManager::LoadModuleChecked<FGameProjectGenerationModule>(TEXT("GameProjectGeneration"));
	LauncherSessionInfo->bPlayUsingLauncherHasCode = GameProjectModule.Get().ProjectHasCodeFiles();

	// Figure out if we need to build anything
	ELauncherProfileBuildModes::Type BuildMode;
	if (EditorPlaySettings->BuildGameBeforeLaunch == EPlayOnBuildMode::PlayOnBuild_Always)
	{
		BuildMode = ELauncherProfileBuildModes::Build;
	}
	else if (EditorPlaySettings->BuildGameBeforeLaunch == EPlayOnBuildMode::PlayOnBuild_Never)
	{
		BuildMode = ELauncherProfileBuildModes::DoNotBuild;
	}
	else
	{
		BuildMode = ELauncherProfileBuildModes::Auto;
	}

	// Assume it's building unless disabled
	LauncherSessionInfo->bPlayUsingLauncherBuild = (BuildMode != ELauncherProfileBuildModes::DoNotBuild);

	// Setup launch profile, keep the setting here to a minimum.
	ILauncherProfileRef LauncherProfile = LauncherServicesModule.CreateProfile(TEXT("Launch On Device"));
	ILauncherProfileBuildCookRunPtr BuildCookRun = LauncherProfile->GetFirstBuildCookRun();
	BuildCookRun->SetBuildMode(BuildMode);
	BuildCookRun->SetBuildConfiguration(BuildConfiguration);
	BuildCookRun->SetFastIterate(true);
	if (InRequestParams.EditorPlaySettings && !InRequestParams.EditorPlaySettings->AdditionalLaunchParameters.IsEmpty())
	{
		BuildCookRun->SetAdditionalCommandLineParameters(InRequestParams.EditorPlaySettings->AdditionalLaunchParameters);
	}

	BuildCookRun->AddCookedPlatform(LaunchPlatformName);

	BuildCookRun->SetDeviceIsASimulator(InRequestParams.LauncherTargetDevice->bIsSimulator);

	// select the quickest cook mode based on which in editor cook mode is enabled
	const UCookerSettings& CookerSettings = *GetDefault<UCookerSettings>();
	const UEditorExperimentalSettings& ExperimentalSettings = *GetDefault<UEditorExperimentalSettings>();

	bool bInEditorCooking = false;
	bool bCookOnTheFly = false;
	ELauncherProfileCookModes::Type CurrentLauncherCookMode = ELauncherProfileCookModes::ByTheBook;
	if (!CookerSettings.bCookOnTheFlyForLaunchOn)
	{
		bInEditorCooking = Algo::AllOf(BuildCookRun->GetCookedPlatforms(),
			[this](const FString& PlatformName) { return CanCookByTheBookInEditor(PlatformName); });
		CurrentLauncherCookMode = bInEditorCooking ? ELauncherProfileCookModes::ByTheBookInEditor: ELauncherProfileCookModes::ByTheBook;
	}
	else
	{
		bCookOnTheFly = true;
		bInEditorCooking = Algo::AllOf(BuildCookRun->GetCookedPlatforms(),
			[this](const FString& PlatformName) { return CanCookOnTheFlyInEditor(PlatformName); });
		CurrentLauncherCookMode = bInEditorCooking ? ELauncherProfileCookModes::OnTheFlyInEditor : ELauncherProfileCookModes::OnTheFly;
	}

	bool bLegacyIterative = false;
	GConfig->GetBool(TEXT("CookSettings"), TEXT("LegacyIterative"), bLegacyIterative, GEditorIni);

	bool bIncrementalCooking =
		!bCookOnTheFly
		&& (CookerSettings.bIterativeCookingForLaunchOn || (bLegacyIterative && ExperimentalSettings.bSharedCookedBuilds));

	if (CurrentLauncherCookMode == ELauncherProfileCookModes::OnTheFlyInEditor ||
		CurrentLauncherCookMode == ELauncherProfileCookModes::ByTheBookInEditor)
	{
		// For now World Partition doesn't support InEditor cooking because its cooking is destructive -
		// it moves UObjects out of the generator package into the streaming packages. To allow cooking it in
		// the editor process, we will need to make it non-destructive or restore the package afterwards.
		FWorldContext& EditorContext = GetEditorWorldContext();
		if (EditorContext.World()->IsPartitionedWorld())
		{
			FString ErrorMsg = FString::Printf(TEXT("Error launching map %s : Quick launch with WorldPartition doesn't yet support cooking in the editor process.\n")
				TEXT("To launch this map using Quick launch, set EditorPerProjectUserSettings.ini:[/Script/UnrealEd.EditorExperimentalSettings]:bDisableCookInEditor=true and relaunch the editor."),
				*EditorContext.World()->GetOutermost()->GetName());
			UE_LOGF(LogPlayLevel, Error, "%ls", *ErrorMsg);
			FMessageLog("EditorErrors").Error(FText::FromString(ErrorMsg));
			FMessageLog("EditorErrors").Open();
			CancelRequestPlaySession();
			return;
		}
	}

	TStringBuilder<256> CookOptions;
	CookOptions << BuildCookRun->GetCookOptions();
	ensure(CookOptions.Len() == 0);
	auto SetCookOption = [&CookOptions](FStringView Option, bool bOptionOn)
	{
		ensure(CookOptions.ToView().Find(Option) == INDEX_NONE);
		if (bOptionOn)
		{
			CookOptions << (CookOptions.Len() > 0 ? TEXTVIEW(" ") : TEXTVIEW(""));
			CookOptions << Option;
		}
	};

	// content only projects won't have multiple targets to pick from, and pasing -target=UnrealGame will fail if what C++ thinks
	// is a content only project needs a temporary target.cs file in UBT, 
	// only set the BuildTarget in code-based projects
	if (LauncherSessionInfo->bPlayUsingLauncherHasCode)
	{
		const FTargetInfo* TargetInfo = GetDefault<UPlatformsMenuSettings>()->GetLaunchOnTargetInfo();
		if (TargetInfo != nullptr)
		{
			BuildCookRun->ClearBuildTargets();
			BuildCookRun->AddBuildTarget(TargetInfo->Name);
			BuildCookRun->SetBuildTargetSpecified(true);
		}
	}
	BuildCookRun->SetCookMode(CurrentLauncherCookMode);
	BuildCookRun->SetUnversionedCooking(!bIncrementalCooking || !bLegacyIterative); // Unversioned cooking is not allowed with legacy incremental cooking
	BuildCookRun->SetIncrementalCooking(bIncrementalCooking);
	SetCookOption(TEXTVIEW("-IgnoreIniSettingsOutOfDate"), bIncrementalCooking && CookerSettings.bIgnoreIniSettingsOutOfDateForIteration);
	SetCookOption(TEXTVIEW("-IgnoreScriptPackagesOutOfDate"), bIncrementalCooking && CookerSettings.bIgnoreScriptPackagesOutOfDateForIteration);
	SetCookOption(TEXTVIEW("-IterateSharedCookedbuild"), bIncrementalCooking && ExperimentalSettings.bSharedCookedBuilds);
	BuildCookRun->SetDeployedDeviceGroup(DeviceGroup);
	BuildCookRun->SetIncrementalDeploying(bIncrementalCooking);
	LauncherProfile->SetEditorExe(FUnrealEdMisc::Get().GetExecutableForCommandlets());
	LauncherProfile->SetShouldUpdateDeviceFlash(InRequestParams.LauncherTargetDevice->bUpdateDeviceFlash);
	BuildCookRun->SetCookOptions(*CookOptions);

	if (!InRequestParams.LauncherTargetDevice->Architecture.IsEmpty())
	{
		BuildCookRun->SetClientArchitectures({&InRequestParams.LauncherTargetDevice->Architecture, 1});
	}
	
	if (LauncherProfile->IsBuildingUAT() && !GetDefault<UEditorPerProjectUserSettings>()->bAlwaysBuildUAT && bUATSuccessfullyCompiledOnce)
	{
		// UAT was built on a first launch and there's no need to rebuild it any more
		LauncherProfile->SetBuildUAT(false);
	}

	const FString DummyIOSDeviceName(FString::Printf(TEXT("All_iOS_On_%s"), FPlatformProcess::ComputerName()));
	const FString DummyTVOSDeviceName(FString::Printf(TEXT("All_tvOS_On_%s"), FPlatformProcess::ComputerName()));

	if ((LaunchPlatformName != TEXT("IOS") && LaunchPlatformName != TEXT("TVOS")) ||
		(!LauncherSessionInfo->PlayUsingLauncherDeviceName.Contains(DummyIOSDeviceName) && !LauncherSessionInfo->PlayUsingLauncherDeviceName.Contains(DummyTVOSDeviceName)))
	{
		BuildCookRun->SetLaunchMode(ELauncherProfileLaunchModes::DefaultRole);
	}

	const bool bUseZenStore = GetDefault<UProjectPackagingSettings>()->GetUseZenStoreEffective();
	BuildCookRun->SetUseZenStore(bUseZenStore);
	if (bUseZenStore)
	{
		static UE::Zen::FScopeZenService EditorStaticZenService;
	}

	if (bUseZenStore || BuildCookRun->GetCookMode() == ELauncherProfileCookModes::OnTheFlyInEditor || BuildCookRun->GetCookMode() == ELauncherProfileCookModes::OnTheFly)
	{
		BuildCookRun->SetDeploymentMode(ELauncherProfileDeploymentModes::FileServer);
	}

	switch(EditorPlaySettings->PackFilesForLaunch)
	{
	default:
	case EPlayOnPakFileMode::NoPak:
		break;
	case EPlayOnPakFileMode::PakNoCompress:
		BuildCookRun->SetCompressed( false );
		BuildCookRun->SetDeployWithUnrealPak( true );
		break;
	case EPlayOnPakFileMode::PakCompress:
		BuildCookRun->SetCompressed( true );
		BuildCookRun->SetDeployWithUnrealPak( true );
		break;
	}


	TArray<UBlueprint*> ErroredBlueprints;
	FInternalPlayLevelUtils::ResolveDirtyBlueprints(!EditorPlaySettings->bAutoCompileBlueprintsOnLaunch, ErroredBlueprints, false);

	const TArray<UE::PIE::IOutOfDateAssetCompilation*> AssetCompilers = IModularFeatures::Get().GetModularFeatureImplementations<UE::PIE::IOutOfDateAssetCompilation>(UE::PIE::IOutOfDateAssetCompilation::GetModularFeatureName());
	for (UE::PIE::IOutOfDateAssetCompilation* AssetCompiler : AssetCompilers)
	{
		TArray<UObject*> AssetsWithErrors;
		AssetCompiler->HandleOutOfDateAssets(InRequestParams, AssetsWithErrors);
	}

	TArray<FString> MapNames;
	FWorldContext & EditorContext = GetEditorWorldContext();

	// Load maps in place as we saved them above
	FString EditorMapName = EditorContext.World()->GetOutermost()->GetName();
	MapNames.Add(EditorMapName);

	FString InitialMapName;
	if (MapNames.Num() > 0)
	{
		InitialMapName = MapNames[0];
	}

	BuildCookRun->GetDefaultLaunchRole()->SetInitialMap(InitialMapName);

	for (const FString& MapName : MapNames)
	{
		BuildCookRun->AddCookedMap(MapName);
	}

	if (BuildCookRun->GetCookMode() == ELauncherProfileCookModes::ByTheBookInEditor)
	{
		TArray<ITargetPlatform*> TargetPlatforms;
		for (const FString& PlatformName : BuildCookRun->GetCookedPlatforms())
		{
			ITargetPlatform* TargetPlatform = GetTargetPlatformManager()->FindTargetPlatform(PlatformName);
			// todo pass in all the target platforms instead of just the single platform
			// crashes if two requests are inflight but we can support having multiple platforms cooking at once
			TargetPlatforms.Add(TargetPlatform);
		}
		const TArray<FString> &CookedMaps = BuildCookRun->GetCookedMaps();

		// const TArray<FString>& CookedMaps = ChainState.Profile->GetCookedMaps();
		TArray<FString> CookDirectories;
		TArray<FString> IniMapSections;

		StartCookByTheBookInEditor(TargetPlatforms, CookedMaps, CookDirectories, GetDefault<UProjectPackagingSettings>()->CulturesToStage, IniMapSections);

		FIsCookFinishedDelegate &CookerFinishedDelegate = LauncherProfile->OnIsCookFinished();

		CookerFinishedDelegate.BindUObject(this, &UEditorEngine::IsCookByTheBookInEditorFinished);

		FCookCanceledDelegate &CookCancelledDelegate = LauncherProfile->OnCookCanceled();

		CookCancelledDelegate.BindUObject(this, &UEditorEngine::CancelCookByTheBookInEditor);
	}

	ILauncherPtr Launcher = LauncherServicesModule.CreateLauncher();
	GEditor->LauncherWorker = Launcher->Launch(TargetDeviceServicesModule.GetDeviceProxyManager(), LauncherProfile);

	// create notification item
	FText LaunchingText = LOCTEXT("LauncherTaskInProgressNotificationNoDevice", "Launching...");
	FNotificationInfo Info(LaunchingText);

	Info.Image = FAppStyle::GetBrush(TEXT("MainFrame.CookContent"));
	Info.bFireAndForget = false;
	Info.ExpireDuration = 10.0f;
	Info.Hyperlink = FSimpleDelegate::CreateStatic(HandleHyperlinkNavigate);
	Info.HyperlinkText = LOCTEXT("ShowOutputLogHyperlink", "Show Output Log");
	Info.ButtonDetails.Add(
		FNotificationButtonInfo(
			LOCTEXT("LauncherTaskCancel", "Cancel"),
			LOCTEXT("LauncherTaskCancelToolTip", "Cancels execution of this task."),
			FSimpleDelegate::CreateStatic(HandleCancelButtonClicked, GEditor->LauncherWorker)
		)
	);

	// Launch doesn't block PIE/Compile requests as it's an async background process, so we just
	// cancel the request to denote it as having been handled. This has to come after we've used
	// anything we might need from the original request.
	CancelRequestPlaySession();

	TSharedPtr<SNotificationItem> NotificationItem = FSlateNotificationManager::Get().AddNotification(Info);

	if (!NotificationItem.IsValid())
	{
		return;
	}

	// analytics for launch on
	TArray<FAnalyticsEventAttribute> AnalyticsParamArray;
	if (LaunchPlatform != nullptr)
	{
		LaunchPlatform->GetPlatformSpecificProjectAnalytics(AnalyticsParamArray);
	}
	FEditorAnalytics::ReportEvent(TEXT("Editor.LaunchOn.Started"), LaunchPlatformName, LauncherSessionInfo->bPlayUsingLauncherHasCode, AnalyticsParamArray);


	NotificationItem->SetCompletionState(SNotificationItem::CS_Pending);

	TWeakPtr<SNotificationItem> NotificationItemPtr(NotificationItem);
	if (GEditor->LauncherWorker.IsValid() && GEditor->LauncherWorker->GetStatus() != ELauncherWorkerStatus::Completed)
	{
		if (EditorPlaySettings->EnablePIEEnterAndExitSounds)
		{
			GEditor->PlayEditorSound(TEXT("/Engine/EditorSounds/Notifications/CompileStart_Cue.CompileStart_Cue"));
		}
		GEditor->LauncherWorker->OnOutputReceived().AddStatic(HandleOutputReceived);
		GEditor->LauncherWorker->OnStageStarted().AddUObject(this, &UEditorEngine::HandleStageStarted, NotificationItemPtr);
		GEditor->LauncherWorker->OnStageCompleted().AddUObject(this, &UEditorEngine::HandleStageCompleted, LauncherSessionInfo->bPlayUsingLauncherHasCode, NotificationItemPtr);
		GEditor->LauncherWorker->OnCompleted().AddUObject(this, &UEditorEngine::HandleLaunchCompleted, LauncherSessionInfo->bPlayUsingLauncherHasCode, NotificationItemPtr);
		GEditor->LauncherWorker->OnCanceled().AddUObject(this, &UEditorEngine::HandleLaunchCanceled, LauncherSessionInfo->bPlayUsingLauncherHasCode, NotificationItemPtr);
	}
	else
	{
		GEditor->LauncherWorker.Reset();
		if (EditorPlaySettings->EnablePIEEnterAndExitSounds)
		{
			GEditor->PlayEditorSound(TEXT("/Engine/EditorSounds/Notifications/CompileFailed_Cue.CompileFailed_Cue"));
		}

		NotificationItem->SetText(LOCTEXT("LauncherTaskFailedNotification", "Failed to launch task!"));
		NotificationItem->SetCompletionState(SNotificationItem::CS_Fail);
		NotificationItem->ExpireAndFadeout();
		
		// analytics for launch on
		TArray<FAnalyticsEventAttribute> ParamArray;
		ParamArray.Add(FAnalyticsEventAttribute(TEXT("Time"), 0.0));
		FEditorAnalytics::ReportEvent(TEXT("Editor.LaunchOn.Failed"), LastPlayUsingLauncherDeviceId.Left(LastPlayUsingLauncherDeviceId.Find(TEXT("@"))), LauncherSessionInfo->bPlayUsingLauncherHasCode, EAnalyticsErrorCodes::LauncherFailed, ParamArray);
		
		LauncherSessionInfo.Reset();
	}
}

void UEditorEngine::CancelPlayingViaLauncher()
{
	if (LauncherWorker.IsValid())
	{
		LauncherWorker->CancelAndWait();
	}
}

/** 
* Cancel Play using Launcher on error 
* 
* if the physical device is not authorized to be launched to, we need to pop an error instead of trying to launch
*/
void UEditorEngine::CancelPlayUsingLauncher()
{
	FText LaunchingText = LOCTEXT("LauncherTaskInProgressNotificationNotAuthorized", "Cannot launch to this device until this computer is authorized from the device");
	FNotificationInfo Info(LaunchingText);
	Info.ExpireDuration = 5.0f;
	TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
	if (Notification.IsValid())
	{
		Notification->SetCompletionState(SNotificationItem::CS_Fail);
		Notification->ExpireAndFadeout();
	}
}

/* FMainFrameActionCallbacks callbacks
 *****************************************************************************/

class FLauncherNotificationTask
{
public:

	FLauncherNotificationTask( TWeakPtr<SNotificationItem> InNotificationItemPtr, SNotificationItem::ECompletionState InCompletionState, const FText& InText )
		: CompletionState(InCompletionState)
		, NotificationItemPtr(InNotificationItemPtr)
		, Text(InText)
	{ }

	void DoTask( ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent )
	{
		if (NotificationItemPtr.IsValid())
		{
			const ULevelEditorPlaySettings* EditorPlaySettings = GetDefault<ULevelEditorPlaySettings>();
			if (EditorPlaySettings->EnablePIEEnterAndExitSounds)
			{
				if (CompletionState == SNotificationItem::CS_Fail)
				{
					GEditor->PlayEditorSound(TEXT("/Engine/EditorSounds/Notifications/CompileFailed_Cue.CompileFailed_Cue"));
				}
				else if (CompletionState == SNotificationItem::CS_Success)
				{
					GEditor->PlayEditorSound(TEXT("/Engine/EditorSounds/Notifications/CompileSuccess_Cue.CompileSuccess_Cue"));
				}
			}

			TSharedPtr<SNotificationItem> NotificationItem = NotificationItemPtr.Pin();
			NotificationItem->SetText(Text);
			NotificationItem->SetCompletionState(CompletionState);
			if (CompletionState == SNotificationItem::CS_Success || CompletionState == SNotificationItem::CS_Fail)
			{
				NotificationItem->ExpireAndFadeout();
			}
		}
	}

	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }
	ENamedThreads::Type GetDesiredThread( ) { return ENamedThreads::GameThread; }
	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FLauncherNotificationTask, STATGROUP_TaskGraphTasks);
	}

private:

	SNotificationItem::ECompletionState CompletionState;
	TWeakPtr<SNotificationItem> NotificationItemPtr;
	FText Text;
};


void UEditorEngine::HandleStageStarted(const FString& InStage, TWeakPtr<SNotificationItem> NotificationItemPtr)
{
	if (!LauncherSessionInfo.IsSet())
	{
		UE_LOGF(LogPlayLevel, Warning, "HandleStageStarted called for Stage: %ls but the session was canceled, ignoring.", *InStage);
		return;
	}

	bool bSetNotification = true;
	FFormatNamedArguments Arguments;
	FText NotificationText;
	if (InStage.Contains(TEXT("Cooking")) || InStage.Contains(TEXT("Cook Task")))
	{
		FString PlatformName = LastPlayUsingLauncherDeviceId.Left(LastPlayUsingLauncherDeviceId.Find(TEXT("@")));
		PlatformName = PlatformInfo::FindPlatformInfo(*PlatformName)->VanillaInfo->Name.ToString();
		Arguments.Add(TEXT("PlatformName"), FText::FromString(PlatformName));
		NotificationText = FText::Format(LOCTEXT("LauncherTaskProcessingNotification", "Processing Assets for {PlatformName}..."), Arguments);
	}
	else if (InStage.Contains(TEXT("Build Task")))
	{
		FString PlatformName = LastPlayUsingLauncherDeviceId.Left(LastPlayUsingLauncherDeviceId.Find(TEXT("@")));
		PlatformName = PlatformInfo::FindPlatformInfo(*PlatformName)->VanillaInfo->Name.ToString();
		Arguments.Add(TEXT("PlatformName"), FText::FromString(PlatformName));
		if (!LauncherSessionInfo->bPlayUsingLauncherBuild)
		{
			NotificationText = FText::Format(LOCTEXT("LauncherTaskValidateNotification", "Validating Executable for {PlatformName}..."), Arguments);
		}
		else
		{
			NotificationText = FText::Format(LOCTEXT("LauncherTaskBuildNotification", "Building Executable for {PlatformName}..."), Arguments);
		}
	}
	else if (InStage.Contains(TEXT("Deploy Task")))
	{
		Arguments.Add(TEXT("DeviceName"), FText::FromString(LauncherSessionInfo->PlayUsingLauncherDeviceName));
		if (LauncherSessionInfo->PlayUsingLauncherDeviceName.Len() == 0)
		{
			NotificationText = FText::Format(LOCTEXT("LauncherTaskStageNotificationNoDevice", "Deploying Executable and Assets..."), Arguments);
		}
		else
		{
			NotificationText = FText::Format(LOCTEXT("LauncherTaskStageNotification", "Deploying Executable and Assets to {DeviceName}..."), Arguments);
		}
	}
	else if (InStage.Contains(TEXT("Run Task")))
	{
		Arguments.Add(TEXT("GameName"), FText::FromString(FApp::GetProjectName()));
		Arguments.Add(TEXT("DeviceName"), FText::FromString(LauncherSessionInfo->PlayUsingLauncherDeviceName));
		if (LauncherSessionInfo->PlayUsingLauncherDeviceName.Len() == 0)
		{
			NotificationText = FText::Format(LOCTEXT("LauncherTaskRunNotificationNoDevice", "Running {GameName}..."), Arguments);
		}
		else
		{
			NotificationText = FText::Format(LOCTEXT("LauncherTaskRunNotification", "Running {GameName} on {DeviceName}..."), Arguments);
		}
	}
	else
	{
		bSetNotification = false;
	}

	if (bSetNotification)
	{
		TGraphTask<FLauncherNotificationTask>::CreateTask().ConstructAndDispatchWhenReady(
			NotificationItemPtr,
			SNotificationItem::CS_Pending,
			NotificationText
		);
	}
}

void UEditorEngine::HandleStageCompleted(const FString& InStage, double StageTime, bool bHasCode, TWeakPtr<SNotificationItem> NotificationItemPtr)
{
	UE_LOGF(LogPlayLevel, Log, "Completed Launch On Stage: %ls, Time: %f", *InStage, StageTime);

	// analytics for launch on
	TArray<FAnalyticsEventAttribute> ParamArray;
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("Time"), StageTime));
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("StageName"), InStage));
	FEditorAnalytics::ReportEvent(TEXT( "Editor.LaunchOn.StageComplete" ), LastPlayUsingLauncherDeviceId.Left(LastPlayUsingLauncherDeviceId.Find(TEXT("@"))), bHasCode, ParamArray);
}

void UEditorEngine::HandleLaunchCanceled(double TotalTime, bool bHasCode, TWeakPtr<SNotificationItem> NotificationItemPtr)
{
	TGraphTask<FLauncherNotificationTask>::CreateTask().ConstructAndDispatchWhenReady(
		NotificationItemPtr,
		SNotificationItem::CS_Fail,
		LOCTEXT("LaunchtaskFailedNotification", "Launch canceled!")
	);

	// analytics for launch on
	TArray<FAnalyticsEventAttribute> ParamArray;
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("Time"), TotalTime));
	FEditorAnalytics::ReportEvent(TEXT( "Editor.LaunchOn.Canceled" ), LastPlayUsingLauncherDeviceId.Left(LastPlayUsingLauncherDeviceId.Find(TEXT("@"))), bHasCode, ParamArray);

	LauncherSessionInfo.Reset();
}

void UEditorEngine::HandleLaunchCompleted(bool Succeeded, double TotalTime, int32 ErrorCode, bool bHasCode, TWeakPtr<SNotificationItem> NotificationItemPtr)
{
	const FString DummyIOSDeviceName(FString::Printf(TEXT("All_iOS_On_%s"), FPlatformProcess::ComputerName()));
	const FString DummyTVOSDeviceName(FString::Printf(TEXT("All_tvOS_On_%s"), FPlatformProcess::ComputerName()));
	if (Succeeded)
	{
		FText CompletionMsg;
		if ((LastPlayUsingLauncherDeviceId.Left(LastPlayUsingLauncherDeviceId.Find(TEXT("@"))) == TEXT("IOS") && LauncherSessionInfo->PlayUsingLauncherDeviceName.Contains(DummyIOSDeviceName)) ||
			(LastPlayUsingLauncherDeviceId.Left(LastPlayUsingLauncherDeviceId.Find(TEXT("@"))) == TEXT("TVOS") && LauncherSessionInfo->PlayUsingLauncherDeviceName.Contains(DummyTVOSDeviceName)))
		{
			CompletionMsg = LOCTEXT("DeploymentTaskCompleted", "Deployment complete! Open the app on your device to launch.");
		}
		else
		{
			CompletionMsg = LOCTEXT("LauncherTaskCompleted", "Launch complete!!");
		}

		TGraphTask<FLauncherNotificationTask>::CreateTask().ConstructAndDispatchWhenReady(
			NotificationItemPtr,
			SNotificationItem::CS_Success,
			CompletionMsg
		);

		// analytics for launch on
		TArray<FAnalyticsEventAttribute> ParamArray;
		ParamArray.Add(FAnalyticsEventAttribute(TEXT("Time"), TotalTime));
		FEditorAnalytics::ReportEvent(TEXT( "Editor.LaunchOn.Completed" ), LastPlayUsingLauncherDeviceId.Left(LastPlayUsingLauncherDeviceId.Find(TEXT("@"))), bHasCode, ParamArray);

		UE_LOGF(LogPlayLevel, Log, "Launch On Completed. Time: %f", TotalTime);

		bUATSuccessfullyCompiledOnce = true;
	}
	else
	{
		FText CompletionMsg;
		if ((LastPlayUsingLauncherDeviceId.Left(LastPlayUsingLauncherDeviceId.Find(TEXT("@"))) == TEXT("IOS") && LauncherSessionInfo->PlayUsingLauncherDeviceName.Contains(DummyIOSDeviceName)) ||
			(LastPlayUsingLauncherDeviceId.Left(LastPlayUsingLauncherDeviceId.Find(TEXT("@"))) == TEXT("TVOS") && LauncherSessionInfo->PlayUsingLauncherDeviceName.Contains(DummyTVOSDeviceName)))
		{
			CompletionMsg = LOCTEXT("DeploymentTaskFailed", "Deployment failed!");
		}
		else
		{
			CompletionMsg = LOCTEXT("LauncherTaskFailed", "Launch failed!");
		}
		
		AsyncTask(ENamedThreads::GameThread, [=]
		{
			FMessageLog MessageLog("PackagingResults");

			MessageLog.Error()
				->AddToken(FTextToken::Create(CompletionMsg))
				->AddToken(FTextToken::Create(FText::FromString(FEditorAnalytics::TranslateErrorCode(ErrorCode))));

			// flush log, because it won't be destroyed until the notification popup closes
			MessageLog.NumMessages(EMessageSeverity::Info);
		});

		TGraphTask<FLauncherNotificationTask>::CreateTask().ConstructAndDispatchWhenReady(
			NotificationItemPtr,
			SNotificationItem::CS_Fail,
			CompletionMsg
		);

		TArray<FAnalyticsEventAttribute> ParamArray;
		ParamArray.Add(FAnalyticsEventAttribute(TEXT("Time"), TotalTime));
		FEditorAnalytics::ReportEvent(TEXT( "Editor.LaunchOn.Failed" ), LastPlayUsingLauncherDeviceId.Left(LastPlayUsingLauncherDeviceId.Find(TEXT("@"))), bHasCode, ErrorCode, ParamArray);
	}

	LauncherSessionInfo.Reset();
}

FString UEditorEngine::GetPlayOnTargetPlatformName() const
{
	return LastPlayUsingLauncherDeviceId.Left(LastPlayUsingLauncherDeviceId.Find(TEXT("@")));
}

#undef LOCTEXT_NAMESPACE // "PlayLevel"
