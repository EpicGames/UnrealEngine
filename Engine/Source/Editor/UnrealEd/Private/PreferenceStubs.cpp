// Copyright Epic Games, Inc. All Rights Reserved.


#include "CoreMinimal.h"
#include "Animation/Skeleton.h"
#include "Preferences/CascadeOptions.h"
#include "Preferences/CurveEdOptions.h"
#include "Preferences/MaterialEditorOptions.h"
#include "Preferences/PersonaOptions.h"
#include "Preferences/AnimationBlueprintEditorOptions.h"
#include "Preferences/PhysicsAssetEditorOptions.h"
#include "Preferences/MaterialStatsOptions.h"
#include "FrameNumberDisplayFormat.h"
#include "RHI.h"
#include "Animation/Skeleton.h"

// @todo find a better place for all of this, preferably in the appropriate modules
// though this would require the classes to be relocated as well

//
// UCascadeOptions
// 
UCascadeOptions::UCascadeOptions(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

///////////////////////////////////////////////////////////////////////////////////////////////////
////////////////// UPhysicsAssetEditorOptions /////////////////////////////////////////////////////
///////////////////////////////////////////////////////////////////////////////////////////////////
UPhysicsAssetEditorOptions::UPhysicsAssetEditorOptions(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PhysicsBlend = 1.0f;
	bUpdateJointsFromAnimation = false;
	MaxFPS = -1;

	// These should duplicate defaults from UPhysicsHandleComponent
	HandleLinearDamping = 200.0f;
	HandleLinearStiffness = 750.0f;
	HandleAngularDamping = 500.0f;
	HandleAngularStiffness = 1500.0f;
	InterpolationSpeed = 50.f;

	bShowConstraintsAsPoints = false;
	bDrawViolatedLimits = false;
	bHideCenterOfMassForKinematicBodies = false;
	bSimulationFloorCollisionEnabled = true;
	COMRenderSize = 1.0f;
	ConstraintDrawSize = 1.0f;

	// view options
	MeshViewMode = EPhysicsAssetEditorMeshViewMode::Solid;
	CenterOfMassViewMode = EPhysicsAssetEditorCenterOfMassViewMode::None;
	CollisionViewMode = EPhysicsAssetEditorCollisionViewMode::Solid;
	ConstraintViewMode = EPhysicsAssetEditorConstraintViewMode::AllLimits;
	SimulationCenterOfMassViewMode = EPhysicsAssetEditorCenterOfMassViewMode::None;
	SimulationMeshViewMode = EPhysicsAssetEditorMeshViewMode::Solid;
	SimulationCollisionViewMode = EPhysicsAssetEditorCollisionViewMode::Solid;
	SimulationConstraintViewMode = EPhysicsAssetEditorConstraintViewMode::None;

	CollisionOpacity = 0.3f;
	bSolidRenderingForSelectedOnly = false;
	bHideSimulatedBodies = false;
	bHideKinematicBodies = false;
	bHideBodyMass = true;
	bCreateConstraintsWhenCreatingBodiesFromSkeletonTree = true;
	bResetClothWhenSimulating = false;
}

UMaterialEditorOptions::UMaterialEditorOptions(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UMaterialStatsOptions::UMaterialStatsOptions(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	uint32 CurrentlyUsedSP = static_cast<int32>(GMaxRHIShaderPlatform);
	if (CurrentlyUsedSP < UE_ARRAY_COUNT(bPlatformUsed))
	{
		bPlatformUsed[CurrentlyUsedSP] = 1;
	}

	// enable a mobile platform by default so we can check if shaders are compiling for mobile
#if PLATFORM_WINDOWS
	bPlatformUsed[SP_PCD3D_ES3_1] = 1;
#elif PLATFORM_MAC
	bPlatformUsed[SP_METAL_ES3_1_IOS] = 1;
#elif PLATFORM_LINUX
	bPlatformUsed[SP_VULKAN_PCES3_1] = 1;
#endif

	bMaterialQualityUsed[EMaterialQualityLevel::High] = 1;

	MaterialStatsDerivedMIOption = EMaterialStatsDerivedMIOption::Ignore;
}

UAnimationBlueprintEditorOptions::UAnimationBlueprintEditorOptions(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UCurveEdOptions::UCurveEdOptions(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void FViewportConfigOptions::SetToDefault()
{
	ViewModeIndex = VMI_Lit;
	ViewFOV = 53.43f;
	CameraSpeedSettings = FEditorViewportCameraSpeedSettings();
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	CameraSpeedSetting_DEPRECATED = 4;
	CameraSpeedScalar_DEPRECATED = 1.0f;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	CameraFollowMode = EAnimationViewportCameraFollowMode::None;
	CameraFollowBoneName = NAME_None;
}

void FAssetEditorOptions::SetViewportConfigsToDefault()
{
	for (FViewportConfigOptions& ViewportConfig : ViewportConfigs)
	{
		ViewportConfig.SetToDefault();
	}
}

UPersonaOptions::UPersonaOptions(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, DefaultLocalAxesSelection(2)
	, DefaultBoneDrawSelection(1)
	, bPauseAnimationOnCameraMove(false)
	, bAllowPreviewMeshCollectionsToSelectFromDifferentSkeletons(true)
{
	AssetEditorOptions.AddUnique(FAssetEditorOptions(TEXT("SkeletonEditor")));
	AssetEditorOptions.AddUnique(FAssetEditorOptions(TEXT("SkeletalMeshEditor")));
	AssetEditorOptions.AddUnique(FAssetEditorOptions(TEXT("AnimationEditor")));
	AssetEditorOptions.AddUnique(FAssetEditorOptions(TEXT("AnimationBlueprintEditor")));
	AssetEditorOptions.AddUnique(FAssetEditorOptions(TEXT("PhysicsAssetEditor")));

	for(FAssetEditorOptions& EditorOptions : AssetEditorOptions)
	{
		EditorOptions.SetViewportConfigsToDefault();
	}

	SectionTimingNodeColor = FLinearColor(0.39f, 0.39f, 1.0f, 0.75f);
	NotifyTimingNodeColor = FLinearColor(0.8f, 0.1f, 0.1f);
	BranchingPointTimingNodeColor = FLinearColor(0.5f, 1.0f, 1.0f);
	
	DefaultBoneColor = FLinearColor(0.0f,0.0f,0.025f,1.0f);
	SelectedBoneColor = FLinearColor(0.2f,1.0f,0.2f,1.0f);
	AffectedBoneColor = FLinearColor(1.0f,1.0f,1.0f,1.0f);
	DisabledBoneColor = FLinearColor(0.4f,0.4f,0.4f,1.0f);
	ParentOfSelectedBoneColor = FLinearColor(0.85f,0.45f,0.12f,1.0f);
	VirtualBoneColor = FLinearColor(0.4f, 0.4f, 1.0f, 1.0f);

	bAutoAlignFloorToMesh = true;

	NumFolderFiltersInAssetBrowser = 2;

	CurveEditorSnapInterval = 0.01f;

	// Default to millisecond resolution
	TimelineScrubSnapValue = 1000;

	TimelineDisplayFormat = EFrameNumberDisplayFormats::Frames;

	bTimelineDisplayPercentage = true;
	bTimelineDisplayFormatSecondary = true;

	bTimelineDisplayCurveKeys = false;

	TimelineEnabledSnaps = { "CompositeSegment", "MontageSection" };

	bExpandTreeOnSelection = true;

	bAllowIncompatibleSkeletonSelection = false;

	USkeleton::AreAllSkeletonsCompatibleDelegate.BindUObject(this, &UPersonaOptions::GetAllowIncompatibleSkeletonSelection);
}

void UPersonaOptions::SetShowGrid( bool bInShowGrid )
{
	bShowGrid = bInShowGrid;
	SaveConfig();
}

void UPersonaOptions::SetHighlightOrigin( bool bInHighlightOrigin )
{
	bHighlightOrigin = bInHighlightOrigin;
	SaveConfig();
}

void UPersonaOptions::SetViewModeIndex( FName InContext, EViewModeIndex InViewModeIndex, int32 InViewportIndex )
{
	check(InViewportIndex >= 0 && InViewportIndex < 4);

	FAssetEditorOptions& Options = GetAssetEditorOptions(InContext);
	Options.ViewportConfigs[InViewportIndex].ViewModeIndex = InViewModeIndex;
	SaveConfig();
}

void UPersonaOptions::SetAutoAlignFloorToMesh(bool bInAutoAlignFloorToMesh)
{
	bAutoAlignFloorToMesh = bInAutoAlignFloorToMesh;
	SaveConfig();
}

void UPersonaOptions::SetMuteAudio( bool bInMuteAudio )
{
	bMuteAudio = bInMuteAudio;
	SaveConfig();
}

void UPersonaOptions::SetViewFOV( FName InContext, float InViewFOV, int32 InViewportIndex )
{
	check(InViewportIndex >= 0 && InViewportIndex < 4);

	FAssetEditorOptions& Options = GetAssetEditorOptions(InContext);
	Options.ViewportConfigs[InViewportIndex].ViewFOV = InViewFOV;
	SaveConfig();
}

void UPersonaOptions::SetCameraSpeedSettings(FName InContext, const FEditorViewportCameraSpeedSettings& SpeedSettings, int32 InViewportIndex)
{
	check(InViewportIndex >= 0 && InViewportIndex < 4);

	FAssetEditorOptions& Options = GetAssetEditorOptions(InContext);
	Options.ViewportConfigs[InViewportIndex].CameraSpeedSettings = SpeedSettings;
	SaveConfig();
}

void UPersonaOptions::SetCameraSpeed(FName InContext, int32 InCameraSpeed, int32 InViewportIndex)
{
	check(InViewportIndex >= 0 && InViewportIndex < 4);

	FAssetEditorOptions& Options = GetAssetEditorOptions(InContext);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Options.ViewportConfigs[InViewportIndex].CameraSpeedSetting_DEPRECATED = InCameraSpeed;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	SaveConfig();
}

void UPersonaOptions::SetCameraSpeedScalar(FName InContext, float InCameraSpeedScalar, int32 InViewportIndex)
{
	check(InViewportIndex >= 0 && InViewportIndex < 4);

	FAssetEditorOptions& Options = GetAssetEditorOptions(InContext);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Options.ViewportConfigs[InViewportIndex].CameraSpeedScalar_DEPRECATED = InCameraSpeedScalar;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	SaveConfig();
}

void UPersonaOptions::SetViewCameraFollow( FName InContext, EAnimationViewportCameraFollowMode InCameraFollowMode, FName InCameraFollowBoneName, int32 InViewportIndex )
{
	check(InViewportIndex >= 0 && InViewportIndex < 4);

	FAssetEditorOptions& Options = GetAssetEditorOptions(InContext);
	Options.ViewportConfigs[InViewportIndex].CameraFollowMode = InCameraFollowMode;
	Options.ViewportConfigs[InViewportIndex].CameraFollowBoneName = InCameraFollowBoneName;
	SaveConfig();
}

void UPersonaOptions::SetDefaultLocalAxesSelection( uint32 InDefaultLocalAxesSelection )
{
	DefaultLocalAxesSelection = InDefaultLocalAxesSelection;
	SaveConfig();
}

void UPersonaOptions::SetDefaultBoneDrawSelection(uint32 InDefaultBoneDrawSelection)
{
	DefaultBoneDrawSelection = InDefaultBoneDrawSelection;
	SaveConfig();
}

void UPersonaOptions::SetShowMeshStats( int32 InShowMeshStats )
{
	ShowMeshStats = InShowMeshStats;
	SaveConfig();
}


void UPersonaOptions::SetSectionTimingNodeColor(const FLinearColor& InColor)
{
	SectionTimingNodeColor = InColor;
	SaveConfig();
}

void UPersonaOptions::SetNotifyTimingNodeColor(const FLinearColor& InColor)
{
	NotifyTimingNodeColor = InColor;
	SaveConfig();
}

void UPersonaOptions::SetBranchingPointTimingNodeColor(const FLinearColor& InColor)
{
	BranchingPointTimingNodeColor = InColor;
	SaveConfig();
}

FAssetEditorOptions& UPersonaOptions::GetAssetEditorOptions(const FName& InContext)
{
	FAssetEditorOptions* FoundOptions = AssetEditorOptions.FindByPredicate([InContext](const FAssetEditorOptions& InOptions)
	{
		return InOptions.Context == InContext;
	});

	if(!FoundOptions)
	{
		AssetEditorOptions.Add(FAssetEditorOptions(InContext));
		return AssetEditorOptions.Last();
	}
	else
	{
		return *FoundOptions;
	}
}

void UPersonaOptions::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	OnSettingsChange.Broadcast(this, PropertyChangedEvent.ChangeType);
}

bool UPersonaOptions::GetAllowIncompatibleSkeletonSelection() const
{
	return bAllowIncompatibleSkeletonSelection;
}

void UPersonaOptions::SetAllowIncompatibleSkeletonSelection(bool bState)
{
	bAllowIncompatibleSkeletonSelection = bState;
	SaveConfig();
}
