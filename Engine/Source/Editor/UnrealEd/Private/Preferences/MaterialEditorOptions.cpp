// Copyright Epic Games, Inc. All Rights Reserved.

#include "Preferences/MaterialEditorOptions.h"

#include "Editor/UnrealEdEngine.h"
#include "Engine/StaticMesh.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "UnrealEdGlobals.h"

UMaterialEditorOptions* UMaterialEditorOptions::Get()
{
	return GetMutableDefault<UMaterialEditorOptions>();
}

UObject* UMaterialEditorOptions::GetPreviewMesh() const
{
	UObject* PreviewMeshObject = PreviewMesh.TryLoad();

	// Fallback to legacy preview static mesh : sphere
	if (!PreviewMeshObject)
	{
		PreviewMeshObject = GUnrealEd->GetThumbnailManager()->EditorSphere;
	}

	return PreviewMeshObject;
}

void UMaterialEditorOptions::SetPreviewMesh(UObject* InPreviewMesh)
{
	if (!InPreviewMesh || PreviewMesh == InPreviewMesh)
	{
		return;
	}

	PreviewMesh = InPreviewMesh->GetPathName();
}
