// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialEditor/PreviewMaterial.h"
#include "Modules/ModuleManager.h"
#include "MaterialEditor/DEditorParameterValue.h"
#include "MaterialEditor/DEditorFontParameterValue.h"
#include "MaterialEditor/DEditorMaterialLayersParameterValue.h"
#include "MaterialEditor/DEditorRuntimeVirtualTextureParameterValue.h"
#include "MaterialEditor/DEditorScalarParameterValue.h"
#include "MaterialEditor/DEditorStaticComponentMaskParameterValue.h"
#include "MaterialEditor/DEditorStaticSwitchParameterValue.h"
#include "MaterialEditor/DEditorTextureParameterValue.h"
#include "MaterialEditor/DEditorVectorParameterValue.h"
#include "MaterialEditor/DEditorDoubleVectorParameterValue.h"
#include "AI/NavigationSystemBase.h"
#include "MaterialEditor/MaterialEditorInstanceConstant.h"
#include "MaterialEditor/MaterialEditorPreviewParameters.h"
#include "MaterialEditor/MaterialEditorMeshComponent.h"
#include "MaterialEditorModule.h"
#include "MaterialCachedData.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialFunctionInstance.h"
#include "Materials/MaterialExpressionScalarParameter.h"
#include "Materials/MaterialExpressionVectorParameter.h"
#include "Materials/MaterialExpressionTextureSampleParameter.h"
#include "Materials/MaterialExpressionRuntimeVirtualTextureSampleParameter.h"
#include "Materials/MaterialExpressionFontSampleParameter.h"
#include "Materials/MaterialExpressionMaterialAttributeLayers.h"
#include "Materials/MaterialExpressionStaticBoolParameter.h"
#include "Materials/MaterialExpressionStaticComponentMaskParameter.h"
#include "Materials/MaterialFunction.h"
#include "UObject/UObjectIterator.h"
#include "PropertyEditorDelegates.h"
#include "IDetailsView.h"
#include "MaterialEditingLibrary.h"
#include "MaterialPropertyHelpers.h"
#include "MaterialStatsCommon.h"
#include "MaterialDomain.h"
#include "RenderUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PreviewMaterial)

/**
 * Class for rendering the material on the preview mesh in the Material Editor
 */
class FPreviewMaterial : public FMaterialResource
{
public:
	virtual ~FPreviewMaterial()
	{
	}

	/**
	 * Should the shader for this material with the given platform, shader type and vertex 
	 * factory type combination be compiled
	 *
	 * @param ShaderType	Which shader is being compiled
	 * @param VertexFactory	Which vertex factory is being compiled (can be NULL)
	 *
	 * @return true if the shader should be compiled
	 */
	virtual bool ShouldCache(const FShaderType* ShaderType, const FVertexFactoryType* VertexFactoryType) const override
	{
		// only generate the needed shaders (which should be very restrictive for fast recompiling during editing)
		// @todo: Add a FindShaderType by fname or something

		if( Material->IsUIMaterial() )
		{
			if (FCString::Stristr(ShaderType->GetName(), TEXT("TSlateMaterialShaderPS")) ||
				FCString::Stristr(ShaderType->GetName(), TEXT("TSlateMaterialShaderVS")))
			{
				return true;
			}
	
		}

		if (Material->IsPostProcessMaterial())
		{
			if (FCString::Stristr(ShaderType->GetName(), TEXT("PostProcess")))
			{
				return true;
			}
		}

		{
			bool bEditorStatsMaterial = Material->bIsMaterialEditorStatsMaterial;

			// Always allow HitProxy shaders.
			if (FCString::Stristr(ShaderType->GetName(), TEXT("HitProxy")))
			{
				return true;
			}

			// we only need local vertex factory for the preview static mesh
			if (VertexFactoryType != FindVertexFactoryType(FName(TEXT("FLocalVertexFactory"), FNAME_Find)) &&
				VertexFactoryType != FindVertexFactoryType(FName(TEXT("FNaniteVertexFactory"), FNAME_Find)))
			{
				//cache for gpu skinned vertex factory if the material allows it
				//this way we can have a preview skeletal mesh
				if (bEditorStatsMaterial ||
					!IsUsedWithSkeletalMesh())
				{
					return false;
				}

				if (
					VertexFactoryType != FindVertexFactoryType(FName(TEXT("TGPUSkinVertexFactoryDefault"), FNAME_Find)) &&
					VertexFactoryType != FindVertexFactoryType(FName(TEXT("TGPUSkinVertexFactoryUnlimited"), FNAME_Find))
					)
				{
					return false;
				}
			}

			// Only allow shaders that are used in the stats.
			if (bEditorStatsMaterial)
			{
				TMap<FName, TArray<FMaterialStatsUtils::FRepresentativeShaderInfo>> ShaderTypeNamesAndDescriptions;
				FMaterialStatsUtils::GetRepresentativeShaderTypesAndDescriptions(ShaderTypeNamesAndDescriptions, this);

				for (auto DescriptionPair : ShaderTypeNamesAndDescriptions)
				{
					auto &DescriptionArray = DescriptionPair.Value;
					if (DescriptionArray.FindByPredicate([ShaderType = ShaderType](auto& Info) { return Info.ShaderName == ShaderType->GetFName(); }))
					{
						return true;
					}
				}

				return false;
			}

			// look for any of the needed type
			bool bShaderTypeMatches = false;

			// For FMaterialResource::GetRepresentativeInstructionCounts
			if (FCString::Stristr(ShaderType->GetName(), TEXT("MaterialCHSFNoLightMapPolicy")))
			{
				bShaderTypeMatches = true;
			}
			else if (FCString::Stristr(ShaderType->GetName(), TEXT("MobileDirectionalLight")))
			{
				bShaderTypeMatches = true;
			}
			else if (FCString::Stristr(ShaderType->GetName(), TEXT("MobileMovableDirectionalLight")))
			{
				bShaderTypeMatches = true;
			}
			else if (FCString::Stristr(ShaderType->GetName(), TEXT("BasePassPSTDistanceFieldShadowsAndLightMapPolicyHQ")))
			{
				bShaderTypeMatches = true;
			}
			else if (FCString::Stristr(ShaderType->GetName(), TEXT("Simple")))
			{
				bShaderTypeMatches = true;
			}
			else if (FCString::Stristr(ShaderType->GetName(), TEXT("BasePassPSFNoLightMapPolicy")))
			{
				bShaderTypeMatches = true;
			}
			else if (FCString::Stristr(ShaderType->GetName(), TEXT("TBasePassCSFNoLightMapPolicy")))
			{
				bShaderTypeMatches = true;
			}
			else if (FCString::Stristr(ShaderType->GetName(), TEXT("CachedPointIndirectLightingPolicy")))
			{
				bShaderTypeMatches = true;
			}
			else if (FCString::Stristr(ShaderType->GetName(), TEXT("PrecomputedVolumetricLightmapLightingPolicy")))
			{
				bShaderTypeMatches = true;
			}
			else if (FCString::Stristr(ShaderType->GetName(), TEXT("BasePassPSFSelfShadowedTranslucencyPolicy")))
			{
				bShaderTypeMatches = true;
			}
			// Pick tessellation shader based on material settings
			else if(FCString::Stristr(ShaderType->GetName(), TEXT("BasePassVSFNoLightMapPolicy")) ||
				FCString::Stristr(ShaderType->GetName(), TEXT("BasePassHSFNoLightMapPolicy")) ||
				FCString::Stristr(ShaderType->GetName(), TEXT("BasePassDSFNoLightMapPolicy")))
			{
				bShaderTypeMatches = true;
			}
			else if (FCString::Stristr(ShaderType->GetName(), TEXT("DepthOnly")))
			{
				bShaderTypeMatches = true;
			}
			else if (FCString::Stristr(ShaderType->GetName(), TEXT("ShadowDepth")))
			{
				bShaderTypeMatches = true;
			}
			else if (FCString::Stristr(ShaderType->GetName(), TEXT("Distortion")))
			{
				bShaderTypeMatches = true;
			}
			else if (FCString::Stristr(ShaderType->GetName(), TEXT("MeshDecal")))
			{
				bShaderTypeMatches = true;
			}
			else if (FCString::Stristr(ShaderType->GetName(), TEXT("TBasePassForForwardShading")))
			{
				bShaderTypeMatches = true;
			}
			else if (FCString::Stristr(ShaderType->GetName(), TEXT("FDebugViewModeVS")))
			{
				bShaderTypeMatches = true;
			}
			else if (FCString::Stristr(ShaderType->GetName(), TEXT("FDebugViewModePS")))
			{
				bShaderTypeMatches = true;
			}
			else if (FCString::Stristr(ShaderType->GetName(), TEXT("TVelocity")))
			{
				bShaderTypeMatches = true;
			}
			else if (FCString::Stristr(ShaderType->GetName(), TEXT("FAnisotropy")))
			{
				bShaderTypeMatches = true;
			}
			else if (FCString::Stristr(ShaderType->GetName(), TEXT("RayTracingDynamicGeometryConverter")))
			{
				bShaderTypeMatches = true;
			}
			else if (FCString::Stristr(ShaderType->GetName(), TEXT("FLumenCard")))
			{
				bShaderTypeMatches = true;
			}
			else if (FCString::Stristr(ShaderType->GetName(), TEXT("FFrontLayerTranslucencyGBuffer")))
			{
				bShaderTypeMatches = true;
			}

			return bShaderTypeMatches;
		}
	
	}

	/**
	 * Should shaders compiled for this material be saved to disk?
	 */
	virtual bool IsPersistent() const override { return false; }
	virtual FString GetAssetName() const override { return FString::Printf(TEXT("Preview:%s"), *FMaterialResource::GetAssetName()); }

	virtual bool IsPreview() const override { return true; }
};

/** Implementation of Preview Material functions*/
UPreviewMaterial::UPreviewMaterial(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FMaterialResource* UPreviewMaterial::AllocateResource()
{
	return new FPreviewMaterial();
}

uint32 UPreviewMaterial::GetUsageFlags() const
{
	// Allows enable static mesh usage in preview.
	return Super::GetUsageFlags() | (1u << MATUSAGE_StaticMesh);
}

bool UPreviewMaterial::GetUsageByFlag(EMaterialUsage Usage) const
{
	// Allows enable static mesh usage in preview.
	return (Usage == MATUSAGE_StaticMesh) || Super::GetUsageByFlag(Usage);
}

void UMaterialEditorPreviewParameters::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PreviewMaterial && PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
		if (OriginalFunction == nullptr)
		{
			bool bLayersParameterChanged = false;
			// If a material layers parameter changed we need to update it on the source instance
			// immediately so parameters contained within the new functions can be collected
			for (FEditorParameterGroup& Group : ParameterGroups)
			{
				for (UDEditorParameterValue* Parameter : Group.Parameters)
				{
					if (UDEditorMaterialLayersParameterValue* LayersParam = Cast<UDEditorMaterialLayersParameterValue>(Parameter))
					{
						UMaterialExpressionMaterialAttributeLayers* LayersNode = nullptr;
						for(UMaterialExpression* Expression : PreviewMaterial->GetExpressions())
						{
							if(Expression->IsA<UMaterialExpressionMaterialAttributeLayers>())
							{
								LayersNode = Cast<UMaterialExpressionMaterialAttributeLayers>(Expression);
								LayersNode->DefaultLayers = LayersParam->ParameterValue;
								bLayersParameterChanged = true;
								break;
							}
						}
					}
				}
			}

			if (bLayersParameterChanged)
			{
				RegenerateArrays();
			}
			
			CopyToSourceInstance();
			PreviewMaterial->PostEditChangeProperty(PropertyChangedEvent);
		}
		else
		{
			ApplySourceFunctionChanges();
			if (OriginalFunction->PreviewMaterial)
			{
				OriginalFunction->PreviewMaterial->PostEditChangeProperty(PropertyChangedEvent);
			}
		}
	}
}

void UMaterialEditorPreviewParameters::AssignParameterToGroup(UDEditorParameterValue* ParameterValue, const FName& InParameterGroupName)
{
	check(ParameterValue);

	FName ParameterGroupName = InParameterGroupName;
	if (ParameterGroupName == TEXT("") || ParameterGroupName == TEXT("None"))
	{
		ParameterGroupName = TEXT("None");
	}
	
	if (ParameterValue->ParameterInfo.Association == EMaterialParameterAssociation::GlobalParameter)
	{
		FString AppendedGroupName = GlobalGroupPrefix.ToString();
		if (ParameterGroupName != TEXT("None"))
		{
			ParameterGroupName.AppendString(AppendedGroupName);
			ParameterGroupName = FName(*AppendedGroupName);
		}
		else
		{
			ParameterGroupName = TEXT("Global");
		}
	}

	FEditorParameterGroup& CurrentGroup = FMaterialPropertyHelpers::GetParameterGroup(PreviewMaterial, ParameterGroupName, ParameterGroups);
	CurrentGroup.GroupAssociation = ParameterValue->ParameterInfo.Association;
	ParameterValue->SetFlags(RF_Transactional);
	CurrentGroup.Parameters.Add(ParameterValue);
}

void UMaterialEditorPreviewParameters::RegenerateArrays()
{
	ParameterGroups.Empty();
	if (PreviewMaterial)
	{
		// Only operate on base materials
		UMaterial* ParentMaterial = PreviewMaterial;

		// This can run before UMaterial::PostEditChangeProperty has a chance to run, so explicitly call UpdateCachedExpressionData here
		PreviewMaterial->UpdateCachedExpressionData();

		// Loop through all types of parameters for this material and add them to the parameter arrays.
		TMap<FMaterialParameterInfo, FMaterialParameterMetadata> ParameterValues;
		for (int32 TypeIndex = 0; TypeIndex < NumMaterialParameterTypes; ++TypeIndex)
		{
			const EMaterialParameterType ParameterType = (EMaterialParameterType)TypeIndex;

			ParentMaterial->GetAllParametersOfType(ParameterType, ParameterValues);
			for (const auto& It : ParameterValues)
			{
				UDEditorParameterValue* Parameter = UDEditorParameterValue::Create(this, ParameterType, It.Key, It.Value);
				Parameter->bOverride = true;
				AssignParameterToGroup(Parameter, It.Value.Group);
			}
		}
	}
	// sort contents of groups
	for (int32 ParameterIdx = 0; ParameterIdx < ParameterGroups.Num(); ParameterIdx++)
	{
		FEditorParameterGroup & ParamGroup = ParameterGroups[ParameterIdx];
		struct FCompareUDEditorParameterValueByParameterName
		{
			FORCEINLINE bool operator()(const UDEditorParameterValue& A, const UDEditorParameterValue& B) const
			{
				FString AName = A.ParameterInfo.Name.ToString();
				FString BName = B.ParameterInfo.Name.ToString();
				return A.SortPriority != B.SortPriority ? A.SortPriority < B.SortPriority : AName < BName;
			}
		};
		ParamGroup.Parameters.Sort(FCompareUDEditorParameterValueByParameterName());
	}

	// sort groups itself pushing defaults to end
	struct FCompareFEditorParameterGroupByName
	{
		FORCEINLINE bool operator()(const FEditorParameterGroup& A, const FEditorParameterGroup& B) const
		{
			FString AName = A.GroupName.ToString();
			FString BName = B.GroupName.ToString();
			if (AName == TEXT("none"))
			{
				return false;
			}
			if (BName == TEXT("none"))
			{
				return false;
			}
			return A.GroupSortPriority != B.GroupSortPriority ? A.GroupSortPriority < B.GroupSortPriority : AName < BName;
		}
	};
	ParameterGroups.Sort(FCompareFEditorParameterGroupByName());
	TArray<struct FEditorParameterGroup> ParameterDefaultGroups;
	for (int32 ParameterIdx = 0; ParameterIdx < ParameterGroups.Num(); ParameterIdx++)
	{
		FEditorParameterGroup & ParamGroup = ParameterGroups[ParameterIdx];

		if (ParamGroup.GroupName == TEXT("None"))
		{
			ParameterDefaultGroups.Add(ParamGroup);
			ParameterGroups.RemoveAt(ParameterIdx);
			break;
		}
	}

	if (ParameterDefaultGroups.Num() > 0)
	{
		ParameterGroups.Append(ParameterDefaultGroups);
	}

}

TObjectPtr<UMaterialInterface> UMaterialEditorPreviewParameters::GetMaterialInterface()
{
	return Cast<UMaterialInterface>(PreviewMaterial);
}

void UMaterialEditorPreviewParameters::CopyToSourceInstance(const bool bForceStaticPermutationUpdate/* = false*/)
{
	if (PreviewMaterial->IsTemplate(RF_ClassDefaultObject) == false && OriginalMaterial != nullptr)
	{
		OriginalMaterial->MarkPackageDirty();

		// Scalar Parameters
		for (int32 GroupIdx = 0; GroupIdx < ParameterGroups.Num(); GroupIdx++)
		{
			FEditorParameterGroup & Group = ParameterGroups[GroupIdx];
			for (int32 ParameterIdx = 0; ParameterIdx < Group.Parameters.Num(); ParameterIdx++)
			{
				UDEditorParameterValue* Parameter = Group.Parameters[ParameterIdx];
				if (Parameter)
				{
					FMaterialParameterMetadata EditorValue;
					if (Parameter->GetValue(EditorValue))
					{
						PreviewMaterial->SetParameterValueEditorOnly(Parameter->ParameterInfo.Name, EditorValue);
					}
					else if (UDEditorMaterialLayersParameterValue* LayersParameter = Cast<UDEditorMaterialLayersParameterValue>(Parameter))
					{
						// find material expression material attribute layer and update it's default/param layers
						for(UMaterialExpression* Expression : PreviewMaterial->GetExpressions())
						{
							if(Expression->IsA<UMaterialExpressionMaterialAttributeLayers>())
							{
								UMaterialExpressionMaterialAttributeLayers* LayersNode = Cast<UMaterialExpressionMaterialAttributeLayers>(Expression);
								LayersNode->DefaultLayers = LayersParameter->ParameterValue;
								break;
							}
						}
					}
				}
			}
		}
	}
}

FName UMaterialEditorPreviewParameters::GlobalGroupPrefix = FName("Global ");

void UMaterialEditorPreviewParameters::ApplySourceFunctionChanges()
{
	if (OriginalFunction != nullptr)
	{
		CopyToSourceInstance();

		OriginalFunction->MarkPackageDirty();
		// Scalar Parameters
		for (int32 GroupIdx = 0; GroupIdx < ParameterGroups.Num(); GroupIdx++)
		{
			FEditorParameterGroup & Group = ParameterGroups[GroupIdx];
			for (int32 ParameterIdx = 0; ParameterIdx < Group.Parameters.Num(); ParameterIdx++)
			{
				UDEditorParameterValue* Parameter = Group.Parameters[ParameterIdx];
				if (Parameter)
				{
					FMaterialParameterMetadata EditorValue;
					if (Parameter->GetValue(EditorValue))
					{
						OriginalFunction->SetParameterValueEditorOnly(Parameter->ParameterInfo.Name, EditorValue);
					}
				}
			}
		}
		UMaterialEditingLibrary::UpdateMaterialFunction(OriginalFunction, PreviewMaterial);
	}
}


#if WITH_EDITOR
void UMaterialEditorPreviewParameters::PostEditUndo()
{
	Super::PostEditUndo();
}
#endif


FName UMaterialEditorInstanceConstant::GlobalGroupPrefix = FName("Global ");

UMaterialEditorInstanceConstant::UMaterialEditorInstanceConstant(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsFunctionPreviewMaterial = false;
	bShowOnlyOverrides = false;

	// Default to override with nothing on MIC (don't inherit parent setting).
	bNaniteOverride = true;
}

void UMaterialEditorInstanceConstant::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (SourceInstance)
	{
		// Warn our source instance that it is about to be updated.
		SourceInstance->PreEditChange(PropertyChangedEvent.Property);

		FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
		bool bLayersParameterChanged = false;

		FNavigationLockContext NavUpdateLock(ENavigationLockReason::MaterialUpdate);

		if(PropertyThatChanged && PropertyThatChanged->GetName()==TEXT("Parent") )
		{
			if(bIsFunctionPreviewMaterial)
			{
				bIsFunctionInstanceDirty = true;
				ApplySourceFunctionChanges();
			}
			else
			{
				FMaterialUpdateContext Context;

				UpdateSourceInstanceParent();

				Context.AddMaterialInstance(SourceInstance);

				// Fully update static parameters before recreating render state for all components
				SetSourceInstance(SourceInstance);
				
				ClearInvalidParameterOverrides();
			}
		}
		else if (!bIsFunctionPreviewMaterial)
		{
			// If a material layers parameter changed we need to update it on the source instance
			// immediately so parameters contained within the new functions can be collected
			for (FEditorParameterGroup& Group : ParameterGroups)
			{
				for (UDEditorParameterValue* Parameter : Group.Parameters)
				{
					if (UDEditorMaterialLayersParameterValue* LayersParam = Cast<UDEditorMaterialLayersParameterValue>(Parameter))
					{
						if (SourceInstance->SetMaterialLayers(LayersParam->ParameterValue))
						{
							bLayersParameterChanged = true;
						}
					}
				}
			}

			if (bLayersParameterChanged)
			{
				RegenerateArrays();
			}
		}
		else if (bIsFunctionPreviewMaterial)
		{
			// If a property changed and we are editing a material function instance
			// we should dirty the function and apply the parameter changes to the source material
			// This way you get live previews and updates of the thumbnails.
			bIsFunctionInstanceDirty = true;
			ApplySourceFunctionChanges();
		}

		CopyToSourceInstance(bLayersParameterChanged);

		SourceInstance->PostEditChangeProperty(PropertyChangedEvent);
		// Also notify preview because it won't receive changes from parent during interactive edits.
		PreviewInstance->PostEditChangeProperty(PropertyChangedEvent);

		// Invalidate the streaming data so that it gets rebuilt.
		SourceInstance->TextureStreamingData.Empty();
	}
}

void  UMaterialEditorInstanceConstant::AssignParameterToGroup(UDEditorParameterValue* ParameterValue, const FName& InParameterGroupName)
{
	check(ParameterValue);

	FName ParameterGroupName = InParameterGroupName;
	if (ParameterGroupName == TEXT("") || ParameterGroupName == TEXT("None"))
	{
		if (bUseOldStyleMICEditorGroups == true)
		{
			ParameterGroupName = ParameterValue->GetDefaultGroupName();
		}
		else
		{
			ParameterGroupName = TEXT("None");
		}

		IMaterialEditorModule* MaterialEditorModule = &FModuleManager::LoadModuleChecked<IMaterialEditorModule>("MaterialEditor");

		// Material layers
		if (ParameterValue->ParameterInfo.Association == EMaterialParameterAssociation::GlobalParameter)
		{
			FString AppendedGroupName = GlobalGroupPrefix.ToString();
			if (ParameterGroupName != TEXT("None"))
			{
				ParameterGroupName.AppendString(AppendedGroupName);
				ParameterGroupName = FName(*AppendedGroupName);
			}
			else
			{
				ParameterGroupName = TEXT("Global");
			}
		}
	}

	FEditorParameterGroup& CurrentGroup = FMaterialPropertyHelpers::GetParameterGroup(Parent->GetMaterial(), ParameterGroupName, ParameterGroups);
	CurrentGroup.GroupAssociation = ParameterValue->ParameterInfo.Association;
	ParameterValue->SetFlags(RF_Transactional);
	CurrentGroup.Parameters.Add(ParameterValue);
}

TObjectPtr<UMaterialInterface> UMaterialEditorInstanceConstant::GetMaterialInterface()
{
	return Cast<UMaterialInterface>(SourceInstance);
}

TObjectPtr<UMaterialInterface> UMaterialEditorInstanceConstant::GetParentMaterialInterface()
{
	return SourceInstance ? SourceInstance->Parent : nullptr;
}

void UMaterialEditorInstanceConstant::RegenerateArrays()
{
	VisibleExpressions.Empty();
	ParameterGroups.Empty();
	PostProcessOverrides.bIsOverrideable = false;
	PostProcessOverrides.UserSceneTextureInputs.Empty();

	if (Parent)
	{	
		// Only operate on base materials
		UMaterial* ParentMaterial = Parent->GetMaterial();
		SourceInstance->UpdateParameterNames();	// Update any parameter names that may have changed.
		SourceInstance->UpdateCachedData();

		// Need to get layer info first as other params are collected from layers
		{
			FMaterialLayersFunctions MaterialLayers;
			if (SourceInstance->GetMaterialLayers(MaterialLayers))
			{
				UDEditorMaterialLayersParameterValue& ParameterValue = *(NewObject<UDEditorMaterialLayersParameterValue>(this));
				ParameterValue.bOverride = true;
				ParameterValue.ParameterValue = MoveTemp(MaterialLayers);
				AssignParameterToGroup(&ParameterValue, FName());
			}
		}

		TMap<FMaterialParameterInfo, FMaterialParameterMetadata> ParameterValues;
		for (int32 TypeIndex = 0; TypeIndex < NumMaterialParameterTypes; ++TypeIndex)
		{
			const EMaterialParameterType ParameterType = (EMaterialParameterType)TypeIndex;
			SourceInstance->GetAllParametersOfType(ParameterType, ParameterValues);
			for (const auto& It : ParameterValues)
			{
				UDEditorParameterValue* Parameter = UDEditorParameterValue::Create(this, ParameterType, It.Key, It.Value);
				AssignParameterToGroup(Parameter, It.Value.Group);
			}
		}

		if (ParentMaterial->MaterialDomain == MD_PostProcess && ParentMaterial->BlendableLocation != BL_ReplacingTonemapper)
		{
			PostProcessOverrides.bIsOverrideable = true;

			UMaterialInterfaceEditorOnlyData* ParentMaterialEditorData = ParentMaterial->GetEditorOnlyData();
			if (ParentMaterialEditorData && ParentMaterialEditorData->CachedExpressionData.IsValid())
			{
				const FMaterialCachedExpressionEditorOnlyData* ParentMaterialExpressionData = ParentMaterialEditorData->CachedExpressionData.Get();
				for (FName UserSceneTextureInput : ParentMaterialExpressionData->UserSceneTextureInputs)
				{
					FName OverrideFound = NAME_None;
					for (const FUserSceneTextureOverride& Override : SourceInstance->UserSceneTextureOverrides)
					{
						if (Override.Key == UserSceneTextureInput)
						{
							OverrideFound = Override.Value;
							break;
						}
					}

					PostProcessOverrides.UserSceneTextureInputs.Add({ UserSceneTextureInput, OverrideFound });
				}
			}
		}

		IMaterialEditorModule* MaterialEditorModule = &FModuleManager::LoadModuleChecked<IMaterialEditorModule>( "MaterialEditor" );
		MaterialEditorModule->GetVisibleMaterialParameters(ParentMaterial, SourceInstance, VisibleExpressions);
	}

	// sort contents of groups
	for(int32 ParameterIdx = 0; ParameterIdx < ParameterGroups.Num(); ParameterIdx++)
	{
		FEditorParameterGroup & ParamGroup = ParameterGroups[ParameterIdx];
		struct FCompareUDEditorParameterValueByParameterName
		{
			FORCEINLINE bool operator()(const UDEditorParameterValue& A, const UDEditorParameterValue& B) const
			{
				FString AName = A.ParameterInfo.Name.ToString();
				FString BName = B.ParameterInfo.Name.ToString();
				return A.SortPriority != B.SortPriority ? A.SortPriority < B.SortPriority : AName < BName;
			}
		};
		ParamGroup.Parameters.Sort( FCompareUDEditorParameterValueByParameterName() );
	}	
	
	// sort groups itself pushing defaults to end
	struct FCompareFEditorParameterGroupByName
	{
		FORCEINLINE bool operator()(const FEditorParameterGroup& A, const FEditorParameterGroup& B) const
		{
			FString AName = A.GroupName.ToString();
			FString BName = B.GroupName.ToString();
			if (AName == TEXT("none"))
			{
				return false;
			}
			if (BName == TEXT("none"))
			{
				return false;
			}
			return A.GroupSortPriority != B.GroupSortPriority ? A.GroupSortPriority < B.GroupSortPriority : AName < BName;
		}
	};
	ParameterGroups.Sort( FCompareFEditorParameterGroupByName() );

	TArray<struct FEditorParameterGroup> ParameterDefaultGroups;
	for(int32 ParameterIdx=0; ParameterIdx<ParameterGroups.Num(); ParameterIdx++)
	{
		FEditorParameterGroup & ParamGroup = ParameterGroups[ParameterIdx];
		if (bUseOldStyleMICEditorGroups == false)
		{			
			if (ParamGroup.GroupName == TEXT("None"))
			{
				ParameterDefaultGroups.Add(ParamGroup);
				ParameterGroups.RemoveAt(ParameterIdx);
				break;
			}
		}
		else
		{
			if (ParamGroup.GroupName == TEXT("Vector Parameter Values") || 
				ParamGroup.GroupName == TEXT("Scalar Parameter Values") ||
				ParamGroup.GroupName == TEXT("Texture Parameter Values") ||
				ParamGroup.GroupName == TEXT("Static Switch Parameter Values") ||
				ParamGroup.GroupName == TEXT("Static Component Mask Parameter Values") ||
				ParamGroup.GroupName == TEXT("Font Parameter Values") ||
				ParamGroup.GroupName == TEXT("Material Layers Parameter Values"))
			{
				ParameterDefaultGroups.Add(ParamGroup);
				ParameterGroups.RemoveAt(ParameterIdx);
			}
		}
	}

	if (ParameterDefaultGroups.Num() >0)
	{
		ParameterGroups.Append(ParameterDefaultGroups);
	}

	if (DetailsView.IsValid())
	{
		// Tell our source instance to update itself so the preview updates.
		DetailsView.Pin()->ForceRefresh();
	}
}

#if WITH_EDITOR
void UMaterialEditorInstanceConstant::CleanParameterStack(int32 Index, EMaterialParameterAssociation MaterialType)
{
	check(GIsEditor);
	TArray<FEditorParameterGroup> CleanedGroups;
	for (FEditorParameterGroup Group : ParameterGroups)
	{
		FEditorParameterGroup DuplicatedGroup = FEditorParameterGroup();
		DuplicatedGroup.GroupAssociation = Group.GroupAssociation;
		DuplicatedGroup.GroupName = Group.GroupName;
		DuplicatedGroup.GroupSortPriority = Group.GroupSortPriority;
		for (UDEditorParameterValue* Parameter : Group.Parameters)
		{
			if (Parameter->ParameterInfo.Association != MaterialType
				|| Parameter->ParameterInfo.Index != Index)
			{
				DuplicatedGroup.Parameters.Add(Parameter);
			}
		}
		CleanedGroups.Add(DuplicatedGroup);
	}

	ParameterGroups = CleanedGroups;
	CopyToSourceInstance(true);
}

void UMaterialEditorInstanceConstant::ResetOverrides(int32 Index, EMaterialParameterAssociation MaterialType)
{
	check(GIsEditor);

	for (const FEditorParameterGroup& Group : ParameterGroups)
	{
		for (UDEditorParameterValue* Parameter : Group.Parameters)
		{
			if (Parameter->ParameterInfo.Association == MaterialType
				&& Parameter->ParameterInfo.Index == Index)
			{
				const EMaterialParameterType ParameterType = Parameter->GetParameterType();
				if (ParameterType != EMaterialParameterType::None)
				{
					FMaterialParameterMetadata SourceValue;
					bool bOverride = false;
					if (SourceInstance->GetParameterValue(ParameterType, Parameter->ParameterInfo, SourceValue, EMaterialGetParameterValueFlags::CheckInstanceOverrides))
					{
						bOverride = SourceValue.bOverride;
					}
					Parameter->bOverride = bOverride;
				}
			}
		}
	}
	CopyToSourceInstance(true);

}

void UMaterialEditorInstanceConstant::ClearInvalidParameterOverrides()
{
	const FMaterialCachedExpressionData* CachedExpressionData = Parent ? &Parent->GetCachedExpressionData() : nullptr;

	// Look for all Atlas Scalar parameters in each parameter group, then if a parameter has
	// an override, disable it unless the atlas texture matches that originally set in the parent Material. 
	for (int32 GroupIdx = 0; GroupIdx < ParameterGroups.Num(); GroupIdx++)
	{
		FEditorParameterGroup& Group = ParameterGroups[GroupIdx];
		for (int32 ParameterIdx = 0; ParameterIdx < Group.Parameters.Num(); ParameterIdx++)
		{
			UDEditorParameterValue* Parameter = Group.Parameters[ParameterIdx];
			if (UDEditorScalarParameterValue* ScalarParameter = Cast<UDEditorScalarParameterValue>(Parameter))
			{
				// Ignore parameters without override.
				if (!Parameter->bOverride)
				{
					continue;
				}

				// Get parent's parameter atlas texture. If identical to the one the editor parameter is using,
				// we can keep the override on (as the selected atlas curve will still make sense).
				FMaterialParameterMetadata Value;
				if (CachedExpressionData && CachedExpressionData->GetParameterValue(EMaterialParameterType::Scalar, ScalarParameter->ParameterInfo, Value)
					&& Value.ScalarAtlas == ScalarParameter->AtlasData.Atlas)
				{
					continue;
				}

				// The atlas texture of the newly bound material are different. Disable the override.
				Parameter->bOverride = false;
			}
		}
	}
}

TSoftObjectPtr<UObject> UMaterialEditorInstanceConstant::GetEnumerationObject(int32 Index) const
{
	if (EnumerationObjects.IsValidIndex(Index) && !EnumerationObjects[Index].IsNull())
	{
		return EnumerationObjects[Index];
	}
	if (UMaterialInstance* ParentInstance = Cast<UMaterialInstance>(Parent))
	{
		return ParentInstance->GetEnumerationObject(Index);
	}
	return TSoftObjectPtr<UObject>();
}

#endif

void UMaterialEditorInstanceConstant::CopyToSourceInstance(const bool bForceStaticPermutationUpdate)
{
	if (SourceInstance && !SourceInstance->IsTemplate(RF_ClassDefaultObject))
	{
		if (bIsFunctionPreviewMaterial)
		{
			bIsFunctionInstanceDirty = true;
		}
		else
		{
			SourceInstance->MarkPackageDirty();
		}

		{
			FMaterialInstanceParameterUpdateContext UpdateContext(SourceInstance, EMaterialInstanceClearParameterFlag::All);
			UpdateContext.SetBasePropertyOverrides(BasePropertyOverrides);
			UpdateContext.SetForceStaticPermutationUpdate(bForceStaticPermutationUpdate);

			for (int32 GroupIdx = 0; GroupIdx < ParameterGroups.Num(); GroupIdx++)
			{
				FEditorParameterGroup& Group = ParameterGroups[GroupIdx];
				for (int32 ParameterIdx = 0; ParameterIdx < Group.Parameters.Num(); ParameterIdx++)
				{
					UDEditorParameterValue* Parameter = Group.Parameters[ParameterIdx];
					if (Parameter && Parameter->bOverride)
					{
						FMaterialParameterMetadata EditorValue;
						if (Parameter->GetValue(EditorValue))
						{
							UpdateContext.SetParameterValueEditorOnly(Parameter->ParameterInfo, EditorValue, EMaterialSetParameterValueFlags::SetCurveAtlas);
						}
						else if (UDEditorMaterialLayersParameterValue* LayersParameter = Cast<UDEditorMaterialLayersParameterValue>(Parameter))
						{
							UpdateContext.SetMaterialLayers(LayersParameter->ParameterValue);
						}
					}
				}
			}
		}

		// Copy phys material back to source instance
		SourceInstance->PhysMaterial = PhysMaterial;
		SourceInstance->bOverridePhysMaterial = bOverridePhysMaterial;

		SourceInstance->NaniteOverrideMaterial.bEnableOverride = bNaniteOverride;
		SourceInstance->NaniteOverrideMaterial.OverrideMaterialEditor = NaniteOverrideMaterial;
		SourceInstance->EnumerationObjects = EnumerationObjects;

		// Copy the Lightmass settings...
		SourceInstance->SetOverrideCastShadowAsMasked(LightmassSettings.CastShadowAsMasked.bOverride);
		SourceInstance->SetCastShadowAsMasked(LightmassSettings.CastShadowAsMasked.ParameterValue);
		SourceInstance->SetOverrideEmissiveBoost(LightmassSettings.EmissiveBoost.bOverride);
		SourceInstance->SetEmissiveBoost(LightmassSettings.EmissiveBoost.ParameterValue);
		SourceInstance->SetOverrideDiffuseBoost(LightmassSettings.DiffuseBoost.bOverride);
		SourceInstance->SetDiffuseBoost(LightmassSettings.DiffuseBoost.ParameterValue);
		SourceInstance->SetOverrideExportResolutionScale(LightmassSettings.ExportResolutionScale.bOverride);
		SourceInstance->SetExportResolutionScale(LightmassSettings.ExportResolutionScale.ParameterValue);

		// Copy Refraction bias setting
		FMaterialParameterInfo RefractionInfo(TEXT("RefractionDepthBias"));
		SourceInstance->SetScalarParameterValueEditorOnly(RefractionInfo, RefractionDepthBias);

		// Copy UserSceneTextureOverrides
		SourceInstance->UserSceneTextureOverrides.Empty();
		for (FEditorUserSceneTextureOverride& Override : PostProcessOverrides.UserSceneTextureInputs)
		{
			if (Override.Value != NAME_None && Override.Value != Override.Key)
			{
				SourceInstance->UserSceneTextureOverrides.Add({ Override.Key, Override.Value });
			}
		}
		if (PostProcessOverrides.UserSceneTextureOutput != NAME_None)
		{
			// UserSceneTextureOutput override uses key of NAME_None
			SourceInstance->UserSceneTextureOverrides.Add({ NAME_None, PostProcessOverrides.UserSceneTextureOutput });
		}

		// Copy other post process overrides (BlendableLocation / BlendablePriority) -- BL_ReplacingTonemapper is disallowed on overrides
		SourceInstance->bOverrideBlendableLocation = PostProcessOverrides.bOverrideBlendableLocation && PostProcessOverrides.BlendableLocationOverride != BL_ReplacingTonemapper;
		SourceInstance->bOverrideBlendablePriority = PostProcessOverrides.bOverrideBlendablePriority;
		if (SourceInstance->bOverrideBlendableLocation)
		{
			SourceInstance->BlendableLocationOverride = PostProcessOverrides.BlendableLocationOverride;
		}
		else
		{
			SourceInstance->BlendableLocationOverride = Parent ? Parent->GetMaterial()->BlendableLocation : TEnumAsByte<EBlendableLocation>(BL_SceneColorAfterTonemapping);
		}
		if (SourceInstance->bOverrideBlendablePriority)
		{
			SourceInstance->BlendablePriorityOverride = PostProcessOverrides.BlendablePriorityOverride;
		}
		else
		{
			SourceInstance->BlendablePriorityOverride = Parent ? Parent->GetMaterial()->BlendablePriority : 0;
		}

		SourceInstance->bOverrideSubsurfaceProfile = bOverrideSubsurfaceProfile;
		SourceInstance->SubsurfaceProfile = SubsurfaceProfile;

		SourceInstance->bOverrideSpecularProfile = bOverrideSpecularProfile;
		SourceInstance->SpecularProfileOverride = SpecularProfile;

		SourceInstance->bOverrideToonProfile = bOverrideToonProfile;
		SourceInstance->ToonProfileOverride = ToonProfile;

		// Update object references and parameter names.
		SourceInstance->UpdateParameterNames();
		VisibleExpressions.Empty();
		
		// force refresh of visibility of properties
		if (Parent)
		{
			UMaterial* ParentMaterial = Parent->GetMaterial();
			IMaterialEditorModule* MaterialEditorModule = &FModuleManager::LoadModuleChecked<IMaterialEditorModule>("MaterialEditor");
			MaterialEditorModule->GetVisibleMaterialParameters(ParentMaterial, SourceInstance, VisibleExpressions);
		}
	}
}

void UMaterialEditorInstanceConstant::ApplySourceFunctionChanges()
{
	if (bIsFunctionPreviewMaterial && bIsFunctionInstanceDirty)
	{
		CopyToSourceInstance();

		// Copy updated function parameter values	
		SourceFunction->ScalarParameterValues = SourceInstance->ScalarParameterValues;
		SourceFunction->VectorParameterValues = SourceInstance->VectorParameterValues;
		SourceFunction->DoubleVectorParameterValues = SourceInstance->DoubleVectorParameterValues;
		SourceFunction->TextureParameterValues = SourceInstance->TextureParameterValues;
		SourceFunction->TextureCollectionParameterValues = SourceInstance->TextureCollectionParameterValues;
		SourceFunction->RuntimeVirtualTextureParameterValues = SourceInstance->RuntimeVirtualTextureParameterValues;
		SourceFunction->SparseVolumeTextureParameterValues = SourceInstance->SparseVolumeTextureParameterValues;
		SourceFunction->FontParameterValues = SourceInstance->FontParameterValues;

		SourceFunction->StaticSwitchParameterValues = SourceInstance->GetStaticParameters().StaticSwitchParameters;
		const FStaticParameterSetEditorOnlyData& StaticParameters = SourceInstance->GetEditorOnlyStaticParameters();
		SourceFunction->StaticComponentMaskParameterValues = StaticParameters.StaticComponentMaskParameters;

		SourceFunction->MarkPackageDirty();
		bIsFunctionInstanceDirty = false;

		UMaterialEditingLibrary::UpdateMaterialFunction(SourceFunction, nullptr);
	}
}

void UMaterialEditorInstanceConstant::SetSourceInstance(UMaterialInstanceConstant* MaterialInterface)
{
	check(MaterialInterface);
	SourceInstance = MaterialInterface;

	PreviewInstance = NewObject<UMaterialInstanceConstant>();
	PreviewInstance->SetParentEditorOnly(MaterialInterface);
	PreviewInstance->BasePropertyOverrides.bOverride_UsageFlags = (1u << MATUSAGE_StaticMesh);
	PreviewInstance->BasePropertyOverrides.UsageFlags = (1u << MATUSAGE_StaticMesh);

	Parent = SourceInstance->Parent;

	bNaniteOverride = SourceInstance->NaniteOverrideMaterial.bEnableOverride;
	NaniteOverrideMaterial = SourceInstance->NaniteOverrideMaterial.OverrideMaterialEditor;
	EnumerationObjects = SourceInstance->EnumerationObjects;

	CopyBasePropertiesFromParent();

	RegenerateArrays();

	//propagate changes to the base material so the instance will be updated if it has a static permutation resource
	FMaterialInstanceParameterUpdateContext UpdateContext(SourceInstance, EMaterialInstanceClearParameterFlag::Static);

	for (int32 GroupIdx = 0; GroupIdx < ParameterGroups.Num(); GroupIdx++)
	{
		FEditorParameterGroup& Group = ParameterGroups[GroupIdx];
		for (int32 ParameterIdx = 0; ParameterIdx < Group.Parameters.Num(); ParameterIdx++)
		{
			UDEditorParameterValue* Parameter = Group.Parameters[ParameterIdx];
			if (Parameter && Parameter->bOverride)
			{
				FMaterialParameterMetadata EditorValue;
				if (Parameter->GetValue(EditorValue))
				{
					// Only want to update static parameters here
					if (IsStaticMaterialParameter(EditorValue.Value.Type))
					{
						UpdateContext.SetParameterValueEditorOnly(Parameter->ParameterInfo, EditorValue);
					}
				}
				else if (UDEditorMaterialLayersParameterValue* LayersParameter = Cast<UDEditorMaterialLayersParameterValue>(Parameter))
				{
					UpdateContext.SetMaterialLayers(LayersParameter->ParameterValue);
				}
			}
		}
	}
}

void UMaterialEditorInstanceConstant::SetSourceFunction(UMaterialFunctionInstance* MaterialFunction)
{
	SourceFunction = MaterialFunction;
	bIsFunctionPreviewMaterial = !!(SourceFunction);
}

void UMaterialEditorInstanceConstant::UpdateSourceInstanceParent()
{
	// If the parent was changed to the source instance, set it to NULL
	if( Parent == SourceInstance )
	{
		Parent = NULL;
	}

	SourceInstance->SetParentEditorOnly( Parent );
	SourceInstance->PostEditChange();
}


void UMaterialEditorInstanceConstant::CopyBasePropertiesFromParent()
{
	BasePropertyOverrides = SourceInstance->BasePropertyOverrides;
	// Copy the overrides (if not yet overridden), so they match their true values in the UI
	if (!BasePropertyOverrides.bOverride_OpacityMaskClipValue)
	{
		BasePropertyOverrides.OpacityMaskClipValue = SourceInstance->GetOpacityMaskClipValue();
	}
	if (!BasePropertyOverrides.bOverride_BlendMode)
	{
		BasePropertyOverrides.BlendMode = SourceInstance->GetBlendMode();
	}
	if (!BasePropertyOverrides.bOverride_ShadingModel)
	{
		if (SourceInstance->IsShadingModelFromMaterialExpression())
		{
			BasePropertyOverrides.ShadingModel = MSM_FromMaterialExpression;
		}
		else
		{
			BasePropertyOverrides.ShadingModel = SourceInstance->GetShadingModels().GetFirstShadingModel(); 
		}
	}
	if (!BasePropertyOverrides.bOverride_TwoSided)
	{
		BasePropertyOverrides.TwoSided = SourceInstance->IsTwoSided();
	}
	if (!BasePropertyOverrides.bOverride_bIsThinSurface)
	{
		BasePropertyOverrides.bIsThinSurface = SourceInstance->IsThinSurface();
	}
	if (!BasePropertyOverrides.bOverride_OutputTranslucentVelocity)
	{
		BasePropertyOverrides.bOutputTranslucentVelocity = SourceInstance->IsTranslucencyWritingVelocity();
	}
	if (!BasePropertyOverrides.bOverride_bHasPixelAnimation)
	{
		BasePropertyOverrides.bHasPixelAnimation = SourceInstance->HasPixelAnimation();
	}
	if (!BasePropertyOverrides.bOverride_bEnableTessellation)
	{
		BasePropertyOverrides.bEnableTessellation = SourceInstance->IsTessellationEnabled();
	}
	if (!BasePropertyOverrides.DitheredLODTransition)
	{
		BasePropertyOverrides.DitheredLODTransition = SourceInstance->IsDitheredLODTransition();
	}
	if (!BasePropertyOverrides.bOverride_DisplacementScaling)
	{
		BasePropertyOverrides.DisplacementScaling = SourceInstance->GetDisplacementScaling();
	}
	if (!BasePropertyOverrides.bOverride_bEnableDisplacementFade)
	{
		BasePropertyOverrides.bEnableDisplacementFade = SourceInstance->IsDisplacementFadeEnabled();
	}
	if (!BasePropertyOverrides.bOverride_DisplacementFadeRange)
	{
		BasePropertyOverrides.DisplacementFadeRange = SourceInstance->GetDisplacementFadeRange();
	}
	if (!BasePropertyOverrides.bOverride_MaxWorldPositionOffsetDisplacement)
	{
		BasePropertyOverrides.MaxWorldPositionOffsetDisplacement = SourceInstance->GetMaxWorldPositionOffsetDisplacement();
	}
	if (!BasePropertyOverrides.bOverride_CastDynamicShadowAsMasked)
	{
		BasePropertyOverrides.bCastDynamicShadowAsMasked = SourceInstance->GetCastDynamicShadowAsMasked();
	}
	if (!BasePropertyOverrides.bOverride_CompatibleWithLumenCardSharing)
	{
		BasePropertyOverrides.bCompatibleWithLumenCardSharing = SourceInstance->IsCompatibleWithLumenCardSharing();
	}

	bOverridePhysMaterial = SourceInstance->bOverridePhysMaterial;
	// If overriden, always keep the assigned value, even if None
	PhysMaterial = bOverridePhysMaterial ? SourceInstance->PhysMaterial : TObjectPtr<UPhysicalMaterial>(SourceInstance->GetPhysicalMaterial());

	// Usage flags and overrides are a bitmask.	
	BasePropertyOverrides.UsageFlags = (BasePropertyOverrides.bOverride_UsageFlags & BasePropertyOverrides.UsageFlags) | (~BasePropertyOverrides.bOverride_UsageFlags & SourceInstance->GetUsageFlags());

	// Copy the Lightmass settings...
	// The lightmass functions (GetCastShadowAsMasked, etc.) check if the value is overridden and returns the current value if so, otherwise returns the parent value
	// So we don't need to wrap these in the same "if not overriding" as above
	LightmassSettings.CastShadowAsMasked.ParameterValue = SourceInstance->GetCastShadowAsMasked();
	LightmassSettings.EmissiveBoost.ParameterValue = SourceInstance->GetEmissiveBoost();
	LightmassSettings.DiffuseBoost.ParameterValue = SourceInstance->GetDiffuseBoost();
	LightmassSettings.ExportResolutionScale.ParameterValue = SourceInstance->GetExportResolutionScale();

	//Copy refraction settings
	SourceInstance->GetRefractionSettings(RefractionDepthBias);

	bOverrideSubsurfaceProfile = SourceInstance->bOverrideSubsurfaceProfile;
	// Copy the subsurface profile. GetSubsurfaceProfile_Internal() will return either the overridden profile or one from a parent
	SubsurfaceProfile = SourceInstance->GetSubsurfaceProfile_Internal();

	if (Substrate::IsSubstrateEnabled())
	{
		bOverrideSpecularProfile = SourceInstance->bOverrideSpecularProfile;
		if (SourceInstance->NumSpecularProfile_Internal() > 0)
		{
			SpecularProfile = SourceInstance->GetSpecularProfileOverride_Internal();
		}

		bOverrideToonProfile = SourceInstance->bOverrideToonProfile;
		if (SourceInstance->NumToonProfile_Internal() > 0)
		{
			ToonProfile = SourceInstance->GetToonProfileOverride_Internal();
		}
	}

	// Post process blendable location and priority overrides
	PostProcessOverrides.bOverrideBlendableLocation = SourceInstance->bOverrideBlendableLocation;
	if (SourceInstance->bOverrideBlendableLocation)
	{
		PostProcessOverrides.BlendableLocationOverride = SourceInstance->BlendableLocationOverride;
	}
	else
	{
		PostProcessOverrides.BlendableLocationOverride = Parent ? Parent->GetMaterial()->BlendableLocation : TEnumAsByte<EBlendableLocation>(BL_SceneColorAfterTonemapping);
	}

	PostProcessOverrides.bOverrideBlendablePriority = SourceInstance->bOverrideBlendablePriority;
	if (SourceInstance->bOverrideBlendablePriority)
	{
		PostProcessOverrides.BlendablePriorityOverride = SourceInstance->BlendablePriorityOverride;
	}
	else
	{
		PostProcessOverrides.BlendablePriorityOverride = Parent ? Parent->GetMaterial()->BlendablePriority : 0;
	}

	// UserSceneTextureOutput override uses Key == NAME_None.  UserSceneTextureInputs are initialized in RegenerateArrays, as those are affected
	// by graph reachability.
	for (const FUserSceneTextureOverride& Override : SourceInstance->UserSceneTextureOverrides)
	{
		if (Override.Key == NAME_None)
		{
			PostProcessOverrides.UserSceneTextureOutput = Override.Value;
			break;
		}
	}
}

#if WITH_EDITOR
void UMaterialEditorInstanceConstant::PostEditUndo()
{
	Super::PostEditUndo();

	if (bIsFunctionPreviewMaterial && SourceFunction)
	{
		bIsFunctionInstanceDirty = true;
		ApplySourceFunctionChanges();
	}
	else if (SourceInstance)
	{
		SourceInstance->PostEditUndo();

		FMaterialUpdateContext Context;

		UpdateSourceInstanceParent();

		Context.AddMaterialInstance(SourceInstance);
	}
}
#endif

UMaterialEditorMeshComponent::UMaterialEditorMeshComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}
