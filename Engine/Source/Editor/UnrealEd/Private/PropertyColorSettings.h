// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "PropertyColorSettings.generated.h"

USTRUCT()
struct FPropertyColorCustomProperty
{
	GENERATED_BODY()

	UPROPERTY(Config)
	FName Name;

	UPROPERTY(Config)
	FText Text;

	UPROPERTY(Config)
	FText TextToolTip;

	UPROPERTY(Config)
	FString PropertyChain;

	UPROPERTY(Config)
	FString PropertyValue;

	UPROPERTY(Config)
	FColor PropertyColor = FColor::Red;

	UPROPERTY(Config)
	FColor DefaultColor = FColor::White;
};

/**
* Implements the settings for Property Color.
*/
UCLASS(MinimalAPI, config = EditorSettings)
class UPropertyColorSettings : public UObject
{
	GENERATED_BODY()

public:
	UPropertyColorSettings()
	{}

	UPROPERTY(Config)
	TArray<FPropertyColorCustomProperty> CustomProperties;
};
