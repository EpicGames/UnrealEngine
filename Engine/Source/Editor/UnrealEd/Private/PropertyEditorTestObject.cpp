// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/PropertyEditorTestObject.h"

#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/SListView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PropertyEditorTestObject)


TArray<FString> APropertyEditorTestActor::GetOptionsFunc() const
{
	return TArray<FString> { TEXT("One"), TEXT("Two"), TEXT("Three") };
}

TSharedRef<SWidget> UPropertyEditorRowGeneratorTest::GenerateWidget()
{
	FPropertyRowGeneratorArgs GeneratorArgs;
	GeneratorArgs.bAllowMultipleTopLevelObjects = true;

	FPropertyEditorModule& Module = FModuleManager::LoadModuleChecked<FPropertyEditorModule>( "PropertyEditor" );
	PropertyRowGenerator = Module.CreatePropertyRowGenerator(GeneratorArgs);
	PropertyRowGenerator->OnRowsRefreshed().AddUObject(this, &UPropertyEditorRowGeneratorTest::OnRowsRefreshed);

	TArray<UObject*> Objects;
	for (int32 Idx = 0; Idx < 10; ++Idx)
	{
		Objects.Add(NewObject<UPropertyEditorTestObject>());
	}

	ListView = SNew(SListView<TSharedPtr<IDetailTreeNode>>)
		.ListItemsSource(&DetailsNodes)
		.OnGenerateRow_UObject(this, &UPropertyEditorRowGeneratorTest::GenerateListRow);

	PropertyRowGenerator->SetObjects(Objects);

	return ListView.ToSharedRef();
}

static void AddChildrenRecursive(TSharedPtr<IDetailTreeNode> Node, TArray<TSharedPtr<IDetailTreeNode>>& OutNodes)
{
	TArray<TSharedRef<IDetailTreeNode>> Children;
	Node->GetChildren(Children);
			
	for (const TSharedRef<IDetailTreeNode>& Child : Children)
	{
		OutNodes.Add(Child);

		AddChildrenRecursive(Child, OutNodes);
	}
}

void UPropertyEditorRowGeneratorTest::OnRowsRefreshed()
{
	DetailsNodes.Reset();

	for (const TSharedRef<IDetailTreeNode>& RootNode : PropertyRowGenerator->GetRootTreeNodes())
	{
		AddChildrenRecursive(RootNode, DetailsNodes);
	}

	ListView->RebuildList();
}

TSharedRef<ITableRow> UPropertyEditorRowGeneratorTest::GenerateListRow(TSharedPtr<IDetailTreeNode> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{ 
	TSharedRef<STableRow<TSharedPtr<IDetailTreeNode>>> TableRow = SNew(STableRow<TSharedPtr<IDetailTreeNode>>, OwnerTable);

	FNodeWidgets NodeWidgets = InItem->CreateNodeWidgets();
	if (NodeWidgets.WholeRowWidget.IsValid())
	{
		TableRow->SetContent(NodeWidgets.WholeRowWidget.ToSharedRef());
	}
	else if (NodeWidgets.NameWidget.IsValid() && NodeWidgets.ValueWidget.IsValid())
	{
		TableRow->SetContent(
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				NodeWidgets.NameWidget.ToSharedRef()
			]
			+ SHorizontalBox::Slot()
			[
				NodeWidgets.ValueWidget.ToSharedRef()
			]
		);
	}

	return TableRow;
}
