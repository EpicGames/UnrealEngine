// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyInfoViewStyle.h"

#include "Framework/Application/SlateApplication.h"
#include "GenericPlatform/GenericApplication.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Geometry.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateBrush.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/ITableRow.h"

class FPaintArgs;
class FSlateRect;
class FWidgetStyle;

namespace PropertyInfoViewStyle
{
	FSlateColor GetIndentBackgroundColor(int32 IndentLevel, bool IsHovered)
	{
		if (IsHovered)
		{
			return FAppStyle::Get().GetSlateColor("Colors.Header");
		}

		if (IndentLevel == 0)
		{
			return FAppStyle::Get().GetSlateColor("Colors.Panel");
		}

		int32 ColorIndex = 0;
		int32 Increment = 1;

		for (int IndentCount = 0; IndentCount < IndentLevel; ++IndentCount)
		{
			ColorIndex += Increment;

			if (ColorIndex == 0 || ColorIndex == 3)
			{
				Increment = -Increment;
			}
		}

		static const uint8 ColorOffsets[] =
		{
			2, 6, 12, 20
		};

		FColor BaseColor = FAppStyle::Get().GetSlateColor("Colors.Panel").GetSpecifiedColor().ToFColor(true);

		FColor ColorWithOffset(
		BaseColor.R + ColorOffsets[ColorIndex], 
		BaseColor.G + ColorOffsets[ColorIndex], 
		BaseColor.B + ColorOffsets[ColorIndex]);

		return FSlateColor(FLinearColor::FromSRGBColor(ColorWithOffset));
	}

	FSlateColor GetRowBackgroundColor(ITableRow* Row)
	{
		check(Row);
		return GetIndentBackgroundColor(Row->GetIndentLevel(), Row->AsWidget()->IsHovered());
	}

	int32 SIndent::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry,
							const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements,
							int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
	{
		TSharedPtr<ITableRow> RowPtr = Row.Pin();
		if (!RowPtr.IsValid())
		{
			return LayerId;
		}

		const FSlateBrush* BackgroundBrush = FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle");
		const FSlateBrush* DropShadowBrush = FAppStyle::Get().GetBrush("DetailsView.ArrayDropShadow");

		int32 IndentLevel = RowPtr->GetIndentLevel();
		for (int32 IndentCount = 0; IndentCount < IndentLevel; ++IndentCount)
		{
			FSlateColor BackgroundColor = GetRowBackgroundColor(IndentCount);

			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(FVector2f(TabSize, AllottedGeometry.GetLocalSize().Y), FSlateLayoutTransform(FVector2f(TabSize * IndentCount, 0.f))),
				BackgroundBrush,
				ESlateDrawEffect::None,
				BackgroundColor.GetColor(InWidgetStyle)
			);

			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId + 1,
				AllottedGeometry.ToPaintGeometry(FVector2f(TabSize, AllottedGeometry.GetLocalSize().Y), FSlateLayoutTransform(FVector2f(TabSize * IndentCount, 0.f))),
				DropShadowBrush
			);
		}
			
		return LayerId + 1;
	}


	void SIndent::Construct(const FArguments& InArgs, TSharedRef<ITableRow> DetailsRow)
	{
		Row = DetailsRow;

		ChildSlot
		[
			SNew(SBox)
				.WidthOverride(this, &SIndent::GetIndentWidth)
		];
	}

	FOptionalSize SIndent::GetIndentWidth() const
	{
		int32 IndentLevel = 0;

		TSharedPtr<ITableRow> RowPtr = Row.Pin();
		if (RowPtr.IsValid())
		{
			IndentLevel = RowPtr->GetIndentLevel();
		}
		return IndentLevel * TabSize;
	}

	FSlateColor SIndent::GetRowBackgroundColor(int32 IndentLevel) const
	{
		const bool IsHovered = Row.IsValid() && Row.Pin()->AsWidget()->IsHovered();
			
		return PropertyInfoViewStyle::GetIndentBackgroundColor(IndentLevel, IsHovered);
	}

	void SExpanderArrow::Construct(const FArguments& InArgs, TSharedRef<ITableRow> DetailsRow)
	{
		Row = DetailsRow;

		HasChildren = InArgs._HasChildren;

		ChildSlot
		[
			SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle"))
						.BorderBackgroundColor_Static(
					             PropertyInfoViewStyle::GetRowBackgroundColor,
					             Row.Pin().Get()
				             )
					[
						SNew(SBox)
							.WidthOverride(20.0f)
							.HeightOverride(16.0f)
					]
				]
				+ SOverlay::Slot()
				[
					SAssignNew(ExpanderArrow, SButton)
						.ButtonStyle(FCoreStyle::Get(), "NoBorder")
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						.ClickMethod(EButtonClickMethod::MouseDown)
						.OnClicked(this, &SExpanderArrow::OnExpanderClicked)
						.ContentPadding(FMargin(0.0f))
						.IsFocusable(false)
					[
						SNew(SImage)
							.Image(this, &SExpanderArrow::GetExpanderImage)
							.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
		];
	}

	EVisibility SExpanderArrow::GetExpanderVisibility() const
	{
		TSharedPtr<ITableRow> RowPtr = Row.Pin();
		if (!RowPtr.IsValid())
		{
			return EVisibility::Collapsed;
		}

		return RowPtr->DoesItemHaveChildren() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	const FSlateBrush* SExpanderArrow::GetExpanderImage() const
	{
		TSharedPtr<ITableRow> RowPtr = Row.Pin();
		if (!RowPtr.IsValid() || !HasChildren.Get())
		{
			return FAppStyle::Get().GetBrush("NoBrush");
		}

		const bool bIsItemExpanded = RowPtr->IsItemExpanded();

		FName ResourceName;
		if (bIsItemExpanded)
		{
			if (ExpanderArrow->IsHovered())
			{
				ResourceName = TEXT("TreeArrow_Expanded_Hovered");
			}
			else
			{
				ResourceName = TEXT("TreeArrow_Expanded");
			}
		}
		else
		{
			if (ExpanderArrow->IsHovered())
			{
				ResourceName = TEXT("TreeArrow_Collapsed_Hovered");
			}
			else
			{
				ResourceName = TEXT("TreeArrow_Collapsed");
			}
		}

		return FAppStyle::Get().GetBrush(ResourceName);
	}

	FReply SExpanderArrow::OnExpanderClicked() const
	{
		TSharedPtr<ITableRow> RowPtr = Row.Pin();
		if (!RowPtr.IsValid())
		{
			return FReply::Unhandled();
		}

		// Recurse the expansion if "shift" is being pressed
		const FModifierKeysState ModKeyState = FSlateApplication::Get().GetModifierKeys();
		if (ModKeyState.IsShiftDown())
		{
			RowPtr->Private_OnExpanderArrowShiftClicked();
		}
		else
		{
			RowPtr->ToggleExpansion();
		}

		return FReply::Handled();
	}

	void STextHighlightOverlay::Construct(const FArguments& InArgs)
	{
		static FSlateBrush HighlightShape = FSlateBrush(*FAppStyle::Get().GetBrush("TextBlock.HighlightShape"));
		HighlightShape.TintColor = FLinearColor(0.f, 0.47f, 1.f, .3f);
		
		// uses overlay to create the highlight so that the text widget can be a button,
		// hyperlink, textblock, etc
		ChildSlot
		[
			SNew(SOverlay)
				+SOverlay::Slot()
				[
					SNew(STextBlock)
						.Text(InArgs._FullText)
						.ColorAndOpacity(FLinearColor::Transparent)
						.HighlightShape(&HighlightShape)
						.HighlightText(InArgs._HighlightText)
				]
				+SOverlay::Slot()
				[
					InArgs._Content.Widget
				]
		];
	}
}
