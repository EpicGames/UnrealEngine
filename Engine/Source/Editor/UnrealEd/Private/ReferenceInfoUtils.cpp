// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReferenceInfoUtils.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/OutputDeviceFile.h"
#include "HAL/IConsoleManager.h"
#include "Model.h"
#include "Modules/ModuleManager.h"
#include "Serialization/ArchiveUObject.h"
#include "UObject/Class.h"
#include "UObject/UObjectIterator.h"
#include "UObject/Package.h"
#include "Engine/Level.h"
#include "GameFramework/Actor.h"
#include "Materials/MaterialInterface.h"
#include "AssetRegistry/AssetData.h"
#include "Editor/UnrealEdEngine.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Editor.h"
#include "UnrealEdGlobals.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Engine/Selection.h"

void ExecuteReferenceInfo(const TArray<FString>& Args, UWorld* InWorld )
{
	bool bShowDefault = true;
	bool bShowScript = true;
	int32 Depth = 0;

	for (int32 ArgIdx = 0; ArgIdx < Args.Num(); ++ArgIdx)
	{
		if (FCString::Stristr(*Args[ArgIdx], TEXT("nodefault")))
		{
			bShowDefault = false;
		}

		if (FCString::Stristr(*Args[ArgIdx], TEXT("noscript")))
		{
			bShowScript = false;
		}

		FParse::Value(*Args[ArgIdx], TEXT("DEPTH="), Depth);
	}

	ReferenceInfoUtils::GenerateOutput(InWorld, Depth, bShowDefault, bShowScript);
}

static FAutoConsoleCommandWithWorldAndArgs ActorReferenceInfoCVar(
	TEXT("ReferenceInfo"),
	TEXT("Outputs reference info for selected actors to a log file. Syntax is: ReferenceInfo [-depth=<depth value>] [-nodefault] [-noscript]"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(ExecuteReferenceInfo)
	);

namespace ReferenceInfoUtils
{
	typedef TMap< TObjectPtr<UObject>, TArray<TObjectPtr<UObject>> > ObjectReferenceGraph;
	typedef TMap<UObject*, int32> ReferenceTreeMap;
	typedef TMap<UObject*, FString> ObjectNameMap;
	
	/**
	 * Data container to hold information about what is referencing a given set of assets.
	 */
	struct FReferencedAssets
	{
		/** The object that holding a reference to the set of assets */
		UObject* Referencer;

		/** The set of assets that are being referenced */
		TArray<UObject*> AssetList;

		/** Default ctor */
		FReferencedAssets()
			: Referencer(NULL)
		{
		}

		/** Sets the name of the referencer */
		FReferencedAssets(UObject* InReferencer)
			: Referencer(InReferencer)
		{
		}

		/** Serializer **/
		friend FArchive& operator<<(FArchive& Ar, FReferencedAssets& Asset)
		{
			return Ar << Asset.Referencer << Asset.AssetList;
		}
	};

	/**
	 * This archive searches objects for assets. It determines the set of assets by whether they support thumbnails or not. Possibly, not best but it displays everything as thumbnails
	 */
	class FFindAssetsArchive : public FArchiveUObject
	{
		/** The root object that was used to being serialization for this archive */
		UObject* StartObject;

		/** The object currently being serialized */
		UObject* CurrentObject;

		/** The array to add any found assets too */
		TArray<UObject*>& AssetList;

		/** Set when the global asset list is updated.  Used to prevent the reference graph from being polluted by calls to the public version of BuildAssetList */
		ObjectReferenceGraph* CurrentReferenceGraph;

		/** If false, ignore all assets referenced only through script */
		bool bIncludeScriptRefs;

		/** If false, ignore all assets referenced only through archetype/class default objects */
		bool bIncludeDefaultRefs;

		/** Maximum depth to recursively serialize objects; 0 indicates no limit to recursion */
		const int32 MaxRecursionDepth;

		/** Current recursion depth */
		int32 CurrentDepth;
		
		/**
		 * Manually serializes the class and archetype for the specified object so that assets which are referenced through the object's class/archetype can be differentiated
		 */
		void HandleReferencedObject(UObject* Obj)
		{
			if (CurrentReferenceGraph != NULL)
			{
				// Here we allow recursion if the current depth is less-than-equal (as opposed to less-than) because the archetype and class are treated as transparent objects
				// Serialization of the class and object are controlled by the "show class refs" and "show default refs" buttons
				if (MaxRecursionDepth == 0 || CurrentDepth < MaxRecursionDepth)
				{
					// Now change the current reference list to the one for this object
					if (bIncludeDefaultRefs == true)
					{
						auto* ReferencedAssets = GetAssetList(Obj);

						// See the comment for the bIncludeScriptRefs block
						UObject* ObjectArc = Obj->GetArchetype();
						ReferencedAssets->AddUnique(ObjectArc);

						UObject* PreviousObject = CurrentObject;
						CurrentObject = ObjectArc;

						if (ObjectArc->HasAnyMarks(OBJECTMARK_TagExp))
						{
							// Temporarily disable serialization of the class, as we need to specially handle that as well
							bool bSkipClassSerialization = ArIgnoreClassRef;
							ArIgnoreClassRef = true;

							ObjectArc->UnMark(OBJECTMARK_TagExp);
							ObjectArc->Serialize(*this);

							ArIgnoreClassRef = bSkipClassSerialization;
						}

						CurrentObject = PreviousObject;
					}

					if (bIncludeScriptRefs == true)
					{
						auto* ReferencedAssets = GetAssetList(Obj);

						// We want to see assets referenced by this object's class, but classes don't have associated thumbnail rendering info
						// So we'll need to serialize the class manually in order to get the object references encountered through the class to fall
						// under the appropriate tree item

						// Serializing the class will result in serializing the class default object; but we need to do this manually (for the same reason
						// that we do it for the class), so temporarily prevent the CDO from being serialized by this archive
						UClass* ObjectClass = Obj->GetClass();
						ReferencedAssets->AddUnique(ObjectClass);

						UObject* PreviousObject = CurrentObject;
						CurrentObject = ObjectClass;

						if (ObjectClass->HasAnyMarks(OBJECTMARK_TagExp))
						{
							ObjectClass->UnMark(OBJECTMARK_TagExp);
							ObjectClass->Serialize(*this);
						}

						CurrentObject = PreviousObject;
					}
				}
			}
		}

		/**
		 * Retrieves the referenced assets list for the specified object
		 */
		TArray<TObjectPtr<UObject>>* GetAssetList(UObject* Referencer)
		{
			check(Referencer);

			auto* ReferencedAssetList = CurrentReferenceGraph->Find(ObjectPtrWrap(Referencer));
			if (ReferencedAssetList == NULL)
			{
				// add a new entry for the specified object
				ReferencedAssetList = &CurrentReferenceGraph->Add(ObjectPtrWrap(Referencer), TArray<TObjectPtr<UObject>>{});
			}

			return ReferencedAssetList;
		}

	public:
		/**
		 * Functor that starts the serialization process
		 */
		FFindAssetsArchive(UObject* Search, TArray<UObject*>& OutAssetList, ObjectReferenceGraph* ReferenceGraph = NULL, int32 MaxRecursion = 0, bool bIncludeClasses = true, bool bIncludeDefaults = false) 
			: StartObject(Search)
			, AssetList(OutAssetList)
			, CurrentReferenceGraph(ReferenceGraph)
			, bIncludeScriptRefs(bIncludeClasses)
			, bIncludeDefaultRefs(bIncludeDefaults)
			, MaxRecursionDepth(MaxRecursion)
			, CurrentDepth(0)
		{
			ArIsObjectReferenceCollector = true;
			ArIgnoreClassRef = !bIncludeScriptRefs;

			CurrentObject = StartObject;

			*this << StartObject;
		}

		/**
		 * Adds the object reference to the asset list if it supports thumbnails
		 * Recursively searches through its references for more assets
		 */
		FArchive& operator<<(UObject*& Obj)
		{
			// Don't check null references or objects already visited
			if (Obj != NULL && Obj->HasAnyMarks(OBJECTMARK_TagExp) &&

				// Ff we wish to filter out assets referenced through script, we need to ignore all class objects, not just the UObject::Class reference
				(!ArIgnoreClassRef || (Cast<UClass>(Obj) == NULL)))
			{
				// Clear the search flag so we don't revisit objects
				Obj->UnMark(OBJECTMARK_TagExp);
				if (Obj->IsA(UField::StaticClass()))
				{
					// Skip all of the other stuff because the serialization of UFields will quickly overflow our stack given the number of temporary variables we create in the below code
					Obj->Serialize(*this);
				}
				else
				{
					// Only report this object reference if it supports thumbnail display this eliminates all of the random objects like functions, properties, etc.
					const bool bCDO = Obj->HasAnyFlags(RF_ClassDefaultObject);
					const bool bIsContent = GUnrealEd->GetThumbnailManager()->GetRenderingInfo(Obj) != NULL;
					const bool bIncludeAnyway = (Obj->GetOuter() == CurrentObject) && (Cast<UClass>(CurrentObject) == NULL);
					const bool bShouldReportAsset = !bCDO && (bIsContent || bIncludeAnyway);

					// Remember which object we were serializing
					UObject* PreviousObject = CurrentObject;
					if (bShouldReportAsset)
					{
						CurrentObject = Obj;

						// Add this object to the list to display
						AssetList.Add(CurrentObject);
						if (CurrentReferenceGraph != NULL)
						{
							auto* CurrentObjectAssets = GetAssetList(PreviousObject);
							check(CurrentObjectAssets);

							// Add this object to the list of objects referenced by the object currently being serialized
							CurrentObjectAssets->Add(CurrentObject);	
							HandleReferencedObject(CurrentObject);
						}
					}
					else if (Obj == StartObject)
					{
						HandleReferencedObject(Obj);
					}

					if (MaxRecursionDepth == 0 || CurrentDepth < MaxRecursionDepth)
					{
						CurrentDepth++;

						// Now recursively search this object for more references
						Obj->Serialize(*this);

						CurrentDepth--;
					}

					// Restore the previous object that was being serialized
					CurrentObject = PreviousObject;
				}
			}

			return *this;
		}
	};

	/** This is a list of classes that should be ignored when building the asset list as they are always loaded and therefore not pertinent */
	TArray<UClass*> IgnoreClasses;

	/** This is a list of packages that should be ignored when building the asset list as they are always loaded and therefore not pertinent */
	TArray<UObject*> IgnorePackages;

	/** Holds the list of assets that are being referenced by the current selection */
	TArray<FReferencedAssets> Referencers;

	/** The object graph for the assets referenced by the currently selected actors */
	ObjectReferenceGraph ReferenceGraph;
	
	/** Caches the names of the objects referenced by the currently selected actors */
	ObjectNameMap ObjectNameCache;
	
	/**
	 * Checks an object to see if it should be included for asset searching
	 */
	bool ShouldSearchForAssets(const UObject* Object, const TArray<UClass*>& ClassesToIgnore, const TArray<UObject*>& PackagesToIgnore, bool bIncludeDefaults = false)
	{
		bool bShouldSearch = true;

		// Package name transition
		if (Object->HasAnyFlags(RF_ClassDefaultObject) && (Object->GetOutermost()->GetFName() == NAME_CoreUObject || Object->GetOutermost()->GetFName() == GLongCoreUObjectPackageName))
		{
			// Ignore all class default objects for classes which are declared in Core
			bShouldSearch = false;
		}

		// Check to see if we should ignore a class
		for (int32 Index = 0; Index < ClassesToIgnore.Num(); Index++)
		{
			// Bail if we are on the ignore list
			if (Object->IsA(ClassesToIgnore[Index]))
			{
				bShouldSearch = false;
				break;
			}
		}

		if (bShouldSearch)
		{
			// Check to see if we should ignore it due to package
			for (int32 Index = 0; Index < PackagesToIgnore.Num(); Index++)
			{
				// If this object belongs to this package, bail
				if (Object->IsIn(PackagesToIgnore[Index]))
				{
					bShouldSearch = false;
					break;
				}
			}
		}

		if (bShouldSearch && !bIncludeDefaults && Object->IsTemplate())
		{
			// If this object is an archetype and we don't want to see assets referenced by defaults, don't include this object
			bShouldSearch = false;
		}

		return bShouldSearch;
	}
	
	/**
	 * Builds a list of assets to display from the currently selected actors.
	 * NOTE: It ignores assets that are there because they are always loaded
	 * such as default materials, textures, etc.
	 */
	void BuildAssetList(UWorld* InWorld, int32 Depth, bool bShowDefault, bool bShowScript)
	{
		// Clear the old list
		Referencers.Empty();
		ReferenceGraph.Empty();
		ObjectNameCache.Empty();

		TArray<UObject*> BspMats;
		// Search all BSP surfaces for ones that are selected and add their
		// Materials to a temp list
		for (int32 Index = 0; Index < InWorld->GetModel()->Surfs.Num(); Index++)
		{
			// Only add materials that are selected
			if (InWorld->GetModel()->Surfs[Index].PolyFlags & PF_Selected)
			{
				// No point showing the default material
				if (InWorld->GetModel()->Surfs[Index].Material != NULL)
				{
					BspMats.AddUnique(InWorld->GetModel()->Surfs[Index].Material);
				}
			}
		}
		// If any BSP surfaces are selected
		if (BspMats.Num() > 0)
		{
			FReferencedAssets& Referencer = Referencers.Emplace_GetRef(InWorld->GetModel());

			// Now copy the array
			Referencer.AssetList = BspMats;
			ReferenceGraph.Add(ObjectPtrWrap(InWorld->GetModel()), ObjectPtrWrap(BspMats));
		}

		// This is the maximum depth to use when searching for references
		const int32 MaxRecursionDepth = Depth;

		USelection* ActorSelection = GEditor->GetSelectedActors();

		// Mark all objects so we don't get into an endless recursion
		for (FThreadSafeObjectIterator It; It; ++It)
		{
			// Skip the level, world, and any packages that should be ignored
			if (ShouldSearchForAssets(*It, IgnoreClasses, IgnorePackages, bShowDefault))
			{
				It->Mark(OBJECTMARK_TagExp);
			}
			else
			{
				It->UnMark(OBJECTMARK_TagExp);
			}
		}

		TArray<AActor*> SelectedActors;
		// Get the list of currently selected actors
		ActorSelection->GetSelectedObjects<AActor>(SelectedActors);

		// Build the list of assets from the set of selected actors
		for (int32 Index = 0; Index < SelectedActors.Num(); Index++)
		{
			// Set the flag for the selected item, as it could have actually been cleared by an
			// earlier selected object, which would result in a crash later
			SelectedActors[Index]->Mark(OBJECTMARK_TagExp);

			// Create a new entry for this actor
			FReferencedAssets& Referencer = Referencers.Emplace_GetRef(SelectedActors[Index]);

			// Add to the list of referenced assets
			FFindAssetsArchive(SelectedActors[Index], Referencer.AssetList, &ReferenceGraph, MaxRecursionDepth, bShowScript, bShowDefault);
		}

		// Rebuild the name cache
		for (int32 RefIndex = 0; RefIndex < Referencers.Num(); RefIndex++)
		{
			FReferencedAssets& Referencer = Referencers[RefIndex];
			if (!ObjectNameCache.Contains(Referencer.Referencer))
			{
				ObjectNameCache.Add(Referencer.Referencer, *Referencer.Referencer->GetName());
			}

			for (int32 AssetIndex = 0; AssetIndex < Referencer.AssetList.Num(); AssetIndex++)
			{
				if (!ObjectNameCache.Contains(Referencer.AssetList[AssetIndex]))
				{
					ObjectNameCache.Add(Referencer.AssetList[AssetIndex], *Referencer.AssetList[AssetIndex]->GetName());
				}
			}
		}
	}

	/**
	 * Helper method... returns the name of a referenced object
	 */
	const TCHAR* GetObjectNameFromCache(UObject* Obj)
	{
		FString* CachedObjectName = ObjectNameCache.Find(Obj);
		if (CachedObjectName == NULL)
		{
			CachedObjectName = &ObjectNameCache.Add(Obj, *Obj->GetName());
		}
		return **CachedObjectName;
	}
	
	/**
	 * Outputs a single item for the details list
	 */
	void OutputDetailsItem(FOutputDeviceFile& FileAr, FString AssetId, UObject* ReferencedObject, FString& ItemString)
	{
		FString ObjName = FString::Printf(TEXT("%s (%s)"), *ItemString, *AssetId);
		FString Underline;
		int32 ObjNameLen = ObjName.Len();

		for (int32 i = 0; i < ObjNameLen; ++i)
		{
			Underline += TEXT("-");
		}

		// Create a string for the resource size.
		const SIZE_T ReferencedObjectResourceSize = ReferencedObject->GetResourceSizeBytes(EResourceSizeMode::EstimatedTotal);
		FString ResourceSizeString;
		if ( ReferencedObjectResourceSize > 0 )
		{
			ResourceSizeString = FString::Printf( TEXT("%.2f"), ((float)ReferencedObjectResourceSize)/1024.f );
		}

		FString ObjectPathName;
		if ( ReferencedObject->GetOuter() != NULL )
		{
			ObjectPathName = ReferencedObject->GetOuter()->GetPathName();
		}

		// Add this referenced asset's information to the list.
		FileAr.Log(TEXT(""));
		FileAr.Log(*ObjName);
		FileAr.Log(*Underline);
		FileAr.Logf(TEXT("Grouping: %s"), *ObjectPathName);
		FileAr.Logf(TEXT("Class: %s"), GetObjectNameFromCache(ReferencedObject->GetClass()));
		FileAr.Logf(TEXT("Size: %s"), *ResourceSizeString);
		FileAr.Logf(TEXT("Info: %s"), *ReferencedObject->GetDesc());
	}

	/**
	 * Recursively transverses the reference tree
	 */
	void OutputReferencedAssets(FOutputDeviceFile& FileAr, int32 CurrentDepth, FString ParentId, UObject* BaseObject, const TArray<UObject*>* AssetList)
	{
		check(AssetList);

		const FString ScriptItemString = NSLOCTEXT("UnrealEd", "Script", "Script").ToString();
		const FString DefaultsItemString = NSLOCTEXT("UnrealEd", "Defaults", "Defaults").ToString();
		for (int32 AssetIndex = 0; AssetIndex < AssetList->Num(); AssetIndex++)
		{
			UObject* ReferencedObject = (*AssetList)[AssetIndex];
			check(ReferencedObject);

			// get the list of assets this object is referencing
			auto* ReferencedAssets = ReferenceGraph.Find(ObjectPtrWrap(ReferencedObject));
			
			// add a new tree item for this referenced asset
			FString ItemString;
			if (ReferencedObject == BaseObject->GetClass())
			{
				ItemString = *ScriptItemString;
				if (ReferencedAssets == NULL || ReferencedAssets->Num() == 0)
				{
					// special case for the "Script" node - don't add it if it doesn't have any children
					continue;
				}
			}
			else if (ReferencedObject == BaseObject->GetArchetype())
			{
				ItemString = *DefaultsItemString;
				if (ReferencedAssets == NULL || ReferencedAssets->Num() == 0)
				{
					// special case for the "Defaults" node - don't add it if it doesn't have any children
					continue;
				}
			}
			else
			{
				if (CurrentDepth > 0)
				{
					ItemString = ReferencedObject->GetPathName();
				}
				else
				{
					ItemString = GetObjectNameFromCache(ReferencedObject);
				}
			}

			FString AssetId = FString::Printf(TEXT("%s.%d"), *ParentId, AssetIndex);
			
			if (CurrentDepth > 0)
			{
				FString TabStr;
				for (int32 i = 0; i < CurrentDepth; ++i)
				{
					TabStr += TEXT("\t");
				}

				FileAr.Logf(TEXT("%s(%s) %s"), *TabStr, *AssetId, *ItemString);
			}
			else
			{
				OutputDetailsItem(FileAr, AssetId, ReferencedObject, ItemString);
			}

			if (ReferencedAssets != NULL)
			{
				// If this object is referencing other objects, output those objects
				OutputReferencedAssets(FileAr, (CurrentDepth == 0)? 0: CurrentDepth + 1, AssetId, ReferencedObject, &ObjectPtrDecay(*ReferencedAssets));
			}
		}
	}

	/**
	 * Outputs the tree view
	 */
	void OutputTree(FOutputDeviceFile& FileAr)
	{
		FileAr.Logf(TEXT("*******************"));
		FileAr.Logf(TEXT("* Reference Graph *"));
		FileAr.Logf(TEXT("*******************"));
		FileAr.Logf(TEXT(""));

		for (int32 ReferenceIndex = 0; ReferenceIndex < Referencers.Num(); ReferenceIndex++)
		{
			// add an item at the root level for the selected actor
			FReferencedAssets& Asset = Referencers[ReferenceIndex];
			FString Id = FString::Printf(TEXT("%d"), ReferenceIndex);

			FileAr.Logf(TEXT("(%s) %s"), *Id, GetObjectNameFromCache(Asset.Referencer));

			auto* ReferencedAssets = ReferenceGraph.Find(ObjectPtrWrap(Asset.Referencer));
			if (ReferencedAssets)
			{
				OutputReferencedAssets(FileAr, 1, Id, Asset.Referencer, &ObjectPtrDecay(*ReferencedAssets));
			}
		}
	}
	
	/**
	 * Outputs the details list
	 */
	void OutputDetails(FOutputDeviceFile& FileAr)
	{
		FileAr.Logf(LINE_TERMINATOR);
		FileAr.Logf(TEXT("*********************"));
		FileAr.Logf(TEXT("* Reference Details *"));
		FileAr.Logf(TEXT("*********************"));

		for (int32 ReferenceIndex = 0; ReferenceIndex < Referencers.Num(); ReferenceIndex++)
		{
			// add an item at the root level for the selected actor
			FReferencedAssets& Asset = Referencers[ReferenceIndex];
			FString Id = FString::Printf(TEXT("%d"), ReferenceIndex);

			FString ItemName = GetObjectNameFromCache(Asset.Referencer);
			
			OutputDetailsItem(FileAr, Id, Asset.Referencer, ItemName);

			auto* ReferencedAssets = ReferenceGraph.Find(ObjectPtrWrap(Asset.Referencer));
			if (ReferencedAssets)
			{
				OutputReferencedAssets(FileAr, 0, Id, Asset.Referencer, &ObjectPtrDecay(*ReferencedAssets));
			}
		}
	}

	void GenerateOutput(UWorld* InWorld, int32 Depth, bool bShowDefault, bool bShowScript)
	{
		ELogTimes::Type PrintLogTimes = GPrintLogTimes;

		// Create log file
		
		const FString PathName = *(FPaths::ProjectLogDir() + TEXT("RefInfo/"));
		IFileManager::Get().MakeDirectory(*PathName);
		const FString Filename = FString::Printf(TEXT("Output-%s.txt"), *FDateTime::Now().ToString(TEXT("%m.%d-%H.%M.%S")));
		const FString FilenameFull = PathName + Filename;
		FOutputDeviceFile FileAr(*FilenameFull);
		
		FileAr.SetSuppressEventTag(true);
		GPrintLogTimes = ELogTimes::None;

		// Set up our ignore lists

		IgnoreClasses.Empty();
		IgnorePackages.Empty();

		IgnoreClasses.Add(ULevel::StaticClass());
		IgnoreClasses.Add(UWorld::StaticClass());

		// Load the asset registry module
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

		TArray<FAssetData> AssetData;
		FARFilter Filter;
		Filter.PackagePaths.Add(FName(TEXT("/Engine/EngineResources")));
		Filter.PackagePaths.Add(FName(TEXT("/Engine/EngineFonts")));
		Filter.PackagePaths.Add(FName(TEXT("/Engine/EngineMaterials")));
		Filter.PackagePaths.Add(FName(TEXT("/Engine/EditorResources")));
		Filter.PackagePaths.Add(FName(TEXT("/Engine/EditorMaterials")));

		AssetRegistryModule.Get().GetAssets(Filter, AssetData);

		for (int32 AssetIdx = 0; AssetIdx < AssetData.Num(); ++AssetIdx)
		{
			IgnorePackages.Add(FindObject<UPackage>(NULL, *AssetData[AssetIdx].PackageName.ToString(), EFindObjectFlags::ExactClass));
		}

		IgnorePackages.Add(GetTransientPackage());

		// Bug?  At this point IgnorePackages often has a handful of null entries, which, completely throws off the filtering process
		IgnorePackages.Remove(nullptr);

		// Generate reference info

		BuildAssetList(InWorld, Depth, bShowDefault, bShowScript);

		// Output reference info

		OutputTree(FileAr);

		OutputDetails(FileAr);

		FileAr.TearDown();

		GPrintLogTimes = PrintLogTimes;

		// Display "completed" popup

		FString AbsPath = FilenameFull;
		FPaths::ConvertRelativePathToFull(AbsPath);
		FFormatNamedArguments Args;
		Args.Add( TEXT("AbsolutePath"), FText::FromString( AbsPath ) );
		FNotificationInfo Info( FText::Format( NSLOCTEXT("UnrealEd", "ReferenceInfoSavedNotification", "Reference info was successfully saved to: {AbsolutePath}"), Args ) );
		Info.ExpireDuration = 3.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
	}
}
