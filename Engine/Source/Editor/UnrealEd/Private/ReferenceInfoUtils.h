// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace ReferenceInfoUtils
{
	/**
	 * Outputs reference info to a log file
	 */
	void GenerateOutput(UWorld* InWorld, int32 Depth, bool bShowDefault, bool bShowScript);
}
