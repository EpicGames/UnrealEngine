// Copyright Epic Games, Inc. All Rights Reserved.


#include "ReferencedAssetsUtils.h"
#include "Modules/ModuleManager.h"
#include "UObject/Class.h"
#include "UObject/UObjectIterator.h"
#include "UObject/Package.h"
#include "Engine/Level.h"
#include "AssetRegistry/AssetData.h"
#include "Editor.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"

DEFINE_LOG_CATEGORY_STATIC(LogReferncedAssetsBrowser, Log, All);

/**
 * Constructor. Builds the list of items to ignore
 */
FFindReferencedAssets::FFindReferencedAssets(void)
{
	OnEditorMapChangeDelegateHandle = FEditorDelegates::MapChange.AddRaw(this, &FFindReferencedAssets::OnEditorMapChange);

	// Set up our ignore lists
	IgnoreClasses.Add(ULevel::StaticClass());
	IgnoreClasses.Add(UWorld::StaticClass());

	// Load the asset registry module
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	TArray<FAssetData> AssetData;
	FARFilter Filter;
	Filter.PackagePaths.Add(FName(TEXT("/Engine/EngineResources")));
	Filter.PackagePaths.Add(FName(TEXT("/Engine/EngineFonts")));
	Filter.PackagePaths.Add(FName(TEXT("/Engine/EngineMaterials")));
	Filter.PackagePaths.Add(FName(TEXT("/Engine/EditorResources")));
	Filter.PackagePaths.Add(FName(TEXT("/Engine/EditorMaterials")));
	
	AssetRegistryModule.Get().GetAssets(Filter, AssetData);

	for (int32 AssetIdx = 0; AssetIdx < AssetData.Num(); ++AssetIdx)
	{
		UPackage* Package = FindObject<UPackage>(NULL, *AssetData[AssetIdx].PackageName.ToString(), EFindObjectFlags::ExactClass);
		if( Package != NULL )
		{
			IgnorePackages.Add(Package);
		}
	}

	IgnorePackages.Add(GetTransientPackage());
}

/**
 * Destructor
 */
FFindReferencedAssets::~FFindReferencedAssets(void)
{
	FEditorDelegates::MapChange.Remove(OnEditorMapChangeDelegateHandle);
}

void FFindReferencedAssets::OnEditorMapChange( uint32 Flag )
{
	if ( Flag != MapChangeEventFlags::Default )
	{
		Referencers.Empty();
		ReferenceGraph.Empty();
	}
}

/* === FGCObject interface === */
void FFindReferencedAssets::AddReferencedObjects( FReferenceCollector& Collector )
{
	// serialize all of our object references
	for( auto& Node : ReferenceGraph )
	{
		Collector.AddReferencedObject( Node.Key );
		Collector.AddReferencedObjects( Node.Value );
	}
	Collector.AddReferencedObjects( IgnoreClasses );
	Collector.AddReferencedObjects( IgnorePackages );
}

/**
 * Checks an object to see if it should be included for asset searching
 *
 * @param Object the object in question
 * @param ClassesToIgnore the list of classes to skip
 * @param PackagesToIgnore the list of packages to skip
 * @param bIncludeDefaults specify true to include content referenced through defaults
 *
 * @return true if it should be searched, false otherwise
 */
bool FFindReferencedAssets::ShouldSearchForAssets( const UObject* Object, const TArray<UClass*>& ClassesToIgnore, const TArray<UPackage*>& PackagesToIgnore, bool bIncludeDefaults/*=false*/ )
{
	bool bShouldSearch = true;

	//@Package name transition
	if ( Object->HasAnyFlags(RF_ClassDefaultObject) && (Object->GetOutermost()->GetFName() == NAME_CoreUObject || Object->GetOutermost()->GetFName() == GLongCoreUObjectPackageName))
	{
		// ignore all class default objects for classes which are declared in Core
		bShouldSearch = false;
	}

	// Check to see if we should ignore a class
	for (int32 Index = 0; Index < ClassesToIgnore.Num(); Index++)
	{
		// Bail if we are on the ignore list
		if ( Object->IsA(ClassesToIgnore[Index]) )
		{
			bShouldSearch = false;
			break;
		}
	}

	if ( bShouldSearch )
	{
		// Check to see if we should ignore it due to package
		for ( int32 Index = 0; Index < PackagesToIgnore.Num(); Index++ )
		{
			// If this object belongs to this package, bail
			if ( Object->IsInPackage(PackagesToIgnore[Index]) )
			{
				bShouldSearch = false;
				break;
			}
		}
	}

	if ( bShouldSearch && !bIncludeDefaults && Object->IsTemplate() )
	{
		// if this object is an archetype and we don't want to see assets referenced by defaults, don't include this object
		bShouldSearch = false;
	}

	return bShouldSearch;
}

/**
 * Returns a list of all assets referenced by the specified UObject.
 */
void FFindReferencedAssets::BuildAssetList(UObject *Object, const TArray<UClass*>& IgnoreClasses, const TArray<UPackage*>& IgnorePackages, TSet<UObject*>& ReferencedAssets, bool bIncludeDefaultRefs, bool bOnlyDirectReferences)
{
	TArray<FReferencedAssets> LocalReferencers;

	// Create a new entry for this actor.
	LocalReferencers.Emplace( Object );

	for (FThreadSafeObjectIterator It; It; ++It)
	{
		// Skip the level, world, and any packages that should be ignored
		if ( ShouldSearchForAssets(*It, IgnoreClasses, IgnorePackages, bIncludeDefaultRefs) )
		{
			It->Mark(OBJECTMARK_TagExp);
		}
		else
		{
			It->UnMark(OBJECTMARK_TagExp);
		}
	}

	// Add to the list of referenced assets.
	const int32 MaxRecursionDepth = bOnlyDirectReferences ? 1 : 0;
	FFindAssetsArchive( Object, LocalReferencers.Last().AssetList, NULL, MaxRecursionDepth, /*bIncludeClasses=*/true, bIncludeDefaultRefs );
	ReferencedAssets = LocalReferencers.Last().AssetList;
}

/**
 * Functor that starts the serialization process
 *
 * @param Search the object to start searching
 * @param IgnoreClasses the list of classes to skip
 * @param IgnorePackages the list of packages to skip
 */
FFindAssetsArchive::FFindAssetsArchive(
	UObject* Search,
	TSet<UObject*>& OutAssetList,
	ObjectReferenceGraph* ReferenceGraph/*=NULL*/,
	int32 MaxRecursion/*=0*/, 
	bool bIncludeClasses/*=true*/, 
	bool bIncludeDefaults/*=false*/,
	bool bReverseReferenceGraph/*=false*/ ) 
: StartObject(Search)
, AssetList(OutAssetList)
, CurrentReferenceGraph(ReferenceGraph)
, bIncludeScriptRefs(bIncludeClasses)
, bIncludeDefaultRefs(bIncludeDefaults)
, MaxRecursionDepth(MaxRecursion)
, CurrentDepth(0)
, bUseReverseReferenceGraph(bReverseReferenceGraph)
{
	ArIsObjectReferenceCollector = true;
	ArIsModifyingWeakAndStrongReferences = true; // While we are not modifying them, we want to follow weak references as well
	ArIgnoreClassRef = !bIncludeScriptRefs;

	CurrentObject = StartObject;

	*this << StartObject;
}

/**
 * Adds the object reference to the asset list if it supports thumbnails.
 * Recursively searches through its references for more assets
 *
 * @param Obj the object to inspect
 */
FArchive& FFindAssetsArchive::operator<<(UObject*& Obj)
{
	// Don't check null references or objects already visited
	if (Obj != NULL &&

		// if we wish to filter out assets referenced through script, we need to ignore
		// all class objects, not just the UObject::Class reference
		(!ArIgnoreClassRef || (Cast<UClass>(Obj) == NULL)))
	{
		bool bUnvisited = Obj->HasAnyMarks(OBJECTMARK_TagExp);

		// Clear the search flag so we don't revisit objects
		Obj->UnMark(OBJECTMARK_TagExp);

		if ( Obj->IsA(UField::StaticClass()) )
		{
			// skip all of the other stuff because the serialization of UFields will quickly overflow
			// our stack given the number of temporary variables we create in the below code
			if (bUnvisited)
			{
				Obj->Serialize(*this);
			}
		}
		else
		{
			bool bRecurse = true;
			const bool bCDO = Obj->HasAnyFlags(RF_ClassDefaultObject);
			const bool bIsContent = Obj->IsAsset();
			const bool bIncludeAnyway = (Obj->GetOuter() == CurrentObject) && (Cast<UClass>(CurrentObject) == NULL);
			const bool bShouldReportAsset = !bCDO && (bIsContent || bIncludeAnyway);

			// remember which object we were serializing
			UObject* PreviousObject = CurrentObject;
			if ( bShouldReportAsset )
			{
				CurrentObject = Obj;

				// Add this object to the list to display
				AssetList.Add(CurrentObject);
				if ( CurrentReferenceGraph != NULL )
				{
					UObject* Key = bUseReverseReferenceGraph? CurrentObject: PreviousObject;
					UObject* Value = bUseReverseReferenceGraph? PreviousObject: CurrentObject;

					auto* CurrentObjectAssets = GetAssetList(Key);
					check(CurrentObjectAssets);

					// add this object to the list of objects referenced by the object currently being serialized
					CurrentObjectAssets->Add(Value);	
					
					if (bUnvisited)
					{
						HandleReferencedObject(CurrentObject);
					}
				}
			}
			else if ( Obj == StartObject )
			{
				if (bUnvisited)
				{
					HandleReferencedObject(Obj);
				}
			}
			else if (Obj->GetOuter() && PreviousObject != Obj->GetOuter() && Obj->GetOuter()->HasAnyMarks(OBJECTMARK_TagExp))
			{
				Obj->Mark(OBJECTMARK_TagExp);
				bRecurse = false;
			}

			if ( bRecurse && ( MaxRecursionDepth == 0 || CurrentDepth < MaxRecursionDepth ) )
			{
				CurrentDepth++;
				CurrentObject = Obj;
				// Now recursively search this object for more references
				if (bUnvisited)
				{
					Obj->Serialize(*this);
				}

				CurrentDepth--;
			}

			// restore the previous object that was being serialized
			CurrentObject = PreviousObject;
		}
	}
	return *this;
}

/**
 * Manually serializes the class and archetype for the specified object so that assets which are referenced
 * through the object's class/archetype can be differentiated.
 */
void FFindAssetsArchive::HandleReferencedObject(UObject* Obj )
{
	if ( CurrentReferenceGraph != NULL )
	{
		// here we allow recursion if the current depth is less-than-equal (as opposed to less-than) because the archetype and class are treated as transparent objects
		// serialization of the class and object are controlled by the "show class refs" and "show default refs" buttons
		if ( MaxRecursionDepth == 0 || CurrentDepth < MaxRecursionDepth )
		{
			// now change the current reference list to the one for this object
			if ( bIncludeDefaultRefs == true )
			{
				UObject* ObjectArc = Obj->GetArchetype();
				UObject* Key = bUseReverseReferenceGraph? ObjectArc: Obj;
				UObject* Value = bUseReverseReferenceGraph? Obj: ObjectArc;
				auto* ReferencedAssets = GetAssetList(Key);

				// @see the comment for the bIncludeScriptRefs block
				ReferencedAssets->Add(Value);

				UObject* PreviousObject = CurrentObject;
				CurrentObject = ObjectArc;

				if ( ObjectArc->HasAnyMarks(OBJECTMARK_TagExp) )
				{
					// temporarily disable serialization of the class, as we need to specially handle that as well
					bool bSkipClassSerialization = ArIgnoreClassRef;
					ArIgnoreClassRef = true;

					ObjectArc->UnMark(OBJECTMARK_TagExp);
					ObjectArc->Serialize(*this);

					ArIgnoreClassRef = bSkipClassSerialization;
				}

				CurrentObject = PreviousObject;
			}

			if ( bIncludeScriptRefs == true )
			{
				UClass* ObjectClass = Obj->GetClass();
				UObject* Key = bUseReverseReferenceGraph? ObjectClass: Obj;
				UObject* Value = bUseReverseReferenceGraph? Obj: ObjectClass;
				auto* ReferencedAssets = GetAssetList(Key);

				// we want to see assets referenced by this object's class, but classes don't have associated thumbnail rendering info
				// so we'll need to serialize the class manually in order to get the object references encountered through the class to fal
				// under the appropriate tree item

				// serializing the class will result in serializing the class default object; but we need to do this manually (for the same reason
				// that we do it for the class), so temporarily prevent the CDO from being serialized by this archive
				ReferencedAssets->Add(Value);

				UObject* PreviousObject = CurrentObject;
				CurrentObject = ObjectClass;

				if ( ObjectClass->HasAnyMarks(OBJECTMARK_TagExp) )
				{
					ObjectClass->UnMark(OBJECTMARK_TagExp);
					ObjectClass->Serialize(*this);
				}

				CurrentObject = PreviousObject;
			}
		}
	}
}

/**
 * Retrieves the referenced assets list for the specified object.
 */
TSet<TObjectPtr<UObject>>* FFindAssetsArchive::GetAssetList( UObject* Referencer )
{
	check(Referencer);

	auto* ReferencedAssetList = CurrentReferenceGraph->Find(Referencer);
	if ( ReferencedAssetList == NULL )
	{
		// add a new entry for the specified object
		ReferencedAssetList = &CurrentReferenceGraph->Add(Referencer, TSet<TObjectPtr<UObject>>{});
	}

	return ReferencedAssetList;
}


