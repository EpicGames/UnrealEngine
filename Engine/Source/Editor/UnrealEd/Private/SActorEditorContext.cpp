// Copyright Epic Games, Inc. All Rights Reserved.

#include "SActorEditorContext.h"
#include "IActorEditorContextClient.h"
#include "Subsystems/ActorEditorContextSubsystem.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Editor.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "ActorEditorContext"

void SActorEditorContext::Construct(const FArguments& InArgs)
{
	World = (InArgs._World);
	bNeedsRebuild = false;
	bIsContextExpanded = true;
	check(GEditor);
	GEditor->GetEditorWorldContext().AddRef(World);
	UActorEditorContextSubsystem::Get()->OnActorEditorContextSubsystemChanged().AddSP(this, &SActorEditorContext::OnActorEditorContextSubsystemChanged);
	FEditorDelegates::MapChange.AddSP(this, &SActorEditorContext::OnEditorMapChange);
	Rebuild();
}

SActorEditorContext::~SActorEditorContext()
{
	FEditorDelegates::MapChange.RemoveAll(this);
	if (GEditor && UObjectInitialized())
	{
		GEditor->GetEditorWorldContext().RemoveRef(World);
		if (UActorEditorContextSubsystem* ActorEditorContextSubsystem = UActorEditorContextSubsystem::Get())
		{
			ActorEditorContextSubsystem->OnActorEditorContextSubsystemChanged().RemoveAll(this);
		}
	}
}

void SActorEditorContext::OnActorEditorContextSubsystemChanged()
{
	bNeedsRebuild = true;
}

void SActorEditorContext::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bNeedsRebuild)
	{
		bNeedsRebuild = false;
		Rebuild();
	}

	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

void SActorEditorContext::Rebuild()
{
	TArray<IActorEditorContextClient*> Clients = UActorEditorContextSubsystem::Get()->GetDisplayableClients();
	if (Clients.Num() > 0 && World)
	{
		TSharedPtr<SHorizontalBox> HBox;
		TSharedPtr<SVerticalBox> VBox;
		ChildSlot.Padding(2, 2, 2, 2)
		[
			SAssignNew(VBox, SVerticalBox)
		];

		VBox->AddSlot()
			.AutoHeight()
			.Padding(0, 2)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2.0f, 1.0f, 2.0f, 1.0f)
					[
						SNew(STextBlock)
						.Text(FText::FromString(TEXT("Current Context")))
						.ShadowOffset(FVector2D(1, 1))
						.Font(FCoreStyle::GetDefaultFontStyle("Bold", 10))
						.ColorAndOpacity(FLinearColor::White)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					[
						SAssignNew(HBox, SHorizontalBox)
						.Visibility_Lambda([this]()
						{
							return !bIsContextExpanded ? EVisibility::Visible : EVisibility::Collapsed;
						})
					]
					+ SHorizontalBox::Slot()					
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.Padding(2.0f, 1.0f, 2.0f, 1.0f)
					[
						SNew(SButton)
						.Cursor(EMouseCursor::Default)
						.ButtonStyle(FAppStyle::Get(), "NoBorder")
						.ContentPadding(0.0f)
						.OnClicked_Lambda([this]()
						{
							bIsContextExpanded = !bIsContextExpanded;
							return FReply::Handled();
						})
						.Content()
						[
							SNew(SImage)
							.Image_Lambda([this]() { return bIsContextExpanded ? FAppStyle::GetBrush("ContentBrowser.SortDown") : FAppStyle::GetBrush("ContentBrowser.SortUp"); })
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					]
				]
			];

		for (IActorEditorContextClient* Client : Clients)
		{
			FActorEditorContextClientDisplayInfo Info;
			Client->GetActorEditorContextDisplayInfo(World, Info);

			HBox->AddSlot()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.Padding(2.0f, 1.0f, 2.0f, 1.0f)
				[
					SNew(SImage)
					.Image(Info.Brush ? Info.Brush : FStyleDefaults::GetNoBrush())
					.DesiredSizeOverride(FVector2D(16, 16))
				]
			];

			VBox->AddSlot()
			.AutoHeight()
			.Padding(0, 2)
			[
				SNew(SBorder)
				.Visibility_Lambda([Client, this]()
				{ 
					FActorEditorContextClientDisplayInfo Info;
					return bIsContextExpanded && Client->GetActorEditorContextDisplayInfo(World, Info) ? EVisibility::Visible : EVisibility::Collapsed;
				})
				.BorderImage(FCoreStyle::Get().GetBrush("Docking.Sidebar.Border"))
				.Content()
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(2.0f, 1.0f, 2.0f, 1.0f)
						[
							SNew(SImage)
							.Image(Info.Brush ? Info.Brush : FStyleDefaults::GetNoBrush())
							.DesiredSizeOverride(FVector2D(16, 16))
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(2.0f, 1.0f, 2.0f, 1.0f)
						[
							SNew(STextBlock)
							.Text(FText::FromString(Info.Title))
							.ShadowOffset(FVector2D(1, 1))
							.Font(FCoreStyle::GetDefaultFontStyle("Bold", 10))
							.ColorAndOpacity(FLinearColor::White)
						]
						+ SHorizontalBox::Slot()
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Center)
						.FillWidth(1.f)
						[
							SNew(SButton)
							.Cursor(EMouseCursor::Default)
							.ButtonStyle(FAppStyle::Get(), "SimpleButton")
							.ContentPadding(0.0f)
							.Visibility_Lambda([Client, this]() { return (Client && Client->CanResetContext(World)) ? EVisibility::Visible : EVisibility::Collapsed; })
							.ToolTipText(FText::Format(LOCTEXT("ResetActorEditorContextTooltip", "Reset Current {0}"), FText::FromString(Info.Title)))
							.OnClicked_Lambda([Client, this]()
							{
								if (Client && Client->CanResetContext(World))
								{
									UActorEditorContextSubsystem::Get()->ResetContext(Client);
								}
								return FReply::Handled();
							})
							.Content()
							[
								SNew(SImage)
								.Image(FAppStyle::GetBrush("Icons.X"))
								.ColorAndOpacity(FSlateColor::UseForeground())
							]
						]
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					// Left padding details (2 + 2 + 16 + 2) : 2 + 2 = Info.Brush left/right padding, + 16 = Info.Brush width, + 2 = Info.Title left padding
					.Padding(2 + 2 + 16 + 2, 2, 0, 2)
					[
						World ? Client->GetActorEditorContextWidget(World) : SNullWidget::NullWidget
					]
				]
			];
		}
	}
	else
	{
		ChildSlot[SNullWidget::NullWidget];
	}
}

bool SActorEditorContext::IsVisible(UWorld* InWorld)
{
	if (InWorld && GEditor && !IsEngineExitRequested())
	{
		return UActorEditorContextSubsystem::Get()->GetDisplayableClients().Num() > 0;
	}
	return false;
}

#undef LOCTEXT_NAMESPACE