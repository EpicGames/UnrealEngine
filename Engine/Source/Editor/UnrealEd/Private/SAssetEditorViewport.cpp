// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAssetEditorViewport.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Editor/UnrealEdEngine.h"
#include "UnrealEdGlobals.h"
#include "EditorViewportCommands.h"
#include "EditorViewportTabContent.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "AssetEditorViewportLayout.h"

#define LOCTEXT_NAMESPACE "SAssetEditorViewport"


void SAssetEditorViewport::BindCommands()
{
	FUICommandList& CommandListRef = *CommandList;

	const FEditorViewportCommands& Commands = FEditorViewportCommands::Get();

	CommandListRef.MapAction(
		Commands.ViewportConfig_OnePane,
		FExecuteAction::CreateSP(this, &SAssetEditorViewport::OnSetViewportConfiguration, EditorViewportConfigurationNames::OnePane),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAssetEditorViewport::IsViewportConfigurationSet, EditorViewportConfigurationNames::OnePane));
	
	CommandListRef.MapAction(
		Commands.ViewportConfig_TwoPanesH,
		FExecuteAction::CreateSP(this, &SAssetEditorViewport::OnSetViewportConfiguration, EditorViewportConfigurationNames::TwoPanesHoriz),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAssetEditorViewport::IsViewportConfigurationSet, EditorViewportConfigurationNames::TwoPanesHoriz));

	CommandListRef.MapAction(
		Commands.ViewportConfig_TwoPanesV,
		FExecuteAction::CreateSP(this, &SAssetEditorViewport::OnSetViewportConfiguration, EditorViewportConfigurationNames::TwoPanesVert),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAssetEditorViewport::IsViewportConfigurationSet, EditorViewportConfigurationNames::TwoPanesVert));

	CommandListRef.MapAction(
		Commands.ViewportConfig_ThreePanesLeft,
		FExecuteAction::CreateSP(this, &SAssetEditorViewport::OnSetViewportConfiguration, EditorViewportConfigurationNames::ThreePanesLeft),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAssetEditorViewport::IsViewportConfigurationSet, EditorViewportConfigurationNames::ThreePanesLeft));

	CommandListRef.MapAction(
		Commands.ViewportConfig_ThreePanesRight,
		FExecuteAction::CreateSP(this, &SAssetEditorViewport::OnSetViewportConfiguration, EditorViewportConfigurationNames::ThreePanesRight),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAssetEditorViewport::IsViewportConfigurationSet, EditorViewportConfigurationNames::ThreePanesRight));

	CommandListRef.MapAction(
		Commands.ViewportConfig_ThreePanesTop,
		FExecuteAction::CreateSP(this, &SAssetEditorViewport::OnSetViewportConfiguration, EditorViewportConfigurationNames::ThreePanesTop),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAssetEditorViewport::IsViewportConfigurationSet, EditorViewportConfigurationNames::ThreePanesTop));

	CommandListRef.MapAction(
		Commands.ViewportConfig_ThreePanesBottom,
		FExecuteAction::CreateSP(this, &SAssetEditorViewport::OnSetViewportConfiguration, EditorViewportConfigurationNames::ThreePanesBottom),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAssetEditorViewport::IsViewportConfigurationSet, EditorViewportConfigurationNames::ThreePanesBottom));

	CommandListRef.MapAction(
		Commands.ViewportConfig_FourPanesLeft,
		FExecuteAction::CreateSP(this, &SAssetEditorViewport::OnSetViewportConfiguration, EditorViewportConfigurationNames::FourPanesLeft),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAssetEditorViewport::IsViewportConfigurationSet, EditorViewportConfigurationNames::FourPanesLeft));

	CommandListRef.MapAction(
		Commands.ViewportConfig_FourPanesRight,
		FExecuteAction::CreateSP(this, &SAssetEditorViewport::OnSetViewportConfiguration, EditorViewportConfigurationNames::FourPanesRight),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAssetEditorViewport::IsViewportConfigurationSet, EditorViewportConfigurationNames::FourPanesRight));

	CommandListRef.MapAction(
		Commands.ViewportConfig_FourPanesTop,
		FExecuteAction::CreateSP(this, &SAssetEditorViewport::OnSetViewportConfiguration, EditorViewportConfigurationNames::FourPanesTop),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAssetEditorViewport::IsViewportConfigurationSet, EditorViewportConfigurationNames::FourPanesTop));

	CommandListRef.MapAction(
		Commands.ViewportConfig_FourPanesBottom,
		FExecuteAction::CreateSP(this, &SAssetEditorViewport::OnSetViewportConfiguration, EditorViewportConfigurationNames::FourPanesBottom),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAssetEditorViewport::IsViewportConfigurationSet, EditorViewportConfigurationNames::FourPanesBottom));

	CommandListRef.MapAction(
		Commands.ViewportConfig_FourPanes2x2,
		FExecuteAction::CreateSP(this, &SAssetEditorViewport::OnSetViewportConfiguration, EditorViewportConfigurationNames::FourPanes2x2),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAssetEditorViewport::IsViewportConfigurationSet, EditorViewportConfigurationNames::FourPanes2x2));

	SEditorViewport::BindCommands();
}


void SAssetEditorViewport::Construct(const FArguments& InArgs, const FAssetEditorViewportConstructionArgs& InViewportConstructionArgs)
{
	ConfigKey = InViewportConstructionArgs.ConfigKey;
	Client = InArgs._EditorViewportClient;
	
	if (InViewportConstructionArgs.ParentLayout)
	{
		ParentTabContent = InViewportConstructionArgs.ParentLayout->GetParentTabContent();
	}

	SEditorViewport::FArguments ViewportArgs;
	if (InArgs._ViewportSize.IsSet())
	{
		ViewportArgs._ViewportSize = InArgs._ViewportSize;
	}
	SEditorViewport::Construct(ViewportArgs);

	if (InViewportConstructionArgs.IsEnabled.IsSet())
	{
		SetEnabled(InViewportConstructionArgs.IsEnabled);
	}

	GetViewportClient()->SetViewportType(InViewportConstructionArgs.ViewportType);
	GetViewportClient()->SetRealtime(InViewportConstructionArgs.bRealtime);
}

void SAssetEditorViewport::OnSetViewportConfiguration(FName ConfigurationName)
{
	if (TSharedPtr<FViewportTabContent> ViewportTabPinned = ParentTabContent.Pin())
	{
		ViewportTabPinned->SetViewportConfiguration(ConfigurationName);
		FSlateApplication::Get().DismissAllMenus();
	}
}

bool SAssetEditorViewport::IsViewportConfigurationSet(FName ConfigurationName) const
{
	if (TSharedPtr<FViewportTabContent> ViewportTabPinned = ParentTabContent.Pin())
	{
		return ViewportTabPinned->IsViewportConfigurationSet(ConfigurationName);
	}
	return false;
}


void SAssetEditorViewport::GenerateLayoutMenu(FMenuBuilder& MenuBuilder) const
{
	// Disable searching in this menu because it only contains visual representations of
	// viewport layouts without any searchable text.
	MenuBuilder.SetSearchable(false);

	MenuBuilder.BeginSection("EditorViewportOnePaneConfigs", LOCTEXT("OnePaneConfigHeader", "One Pane"));
	{
		FSlimHorizontalToolBarBuilder OnePaneButton(CommandList, FMultiBoxCustomization::None);
		OnePaneButton.SetLabelVisibility(EVisibility::Collapsed);
		OnePaneButton.SetStyle(&FAppStyle::Get(), "ViewportLayoutToolbar");

		OnePaneButton.AddToolBarButton(FEditorViewportCommands::Get().ViewportConfig_OnePane);

		MenuBuilder.AddWidget(
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				OnePaneButton.MakeWidget()
			]
		+ SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SNullWidget::NullWidget
			],
			FText::GetEmpty(), true
			);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("EditorViewportTwoPaneConfigs", LOCTEXT("TwoPaneConfigHeader", "Two Panes"));
	{
		FSlimHorizontalToolBarBuilder TwoPaneButtons(CommandList, FMultiBoxCustomization::None);
		TwoPaneButtons.SetLabelVisibility(EVisibility::Collapsed);
		TwoPaneButtons.SetStyle(&FAppStyle::Get(), "ViewportLayoutToolbar");

		TwoPaneButtons.AddToolBarButton(FEditorViewportCommands::Get().ViewportConfig_TwoPanesH, NAME_None, FText());
		TwoPaneButtons.AddToolBarButton(FEditorViewportCommands::Get().ViewportConfig_TwoPanesV, NAME_None, FText());

		MenuBuilder.AddWidget(
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				TwoPaneButtons.MakeWidget()
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SNullWidget::NullWidget
			],
			FText::GetEmpty(), true
			);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("EditorViewportThreePaneConfigs", LOCTEXT("ThreePaneConfigHeader", "Three Panes"));
	{
		FSlimHorizontalToolBarBuilder ThreePaneButtons(CommandList, FMultiBoxCustomization::None);
		ThreePaneButtons.SetLabelVisibility(EVisibility::Collapsed);
		ThreePaneButtons.SetStyle(&FAppStyle::Get(), "ViewportLayoutToolbar");

		ThreePaneButtons.AddToolBarButton(FEditorViewportCommands::Get().ViewportConfig_ThreePanesLeft, NAME_None, FText());
		ThreePaneButtons.AddToolBarButton(FEditorViewportCommands::Get().ViewportConfig_ThreePanesRight, NAME_None, FText());
		ThreePaneButtons.AddToolBarButton(FEditorViewportCommands::Get().ViewportConfig_ThreePanesTop, NAME_None, FText());
		ThreePaneButtons.AddToolBarButton(FEditorViewportCommands::Get().ViewportConfig_ThreePanesBottom, NAME_None, FText());

		MenuBuilder.AddWidget(
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				ThreePaneButtons.MakeWidget()
			]
		+ SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SNullWidget::NullWidget
			],
			FText::GetEmpty(), true
			);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("EditorViewportFourPaneConfigs", LOCTEXT("FourPaneConfigHeader", "Four Panes"));
	{
		FSlimHorizontalToolBarBuilder FourPaneButtons(CommandList, FMultiBoxCustomization::None);
		FourPaneButtons.SetLabelVisibility(EVisibility::Collapsed);
		FourPaneButtons.SetStyle(&FAppStyle::Get(), "ViewportLayoutToolbar");

		FourPaneButtons.AddToolBarButton(FEditorViewportCommands::Get().ViewportConfig_FourPanes2x2, NAME_None, FText());
		FourPaneButtons.AddToolBarButton(FEditorViewportCommands::Get().ViewportConfig_FourPanesLeft, NAME_None, FText());
		FourPaneButtons.AddToolBarButton(FEditorViewportCommands::Get().ViewportConfig_FourPanesRight, NAME_None, FText());
		FourPaneButtons.AddToolBarButton(FEditorViewportCommands::Get().ViewportConfig_FourPanesTop, NAME_None, FText());
		FourPaneButtons.AddToolBarButton(FEditorViewportCommands::Get().ViewportConfig_FourPanesBottom, NAME_None, FText());

		MenuBuilder.AddWidget(
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				FourPaneButtons.MakeWidget()
			]
		+ SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SNullWidget::NullWidget
			],
			FText::GetEmpty(), true
			);
	}
	MenuBuilder.EndSection();

}

TSharedRef<FEditorViewportClient> SAssetEditorViewport::MakeEditorViewportClient()
{
	if (!Client.IsValid())
	{
		Client = MakeShareable(new FEditorViewportClient(nullptr));
	}

	return Client.ToSharedRef();
}

#undef LOCTEXT_NAMESPACE
