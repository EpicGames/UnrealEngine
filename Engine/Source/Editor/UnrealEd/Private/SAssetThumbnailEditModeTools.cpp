// Copyright Epic Games, Inc. All Rights Reserved.


#include "SAssetThumbnailEditModeTools.h"

#include "AssetRegistry/AssetData.h"
#include "AssetThumbnail.h"
#include "AssetToolsModule.h"
#include "Containers/EnumAsByte.h"
#include "Delegates/Delegate.h"
#include "Editor/UnrealEdEngine.h"
#include "GenericPlatform/ICursor.h"
#include "IAssetTools.h"
#include "IAssetTypeActions.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/Casts.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "ThumbnailRendering/SceneThumbnailInfoWithPrimitive.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPath.h"
#include "UnrealEdGlobals.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SAssetThumbnailEditModeTools"

void SAssetThumbnailEditModeTools::Construct(const FArguments& InArgs, const TSharedPtr<FAssetThumbnail>& InAssetThumbnail)
{
	AssetThumbnail = InAssetThumbnail;
	bModifiedThumbnailWhileDragging = false;
	DragStartLocation = FIntPoint(ForceInitToZero);
	bInSmallView = InArgs._SmallView;

	constexpr float EditModeButtonSize = 20.f;
	constexpr float EditModeButtonPadding = 4.f;
	constexpr float EditModeButtonContentPadding = 2.f;

	ChildSlot
	[
		SNew(SHorizontalBox)

		// Primitive tools
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Bottom)
		.HAlign(HAlign_Left)
		.Padding(EditModeButtonPadding, 0.f, 0.f, EditModeButtonPadding)
		[
			SNew(SBox)
			.HeightOverride(EditModeButtonSize)
			.WidthOverride(EditModeButtonSize)
			[
				SNew(SButton)
				.Visibility(this, &SAssetThumbnailEditModeTools::GetPrimitiveToolsVisibility)
				.ButtonStyle(FAppStyle::Get(), "AssetThumbnail.EditMode.Primitive")
				.OnClicked(this, &SAssetThumbnailEditModeTools::ChangePrimitive)
				.ToolTipText(LOCTEXT("CyclePrimitiveThumbnailShapes", "Cycle through primitive shape for this thumbnail"))
				.ContentPadding(EditModeButtonContentPadding)
				.Content()
				[
					SNew(SImage)
					.Image(this, &SAssetThumbnailEditModeTools::GetCurrentPrimitiveBrush)
				]
			]
		]

		+SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Bottom)
		.Padding(0.f, 0.f, EditModeButtonPadding, EditModeButtonPadding)
		[
			SNew(SBox)
			.HeightOverride(EditModeButtonSize)
			.WidthOverride(EditModeButtonSize)
			[
				SNew(SButton)
				.Visibility(this, &SAssetThumbnailEditModeTools::GetPrimitiveToolsResetToDefaultVisibility)
				.ButtonStyle(FAppStyle::Get(), "AssetThumbnail.EditMode.Primitive")
				.OnClicked(this, &SAssetThumbnailEditModeTools::ResetToDefault)
				.ToolTipText(LOCTEXT("ResetThumbnailToDefault", "Resets thumbnail to the default"))
				.ContentPadding(EditModeButtonContentPadding)
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("PropertyWindow.DiffersFromDefault"))
				]
			]
		]
	];
}

EVisibility SAssetThumbnailEditModeTools::GetPrimitiveToolsVisibility() const
{
	const bool bIsVisible = bInSmallView && (GetSceneThumbnailInfo() != nullptr && GetSceneThumbnailInfoWithPrimitive() != nullptr);
	return bIsVisible ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SAssetThumbnailEditModeTools::GetPrimitiveToolsResetToDefaultVisibility() const
{
	USceneThumbnailInfo* ThumbnailInfo = GetSceneThumbnailInfo();
	
	EVisibility ResetToDefaultVisibility = EVisibility::Collapsed;
	if (ThumbnailInfo)
	{
		ResetToDefaultVisibility = ThumbnailInfo && ThumbnailInfo->DiffersFromDefault() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	return ResetToDefaultVisibility;
}

const FSlateBrush* SAssetThumbnailEditModeTools::GetCurrentPrimitiveBrush() const
{
	if (USceneThumbnailInfoWithPrimitive* ThumbnailInfo = GetSceneThumbnailInfoWithPrimitive())
	{
		// Note this is for the icon only.  we are assuming the thumbnail renderer does the right thing when rendering
		const EThumbnailPrimType PrimType = ThumbnailInfo->bUserModifiedShape ? ThumbnailInfo->PrimitiveType.GetValue() : (EThumbnailPrimType)ThumbnailInfo->DefaultPrimitiveType.Get(EThumbnailPrimType::TPT_Sphere);
		switch (PrimType)
		{
			case TPT_None: return FAppStyle::GetBrush("ContentBrowser.PrimitiveCustom");
			case TPT_Sphere: return FAppStyle::GetBrush("ContentBrowser.PrimitiveSphere");
			case TPT_Cube: return FAppStyle::GetBrush("ContentBrowser.PrimitiveCube");
			case TPT_Cylinder: return FAppStyle::GetBrush("ContentBrowser.PrimitiveCylinder");
			case TPT_Plane:
			default:
			// Fall through and return a plane
			break;
		}
	}

	return FAppStyle::GetBrush("ContentBrowser.PrimitivePlane");
}

FReply SAssetThumbnailEditModeTools::ChangePrimitive()
{
	USceneThumbnailInfoWithPrimitive* ThumbnailInfo = GetSceneThumbnailInfoWithPrimitive();
	if (ThumbnailInfo)
	{
		uint8 PrimitiveIdx = ThumbnailInfo->PrimitiveType.GetIntValue() + 1;
		if (PrimitiveIdx >= TPT_MAX)
		{
			if (ThumbnailInfo->PreviewMesh.IsValid())
			{
				PrimitiveIdx = TPT_None;
			}
			else
			{
				PrimitiveIdx = TPT_None + 1;
			}
		}

		ThumbnailInfo->PrimitiveType = TEnumAsByte<EThumbnailPrimType>(PrimitiveIdx);
		ThumbnailInfo->bUserModifiedShape = true;

		if (AssetThumbnail.IsValid())
		{
			AssetThumbnail.Pin()->RefreshThumbnail();
		}

		ThumbnailInfo->MarkPackageDirty();
	}

	return FReply::Handled();
}

FReply SAssetThumbnailEditModeTools::ResetToDefault()
{
	if (USceneThumbnailInfo* ThumbnailInfo = GetSceneThumbnailInfo())
	{
		ThumbnailInfo->ResetToDefault();

		if (AssetThumbnail.IsValid())
		{
			AssetThumbnail.Pin()->RefreshThumbnail();
		}

		ThumbnailInfo->MarkPackageDirty();

	}

	return FReply::Handled();
}

FReply SAssetThumbnailEditModeTools::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (AssetThumbnail.IsValid() &&
		(MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton || MouseEvent.GetEffectingButton() == EKeys::RightMouseButton))
	{
		// Load the asset, unless it is in an unloaded map package or already loaded
		const FAssetData& AssetData = AssetThumbnail.Pin()->GetAssetData();
		
		// Getting the asset loads it, if it isn't already.
		UObject* Asset = AssetData.GetAsset();

		USceneThumbnailInfo* ThumbnailInfo = GetSceneThumbnailInfo();
		if ( ThumbnailInfo )
		{
			FThumbnailRenderingInfo* RenderInfo = GUnrealEd->GetThumbnailManager()->GetRenderingInfo(Asset);
			if (RenderInfo != nullptr && RenderInfo->Renderer != nullptr)
			{
				bModifiedThumbnailWhileDragging = false;
				DragStartLocation = FIntPoint(FMath::TruncToInt32(MouseEvent.GetScreenSpacePosition().X), FMath::TruncToInt32(MouseEvent.GetScreenSpacePosition().Y));

				bIsEditing = true;
				return FReply::Handled().CaptureMouse(AsShared()).UseHighPrecisionMouseMovement(AsShared()).PreventThrottling();
			}
		}

		// This thumbnail does not have a scene thumbnail info but thumbnail editing is enabled. Just consume the input.
		return FReply::Handled();
	}
		
	return FReply::Unhandled();
}

FReply SAssetThumbnailEditModeTools::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (HasMouseCapture())
	{
		if (bModifiedThumbnailWhileDragging)
		{
			USceneThumbnailInfo* ThumbnailInfo = GetSceneThumbnailInfo();
			if (ThumbnailInfo)
			{
				ThumbnailInfo->MarkPackageDirty();
			}

			bModifiedThumbnailWhileDragging = false;
		}

		bIsEditing = false;
		return FReply::Handled().ReleaseMouseCapture().SetMousePos(DragStartLocation);
	}

	return FReply::Unhandled();
}

FReply SAssetThumbnailEditModeTools::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (HasMouseCapture())
	{
		if (!MouseEvent.GetCursorDelta().IsZero())
		{
			if (USceneThumbnailInfo* ThumbnailInfo = GetSceneThumbnailInfo())
			{
				const bool bLeftMouse = MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton);
				const bool bRightMouse = MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton);

				if (bLeftMouse)
				{
					ThumbnailInfo->OrbitYaw += -MouseEvent.GetCursorDelta().X;
					ThumbnailInfo->OrbitPitch += -MouseEvent.GetCursorDelta().Y;

					// Normalize the values
					if (ThumbnailInfo->OrbitYaw > 180)
					{
						ThumbnailInfo->OrbitYaw -= 360;
					}
					else if (ThumbnailInfo->OrbitYaw < -180)
					{
						ThumbnailInfo->OrbitYaw += 360;
					}
					
					if (ThumbnailInfo->OrbitPitch > 90)
					{
						ThumbnailInfo->OrbitPitch = 90;
					}
					else if (ThumbnailInfo->OrbitPitch < -90)
					{
						ThumbnailInfo->OrbitPitch = -90;
					}
				}
				else if (bRightMouse)
				{
					// Since zoom is a modifier of on the camera distance from the bounding sphere of the object, it is normalized in the thumbnail preview scene.
					ThumbnailInfo->OrbitZoom += MouseEvent.GetCursorDelta().Y;
				}

				// Dirty the package when the mouse is released
				bModifiedThumbnailWhileDragging = true;
			}
		}

		// Refresh the thumbnail. Do this even if the mouse did not move in case the thumbnail varies with time.
		if (AssetThumbnail.IsValid())
		{
			AssetThumbnail.Pin()->RefreshThumbnail();
		}

		return FReply::Handled().PreventThrottling();
	}

	return FReply::Unhandled();
}

FCursorReply SAssetThumbnailEditModeTools::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	return HasMouseCapture() ? 
		FCursorReply::Cursor(EMouseCursor::None) :
		FCursorReply::Cursor(EMouseCursor::Default);
}

bool SAssetThumbnailEditModeTools::IsEditingThumbnail() const
{
	return bIsEditing;
}

USceneThumbnailInfo* SAssetThumbnailEditModeTools::GetSceneThumbnailInfo() const
{
	USceneThumbnailInfo* SceneThumbnailInfo = SceneThumbnailInfoPtr.Get();
	
	if (!SceneThumbnailInfo)
	{
		if (AssetThumbnail.IsValid())
		{
			if (UObject* Asset = AssetThumbnail.Pin()->GetAsset())
			{
				static const FName AssetToolsName("AssetTools");
				FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(AssetToolsName);
				TWeakPtr<IAssetTypeActions> AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(Asset->GetClass());
				if (AssetTypeActions.IsValid())
				{
					SceneThumbnailInfo = Cast<USceneThumbnailInfo>(AssetTypeActions.Pin()->GetThumbnailInfo(Asset));
				}
			}
		}
	}

	return SceneThumbnailInfo;
}

USceneThumbnailInfoWithPrimitive* SAssetThumbnailEditModeTools::GetSceneThumbnailInfoWithPrimitive() const
{
	return Cast<USceneThumbnailInfoWithPrimitive>(GetSceneThumbnailInfo());
}

#undef LOCTEXT_NAMESPACE
