// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/SCompoundWidget.h"

class FAssetThumbnail;
class USceneThumbnailInfo;
class USceneThumbnailInfoWithPrimitive;
struct FSlateBrush;

// This is the equivalent of the ContentBrowser one, with some Graphic changes.
// The ContentBrowser one will be later removed when the new style will be enabled by default.

template <>
struct TWidgetTypeTraits<class SAssetThumbnailEditModeTools>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * AssetThumbnail EditMode widget, used for the Edit mode of the thumbnails (Ex: changing Shape/Angle)
 */
class SAssetThumbnailEditModeTools : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAssetThumbnailEditModeTools)
		: _SmallView(false)
	{}
		
	SLATE_ARGUMENT(bool, SmallView)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const TSharedPtr<FAssetThumbnail>& InAssetThumbnail);

	// SWidget Interface
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;

	/** True if currently editing the thumbnail, false otherwise */
	bool IsEditingThumbnail() const;

protected:
	/** Gets the visibility for the primitives toolbar */
	EVisibility GetPrimitiveToolsVisibility() const;

	/** Gets the visibility for the the thumbnail reset to default button */
	EVisibility GetPrimitiveToolsResetToDefaultVisibility() const;

	/** Gets the brush used to display the currently used primitive */
	const FSlateBrush* GetCurrentPrimitiveBrush() const;

	/** Sets the primitive type for the supplied thumbnail, if possible */
	FReply ChangePrimitive();

	/** Resets the primitive to default */
	FReply ResetToDefault();

	/** Helper accessors for ThumbnailInfo objects */
	USceneThumbnailInfo* GetSceneThumbnailInfo() const;
	USceneThumbnailInfoWithPrimitive* GetSceneThumbnailInfoWithPrimitive() const;

protected:
	bool bInSmallView = false;
	bool bModifiedThumbnailWhileDragging = false;
	FIntPoint DragStartLocation = FIntPoint(0, 0);
	TWeakPtr<FAssetThumbnail> AssetThumbnail;

private:
	/** True if currently editing the thumbnail, false otherwise */
	bool bIsEditing = false;

	// Never access this directly.
	mutable TWeakObjectPtr<USceneThumbnailInfo> SceneThumbnailInfoPtr;
};
