// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCSVImportOptions.h"

#include "DataTableEditorUtils.h"
#include "DetailsViewArgs.h"
#include "Engine/DataTable.h"
#include "Framework/Views/TableViewTypeTraits.h"
#include "IDetailsView.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "ObjectEditorUtils.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "SPrimaryButton.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Types/SlateStructs.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealType.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

class UScriptStruct;

#define LOCTEXT_NAMESPACE "CSVImportFactory"

void SCSVImportOptions::Construct(const FArguments& InArgs)
{
	WidgetWindow = InArgs._WidgetWindow;
	TempImportDataTable = InArgs._TempImportDataTable;
	bReimportingCurveTable = InArgs._bReimportingCurveTable;

	// Make array of enum pointers
	TSharedPtr<ECSVImportType> DataTableTypePtr = MakeShareable(new ECSVImportType(ECSVImportType::ECSV_DataTable));
	ImportTypes.Add( DataTableTypePtr );
	TSharedPtr<ECSVImportType> CurveTableTypePtr = MakeShareable(new ECSVImportType(ECSVImportType::ECSV_CurveTable));
	ImportTypes.Add(CurveTableTypePtr);
	ImportTypes.Add(MakeShareable(new ECSVImportType(ECSVImportType::ECSV_CurveFloat)));
	ImportTypes.Add(MakeShareable(new ECSVImportType(ECSVImportType::ECSV_CurveVector)));
	ImportTypes.Add(MakeShareable(new ECSVImportType(ECSVImportType::ECSV_CurveLinearColor)));

	// Populate the valid interpolation modes
	CurveInterpModes.Add( MakeShareable( new ERichCurveInterpMode(ERichCurveInterpMode::RCIM_Constant) ) );
	CurveInterpModes.Add( MakeShareable( new ERichCurveInterpMode(ERichCurveInterpMode::RCIM_Linear) ) );
	CurveInterpModes.Add( MakeShareable( new ERichCurveInterpMode(ERichCurveInterpMode::RCIM_Cubic) ) );

	// Create properties view
	FPropertyEditorModule & EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	PropertyView = EditModule.CreateDetailView(DetailsViewArgs);

	PropertyView->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateLambda([](const FPropertyAndParent& InPropertyAndParent)
	{
		static FName ImportOptions = FName(TEXT("ImportOptions"));

		// Only show import options
		FName CategoryName = FObjectEditorUtils::GetCategoryFName(&InPropertyAndParent.Property);

		if (CategoryName == ImportOptions)
		{
			return true;
		}

		return false;
	}));

	RowStructCombo = FDataTableEditorUtils::MakeRowStructureComboBox(FDataTableEditorUtils::FOnDataTableStructSelected::CreateSP(this, &SCSVImportOptions::OnStructSelected));
	RowStructCombo->SetVisibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &SCSVImportOptions::GetTableRowOptionVis)));

	// Create widget
	this->ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush(TEXT("Brushes.Panel")))
		.Padding(10)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBox)
				.Padding(FMargin(3))
				.Visibility( InArgs._FullPath.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible )
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("Import_CurrentFileTitle", "Current File "))
					]
					+SHorizontalBox::Slot()
					.Padding(5, 0, 0, 0)
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(InArgs._FullPath)
					]
				]
			]

			// Import type
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2)
			[
				SNew(STextBlock)
				.Text( LOCTEXT("ChooseAssetType", "Import As") )
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(ImportTypeCombo, SComboBox< TSharedPtr<ECSVImportType> >)
				.OptionsSource( &ImportTypes )
				// If we are reimporting a curve table, we do not want to let the user pick a different import type
				.IsEnabled(!bReimportingCurveTable)
				.OnGenerateWidget( this, &SCSVImportOptions::MakeImportTypeItemWidget )
				.OnSelectionChanged( this, &SCSVImportOptions::OnImportTypeSelected)
				[
					SNew(STextBlock)
					.Text(this, &SCSVImportOptions::GetSelectedItemText)
				]
			]
			// Data row struct
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2)
			[
				SNew(STextBlock)
				.Text( LOCTEXT("ChooseRowType", "Choose DataTable Row Type") )
				.Visibility( this, &SCSVImportOptions::GetTableRowOptionVis )
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				RowStructCombo.ToSharedRef()
			]
			// Curve interpolation
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2)
			[
				SNew(STextBlock)
				.Text( LOCTEXT("ChooseCurveType", "Choose Curve Interpolation Type") )
				.Visibility( this, &SCSVImportOptions::GetCurveTypeVis )
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(CurveInterpCombo, SComboBox<CurveInterpModePtr>)
				.OptionsSource( &CurveInterpModes )
				.OnGenerateWidget( this, &SCSVImportOptions::MakeCurveTypeWidget )
				.InitiallySelectedItem( CurveInterpModes[0] )
				.Visibility( this, &SCSVImportOptions::GetCurveTypeVis )
				[
					SNew(STextBlock)
					.Text(this, &SCSVImportOptions::GetSelectedCurveTypeText)
				]
			]
			// Details panel
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2)
			[
				SNew(SBox)
				.WidthOverride(400.0f)
				.Visibility(this, &SCSVImportOptions::GetDetailsPanelVis)
				[
					PropertyView.ToSharedRef()
				]
			]
			// Apply/Apply to All/Cancel
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.Padding(2)
				[
					SNew(SPrimaryButton)
					.Text(LOCTEXT("Import", "Apply"))
					.OnClicked( this, &SCSVImportOptions::OnImport )
					.IsEnabled( this, &SCSVImportOptions::CanImport )
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2)
				[
					SNew(SButton)
					.Text(LOCTEXT("ImportAll", "Apply to All"))
					.OnClicked( this, &SCSVImportOptions::OnImportAll )
					.IsEnabled( this, &SCSVImportOptions::CanImport )
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2)
				[
					SNew(SButton)
					.Text(LOCTEXT("Cancel", "Cancel"))
					.OnClicked( this, &SCSVImportOptions::OnCancel )
				]
			]
		]
	];

	// set-up selection
	ImportTypeCombo->SetSelectedItem(bReimportingCurveTable ? CurveTableTypePtr : DataTableTypePtr);
	PropertyView->SetObject(TempImportDataTable.Get());
}

	/** If we should import */
bool SCSVImportOptions::ShouldImport()
{
	return ((SelectedStruct != nullptr) || GetSelectedImportType() != ECSVImportType::ECSV_DataTable) && (UserDlgResponse != ECSVImportOptionDlgResponse::Cancel);
}

bool SCSVImportOptions::ShouldImportAll()
{
	return UserDlgResponse == ECSVImportOptionDlgResponse::ImportAll;
}

/** Get the row struct we selected */
UScriptStruct* SCSVImportOptions::GetSelectedRowStruct()
{
	return SelectedStruct;
}

/** Get the import type we selected */
ECSVImportType SCSVImportOptions::GetSelectedImportType()
{
	return SelectedImportType;
}

/** Get the interpolation mode we selected */
ERichCurveInterpMode SCSVImportOptions::GetSelectedCurveIterpMode()
{
	return SelectedCurveInterpMode;
}
	
/** Whether to show table row options */
EVisibility SCSVImportOptions::GetTableRowOptionVis() const
{
	return (ImportTypeCombo.IsValid() && *ImportTypeCombo->GetSelectedItem() == ECSVImportType::ECSV_DataTable) ? EVisibility::Visible : EVisibility::Collapsed;
}

/** Whether to show table row options */
EVisibility SCSVImportOptions::GetCurveTypeVis() const
{
	return (ImportTypeCombo.IsValid() && *ImportTypeCombo->GetSelectedItem() == ECSVImportType::ECSV_CurveTable) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SCSVImportOptions::GetDetailsPanelVis() const
{
	return (ImportTypeCombo.IsValid() && *ImportTypeCombo->GetSelectedItem() == ECSVImportType::ECSV_DataTable) ? EVisibility::Visible : EVisibility::Hidden;
}

FString SCSVImportOptions::GetImportTypeText(TSharedPtr<ECSVImportType> Type) const
{
	FString EnumString;
	if (*Type == ECSVImportType::ECSV_DataTable)
	{
		EnumString = TEXT("DataTable");
	}
	else if (*Type == ECSVImportType::ECSV_CurveTable)
	{
		EnumString = TEXT("CurveTable");
	}
	else if (*Type == ECSVImportType::ECSV_CurveFloat)
	{
		EnumString = TEXT("Float Curve");
	}
	else if (*Type == ECSVImportType::ECSV_CurveVector)
	{
		EnumString = TEXT("Vector Curve");
	}
	else if (*Type == ECSVImportType::ECSV_CurveLinearColor)
	{
		EnumString = TEXT("Linear Color Curve");
	}
	return EnumString;
}

/** Called to create a widget for each struct */
TSharedRef<SWidget> SCSVImportOptions::MakeImportTypeItemWidget(TSharedPtr<ECSVImportType> Type)
{
	return	SNew(STextBlock)
			.Text(FText::FromString(GetImportTypeText(Type)));
}

void SCSVImportOptions::OnImportTypeSelected(TSharedPtr<ECSVImportType> Selection, ESelectInfo::Type SelectionType)
{
	if (Selection.IsValid() && *Selection == ECSVImportType::ECSV_DataTable)
	{
		PropertyView->SetObject(TempImportDataTable.Get());
	}
	else
	{
		PropertyView->SetObject(nullptr);
	}
}

void SCSVImportOptions::OnStructSelected(UScriptStruct* NewStruct)
{
	SelectedStruct = NewStruct;
}

FString SCSVImportOptions::GetCurveTypeText(CurveInterpModePtr InterpMode) const
{
	FString EnumString;

	switch(*InterpMode)
	{
		case ERichCurveInterpMode::RCIM_Constant : 
			EnumString = TEXT("Constant");
			break;

		case ERichCurveInterpMode::RCIM_Linear : 
			EnumString = TEXT("Linear");
			break;

		case ERichCurveInterpMode::RCIM_Cubic : 
			EnumString = TEXT("Cubic");
			break;
	}
	return EnumString;
}

/** Called to create a widget for each curve interpolation enum */
TSharedRef<SWidget> SCSVImportOptions::MakeCurveTypeWidget(CurveInterpModePtr InterpMode)
{
	FString Label = GetCurveTypeText(InterpMode);
	return SNew(STextBlock) .Text( FText::FromString(Label) );
}

/** Called when 'OK' button is pressed */
FReply SCSVImportOptions::OnImport()
{
	UserDlgResponse = ECSVImportOptionDlgResponse::Import;
	return HandleImport();
}

bool SCSVImportOptions::CanImport() const
{
	ECSVImportType ImportType = *ImportTypeCombo->GetSelectedItem();

	switch (ImportType)
	{
	case ECSVImportType::ECSV_DataTable:
		return SelectedStruct != nullptr;
	case ECSVImportType::ECSV_CurveTable:
		return CurveInterpCombo->GetSelectedItem().IsValid();
	case ECSVImportType::ECSV_CurveFloat:
	case ECSVImportType::ECSV_CurveVector:
	case ECSVImportType::ECSV_CurveLinearColor:
		return true;
	}
	
	return false;
}

/** Called when 'Cancel' button is pressed */
FReply SCSVImportOptions::OnCancel()
{
	UserDlgResponse = ECSVImportOptionDlgResponse::Cancel;
	if (WidgetWindow.IsValid())
	{
		WidgetWindow.Pin()->RequestDestroyWindow();
	}
	return FReply::Handled();
}

FText SCSVImportOptions::GetSelectedItemText() const
{
	TSharedPtr<ECSVImportType> SelectedType = ImportTypeCombo->GetSelectedItem();

	return (SelectedType.IsValid())
		? FText::FromString(GetImportTypeText(SelectedType))
		: FText::GetEmpty();
}

FText SCSVImportOptions::GetSelectedCurveTypeText() const
{
	CurveInterpModePtr CurveModePtr = CurveInterpCombo->GetSelectedItem();
	return (CurveModePtr.IsValid())
		? FText::FromString(GetCurveTypeText(CurveModePtr))
		: FText::GetEmpty();
}

FReply SCSVImportOptions::HandleImport()
{
	SelectedImportType = *ImportTypeCombo->GetSelectedItem();
	if (CurveInterpCombo->GetSelectedItem().IsValid())
	{
		SelectedCurveInterpMode = *CurveInterpCombo->GetSelectedItem();
	}
	if (WidgetWindow.IsValid())
	{
		WidgetWindow.Pin()->RequestDestroyWindow();
	}
	return FReply::Handled();
}

FReply SCSVImportOptions::OnImportAll()
{
	UserDlgResponse = ECSVImportOptionDlgResponse::ImportAll;
	return HandleImport();
}

#undef LOCTEXT_NAMESPACE
