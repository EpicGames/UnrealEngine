// Copyright Epic Games, Inc. All Rights Reserved.


#include "Kismet2/SClassPickerDialog.h"
#include "Misc/MessageDialog.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "Components/ActorComponent.h"
#include "Engine/Blueprint.h"
#include "Editor/UnrealEdEngine.h"
#include "Editor.h"
#include "UnrealEdGlobals.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "SClassViewer.h"
#include "ClassViewerFilter.h"
#include "EditorClassUtils.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Styling/SlateIconFinder.h"
#include "SPrimaryButton.h"

#define LOCTEXT_NAMESPACE "SClassPicker"

void SClassPickerDialog::Construct(const FArguments& InArgs)
{
	WeakParentWindow = InArgs._ParentWindow;
	bAllowNone = InArgs._Options.bShowNoneOption;

	bPressedOk = false;
	ChosenClass = NULL;

	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	if (InArgs._Options.bShowClassesViewer)
	{
		ClassViewer = StaticCastSharedRef<SClassViewer>(ClassViewerModule.CreateClassViewer(InArgs._Options, FOnClassPicked::CreateSP(this, &SClassPickerDialog::OnClassPicked)));
	}

	if (InArgs._Options.bShowDefaultClasses)
	{
		const TSharedPtr<IClassViewerFilter>& GlobalClassFilter = ClassViewerModule.GetGlobalClassViewerFilter();
		TSharedRef<FClassViewerFilterFuncs> ClassFilterFuncs = ClassViewerModule.CreateFilterFuncs();

		// Load in default settings
		for (const FClassPickerDefaults& DefaultObj : GUnrealEd->GetUnrealEdOptions()->GetNewAssetDefaultClasses())
		{
			{
				UClass* AssetType = LoadClass<UObject>(NULL, *DefaultObj.AssetClass, NULL, LOAD_None, NULL);

				if (!InArgs._AssetType->IsChildOf(AssetType))
				{
					continue;
				}

				if (InArgs._Options.bEditorClassesOnly && !IsEditorOnlyObject(AssetType))
				{
					// Don't add if we are looking for editor classes only and this isn't an editor only class
					continue;
				}
			}

			{
				UClass* Class = LoadClass<UObject>(NULL, *DefaultObj.ClassName, NULL, LOAD_None, NULL);

				if (Class && GlobalClassFilter && !GlobalClassFilter->IsClassAllowed(InArgs._Options, Class, ClassFilterFuncs))
				{
					continue;
				}
			}

			AssetDefaultClasses.Add(MakeShareable(new FClassPickerDefaults(DefaultObj)));
		}

		for (UClass* CommonClass : InArgs._Options.ExtraPickerCommonClasses)
		{
			if (GlobalClassFilter && !GlobalClassFilter->IsClassAllowed(InArgs._Options, CommonClass, ClassFilterFuncs))
			{
				continue;
			}

			TSharedPtr<FClassPickerDefaults> PickerDefault = MakeShareable(new FClassPickerDefaults());
			PickerDefault->AssetClass = InArgs._AssetType->GetPathName();
			PickerDefault->ClassName  = CommonClass->GetPathName();

			AssetDefaultClasses.Add(PickerDefault);
		}
	}

	const bool bHasDefaultClasses = AssetDefaultClasses.Num() > 0;

	TSharedPtr<SListView<TSharedPtr<FClassPickerDefaults>>> DefaultClassViewer;
	if (bHasDefaultClasses)
	{
		SAssignNew(DefaultClassViewer, SListView < TSharedPtr<FClassPickerDefaults> >)
			.SelectionMode(ESelectionMode::None)
			.ListItemsSource(&AssetDefaultClasses)
			.OnGenerateRow(this, &SClassPickerDialog::GenerateListRow);
	}

	bool bExpandDefaultClassPicker = true;
	bool bExpandCustomClassPicker = !bHasDefaultClasses;
	if (bHasDefaultClasses && ClassViewer)
	{
		GConfig->GetBool(TEXT("/Script/UnrealEd.UnrealEdOptions"), TEXT("bExpandClassPickerDefaultClassList"), bExpandDefaultClassPicker, GEditorIni);
		GConfig->GetBool(TEXT("/Script/UnrealEd.UnrealEdOptions"), TEXT("bExpandCustomClassPickerClassList"), bExpandCustomClassPicker, GEditorIni);
	}

	ensureMsgf(ClassViewer || DefaultClassViewer, TEXT("Either ClassViewer or DefaultClassViewer should be used."));

	TSharedRef<SVerticalBox> Container = SNew(SVerticalBox);

	if (DefaultClassViewer)
	{
		Container->AddSlot()
		.AutoHeight()
		[
			SNew(SExpandableArea)
			.BorderImage(FStyleDefaults::GetNoBrush())
			.BodyBorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
			.HeaderPadding(FMargin(5.0f, 3.0f))
			.AllowAnimatedTransition(false)
			.InitiallyCollapsed(!bExpandDefaultClassPicker)
			.OnAreaExpansionChanged(this, &SClassPickerDialog::OnDefaultAreaExpansionChanged)
			.HeaderContent()
			[
				SNew(STextBlock)
				.Text(NSLOCTEXT("SClassPickerDialog", "CommonClassesAreaTitle", "Common"))
				.TextStyle(FAppStyle::Get(), "ButtonText")
				.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
				.TransformPolicy(ETextTransformPolicy::ToUpper)
			]
			.BodyContent()
			[
				DefaultClassViewer.ToSharedRef()
			]
		];
	}

	if (ClassViewer)
	{
		FMargin Padding = DefaultClassViewer ? FMargin(0.0f, 10.0f, 0.0f, 0.0f) : FMargin();
		Container->AddSlot()
		.AutoHeight()
		.Padding(Padding)
		[
			SNew(SExpandableArea)
			.BorderImage(FStyleDefaults::GetNoBrush())
			.BodyBorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
			.HeaderPadding(FMargin(5.0f, 3.0f))
			.AllowAnimatedTransition(false)
			.MaxHeight(320.f)
			.InitiallyCollapsed(!bExpandCustomClassPicker)
			.OnAreaExpansionChanged(this, &SClassPickerDialog::OnCustomAreaExpansionChanged)
			.HeaderContent()
			[
				SNew(STextBlock)
				.Text(NSLOCTEXT("SClassPickerDialog", "AllClassesAreaTitle", "All Classes"))
				.TextStyle(FAppStyle::Get(), "ButtonText")
				.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
				.TransformPolicy(ETextTransformPolicy::ToUpper)
			]
			.BodyContent()
			[
				ClassViewer.ToSharedRef()
			]
		];
	}

	Container->AddSlot()
	.HAlign(HAlign_Right)
	.VAlign(VAlign_Bottom)
	.Padding(8.f)
	[
		SNew(SUniformGridPanel)
		.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
		+SUniformGridPanel::Slot(0,0)
		[
			SNew(SPrimaryButton)
			.Text(NSLOCTEXT("SClassPickerDialog", "ClassPickerSelectButton", "Select"))
			.Visibility( this, &SClassPickerDialog::GetSelectButtonVisibility )
			.OnClicked(this, &SClassPickerDialog::OnClassPickerConfirmed)
		]
		+SUniformGridPanel::Slot(1,0)
		[
			SNew(SButton)
			.Text(NSLOCTEXT("SClassPickerDialog", "ClassPickerCancelButton", "Cancel"))
			.HAlign(HAlign_Center)
			.OnClicked(this, &SClassPickerDialog::OnClassPickerCanceled)
		]
	];

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Brushes.Panel"))
		[
			SNew(SBox)
			.WidthOverride(610.0f)
			[
				Container
			]
		]
	];

	if (WeakParentWindow.IsValid())
	{
		if (bExpandCustomClassPicker && ClassViewer)
		{
			WeakParentWindow.Pin().Get()->SetWidgetToFocusOnActivate(ClassViewer);
		}
		else if (DefaultClassViewer)
		{
			WeakParentWindow.Pin().Get()->SetWidgetToFocusOnActivate(DefaultClassViewer);
		}
	}
}

bool SClassPickerDialog::PickClass(const FText& TitleText, const FClassViewerInitializationOptions& ClassViewerOptions, UClass*& OutChosenClass, UClass* AssetType)
{
	// Create the window to pick the class
	TSharedRef<SWindow> PickerWindow = SNew(SWindow)
		.Title(TitleText)
		.SizingRule( ESizingRule::Autosized )
		.ClientSize( FVector2D( 0.f, 300.f ))
		.SupportsMaximize(false)
		.SupportsMinimize(false);

	TSharedRef<SClassPickerDialog> ClassPickerDialog = SNew(SClassPickerDialog)
		.ParentWindow(PickerWindow)
		.Options(ClassViewerOptions)
		.AssetType(AssetType);

	PickerWindow->SetContent(ClassPickerDialog);

	GEditor->EditorAddModalWindow(PickerWindow);

	if (ClassPickerDialog->bPressedOk)
	{
		OutChosenClass = ClassPickerDialog->ChosenClass;
		return true;
	}
	else
	{
		// Ok was not selected, NULL the class
		OutChosenClass = NULL;
		return false;
	}
}

void SClassPickerDialog::OnClassPicked(UClass* InChosenClass)
{
	ChosenClass = InChosenClass;
}

TSharedRef<ITableRow> SClassPickerDialog::GenerateListRow(TSharedPtr<FClassPickerDefaults> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	FClassPickerDefaults* Obj = InItem.Get();
	UClass* ItemClass = LoadClass<UObject>(NULL, *Obj->ClassName, NULL, LOAD_None, NULL);
	const FSlateBrush* ItemBrush = FSlateIconFinder::FindIconBrushForClass(ItemClass);

	TSharedRef<SWidget> DocumentationWidget = FEditorClassUtils::GetDocumentationLinkWidget(ItemClass);
	
	return 
	SNew(STableRow< TSharedPtr<FClassPickerDefaults> >, OwnerTable)
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.MaxHeight(30.0f)
		.Padding(10.0f, 6.0f, 0.0f, 4.0f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.FillWidth(0.34f)
			[
				SNew(SButton)
				.OnClicked(this, &SClassPickerDialog::OnDefaultClassPicked, ItemClass)
				.ToolTipText(Obj->GetName())
				.Content()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(ItemBrush)
					]
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(4.0f, 0.0f)
					[
						SNew(STextBlock)
						.Text(Obj->GetName())
						.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
					]
				]
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(10.0f, 0.0f, 4.0f, 0.0f)
			.FillWidth(0.59f)
			[
				SNew(STextBlock)
				.Text(Obj->GetDescription())
				.ToolTipText(Obj->GetDescription())
				.AutoWrapText(true)
				.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.FillWidth(0.07f)
			[
				SNew(SBox)
				[
					DocumentationWidget
				]
			]
		]
	];
}

FReply SClassPickerDialog::OnDefaultClassPicked(UClass* InChosenClass)
{
	ChosenClass = InChosenClass;
	bPressedOk = true;
	if (WeakParentWindow.IsValid())
	{
		WeakParentWindow.Pin()->RequestDestroyWindow();
	}
	return FReply::Handled();
}

FReply SClassPickerDialog::OnClassPickerConfirmed()
{
	if (!bAllowNone && ChosenClass == NULL)
	{
		FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("EditorFactories", "MustChooseClassWarning", "You must choose a class."));
	}
	else
	{
		bPressedOk = true;

		if (WeakParentWindow.IsValid())
		{
			WeakParentWindow.Pin()->RequestDestroyWindow();
		}
	}
	return FReply::Handled();
}

FReply SClassPickerDialog::OnClassPickerCanceled()
{
	if (WeakParentWindow.IsValid())
	{
		WeakParentWindow.Pin()->RequestDestroyWindow();
	}
	return FReply::Handled();
}

void SClassPickerDialog::OnDefaultAreaExpansionChanged(bool bExpanded)
{
	if (bExpanded && WeakParentWindow.IsValid() && ClassViewer)
	{
		WeakParentWindow.Pin().Get()->SetWidgetToFocusOnActivate(ClassViewer);
	}

	if (AssetDefaultClasses.Num() > 0)
	{
		GConfig->SetBool(TEXT("/Script/UnrealEd.UnrealEdOptions"), TEXT("bExpandClassPickerDefaultClassList"), bExpanded, GEditorIni);
	}
}

void SClassPickerDialog::OnCustomAreaExpansionChanged(bool bExpanded)
{
	check(ClassViewer);
	if (bExpanded && WeakParentWindow.IsValid())
	{
		WeakParentWindow.Pin().Get()->SetWidgetToFocusOnActivate(ClassViewer);
	}

	if (AssetDefaultClasses.Num() > 0)
	{
		GConfig->SetBool(TEXT("/Script/UnrealEd.UnrealEdOptions"), TEXT("bExpandCustomClassPickerClassList"), bExpanded, GEditorIni);
	}
}

EVisibility SClassPickerDialog::GetSelectButtonVisibility() const
{
	EVisibility ButtonVisibility = EVisibility::Hidden;
	if(bAllowNone || ChosenClass != nullptr )
	{
		ButtonVisibility = EVisibility::Visible;
	}
	return ButtonVisibility;
}

/** Overridden from SWidget: Called when a key is pressed down - capturing copy */
FReply SClassPickerDialog::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	if (ClassViewer)
	{
		WeakParentWindow.Pin().Get()->SetWidgetToFocusOnActivate(ClassViewer);
	}

	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		return OnClassPickerCanceled();
	}
	else if (InKeyEvent.GetKey() == EKeys::Enter)
	{
		OnClassPickerConfirmed();
	}
	else if (ClassViewer)
	{
		return ClassViewer->OnKeyDown(MyGeometry, InKeyEvent);
	}
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
