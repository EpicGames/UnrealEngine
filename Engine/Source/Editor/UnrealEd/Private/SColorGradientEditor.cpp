// Copyright Epic Games, Inc. All Rights Reserved.

#include "SColorGradientEditor.h"
#include "Fonts/SlateFontInfo.h"
#include "Misc/Paths.h"
#include "Rendering/DrawElementPayloads.h"
#include "Rendering/DrawElements.h"
#include "Widgets/SBoxPanel.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Styling/CoreStyle.h"
#include "Styling/AppStyle.h"
#include "Editor.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Colors/SColorPicker.h"
#include "SCurveEditor.h"
#include "ScopedTransaction.h"
#include "Misc/Optional.h"

#define LOCTEXT_NAMESPACE "SColorGradientEditor"

struct FColorGradientEditorGeometrySolver
{
	/** Constructs the solver with or without expecting an alpha channel */
	FColorGradientEditorGeometrySolver(const bool bInWithAlphaChannel)
		: bWithAlphaChannel(bInWithAlphaChannel)
	{}

	static constexpr float HandleWidth = 13.0f;
	static constexpr float HandleHeight = 16.0f;
	static constexpr float HandleColorWidth = 7.0f;
	static constexpr float HandleColorHeight = 7.0f;
	static constexpr float HandleColorXOffset = (HandleWidth - HandleColorWidth) / 2;
	static constexpr float HandleColorYOffsetForColor = HandleColorXOffset + 1;
	static constexpr float HandleColorYOffsetForAlpha = HandleHeight - HandleColorHeight - HandleColorXOffset - 1;

	const FSlateRect HandleRect = FSlateRect(
		-HandleWidth / 2.0f,
		0.0f,
		HandleWidth / 2.0f,
		HandleHeight);

	FGeometry GetColorMarkAreaPaintGeometry(const FGeometry& WidgetGeometry)
	{
		const float ClampedHandleWidth = FMath::Max(WidgetGeometry.GetLocalSize().X - HandleWidth, 0.f);
		
		return WidgetGeometry.MakeChild(
			FVector2D(ClampedHandleWidth, HandleHeight),
			FSlateLayoutTransform(FVector2D(HandleWidth / 2, 0)));
	}

	FGeometry GetColorMarkAreaMouseGeometry(const FGeometry& WidgetGeometry)
	{
		return WidgetGeometry.MakeChild(
			FVector2D(WidgetGeometry.GetLocalSize().X, HandleHeight),
			FSlateLayoutTransform());
	}

	TOptional<FGeometry> GetAlphaMarkAreaPaintGeometry(const FGeometry& WidgetGeometry)
	{
		if (bWithAlphaChannel)
		{
			return WidgetGeometry.MakeChild(
				FVector2D(WidgetGeometry.GetLocalSize().X - HandleWidth, HandleHeight),
				FSlateLayoutTransform(FVector2D(HandleWidth / 2, WidgetGeometry.GetLocalSize().Y - HandleHeight)));
		}

		return TOptional<FGeometry>();
	}

	TOptional<FGeometry> GetAlphaMarkAreaMouseGeometry(const FGeometry& WidgetGeometry)
	{
		if (bWithAlphaChannel)
		{
			return WidgetGeometry.MakeChild(
				FVector2D(WidgetGeometry.GetLocalSize().X, HandleHeight),
				FSlateLayoutTransform(FVector2D(0.0f, WidgetGeometry.GetLocalSize().Y - HandleHeight)));
		}

		return TOptional<FGeometry>();
	}

	FGeometry GetGradientAreaPaintGeometry(const FGeometry& WidgetGeometry, const bool bHideStops)
	{
		const float HandleWith = bHideStops ? 0.f : HandleWidth;
		const float GradientWidth = WidgetGeometry.GetLocalSize().X - HandleWith;

		const float HandlesHeight = bHideStops ? 0.f : bWithAlphaChannel ? 2.f * HandleHeight : HandleHeight;
		const float GradientHeight = WidgetGeometry.GetLocalSize().Y - HandlesHeight;

		return WidgetGeometry.MakeChild(
			FVector2D(GradientWidth, GradientHeight),
			bHideStops ? FSlateLayoutTransform() : FSlateLayoutTransform(FVector2D(HandleWidth / 2, HandleHeight))
		);
	}

	FTrackScaleInfo GetTimeScaleForWidgetGeometry(const FGeometry& WidgetGeometry, const float ViewMinInput, const float ViewMaxInput, const bool bHideStops)
	{
		return FTrackScaleInfo(ViewMinInput, ViewMaxInput, 0.0f, 1.0f, GetGradientAreaPaintGeometry(WidgetGeometry, bHideStops).GetLocalSize());
	}

	FTrackScaleInfo GetTimeScaleForAreaPaintGeomtry(const FGeometry& AreaGeometry, float ViewMinInput, float ViewMaxInput)
	{
		return FTrackScaleInfo(ViewMinInput, ViewMaxInput, 0.0f, 1.0f, AreaGeometry.GetLocalSize());
	}

	float GetTimeForAbsolutePosition(const FGeometry& WidgetGeometry, const float ViewMinInput, const float ViewMaxInput, bool bHideStops, const FVector2D& AbsolutePosition)
	{
		FGeometry GradientAreaPaintGeometry = GetGradientAreaPaintGeometry(WidgetGeometry, bHideStops);
		FTrackScaleInfo TimeScaleInfo = FTrackScaleInfo(ViewMinInput, ViewMaxInput, 0.0f, 1.0f, GradientAreaPaintGeometry.GetLocalSize());
		return TimeScaleInfo.LocalXToInput(GradientAreaPaintGeometry.AbsoluteToLocal(AbsolutePosition).X);
	}

private:
	const bool bWithAlphaChannel;
};

FGradientStopMark::FGradientStopMark()
{}

FGradientStopMark::FGradientStopMark( float InTime, FKeyHandle InRedKeyHandle, FKeyHandle InGreenKeyHandle, FKeyHandle InBlueKeyHandle, FKeyHandle InAlphaKeyHandle )
	: Time( InTime )
	, RedKeyHandle( InRedKeyHandle )
	, GreenKeyHandle( InGreenKeyHandle )
	, BlueKeyHandle( InBlueKeyHandle )
	, AlphaKeyHandle( InAlphaKeyHandle )
{}

bool FGradientStopMark::operator==( const FGradientStopMark& Other ) const 
{
	return Time == Other.Time && 
		( (RedKeyHandle == Other.RedKeyHandle && GreenKeyHandle == Other.GreenKeyHandle && BlueKeyHandle == Other.BlueKeyHandle ) ||
		( AlphaKeyHandle == Other.AlphaKeyHandle ) );
}

bool FGradientStopMark::IsValid( FCurveOwnerInterface& CurveOwner )
{
	TArray<FRichCurveEditInfo> Curves = CurveOwner.GetCurves();

	return IsValidColorMark( Curves ) || IsValidAlphaMark( Curves );
}

bool FGradientStopMark::IsValidAlphaMark( const TArray<FRichCurveEditInfo>& Curves ) const
{
	const FRealCurve* AlphaCurve = Curves[3].CurveToEdit;

	return AlphaCurve->IsKeyHandleValid( AlphaKeyHandle ) && AlphaCurve->GetKeyTime( AlphaKeyHandle ) == Time;
}

bool FGradientStopMark::IsValidColorMark( const TArray<FRichCurveEditInfo>& Curves ) const
{
	const FRealCurve* RedCurve = Curves[0].CurveToEdit;
	const FRealCurve* GreenCurve = Curves[1].CurveToEdit;
	const FRealCurve* BlueCurve = Curves[2].CurveToEdit;

	return	RedCurve->IsKeyHandleValid( RedKeyHandle ) && RedCurve->GetKeyTime( RedKeyHandle ) == Time &&
			GreenCurve->IsKeyHandleValid( GreenKeyHandle ) && GreenCurve->GetKeyTime( GreenKeyHandle ) == Time &&
			BlueCurve->IsKeyHandleValid( BlueKeyHandle ) && BlueCurve->GetKeyTime( BlueKeyHandle ) == Time;
				
}

FLinearColor FGradientStopMark::GetColor( FCurveOwnerInterface& CurveOwner ) const
{
	return CurveOwner.GetLinearColorValue(Time);
}

float FGradientStopMark::GetTime( FCurveOwnerInterface& CurveOwner ) const
{
	return Time;
}

void FGradientStopMark::SetColor( const FLinearColor& InColor, FCurveOwnerInterface& CurveOwner )
{
	TArray<FRichCurveEditInfo> Curves = CurveOwner.GetCurves();

	// Update the color component on each curve
	if( IsValidColorMark(Curves) )
	{
		FRealCurve* RedCurve = Curves[0].CurveToEdit;
		FRealCurve* GreenCurve = Curves[1].CurveToEdit;
		FRealCurve* BlueCurve = Curves[2].CurveToEdit;

		RedCurve->SetKeyValue( RedKeyHandle, InColor.R );
		GreenCurve->SetKeyValue( GreenKeyHandle, InColor.G );
		BlueCurve->SetKeyValue( BlueKeyHandle, InColor.B );
	}
	else if( IsValidAlphaMark(Curves) )
	{
		FRealCurve* AlphaCurve = Curves[3].CurveToEdit;

		AlphaCurve->SetKeyValue( AlphaKeyHandle, InColor.A );
	}
}

void FGradientStopMark::SetTime( float NewTime, FCurveOwnerInterface& CurveOwner )
{
	TArray<FRichCurveEditInfo> Curves = CurveOwner.GetCurves();

	// Update the time on each curve
	if( IsValidColorMark(Curves) )
	{
		FRealCurve* RedCurve = Curves[0].CurveToEdit;
		FRealCurve* GreenCurve = Curves[1].CurveToEdit;
		FRealCurve* BlueCurve = Curves[2].CurveToEdit;

		RedCurve->SetKeyTime( RedKeyHandle, NewTime );
		GreenCurve->SetKeyTime( GreenKeyHandle, NewTime );
		BlueCurve->SetKeyTime( BlueKeyHandle, NewTime );
	}
	else if( IsValidAlphaMark(Curves) )
	{
		FRealCurve* AlphaCurve = Curves[3].CurveToEdit;

		AlphaCurve->SetKeyTime( AlphaKeyHandle, NewTime );
	} 

	Time = NewTime;
}

SColorGradientEditor::SColorGradientEditor()
	: DesiredSizeOverride(*this)
{}

bool SColorGradientEditor::ShouldHideStops() const
{
	return (bRequestHideStops.Get() && !IsEditingEnabled.Get());
}

void SColorGradientEditor::Construct( const FArguments& InArgs )
{
	IsEditingEnabled = InArgs._IsEditingEnabled;
	CurveOwner = NULL;
	ViewMinInput = InArgs._ViewMinInput;
	ViewMaxInput = InArgs._ViewMaxInput;
	bClampStopsToViewRange = InArgs._ClampStopsToViewRange;
	bDraggingAlphaValue = false;
	bDrawColorAndAlphaSeparate = InArgs._WithAlphaChannel && InArgs._DrawColorAndAlphaSeparate;
	bDraggingStop = false;
	DistanceDragged = 0.0f;
	ContextMenuPosition = FVector2D::ZeroVector;
	DesiredSizeOverride.Assign(*this, InArgs._DesiredSizeOverride);
	bRequestHideStops = InArgs._HideStops;

	GeometrySolver = MakePimpl<FColorGradientEditorGeometrySolver>(InArgs._WithAlphaChannel);
}

int32 SColorGradientEditor::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	const TSharedRef< FSlateFontMeasure > FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	if( CurveOwner )
	{
		const bool bHideStops = ShouldHideStops();

		// Split the geometry into areas for stops and the gradient
		const FGeometry ColorMarkAreaGeometry = GeometrySolver->GetColorMarkAreaPaintGeometry( AllottedGeometry );
		const FGeometry GradientAreaGeometry = GeometrySolver->GetGradientAreaPaintGeometry(AllottedGeometry, bHideStops);
		const TOptional<FGeometry> AlphaMarkAreaGeometry = GeometrySolver->GetAlphaMarkAreaPaintGeometry( AllottedGeometry );

		bool bEnabled = ShouldBeEnabled( bParentEnabled );
		ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

		// Pixel to value input converter
		constexpr bool bWithHideStops = false;
		const FTrackScaleInfo ScaleInfo = GeometrySolver->GetTimeScaleForWidgetGeometry(AllottedGeometry, ViewMinInput.Get(), ViewMaxInput.Get(), bWithHideStops);

		// The start and end location in slate units of the area to draw
		int32 Start = 0;
		int32 Finish = FMath::TruncToInt( GradientAreaGeometry.GetLocalSize().X );

		TArray<FSlateGradientStop> ColorStops;
		TArray<FSlateGradientStop> AlphaStops;

		// If no alpha keys are available, treat the curve as being completely opaque for drawing purposes
		const bool bHasAnyAlphaKeys = CurveOwner->HasAnyAlphaKeys(); 

		// If any transparency (A < 1) is found, we'll draw a checkerboard to visualize the color with alpha
		bool bHasTransparency = false;
		static const FSlateBrush* WhiteBrush = FAppStyle::GetBrush("WhiteBrush");

		if (bColorAreaHovered)
		{
			// Draw a checkerboard behind there is any transparency visible
			FSlateDrawElement::MakeBox
			(
				OutDrawElements,
				LayerId,
				ColorMarkAreaGeometry.ToPaintGeometry(),
				WhiteBrush,
				DrawEffects,
				FLinearColor(.5f, .5f, .5f, .15f)
				);
			++LayerId;
		}
		else if (AlphaMarkAreaGeometry.IsSet() && bAlphaAreaHovered)
		{
			// Draw a checkerboard behind there is any transparency visible
			FSlateDrawElement::MakeBox
			(
				OutDrawElements,
				LayerId,
				AlphaMarkAreaGeometry.GetValue().ToPaintGeometry(),
				WhiteBrush,
				DrawEffects,
				FLinearColor(.5f, .5f, .5f, .15f)
			);
			++LayerId;
		}
			
		// Sample the curve every 2 units.  THe curve could be non-linear so sampling at each stop would display an incorrect gradient
		for (int32 CurrentStep = Start; CurrentStep < Finish; CurrentStep += 2)
		{
			// Figure out the time from the current screen unit
			const float Time = ScaleInfo.LocalXToInput(CurrentStep);

			// Sample the curve
			FLinearColor Color = CurveOwner->GetLinearColorValue(Time);
			if (!bHasAnyAlphaKeys)
			{
				// Only show alpha if there is at least one key.  For some curves, alpha may not be important
				Color.A = 1.0f;
				bHasTransparency = false;
			}
			else
			{
				bHasTransparency |= (Color.A < 1.0f);
			}

			if (bDrawColorAndAlphaSeparate)
			{
				ColorStops.Emplace(FVector2D(CurrentStep, 0.0f), FLinearColor(Color.R, Color.G, Color.B));
				AlphaStops.Emplace(FVector2D(CurrentStep, 0.0f), FLinearColor(Color.A, Color.A, Color.A));
			}
			else
			{
				ColorStops.Emplace(FVector2D(CurrentStep, 0.0f), Color);
			}
		}

		if (ColorStops.Num() > 0)
		{
			// Draw the color gradient
			if (bDrawColorAndAlphaSeparate)
			{
				const FVector2D GradientLocalSize = GradientAreaGeometry.GetLocalSize();
				const FGeometry ColorGradientAreaGeometry = GradientAreaGeometry.MakeChild(FVector2D(GradientLocalSize.X, GradientLocalSize.Y * 0.70f), FSlateLayoutTransform(FVector2D(0.0f, 0.0f)));
				const FGeometry AlphaGradientAreaGeometry = GradientAreaGeometry.MakeChild(FVector2D(GradientLocalSize.X, GradientLocalSize.Y * 0.30f), FSlateLayoutTransform(FVector2D(0.0f, GradientLocalSize.Y * 0.70f)));

				FSlateDrawElement::MakeGradient
				(
					OutDrawElements,
					LayerId,
					ColorGradientAreaGeometry.ToPaintGeometry(),
					ColorStops,
					Orient_Vertical,
					DrawEffects
				);
				FSlateDrawElement::MakeGradient
				(
					OutDrawElements,
					LayerId,
					AlphaGradientAreaGeometry.ToPaintGeometry(),
					AlphaStops,
					Orient_Vertical,
					DrawEffects
				);
			}
			else
			{
				if (bHasTransparency)
				{
					// Draw a checkerboard behind there is any transparency visible
					FSlateDrawElement::MakeBox
					(
						OutDrawElements,
						LayerId,
						GradientAreaGeometry.ToPaintGeometry(),
						FAppStyle::GetBrush("Checkerboard"),
						DrawEffects
					);
				}

				FSlateDrawElement::MakeGradient
				(
					OutDrawElements,
					LayerId,
					GradientAreaGeometry.ToPaintGeometry(),
					ColorStops,
					Orient_Vertical,
					DrawEffects
				);
			}
		}

		if (!bHideStops)
		{
			// Get actual editable stop marks
			TArray<FGradientStopMark> ColorMarks;
			TArray<FGradientStopMark> AlphaMarks;
			GetGradientStopMarks(ColorMarks, AlphaMarks);

			// Draw each color stop
			for (int32 ColorIndex = 0; ColorIndex < ColorMarks.Num(); ++ColorIndex)
			{
				const FGradientStopMark& Mark = ColorMarks[ColorIndex];

				float XVal = ScaleInfo.InputToLocalX(Mark.Time);

				// Dont draw stops which are not visible
				if (XVal >= 0 && XVal <= ColorMarkAreaGeometry.GetLocalSize().X)
				{
					FLinearColor Color = CurveOwner->GetLinearColorValue(Mark.Time);
					Color.A = 1.0f;
					DrawGradientStopMark(Mark, ColorMarkAreaGeometry, XVal, Color, OutDrawElements, LayerId, MyCullingRect, DrawEffects, true, InWidgetStyle);
				}
			}

			if (AlphaMarkAreaGeometry.IsSet())
			{
				// Draw each alpha stop
				for (int32 ColorIndex = 0; ColorIndex < AlphaMarks.Num(); ++ColorIndex)
				{
					const FGradientStopMark& Mark = AlphaMarks[ColorIndex];

					float XVal = ScaleInfo.InputToLocalX(Mark.Time);

					// Dont draw stops which are not visible
					if (XVal >= 0 && XVal <= AlphaMarkAreaGeometry.GetValue().GetLocalSize().X)
					{
						float Alpha = CurveOwner->GetLinearColorValue(Mark.Time).A;
						DrawGradientStopMark(Mark, AlphaMarkAreaGeometry.GetValue(), XVal, FLinearColor(Alpha, Alpha, Alpha, 1.0f), OutDrawElements, LayerId, MyCullingRect, DrawEffects, false, InWidgetStyle);
					}
				}
			}

			// Draw some hint messages about how to add stops if no stops exist
			if (ColorMarks.Num() == 0 && AlphaMarks.Num() == 0 && IsEditingEnabled.Get() == true)
			{
				static FString GradientColorMessage(LOCTEXT("ClickToAddColorStop", "Click in this area add color stops").ToString());
				static FString GradientAlphaMessage(LOCTEXT("ClickToAddAlphaStop", "Click in this area add opacity stops").ToString());

				// Draw the text centered in the color region
				{
					FVector2D StringSize = FontMeasureService->Measure(GradientColorMessage, FCoreStyle::GetDefaultFontStyle("Regular", 8));
					FPaintGeometry PaintGeom = ColorMarkAreaGeometry.ToPaintGeometry(FSlateLayoutTransform(FVector2D((ColorMarkAreaGeometry.GetLocalSize().X - StringSize.X) * 0.5f, 1.0f)));

					FSlateDrawElement::MakeText
					(
						OutDrawElements,
						LayerId,
						PaintGeom,
						GradientColorMessage,
						FCoreStyle::GetDefaultFontStyle("Regular", 8),
						DrawEffects,
						FLinearColor(.5f, .5f, .5f, .85f)
					);
				}

				// Draw the text centered in the alpha region
				if (AlphaMarkAreaGeometry.IsSet())
				{
					FVector2D StringSize = FontMeasureService->Measure(GradientAlphaMessage, FCoreStyle::GetDefaultFontStyle("Regular", 8));
					const FPaintGeometry PaintGeom = AlphaMarkAreaGeometry.GetValue().ToPaintGeometry(FSlateLayoutTransform(FVector2D((AlphaMarkAreaGeometry.GetValue().GetLocalSize().X - StringSize.X) * 0.5f, 1.0f)));

					FSlateDrawElement::MakeText
					(
						OutDrawElements,
						LayerId,
						PaintGeom,
						GradientAlphaMessage,
						FCoreStyle::GetDefaultFontStyle("Regular", 8),
						DrawEffects,
						FLinearColor(.5f, .5f, .5f, .85f)
					);
				}
			}
		}
	}

	return LayerId;
}

FReply SColorGradientEditor::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if( IsEditingEnabled.Get() == true )
	{
		// Don't capture shift+click as the Curve Editor already handles this
		if( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && !MouseEvent.IsShiftDown() )
		{
			// Select the stop under the mouse if any and capture the mouse to get detect dragging
			SelectedStop = GetGradientStopAtPoint( MouseEvent.GetScreenSpacePosition(), MyGeometry );
			return FReply::Handled().CaptureMouse( SharedThis(this) );
		}
		else if( MouseEvent.GetEffectingButton() == EKeys::RightMouseButton )
		{
			FGradientStopMark PossibleSelectedStop = GetGradientStopAtPoint( MouseEvent.GetScreenSpacePosition(), MyGeometry );
			if( PossibleSelectedStop.IsValid( *CurveOwner ) )
			{
				// Only change selection on right click if something was selected
				SelectedStop = PossibleSelectedStop;

				return FReply::Handled().CaptureMouse( SharedThis( this ) );
			}

		}
	}
	
	return FReply::Unhandled();
}

FReply SColorGradientEditor::OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent )
{
	if( IsEditingEnabled.Get() == true )
	{
		// Select the stop under the mouse and open a color picker when it is double clicked
		SelectedStop = GetGradientStopAtPoint( InMouseEvent.GetScreenSpacePosition(), InMyGeometry );
		if( SelectedStop.IsValid( *CurveOwner ) )
		{
			ContextMenuPosition = InMouseEvent.GetScreenSpacePosition();		
			OpenGradientStopColorPicker();

			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

FReply SColorGradientEditor::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	const FGeometry ColorMarkAreaMouseGeometry = GeometrySolver->GetColorMarkAreaMouseGeometry(MyGeometry);
	const TOptional<FGeometry> AlphaMarkAreaMouseGeometry = GeometrySolver->GetAlphaMarkAreaMouseGeometry(MyGeometry);

	if (ColorMarkAreaMouseGeometry.IsUnderLocation(MouseEvent.GetScreenSpacePosition()))
	{
		bColorAreaHovered = true;
		bAlphaAreaHovered = false;
	}
	else if (AlphaMarkAreaMouseGeometry.IsSet() &&
		AlphaMarkAreaMouseGeometry.GetValue().IsUnderLocation(MouseEvent.GetScreenSpacePosition()))
	{
		bColorAreaHovered = false;
		bAlphaAreaHovered = true;
	}
	else
	{
		bColorAreaHovered = false;
		bAlphaAreaHovered = false;
	}

	if( HasMouseCapture() && IsEditingEnabled.Get() == true )
	{
		DistanceDragged += FMath::Abs( MouseEvent.GetCursorDelta().X );
			
		if( MouseEvent.IsMouseButtonDown( EKeys::LeftMouseButton ) && SelectedStop.IsValid( *CurveOwner ) )
		{
			const float DragThresholdDist = 5.0f;
			if( !bDraggingStop )
			{
				if( DistanceDragged >= DragThresholdDist )
				{
					// Start a transaction, we just started dragging a stop
					bDraggingStop = true;
					GEditor->BeginTransaction( LOCTEXT("MoveGradientStop", "Move Gradient Stop") );
					CurveOwner->ModifyOwner();
				}

				return FReply::Handled();
			}
			else
			{
				const float MouseTime = [&]()
					{
						// Already dragging a stop, move it
						const bool bHideStops = ShouldHideStops();

						float Result = GeometrySolver->GetTimeForAbsolutePosition(
							MyGeometry, 
							ViewMinInput.Get(), 
							ViewMaxInput.Get(), 
							bHideStops, 
							MouseEvent.GetScreenSpacePosition());

						if (bClampStopsToViewRange)
						{
							Result = FMath::Clamp(Result, ViewMinInput.Get(), ViewMaxInput.Get());
						}

						return Result;
					}();

				MoveStop( SelectedStop, MouseTime );

				return FReply::Handled();
			}
		}
	}

	return FReply::Unhandled();
}

FReply SColorGradientEditor::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	const float DragThresholdDist = 5.0f;

	if( IsEditingEnabled.Get() == true )
	{
		if( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
		{
			if( bDraggingStop == true )
			{
				// We stopped dragging
				GEditor->EndTransaction();
			}
			else if( DistanceDragged < DragThresholdDist && !SelectedStop.IsValid( *CurveOwner ) )
			{
				const FGeometry ColorMarkAreaMouseGeometry = GeometrySolver->GetColorMarkAreaMouseGeometry( MyGeometry );
				const TOptional<FGeometry> AlphaMarkAreaMouseGeometry = GeometrySolver->GetAlphaMarkAreaMouseGeometry( MyGeometry );

				TOptional<FLinearColor> StopColor;
				if (MouseEvent.IsControlDown())
				{
					StopColor = FLinearColor::White;
				}
				else if (MouseEvent.IsAltDown())
				{
					StopColor = FLinearColor::Transparent;
				}

				if( ColorMarkAreaMouseGeometry.IsUnderLocation( MouseEvent.GetScreenSpacePosition() ) )
				{
					// Add a new color mark
					bool bColorStop = true;
					SelectedStop = AddStop( MouseEvent.GetScreenSpacePosition(), MyGeometry, bColorStop, StopColor);

					return FReply::Handled().CaptureMouse( SharedThis(this) );

				}
				else if( 
					AlphaMarkAreaMouseGeometry.IsSet() &&
					AlphaMarkAreaMouseGeometry.GetValue().IsUnderLocation(MouseEvent.GetScreenSpacePosition()))
				{
					// Add a new alpha mark
					bool bColorStop = false;
					SelectedStop = AddStop( MouseEvent.GetScreenSpacePosition(), MyGeometry, bColorStop, StopColor);

					return FReply::Handled().CaptureMouse( SharedThis(this) );
				}
			}
			DistanceDragged = 0;
			bDraggingStop = false;
			// during interactive drag we called OnCurveChanged with EPropertyChangeType::Interactive
			//	now the drag is over, do OnCurveChanged with EPropertyChangeType::ValueSet
			CurveOwner->OnCurveChanged(CurveOwner->GetCurves());
			return FReply::Handled().ReleaseMouseCapture();
		}
		else if( MouseEvent.GetEffectingButton() == EKeys::RightMouseButton && !bDraggingStop)
		{
			// Didnt move the mouse too far, open a context menu
			if( DistanceDragged < DragThresholdDist )
			{
				if (SelectedStop.IsValid(*CurveOwner))
				{
					OpenGradientStopContextMenu(MouseEvent);
				}
				else
				{
					const bool bHideStops = ShouldHideStops();
					const FGeometry GradientAreaPaintGeometry = GeometrySolver->GetGradientAreaPaintGeometry(MyGeometry, bHideStops);
					if (GradientAreaPaintGeometry.IsUnderLocation(MouseEvent.GetScreenSpacePosition()))
					{
						OpenGradientOptionsMenu(MouseEvent);
					}
				}
			}

			DistanceDragged = 0;
			return FReply::Handled().ReleaseMouseCapture();
		}
	}

	return FReply::Unhandled();
}

void SColorGradientEditor::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	if (bDraggingStop == true)
	{
		GEditor->EndTransaction();
		DistanceDragged = 0;
		bDraggingStop = false;
		CurveOwner->OnCurveChanged(CurveOwner->GetCurves());
	}

}

FReply SColorGradientEditor::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	if ( IsEditingEnabled.Get() == true && InKeyEvent.GetKey() == EKeys::Platform_Delete )
	{
		DeleteStop( SelectedStop );
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SColorGradientEditor::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	bColorAreaHovered = false;
	bAlphaAreaHovered = false;
}

FVector2D SColorGradientEditor::ComputeDesiredSize( float ) const
{
	const TOptional<FVector2D>& CurrentSizeOverride = DesiredSizeOverride.Get();
	if (CurrentSizeOverride.IsSet())
	{
		return CurrentSizeOverride.GetValue();
	}
	return FVector2D( 1000, 75 );
}
	

void SColorGradientEditor::SetCurveOwner( FCurveOwnerInterface* InCurveOwner ) 
{ 
	CurveOwner = InCurveOwner;
}


void SColorGradientEditor::OpenGradientOptionsMenu(const FPointerEvent& MouseEvent)
{
	FMenuBuilder MenuBuilder(true, NULL);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("DrawColorAndAlphaSeparate", "Draw Color And Alpha Separate"),
		FText(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]() { bDrawColorAndAlphaSeparate = !bDrawColorAndAlphaSeparate; }),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]() { return bDrawColorAndAlphaSeparate; })
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	ContextMenuPosition = MouseEvent.GetScreenSpacePosition();

	FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
	FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuBuilder.MakeWidget(), ContextMenuPosition, FPopupTransitionEffect::ContextMenu);
}

void SColorGradientEditor::OpenGradientStopContextMenu(const FPointerEvent& MouseEvent)
{
	const FVector2D& Location = MouseEvent.GetScreenSpacePosition();

	FMenuBuilder GradientStopMenu( true, NULL );

	
	FUIAction RemoveStopAction( FExecuteAction::CreateSP( this, &SColorGradientEditor::OnRemoveSelectedGradientStop ) );
	{
		TSharedPtr<SWidget> WidgetToFocus;

		// Set color
		if( SelectedStop.IsValidColorMark( CurveOwner->GetCurves() ) )
		{
			GradientStopMenu.BeginSection( NAME_None, LOCTEXT("ColorMenuSecton", "Color") );

			FUIAction SetColorAction( FExecuteAction::CreateSP( this, &SColorGradientEditor::OpenGradientStopColorPicker ) );

			GradientStopMenu.AddMenuEntry( LOCTEXT("SetColorMenuItem", "Choose Color..."), LOCTEXT("SetColorMenuItem_ToolTip", "Opens a color picker to change the color of the stop"), FSlateIcon(), SetColorAction );

			GradientStopMenu.EndSection();
		}
		else
		{
			GradientStopMenu.BeginSection( NAME_None, LOCTEXT("AlphaMenuSection", "Opacity") );


			TSharedRef<SWidget> Widget = 
				SNew( SBox )
				.WidthOverride( 100.0f )
				[
					SNew( SSpinBox<float> )
					.MinSliderValue(0.0f)
					.MaxSliderValue(1.0f)
					.MinValue(-FLT_MAX)
					.MaxValue(FLT_MAX)
					.Value( SelectedStop.GetColor( *CurveOwner ).A )
					.OnBeginSliderMovement( this, &SColorGradientEditor::OnBeginChangeAlphaValue )
					.OnEndSliderMovement( this, &SColorGradientEditor::OnEndChangeAlphaValue )
					.OnValueChanged( this, &SColorGradientEditor::OnAlphaValueChanged )
					.OnValueCommitted( this, &SColorGradientEditor::OnAlphaValueCommitted )
				];

			GradientStopMenu.AddWidget( Widget, FText::GetEmpty() );

			GradientStopMenu.EndSection();
		}

		// Set time
		{
			TSharedRef<SSpinBox<float>> TimeWidget =
				SNew(SSpinBox<float>)
				.Value( SelectedStop.GetTime(*CurveOwner) )
				.OnValueCommitted(this, &SColorGradientEditor::OnTimeValueCommited);

			GradientStopMenu.BeginSection( NAME_None, LOCTEXT("TimeMenuSection", "Time") );

			GradientStopMenu.AddWidget(TimeWidget, FText::GetEmpty() );

			GradientStopMenu.EndSection();

			WidgetToFocus = TimeWidget.ToSharedPtr();
		}


		GradientStopMenu.AddMenuSeparator();

		// Add a Remove option
		GradientStopMenu.AddMenuEntry( LOCTEXT("RemoveGradientStop", "Remove Stop"), LOCTEXT("RemoveGradientStopTooltip", "Removes the selected gradient stop"), FSlateIcon(), RemoveStopAction );

		FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
		FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, GradientStopMenu.MakeWidget(), Location, FPopupTransitionEffect::ContextMenu);

		FSlateApplication::Get().SetKeyboardFocus( WidgetToFocus.ToSharedRef() );
	}

	ContextMenuPosition = Location;
}

void SColorGradientEditor::OpenGradientStopColorPicker()
{
	TArray<FRichCurveEditInfo> Curves = CurveOwner->GetCurves();

	if( SelectedStop.IsValidAlphaMark( Curves ) )
	{
		// Show a slider to change the alpha value
		TSharedRef<SWidget> AlphaSlider = 
		SNew( SBorder )
		.BorderImage( FAppStyle::GetBrush("Menu.Background") )
		[
			SNew( SVerticalBox )
			+ SVerticalBox::Slot()
			[
				SNew( STextBlock )
				.Text( LOCTEXT("AlphaLabel","Opacity") )
				.TextStyle( FAppStyle::Get(), "Menu.Heading" )
			]
			+ SVerticalBox::Slot()
			.Padding( 3.0f, 2.0f, 3.0f, 5.0f )
			.AutoHeight()
			[
				SNew( SBox )
				.WidthOverride( 100.0f )
				[
					SNew( SSpinBox<float> )
					.MinSliderValue(0.0f)
					.MaxSliderValue(1.0f)
					.MinValue(-FLT_MAX)
					.MaxValue(FLT_MAX)
					.Value( SelectedStop.GetColor( *CurveOwner ).A )
					.OnBeginSliderMovement( this, &SColorGradientEditor::OnBeginChangeAlphaValue )
					.OnEndSliderMovement( this, &SColorGradientEditor::OnEndChangeAlphaValue )
					.OnValueChanged( this, &SColorGradientEditor::OnAlphaValueChanged )
					.OnValueCommitted( this, &SColorGradientEditor::OnAlphaValueCommitted )
				]
			]
		];

		FSlateApplication::Get().PushMenu( SharedThis( this ), FWidgetPath(), AlphaSlider, ContextMenuPosition, FPopupTransitionEffect::TypeInPopup );
	}
	else
	{
		// Open a color picker
		FColorPickerArgs ColorPickerArgs;

		ColorPickerArgs.bOnlyRefreshOnMouseUp = true;
		ColorPickerArgs.bIsModal = false;
		ColorPickerArgs.ParentWidget = SharedThis( this );
		ColorPickerArgs.bUseAlpha = false;
		ColorPickerArgs.InitialColor = SelectedStop.GetColor( *CurveOwner );
		ColorPickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateSP( this, &SColorGradientEditor::OnSelectedStopColorChanged );
		ColorPickerArgs.OnColorPickerCancelled  = FOnColorPickerCancelled::CreateSP( this, &SColorGradientEditor::OnCancelSelectedStopColorChange );
		OpenColorPicker( ColorPickerArgs );
	}
}

void SColorGradientEditor::OnSelectedStopColorChanged( FLinearColor InNewColor )
{
	// Don't add a transaction if nothing has changed
	if (SelectedStop.GetColor(*CurveOwner).Equals(InNewColor))
	{
		return;
	}

	FScopedTransaction ColorChange( LOCTEXT("ChangeGradientStopColor", "Change Gradient Stop Color") );
	CurveOwner->ModifyOwner();
	SelectedStop.SetColor( InNewColor, *CurveOwner );
	TArray<FRichCurveEditInfo> ChangedCurves{ CurveOwner->GetCurves()[0], CurveOwner->GetCurves()[1], CurveOwner->GetCurves()[2] };
	CurveOwner->OnCurveChanged(ChangedCurves);
}

void SColorGradientEditor::OnCancelSelectedStopColorChange( FLinearColor PreviousColor )
{
	CurveOwner->ModifyOwner();
	SelectedStop.SetColor( PreviousColor, *CurveOwner );
	TArray<FRichCurveEditInfo> ChangedCurves{ CurveOwner->GetCurves()[0], CurveOwner->GetCurves()[1], CurveOwner->GetCurves()[2] };
	CurveOwner->OnCurveChanged(ChangedCurves);
}

void SColorGradientEditor::OnBeginChangeAlphaValue()
{
	GEditor->BeginTransaction( LOCTEXT("ChangeGradientStopAlpha", "Change Gradient Stop Alpha") );
	CurveOwner->ModifyOwner();

	bDraggingAlphaValue = true;
}

void SColorGradientEditor::OnEndChangeAlphaValue( float NewValue )
{
	if( bDraggingAlphaValue )
	{
		GEditor->EndTransaction();
	}

	bDraggingAlphaValue = false;
}


void SColorGradientEditor::OnAlphaValueChanged( float NewValue )
{
	if( bDraggingAlphaValue )
	{
		// RGB is ignored in this case
		SelectedStop.SetColor( FLinearColor( 0,0,0, NewValue ), *CurveOwner );
		TArray<FRichCurveEditInfo> ChangedCurves{ CurveOwner->GetCurves()[3] };
		CurveOwner->OnCurveChanged(ChangedCurves);
	}
}

void SColorGradientEditor::OnAlphaValueCommitted( float NewValue, ETextCommit::Type )
{
	if( !GEditor->IsTransactionActive() )
	{
		// Don't add a transaction if nothing has changed
		if (FMath::IsNearlyEqual(SelectedStop.GetColor(*CurveOwner).A, NewValue))
		{
			return;
		}

		// Value was typed in, no transaction is active
		FScopedTransaction ChangeAlphaTransaction( LOCTEXT("ChangeGradientStopAlpha", "Change Gradient Stop Alpha") );
		CurveOwner->ModifyOwner();
		SelectedStop.SetColor( FLinearColor( 0,0,0, NewValue ), *CurveOwner );
		TArray<FRichCurveEditInfo> ChangedCurves{ CurveOwner->GetCurves()[3] };
		CurveOwner->OnCurveChanged(ChangedCurves);
	}
	else
	{
		SelectedStop.SetColor( FLinearColor( 0,0,0, NewValue ), *CurveOwner );
		TArray<FRichCurveEditInfo> ChangedCurves{ CurveOwner->GetCurves()[3] };
		CurveOwner->OnCurveChanged(ChangedCurves);
	}
}

void SColorGradientEditor::OnRemoveSelectedGradientStop()
{
	DeleteStop( SelectedStop );
}

void SColorGradientEditor::OnTimeValueCommited(float NewValue, ETextCommit::Type Type)
{
	if (NewValue != SelectedStop.GetTime(*CurveOwner) )
	{
		FScopedTransaction Transaction(LOCTEXT("ChangeGradientStopTime", "Change Gradient Stop Time"));
		CurveOwner->ModifyOwner();
		SelectedStop.SetTime(NewValue, *CurveOwner);
		CurveOwner->OnCurveChanged(CurveOwner->GetCurves());
	}
}

void SColorGradientEditor::DrawGradientStopMark( const FGradientStopMark& Mark, const FGeometry& Geometry, float XPos, const FLinearColor& Color, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FSlateRect& InClippingRect, ESlateDrawEffect DrawEffects, bool bColor, const FWidgetStyle& InWidgetStyle ) const
{
	static const FSlateBrush* WhiteBrush = FAppStyle::GetBrush("WhiteBrush");
	static const FSlateBrush* ColorStopBrush = FAppStyle::GetBrush("CurveEditor.Gradient.HandleDown");
	static const FSlateBrush* AlphaStopBrush = FAppStyle::GetBrush("CurveEditor.Gradient.HandleUp");
	static const FLinearColor SelectionColor = FAppStyle::GetSlateColor("SelectionColor").GetColor( InWidgetStyle );

	bool bSelected = Mark == SelectedStop;
	if( bSelected )
	{
		// Show selected stops above other stops
		++LayerId;
	}

	// Draw a box for the non colored area
	const FVector2f HandleSize = GeometrySolver->HandleRect.GetSize();
	const FVector2f HandlePosition = GeometrySolver->HandleRect.GetTopLeft2f() + FVector2f(XPos, 0);
	FSlateDrawElement::MakeBox
	( 
		OutDrawElements,
		LayerId,
		Geometry.ToPaintGeometry( HandleSize, FSlateLayoutTransform(HandlePosition) ),
		bColor ? ColorStopBrush : AlphaStopBrush,
		DrawEffects,
		bSelected ? SelectionColor : FLinearColor::White
	);

	// Draw a box with the gradient stop color
	//Slate MakeGradient call expects the linear colors to be pre-converted to sRGB
	const FVector2f HandleColorSize = FVector2f(GeometrySolver->HandleColorWidth, GeometrySolver->HandleColorHeight);
	const FVector2f HandleColorPosition = HandlePosition + FVector2f(
		GeometrySolver->HandleColorXOffset, 
		bColor 
			? GeometrySolver->HandleColorYOffsetForColor 
			: GeometrySolver->HandleColorYOffsetForAlpha);
	FSlateDrawElement::MakeBox
	( 
		OutDrawElements,
		LayerId+1,
		Geometry.ToPaintGeometry( HandleColorSize, FSlateLayoutTransform(HandleColorPosition) ),
		WhiteBrush,
		DrawEffects,
		Color.ToFColor(true)
	);
}

FGradientStopMark SColorGradientEditor::GetGradientStopAtPoint( const FVector2D& MousePos, const FGeometry& MyGeometry )
{
	FGradientStopMark StopUnderMouse;
	const FGeometry ColorMarkAreaMouseGeometry = GeometrySolver->GetColorMarkAreaMouseGeometry( MyGeometry );
	const TOptional<FGeometry> AlphaMarkAreaMouseGeometry = GeometrySolver->GetAlphaMarkAreaMouseGeometry( MyGeometry );
	const bool bUnderColorMarkAreaGeometry = ColorMarkAreaMouseGeometry.IsUnderLocation(MousePos);
	const bool bUnderAlphaMarkAreaGeometry = AlphaMarkAreaMouseGeometry.IsSet() && AlphaMarkAreaMouseGeometry.GetValue().IsUnderLocation(MousePos);

	if( bUnderColorMarkAreaGeometry || bUnderAlphaMarkAreaGeometry )
	{
		TArray<FGradientStopMark> ColorMarks;
		TArray<FGradientStopMark> AlphaMarks;
		GetGradientStopMarks( ColorMarks, AlphaMarks );

		auto GetMarkUnderMouse = [this](const FVector2D& MousePos, const TArray<FGradientStopMark>& Marks, const FGeometry& AreaPaintGeometry)
		{
			const FTrackScaleInfo TimeScaleInfo = GeometrySolver->GetTimeScaleForAreaPaintGeomtry(AreaPaintGeometry, ViewMinInput.Get(), ViewMaxInput.Get());
			for (int32 MarkIndex = 0; MarkIndex < Marks.Num(); ++MarkIndex)
			{
				const FGradientStopMark& Mark = Marks[MarkIndex];

				// Convert the time to a screen coordinate
				float XVal = TimeScaleInfo.InputToLocalX(Mark.Time);

				if (XVal >= 0)
				{
					const FVector2f HandleSize = GeometrySolver->HandleRect.GetSize2f();
					const FVector2f HandlePosition = GeometrySolver->HandleRect.GetTopLeft2f() + FVector2f(XVal, 0.0f);
					const FGeometry HandleGeometry = AreaPaintGeometry.MakeChild(HandleSize, FSlateLayoutTransform(HandlePosition));
					if (HandleGeometry.IsUnderLocation(MousePos))
					{
						return Mark;
					}
				}
			}
			return FGradientStopMark();
		};

		if (bUnderColorMarkAreaGeometry)
		{
			const FGeometry ColorMarkAreaPaintGeometry = GeometrySolver->GetColorMarkAreaPaintGeometry(MyGeometry);
			StopUnderMouse = GetMarkUnderMouse(MousePos, ColorMarks, ColorMarkAreaPaintGeometry);
		}
		else // bUnderAlphaMarkAreaGeometry
		{
			const TOptional<FGeometry> AlphaMarkAreaPaintGeometry = GeometrySolver->GetAlphaMarkAreaPaintGeometry(MyGeometry);
			if (AlphaMarkAreaPaintGeometry.IsSet())
			{
				StopUnderMouse = GetMarkUnderMouse(MousePos, AlphaMarks, AlphaMarkAreaPaintGeometry.GetValue());
			}
		}
	}
	return StopUnderMouse;
}

void SColorGradientEditor::GetGradientStopMarks( TArray<FGradientStopMark>& OutColorMarks, TArray<FGradientStopMark>& OutAlphaMarks ) const
{
	TArray<FRichCurveEditInfo> Curves = CurveOwner->GetCurves();

	check( Curves.Num() == 4 );

	// Find Gradient stops

	// Assume indices 0,1,2 hold R,G,B;
	const FRealCurve* RedCurve = Curves[0].CurveToEdit;
	const FRealCurve* GreenCurve = Curves[1].CurveToEdit;
	const FRealCurve* BlueCurve = Curves[2].CurveToEdit;
	const FRealCurve* AlphaCurve = Curves[3].CurveToEdit;

	
	// Use the red curve to check the other color channels for keys at the same time
	for( auto It = RedCurve->GetKeyHandleIterator(); It; ++It )
	{
		FKeyHandle RedKeyHandle = *It;
		float Time = RedCurve->GetKeyTime( RedKeyHandle );
		
		FKeyHandle GreenKeyHandle = GreenCurve->FindKey( Time );
			
		FKeyHandle BlueKeyHandle = BlueCurve->FindKey( Time );

		if( GreenCurve->IsKeyHandleValid( GreenKeyHandle ) && BlueCurve->IsKeyHandleValid( BlueKeyHandle ) )
		{
			// each curve has a handle at the current time.  It can be a gradient stop
			FGradientStopMark( Time, RedKeyHandle, GreenKeyHandle, BlueKeyHandle );
			OutColorMarks.Add( FGradientStopMark( Time, RedKeyHandle, GreenKeyHandle, BlueKeyHandle ) );
		}

	}

	// Add an alpha gradient stop mark for each alpha key
	for( auto It = AlphaCurve->GetKeyHandleIterator(); It; ++It )
	{
		FKeyHandle KeyHandle = *It;
		float Time = AlphaCurve->GetKeyTime( KeyHandle );
		OutAlphaMarks.Add( FGradientStopMark( Time, FKeyHandle(), FKeyHandle(), FKeyHandle(), KeyHandle ) );

	}
}

void SColorGradientEditor::DeleteStop( const FGradientStopMark& InMark )
{
	FScopedTransaction DeleteStopTrans( LOCTEXT("DeleteGradientStop", "Delete Gradient Stop") );
	CurveOwner->ModifyOwner();

	TArray<FRichCurveEditInfo> Curves = CurveOwner->GetCurves();

	FRealCurve* RedCurve = Curves[0].CurveToEdit;
	FRealCurve* GreenCurve = Curves[1].CurveToEdit;
	FRealCurve* BlueCurve = Curves[2].CurveToEdit;
	FRealCurve* AlphaCurve = Curves[3].CurveToEdit;

	if( InMark.IsValidAlphaMark( Curves ) )
	{
		AlphaCurve->DeleteKey( InMark.AlphaKeyHandle );
	}
	else if( InMark.IsValidColorMark( Curves ) ) 
	{
		RedCurve->DeleteKey( InMark.RedKeyHandle );
		GreenCurve->DeleteKey( InMark.GreenKeyHandle );
		BlueCurve->DeleteKey( InMark.BlueKeyHandle );
	}

	CurveOwner->OnCurveChanged(CurveOwner->GetCurves());
}

FGradientStopMark SColorGradientEditor::AddStop( const FVector2D& Position, const FGeometry& MyGeometry, bool bColorStop, TOptional<FLinearColor> OptionalColor)
{
	FScopedTransaction AddStopTrans( LOCTEXT("AddGradientStop", "Add Gradient Stop") );

	CurveOwner->ModifyOwner();

	const bool bHideStops = ShouldHideStops();
	const float NewStopTime = GeometrySolver->GetTimeForAbsolutePosition(MyGeometry, ViewMinInput.Get(), ViewMaxInput.Get(), bHideStops, Position);

	TArray<FRichCurveEditInfo> Curves = CurveOwner->GetCurves();

	FGradientStopMark NewStop;
	NewStop.Time = NewStopTime;

	if( bColorStop )
	{
		FRealCurve* RedCurve = Curves[0].CurveToEdit;
		FRealCurve* GreenCurve = Curves[1].CurveToEdit;
		FRealCurve* BlueCurve = Curves[2].CurveToEdit;

		const float RValue = OptionalColor.IsSet() ? OptionalColor.GetValue().R : RedCurve->Eval(NewStopTime, 1.0f);
		const float GValue = OptionalColor.IsSet() ? OptionalColor.GetValue().G : GreenCurve->Eval(NewStopTime, 1.0f);
		const float BValue = OptionalColor.IsSet() ? OptionalColor.GetValue().B : BlueCurve->Eval(NewStopTime, 1.0f);

		NewStop.RedKeyHandle = RedCurve->AddKey( NewStopTime, RValue );
		NewStop.GreenKeyHandle = GreenCurve->AddKey( NewStopTime, GValue );
		NewStop.BlueKeyHandle = BlueCurve->AddKey( NewStopTime, BValue );
	}
	else
	{
		FRealCurve* AlphaCurve = Curves[3].CurveToEdit;
		const float AlphaValue = OptionalColor.IsSet() ? OptionalColor.GetValue().A : AlphaCurve->Eval(NewStopTime, 1.0f);
		NewStop.AlphaKeyHandle = AlphaCurve->AddKey( NewStopTime, AlphaValue );
	}

	CurveOwner->OnCurveChanged(CurveOwner->GetCurves());

	return NewStop;
}

void SColorGradientEditor::MoveStop( FGradientStopMark& Mark, float NewTime )
{
	CurveOwner->ModifyOwner();
	Mark.SetTime( NewTime, *CurveOwner );
	EPropertyChangeType::Type ChangeType = bDraggingStop ? EPropertyChangeType::Interactive : EPropertyChangeType::ValueSet;
	// we want to be able to pass ChangeType as an arg to OnCurveChanged for interactive drags, but OnCurveChanged doesn't take that
	// there are lots of implementations of OnCurveChanged so changing the function now is not great
	//	instead set a bool on the curve to indicate the next OnCurveChanged type
	CurveOwner->SetOnCurveChangedIsInteractive(bDraggingStop);
	CurveOwner->OnCurveChanged(CurveOwner->GetCurves());
	CurveOwner->SetOnCurveChangedIsInteractive(false);
}


#undef LOCTEXT_NAMESPACE
