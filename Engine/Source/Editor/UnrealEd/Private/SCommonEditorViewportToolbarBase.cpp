// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCommonEditorViewportToolbarBase.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/SToolTip.h"
#include "Styling/AppStyle.h"

#include "STransformViewportToolbar.h"
#include "SEditorViewport.h"
#include "EditorViewportCommands.h"
#include "SEditorViewportToolBarMenu.h"
#include "SEditorViewportToolBarButton.h"
#include "SEditorViewportViewMenu.h"
#include "Editor/EditorPerformanceSettings.h"
#include "Settings/EditorProjectSettings.h"
#include "Scalability.h"
#include "SceneView.h"
#include "SScalabilitySettings.h"
#include "SAssetEditorViewport.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ShowFlagMenuCommands.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SCommonEditorViewportToolbarBase)

#define LOCTEXT_NAMESPACE "SCommonEditorViewportToolbarBase"

namespace UE::UnrealEd::Private
{
// Used internally:
// - by old viewport toolbar menu SCommonEditorViewportToolbarBase::GenerateOptionsMenu()
// - by deprecated API function SCommonEditorViewportToolbarBase::ConstructScreenPercentageMenu(FMenuBuilder& MenuBuilder, FEditorViewportClient* InViewportClient)
void AddScreenPercentageMenu(FMenuBuilder& InMenuBuilder, const FEditorViewportClient* InViewportClient)
{
	if (!InViewportClient)
	{
		return;
	}

	TSharedPtr<SEditorViewport> EditorViewport = InViewportClient->GetEditorViewportWidget();
	if (!EditorViewport)
	{
		return;
	}

	FName OldScreenPercentageMenuName = "CommonEditorViewport.OldViewportToolbar.ScreenPercentage";
	if (!UToolMenus::Get()->IsMenuRegistered(OldScreenPercentageMenuName))
	{
		UToolMenu* Menu =
			UToolMenus::Get()->RegisterMenu(OldScreenPercentageMenuName, NAME_None, EMultiBoxType::Menu, false);

		FToolMenuSection& UnnamedSection = Menu->FindOrAddSection(NAME_None);
		UnnamedSection.AddEntry(CreateScreenPercentageSubmenu());
	}

	FToolMenuContext MenuContext;
	{
		MenuContext.AppendCommandList(EditorViewport->GetCommandList());

		// Add the UnrealEd viewport toolbar context.
		{
			UUnrealEdViewportToolbarContext* const ContextObject =
				UE::UnrealEd::CreateViewportToolbarDefaultContext(EditorViewport);

			MenuContext.AddObject(ContextObject);
		}
	}

	InMenuBuilder.AddWidget(UToolMenus::Get()->GenerateWidget(OldScreenPercentageMenuName, MenuContext), FText());
}

TSharedPtr<const SCommonEditorViewportToolbarBase> FindToolbarWidget(UToolMenu* ToolMenu)
{
	if (const UCommonViewportToolbarBaseMenuContext* Context = ToolMenu->FindContext<UCommonViewportToolbarBaseMenuContext>())
	{
		if (TSharedPtr<const SCommonEditorViewportToolbarBase> ToolbarWidget = Context->ToolbarWidget.Pin())
		{
			return ToolbarWidget;
		}
	}
	return nullptr;
}

TSharedPtr<const SCommonEditorViewportToolbarBase> FindToolbarWidget(const FToolMenuSection& ToolSection)
{
	if (const UCommonViewportToolbarBaseMenuContext* Context = ToolSection.FindContext<UCommonViewportToolbarBaseMenuContext>())
	{
		if (TSharedPtr<const SCommonEditorViewportToolbarBase> ToolbarWidget = Context->ToolbarWidget.Pin())
		{
			return ToolbarWidget;
		}
	}
	return nullptr;
}

} // namespace UE::UnrealEd::Private

TSharedPtr<IPreviewProfileController> UCommonViewportToolbarBaseMenuContext::GetPreviewProfileController() const
{
	if (TSharedPtr<const SCommonEditorViewportToolbarBase> Toolbar = ToolbarWidget.Pin())
	{
		if (TSharedPtr<IPreviewProfileController> Controller = Toolbar->GetPreviewProfileController())
		{
			return Controller;
		}
	}
	return UUnrealEdViewportToolbarContext::GetPreviewProfileController();
}

//////////////////////////////////////////////////////////////////////////
// SPreviewSceneProfileSelector

void SPreviewSceneProfileSelector::Construct(const FArguments& InArgs)
{
	PreviewProfileController = InArgs._PreviewProfileController;

	// clang-format off
	TSharedRef<SHorizontalBox> ButtonContent = 
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		.AutoWidth()
		.Padding(4.0f, 0.0f, 4.f, 0.0f)
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush("AssetEditor.PreviewSceneSettings"))
			.ColorAndOpacity(FSlateColor::UseForeground())
		]
		+SHorizontalBox::Slot()
		.Padding(0.0f, 0.0f, 4.f, 0.0f)
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Margin(FMargin(0))
			.Text_Lambda(
				[this]() -> FText
				{
					return FText::FromString(PreviewProfileController->GetActiveProfile());
				}
			)
		];
	// clang-format on

	// clang-format off
	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(AssetViewerProfileComboButton, SComboButton)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("EditorViewportToolBar.Button"))
			.ContentPadding(FMargin(0))
			.HasDownArrow(false)
			.OnGetMenuContent(this, &SPreviewSceneProfileSelector::BuildComboMenu)
			.ButtonContent()
			[
				ButtonContent
			]
		]
	];
	// clang-format on
}

TSharedRef<SWidget> SPreviewSceneProfileSelector::BuildComboMenu()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	TSharedPtr<const FUICommandList> CommandList = nullptr;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("PreviewSceneProfilesSectionLabel", "Preview Scene Profiles"));

	int32 UnusedActiveIndex;
	const FName UnusedExtensionHook = NAME_None;
	const TArray<FString> PreviewProfiles = PreviewProfileController->GetPreviewProfiles(UnusedActiveIndex);
	for (const FString& ProfileName : PreviewProfiles)
	{
		MenuBuilder.AddMenuEntry(
			FText::FromString(ProfileName),
			FText(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda(
					[this, WeakController = PreviewProfileController.ToWeakPtr(), ProfileName]()
					{
						if (TSharedPtr<IPreviewProfileController> PinnedController = WeakController.Pin())
						{
							PinnedController->SetActiveProfile(ProfileName);
						}
					}
				),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda(
					[WeakController = PreviewProfileController.ToWeakPtr(), ProfileName]()
					{
						if (TSharedPtr<IPreviewProfileController> PinnedController = WeakController.Pin())
						{
							return ProfileName == PinnedController->GetActiveProfile();
						}

						return false;
					}
				)
			),
			UnusedExtensionHook,
			EUserInterfaceActionType::RadioButton
		);
	}

	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

//////////////////////////////////////////////////////////////////////////
// SCommonEditorViewportToolbarBase


void SCommonEditorViewportToolbarBase::Construct(const FArguments& InArgs, TSharedPtr<class ICommonEditorViewportToolbarInfoProvider> InInfoProvider)
{
	InfoProviderPtr = InInfoProvider;

	// Create a blank menu to be treated as "null" for the purposes of detecting whether a custom view menu has been defined.
	BlankViewMenu = MakeShared<SEditorViewportViewMenu>();

	const FMargin ToolbarSlotPadding(4.0f, 1.0f);
	const FMargin ToolbarButtonPadding(4.0f, 0.0f);

	TSharedRef<SEditorViewport> ViewportRef = GetInfoProvider().GetViewportWidget();
	TSharedRef<SHorizontalBox> MainBox = SNew(SHorizontalBox);

	// Options menu
	MainBox->AddSlot()
		.AutoWidth()
		.Padding(ToolbarSlotPadding)
		[
			SNew(SEditorViewportToolbarMenu)
			.ParentToolBar(SharedThis(this))
			.Cursor(EMouseCursor::Default)
			.Image("EditorViewportToolBar.OptionsDropdown")
			.OnGetMenuContent(this, &SCommonEditorViewportToolbarBase::GenerateOptionsMenu)
		];

	// Camera mode menu
	MainBox->AddSlot()
		.AutoWidth()
		.Padding(ToolbarSlotPadding)
		[
			SNew(SEditorViewportToolbarMenu)
			.ParentToolBar(SharedThis(this))
			.Cursor(EMouseCursor::Default)
			.Label(this, &SCommonEditorViewportToolbarBase::GetCameraMenuLabel)
			.OnGetMenuContent(this, &SCommonEditorViewportToolbarBase::GenerateCameraMenu)
		];

	// View menu
	MainBox->AddSlot()
		.AutoWidth()
		.Padding(ToolbarSlotPadding)
		[
			MakeViewMenu()
		];

	// Show menu
	MainBox->AddSlot()
		.AutoWidth()
		.Padding(ToolbarSlotPadding)
		[
			SNew(SEditorViewportToolbarMenu)
			.Label(LOCTEXT("ShowMenuTitle", "Show"))
			.Cursor(EMouseCursor::Default)
			.ParentToolBar(SharedThis(this))
			.OnGetMenuContent(this, &SCommonEditorViewportToolbarBase::GenerateShowMenu)
		];

	// Profile menu (Controls the Preview Scene Settings)
	if (InArgs._PreviewProfileController)
	{
		PreviewProfileController = InArgs._PreviewProfileController;
		MainBox->AddSlot()
			.AutoWidth()
			.Padding(ToolbarSlotPadding)
			[
				SNew(SPreviewSceneProfileSelector).PreviewProfileController(PreviewProfileController)
			];
	}

	// Realtime button
	if (InArgs._AddRealtimeButton)
	{
		MainBox->AddSlot()
			.AutoWidth()
			.Padding(ToolbarSlotPadding)
			[
				SNew(SEditorViewportToolBarButton)
				.Cursor(EMouseCursor::Default)
				.ButtonType(EUserInterfaceActionType::Button)
				.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("EditorViewportToolBar.WarningButton"))
				.OnClicked(this, &SCommonEditorViewportToolbarBase::OnRealtimeWarningClicked)
				.Visibility(this, &SCommonEditorViewportToolbarBase::GetRealtimeWarningVisibility)
				.ToolTipText(LOCTEXT("RealtimeOff_ToolTip", "This viewport is not updating in realtime.  Click to turn on realtime mode."))
				.Content()
				[
					SNew(STextBlock)
					.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("SmallText"))
					.Text(LOCTEXT("RealtimeOff", "Realtime Off"))
				]
			];
	}

	MainBox->AddSlot()
		.AutoWidth()
		.Padding(ToolbarSlotPadding)
		[
			SNew(SEditorViewportToolbarMenu)
			.Label(LOCTEXT("ViewParamMenuTitle", "View Mode Options"))
			.Cursor(EMouseCursor::Default)
			.ParentToolBar(SharedThis(this))
			.Visibility(this, &SCommonEditorViewportToolbarBase::GetViewModeOptionsVisibility)
			.OnGetMenuContent(this, &SCommonEditorViewportToolbarBase::GenerateViewModeOptionsMenu)
		];

	MainBox->AddSlot()
		.AutoWidth()
		.Padding(ToolbarSlotPadding)
		[
			// Button to show scalability warnings
			SNew(SEditorViewportToolbarMenu)
			.ParentToolBar(SharedThis(this))
			.Label_Static(&UE::UnrealEd::GetScalabilityWarningLabel)
			.MenuStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("EditorViewportToolBar.WarningButton"))
			.OnGetMenuContent(this, &SCommonEditorViewportToolbarBase::GetScalabilityWarningMenuContent)
			.Visibility(this, &SCommonEditorViewportToolbarBase::GetScalabilityWarningVisibility)
			.ToolTipText_Static(&UE::UnrealEd::GetScalabilityWarningTooltip)
		];

	// Add optional toolbar slots to be added by child classes inherited from this common viewport toolbar
	ExtendLeftAlignedToolbarSlots(MainBox, SharedThis(this));

	// Transform toolbar
	MainBox->AddSlot()
		.Padding(ToolbarSlotPadding)
		.HAlign(HAlign_Right)
		[
			SNew(STransformViewportToolBar)
			.Viewport(ViewportRef)
			.CommandList(ViewportRef->GetCommandList())
			.Extenders(GetInfoProvider().GetExtenders())
			.Visibility(ViewportRef, &SEditorViewport::GetTransformToolbarVisibility)
		];

	// Custom view menus and widgets added to the left side of the menu will retain
	// the old toolbar widget appearance. Simply including them alongside new elements
	// will mix styles undesirably, ultimately looking more broken than just using the old
	// toolbar.
	// Thus, the automatic toolbar upgrade is only enabled when customizations (e.g. show menus)
	// can be incorporated into the new design.
	const bool bUseUpgradedToolbar = !bHasExtendedLeftSide && bUsesDefaultViewMenu;
	if (bUseUpgradedToolbar)
	{
		const FName ViewportToolbarName = "UnrealEd.ViewportToolbar";
		
		if (!UToolMenus::Get()->IsMenuRegistered(ViewportToolbarName))
		{
			UToolMenu* const ViewportToolbarMenu = UToolMenus::Get()->RegisterMenu(
				ViewportToolbarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar
			);
			
			ViewportToolbarMenu->StyleName = "ViewportToolbar";
			
			FToolMenuSection& LeftSection = ViewportToolbarMenu->AddSection("Left");
			{
				LeftSection.AddEntry(UE::UnrealEd::CreateTransformsSubmenu());
				LeftSection.AddEntry(UE::UnrealEd::CreateSnappingSubmenu());
			}
			
			FToolMenuSection& RightSection = ViewportToolbarMenu->AddSection("Right");
			RightSection.Alignment = EToolMenuSectionAlign::Last;
			{
				// Camera Menu
				RightSection.AddEntry(UE::UnrealEd::CreateCameraSubmenu(UE::UnrealEd::FViewportCameraMenuOptions().ShowAll()));
				
				// View Menu
				{
					// Include backwards-compatability with earlier toolbars
					// Create our grandparent menu.
					if (!UToolMenus::Get()->IsMenuRegistered("UnrealEd.ViewportToolbar.View"))
					{
						UToolMenus::Get()->RegisterMenu("UnrealEd.ViewportToolbar.View");
					}
					
					// Create our menu.
					UToolMenus::Get()->RegisterMenu(
						"UnrealEd.ViewportToolbar.ViewModes", "UnrealEd.ViewportToolbar.View"
					);
					
					RightSection.AddEntry(UE::UnrealEd::CreateViewModesSubmenu());
				}
				
				// Show Menu
				{
					// Include backwards-compatability with earlier toolbars
					if (!UToolMenus::Get()->IsMenuRegistered("ViewportToolbarBase.Show"))
					{
						UToolMenus::Get()->RegisterMenu("ViewportToolbarBase.Show");
					}
					UToolMenus::Get()->RegisterMenu(UToolMenus::JoinMenuPaths(ViewportToolbarName, "Show"), "ViewportToolbarBase.Show");
				
					RightSection.AddEntry(UE::UnrealEd::CreateShowSubmenu(
						FNewToolMenuDelegate::CreateLambda([](UToolMenu* Submenu)
						{
							Submenu->AddDynamicSection("Flags", FNewToolMenuDelegate::CreateLambda([](UToolMenu* Menu)
							{
								TSharedPtr<const SCommonEditorViewportToolbarBase> ToolbarWidget = UE::UnrealEd::Private::FindToolbarWidget(Menu);
								if (!ToolbarWidget)
								{
									return;
								}
									
								if (!ToolbarWidget->bIsGeneratingToolMenuWidget)
								{
									if (TSharedPtr<SWidget> LegacyWidget = ToolbarWidget->MakeLegacyShowMenu())
									{
										// Display legacy menu
										Menu->AddSection("LegacyWidget").AddEntry(FToolMenuEntry::InitWidget(
											"LegacyWidget",
											LegacyWidget.ToSharedRef(),
											FText::GetEmpty(), // No label
											true, // No indent
											true, // Searchable
											true // No padding
										));
									}
									else
									{
										ToolbarWidget->FillShowFlagsMenu(Menu);	
									}
								}
							}));
						})
					));
				}
				
				RightSection.AddEntry(UE::UnrealEd::CreatePerformanceAndScalabilitySubmenu());
				
				RightSection.AddEntry(UE::UnrealEd::CreateAssetViewerProfileSubmenu());
					
				RightSection.AddDynamicEntry("LegacyOptionsMenu", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& Section)
				{
					TSharedPtr<const SCommonEditorViewportToolbarBase> ToolbarWidget = UE::UnrealEd::Private::FindToolbarWidget(Section);
					if (!ToolbarWidget)
					{
						return;
					}
					
					if (ToolbarWidget->ShouldCreateOptionsMenu())
					{
						Section.AddSubMenu(
							"Settings",
							LOCTEXT("SettingsSubmenuLabel", "Settings"),
							LOCTEXT("SettingsSubmenuTooltip", "Viewport-related settings"),
							FNewToolMenuDelegate::CreateLambda([](UToolMenu* Submenu)
							{
								Submenu->AddDynamicSection("Settings", FNewToolMenuDelegate::CreateLambda([](UToolMenu* Menu)
								{
									TSharedPtr<const SCommonEditorViewportToolbarBase> Toolbar = UE::UnrealEd::Private::FindToolbarWidget(Menu);
									if (!Toolbar)
									{
										return;
									}
								
									FToolMenuSection& Section = Menu->FindOrAddSection("Settings", LOCTEXT("SettingsSectionLabel", "Settings"));	
									FMenuBuilder LegacyMenuBuilder(true, nullptr);
									Toolbar->ExtendOptionsMenu(LegacyMenuBuilder);
									Section.AddEntry(FToolMenuEntry::InitWidget(
										"LegacySettingsMenus",
										LegacyMenuBuilder.MakeWidget(),
										FText::GetEmpty(), // No label
										true, // No indent
										true, // Searchable
										true // No padding
									));
								}));
							})
						);
					}
				}));
			}
		}
		
		FToolMenuContext ViewportToolbarContext;
		{
			ViewportToolbarContext.AppendCommandList(GetInfoProvider().GetViewportWidget()->GetCommandList());
		
			UCommonViewportToolbarBaseMenuContext* Context = NewObject<UCommonViewportToolbarBaseMenuContext>();
			Context->ToolbarWidget = SharedThis(this);
			
			Context->Viewport = GetInfoProvider().GetViewportWidget();
			Context->IsViewModeSupported.BindSP(this, &SCommonEditorViewportToolbarBase::IsViewModeSupported);
			
			ViewportToolbarContext.AddObject(Context);
			ViewportToolbarContext.AddExtender(GetViewMenuExtender());
		}
		
		bIsGeneratingToolMenuWidget = true;
		TSharedRef<SWidget> ToolMenuWidget = UToolMenus::Get()->GenerateWidget(ViewportToolbarName, ViewportToolbarContext);
		bIsGeneratingToolMenuWidget = false;

		// Allow the new toolbar to fall back to the old look & behavior
		ChildSlot
		[
			SNew( SBorder )
			.BorderImage(FAppStyle::Get().GetBrush("EditorViewportToolBar.Background"))
			.Cursor(EMouseCursor::Default)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SBox)
					.Visibility_Lambda([]
					{
						PRAGMA_DISABLE_DEPRECATION_WARNINGS
						return UE::UnrealEd::ShowNewViewportToolbars() ? EVisibility::Visible: EVisibility::Collapsed;
						PRAGMA_ENABLE_DEPRECATION_WARNINGS
					})
					[
						ToolMenuWidget
					]
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SBox)
					.Visibility_Lambda([]
					{
						PRAGMA_DISABLE_DEPRECATION_WARNINGS
						return UE::UnrealEd::ShowOldViewportToolbars() ? EVisibility::Visible: EVisibility::Collapsed;
						PRAGMA_ENABLE_DEPRECATION_WARNINGS
					})
					[
						MainBox
					]
				]
			]
		];
		
		// Register the child widget as automatically upgradeable in the viewport
		if (InInfoProvider)
		{
			if (TSharedPtr<SEditorViewport> Viewport = InInfoProvider->GetViewportWidget())
            {
            	Viewport->MarkLegacyToolbarChildAsAutomaticallyUpgradable(ChildSlot.GetWidget());
            }
		}
	}
	else
	{
		ChildSlot
		[
			SNew( SBorder )
			.BorderImage(FAppStyle::Get().GetBrush("EditorViewportToolBar.Background"))
			.Cursor(EMouseCursor::Default)
			[
				MainBox	
			]
		];
	}

	SViewportToolBar::Construct(SViewportToolBar::FArguments());
}

void SCommonEditorViewportToolbarBase::ConstructScreenPercentageMenu(FMenuBuilder& MenuBuilder, FEditorViewportClient* InViewportClient)
{
	UE::UnrealEd::Private::AddScreenPercentageMenu(MenuBuilder, InViewportClient);
}

FText SCommonEditorViewportToolbarBase::GetCameraMenuLabel() const
{
	return UE::UnrealEd::GetCameraSubmenuLabelFromViewportType(GetViewportClient().GetViewportType());
}

FSlateIcon SCommonEditorViewportToolbarBase::GetCameraMenuIcon() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), UE::UnrealEd::GetCameraSubmenuIconFNameFromViewportType(GetViewportClient().GetViewportType()));
}

EVisibility SCommonEditorViewportToolbarBase::GetViewModeOptionsVisibility() const
{
	const FEditorViewportClient& ViewClient = GetViewportClient();
	if (ViewClient.GetViewMode() == VMI_MeshUVDensityAccuracy || ViewClient.GetViewMode() == VMI_MaterialTextureScaleAccuracy || ViewClient.GetViewMode() == VMI_RequiredTextureResolution)
	{
		return EVisibility::SelfHitTestInvisible;
	}
	else
	{
		return EVisibility::Collapsed;
	}
}

TSharedRef<SWidget> SCommonEditorViewportToolbarBase::GenerateViewModeOptionsMenu() const
{
	GetInfoProvider().OnFloatingButtonClicked();
	TSharedRef<SEditorViewport> ViewportRef = GetInfoProvider().GetViewportWidget();
	TSharedRef<FEditorViewportClient> ViewClient = GetInfoProvider().GetViewportWidget()->GetViewportClient().ToSharedRef();
	const UWorld* World = ViewClient->GetWorld();
	
	FMenuBuilder MenuBuilder( true, ViewportRef->GetCommandList() );
	const UE::UnrealEd::FViewModeOptionsArgs ViewModeArgs
	{
		.ViewModeIndex = ViewClient->GetViewMode(),
		.FeatureLevel = World ? World->GetFeatureLevel() : GMaxRHIFeatureLevel,
		.ParamNameMap = ViewClient->GetViewModeParamNameMap(),
		.ViewportClient = ViewClient
	};
	BuildViewModeOptionsMenu(MenuBuilder, ViewModeArgs);
	return MenuBuilder.MakeWidget();
}


TSharedRef<SWidget> SCommonEditorViewportToolbarBase::GenerateOptionsMenu() const
{
	GetInfoProvider().OnFloatingButtonClicked();
	TSharedRef<SEditorViewport> ViewportRef = GetInfoProvider().GetViewportWidget();

	const bool bIsPerspective = GetViewportClient().GetViewportType() == LVT_Perspective;

	const bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder OptionsMenuBuilder(bInShouldCloseWindowAfterMenuSelection, ViewportRef->GetCommandList());
	{
		OptionsMenuBuilder.BeginSection("LevelViewportViewportOptions", LOCTEXT("OptionsMenuHeader", "Viewport Options") );
		{
			OptionsMenuBuilder.AddMenuEntry( FEditorViewportCommands::Get().ToggleRealTime );
			OptionsMenuBuilder.AddMenuEntry( FEditorViewportCommands::Get().ToggleStats );
			OptionsMenuBuilder.AddMenuEntry( FEditorViewportCommands::Get().ToggleFPS );

			if (bIsPerspective)
			{
				OptionsMenuBuilder.AddWidget( UE::UnrealEd::CreateFOVMenuWidget(ViewportRef), LOCTEXT("FOVAngle", "Field of View (H)") );
				OptionsMenuBuilder.AddWidget( UE::UnrealEd::CreateFarViewPlaneMenuWidget(ViewportRef), LOCTEXT("FarViewPlane", "Far View Plane") );
			}

			UE::UnrealEd::Private::AddScreenPercentageMenu(OptionsMenuBuilder, &GetViewportClient());
		}
		OptionsMenuBuilder.EndSection();

 		TSharedPtr<SAssetEditorViewport> AssetEditorViewportPtr = StaticCastSharedRef<SAssetEditorViewport>(ViewportRef);
 		if (AssetEditorViewportPtr.IsValid())
		{
			OptionsMenuBuilder.BeginSection("EditorViewportLayouts");
			{
				OptionsMenuBuilder.AddSubMenu(
					LOCTEXT("ConfigsSubMenu", "Layouts"),
					FText::GetEmpty(),
					FNewMenuDelegate::CreateSP(AssetEditorViewportPtr.Get(), &SAssetEditorViewport::GenerateLayoutMenu));
			}
			OptionsMenuBuilder.EndSection();
		}

		ExtendOptionsMenu(OptionsMenuBuilder);
	}

	return OptionsMenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SCommonEditorViewportToolbarBase::GenerateCameraMenu() const
{
	GetInfoProvider().OnFloatingButtonClicked();
	TSharedRef<SEditorViewport> ViewportRef = GetInfoProvider().GetViewportWidget();

	return UE::UnrealEd::CreateCameraMenuWidget(ViewportRef);
}

TSharedRef<SWidget> SCommonEditorViewportToolbarBase::GenerateShowMenu() const
{
	if (bIsBuildingToolMenu)
	{
		// Defer to the newer system
		return SNullWidget::NullWidget;
	}

	GetInfoProvider().OnFloatingButtonClicked();

	static const FName MenuName("ViewportToolbarBase.Show");
	if (!UToolMenus::Get()->IsMenuRegistered(MenuName))
	{
		UToolMenu* ShowMenu = UToolMenus::Get()->RegisterMenu(MenuName);
		ShowMenu->AddDynamicSection("Flags", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			if (UCommonViewportToolbarBaseMenuContext* ContextObject = InMenu->FindContext<UCommonViewportToolbarBaseMenuContext>())
			{
				if (TSharedPtr<const SCommonEditorViewportToolbarBase> ToolbarWidgetPin = ContextObject->ToolbarWidget.Pin())
				{
					ToolbarWidgetPin->FillShowFlagsMenu(InMenu);
				}
			}
		}));
	}

	FToolMenuContext NewMenuContext;
	UCommonViewportToolbarBaseMenuContext* ContextObject = NewObject<UCommonViewportToolbarBaseMenuContext>();
	ContextObject->ToolbarWidget = SharedThis(this);
	NewMenuContext.AddObject(ContextObject);
	if (TSharedPtr<SEditorViewport> ViewportWidget = GetInfoProvider().GetViewportWidget())
	{
		NewMenuContext.AppendCommandList(GetInfoProvider().GetViewportWidget()->GetCommandList());
	}
	return UToolMenus::Get()->GenerateWidget(MenuName, NewMenuContext);
}

void SCommonEditorViewportToolbarBase::FillShowFlagsMenu(UToolMenu* InMenu) const
{
	FShowFlagMenuCommands::Get().BuildShowFlagsMenu(InMenu);
}

TSharedRef<SWidget> SCommonEditorViewportToolbarBase::GenerateFOVMenu() const
{
	const float FOVMin = 5.f;
	const float FOVMax = 170.f;

	return
		SNew( SBox )
		.HAlign( HAlign_Right )
		[
			SNew( SBox )
			.Padding( FMargin(4.0f, 0.0f, 0.0f, 0.0f) )
			.WidthOverride( 100.0f )
			[
				SNew ( SBorder )
				.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
				.Padding(FMargin(1.0f))
				[
					SNew(SSpinBox<float>)
					.Style(&FAppStyle::Get(), "Menu.SpinBox")
					.Font( FAppStyle::GetFontStyle( TEXT( "MenuItem.Font" ) ) )
					.MinValue(FOVMin)
					.MaxValue(FOVMax)
					.Value(this, &SCommonEditorViewportToolbarBase::OnGetFOVValue)
					.OnValueChanged(this, &SCommonEditorViewportToolbarBase::OnFOVValueChanged)
				]
			]
		];
}

float SCommonEditorViewportToolbarBase::OnGetFOVValue() const
{
	return GetViewportClient().ViewFOV;
}

void SCommonEditorViewportToolbarBase::OnFOVValueChanged(float NewValue) const
{
	FEditorViewportClient& ViewportClient = GetViewportClient();
	ViewportClient.FOVAngle = NewValue;
	ViewportClient.ViewFOV = NewValue;
	ViewportClient.Invalidate();
}

TSharedRef<SWidget> SCommonEditorViewportToolbarBase::GenerateFarViewPlaneMenu() const
{
	return
		SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
			.WidthOverride(100.0f)
			[
				SNew ( SBorder )
				.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
				.Padding(FMargin(1.0f))
				[
					SNew(SSpinBox<float>)
					.Style(&FAppStyle::Get(), "Menu.SpinBox")
					.ToolTipText(LOCTEXT("FarViewPlaneTooltip", "Distance to use as the far view plane, or zero to enable an infinite far view plane"))
					.MinValue(0.0f)
					.MaxValue(100000.0f)
					.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
					.Value(this, &SCommonEditorViewportToolbarBase::OnGetFarViewPlaneValue)
					.OnValueChanged(const_cast<SCommonEditorViewportToolbarBase*>(this), &SCommonEditorViewportToolbarBase::OnFarViewPlaneValueChanged)
				]
			]
		];
}

float SCommonEditorViewportToolbarBase::OnGetFarViewPlaneValue() const
{
	return GetViewportClient().GetFarClipPlaneOverride();
}

void SCommonEditorViewportToolbarBase::OnFarViewPlaneValueChanged(float NewValue)
{
	FEditorViewportClient& ViewportClient = GetViewportClient();
	ViewportClient.OverrideFarClipPlane(NewValue);
	ViewportClient.Invalidate();
}

FReply SCommonEditorViewportToolbarBase::OnRealtimeWarningClicked()
{
	FEditorViewportClient& ViewportClient = GetViewportClient();
	ViewportClient.SetRealtime(true);

	return FReply::Handled();
}

EVisibility SCommonEditorViewportToolbarBase::GetRealtimeWarningVisibility() const
{
	FEditorViewportClient& ViewportClient = GetViewportClient();
	// If the viewport is not realtime and there is no override then realtime is off
	return !ViewportClient.IsRealtime() && !ViewportClient.IsRealtimeOverrideSet() ? EVisibility::Visible : EVisibility::Collapsed;
}

TSharedPtr<FExtender> SCommonEditorViewportToolbarBase::GetCombinedExtenderList(TSharedRef<FExtender> MenuExtender) const
{
	TSharedPtr<FExtender> HostEditorExtenders = GetInfoProvider().GetExtenders();

	TArray<TSharedPtr<FExtender>> Extenders;
	Extenders.Reserve(2);
	Extenders.Add(HostEditorExtenders);
	Extenders.Add(MenuExtender);

	return FExtender::Combine(Extenders);
}

TSharedPtr<FExtender> SCommonEditorViewportToolbarBase::GetViewMenuExtender() const
{
	TSharedRef<FExtender> ViewModeExtender(new FExtender());
	ViewModeExtender->AddMenuExtension(
		TEXT("ViewMode"),
		EExtensionHook::After,
		GetInfoProvider().GetViewportWidget()->GetCommandList(),
		FMenuExtensionDelegate::CreateSP(const_cast<SCommonEditorViewportToolbarBase*>(this), &SCommonEditorViewportToolbarBase::CreateViewMenuExtensions));

	return GetCombinedExtenderList(ViewModeExtender);
}

void SCommonEditorViewportToolbarBase::CreateViewMenuExtensions(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("LevelViewportDeferredRendering", LOCTEXT("DeferredRenderingHeader", "Deferred Rendering") );
	MenuBuilder.EndSection();

//FINDME
// 	MenuBuilder.BeginSection("LevelViewportLandscape", LOCTEXT("LandscapeHeader", "Landscape") );
// 	{
// 		MenuBuilder.AddSubMenu(LOCTEXT("LandscapeLODDisplayName", "LOD"), LOCTEXT("LandscapeLODMenu_ToolTip", "Override Landscape LOD in this viewport"), FNewMenuDelegate::CreateStatic(&Local::BuildLandscapeLODMenu, this), /*Default*/false, FSlateIcon());
// 	}
// 	MenuBuilder.EndSection();
}

ICommonEditorViewportToolbarInfoProvider& SCommonEditorViewportToolbarBase::GetInfoProvider() const
{
	return *InfoProviderPtr.Pin().Get();
}

FEditorViewportClient& SCommonEditorViewportToolbarBase::GetViewportClient() const
{
	return *GetInfoProvider().GetViewportWidget()->GetViewportClient().Get();
}

TSharedRef<SEditorViewportViewMenu> SCommonEditorViewportToolbarBase::MakeViewMenu()
{
	// Mark that the viewport uses the default view menu, and is potentially upgradable.
	bUsesDefaultViewMenu = true;
	
	TSharedRef<SEditorViewport> ViewportRef = GetInfoProvider().GetViewportWidget();

	return SNew(SEditorViewportViewMenu, ViewportRef, SharedThis(this))
		.Cursor(EMouseCursor::Default)
		.MenuExtenders(GetViewMenuExtender());
}

FText SCommonEditorViewportToolbarBase::GetScalabilityWarningLabel() const
{
	const int32 QualityLevel = Scalability::GetQualityLevels().GetMinQualityLevel();
	if (QualityLevel >= 0)
	{
		return FText::Format(LOCTEXT("ScalabilityWarning", "Scalability: {0}"), Scalability::GetScalabilityNameFromQualityLevel(QualityLevel));
	}

	return FText::GetEmpty();
}

EVisibility SCommonEditorViewportToolbarBase::GetScalabilityWarningVisibility() const
{
	return UE::UnrealEd::IsScalabilityWarningVisible() && GetShowScalabilityMenu() ? EVisibility::Visible
																				   : EVisibility::Collapsed;
}

TSharedRef<SWidget> SCommonEditorViewportToolbarBase::GetScalabilityWarningMenuContent() const
{
	return
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		[
			SNew(SScalabilitySettings)
		];
}

//////////////////////////////////////////////////////////////////////////
/// Automatic Legacy Upgrade support
/// --------------------------------------------------------------------
/// These functions are seriously hairy so that clients of this widget
/// with simple needs are converted directly to the new form.
//////////////////////////////////////////////////////////////////////////
TSharedPtr<SWidget> SCommonEditorViewportToolbarBase::MakeLegacyShowMenu() const
{
	bIsBuildingToolMenu = true;
	const TSharedRef<SWidget> Menu = GenerateShowMenu();
	bIsBuildingToolMenu = false;
	
	return Menu == SNullWidget::NullWidget ? TSharedPtr<SWidget>(nullptr) : Menu.ToSharedPtr();
}

void SCommonEditorViewportToolbarBase::ExtendOptionsMenu(FMenuBuilder& OptionsMenuBuilder) const
{
	// This flag allows the new toolbar to detect whether a settings menu needs to be created.
	bHasExtendedSettingsMenu = false;
}

void SCommonEditorViewportToolbarBase::ExtendLeftAlignedToolbarSlots(TSharedPtr<SHorizontalBox> MainBoxPtr, TSharedPtr<SViewportToolBar> ParentToolBarPtr) const
{
	// This flag allows detection on whether the client intends to extend the left side. 
	bHasExtendedLeftSide = false;
}

bool SCommonEditorViewportToolbarBase::ShouldCreateOptionsMenu() const
{
	FMenuBuilder LegacyMenuBuilder(true, nullptr);
	ExtendOptionsMenu(LegacyMenuBuilder);
	return bHasExtendedSettingsMenu;
}

#undef LOCTEXT_NAMESPACE
