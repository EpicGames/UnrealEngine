// Copyright Epic Games, Inc. All Rights Reserved.

#include "SComponentClassCombo.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SComboBox.h"
#include "Styling/AppStyle.h"
#include "Components/SceneComponent.h"
#include "Engine/Blueprint.h"
#include "Engine/Selection.h"
#include "Editor.h"
#include "Styling/SlateIconFinder.h"
#include "ComponentAssetBroker.h"
#include "ComponentTypeRegistry.h"
#include "EditorClassUtils.h"
#include "Widgets/Input/SSearchBox.h"
#include "SListViewSelectorDropdownMenu.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "SPositiveActionButton.h"
#include "Widgets/Layout/SSeparator.h"
#include "Modules/ModuleManager.h"
#include "ClassViewerModule.h"
#include "ClassViewerFilter.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "ComponentClassCombo"

FString FComponentClassComboEntry::GetClassDisplayName() const
{
	return ComponentClass != nullptr ? ComponentClass->GetDisplayNameText().ToString() : ComponentName;
}

FString FComponentClassComboEntry::GetClassName() const
{
	return ComponentClass != nullptr ? ComponentClass->GetName() : ComponentName;
}

void FComponentClassComboEntry::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(ComponentClass.GetGCPtr());
	UClass* RawClass = ComponentClass;
	if(RawClass && RawClass->IsChildOf(UActorComponent::StaticClass()))
	{
		ComponentClass = RawClass;
	}
	else
	{
		ComponentClass = nullptr;
	}

	Collector.AddReferencedObject(IconClass);
}

bool FComponentClassComboEntry::OnBlueprintGeneratedClassUnloaded(UBlueprintGeneratedClass* BlueprintGeneratedClass)
{
	bool bModified = false;

	if (BlueprintGeneratedClass == ComponentClass)
	{
		ComponentClass = nullptr;
		bModified = true;
	}

	if (BlueprintGeneratedClass == IconClass)
	{
		IconClass = nullptr;
		bModified = true;
	}

	return bModified;
}

void SComponentClassCombo::Construct(const FArguments& InArgs)
{
	PrevSelectedIndex = INDEX_NONE;
	OnComponentClassSelected = InArgs._OnComponentClassSelected;
	OnSubobjectClassSelected = InArgs._OnSubobjectClassSelected;
	TextFilter = MakeShared<FTextFilterExpressionEvaluator>(ETextFilterExpressionEvaluatorMode::BasicString);

	ComponentClassFilterData.InitOptions = MakeShared<FClassViewerInitializationOptions>();
	ComponentClassFilterData.InitOptions->ClassFilters.Append(InArgs._CustomClassFilters);
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");
	ComponentClassFilterData.ClassFilter = ClassViewerModule.CreateClassFilter(*ComponentClassFilterData.InitOptions);
	ComponentClassFilterData.FilterFuncs = ClassViewerModule.CreateFilterFuncs();
	ClassViewerModule.GetOnGlobalClassViewerFilterModified().AddRaw(this, &SComponentClassCombo::UpdateComponentClassList);

	FComponentTypeRegistry::Get().SubscribeToComponentList(ComponentClassList).AddRaw(this, &SComponentClassCombo::UpdateComponentClassList);

	UpdateComponentClassList();

	SAssignNew(ComponentClassListView, SListView<FComponentClassComboEntryPtr>)
		.ListItemsSource(&FilteredComponentClassList)
		.OnSelectionChanged( this, &SComponentClassCombo::OnAddComponentSelectionChanged )
		.OnGenerateRow( this, &SComponentClassCombo::GenerateAddComponentRow )
		.SelectionMode(ESelectionMode::Single);

	SAssignNew(SearchBox, SSearchBox)
		.HintText( LOCTEXT( "BlueprintAddComponentSearchBoxHint", "Search Components" ) )
		.OnTextChanged( this, &SComponentClassCombo::OnSearchBoxTextChanged )
		.OnTextCommitted( this, &SComponentClassCombo::OnSearchBoxTextCommitted );

	ChildSlot
	[
		SAssignNew(AddNewButton, SPositiveActionButton)
		.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
		.Text(LOCTEXT("Add", "Add"))
		.OnComboBoxOpened(this, &SComponentClassCombo::ClearSelection)
		.MenuContent()
		[
			SNew(SListViewSelectorDropdownMenu<FComponentClassComboEntryPtr>, SearchBox, ComponentClassListView)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("Menu.Background"))
				.Padding(2.0f)
				[
					SNew(SBox)
					.WidthOverride(250.0f)
					[				
						SNew(SVerticalBox)
						+SVerticalBox::Slot()
						.Padding(1.f)
						.AutoHeight()
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							[
								SearchBox.ToSharedRef()
							]
							+SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(2.0f, 2.0f)
							[
								SNew(SComboButton)
								.ContentPadding(0.0f)
								.ForegroundColor(FSlateColor::UseForeground())
								.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
								.HasDownArrow(false)
								.Visibility(this, &SComponentClassCombo::GetFilterOptionsButtonVisibility)
								.OnGetMenuContent(this, &SComponentClassCombo::GetFilterOptionsMenuContent)
								.ButtonContent()
								[
									SNew(SImage)
									.Image(FAppStyle::Get().GetBrush("Icons.Settings"))
									.ColorAndOpacity(FSlateColor::UseForeground())
								]
							]
						]
						+SVerticalBox::Slot()
						.MaxHeight(400)
						[
							ComponentClassListView.ToSharedRef()
						]
					]
				]
			]
		]
	];

	

	ComponentClassListView->EnableToolTipForceField( true );
	// The button can automatically handle setting focus to a specified control when the combo button is opened
	AddNewButton->SetMenuContentWidgetToFocus( SearchBox );
}

SComponentClassCombo::~SComponentClassCombo()
{
	FComponentTypeRegistry::Get().GetOnComponentTypeListChanged().RemoveAll(this);
	
	if (FClassViewerModule* ClassViewerModule = FModuleManager::Get().GetModulePtr<FClassViewerModule>("ClassViewer"))
	{
		ClassViewerModule->GetOnGlobalClassViewerFilterModified().RemoveAll(this);
	}
}

void SComponentClassCombo::ClearSelection()
{
	SearchBox->SetText(FText::GetEmpty());

	PrevSelectedIndex = INDEX_NONE;
	
	// Clear the selection in such a way as to also clear the keyboard selector
	ComponentClassListView->SetSelection(NULL, ESelectInfo::OnNavigation);

	// Make sure we scroll to the top
	if (ComponentClassList->Num() > 0)
	{
		ComponentClassListView->RequestScrollIntoView((*ComponentClassList)[0]);
	}
}

void SComponentClassCombo::GenerateFilteredComponentList()
{
	FilteredComponentClassList.Reset();

	int32 LastHeadingIndex = INDEX_NONE;
	FComponentClassComboEntryPtr* LastHeadingPtr = nullptr;

	int32 LastSeparatorIndex = INDEX_NONE;
	FComponentClassComboEntryPtr* LastSeparatorPtr = nullptr;

	const bool bHasFilterText = !TextFilter->GetFilterText().IsEmpty();

	for (int32 ComponentIndex = 0; ComponentIndex < ComponentClassList->Num(); ComponentIndex++)
	{
		FComponentClassComboEntryPtr& CurrentEntry = (*ComponentClassList)[ComponentIndex];

		if (CurrentEntry->IsHeading())
		{
			LastHeadingIndex = FilteredComponentClassList.Num();
			LastHeadingPtr = &CurrentEntry;
		}
		else if (CurrentEntry->IsSeparator())
		{
			LastSeparatorIndex = FilteredComponentClassList.Num();
			LastSeparatorPtr = &CurrentEntry;
		}
		else if(CurrentEntry->IsClass())
		{
			// Disallow class entries that are not to be seen when searching via text.
			bool bAllowEntry = !bHasFilterText || CurrentEntry->IsIncludedInFilter();
			if(bAllowEntry)
			{
				// Disallow class entries that don't match the custom class filter, if set.
				bAllowEntry = IsComponentClassAllowed(CurrentEntry);
				if (bAllowEntry && bHasFilterText)
				{
					// Finally, disallow class entries that don't match the search box text.
					const FString ComponentName = CurrentEntry->GetClassName();
					bAllowEntry = TextFilter->TestTextFilter(FBasicStringFilterExpressionContext(ComponentName));
					if(!bAllowEntry)
					{
						const FString FriendlyComponentName = GetSanitizedComponentName(CurrentEntry);
						bAllowEntry = TextFilter->TestTextFilter(FBasicStringFilterExpressionContext(FriendlyComponentName));
					}
				}
			}

			if (bAllowEntry)
			{
				// Add the heading first if it hasn't already been added
				if (LastHeadingPtr && LastHeadingIndex != INDEX_NONE)
				{
					FilteredComponentClassList.Insert(*LastHeadingPtr, LastHeadingIndex);
					LastHeadingIndex = INDEX_NONE;
					LastHeadingPtr = nullptr;
				}

				// Add the separator next so that it will precede the heading
				if (LastSeparatorPtr && LastSeparatorIndex != INDEX_NONE)
				{
					FilteredComponentClassList.Insert(*LastSeparatorPtr, LastSeparatorIndex);
					LastSeparatorIndex = INDEX_NONE;
					LastSeparatorPtr = nullptr;
				}

				// Add the class
				FilteredComponentClassList.Add(CurrentEntry);
			}
		}
	}

	if (ComponentClassListView.IsValid())
	{
		// Select the first non-category item that passed the filter
		for (FComponentClassComboEntryPtr& TestEntry : FilteredComponentClassList)
		{
			if (TestEntry->IsClass())
			{
				ComponentClassListView->SetSelection(TestEntry, ESelectInfo::OnNavigation);
				break;
			}
		}
	}
}

FText SComponentClassCombo::GetCurrentSearchString() const
{
	return TextFilter->GetFilterText();
}

void SComponentClassCombo::OnSearchBoxTextChanged( const FText& InSearchText )
{
	TextFilter->SetFilterText(InSearchText);
	SearchBox->SetError(TextFilter->GetFilterErrorText());

	UpdateComponentClassList();
}

void SComponentClassCombo::OnSearchBoxTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo)
{
	if(CommitInfo == ETextCommit::OnEnter)
	{
		auto SelectedItems = ComponentClassListView->GetSelectedItems();
		if(SelectedItems.Num() > 0)
		{
			ComponentClassListView->SetSelection(SelectedItems[0]);
		}
	}
}

// @todo: move this to FKismetEditorUtilities
static UClass* GetAuthoritativeBlueprintClass(UBlueprint const* const Blueprint)
{
	UClass* BpClass = (Blueprint->SkeletonGeneratedClass != nullptr) ? Blueprint->SkeletonGeneratedClass :
		Blueprint->GeneratedClass;

	if (BpClass == nullptr)
	{
		BpClass = Blueprint->ParentClass;
	}

	UClass* AuthoritativeClass = BpClass;
	if (BpClass != nullptr)
	{
		AuthoritativeClass = BpClass->GetAuthoritativeClass();
	}
	return AuthoritativeClass;
}

void SComponentClassCombo::OnAddComponentSelectionChanged( FComponentClassComboEntryPtr InItem, ESelectInfo::Type SelectInfo )
{
	if ( InItem.IsValid() && InItem->IsClass() && SelectInfo != ESelectInfo::OnNavigation)
	{
		// We don't want the item to remain selected
		ClearSelection();

		if ( InItem->IsClass() )
		{
			// Neither do we want the combo dropdown staying open once the user has clicked on a valid option
			AddNewButton->SetIsMenuOpen(false, false);

			if( OnComponentClassSelected.IsBound() || OnSubobjectClassSelected.IsBound() )
			{
				UClass* ComponentClass = InItem->GetComponentClass();
				if (ComponentClass == nullptr)
				{
					// The class is not loaded yet, so load it:
					if (UObject* LoadedObject = LoadObject<UObject>(nullptr, *InItem->GetComponentPath()))
					{
						if (UClass* LoadedClass = Cast<UClass>(LoadedObject))
						{
							ComponentClass = LoadedClass;
						}
						else if (UBlueprint* LoadedBP = Cast<UBlueprint>(LoadedObject))
						{
							ComponentClass = GetAuthoritativeBlueprintClass(LoadedBP);
						}
					}
				}
				
				FSubobjectDataHandle NewActorCompHandle =
					OnSubobjectClassSelected.IsBound() ?
					OnSubobjectClassSelected.Execute(ComponentClass, InItem->GetComponentCreateAction(), InItem->GetAssetOverride())
					: FSubobjectDataHandle::InvalidHandle;

				if(NewActorCompHandle.IsValid())
				{
					InItem->GetOnSubobjectCreated().ExecuteIfBound(NewActorCompHandle);
				}
			}
		}
	}
	else if ( InItem.IsValid() && SelectInfo != ESelectInfo::OnMouseClick )
	{
		int32 SelectedIdx = INDEX_NONE;
		if (FilteredComponentClassList.Find(InItem, /*out*/ SelectedIdx))
		{
			if (!InItem->IsClass())
			{
				int32 SelectionDirection = SelectedIdx - PrevSelectedIndex;

				// Update the previous selected index
				PrevSelectedIndex = SelectedIdx;

				// Make sure we select past the category header if we started filtering with it selected somehow (avoiding the infinite loop selecting the same item forever)
				if (SelectionDirection == 0)
				{
					SelectionDirection = 1;
				}

				if(SelectedIdx + SelectionDirection >= 0 && SelectedIdx + SelectionDirection < FilteredComponentClassList.Num())
				{
					ComponentClassListView->SetSelection(FilteredComponentClassList[SelectedIdx + SelectionDirection], ESelectInfo::OnNavigation);
				}
			}
			else
			{
				// Update the previous selected index
				PrevSelectedIndex = SelectedIdx;
			}
		}
	}
}

TSharedRef<ITableRow> SComponentClassCombo::GenerateAddComponentRow( FComponentClassComboEntryPtr Entry, const TSharedRef<STableViewBase> &OwnerTable ) const
{
	check( Entry->IsHeading() || Entry->IsSeparator() || Entry->IsClass() );

	if ( Entry->IsHeading() )
	{
		return 
			SNew( STableRow< TSharedPtr<FString> >, OwnerTable )
				.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.NoHoverTableRow"))
				.ShowSelection(false)
			[
				SNew(SBox)
				.Padding(1.f)
				[
					SNew(STextBlock)
					.Text(FText::FromString(Entry->GetHeadingText()))
					.TextStyle(FAppStyle::Get(), TEXT("Menu.Heading"))
				]
			];
	}
	else if ( Entry->IsSeparator() )
	{
		return 
			SNew( STableRow< TSharedPtr<FString> >, OwnerTable )
				.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.NoHoverTableRow"))
				.ShowSelection(false)
			[
				SNew(SSeparator)
				.SeparatorImage(FAppStyle::Get().GetBrush("Menu.Separator"))
				.Thickness(1.0f)
			];
	}
	else
	{
		
		return
			SNew( SComboRow< TSharedPtr<FString> >, OwnerTable )
			.ToolTip( GetComponentToolTip(Entry) )
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SSpacer)
					.Size(FVector2D(8.0f,1.0f))
				]
				+SHorizontalBox::Slot()
				.Padding(1.0f)
				.AutoWidth()
				[
					SNew(SImage)
					.Image( FSlateIconFinder::FindIconBrushForClass( Entry->GetIconOverrideBrushName() == NAME_None ? Entry->GetIconClass() : nullptr, Entry->GetIconOverrideBrushName() ) )
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SSpacer)
					.Size(FVector2D(3.0f,1.0f))
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.HighlightText(this, &SComponentClassCombo::GetCurrentSearchString)
					.Text(this, &SComponentClassCombo::GetFriendlyComponentName, Entry)
				]
			];
	}
}

void SComponentClassCombo::UpdateComponentClassList()
{
	// Regenerate the filtered list
	GenerateFilteredComponentList();

	// Ask the combo to update its contents on next tick
	if (ComponentClassListView.IsValid())
	{
		ComponentClassListView->RequestListRefresh();
	}
}

FText SComponentClassCombo::GetFriendlyComponentName(FComponentClassComboEntryPtr Entry) const
{
	// Get a user friendly string from the component name
	FString FriendlyComponentName;

	if( Entry->GetComponentCreateAction() == EComponentCreateAction::CreateNewCPPClass )
	{
		FriendlyComponentName = LOCTEXT("NewCPPComponentFriendlyName", "New C++ Component...").ToString();
	}
	else if (Entry->GetComponentCreateAction() == EComponentCreateAction::CreateNewBlueprintClass )
	{
		FriendlyComponentName = LOCTEXT("NewBlueprintComponentFriendlyName", "New Blueprint Script Component...").ToString();
	}
	else
	{
		FriendlyComponentName = GetSanitizedComponentName(Entry);

		// Don't try to match up assets for USceneComponent it will match lots of things and doesn't have any nice behavior for asset adds 
		if (Entry->GetComponentClass() != USceneComponent::StaticClass() && Entry->GetComponentNameOverride().IsEmpty())
		{
			// Search the selected assets and look for any that can be used as a source asset for this type of component
			// If there is one we append the asset name to the component name, if there are many we append "Multiple Assets"
			FString AssetName;
			UObject* PreviousMatchingAsset = NULL;

			FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();
			USelection* Selection = GEditor->GetSelectedObjects();
			for(FSelectionIterator ObjectIter(*Selection); ObjectIter; ++ObjectIter)
			{
				UObject* Object = *ObjectIter;
				check(Object);
				UClass* Class = Object->GetClass();

				TArray<TSubclassOf<UActorComponent> > ComponentClasses = FComponentAssetBrokerage::GetComponentsForAsset(Object);
				for(int32 ComponentIndex = 0; ComponentIndex < ComponentClasses.Num(); ComponentIndex++)
				{
					if(ComponentClasses[ComponentIndex]->IsChildOf(Entry->GetComponentClass()))
					{
						if(AssetName.IsEmpty())
						{
							// If there is no previous asset then we just accept the name
							AssetName = Object->GetName();
							PreviousMatchingAsset = Object;
						}
						else
						{
							// if there is a previous asset then check that we didn't just find multiple appropriate components
							// in a single asset - if the asset differs then we don't display the name, just "Multiple Assets"
							if(PreviousMatchingAsset != Object)
							{
								AssetName = LOCTEXT("MultipleAssetsForComponentAnnotation", "Multiple Assets").ToString();
								PreviousMatchingAsset = Object;
							}
						}
					}
				}
			}

			if(!AssetName.IsEmpty())
			{
				FriendlyComponentName += FString(" (") + AssetName + FString(")");
			}
		}
	}
	return FText::FromString(FriendlyComponentName);
}

FString SComponentClassCombo::GetSanitizedComponentName(FComponentClassComboEntryPtr Entry)
{
	FString DisplayName;
	if (Entry->GetComponentNameOverride() != FString())
	{
		DisplayName = Entry->GetComponentNameOverride();
	}
	else if (UClass* ComponentClass = Entry->GetComponentClass())
	{
		if (ComponentClass->HasMetaData(TEXT("DisplayName")))
		{
			DisplayName = ComponentClass->GetMetaData(TEXT("DisplayName"));
		}
		else
		{
			DisplayName = ComponentClass->GetDisplayNameText().ToString();
			if (!ComponentClass->HasAnyClassFlags(CLASS_CompiledFromBlueprint))
			{
				DisplayName.RemoveFromEnd(TEXT("Component"), ESearchCase::IgnoreCase);
			}
		}
	}
	else
	{
		DisplayName = Entry->GetClassDisplayName();
	}
	return FName::NameToDisplayString(DisplayName, false);
}

TSharedRef<SToolTip> SComponentClassCombo::GetComponentToolTip(FComponentClassComboEntryPtr Entry) const
{
	// Special handling for the "New..." options
	if (Entry->GetComponentCreateAction() == EComponentCreateAction::CreateNewCPPClass)
	{
		return SNew(SToolTip)
			.Text(LOCTEXT("NewCPPComponentToolTip", "Create a custom actor component using C++"));
	}
	else if (Entry->GetComponentCreateAction() == EComponentCreateAction::CreateNewBlueprintClass)
	{
		return SNew(SToolTip)
			.Text(LOCTEXT("NewBlueprintComponentToolTip", "Create a custom actor component using Blueprints"));
	}
	
	// Handle components which have a currently loaded class
	if (const UClass* ComponentClass = Entry->GetComponentClass())
	{
		return FEditorClassUtils::GetTooltip(ComponentClass);
	}

	// Fallback for components that don't currently have a loaded class
	return SNew(SToolTip)
		.Text(FText::FromString(Entry->GetClassDisplayName()));
}

bool SComponentClassCombo::IsComponentClassAllowed(FComponentClassComboEntryPtr Entry) const
{
	if (Entry.IsValid() && Entry->IsClass())
	{
		check(ComponentClassFilterData.InitOptions.IsValid());
		check(ComponentClassFilterData.ClassFilter.IsValid());
		check(ComponentClassFilterData.FilterFuncs.IsValid());

		if (const UClass* ComponentClass = Entry->GetComponentClass())
		{
			return ComponentClassFilterData.ClassFilter->IsClassAllowed(*ComponentClassFilterData.InitOptions, ComponentClass, ComponentClassFilterData.FilterFuncs.ToSharedRef());
		}
		else
		{
			TSharedPtr<IUnloadedBlueprintData> UnloadedBlueprintData = Entry->GetUnloadedBlueprintData();
			if (UnloadedBlueprintData.IsValid())
			{
				return ComponentClassFilterData.ClassFilter->IsUnloadedClassAllowed(*ComponentClassFilterData.InitOptions, UnloadedBlueprintData.ToSharedRef(), ComponentClassFilterData.FilterFuncs.ToSharedRef());
			}
		}
	}

	// Allow all entries to otherwise pass by default.
	return true;
}

void SComponentClassCombo::GetComponentClassFilterOptions(TArray<TSharedRef<FClassViewerFilterOption>>& OutFilterOptions) const
{
	if (ComponentClassFilterData.InitOptions.IsValid())
	{
		TArray<TSharedRef<FClassViewerFilterOption>> FilterOptions;
		if(ComponentClassFilterData.ClassFilter.IsValid())
		{
			ComponentClassFilterData.ClassFilter->GetFilterOptions(FilterOptions);
			OutFilterOptions.Append(FilterOptions);
		}

		for (const TSharedRef<IClassViewerFilter>& ClassFilter : ComponentClassFilterData.InitOptions->ClassFilters)
		{
			FilterOptions.Reset();
			ClassFilter->GetFilterOptions(FilterOptions);

			OutFilterOptions.Append(FilterOptions);
		}
	}
}

TSharedRef<SWidget> SComponentClassCombo::GetFilterOptionsMenuContent()
{
	const bool bCloseSelfOnly = true;
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, /* InCommandList = */nullptr, /* InExtender = */nullptr, bCloseSelfOnly);

	TArray<TSharedRef<FClassViewerFilterOption>> ClassFilterOptions;
	GetComponentClassFilterOptions(ClassFilterOptions);

	if (ClassFilterOptions.Num() > 0)
	{
		MenuBuilder.BeginSection("ClassFilterOptions", LOCTEXT("ClassFilterOptionsHeading", "Class Filters"));
		{
			for (const TSharedRef<FClassViewerFilterOption>& ClassFilterOption : ClassFilterOptions)
			{
				MenuBuilder.AddMenuEntry(
					ClassFilterOption->LabelText,
					ClassFilterOption->ToolTipText,
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateSP(this, &SComponentClassCombo::ToggleFilterOption, ClassFilterOption),
						FCanExecuteAction(),
						FIsActionChecked::CreateSP(this, &SComponentClassCombo::IsFilterOptionEnabled, ClassFilterOption)
					),
					NAME_None,
					EUserInterfaceActionType::ToggleButton
				);
			}
		}
		MenuBuilder.EndSection();
	}

	return MenuBuilder.MakeWidget();
}

void SComponentClassCombo::ToggleFilterOption(TSharedRef<FClassViewerFilterOption> FilterOption)
{
	FilterOption->bEnabled = !FilterOption->bEnabled;

	if (FilterOption->OnOptionChanged.IsBound())
	{
		FilterOption->OnOptionChanged.Execute(FilterOption->bEnabled);
	}

	UpdateComponentClassList();
}

bool SComponentClassCombo::IsFilterOptionEnabled(TSharedRef<FClassViewerFilterOption> FilterOption) const
{
	return FilterOption->bEnabled;
}

EVisibility SComponentClassCombo::GetFilterOptionsButtonVisibility() const
{
	TArray<TSharedRef<FClassViewerFilterOption>> FilterOptions;
	GetComponentClassFilterOptions(FilterOptions);

	return FilterOptions.Num() > 0 ? EVisibility::Visible : EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE
