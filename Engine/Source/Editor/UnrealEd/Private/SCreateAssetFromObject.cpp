// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCreateAssetFromObject.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "GameFramework/Actor.h"
#include "AssetRegistry/AssetData.h"
#include "Engine/Selection.h"
#include "Editor.h"
#include "Misc/FileHelper.h"
#include "IAssetTools.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "AssetToolsModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "PackageTools.h"
#include "SPrimaryButton.h"

#define LOCTEXT_NAMESPACE "SCreateAssetFromActor"

void SCreateAssetFromObject::Construct(const FArguments& InArgs, TSharedPtr<SWindow> InParentWindow)
{
	AssetFilenameSuffix = InArgs._AssetFilenameSuffix;
	HeadingText = InArgs._HeadingText;
	CreateButtonText = InArgs._CreateButtonText;
	OnCreateAssetAction = InArgs._OnCreateAssetAction;

	if (InArgs._AssetPath.IsEmpty())
	{
		AssetPath = FString("/Game");
	}
	else
	{
		AssetPath = InArgs._AssetPath;
	}

	bIsReportingError = false;

	FPathPickerConfig PathPickerConfig;
	PathPickerConfig.DefaultPath = AssetPath;
	PathPickerConfig.OnPathSelected = FOnPathSelected::CreateRaw(this, &SCreateAssetFromObject::OnSelectAssetPath);

	SelectionDelegateHandle = USelection::SelectionChangedEvent.AddSP(this, &SCreateAssetFromObject::OnLevelSelectionChanged);

	// Set up PathPickerConfig.
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	ParentWindow = InParentWindow;

	FString PackageName;
	ActorInstanceLabel.Empty();

	if( InArgs._DefaultNameOverride.IsEmpty() )
	{
		USelection* SelectedActors = GEditor->GetSelectedActors();
		for(FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
		{
			AActor* Actor = Cast<AActor>(*Iter);
			if(Actor)
			{
				ActorInstanceLabel += Actor->GetActorLabel();
				ActorInstanceLabel += TEXT("_");
				break;
			}
		}
	}
	else
	{
		ActorInstanceLabel = InArgs._DefaultNameOverride.ToString();
	}

	ActorInstanceLabel = UPackageTools::SanitizePackageName(ActorInstanceLabel + AssetFilenameSuffix);

	FString AssetName = ActorInstanceLabel;
	FString BasePath = AssetPath / AssetName;
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	AssetToolsModule.Get().CreateUniqueAssetName(BasePath, TEXT(""), PackageName, AssetName);

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			ContentBrowserModule.Get().CreatePathPicker(PathPickerConfig)
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
			.Padding(16.f)
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(0.0f, 0.0f, 8.0f, 0.0f)
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(HeadingText)
					]

					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						SAssignNew(FileNameWidget, SEditableTextBox)
						.Text(FText::FromString(AssetName))
						.OnTextChanged(this, &SCreateAssetFromObject::OnFilenameChanged)
					]
				]

				+ SVerticalBox::Slot()
				.HAlign(HAlign_Right)
				.AutoHeight()
				.Padding(0.f, 16.f, 0.f, 0.f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SPrimaryButton)
						.OnClicked(this, &SCreateAssetFromObject::OnCreateAssetFromActorClicked)
						.Text(CreateButtonText)
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(16.f, 0.f, 0.f, 0.f)
					[
						SNew(SButton)
						.Text(LOCTEXT("CancelButtonText", "Cancel"))
						.OnClicked(this, &SCreateAssetFromObject::OnCancelCreateAssetFromActor)
					]
				]
			]
		]
	];

	OnFilenameChanged(FText::FromString(AssetName));
}

void SCreateAssetFromObject::RequestDestroyParentWindow()
{
	USelection::SelectionChangedEvent.Remove(SelectionDelegateHandle);

	if (ParentWindow.IsValid())
	{
		ParentWindow.Pin()->RequestDestroyWindow();
	}
}

FReply SCreateAssetFromObject::OnCreateAssetFromActorClicked()
{
	RequestDestroyParentWindow();
	OnCreateAssetAction.ExecuteIfBound(AssetPath / FileNameWidget->GetText().ToString());
	return FReply::Handled();
}

FReply SCreateAssetFromObject::OnCancelCreateAssetFromActor()
{
	RequestDestroyParentWindow();
	return FReply::Handled();
}

void SCreateAssetFromObject::OnSelectAssetPath(const FString& Path)
{
	AssetPath = Path;
	OnFilenameChanged(FileNameWidget->GetText());
}

void SCreateAssetFromObject::OnLevelSelectionChanged(UObject* InObjectSelected)
{
	// When actor selection changes, this window should be destroyed.
	RequestDestroyParentWindow();
}

void SCreateAssetFromObject::OnFilenameChanged(const FText& InNewName)
{
	TArray<FAssetData> AssetData;
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	AssetRegistryModule.Get().GetAssetsByPath(FName(*AssetPath), AssetData);

	FText ErrorText;
	if (!FFileHelper::IsFilenameValidForSaving(InNewName.ToString(), ErrorText) || !FName(*InNewName.ToString()).IsValidObjectName(ErrorText))
	{
		FileNameWidget->SetError(ErrorText);
		bIsReportingError = true;
		return;
	}
	else
	{
		// Check to see if the name conflicts
		for (auto Iter = AssetData.CreateConstIterator(); Iter; ++Iter)
		{
			if (Iter->AssetName.ToString() == InNewName.ToString())
			{
				FileNameWidget->SetError(LOCTEXT("AssetInUseError", "Asset name already in use!"));
				bIsReportingError = true;
				return;
			}
		}
	}

	FileNameWidget->SetError(FText::FromString(TEXT("")));
	bIsReportingError = false;
}

bool SCreateAssetFromObject::IsCreateAssetFromActorEnabled() const
{
	return !bIsReportingError;
}

#undef LOCTEXT_NAMESPACE
