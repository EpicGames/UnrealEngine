// Copyright Epic Games, Inc. All Rights Reserved.


#include "SCurveEditor.h"
#include "Fonts/SlateFontInfo.h"
#include "Rendering/DrawElements.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/Input/SCheckBox.h"
#include "Styling/AppStyle.h"
#include "Factories/Factory.h"
#include "Factories/CurveFactory.h"
#include "Editor.h"
#include "Curves/SimpleCurve.h"
#include "CurveEditorCommands.h"
#include "CurveEditorSettings.h"
#include "ScopedTransaction.h"
#include "Framework/Commands/GenericCommands.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/STextEntryPopup.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "IPropertyUtilities.h"
#include "PropertyHandle.h"

#define LOCTEXT_NAMESPACE "SCurveEditor"
 
DEFINE_LOG_CATEGORY(LogCurveEditor);

const static FVector2D	CONST_KeySize		= FVector2D(11,11);
const static FVector2D	CONST_TangentSize	= FVector2D(7,7);
const static FVector2D	CONST_CurveSize		= FVector2D(12,12);

const static float		CONST_FitMargin		= 0.05f;
const static float		CONST_MinViewRange	= 0.01f;
const static float		CONST_DefaultZoomRange	= 1.0f;
const static float      CONST_KeyTangentOffset = 60.0f;


//////////////////////////////////////////////////////////////////////////
// SCurveEditor

void SCurveEditor::Construct(const FArguments& InArgs)
{
	CurveFactory = NULL;
	Commands = TSharedPtr< FUICommandList >(new FUICommandList);
	CurveOwner = NULL;
	
	// view input
	ViewMinInput = InArgs._ViewMinInput;
	ViewMaxInput = InArgs._ViewMaxInput;
	// data input - only used when it's set
	DataMinInput = InArgs._DataMinInput;
	DataMaxInput = InArgs._DataMaxInput;

	ViewMinOutput = InArgs._ViewMinOutput;
	ViewMaxOutput = InArgs._ViewMaxOutput;

	InputSnap = InArgs._InputSnap;
	OutputSnap = InArgs._OutputSnap;
	bInputSnappingEnabled = InArgs._InputSnappingEnabled;
	bOutputSnappingEnabled = InArgs._OutputSnappingEnabled;
	bShowTimeInFrames = InArgs._ShowTimeInFrames;

	bZoomToFitVertical = InArgs._ZoomToFitVertical;
	bZoomToFitHorizontal = InArgs._ZoomToFitHorizontal;
	DesiredSize = InArgs._DesiredSize;

	GridColor = InArgs._GridColor;

	bIsUsingSlider = false;
	bAllowAutoFrame = true;
	bRequireFocusToZoom = false;

	bIsPendingRebuilt = false;

	// if editor size is set, use it, otherwise, use default value
	if (DesiredSize.Get().IsZero())
	{
		DesiredSize.Set(FVector2D(128, 64));
	}

	TimelineLength = InArgs._TimelineLength;
	SetInputViewRangeHandler = InArgs._OnSetInputViewRange;
	SetOutputViewRangeHandler = InArgs._OnSetOutputViewRange;
	bDrawCurve = InArgs._DrawCurve;
	bHideUI = InArgs._HideUI;
	bAllowZoomOutput = InArgs._AllowZoomOutput;
	bAlwaysDisplayColorCurves = InArgs._AlwaysDisplayColorCurves;
	bAlwaysHideGradientEditor = InArgs._AlwaysHideGradientEditor;
	bShowZoomButtons = InArgs._ShowZoomButtons;
	bShowCurveSelector = InArgs._ShowCurveSelector;
	bDrawInputGridNumbers = InArgs._ShowInputGridNumbers;
	bDrawOutputGridNumbers = InArgs._ShowOutputGridNumbers;
	bAreCurvesVisible = InArgs._AreCurvesVisible;
	SetAreCurvesVisibleHandler = InArgs._OnSetAreCurvesVisible;

	OnCreateAsset = InArgs._OnCreateAsset;

	DragState = EDragState::None;
	DragThreshold = 4;

	MovementAxisLock = EMovementAxisLock::None;

	TransactionIndex = -1;

	ReduceTolerance = 0.001f;

	Settings = GetMutableDefault<UCurveEditorSettings>();

	Commands->MapAction(FGenericCommands::Get().Undo,
		FExecuteAction::CreateSP(this, &SCurveEditor::UndoAction));

	Commands->MapAction(FGenericCommands::Get().Redo,
		FExecuteAction::CreateSP(this, &SCurveEditor::RedoAction));

	Commands->MapAction(FCurveEditorCommands::Get().ZoomToFitHorizontal,
		FExecuteAction::CreateSP(this, &SCurveEditor::ZoomToFitHorizontal, false));

	Commands->MapAction(FCurveEditorCommands::Get().ZoomToFitVertical,
		FExecuteAction::CreateSP(this, &SCurveEditor::ZoomToFitVertical, false));

	Commands->MapAction(FCurveEditorCommands::Get().ZoomToFit,
		FExecuteAction::CreateSP(this, &SCurveEditor::ZoomToFit, false));

	Commands->MapAction(FCurveEditorCommands::Get().ZoomToFitAll,
		FExecuteAction::CreateSP(this, &SCurveEditor::ZoomToFit, true));

	Commands->MapAction(FCurveEditorCommands::Get().ToggleInputSnapping,
		FExecuteAction::CreateSP(this, &SCurveEditor::ToggleInputSnapping),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SCurveEditor::IsInputSnappingEnabled));

	Commands->MapAction(FCurveEditorCommands::Get().ToggleOutputSnapping,
		FExecuteAction::CreateSP(this, &SCurveEditor::ToggleOutputSnapping),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SCurveEditor::IsOutputSnappingEnabled));

	// Interpolation
	Commands->MapAction(FCurveEditorCommands::Get().InterpolationConstant,
		FExecuteAction::CreateSP(this, &SCurveEditor::OnSelectInterpolationMode, RCIM_Constant, RCTM_Auto),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SCurveEditor::IsInterpolationModeSelected, RCIM_Constant, RCTM_Auto));

	Commands->MapAction(FCurveEditorCommands::Get().InterpolationLinear,
		FExecuteAction::CreateSP(this, &SCurveEditor::OnSelectInterpolationMode, RCIM_Linear, RCTM_Auto),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SCurveEditor::IsInterpolationModeSelected, RCIM_Linear, RCTM_Auto));

	Commands->MapAction(FCurveEditorCommands::Get().InterpolationCubicAuto,
		FExecuteAction::CreateSP(this, &SCurveEditor::OnSelectInterpolationMode, RCIM_Cubic, RCTM_Auto),
		FCanExecuteAction::CreateSP(this, &SCurveEditor::HasRichCurves),
		FIsActionChecked::CreateSP(this, &SCurveEditor::IsInterpolationModeSelected, RCIM_Cubic, RCTM_Auto));

	Commands->MapAction(FCurveEditorCommands::Get().InterpolationCubicSmartAuto,
		FExecuteAction::CreateSP(this, &SCurveEditor::OnSelectInterpolationMode, RCIM_Cubic, RCTM_SmartAuto),
		FCanExecuteAction::CreateSP(this, &SCurveEditor::HasRichCurves),
		FIsActionChecked::CreateSP(this, &SCurveEditor::IsInterpolationModeSelected, RCIM_Cubic, RCTM_SmartAuto));

	Commands->MapAction(FCurveEditorCommands::Get().InterpolationCubicUser,
		FExecuteAction::CreateSP(this, &SCurveEditor::OnSelectInterpolationMode, RCIM_Cubic, RCTM_User),
		FCanExecuteAction::CreateSP(this, &SCurveEditor::HasRichCurves),
		FIsActionChecked::CreateSP(this, &SCurveEditor::IsInterpolationModeSelected, RCIM_Cubic, RCTM_User));

	Commands->MapAction(FCurveEditorCommands::Get().InterpolationCubicBreak,
		FExecuteAction::CreateSP(this, &SCurveEditor::OnSelectInterpolationMode, RCIM_Cubic, RCTM_Break),
		FCanExecuteAction::CreateSP(this, &SCurveEditor::HasRichCurves),
		FIsActionChecked::CreateSP(this, &SCurveEditor::IsInterpolationModeSelected, RCIM_Cubic, RCTM_Break));

	// Tangents
	Commands->MapAction(FCurveEditorCommands::Get().FlattenTangents,
		FExecuteAction::CreateSP(this, &SCurveEditor::OnFlattenOrStraightenTangents, true),
		FCanExecuteAction::CreateSP(this, &SCurveEditor::HasRichCurves));

	Commands->MapAction(FCurveEditorCommands::Get().StraightenTangents,
		FExecuteAction::CreateSP(this, &SCurveEditor::OnFlattenOrStraightenTangents, false),
		FCanExecuteAction::CreateSP(this, &SCurveEditor::HasRichCurves));

	// Bake and reduce
	Commands->MapAction(FCurveEditorCommands::Get().BakeCurve,
		FExecuteAction::CreateSP(this, &SCurveEditor::OnBakeCurve));

	Commands->MapAction(FCurveEditorCommands::Get().ReduceCurve,
		FExecuteAction::CreateSP(this, &SCurveEditor::OnReduceCurve));

	// Pre infinity extrapolation
	Commands->MapAction(FCurveEditorCommands::Get().SetPreInfinityExtrapCycle,
		FExecuteAction::CreateSP(this, &SCurveEditor::OnSelectPreInfinityExtrap, RCCE_Cycle),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SCurveEditor::IsPreInfinityExtrapSelected, RCCE_Cycle));

	Commands->MapAction(FCurveEditorCommands::Get().SetPreInfinityExtrapCycleWithOffset,
		FExecuteAction::CreateSP(this, &SCurveEditor::OnSelectPreInfinityExtrap, RCCE_CycleWithOffset),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SCurveEditor::IsPreInfinityExtrapSelected, RCCE_CycleWithOffset));

	Commands->MapAction(FCurveEditorCommands::Get().SetPreInfinityExtrapOscillate,
		FExecuteAction::CreateSP(this, &SCurveEditor::OnSelectPreInfinityExtrap, RCCE_Oscillate),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SCurveEditor::IsPreInfinityExtrapSelected, RCCE_Oscillate));

	Commands->MapAction(FCurveEditorCommands::Get().SetPreInfinityExtrapLinear,
		FExecuteAction::CreateSP(this, &SCurveEditor::OnSelectPreInfinityExtrap, RCCE_Linear),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SCurveEditor::IsPreInfinityExtrapSelected, RCCE_Linear));

	Commands->MapAction(FCurveEditorCommands::Get().SetPreInfinityExtrapConstant,
		FExecuteAction::CreateSP(this, &SCurveEditor::OnSelectPreInfinityExtrap, RCCE_Constant),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SCurveEditor::IsPreInfinityExtrapSelected, RCCE_Constant));

	// Post infinity extrapolation
	Commands->MapAction(FCurveEditorCommands::Get().SetPostInfinityExtrapCycle,
		FExecuteAction::CreateSP(this, &SCurveEditor::OnSelectPostInfinityExtrap, RCCE_Cycle),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SCurveEditor::IsPostInfinityExtrapSelected, RCCE_Cycle));

	Commands->MapAction(FCurveEditorCommands::Get().SetPostInfinityExtrapCycleWithOffset,
		FExecuteAction::CreateSP(this, &SCurveEditor::OnSelectPostInfinityExtrap, RCCE_CycleWithOffset),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SCurveEditor::IsPostInfinityExtrapSelected, RCCE_CycleWithOffset));

	Commands->MapAction(FCurveEditorCommands::Get().SetPostInfinityExtrapOscillate,
		FExecuteAction::CreateSP(this, &SCurveEditor::OnSelectPostInfinityExtrap, RCCE_Oscillate),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SCurveEditor::IsPostInfinityExtrapSelected, RCCE_Oscillate));

	Commands->MapAction(FCurveEditorCommands::Get().SetPostInfinityExtrapLinear,
		FExecuteAction::CreateSP(this, &SCurveEditor::OnSelectPostInfinityExtrap, RCCE_Linear),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SCurveEditor::IsPostInfinityExtrapSelected, RCCE_Linear));

	Commands->MapAction(FCurveEditorCommands::Get().SetPostInfinityExtrapConstant,
		FExecuteAction::CreateSP(this, &SCurveEditor::OnSelectPostInfinityExtrap, RCCE_Constant),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SCurveEditor::IsPostInfinityExtrapSelected, RCCE_Constant));

	// Tangent Visibility
	Commands->MapAction(FCurveEditorCommands::Get().SetAllTangentsVisibility,
		FExecuteAction::CreateLambda( [this]{ Settings->SetTangentVisibility( ECurveEditorTangentVisibility::AllTangents ); } ),
		FCanExecuteAction::CreateLambda( []{ return true; } ),
		FIsActionChecked::CreateLambda( [this]{ return Settings->GetTangentVisibility() == ECurveEditorTangentVisibility::AllTangents; } ) );

	Commands->MapAction(FCurveEditorCommands::Get().SetUserTangentsVisibility,
		FExecuteAction::CreateLambda( [this]{ Settings->SetTangentVisibility( ECurveEditorTangentVisibility::UserTangents ); } ),
		FCanExecuteAction::CreateLambda( []{ return true; } ),
		FIsActionChecked::CreateLambda( [this]{ return Settings->GetTangentVisibility() == ECurveEditorTangentVisibility::UserTangents; } ) );
	
	Commands->MapAction(FCurveEditorCommands::Get().SetSelectedKeysTangentVisibility,
		FExecuteAction::CreateLambda( [this]{ Settings->SetTangentVisibility( ECurveEditorTangentVisibility::SelectedKeys ); } ),
		FCanExecuteAction::CreateLambda( []{ return true; } ),
		FIsActionChecked::CreateLambda( [this]{ return Settings->GetTangentVisibility() == ECurveEditorTangentVisibility::SelectedKeys; } ) );

	Commands->MapAction(FCurveEditorCommands::Get().SetNoTangentsVisibility,
		FExecuteAction::CreateLambda( [this]{ Settings->SetTangentVisibility( ECurveEditorTangentVisibility::NoTangents ); } ),
		FCanExecuteAction::CreateLambda( []{ return true; } ),
		FIsActionChecked::CreateLambda( [this]{ return Settings->GetTangentVisibility() == ECurveEditorTangentVisibility::NoTangents; } ) );

	Commands->MapAction(FCurveEditorCommands::Get().ToggleAutoFrameCurveEditor,
		FExecuteAction::CreateLambda( [this]{ Settings->SetAutoFrameCurveEditor( !Settings->GetAutoFrameCurveEditor() ); } ),
		FCanExecuteAction::CreateLambda( []{ return true; } ),
		FIsActionChecked::CreateLambda( [this]{ return Settings->GetAutoFrameCurveEditor(); } ) );

	Commands->MapAction(FCurveEditorCommands::Get().ToggleShowCurveEditorCurveToolTips,
		FExecuteAction::CreateLambda( [this]{ 
		Settings->SetShowCurveEditorCurveToolTips( !Settings->GetShowCurveEditorCurveToolTips() ); 
		if (!Settings->GetShowCurveEditorCurveToolTips())
		{
			CurveToolTip.Reset();
			SetToolTip(CurveToolTip);
		} } ),
		FCanExecuteAction::CreateLambda( []{ return true; } ),
		FIsActionChecked::CreateLambda( [this]{ return Settings->GetShowCurveEditorCurveToolTips(); } ) );

	FCoreUObjectDelegates::OnPackageReloaded.AddSP(this, &SCurveEditor::HandlePackageReloaded);

	SAssignNew(WarningMessageText, SErrorText);

	TSharedRef<SBox> CurveSelector = SNew(SBox)
		.VAlign(VAlign_Top)
		.Visibility(this, &SCurveEditor::GetCurveSelectorVisibility)
		[
			CreateCurveSelectionWidget()
		];

	CurveSelectionWidget = CurveSelector;

	InputAxisName = InArgs._XAxisName.IsSet() ? FText::FromString(InArgs._XAxisName.GetValue()) : LOCTEXT("Time", "Time");
	InputFrameAxisName = InArgs._XAxisName.IsSet() ? FText::FromString(InArgs._XAxisName.GetValue()) : LOCTEXT("Frame", "Frame");
	OutputAxisName = InArgs._YAxisName.IsSet() ? FText::FromString(InArgs._YAxisName.GetValue()) : LOCTEXT("Value", "Value");

	ChildSlot
	[
		SNew( SVerticalBox )
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SHorizontalBox)
			.Visibility( this, &SCurveEditor::GetCurveAreaVisibility )
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(30, 12, 0, 0))
			[
				CurveSelector
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBorder)
				.VAlign(VAlign_Top)
				.HAlign(HAlign_Left)
				.BorderImage( FAppStyle::GetBrush("NoBorder") )
				.DesiredSizeScale(FVector2D(256.0f,32.0f))
				.Padding(FMargin(2, 12, 0, 0))
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.ToolTipText(LOCTEXT("ZoomToFitHorizontal", "Zoom To Fit Horizontal"))
						.Visibility(this, &SCurveEditor::GetZoomButtonVisibility)
						.OnClicked(this, &SCurveEditor::ZoomToFitHorizontalClicked)
						.ContentPadding(1)
						[
							SNew(SImage) 
							.Image( FAppStyle::GetBrush("CurveEd.FitHorizontal") ) 
							.ColorAndOpacity( FSlateColor::UseForeground() ) 
						]
					]
						
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.ToolTipText(LOCTEXT("ZoomToFitVertical", "Zoom To Fit Vertical"))
						.Visibility(this, &SCurveEditor::GetZoomButtonVisibility)
						.OnClicked(this, &SCurveEditor::ZoomToFitVerticalClicked)
						.ContentPadding(1)
						[
							SNew(SImage) 
							.Image( FAppStyle::GetBrush("CurveEd.FitVertical") ) 
							.ColorAndOpacity( FSlateColor::UseForeground() ) 
						]
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.Padding(6.0f, 0.0, 3.0f, 0.0f)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Visibility(this, &SCurveEditor::GetEditVisibility)
							.Text(this, &SCurveEditor::GetInputAxisName)
							.ShadowOffset(FVector2D(1,1))
						]
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						[
							SNew(SNumericEntryBox<float>)
							.IsEnabled(this, &SCurveEditor::GetInputEditEnabled)
							.Value(this, &SCurveEditor::OnGetTime)
							.UndeterminedString(LOCTEXT("MultipleValues", "Multiple Values"))
							.OnValueCommitted(this, &SCurveEditor::OnTimeComitted)
							.OnValueChanged(this, &SCurveEditor::OnTimeChanged)
							.OnBeginSliderMovement(this, &SCurveEditor::OnBeginSliderMovement, LOCTEXT("SetTime", "Set New Time"))
							.OnEndSliderMovement(this, &SCurveEditor::OnEndSliderMovement)
							.AllowSpin(true)
							.MinValue(TOptional<float>())
							.MaxValue(TOptional<float>())
							.MaxSliderValue(TOptional<float>())
							.MinSliderValue(TOptional<float>())
							.Delta(this, &SCurveEditor::GetInputNumericEntryBoxDelta)
							.MinDesiredValueWidth(60.0f)
							.Visibility(this, &SCurveEditor::GetTimeEditVisibility)
								
						]
						+ SHorizontalBox::Slot()
						.Padding(3.0f, 0.0f)
						.VAlign(VAlign_Center)
						.AutoWidth()
						[
							SNew(SNumericEntryBox<int32>)
							.IsEnabled(this, &SCurveEditor::GetInputEditEnabled)
							.Value(this, &SCurveEditor::OnGetTimeInFrames)
							.UndeterminedString(LOCTEXT("MultipleValues", "Multiple Values"))
							.OnValueCommitted(this, &SCurveEditor::OnTimeInFramesComitted)
							.OnValueChanged(this, &SCurveEditor::OnTimeInFramesChanged)
							.OnBeginSliderMovement(this, &SCurveEditor::OnBeginSliderMovement, LOCTEXT("SetFrame", "Set New Frame"))
							.OnEndSliderMovement(this, &SCurveEditor::OnEndSliderMovement)
							.LabelVAlign(VAlign_Center)
							.AllowSpin(true)
							.MinValue(TOptional<int32>())
							.MaxValue(TOptional<int32>())
							.MaxSliderValue(TOptional<int32>())
							.MinSliderValue(TOptional<int32>())
							.Delta(1)
							.MinDesiredValueWidth(60.0f)
							.Visibility(this, &SCurveEditor::GetFrameEditVisibility)
						]
					]
					+ SHorizontalBox::Slot()
					.Padding(3.0f, 0.0f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Visibility(this, &SCurveEditor::GetEditVisibility)
						.Text(OutputAxisName)
						.ShadowOffset(FVector2D(1, 1))
					]
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(SNumericEntryBox<float>)
						.Visibility(this, &SCurveEditor::GetEditVisibility)
						.Value(this, &SCurveEditor::OnGetValue)
						.UndeterminedString(LOCTEXT("MultipleValues", "Multiple Values"))
						.OnValueCommitted(this, &SCurveEditor::OnValueComitted)
						.OnValueChanged(this, &SCurveEditor::OnValueChanged)
						.OnBeginSliderMovement(this, &SCurveEditor::OnBeginSliderMovement, LOCTEXT("SetValue", "Set New Value"))
						.OnEndSliderMovement(this, &SCurveEditor::OnEndSliderMovement)
						.AllowSpin(true)
						.MinValue(TOptional<float>())
						.MaxValue(TOptional<float>())
						.MaxSliderValue(TOptional<float>())
						.MinSliderValue(TOptional<float>())
						.Delta(this, &SCurveEditor::GetOutputNumericEntryBoxDelta)
						.MinDesiredValueWidth(60.0f)
					]
				]
			]
		]
		+ SVerticalBox::Slot()
		.VAlign(VAlign_Bottom)
		.FillHeight(.75f)
		[
			SNew( SBorder )
			.Visibility( this, &SCurveEditor::GetColorGradientVisibility )
			.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
			.BorderBackgroundColor( FLinearColor( .8f, .8f, .8f, .60f ) )
			.Padding(1.0f)
			[
				SAssignNew( GradientViewer, SColorGradientEditor )
				.ViewMinInput( ViewMinInput )
				.ViewMaxInput( ViewMaxInput )
				.IsEditingEnabled( this, &SCurveEditor::IsEditingEnabled )
			]
		]
	];

	if (GEditor != NULL)
	{
		GEditor->RegisterForUndo(this);
	}

	FCoreUObjectDelegates::OnObjectPropertyChanged.AddSP(this, &SCurveEditor::OnObjectPropertyChanged);
}

FText SCurveEditor::GetIsCurveVisibleToolTip(TSharedPtr<FCurveViewModel> CurveViewModel) const
{
	return CurveViewModel->bIsVisible ?
		FText::Format(LOCTEXT("HideFormat", "Hide {0} curve"), FText::FromName(CurveViewModel->CurveInfo.CurveName)) :
		FText::Format(LOCTEXT("ShowFormat", "Show {0} curve"), FText::FromName(CurveViewModel->CurveInfo.CurveName));
}

ECheckBoxState SCurveEditor::IsCurveVisible(TSharedPtr<FCurveViewModel> CurveViewModel) const
{
	return CurveViewModel->bIsVisible ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SCurveEditor::OnCurveIsVisibleChanged(ECheckBoxState NewCheckboxState, TSharedPtr<FCurveViewModel> CurveViewModel)
{
	if (NewCheckboxState == ECheckBoxState::Checked)
	{
		CurveViewModel->bIsVisible = true;
	}
	else
	{
		CurveViewModel->bIsVisible = false;
		RemoveCurveKeysFromSelection(CurveViewModel);
	}
}

FText SCurveEditor::GetIsCurveLockedToolTip(TSharedPtr<FCurveViewModel> CurveViewModel) const
{
	return CurveViewModel->bIsLocked ?
		FText::Format(LOCTEXT("UnlockFormat", "Unlock {0} curve for editing"), FText::FromName(CurveViewModel->CurveInfo.CurveName)) :
		FText::Format(LOCTEXT("LockFormat", "Lock {0} curve for editing"), FText::FromName(CurveViewModel->CurveInfo.CurveName));
}

ECheckBoxState SCurveEditor::IsCurveLocked(TSharedPtr<FCurveViewModel> CurveViewModel) const
{
	return CurveViewModel->bIsLocked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SCurveEditor::OnCurveIsLockedChanged(ECheckBoxState NewCheckboxState, TSharedPtr<FCurveViewModel> CurveViewModel)
{
	if (NewCheckboxState == ECheckBoxState::Checked)
	{
		CurveViewModel->bIsLocked = true;
		RemoveCurveKeysFromSelection(CurveViewModel);
	}
	else
	{
		CurveViewModel->bIsLocked = false;
	}
}

void SCurveEditor::RemoveCurveKeysFromSelection(TSharedPtr<FCurveViewModel> CurveViewModel)
{
	TArray<FSelectedCurveKey> SelectedKeysForLockedCurve;
	for (auto SelectedKey : SelectedKeys)
	{
		if (SelectedKey.Curve == CurveViewModel->CurveInfo.CurveToEdit)
		{
			SelectedKeysForLockedCurve.Add(SelectedKey);
		}
	}
	for (auto KeyToDeselect : SelectedKeysForLockedCurve)
	{
		RemoveFromKeySelection(KeyToDeselect);
	}
}

FText SCurveEditor::GetCurveToolTipNameText() const
{
	return CurveToolTipNameText;
}

FText SCurveEditor::GetCurveToolTipInputText() const
{
	return CurveToolTipInputText;
}

FText SCurveEditor::GetCurveToolTipOutputText() const
{
	return CurveToolTipOutputText;
}

FText SCurveEditor::GetInputAxisName() const
{
	return ShowTimeInFrames() ? InputFrameAxisName : InputAxisName;
}

SCurveEditor::~SCurveEditor()
{
	if (GEditor != NULL)
	{
		GEditor->UnregisterForUndo(this);
	}

	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
}

TSharedRef<SWidget> SCurveEditor::CreateCurveSelectionWidget() const
{
	TSharedRef<SVerticalBox> CurveBox = SNew(SVerticalBox);
	if (CurveViewModels.Num() > 1)
	{
		// Only create curve controls if there are more than one.
		for (auto CurveViewModel : CurveViewModels)
		{
			CurveBox->AddSlot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.Padding(0, 0, 5, 0)
				.FillWidth(1.0f)
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("CurveEd.LabelFont"))
					.ColorAndOpacity(CurveViewModel->Color)
					.Text(FText::FromName(CurveViewModel->CurveInfo.CurveName))
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Fill)
				[
					SNew(SCheckBox)
					.Style(&FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox"))
					.IsChecked(this, &SCurveEditor::IsCurveVisible, CurveViewModel)
					.OnCheckStateChanged(const_cast<SCurveEditor*>(this), &SCurveEditor::OnCurveIsVisibleChanged, CurveViewModel)
					.ToolTipText(this, &SCurveEditor::GetIsCurveVisibleToolTip, CurveViewModel)
					.CheckedImage(FAppStyle::Get().GetBrush("Icons.Visible"))
					.CheckedHoveredImage(FAppStyle::Get().GetBrush("Icons.Visible"))
					.CheckedPressedImage(FAppStyle::Get().GetBrush("Icons.Visible"))
					.UncheckedImage(FAppStyle::Get().GetBrush("Icons.Hidden"))
					.UncheckedHoveredImage(FAppStyle::Get().GetBrush("Icons.Hidden"))
					.UncheckedPressedImage(FAppStyle::Get().GetBrush("Icons.Hidden"))
					.Padding(8.0f)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Fill)
				.Padding(2, 0, 0, 0)
				[
					SNew(SCheckBox)
					.Style(&FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox"))
					.IsChecked(this, &SCurveEditor::IsCurveLocked, CurveViewModel)
					.OnCheckStateChanged(const_cast<SCurveEditor*>(this), &SCurveEditor::OnCurveIsLockedChanged, CurveViewModel)
					.ToolTipText(this, &SCurveEditor::GetIsCurveLockedToolTip, CurveViewModel)
					.CheckedImage(FAppStyle::Get().GetBrush("Icons.Lock"))
					.CheckedHoveredImage(FAppStyle::Get().GetBrush("Icons.Lock"))
					.CheckedPressedImage(FAppStyle::Get().GetBrush("Icons.Lock"))
					.UncheckedImage(FAppStyle::Get().GetBrush("Icons.Unlock"))
					.UncheckedHoveredImage(FAppStyle::Get().GetBrush("Icons.Unlock"))
					.UncheckedPressedImage(FAppStyle::Get().GetBrush("Icons.Unlock"))
					.Padding(8.0f)
					.Visibility(bCanEditTrack ? EVisibility::Visible : EVisibility::Collapsed)
				]
			];
		}
	}

	TSharedRef<SBorder> Border = SNew(SBorder)
		.Padding(FMargin(3, 2, 2, 2))
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		.BorderBackgroundColor(FLinearColor(0.0f, 0.0f, 0.0f, 0.3f))
		[
			CurveBox
		];

	return Border;
}

void SCurveEditor::PushWarningMenu( FVector2D Position, const FText& Message )
{
	WarningMessageText->SetError(Message);

	FSlateApplication::Get().PushMenu(
		SharedThis( this ),
		FWidgetPath(),
		WarningMessageText->AsWidget(),
		Position,
		FPopupTransitionEffect( FPopupTransitionEffect::ContextMenu));
}

void SCurveEditor::PushKeyMenu(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	FMenuBuilder MenuBuilder(true, Commands.ToSharedRef());
	MenuBuilder.BeginSection("CurveEditorInterpolation", LOCTEXT("KeyInterpolationMode", "Key Interpolation"));
	{
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().InterpolationCubicAuto);
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().InterpolationCubicUser);
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().InterpolationCubicBreak);
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().InterpolationLinear);
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().InterpolationConstant);
	}
	MenuBuilder.EndSection(); //CurveEditorInterpolation

	MenuBuilder.BeginSection("CurveEditorTangents", LOCTEXT("Tangents", "Tangents"));
	{
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().FlattenTangents);
		MenuBuilder.AddMenuEntry(FCurveEditorCommands::Get().StraightenTangents);
	}
	MenuBuilder.EndSection(); //CurveEditorTangents

	FWidgetPath WidgetPath = InMouseEvent.GetEventPath() != nullptr ? *InMouseEvent.GetEventPath() : FWidgetPath();
	FVector2D Position = InMouseEvent.GetScreenSpacePosition();
	FSlateApplication::Get().PushMenu(
		SharedThis( this ),
		WidgetPath,
		MenuBuilder.MakeWidget(),
		Position,
		FPopupTransitionEffect( FPopupTransitionEffect::ContextMenu));
}


FVector2D SCurveEditor::ComputeDesiredSize( float ) const
{
	return DesiredSize.Get();
}

EVisibility SCurveEditor::GetCurveAreaVisibility() const
{
	return AreCurvesVisible() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SCurveEditor::GetCurveSelectorVisibility() const
{
	return  (IsHovered() || (false == bHideUI)) && bShowCurveSelector ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SCurveEditor::GetEditVisibility() const
{
	return (SelectedKeys.Num() > 0) && (IsHovered() || (false == bHideUI)) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SCurveEditor::GetColorGradientVisibility() const
{
	return IsGradientEditorVisible() && IsLinearColorCurve() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SCurveEditor::GetZoomButtonVisibility() const
{
	return (IsHovered() || (false == bHideUI)) && bShowZoomButtons ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SCurveEditor::GetTimeEditVisibility() const
{
	if (GetEditVisibility().IsVisible())
	{
		return ShowTimeInFrames() ? EVisibility::Collapsed : EVisibility::Visible;
	}
	return EVisibility::Collapsed;
}

EVisibility SCurveEditor::GetFrameEditVisibility() const
{
	if (GetEditVisibility().IsVisible())
	{
		return ShowTimeInFrames() ? EVisibility::Visible : EVisibility::Collapsed;
	}
	return EVisibility::Collapsed;
}

bool SCurveEditor::GetInputEditEnabled() const
{
	bool bKeysOnSameCurves = false;
	for (int32 SelectedIndex = 0; SelectedIndex < SelectedKeys.Num() - 1; SelectedIndex++)
	{
		for (int32 CompareIndex = SelectedIndex + 1; CompareIndex < SelectedKeys.Num(); CompareIndex++)
		{
			if (SelectedKeys[SelectedIndex].Curve == SelectedKeys[CompareIndex].Curve)
			{
				bKeysOnSameCurves = true;
				break;
			}
		}
	}
	return (SelectedKeys.Num() == 1) || !bKeysOnSameCurves;
}

int32 SCurveEditor::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	// Rendering info
	bool bEnabled = ShouldBeEnabled( bParentEnabled );
	ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
	const FSlateBrush* TimelineAreaBrush = FAppStyle::GetBrush("CurveEd.TimelineArea");
	const FSlateBrush* WhiteBrush = FAppStyle::GetBrush("WhiteTexture");

	FGeometry CurveAreaGeometry = AllottedGeometry;

	// Positioning info
	FTrackScaleInfo ScaleInfo(ViewMinInput.Get(),  ViewMaxInput.Get(), ViewMinOutput.Get(), ViewMaxOutput.Get(), CurveAreaGeometry.GetLocalSize());

	if (FMath::IsNearlyEqual(ViewMinInput.Get(), ViewMaxInput.Get()) || FMath::IsNearlyEqual(ViewMinOutput.Get(), ViewMaxOutput.Get()))
	{
		return 0;
	}

	// Draw background to indicate valid timeline area
	float ZeroInputX = ScaleInfo.InputToLocalX(0.f);
	float ZeroOutputY = ScaleInfo.OutputToLocalY(0.f);

	// timeline background
	int32 BackgroundLayerId = LayerId;
	float TimelineMaxX = ScaleInfo.InputToLocalX(TimelineLength.Get());
	FSlateDrawElement::MakeBox
		(
		OutDrawElements,
		BackgroundLayerId,
		CurveAreaGeometry.ToPaintGeometry(FVector2D(TimelineMaxX - ZeroInputX, CurveAreaGeometry.GetLocalSize().Y), FSlateLayoutTransform(FVector2D(ZeroInputX, 0.f))),
		TimelineAreaBrush,
		DrawEffects,
		TimelineAreaBrush->GetTint(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
		);

	// grid lines.
	int32 GridLineLayerId = BackgroundLayerId + 1;
	PaintGridLines(CurveAreaGeometry, ScaleInfo, OutDrawElements, GridLineLayerId, MyCullingRect, DrawEffects);

	// time=0 line
	int32 ZeroLineLayerId = GridLineLayerId + 1;
	TArray<FVector2D> ZeroLinePoints;
	ZeroLinePoints.Add( FVector2D( ZeroInputX, 0 ) );
	ZeroLinePoints.Add( FVector2D( ZeroInputX, CurveAreaGeometry.GetLocalSize().Y ) );
	FSlateDrawElement::MakeLines(
		OutDrawElements,
		ZeroLineLayerId,
		AllottedGeometry.ToPaintGeometry(),
		ZeroLinePoints,
		DrawEffects,
		FLinearColor::White,
		false );

	// value=0 line
	if( AreCurvesVisible() )
	{
		FSlateDrawElement::MakeBox
		(
			OutDrawElements,
			ZeroLineLayerId,
			CurveAreaGeometry.ToPaintGeometry( FVector2D(CurveAreaGeometry.Size.X, 1), FSlateLayoutTransform(FVector2D(0.f, ZeroOutputY)) ),
			WhiteBrush,
			DrawEffects,
			WhiteBrush->GetTint( InWidgetStyle ) * InWidgetStyle.GetColorAndOpacityTint()
		);
	}

	int32 LockedCurveLayerID = ZeroLineLayerId + 1;
	int32 CurveLayerId = LockedCurveLayerID + 1;

	int32 KeyLayerId = CurveLayerId + 1;
	int32 SelectedKeyLayerId = KeyLayerId + 1;

	bool bAnyCurveViewModelsSelected = AnyCurveViewModelsSelected();

	if( AreCurvesVisible() )
	{
		//Paint the curves, unlocked curves will be on top
		for ( auto CurveViewModel : CurveViewModels )
		{
			if (CurveViewModel->bIsVisible)
			{
				PaintCurve(CurveViewModel, CurveAreaGeometry, ScaleInfo, OutDrawElements, CurveViewModel->bIsLocked ? LockedCurveLayerID : CurveLayerId, MyCullingRect, DrawEffects, InWidgetStyle, bAnyCurveViewModelsSelected);
			}
		}

		//Paint the keys on top of the curve
		for (auto CurveViewModel : CurveViewModels)
		{
			if (CurveViewModel->bIsVisible)
			{
				PaintKeys(CurveViewModel, ScaleInfo, OutDrawElements, KeyLayerId, SelectedKeyLayerId, CurveAreaGeometry, MyCullingRect, DrawEffects, InWidgetStyle, bAnyCurveViewModelsSelected);
			}
		}
	}

	// Paint children
	int32 ChildrenLayerId = SelectedKeyLayerId + 1;
	int32 MarqueeLayerId = SCompoundWidget::OnPaint(Args, CurveAreaGeometry, MyCullingRect, OutDrawElements, ChildrenLayerId, InWidgetStyle, bParentEnabled);

	// Paint marquee
	if (DragState == EDragState::MarqueeSelect)
	{
		PaintMarquee(AllottedGeometry, MyCullingRect, OutDrawElements, MarqueeLayerId);
	}

	return MarqueeLayerId + 1;
}

void SCurveEditor::PaintCurve(TSharedPtr<FCurveViewModel> CurveViewModel, const FGeometry &AllottedGeometry, FTrackScaleInfo &ScaleInfo, FSlateWindowElementList &OutDrawElements,
	int32 LayerId, const FSlateRect& MyCullingRect, ESlateDrawEffect DrawEffects, const FWidgetStyle &InWidgetStyle, bool bAnyCurveViewModelsSelected )const
{
	if (CurveViewModel.IsValid())
	{
		if (bDrawCurve)
		{
			FLinearColor Color = InWidgetStyle.GetColorAndOpacityTint() * CurveViewModel->Color;

			// Fade out curves that are not selected.
			if (!CurveViewModel->bIsSelected && bAnyCurveViewModelsSelected)
			{
				Color *= FLinearColor(1.0f,1.0f,1.0f,0.2f); 
			}

			// Fade out curves which are locked.
			if(CurveViewModel->bIsLocked)
			{
				Color *= FLinearColor(1.0f,1.0f,1.0f,0.35f); 
			}

			TArray<FVector2D> LinePoints;
			int32 CurveDrawInterval = 1;

			FRealCurve* Curve = CurveViewModel->CurveInfo.CurveToEdit;
			const float NumKeys = Curve->GetNumKeys();

			if (NumKeys < 2) 
			{
				//Not enough point, just draw flat line
				float Value = Curve->Eval(0.0f);
				float Y = ScaleInfo.OutputToLocalY(Value);
				LinePoints.Add(FVector2D(0.0f, Y));
				LinePoints.Add(FVector2D(AllottedGeometry.GetLocalSize().X, Y));

				FSlateDrawElement::MakeLines(OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(), LinePoints, DrawEffects, Color);
				LinePoints.Empty();
			}
			else
			{
				TArray<FKeyHandle> KeyHandles;
				TArray<TPair<float,float>> Key_TimeValuePairs;
				KeyHandles.Reserve(NumKeys);
				Key_TimeValuePairs.Reserve(NumKeys);
				for (auto It = Curve->GetKeyHandleIterator(); It; ++It)
				{
					const FKeyHandle& KeyHandle = *It;

					KeyHandles.Add(KeyHandle);
					Key_TimeValuePairs.Emplace(Curve->GetKeyTimeValuePair(KeyHandle));
				}

				//Add arrive and exit lines
				{
					float ArriveX = ScaleInfo.InputToLocalX(Key_TimeValuePairs[0].Key);
					float ArriveY = ScaleInfo.OutputToLocalY(Key_TimeValuePairs[0].Value);
					float LeaveY  = ScaleInfo.OutputToLocalY(Key_TimeValuePairs.Last().Value);
					float LeaveX  = ScaleInfo.InputToLocalX(Key_TimeValuePairs.Last().Key);

					//Arrival line
					LinePoints.Add(FVector2D(0.0f, ArriveY));
					LinePoints.Add(FVector2D(ArriveX, ArriveY));
					FSlateDrawElement::MakeLines(OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(), LinePoints, DrawEffects, Color);
					LinePoints.Empty();

					//Leave line
					LinePoints.Add(FVector2D(AllottedGeometry.GetLocalSize().X, LeaveY));
					LinePoints.Add(FVector2D(LeaveX, LeaveY));
					FSlateDrawElement::MakeLines(OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(), LinePoints, DrawEffects, Color);
					LinePoints.Empty();
				}


				//Add enclosed segments
				for (int32 i = 0;i<NumKeys-1;++i)
				{
					CreateLinesForSegment(Curve, Curve->GetKeyInterpMode(KeyHandles[i]), Key_TimeValuePairs[i], Key_TimeValuePairs[i+1], LinePoints, ScaleInfo);
					FSlateDrawElement::MakeLines(OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(), LinePoints, DrawEffects, Color);
					LinePoints.Empty();
				}
			}
		}
	}
}


void SCurveEditor::CreateLinesForSegment( FRealCurve* Curve, ERichCurveInterpMode InterpMode, const TPair<float,float>& Key1_TimeValue, const TPair<float,float>& Key2_TimeValue, TArray<FVector2D>& Points, FTrackScaleInfo &ScaleInfo ) const
{
	switch(InterpMode)
	{
	case RCIM_Constant:
		{
			//@todo: should really only need 3 points here but something about the line rendering isn't quite behaving as I'd expect, so need extras
			Points.Add(FVector2D(Key1_TimeValue.Key, Key1_TimeValue.Value));
			Points.Add(FVector2D(Key2_TimeValue.Key, Key1_TimeValue.Value));
			Points.Add(FVector2D(Key2_TimeValue.Key, Key1_TimeValue.Value));
			Points.Add(FVector2D(Key2_TimeValue.Key, Key2_TimeValue.Value));
			Points.Add(FVector2D(Key2_TimeValue.Key, Key1_TimeValue.Value));
		}break;
	case RCIM_Linear:
		{
			Points.Add(FVector2D(Key1_TimeValue.Key, Key1_TimeValue.Value));
			Points.Add(FVector2D(Key2_TimeValue.Key, Key2_TimeValue.Value));
		}break;
	case RCIM_Cubic:
		{
			const float StepSize = 1.0f;
			//clamp to screen to avoid massive slowdown when zoomed in
			float StartX	= FMath::Max(ScaleInfo.InputToLocalX(Key1_TimeValue.Key), 0.0f) ;
			float EndX		= FMath::Min(ScaleInfo.InputToLocalX(Key2_TimeValue.Key),ScaleInfo.WidgetSize.X);
			for(;StartX<EndX; StartX += StepSize)
			{
				float CurveIn = ScaleInfo.LocalXToInput(FMath::Min(StartX,EndX));
				float CurveOut = Curve->Eval(CurveIn);
				Points.Add(FVector2D(CurveIn,CurveOut));
			}
			Points.Add(FVector2D(Key2_TimeValue.Key,Key2_TimeValue.Value));

		}break;
	}

	//Transform to screen
	for(auto It = Points.CreateIterator();It;++It)
	{
		FVector2D Vec2D = *It;
		Vec2D.X = ScaleInfo.InputToLocalX(Vec2D.X);
		Vec2D.Y = ScaleInfo.OutputToLocalY(Vec2D.Y);
		*It = Vec2D;
	}
}

void SCurveEditor::PaintKeys(TSharedPtr<FCurveViewModel> CurveViewModel, FTrackScaleInfo &ScaleInfo, FSlateWindowElementList &OutDrawElements, int32 LayerId, int32 SelectedLayerId, const FGeometry &AllottedGeometry, const FSlateRect& MyCullingRect, ESlateDrawEffect DrawEffects, const FWidgetStyle &InWidgetStyle, bool bAnyCurveViewModelsSelected ) const
{
	FLinearColor KeyColor = CurveViewModel->bIsLocked ? FLinearColor(0.1f,0.1f,0.1f,1.f) : InWidgetStyle.GetColorAndOpacityTint();

	const bool bHasRichCurves = CurveOwner->HasRichCurves();

	// Iterate over each key
	ERichCurveInterpMode LastInterpMode = RCIM_Linear;
	FRealCurve* Curve = CurveViewModel->CurveInfo.CurveToEdit;
	for (auto It(Curve->GetKeyHandleIterator()); It; ++It)
	{
		FKeyHandle KeyHandle = *It;

		// Work out where it is
		FVector2D KeyLocation(
			ScaleInfo.InputToLocalX(Curve->GetKeyTime(KeyHandle)),
			ScaleInfo.OutputToLocalY(Curve->GetKeyValue(KeyHandle)));
		FVector2D KeyIconLocation = KeyLocation - (CONST_KeySize / 2);

		// Get brush
		bool IsSelected = IsKeySelected(FSelectedCurveKey(Curve,KeyHandle));
		const FSlateBrush* KeyBrush = IsSelected ? FAppStyle::GetBrush("CurveEd.CurveKeySelected") : FAppStyle::GetBrush("CurveEd.CurveKey");
		int32 LayerToUse = IsSelected ? SelectedLayerId: LayerId;

		// Fade out keys that are not selected and whose curve is not selected as well.
		FLinearColor SelectionTint = !CurveViewModel->bIsSelected && !IsSelected && bAnyCurveViewModelsSelected ? FLinearColor(1.0f,1.0f,1.0f,0.2f) : FLinearColor(1.0f,1.0f,1.0f,1.0f); 

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerToUse,
			AllottedGeometry.ToPaintGeometry( CONST_KeySize, FSlateLayoutTransform(KeyIconLocation) ),
			KeyBrush,
			DrawEffects,
			KeyBrush->GetTint( InWidgetStyle ) * InWidgetStyle.GetColorAndOpacityTint() * KeyColor * SelectionTint
			);

		//Handle drawing the tangent controls for curve
		bool bIsTangentSelected = false;
		bool bIsArrivalSelected = false;
		bool bIsLeaveSelected = false;
		if (bHasRichCurves)
		{
			FRichCurve* RichCurve = (FRichCurve*)Curve;
			if (IsTangentVisible(RichCurve, KeyHandle, bIsTangentSelected, bIsArrivalSelected, bIsLeaveSelected) && (RichCurve->GetKeyInterpMode(KeyHandle) == RCIM_Cubic || LastInterpMode == RCIM_Cubic))
			{
				PaintTangent(CurveViewModel, ScaleInfo, RichCurve, KeyHandle, KeyLocation, OutDrawElements, LayerId, AllottedGeometry, MyCullingRect, DrawEffects, LayerToUse, InWidgetStyle, bIsTangentSelected, bIsArrivalSelected, bIsLeaveSelected, bAnyCurveViewModelsSelected);
			}
		}

		LastInterpMode = Curve->GetKeyInterpMode(KeyHandle);
	}
}


void SCurveEditor::PaintTangent( TSharedPtr<FCurveViewModel> CurveViewModel, FTrackScaleInfo &ScaleInfo, FRichCurve* Curve, FKeyHandle KeyHandle, FVector2D KeyLocation, FSlateWindowElementList &OutDrawElements, int32 LayerId, const FGeometry &AllottedGeometry, const FSlateRect& MyCullingRect, ESlateDrawEffect DrawEffects, int32 LayerToUse, const FWidgetStyle &InWidgetStyle, bool bTangentSelected, bool bIsArrivalSelected, bool bIsLeaveSelected, bool bAnyCurveViewModelsSelected ) const
{
	FVector2D ArriveTangentLocation, LeaveTangentLocation;
	GetTangentPoints(ScaleInfo, FSelectedCurveKey(Curve,KeyHandle), ArriveTangentLocation, LeaveTangentLocation);

	FVector2D ArriveTangentIconLocation = ArriveTangentLocation - (CONST_TangentSize / 2);
	FVector2D LeaveTangentIconLocation = LeaveTangentLocation - (CONST_TangentSize / 2);

	const FSlateBrush* TangentBrush = FAppStyle::GetBrush("CurveEd.Tangent");
	const FSlateBrush* TangentBrushSelected = FAppStyle::GetBrush("CurveEd.TangentSelected");
	const FLinearColor TangentColor = FAppStyle::GetColor("CurveEd.TangentColor");
	const FLinearColor TangentColorSelected = FAppStyle::GetColor("CurveEd.TangentColorSelected");

	bool LeaveTangentSelected = bTangentSelected && bIsLeaveSelected;
	bool ArriveTangentSelected = bTangentSelected && bIsArrivalSelected;

	FLinearColor LeaveSelectionTint = !CurveViewModel->bIsSelected && !LeaveTangentSelected && bAnyCurveViewModelsSelected ? FLinearColor(1.0f,1.0f,1.0f,0.2f) : FLinearColor(1.0f,1.0f,1.0f,1.0f); 
	FLinearColor ArriveSelectionTint = !CurveViewModel->bIsSelected && !ArriveTangentSelected && bAnyCurveViewModelsSelected ? FLinearColor(1.0f,1.0f,1.0f,0.2f) : FLinearColor(1.0f,1.0f,1.0f,1.0f); 

	//Add lines from tangent control point to 'key'
	TArray<FVector2D> LinePoints;
	LinePoints.Add(FVector2D(KeyLocation));
	LinePoints.Add(FVector2D(ArriveTangentLocation));
	FSlateDrawElement::MakeLines(
		OutDrawElements,
		LayerId,
		AllottedGeometry.ToPaintGeometry(),
		LinePoints,
		DrawEffects,
		ArriveTangentSelected ? TangentColorSelected * ArriveSelectionTint : TangentColor * ArriveSelectionTint
		);

	LinePoints.Empty();
	LinePoints.Add(FVector2D(KeyLocation));
	LinePoints.Add(FVector2D(LeaveTangentLocation));
	FSlateDrawElement::MakeLines(
		OutDrawElements,
		LayerId,
		AllottedGeometry.ToPaintGeometry(),
		LinePoints,
		DrawEffects,
		LeaveTangentSelected ? TangentColorSelected * LeaveSelectionTint : TangentColor * LeaveSelectionTint
		);

	//Arrive tangent control
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerToUse,
		AllottedGeometry.ToPaintGeometry( CONST_TangentSize, FSlateLayoutTransform(ArriveTangentIconLocation) ),
		ArriveTangentSelected ? TangentBrushSelected : TangentBrush,
		DrawEffects,
		ArriveTangentSelected ? TangentBrushSelected->GetTint( InWidgetStyle ) * ArriveSelectionTint : TangentBrush->GetTint( InWidgetStyle ) * InWidgetStyle.GetColorAndOpacityTint() * ArriveSelectionTint
		);
	//Leave tangent control
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerToUse,
		AllottedGeometry.ToPaintGeometry( CONST_TangentSize, FSlateLayoutTransform(LeaveTangentIconLocation) ),
		LeaveTangentSelected ? TangentBrushSelected : TangentBrush,
		DrawEffects,
		LeaveTangentSelected ? TangentBrushSelected->GetTint( InWidgetStyle ) * LeaveSelectionTint : TangentBrush->GetTint( InWidgetStyle ) * InWidgetStyle.GetColorAndOpacityTint() * LeaveSelectionTint
		);
}


float SCurveEditor::CalcGridLineStepDistancePow2(double RawValue)
{
	return float(double(FMath::RoundUpToPowerOfTwo(uint32(RawValue*1024.0))>>1)/1024.0);
}

float SCurveEditor::GetTimeStep(FTrackScaleInfo &ScaleInfo) const
{
	const float MaxGridPixelSpacing = 150.0f;

	const float GridPixelSpacing = FMath::Min(ScaleInfo.WidgetSize.GetMin()/1.5f, MaxGridPixelSpacing);

	double MaxTimeStep = ScaleInfo.LocalXToInput(ViewMinInput.Get() + GridPixelSpacing) - ScaleInfo.LocalXToInput(ViewMinInput.Get());

	return CalcGridLineStepDistancePow2(MaxTimeStep);
}

void SCurveEditor::PaintGridLines(const FGeometry &AllottedGeometry, FTrackScaleInfo &ScaleInfo, FSlateWindowElementList &OutDrawElements, 
	int32 LayerId, const FSlateRect& MyCullingRect, ESlateDrawEffect DrawEffects )const
{
	const float MaxGridPixelSpacing = 150.0f;

	const float GridPixelSpacing = FMath::Min(ScaleInfo.WidgetSize.GetMin()/1.5f, MaxGridPixelSpacing);

	const FLinearColor GridTextColor = FLinearColor(1.0f,1.0f,1.0f, 0.75f) ;

	//Vertical grid(time)
	{
		float TimeStep = GetTimeStep(ScaleInfo);
		float ScreenStepTime = ScaleInfo.InputToLocalX(TimeStep) -  ScaleInfo.InputToLocalX(0.0f);

		if(ScreenStepTime >= 1.0f)
		{
			float StartTime = ScaleInfo.LocalXToInput(0.0f);
			TArray<FVector2D> LinePoints;
			float ScaleX = (TimeStep)/(AllottedGeometry.GetLocalSize().X);

			//draw vertical grid lines
			float StartOffset = -FMath::Fractional(StartTime / TimeStep)*ScreenStepTime;
			float Time =  ScaleInfo.LocalXToInput(StartOffset);
			for(float X = StartOffset;X< AllottedGeometry.GetLocalSize().X;X+= ScreenStepTime, Time += TimeStep)
			{
				if(SMALL_NUMBER < FMath::Abs(X)) //don't show at 0 to avoid overlapping with center axis 
				{
					LinePoints.Add(FVector2D(X, 0.0));
					LinePoints.Add(FVector2D(X, AllottedGeometry.GetLocalSize().Y));
					FSlateDrawElement::MakeLines(
						OutDrawElements,
						LayerId,
						AllottedGeometry.ToPaintGeometry(),
						LinePoints,
						DrawEffects,
						GridColor,
						false);

					//Show grid time
					if (bDrawInputGridNumbers)
					{
						FString TimeStr = FString::Printf(TEXT("%.2f"), Time);
						FSlateDrawElement::MakeText(OutDrawElements,LayerId,AllottedGeometry.MakeChild(FVector2D(1.0f, ScaleX ), FSlateLayoutTransform(FVector2D(X, 0.0))).ToPaintGeometry(),TimeStr,
							FAppStyle::GetFontStyle("CurveEd.InfoFont"), DrawEffects, GridTextColor );
					}

					LinePoints.Empty();
				}
			}
		}
	}

	//Horizontal grid(values)
	// This is only useful if the curves are visible
	if( AreCurvesVisible() )
	{
		double MaxValueStep = ScaleInfo.LocalYToOutput(0) - ScaleInfo.LocalYToOutput(GridPixelSpacing) ;
		float ValueStep = CalcGridLineStepDistancePow2(MaxValueStep);
		float ScreenStepValue = ScaleInfo.OutputToLocalY(0.0f) - ScaleInfo.OutputToLocalY(ValueStep);  
		if(ScreenStepValue >= 1.0f)
		{
			float StartValue = ScaleInfo.LocalYToOutput(0.0f);
			TArray<FVector2D> LinePoints;

			float StartOffset = FMath::Fractional(StartValue / ValueStep)*ScreenStepValue;
			float Value = ScaleInfo.LocalYToOutput(StartOffset);
			float ScaleY = (ValueStep)/(AllottedGeometry.GetLocalSize().Y);

			for(float Y = StartOffset;Y< AllottedGeometry.GetLocalSize().Y;Y+= ScreenStepValue, Value-=ValueStep)
			{
				if(SMALL_NUMBER < FMath::Abs(Y)) //don't show at 0 to avoid overlapping with center axis 
				{
					LinePoints.Add(FVector2D(0.0f, Y));
					LinePoints.Add(FVector2D(AllottedGeometry.GetLocalSize().X,Y));
					FSlateDrawElement::MakeLines(
						OutDrawElements,
						LayerId,
						AllottedGeometry.ToPaintGeometry(),
						LinePoints,
						DrawEffects,
						GridColor,
						false);

					//Show grid value
					if (bDrawOutputGridNumbers)
					{
						FString ValueStr = FString::Printf(TEXT("%.2f"), Value);
						FSlateFontInfo Font = FAppStyle::GetFontStyle("CurveEd.InfoFont");

						const TSharedRef< FSlateFontMeasure > FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
						FVector2D DrawSize = FontMeasureService->Measure(ValueStr, Font);

						// draw at the start
						FSlateDrawElement::MakeText(OutDrawElements,LayerId,AllottedGeometry.MakeChild(FVector2D(ScaleY, 1.0f ), FSlateLayoutTransform(FVector2D(0.0f, Y))).ToPaintGeometry(),
							ValueStr, Font, DrawEffects, GridTextColor );

						// draw at the last since sometimes start can be hidden
						FSlateDrawElement::MakeText(OutDrawElements,LayerId,AllottedGeometry.MakeChild(FVector2D(ScaleY, 1.0f ), FSlateLayoutTransform(FVector2D(AllottedGeometry.GetLocalSize().X-DrawSize.X, Y))).ToPaintGeometry(),
							ValueStr, Font, DrawEffects, GridTextColor );
					}
					
					LinePoints.Empty();
				}
			}
		}
	}
}

void SCurveEditor::PaintMarquee(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	FVector2D MarqueTopLeft(
		FMath::Min(MouseDownLocation.X, MouseMoveLocation.X),
		FMath::Min(MouseDownLocation.Y, MouseMoveLocation.Y)
		);

	FVector2D MarqueBottomRight(
		FMath::Max(MouseDownLocation.X, MouseMoveLocation.X),
		FMath::Max(MouseDownLocation.Y, MouseMoveLocation.Y)
		);

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId,
		AllottedGeometry.ToPaintGeometry(MarqueBottomRight - MarqueTopLeft, FSlateLayoutTransform(MarqueTopLeft)),
		FAppStyle::GetBrush(TEXT("MarqueeSelection"))
		);
}

float SCurveEditor::GetInputNumericEntryBoxDelta() const
{
	return bInputSnappingEnabled.Get() ? InputSnap.Get() : 0;
}

float SCurveEditor::GetOutputNumericEntryBoxDelta() const
{
	return bOutputSnappingEnabled.Get() ? OutputSnap.Get() : 0;
}

void SCurveEditor::SetCurveOwner(FCurveOwnerInterface* InCurveOwner, bool bCanEdit)
{
	if(InCurveOwner != CurveOwner)
	{
		EmptyAllSelection();
	}

	GradientViewer->SetCurveOwner(InCurveOwner);

	CurveOwner = InCurveOwner;
	bCanEditTrack = bCanEdit;

	if (bAreCurvesVisible.IsBound() == false || SetAreCurvesVisibleHandler.IsBound() == false)
	{
		bAreCurvesVisible = !IsLinearColorCurve();
	}

	bIsGradientEditorVisible = IsLinearColorCurve();

	CurveViewModels.Empty();
	if(CurveOwner != NULL)
	{
		for (const FRichCurveEditInfo& CurveInfo : CurveOwner->GetCurves())
		{
			CurveViewModels.Add(TSharedPtr<FCurveViewModel>(new FCurveViewModel(CurveInfo, CurveOwner->GetCurveColor(CurveInfo), !bCanEdit)));
		}
		if (bCanEdit)
		{
			CurveOwner->MakeTransactional();
		}
	}

	ValidateSelection();

	if (GetAutoFrame())
	{
		if( bZoomToFitVertical )
		{
			ZoomToFitVertical();
		}

		if ( bZoomToFitHorizontal )
		{
			ZoomToFitHorizontal();
		}
	}

	CurveSelectionWidget.Pin()->SetContent(CreateCurveSelectionWidget());
}

void SCurveEditor::RegisterToPropertyChangedEvent(const TSharedPtr<IPropertyHandle>& InRootPropertyHandle)
{
	RootPropertyHandleWeak = InRootPropertyHandle.ToWeakPtr();
	const auto OnChildPropertyChanged = TDelegate<void(const FPropertyChangedEvent&)>::CreateSP(this, &SCurveEditor::OnRootPropertyChanged);

	if (const TSharedPtr<IPropertyHandle> RootPropertyHandle = RootPropertyHandleWeak.Pin())
	{
		RootPropertyHandle->SetOnChildPropertyValueChangedWithData(OnChildPropertyChanged);
	}
}

void SCurveEditor::SetZoomToFit(bool bNewZoomToFitVertical, bool bNewZoomToFitHorizontal)
{
	bZoomToFitVertical = bNewZoomToFitVertical;
	bZoomToFitHorizontal = bNewZoomToFitHorizontal;
}

void SCurveEditor::SetRequireFocusToZoom(bool bInRequireFocusToZoom)
{
	bRequireFocusToZoom = bInRequireFocusToZoom;
}

TOptional<bool> SCurveEditor::OnQueryShowFocus(const EFocusCause InFocusCause) const
{
	if (bRequireFocusToZoom)
	{
		// Enable showing a focus rectangle when the widget has keyboard focus
		return TOptional<bool>(true);
	}
	return TOptional<bool>();
}

FCurveOwnerInterface* SCurveEditor::GetCurveOwner() const
{
	return CurveOwner;
}

FRealCurve* SCurveEditor::GetCurve(int32 CurveIndex) const
{
	if(CurveIndex < CurveViewModels.Num())
	{
		return CurveViewModels[CurveIndex]->CurveInfo.CurveToEdit;
	}
	return NULL;
}

void SCurveEditor::DeleteSelectedKeys()
{
	const FScopedTransaction Transaction(LOCTEXT("CurveEditor_RemoveKeys", "Delete Key(s)"));
	CurveOwner->ModifyOwner();
	TSet<FRealCurve*> ChangedCurves;

	// While there are still keys
	while(SelectedKeys.Num() > 0)
	{
		// Pull one out of the selected set
		FSelectedCurveKey Key = SelectedKeys.Pop();
		if(IsValidCurve(Key.Curve))
		{
			// Remove from the curve
			Key.Curve->DeleteKey(Key.KeyHandle);
			ChangedCurves.Add(Key.Curve);
		}
	}

	TArray<FRichCurveEditInfo> ChangedCurveEditInfos;
	for (auto CurveViewModel : CurveViewModels)
	{
		if (ChangedCurves.Contains(CurveViewModel->CurveInfo.CurveToEdit))
		{
			ChangedCurveEditInfos.Add(CurveViewModel->CurveInfo);
		}
	}

	CurveOwner->OnCurveChanged(ChangedCurveEditInfos);
}

FReply SCurveEditor::OnMouseButtonDown( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	// End any transactions that weren't ended cleanly
	EndDragTransaction();

	const bool bLeftMouseButton = InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton;
	const bool bMiddleMouseButton = InMouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton;
	const bool bRightMouseButton = InMouseEvent.GetEffectingButton() == EKeys::RightMouseButton;

	DragState = EDragState::PreDrag;
	MovementAxisLock = EMovementAxisLock::None;

	if (bLeftMouseButton || bMiddleMouseButton || bRightMouseButton)
	{
		MouseDownLocation = InMyGeometry.AbsoluteToLocal(InMouseEvent.GetScreenSpacePosition());

		// Set keyboard focus to this so that selected text box doesn't try to apply to newly selected keys
		if(!HasKeyboardFocus())
		{
			FSlateApplication::Get().SetKeyboardFocus(SharedThis(this), EFocusCause::SetDirectly);
		}

		// Always capture mouse if we left or right click on the widget
		return FReply::Handled().CaptureMouse(SharedThis(this));
	}

	return FReply::Unhandled();
}

void SCurveEditor::AddNewKey(FGeometry InMyGeometry, FVector2D ScreenPosition, TSharedPtr<TArray<TSharedPtr<FCurveViewModel>>> CurvesToAddKeysTo, bool bAddKeysInline)
{
	const FScopedTransaction Transaction(LOCTEXT("CurveEditor_AddKey", "Add Key(s)"));
	CurveOwner->ModifyOwner();
	TArray<FRichCurveEditInfo> ChangedCurveEditInfos;
	for (TSharedPtr<FCurveViewModel> CurveViewModel : *CurvesToAddKeysTo)
	{
		if (!CurveViewModel->bIsLocked)
		{
			FRealCurve* SelectedCurve = CurveViewModel->CurveInfo.CurveToEdit;
			if (IsValidCurve(SelectedCurve))
			{
				FTrackScaleInfo ScaleInfo(ViewMinInput.Get(), ViewMaxInput.Get(), ViewMinOutput.Get(), ViewMaxOutput.Get(), InMyGeometry.GetLocalSize());

				FVector2D LocalClickPos = InMyGeometry.AbsoluteToLocal(ScreenPosition);

				float Input = ScaleInfo.LocalXToInput(LocalClickPos.X);
				float Output;
				if (bAddKeysInline)
			{
					Output = SelectedCurve->Eval(Input);
				}
				else
				{
					Output = ScaleInfo.LocalYToOutput(LocalClickPos.Y);
				}
				FVector2D NewKeyLocation = SnapLocation(FVector2D(Input, Output));
				FKeyHandle NewKeyHandle = SelectedCurve->AddKey(NewKeyLocation.X, NewKeyLocation.Y);

				EmptyAllSelection();
				AddToKeySelection(FSelectedCurveKey(SelectedCurve, NewKeyHandle));
				ChangedCurveEditInfos.Add(CurveViewModel->CurveInfo);
			}
		}
	}

	if (ChangedCurveEditInfos.Num() > 0)
	{
		CurveOwner->OnCurveChanged(ChangedCurveEditInfos);
	}
}

void SCurveEditor::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	// if we began a drag transaction we need to finish it to make sure undo doesn't get out of sync
	if (DragState == EDragState::DragKey || DragState == EDragState::FreeDrag || DragState == EDragState::DragTangent)
	{
		EndDragTransaction();
	}
	DragState = EDragState::None;
}

FReply SCurveEditor::OnMouseButtonUp( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent )
{
	if (this->HasMouseCapture())
	{
		if (DragState == EDragState::PreDrag)
		{
			// If the user didn't start dragging, handle the mouse operation as a click.
			ProcessClick(InMyGeometry, InMouseEvent);
		}
		else
		{
			EndDrag(InMyGeometry, InMouseEvent);
		}
		return FReply::Handled().ReleaseMouseCapture();
	}
	return FReply::Unhandled();
}

void ClampViewRangeToDataIfBound( float& NewViewMin, float& NewViewMax, const TAttribute< TOptional<float> > & DataMin, const TAttribute< TOptional<float> > & DataMax, const float ViewRange)
{
	// if we have data bound
	const TOptional<float> & Min = DataMin.Get();
	const TOptional<float> & Max = DataMax.Get();
	if ( Min.IsSet() && NewViewMin < Min.GetValue())
	{
		// if we have min data set
		NewViewMin = Min.GetValue();
		NewViewMax = ViewRange;
	}
	else if ( Max.IsSet() && NewViewMax > Max.GetValue() )
	{
		// if we have min data set
		NewViewMin = Max.GetValue() - ViewRange;
		NewViewMax = Max.GetValue();
	}
}

FReply SCurveEditor::OnMouseMove( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent )
{
	UpdateCurveToolTip(InMyGeometry, InMouseEvent);

	FRealCurve* Curve = GetCurve(0);
	if( Curve != NULL && this->HasMouseCapture())
	{
		if (DragState == EDragState::PreDrag)
		{
			TryStartDrag(InMyGeometry, InMouseEvent);
		}
		if (DragState != EDragState::None)
		{
			ProcessDrag(InMyGeometry, InMouseEvent);
		}
		MouseMoveLocation = InMyGeometry.AbsoluteToLocal(InMouseEvent.GetScreenSpacePosition());
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SCurveEditor::UpdateCurveToolTip(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (Settings->GetShowCurveEditorCurveToolTips())
	{
		TSharedPtr<FCurveViewModel> HoveredCurve = HitTestCurves(InMyGeometry, InMouseEvent);
		//Display the tooltip only when the curve is visible
		if (HoveredCurve.IsValid() && HoveredCurve->bIsVisible)
		{
			FTrackScaleInfo ScaleInfo(ViewMinInput.Get(), ViewMaxInput.Get(), ViewMinOutput.Get(), ViewMaxOutput.Get(), InMyGeometry.GetLocalSize());
			const FVector2D HitPosition = InMyGeometry.AbsoluteToLocal(InMouseEvent.GetScreenSpacePosition());
			float Time = ScaleInfo.LocalXToInput(HitPosition.X);
			float Value = HoveredCurve->CurveInfo.CurveToEdit->Eval(Time);

			FNumberFormattingOptions FormattingOptions;
			FormattingOptions.MaximumFractionalDigits = 2;
			CurveToolTipNameText = FText::FromName(HoveredCurve->CurveInfo.CurveName);
			CurveToolTipOutputText = FText::Format(LOCTEXT("CurveToolTipValueFormat", "{0}: {1}"), OutputAxisName, FText::AsNumber(Value, &FormattingOptions));

			if (ShowTimeInFrames())
			{
				CurveToolTipInputText = FText::Format(LOCTEXT("CurveToolTipFrameFormat", "{0}: {1}"), GetInputAxisName(), FText::AsNumber(TimeToFrame(Time)));
			}
			else
			{
				CurveToolTipInputText = FText::Format(LOCTEXT("CurveToolTipTimeFormat", "{0}: {1}"), GetInputAxisName(), FText::AsNumber(Time, &FormattingOptions));
			}

			if (CurveToolTip.IsValid() == false)
			{
				SetToolTip(
					SAssignNew(CurveToolTip, SToolTip)
					.BorderImage( FCoreStyle::Get().GetBrush( "ToolTip.BrightBackground" ) )
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						[
							SNew(STextBlock)
							.Text(this, &SCurveEditor::GetCurveToolTipNameText)
							.Font(FCoreStyle::Get().GetFontStyle("ToolTip.LargerFont"))
							.ColorAndOpacity( FLinearColor::Black)
						]
						+ SVerticalBox::Slot()
						[
							SNew(STextBlock)
							.Text(this, &SCurveEditor::GetCurveToolTipInputText)
							.Font(FCoreStyle::Get().GetFontStyle("ToolTip.LargerFont"))
							.ColorAndOpacity(FLinearColor::Black)
						]
						+ SVerticalBox::Slot()
						[
							SNew(STextBlock)
							.Text(this, &SCurveEditor::GetCurveToolTipOutputText)
							.Font(FCoreStyle::Get().GetFontStyle("ToolTip.LargerFont"))
							.ColorAndOpacity(FLinearColor::Black)
						]
					]);
			}
		}
		else
		{
			CurveToolTip.Reset();
			SetToolTip(CurveToolTip);
		}
	}
}

FReply SCurveEditor::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (!bRequireFocusToZoom || HasKeyboardFocus())
	{
		ZoomView(FVector2D(MouseEvent.GetWheelDelta(), MouseEvent.GetWheelDelta()));
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SCurveEditor::ZoomView(FVector2D Delta)
{
	const FVector2D ZoomDelta = -0.1f * Delta;

	if (bAllowZoomOutput)
	{
		const float OutputViewSize = ViewMaxOutput.Get() - ViewMinOutput.Get();
		const float OutputChange = OutputViewSize * ZoomDelta.Y;

		const float NewMinOutput = (ViewMinOutput.Get() - (OutputChange * 0.5f));
		const float NewMaxOutput = (ViewMaxOutput.Get() + (OutputChange * 0.5f));

		SetOutputMinMax(NewMinOutput, NewMaxOutput);
	}

	{
		const float InputViewSize = ViewMaxInput.Get() - ViewMinInput.Get();
		const float InputChange = InputViewSize * ZoomDelta.X;

		const float NewMinInput = ViewMinInput.Get() - (InputChange * 0.5f);
		const float NewMaxInput = ViewMaxInput.Get() + (InputChange * 0.5f);

		SetInputMinMax(NewMinInput, NewMaxInput);
	}
}

FReply SCurveEditor::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	if (InKeyEvent.GetKey() == EKeys::Platform_Delete && SelectedKeys.Num() != 0)
	{
		DeleteSelectedKeys();
		return FReply::Handled();
	}
	else
	{
		if( Commands->ProcessCommandBindings( InKeyEvent ) )
		{
			return FReply::Handled();
		}
		return FReply::Unhandled();
	}
}

void SCurveEditor::TryStartDrag(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	const bool bLeftMouseButton = InMouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton);
	const bool bMiddleMouseButton = InMouseEvent.IsMouseButtonDown(EKeys::MiddleMouseButton);
	const bool bRightMouseButton = InMouseEvent.IsMouseButtonDown(EKeys::RightMouseButton);
	const bool bControlDown = InMouseEvent.IsControlDown();
	const bool bShiftDown = InMouseEvent.IsShiftDown();
	const bool bAltDown = InMouseEvent.IsAltDown();

	FVector2D MousePosition = InMyGeometry.AbsoluteToLocal(InMouseEvent.GetScreenSpacePosition());
	FVector2D DragVector = MousePosition - MouseDownLocation;
	if (DragVector.SizeSquared() >= FMath::Square(DragThreshold))
	{
		if (bShiftDown)
		{
			if(FMath::Abs(MousePosition.X - MouseDownLocation.X) > FMath::Abs(MousePosition.Y - MouseDownLocation.Y))
			{
				MovementAxisLock = EMovementAxisLock::AxisLock_Horizontal;
			}
			else
			{
				MovementAxisLock = EMovementAxisLock::AxisLock_Vertical;
			}
		}

		if (bLeftMouseButton)
		{
			// Check if we should start dragging keys.
			FSelectedCurveKey HitKey = HitTestKeys(InMyGeometry, InMyGeometry.LocalToAbsolute(MouseDownLocation));
			if (HitKey.IsValid())
			{
				EmptyTangentSelection();

				if (!IsKeySelected(HitKey))
				{
					if (!bControlDown)
					{
						EmptyKeySelection();
					}
					AddToKeySelection(HitKey);
				}

				BeginDragTransaction();
				DragState = EDragState::DragKey;
				DraggedKeyHandle = HitKey.KeyHandle;
				PreDragKeyLocations.Empty();
				for (auto SelectedKey : SelectedKeys)
				{
					PreDragKeyLocations.Add(SelectedKey.KeyHandle, FVector2D
					(
						SelectedKey.Curve->GetKeyTime(SelectedKey.KeyHandle),
						SelectedKey.Curve->GetKeyValue(SelectedKey.KeyHandle)
					));
				}
			}
			else
			{
				// Check if we should start dragging a tangent.
				FSelectedTangent Tangent = HitTestCubicTangents(InMyGeometry, InMyGeometry.LocalToAbsolute(MouseDownLocation));
				if (Tangent.IsValid())
				{
					EmptyKeySelection();

					if (!IsTangentSelected(Tangent))
					{
						if (!bControlDown)
						{
							EmptyTangentSelection();
						}
						AddToTangentSelection(Tangent);
					}

					BeginDragTransaction();
					DragState = EDragState::DragTangent;
					PreDragTangents.Empty();
					for (auto SelectedTangent : SelectedTangents)
					{
						FRichCurve* Curve = (FRichCurve*)SelectedTangent.Key.Curve;
						FKeyHandle KeyHandle = SelectedTangent.Key.KeyHandle;

						float ArriveTangent = Curve->GetKey(KeyHandle).ArriveTangent;
						float LeaveTangent = Curve->GetKey(KeyHandle).LeaveTangent;

						PreDragTangents.Add(KeyHandle, FVector2D(ArriveTangent, LeaveTangent));
					}
				}
				else
				{
					// Otherwise if the user left clicked on nothing and start a marquee select.
					DragState = EDragState::MarqueeSelect;
				}
			}
		}
		else if (bMiddleMouseButton)
		{
			if (bAltDown)
			{
				DragState = EDragState::Pan;
			}
			else if (SelectedTangents.Num())
			{
				BeginDragTransaction();
				DragState = EDragState::DragTangent;
				PreDragTangents.Empty();
				for (auto SelectedTangent : SelectedTangents)
				{
					FRichCurve* Curve = (FRichCurve*)SelectedTangent.Key.Curve;
					FKeyHandle KeyHandle = SelectedTangent.Key.KeyHandle;

					float ArriveTangent = Curve->GetKey(KeyHandle).ArriveTangent;
					float LeaveTangent = Curve->GetKey(KeyHandle).LeaveTangent;

					PreDragTangents.Add(KeyHandle, FVector2D(ArriveTangent, LeaveTangent));
				}
			}
			else if (SelectedKeys.Num())
			{
				BeginDragTransaction();
				DragState = EDragState::FreeDrag;
				PreDragKeyLocations.Empty();
				for (auto selectedKey : SelectedKeys)
				{
					PreDragKeyLocations.Add(selectedKey.KeyHandle, FVector2D
					(
						selectedKey.Curve->GetKeyTime(selectedKey.KeyHandle),
						selectedKey.Curve->GetKeyValue(selectedKey.KeyHandle)
					));
				}
			}
		}
		else if (bRightMouseButton)
		{
			if (bAltDown)
			{
				DragState = EDragState::Zoom;
			}
			else
			{
				DragState = EDragState::Pan;
			}
		}
		else
		{
			DragState = EDragState::None;
		}
	}
}

/* Given a tangent value for a key, calculates the 2D delta vector from that key in curve space */
static inline FVector2D CalcTangentDir(float Tangent)
{
	const float Angle = FMath::Atan(Tangent);
	return FVector2D( FMath::Cos(Angle), -FMath::Sin(Angle) );
}

/*Given a 2d delta vector in curve space, calculates a tangent value */
static inline float CalcTangent(const FVector2D& HandleDelta)
{
	// Ensure X is positive and non-zero.
	// Tangent is gradient of handle.
	return HandleDelta.Y / FMath::Max<double>(HandleDelta.X, KINDA_SMALL_NUMBER);
}

void SCurveEditor::ProcessDrag(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	FTrackScaleInfo ScaleInfo(ViewMinInput.Get(), ViewMaxInput.Get(), ViewMinOutput.Get(), ViewMaxOutput.Get(), InMyGeometry.GetLocalSize());
	FVector2D ScreenDelta = InMouseEvent.GetCursorDelta();

	FVector2D InputDelta;
	InputDelta.X = ScreenDelta.X / ScaleInfo.PixelsPerInput;
	InputDelta.Y = -ScreenDelta.Y / ScaleInfo.PixelsPerOutput;

	if (DragState == EDragState::DragKey)
	{
		FVector2D MousePosition = InMyGeometry.AbsoluteToLocal(InMouseEvent.GetScreenSpacePosition());
		FVector2D NewLocation = FVector2D(ScaleInfo.LocalXToInput(MousePosition.X), ScaleInfo.LocalYToOutput(MousePosition.Y));
		FVector2D SnappedNewLocation = SnapLocation(NewLocation);
		FVector2D Delta = SnappedNewLocation - PreDragKeyLocations[DraggedKeyHandle];

		MoveSelectedKeys(Delta);
	}
	else if (DragState == EDragState::FreeDrag)
	{
		FVector2D MousePosition = InMyGeometry.AbsoluteToLocal(InMouseEvent.GetScreenSpacePosition());
		FVector2D NewLocation = FVector2D(ScaleInfo.LocalXToInput(MousePosition.X), ScaleInfo.LocalYToOutput(MousePosition.Y));
		FVector2D Delta = NewLocation - FVector2D(ScaleInfo.LocalXToInput(MouseDownLocation.X), ScaleInfo.LocalYToOutput(MouseDownLocation.Y));

		MoveSelectedKeys(Delta);
	}
	else if (DragState == EDragState::DragTangent)
	{
		FVector2D MousePositionScreen = InMyGeometry.AbsoluteToLocal(InMouseEvent.GetScreenSpacePosition());
		FVector2D MouseDownPositionScreen = MouseDownLocation;
		MoveTangents(ScaleInfo, MousePositionScreen - MouseDownPositionScreen);
	}
	else if (DragState == EDragState::Pan)
	{
		if (MovementAxisLock == EMovementAxisLock::AxisLock_Horizontal)
		{
			InputDelta.Y = 0;
		}
		else if (MovementAxisLock == EMovementAxisLock::AxisLock_Vertical)
		{
			InputDelta.X = 0;
		}

		// Output is not clamped.
		const float NewMinOutput = (ViewMinOutput.Get() - InputDelta.Y);
		const float NewMaxOutput = (ViewMaxOutput.Get() - InputDelta.Y);

		SetOutputMinMax(NewMinOutput, NewMaxOutput);

		// Input maybe clamped if DataMinInput or DataMaxOutput was set.
		float NewMinInput = ViewMinInput.Get() - InputDelta.X;
		float NewMaxInput = ViewMaxInput.Get() - InputDelta.X;
		ClampViewRangeToDataIfBound(NewMinInput, NewMaxInput, DataMinInput, DataMaxInput, ScaleInfo.ViewInputRange);

		SetInputMinMax(NewMinInput, NewMaxInput);
	}
	else if (DragState == EDragState::Zoom)
	{
		FVector2D Delta = FVector2D(ScreenDelta.X * 0.05f, ScreenDelta.X * 0.05f);

		if (MovementAxisLock == EMovementAxisLock::AxisLock_Horizontal)
		{
			Delta.Y = 0;
		}
		else if (MovementAxisLock == EMovementAxisLock::AxisLock_Vertical)
		{
			Delta.X = 0;
			Delta.Y = -ScreenDelta.Y * 0.1f;
		}

		ZoomView(Delta);
	}
}

void SCurveEditor::EndDrag(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	const bool bControlDown = InMouseEvent.IsControlDown();
	const bool bShiftDown = InMouseEvent.IsShiftDown();

	if (DragState == EDragState::DragKey || DragState == EDragState::FreeDrag || DragState == EDragState::DragTangent)
	{
		EndDragTransaction();
	}
	else if (DragState == EDragState::MarqueeSelect)
	{
		FVector2D MarqueTopLeft
		(
			FMath::Min(MouseDownLocation.X, MouseMoveLocation.X),
			FMath::Min(MouseDownLocation.Y, MouseMoveLocation.Y)
			);

		FVector2D MarqueBottomRight
		(
			FMath::Max(MouseDownLocation.X, MouseMoveLocation.X),
			FMath::Max(MouseDownLocation.Y, MouseMoveLocation.Y)
			);

		TArray<FSelectedTangent> SelectedCurveTangents = GetEditableTangentsWithinMarquee(InMyGeometry, MarqueTopLeft, MarqueBottomRight);

		TArray<FSelectedCurveKey> SelectedCurveKeys = GetEditableKeysWithinMarquee(InMyGeometry, MarqueTopLeft, MarqueBottomRight);

		if (!bControlDown && !bShiftDown)
		{
			EmptyAllSelection();
		}

		if (SelectedCurveKeys.Num())
		{
			EmptyTangentSelection();

			for (auto SelectedCurveKey : SelectedCurveKeys)
			{			
				if (IsKeySelected(SelectedCurveKey))
				{
					RemoveFromKeySelection(SelectedCurveKey);
				}
				else
				{
					AddToKeySelection(SelectedCurveKey);
				}
			}
		}

		if (!SelectedCurveKeys.Num())
		{
			EmptyKeySelection();

			for (auto SelectedCurveTangent : SelectedCurveTangents)
			{			
				if (IsTangentSelected(SelectedCurveTangent))
				{
					RemoveFromTangentSelection(SelectedCurveTangent);
				}
				else
				{
					AddToTangentSelection(SelectedCurveTangent);
				}
			}
		}
	}
	DragState = EDragState::None;
	MovementAxisLock = EMovementAxisLock::None;
}

void SCurveEditor::ProcessClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	const bool bLeftMouseButton = InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton;
	const bool bRightMouseButton = InMouseEvent.GetEffectingButton() == EKeys::RightMouseButton;
	const bool bControlDown = InMouseEvent.IsControlDown();
	const bool bShiftDown = InMouseEvent.IsShiftDown();

	FSelectedCurveKey HitKey = HitTestKeys(InMyGeometry, InMouseEvent.GetScreenSpacePosition());
	FSelectedTangent HitTangent = HitTestCubicTangents(InMyGeometry, InMouseEvent.GetScreenSpacePosition());

	if (bLeftMouseButton)
	{
		// If the user left clicked a key, update selection based on modifier key state.
		if (HitKey.IsValid())
		{
			EmptyTangentSelection();

			if (!IsKeySelected(HitKey))
			{
				if (!bControlDown && !bShiftDown)
				{
					EmptyAllSelection();
				}
				AddToKeySelection(HitKey);
			}
			else if (bControlDown)
			{
				RemoveFromKeySelection(HitKey);
			}
		}
		else if (HitTangent.IsValid())
		{
			EmptyKeySelection();

			if (!IsTangentSelected(HitTangent))
			{
				if (!bControlDown && !bShiftDown)
				{
					EmptyAllSelection();
				}
				AddToTangentSelection(HitTangent);
			}
			else if (bControlDown)
			{
				RemoveFromTangentSelection(HitTangent);
			}
		}
		else
		{
			// If the user didn't click a key, add a new one if shift is held down, or try to select a curve.
			if (bShiftDown && IsEditingEnabled())
			{
				TSharedPtr<TArray<TSharedPtr<FCurveViewModel>>> CurvesToAddKeysTo = MakeShareable(new TArray<TSharedPtr<FCurveViewModel>>());
				TSharedPtr<FCurveViewModel> HoveredCurve = HitTestCurves(InMyGeometry, InMouseEvent);
				bool bAddKeysInline;
				//To snap a point on the hovered curve the curve must be visible and unlock
				if (HoveredCurve.IsValid() && !HoveredCurve->bIsLocked && HoveredCurve->bIsVisible)
				{
					CurvesToAddKeysTo->Add(HoveredCurve);
					bAddKeysInline = true;
				}
				else
				{
					//Add all unlock curves in the editable array
					for (auto CurveViewModel : CurveViewModels)
					{
						if (!CurveViewModel->bIsLocked)
						{
							CurvesToAddKeysTo->Add(CurveViewModel);
						}
					}

					//If linear color curve and no show curve, always insert inline
					//If the user is holding shift-ctrl we snap all curve to the mouse position. (false value)
					//If the user is holding shift we snap to mouse only if there is only one editable curve. (false value)
					//In all other case we add key directly on the curve. (true value)
					bAddKeysInline = (IsLinearColorCurve() && !bAreCurvesVisible.Get()) || ((!bControlDown) && (CurvesToAddKeysTo->Num() != 1));
				}
				AddNewKey(InMyGeometry, InMouseEvent.GetScreenSpacePosition(), CurvesToAddKeysTo, bAddKeysInline);
			}
			else
			{
				// clicking on background clears all selection
				EmptyAllSelection();
			}
		}
	}
	else if (bRightMouseButton)
	{
		// If the user right clicked, handle opening context menus.
		if (HitKey.IsValid())
		{
			// Make sure key is selected in readiness for context menu
			EmptyTangentSelection();

			if (!IsKeySelected(HitKey))
			{
				EmptyAllSelection();
				AddToKeySelection(HitKey);
			}
			PushKeyMenu(InMyGeometry, InMouseEvent);
		}
		else if (HitTangent.IsValid())
		{
			// Make sure key is selected in readiness for context menu
			EmptyKeySelection();

			if (!IsTangentSelected(HitTangent))
			{
				EmptyAllSelection();
				AddToTangentSelection(HitTangent);
			}
			PushKeyMenu(InMyGeometry, InMouseEvent);
		}
		else
		{
			CreateContextMenu(InMyGeometry, InMouseEvent);
		}
	}
}

TOptional<float> SCurveEditor::OnGetTime() const
{
	TOptional<float> Time;

	// Return the time if all selected keys have the same time, otherwise return an unset value
	if (SelectedKeys.Num() > 0)
	{
		Time = GetKeyTime(SelectedKeys[0]);
		for (int32 i = 1; i < SelectedKeys.Num(); i++)
		{
			TOptional<float> NewTime = GetKeyTime(SelectedKeys[i]);
			bool bAreEqual = ((!Time.IsSet() && !NewTime.IsSet()) || (Time.IsSet() && NewTime.IsSet() && Time.GetValue() == NewTime.GetValue()));
			if (!bAreEqual)
			{
				return TOptional<float>();
			}
		}
	}

	return Time;
}

void SCurveEditor::OnTimeComitted(float NewTime, ETextCommit::Type CommitType)
{
	// Don't digest the number if we just clicked away from the pop-up
	if ( !bIsUsingSlider && ((CommitType == ETextCommit::OnEnter) || ( CommitType == ETextCommit::OnUserMovedFocus )) )
	{
		for(FSelectedCurveKey Key : SelectedKeys)
		{
			UpdateCurveTimeSingleKey(Key, NewTime);
		}

		FSlateApplication::Get().DismissAllMenus();
	}
}

void SCurveEditor::OnTimeChanged(float NewTime)
{
	if ( bIsUsingSlider )
	{
		for (FSelectedCurveKey Key : SelectedKeys)
		{
			UpdateCurveTimeSingleKey(Key, NewTime);
		}
	}
}

TOptional<int32> SCurveEditor::OnGetTimeInFrames() const
{
	TOptional<float> Time;

	// Return the time in frames if all selected keys have the same time, otherwise return an unset value
	if (SelectedKeys.Num() > 0)
	{
		Time = GetKeyTime(SelectedKeys[0]);
		for (int32 i = 1; i < SelectedKeys.Num(); i++)
		{
			TOptional<float> NewTime = GetKeyTime(SelectedKeys[i]);
			bool bAreEqual = ((!Time.IsSet() && !NewTime.IsSet()) || (Time.IsSet() && NewTime.IsSet() && Time.GetValue() == NewTime.GetValue()));
			if (!bAreEqual)
			{
				return TOptional<int32>();
			}
		}
	}

	if (Time.IsSet())
	{
		return TOptional<int32>(TimeToFrame(Time.GetValue()));
	}

	return TOptional<int32>();
}

void SCurveEditor::OnTimeInFramesComitted(int32 NewFrame, ETextCommit::Type CommitType)
{
	// Don't digest the number if we just clicked away from the pop-up
	if ( !bIsUsingSlider && ((CommitType == ETextCommit::OnEnter) || ( CommitType == ETextCommit::OnUserMovedFocus )) )
	{
		for (FSelectedCurveKey Key : SelectedKeys)
		{
			UpdateCurveTimeSingleKey(Key, NewFrame);
		}

		FSlateApplication::Get().DismissAllMenus();
	}
}

void SCurveEditor::OnTimeInFramesChanged(int32 NewFrame)
{
	if ( bIsUsingSlider )
	{
		for (FSelectedCurveKey Key : SelectedKeys)
		{
			UpdateCurveTimeSingleKey(Key, NewFrame);
		}
	}
}

void SCurveEditor::UpdateCurveTimeSingleKey(FSelectedCurveKey Key, float NewTime, bool bSetFromFrame)
{
	// If the curve is valid and doesn't already have a key at that time
	if (IsValidCurve(Key.Curve))
	{
		if (!Key.Curve->KeyExistsAtTime(NewTime))
		{
			FText TransactionText = bSetFromFrame ? LOCTEXT("CurveEditor_NewFrame", "New Frame Entered") : LOCTEXT("CurveEditor_NewTime", "New Time Entered");
			const FScopedTransaction Transaction(TransactionText);
			CurveOwner->ModifyOwner();
			Key.Curve->SetKeyTime(Key.KeyHandle, NewTime);
			TArray<FRichCurveEditInfo> ChangedCurveEditInfos;
			ChangedCurveEditInfos.Add(GetViewModelForCurve(Key.Curve)->CurveInfo);
			CurveOwner->OnCurveChanged(ChangedCurveEditInfos);
		}
		// if the existing key is not this key
		else if (Key.Curve->FindKey(NewTime) != Key.KeyHandle)
		{
			LogAndToastCurveTimeWarning(Key.Curve);
		}
	}
}

void SCurveEditor::UpdateCurveTimeSingleKey(FSelectedCurveKey Key, int32 NewFrame)
{
	UpdateCurveTimeSingleKey(Key, FrameToTime(NewFrame), true);
}

void SCurveEditor::LogAndToastCurveTimeWarning(FRealCurve* Curve)
{
	FText Error = FText::Format(LOCTEXT("KeyTimeCollision","A key on {0} could not be moved because there was already a key at that time."), FText::FromName(GetViewModelForCurve(Curve)->CurveInfo.CurveName));
	FNotificationInfo Info(Error);
	Info.ExpireDuration = 5.0f;
	FSlateNotificationManager::Get().AddNotification(Info);

	UE_LOGF(LogCurveEditor, Warning, "%ls", *Error.ToString());
}

TOptional<float> SCurveEditor::OnGetValue() const
{
	TOptional<float> Value;

	// Return the value string if all selected keys have the same output string, otherwise empty
	if ( SelectedKeys.Num() > 0 )
	{
		Value = GetKeyValue(SelectedKeys[0]);
		for ( int32 i=1; i < SelectedKeys.Num(); i++ )
		{
			TOptional<float> NewValue = GetKeyValue(SelectedKeys[i]);
			bool bAreEqual = ( ( !Value.IsSet() && !NewValue.IsSet() ) || ( Value.IsSet() && NewValue.IsSet() && Value.GetValue() == NewValue.GetValue() ) );
			if ( !bAreEqual )
			{
				return TOptional<float>();
			}
		}
	}

	return Value;
}

void SCurveEditor::OnValueComitted(float NewValue, ETextCommit::Type CommitType)
{
	// Don't digest the number if we just clicked away from the popup
	if ( !bIsUsingSlider && ((CommitType == ETextCommit::OnEnter) || ( CommitType == ETextCommit::OnUserMovedFocus )) )
	{
		const FScopedTransaction Transaction( LOCTEXT( "CurveEditor_NewValue", "New Value Entered" ) );
		CurveOwner->ModifyOwner();
		TSet<FRealCurve*> ChangedCurves;

		// Iterate over selected set
		for ( int32 i=0; i < SelectedKeys.Num(); i++ )
		{
			auto Key = SelectedKeys[i];
			if ( IsValidCurve(Key.Curve) )
			{
				// Fill in each element of this key
				Key.Curve->SetKeyValue(Key.KeyHandle, NewValue);
				ChangedCurves.Add(Key.Curve);
			}
		}

		TArray<FRichCurveEditInfo> ChangedCurveEditInfos;
		for (auto CurveViewModel : CurveViewModels)
		{
			if (ChangedCurves.Contains(CurveViewModel->CurveInfo.CurveToEdit))
			{
				ChangedCurveEditInfos.Add(CurveViewModel->CurveInfo);
			}
		}
		CurveOwner->OnCurveChanged(ChangedCurveEditInfos);

		FSlateApplication::Get().DismissAllMenus();
	}
}

void SCurveEditor::OnValueChanged(float NewValue)
{
	if ( bIsUsingSlider )
	{
		const FScopedTransaction Transaction(LOCTEXT("CurveEditor_NewValue", "New Value Entered"));
		TSet<FRealCurve*> ChangedCurves;

		// Iterate over selected set
		for ( int32 i=0; i < SelectedKeys.Num(); i++ )
		{
			auto Key = SelectedKeys[i];
			if ( IsValidCurve(Key.Curve) )
			{
				CurveOwner->ModifyOwner();

				// Fill in each element of this key
				Key.Curve->SetKeyValue(Key.KeyHandle, NewValue);
				ChangedCurves.Add(Key.Curve);
			}
		}

		TArray<FRichCurveEditInfo> ChangedCurveEditInfos;
		for (auto CurveViewModel : CurveViewModels)
		{
			if (ChangedCurves.Contains(CurveViewModel->CurveInfo.CurveToEdit))
			{
				ChangedCurveEditInfos.Add(CurveViewModel->CurveInfo);
			}
		}
		CurveOwner->OnCurveChanged(ChangedCurveEditInfos);
	}
}

void SCurveEditor::OnBeginSliderMovement(FText TransactionName)
{
	bIsUsingSlider = true;

	GEditor->BeginTransaction(TransactionName);
}

void SCurveEditor::OnEndSliderMovement(float NewValue)
{
	bIsUsingSlider = false;

	GEditor->EndTransaction();
}

void SCurveEditor::OnEndSliderMovement(int32 NewValue)
{
	bIsUsingSlider = false;

	GEditor->EndTransaction();
}


SCurveEditor::FSelectedCurveKey SCurveEditor::HitTestKeys(const FGeometry& InMyGeometry, const FVector2D& HitScreenPosition)
{	
	FSelectedCurveKey SelectedKey(NULL,FKeyHandle::Invalid());

	if( AreCurvesVisible() )
	{
		bool bAnyCurveViewModelsSelected = AnyCurveViewModelsSelected();

		FTrackScaleInfo ScaleInfo(ViewMinInput.Get(),  ViewMaxInput.Get(), ViewMinOutput.Get(), ViewMaxOutput.Get(), InMyGeometry.GetLocalSize());

		const FVector2D HitPosition = InMyGeometry.AbsoluteToLocal( HitScreenPosition );

		for(auto CurveViewModel : CurveViewModels)
		{
			if (IsCurveSelectable(CurveViewModel))
			{
				FRealCurve* Curve = CurveViewModel->CurveInfo.CurveToEdit;
				if(Curve != NULL)
				{
					for (auto It(Curve->GetKeyHandleIterator()); It; ++It)
					{
						float KeyScreenX = ScaleInfo.InputToLocalX(Curve->GetKeyTime(*It));
						float KeyScreenY = ScaleInfo.OutputToLocalY(Curve->GetKeyValue(*It));

						if(	HitPosition.X > (KeyScreenX - (0.5f * CONST_KeySize.X)) && 
							HitPosition.X < (KeyScreenX + (0.5f * CONST_KeySize.X)) &&
							HitPosition.Y > (KeyScreenY - (0.5f * CONST_KeySize.Y)) &&
							HitPosition.Y < (KeyScreenY + (0.5f * CONST_KeySize.Y)) )
						{
							return FSelectedCurveKey(Curve, *It);
						}
					}
				}
			}
		}
	}

	return SelectedKey;
}

void SCurveEditor::MoveSelectedKeys(FVector2D Delta)
{
	TArray<FRichCurveEditInfo> ChangedCurveEditInfos;

	const FScopedTransaction Transaction( LOCTEXT("CurveEditor_MoveKeys", "Move Keys") );
	CurveOwner->ModifyOwnerChange();

	// track all unique curves encountered so their tangents can be updated later
	TSet<FRealCurve*> UniqueCurves;

	// The total move distance for all keys is the difference between the current snapped location
	// and the start location of the key which was actually dragged.

	FVector2D TotalMoveDistance = Delta;

	for (int32 i = 0; i < SelectedKeys.Num(); i++)
	{
		FSelectedCurveKey OldKey = SelectedKeys[i];

		if (!IsValidCurve(OldKey.Curve))
		{
			continue;
		}

		FKeyHandle OldKeyHandle = OldKey.KeyHandle;
		FRealCurve* Curve = OldKey.Curve;

		FVector2D PreDragLocation = PreDragKeyLocations[OldKeyHandle];
		FVector2D NewLocation = PreDragLocation + TotalMoveDistance;

		// Update the key's value without updating the tangents.
		if (MovementAxisLock != EMovementAxisLock::AxisLock_Horizontal)
		{
			Curve->SetKeyValue(OldKeyHandle, NewLocation.Y, false);
		}

		// Changing the time of a key returns a new handle, so make sure to update existing references.
		if (MovementAxisLock != EMovementAxisLock::AxisLock_Vertical)
		{
			Curve->SetKeyTime(OldKeyHandle, NewLocation.X);
			SelectedKeys[i] = FSelectedCurveKey(Curve, OldKeyHandle);
			PreDragKeyLocations.Remove(OldKeyHandle);
			PreDragKeyLocations.Add(OldKeyHandle, PreDragLocation);
		}

		UniqueCurves.Add(Curve);
		ChangedCurveEditInfos.Add(GetViewModelForCurve(Curve)->CurveInfo);
	}

	if (CurveOwner->HasRichCurves())
	{
		// update auto tangents for all curves encountered, once each only
		for(TSet<FRealCurve*>::TIterator SetIt(UniqueCurves);SetIt;++SetIt)
		{
			((FRichCurve*)(*SetIt))->AutoSetTangents();
		}
	}

	if (ChangedCurveEditInfos.Num())
	{
		CurveOwner->OnCurveChanged(ChangedCurveEditInfos);
	}
}

TOptional<float> SCurveEditor::GetKeyValue(FSelectedCurveKey Key) const
{
	if(IsValidCurve(Key.Curve))
	{
		return Key.Curve->GetKeyValue(Key.KeyHandle);
	}

	return TOptional<float>();
}

TOptional<float> SCurveEditor::GetKeyTime(FSelectedCurveKey Key) const
{
	if ( IsValidCurve(Key.Curve) )
	{
		return Key.Curve->GetKeyTime(Key.KeyHandle);
	}

	return TOptional<float>();
}

void SCurveEditor::EmptyKeySelection()
{
	SelectedKeys.Empty();
}

void SCurveEditor::AddToKeySelection(FSelectedCurveKey Key)
{
	SelectedKeys.AddUnique(Key);
}

void SCurveEditor::RemoveFromKeySelection(FSelectedCurveKey Key)
{
	SelectedKeys.Remove(Key);
}

bool SCurveEditor::IsKeySelected(FSelectedCurveKey Key) const
{
	return SelectedKeys.Contains(Key);
}

bool SCurveEditor::AreKeysSelected() const
{
	return SelectedKeys.Num() > 0;
}

void SCurveEditor::EmptyTangentSelection()
{
	SelectedTangents.Empty();
}

void SCurveEditor::AddToTangentSelection(FSelectedTangent Tangent)
{
	SelectedTangents.Add(Tangent);
}

void SCurveEditor::RemoveFromTangentSelection(FSelectedTangent Tangent)
{
	SelectedTangents.Remove(Tangent);
}

bool SCurveEditor::IsTangentSelected(FSelectedTangent Tangent) const
{
	return SelectedTangents.Contains(Tangent);
}

bool SCurveEditor::AreTangentsSelected() const
{
	return SelectedTangents.Num() > 0;
}

bool SCurveEditor::IsTangentVisible(FRichCurve* Curve, FKeyHandle KeyHandle, bool& bIsTangentSelected, bool& bIsArrivalSelected, bool& bIsLeaveSelected) const
{
	bIsTangentSelected = false;
	bIsArrivalSelected = false;
	bIsLeaveSelected = false;

	bool IsSelected = IsKeySelected(FSelectedCurveKey(Curve,KeyHandle));
	for (auto SelectedTangent : SelectedTangents)
	{
		if (SelectedTangent.Key.KeyHandle == KeyHandle)
		{
			if (SelectedTangent.bIsArrival)
				bIsArrivalSelected = true;
			else
				bIsLeaveSelected = true;
			bIsTangentSelected = true;
		}
	}

	return (IsSelected || bIsTangentSelected || Settings->GetTangentVisibility() == ECurveEditorTangentVisibility::AllTangents) && Settings->GetTangentVisibility() != ECurveEditorTangentVisibility::NoTangents;
}

void SCurveEditor::EmptyAllSelection()
{
	EmptyKeySelection();
	EmptyTangentSelection();
}

void SCurveEditor::ValidateSelection()
{
	//remove all selection if we no longer have a curve interface (Curve Owner)
	if (!CurveOwner)
	{
		EmptyAllSelection();
		return;
	}

	//remove any invalid keys
	for(int32 i = 0;i<SelectedKeys.Num();++i)
	{
		auto Key = SelectedKeys[i];
		if(!IsValidCurve(Key.Curve) || !Key.IsValid())
		{
			SelectedKeys.RemoveAt(i);
			i--;
		}
	}

	// remove any invalid tangents
	for (int32 i = 0;i<SelectedTangents.Num();++i)
	{
		auto Tangent = SelectedTangents[i];
		if (!IsValidCurve(Tangent.Key.Curve) || !Tangent.Key.IsValid())
		{
			SelectedTangents.RemoveAt(i);
			i--;
		}
	}
}

bool SCurveEditor::GetAutoFrame() const
{
	return Settings->GetAutoFrameCurveEditor() && GetAllowAutoFrame();
}

TArray<FRealCurve*> SCurveEditor::GetCurvesToFit() const
{
	TArray<FRealCurve*> FitCurves;

	for(const TSharedPtr<FCurveViewModel>& CurveViewModel : CurveViewModels)
	{
		if (CurveViewModel->bIsVisible && CurveViewModel->CurveInfo.CurveToEdit != nullptr)
		{
			FitCurves.Add(CurveViewModel->CurveInfo.CurveToEdit);
		}
	}

	return FitCurves;
}


void SCurveEditor::ZoomToFitHorizontal(const bool bZoomToFitAll)
{
	TArray<FRealCurve*> CurvesToFit = GetCurvesToFit();

	if(CurveViewModels.Num() > 0)
	{
		float InMin = FLT_MAX;
		float InMax = -FLT_MAX;
		int32 TotalKeys = 0;

		if (SelectedKeys.Num() && !bZoomToFitAll)
		{
			for (auto SelectedKey : SelectedKeys)
			{
				TotalKeys++;
				float KeyTime = SelectedKey.Curve->GetKeyTime(SelectedKey.KeyHandle);
				InMin = FMath::Min(KeyTime, InMin);
				InMax = FMath::Max(KeyTime, InMax);

				FKeyHandle NextKeyHandle = SelectedKey.Curve->GetNextKey(SelectedKey.KeyHandle);
				if (SelectedKey.Curve->IsKeyHandleValid(NextKeyHandle))
				{
					float NextKeyTime = SelectedKey.Curve->GetKeyTime(NextKeyHandle);
					InMin = FMath::Min(NextKeyTime, InMin);
					InMax = FMath::Max(NextKeyTime, InMax);
				}

				FKeyHandle PreviousKeyHandle = SelectedKey.Curve->GetPreviousKey(SelectedKey.KeyHandle);
				if (SelectedKey.Curve->IsKeyHandleValid(PreviousKeyHandle))
				{
					float PreviousKeyTime = SelectedKey.Curve->GetKeyTime(PreviousKeyHandle);
					InMin = FMath::Min(PreviousKeyTime, InMin);
					InMax = FMath::Max(PreviousKeyTime, InMax);
				}
			}
		}
		else
		{
			for (FRealCurve* Curve : CurvesToFit)
			{
				float MinTime, MaxTime;
				Curve->GetTimeRange(MinTime, MaxTime);
				InMin = FMath::Min(MinTime, InMin);
				InMax = FMath::Max(MaxTime, InMax);
				TotalKeys += Curve->GetNumKeys();
			}
		}

		if (TotalKeys > 0)
		{
			// Clamp the minimum size
			float Size = InMax - InMin;
			if (Size < CONST_MinViewRange)
			{
				InMin -= (0.5f*CONST_MinViewRange);
				InMax += (0.5f*CONST_MinViewRange);
				Size = InMax - InMin;
			}

			// add margin
			InMin -= CONST_FitMargin*Size;
			InMax += CONST_FitMargin*Size;
		}
		else
		{
			InMin = -CONST_FitMargin*2.0f;
			InMax = (CONST_DefaultZoomRange + CONST_FitMargin)*2.0;
		}

		SetInputMinMax(InMin, InMax);
	}
}

FReply SCurveEditor::ZoomToFitHorizontalClicked()
{
	ZoomToFitHorizontal();
	return FReply::Handled();
}

/** Set Default output values when range is too small **/
void SCurveEditor::SetDefaultOutput(const float MinZoomRange)
{
	const float NewMinOutput = (ViewMinOutput.Get() - (0.5f*MinZoomRange));
	const float NewMaxOutput = (ViewMaxOutput.Get() + (0.5f*MinZoomRange));

	SetOutputMinMax(NewMinOutput, NewMaxOutput);
}

void SCurveEditor::ZoomToFitVertical(const bool bZoomToFitAll)
{
	TArray<FRealCurve*> CurvesToFit = GetCurvesToFit();

	if(CurvesToFit.Num() > 0)
	{
		float InMin = FLT_MAX;
		float InMax = -FLT_MAX;
		int32 TotalKeys = 0;

		if (SelectedKeys.Num() != 0 && !bZoomToFitAll)
		{
			for (auto SelectedKey : SelectedKeys)
			{
				TotalKeys++;
				float KeyValue = SelectedKey.Curve->GetKeyValue(SelectedKey.KeyHandle);
				InMin = FMath::Min(KeyValue, InMin);
				InMax = FMath::Max(KeyValue, InMax);

				FKeyHandle NextKeyHandle = SelectedKey.Curve->GetNextKey(SelectedKey.KeyHandle);
				if (SelectedKey.Curve->IsKeyHandleValid(NextKeyHandle))
				{
					float NextKeyValue = SelectedKey.Curve->GetKeyValue(NextKeyHandle);
					InMin = FMath::Min(NextKeyValue, InMin);
					InMax = FMath::Max(NextKeyValue, InMax);
				}

				FKeyHandle PreviousKeyHandle = SelectedKey.Curve->GetPreviousKey(SelectedKey.KeyHandle);
				if (SelectedKey.Curve->IsKeyHandleValid(PreviousKeyHandle))
				{
					float PreviousKeyValue = SelectedKey.Curve->GetKeyValue(PreviousKeyHandle);
					InMin = FMath::Min(PreviousKeyValue, InMin);
					InMax = FMath::Max(PreviousKeyValue, InMax);
				}
			}
		}
		else
		{
			for (FRealCurve* Curve : CurvesToFit)
			{
				float MinVal, MaxVal;
				Curve->GetValueRange(MinVal, MaxVal);
				InMin = FMath::Min(MinVal, InMin);
				InMax = FMath::Max(MaxVal, InMax);
				TotalKeys += Curve->GetNumKeys();
			}
		}

		const float MinZoomRange = (TotalKeys > 0 ) ? CONST_MinViewRange: CONST_DefaultZoomRange;

		// if in max and in min is same, then include 0.f
		if (InMax == InMin)
		{
			InMax = FMath::Max(InMax, 0.f);
			InMin = FMath::Min(InMin, 0.f);
		}

		// Clamp the minimum size
		float Size = InMax - InMin;
		if( Size < MinZoomRange )
		{
			SetDefaultOutput(MinZoomRange);
			InMin = ViewMinOutput.Get();
			InMax = ViewMaxOutput.Get();
			Size = InMax - InMin;
		}

		// add margin
		const float NewMinOutput = (InMin - CONST_FitMargin*Size);
		const float NewMaxOutput = (InMax + CONST_FitMargin*Size);

		SetOutputMinMax(NewMinOutput, NewMaxOutput);
	}
}

FReply SCurveEditor::ZoomToFitVerticalClicked()
{
	ZoomToFitVertical();
	return FReply::Handled();
}

void SCurveEditor::ZoomToFit(const bool bZoomToFitAll)
{
	ZoomToFitHorizontal(bZoomToFitAll);
	ZoomToFitVertical(bZoomToFitAll);
}

void SCurveEditor::ToggleInputSnapping()
{
	if (bInputSnappingEnabled.IsBound() == false)
	{
		bInputSnappingEnabled = !bInputSnappingEnabled.Get();
	}
}

void SCurveEditor::ToggleOutputSnapping()
{
	if (bOutputSnappingEnabled.IsBound() == false)
	{
		bOutputSnappingEnabled = !bOutputSnappingEnabled.Get();
	}
}

bool SCurveEditor::IsInputSnappingEnabled() const
{
	return bInputSnappingEnabled.Get();
}

bool SCurveEditor::IsOutputSnappingEnabled() const
{
	return bOutputSnappingEnabled.Get();
}

bool SCurveEditor::ShowTimeInFrames() const
{
	return bShowTimeInFrames.Get();
}

void SCurveEditor::CreateContextMenu(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	TSharedPtr<TArray<TSharedPtr<FCurveViewModel>>> CurvesToAddKeysTo = MakeShareable(new TArray<TSharedPtr<FCurveViewModel>>());

	bool bHoveredCurveValid = false;
	TSharedPtr<FCurveViewModel> HoveredCurve = HitTestCurves(InMyGeometry, InMouseEvent);
	//Curve must be visible and unlocked to show context menu
	if (HoveredCurve.IsValid() && !HoveredCurve->bIsLocked && HoveredCurve->bIsVisible)
	{
		CurvesToAddKeysTo->Add(HoveredCurve);
		bHoveredCurveValid = true;
	}
	else
	{
		// Get all editable curves
		for (auto CurveViewModel : CurveViewModels)
		{
			if (!CurveViewModel->bIsLocked)
			{
				CurvesToAddKeysTo->Add(CurveViewModel);
			}
		}
	}

	const bool bCreateExternalCurve = OnCreateAsset.IsBound() && IsEditingEnabled();
	const bool bShowLinearColorCurve = IsLinearColorCurve();

	// Early out if there's no menu items to show
	if (CurvesToAddKeysTo->Num() == 0 && !bCreateExternalCurve && !bShowLinearColorCurve)
	{
		return;
	}	

	const FVector2D& ScreenPosition = InMouseEvent.GetScreenSpacePosition();

	const bool CloseAfterSelection = true;
	FMenuBuilder MenuBuilder( CloseAfterSelection, NULL );

	MenuBuilder.BeginSection("EditCurveEditorActions", LOCTEXT("Actions", "Actions"));
	{
		FText MenuItemLabel;
		FText MenuItemToolTip;

		FText AddKeyToCurveLabelFormat = LOCTEXT("AddKeyToCurveLabelFormat", "Add key to {0}");
		FText AddKeyToCurveToolTipFormat = LOCTEXT("AddKeyToCurveToolTipFormat", "Add a new key at the hovered time to the {0} curve.  Keys can also be added with Shift + Click.");

		FVector2D Position = InMouseEvent.GetScreenSpacePosition();
	
		if (bHoveredCurveValid)
		{
			MenuItemLabel = FText::Format(AddKeyToCurveLabelFormat, FText::FromName(HoveredCurve->CurveInfo.CurveName));
			MenuItemToolTip = FText::Format(AddKeyToCurveToolTipFormat, FText::FromName(HoveredCurve->CurveInfo.CurveName));
			FUIAction Action = FUIAction(FExecuteAction::CreateSP(this, &SCurveEditor::AddNewKey, InMyGeometry, Position, CurvesToAddKeysTo, true));
			MenuBuilder.AddMenuEntry(MenuItemLabel, MenuItemToolTip, FSlateIcon(), Action);
		}
		else
		{
			if (CurvesToAddKeysTo->Num() == 1)
			{
				MenuItemLabel = FText::Format(AddKeyToCurveLabelFormat, FText::FromName((*CurvesToAddKeysTo)[0]->CurveInfo.CurveName));
				MenuItemToolTip = FText::Format(AddKeyToCurveToolTipFormat, FText::FromName((*CurvesToAddKeysTo)[0]->CurveInfo.CurveName));
				FUIAction Action = FUIAction(FExecuteAction::CreateSP(this, &SCurveEditor::AddNewKey, InMyGeometry, Position, CurvesToAddKeysTo, false));
				MenuBuilder.AddMenuEntry(MenuItemLabel, MenuItemToolTip, FSlateIcon(), Action);
			}
			else if (CurvesToAddKeysTo->Num() > 1) //Dont show the menu if we cannot edit any curve
			{
				//add key to all curve menu entry
				MenuItemLabel = LOCTEXT("AddKeyToAllCurves", "Add key to all curves");
				MenuItemToolTip = LOCTEXT("AddKeyToAllCurveToolTip", "Adds a key at the hovered time to all curves.  Keys can also be added with Shift + Click.");
				FUIAction Action = FUIAction(FExecuteAction::CreateSP(this, &SCurveEditor::AddNewKey, InMyGeometry, Position, CurvesToAddKeysTo, true));
				MenuBuilder.AddMenuEntry(MenuItemLabel, MenuItemToolTip, FSlateIcon(), Action);
				
				//This menu is not required when there is no curve display (color track can hide and show curves)
				if (bAreCurvesVisible.Get())
				{
					//add key and value to all curve menu entry
					MenuItemLabel = LOCTEXT("AddKeyValueToAllCurves", "Add key & value to all curves");
					MenuItemToolTip = LOCTEXT("AddKeyValueToAllCurveToolTip", "Adds a key & value at the hovered time to all curves.  Keys can also be added with Shift + ctrl + Click.");
					Action = FUIAction(FExecuteAction::CreateSP(this, &SCurveEditor::AddNewKey, InMyGeometry, Position, CurvesToAddKeysTo, false));
					MenuBuilder.AddMenuEntry(MenuItemLabel, MenuItemToolTip, FSlateIcon(), Action);
				}
			}
		}
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("CurveEditorActions", LOCTEXT("CurveAction", "Curve Actions") );
	{
		if( bCreateExternalCurve )
		{
			FUIAction Action = FUIAction( FExecuteAction::CreateSP( this, &SCurveEditor::OnCreateExternalCurveClicked ) );
			MenuBuilder.AddMenuEntry
			( 
				LOCTEXT("CreateExternalCurve", "Create External Curve"),
				LOCTEXT("CreateExternalCurve_ToolTip", "Create an external asset using this internal curve"), 
				FSlateIcon(), 
				Action
			);
		}

		if( IsLinearColorCurve() && !bAlwaysDisplayColorCurves )
		{
			FUIAction ShowCurveAction( FExecuteAction::CreateSP( this, &SCurveEditor::OnShowCurveToggled ), FCanExecuteAction(), FIsActionChecked::CreateSP( this, &SCurveEditor::AreCurvesVisible ) );
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ShowCurves","Show Curves"),
				LOCTEXT("ShowCurves_ToolTip", "Toggles displaying the curves for linear colors"),
				FSlateIcon(),
				ShowCurveAction,
				NAME_None,
				EUserInterfaceActionType::ToggleButton 
			);
		}

		if( IsLinearColorCurve() && !bAlwaysHideGradientEditor)
		{
			FUIAction ShowGradientAction( FExecuteAction::CreateSP( this, &SCurveEditor::OnShowGradientToggled ), FCanExecuteAction(), FIsActionChecked::CreateSP( this, &SCurveEditor::IsGradientEditorVisible ) );
			MenuBuilder.AddMenuEntry
			(
				LOCTEXT("ShowGradient","Show Gradient"),
				LOCTEXT("ShowGradient_ToolTip", "Toggles displaying the gradient for linear colors"),
				FSlateIcon(),
				ShowGradientAction,
				NAME_None,
				EUserInterfaceActionType::ToggleButton 
			);
		}
	}

	MenuBuilder.EndSection();

	FWidgetPath WidgetPath = InMouseEvent.GetEventPath() != nullptr ? *InMouseEvent.GetEventPath() : FWidgetPath();
	FSlateApplication::Get().PushMenu(SharedThis(this), WidgetPath, MenuBuilder.MakeWidget(), FSlateApplication::Get().GetCursorPos(), FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
}

void SCurveEditor::OnCreateExternalCurveClicked()
{
	OnCreateAsset.ExecuteIfBound();
}

void SCurveEditor::OnShowCurveToggled()
{
	if (bAreCurvesVisible.IsBound() && SetAreCurvesVisibleHandler.IsBound())
	{
		SetAreCurvesVisibleHandler.Execute(!bAreCurvesVisible.Get());
	}
	else
	{
		bAreCurvesVisible = !bAreCurvesVisible.Get();
	}
}

UObject* SCurveEditor::CreateCurveObject( TSubclassOf<UCurveBase> CurveType, UObject* PackagePtr, FName& AssetName )
{
	UObject* NewObj = NULL;
	CurveFactory = Cast<UCurveFactory>(NewObject<UFactory>(GetTransientPackage(), UCurveFactory::StaticClass()));
	if(CurveFactory)
	{
		CurveFactory->CurveClass = CurveType;
		NewObj = CurveFactory->FactoryCreateNew( CurveFactory->GetSupportedClass(), PackagePtr, AssetName, RF_Public|RF_Standalone, NULL, GWarn );
	}
	CurveFactory = NULL;
	return NewObj;
}

bool SCurveEditor::IsEditingEnabled() const
{
	return bCanEditTrack;
}

void SCurveEditor::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObject( Settings );
	Collector.AddReferencedObject( CurveFactory );
}

TSharedPtr<FUICommandList> SCurveEditor::GetCommands()
{
	return Commands;
}

bool SCurveEditor::IsValidCurve( FRealCurve* Curve ) const
{
	bool bIsValid = false;
	if(Curve && CurveOwner)
	{
		for(auto CurveViewModel : CurveViewModels)
		{
			if(CurveViewModel->CurveInfo.CurveToEdit == Curve && CurveOwner->IsValidCurve(CurveViewModel->CurveInfo))
			{
				bIsValid = true;
				break;
			}
		}
	}
	return bIsValid;
}

void SCurveEditor::SetInputMinMax(float NewMin, float NewMax)
{
	if (SetInputViewRangeHandler.IsBound())
	{
		SetInputViewRangeHandler.Execute(NewMin, NewMax);
	}
	else
	{
		//if no delegate and view min input isn't using a delegate just set value directly
		if (ViewMinInput.IsBound() == false)
		{
			ViewMinInput.Set(NewMin);
		}

		if (ViewMaxInput.IsBound() == false)
		{
			ViewMaxInput.Set(NewMax);
		}
	}
}

void SCurveEditor::SetOutputMinMax(float NewMin, float NewMax)
{
	if (SetOutputViewRangeHandler.IsBound())
	{
		SetOutputViewRangeHandler.Execute(NewMin, NewMax);
	}
	else
	{
		//if no delegate and view min output isn't using a delegate just set value directly
		if (ViewMinOutput.IsBound() == false)
		{
			ViewMinOutput.Set(NewMin);
		}

		if (ViewMaxOutput.IsBound() == false)
		{
			ViewMaxOutput.Set(NewMax);
		}
	}
}

void SCurveEditor::EmptyAllCurveViewModels()
{
	CurveViewModels.Empty();
}

void SCurveEditor::ClearSelectedCurveViewModels()
{
	for(auto CurveViewModel : CurveViewModels)
	{
		CurveViewModel->bIsSelected = false;
	}
}

void SCurveEditor::SetSelectedCurveViewModel(FRealCurve* CurveToSelect)
{
	TSharedPtr<FCurveViewModel> ViewModel = GetViewModelForCurve(CurveToSelect);
	if (ViewModel.IsValid())
	{
		ViewModel.Get()->bIsSelected = true;
	}
}

bool SCurveEditor::AnyCurveViewModelsSelected() const
{
	for (auto CurveViewModel : CurveViewModels)
	{
		if (CurveViewModel->bIsSelected)
		{
			return true;
		}
	}
	return false;
}

TSharedPtr<FCurveViewModel> SCurveEditor::HitTestCurves(  const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent )
{
	if( AreCurvesVisible() )
	{
		FTrackScaleInfo ScaleInfo(ViewMinInput.Get(),  ViewMaxInput.Get(), ViewMinOutput.Get(), ViewMaxOutput.Get(), InMyGeometry.GetLocalSize());

		const FVector2D HitPosition = InMyGeometry.AbsoluteToLocal( InMouseEvent.GetScreenSpacePosition()  );

		TArray<FRealCurve*> CurvesHit;

		for(auto CurveViewModel : CurveViewModels)
		{

			FRealCurve* Curve = CurveViewModel->CurveInfo.CurveToEdit;

			if (IsValidCurve(Curve))
			{
				float Time		 = ScaleInfo.LocalXToInput(HitPosition.X);
				float KeyScreenY = ScaleInfo.OutputToLocalY(Curve->Eval(Time));

				if( HitPosition.Y > (KeyScreenY - (0.5f * CONST_CurveSize.Y)) &&
					HitPosition.Y < (KeyScreenY + (0.5f * CONST_CurveSize.Y)))
				{
					return CurveViewModel;
				}
			}
		}
	}

	return TSharedPtr<FCurveViewModel>();
}

bool SCurveEditor::IsCurveSelectable(TSharedPtr<FCurveViewModel> CurveViewModel) const
{		
	bool bAnyCurveViewModelsSelected = AnyCurveViewModelsSelected();
	bool bDisabled = bAnyCurveViewModelsSelected && !CurveViewModel->bIsSelected;

	return !CurveViewModel->bIsLocked && CurveViewModel->bIsVisible && !bDisabled;
}

SCurveEditor::FSelectedTangent SCurveEditor::HitTestCubicTangents( const FGeometry& InMyGeometry, const FVector2D& HitScreenPosition )
{
	FSelectedTangent Tangent;

	if( AreCurvesVisible() && CurveOwner && CurveOwner->HasRichCurves() )
	{
		FTrackScaleInfo ScaleInfo(ViewMinInput.Get(),  ViewMaxInput.Get(), ViewMinOutput.Get(), ViewMaxOutput.Get(), InMyGeometry.GetLocalSize());

		const FVector2D HitPosition = InMyGeometry.AbsoluteToLocal( HitScreenPosition);

		for (auto CurveViewModel : CurveViewModels)
		{
			if (IsCurveSelectable(CurveViewModel))
			{
				FRichCurve* Curve = (FRichCurve*)CurveViewModel->CurveInfo.CurveToEdit;
				if (Curve != NULL)
				{
					for (auto It(Curve->GetKeyHandleIterator()); It; ++It)
					{
						FKeyHandle KeyHandle = *It;
						FSelectedCurveKey SelectedCurveKey(Curve, KeyHandle);

						if(SelectedCurveKey.IsValid())
						{
							bool bIsTangentSelected = false;
							bool bIsArrivalSelected = false;
							bool bIsLeaveSelected = false;
							bool bIsTangentVisible = IsTangentVisible(Curve, KeyHandle, bIsTangentSelected, bIsArrivalSelected, bIsLeaveSelected);

							if (bIsTangentVisible)
							{
								float Time		 = ScaleInfo.LocalXToInput(HitPosition.X);
								float KeyScreenY = ScaleInfo.OutputToLocalY(Curve->Eval(Time));

								FVector2D Arrive, Leave;
								GetTangentPoints(ScaleInfo, SelectedCurveKey, Arrive, Leave);

								if( HitPosition.Y > (Arrive.Y - (0.5f * CONST_CurveSize.Y)) &&
									HitPosition.Y < (Arrive.Y + (0.5f * CONST_CurveSize.Y)) &&
									HitPosition.X > (Arrive.X - (0.5f * CONST_TangentSize.X)) && 
									HitPosition.X < (Arrive.X + (0.5f * CONST_TangentSize.X)))
								{
									Tangent.Key = SelectedCurveKey;
									Tangent.bIsArrival = true;
									break;
								}
								if( HitPosition.Y > (Leave.Y - (0.5f * CONST_CurveSize.Y)) &&
									HitPosition.Y < (Leave.Y  + (0.5f * CONST_CurveSize.Y)) &&
									HitPosition.X > (Leave.X - (0.5f * CONST_TangentSize.X)) && 
									HitPosition.X < (Leave.X + (0.5f * CONST_TangentSize.X)))
								{
									Tangent.Key = SelectedCurveKey;
									Tangent.bIsArrival = false;
									break;
								}
							}
						}
					}
				}
			}
		}
	}

	return Tangent;
}

void SCurveEditor::OnSelectInterpolationMode(ERichCurveInterpMode InterpMode, ERichCurveTangentMode TangentMode)
{
	const bool bHasRichCurves = CurveOwner->HasRichCurves();
	if((SelectedKeys.Num() > 0 || SelectedTangents.Num() > 0) && (InterpMode != RCIM_Cubic || bHasRichCurves))
	{
		const FScopedTransaction Transaction(LOCTEXT("CurveEditor_SetInterpolationMode", "Select Interpolation Mode"));
		CurveOwner->ModifyOwner();

		if (bHasRichCurves)
		{
			for (auto It = SelectedKeys.CreateIterator(); It; ++It)
			{
				FSelectedCurveKey& Key = *It;
				FRichCurve* RichCurve = (FRichCurve*)Key.Curve;
				check(IsValidCurve(RichCurve));
				RichCurve->SetKeyInterpMode(Key.KeyHandle, InterpMode);
				RichCurve->SetKeyTangentMode(Key.KeyHandle, TangentMode);
			}

			for (auto It = SelectedTangents.CreateIterator(); It; ++It)
			{
				FSelectedTangent& Tangent = *It;
				FRichCurve* RichCurve = (FRichCurve*)Tangent.Key.Curve;
				check(IsValidCurve(RichCurve));
				RichCurve->SetKeyInterpMode(Tangent.Key.KeyHandle, InterpMode);
				RichCurve->SetKeyTangentMode(Tangent.Key.KeyHandle, TangentMode);
			}
		}
		else
		{
			for (auto It = SelectedKeys.CreateIterator(); It; ++It)
			{
				FSelectedCurveKey& Key = *It;
				FSimpleCurve* SimpleCurve = (FSimpleCurve*)Key.Curve;
				check(IsValidCurve(SimpleCurve));
				SimpleCurve->SetKeyInterpMode(InterpMode);
			}
		}

		TArray<FRichCurveEditInfo> ChangedCurveEditInfos;
		CurveOwner->OnCurveChanged(ChangedCurveEditInfos);
	}
}

bool SCurveEditor::IsInterpolationModeSelected(ERichCurveInterpMode InterpMode, ERichCurveTangentMode TangentMode)
{
	const bool bHasRichCurves = CurveOwner->HasRichCurves();
	if (SelectedKeys.Num() > 0)
	{
		for (auto SelectedKey : SelectedKeys)
		{
			if (SelectedKey.Curve->GetKeyInterpMode(SelectedKey.KeyHandle) != InterpMode || (bHasRichCurves && ((FRichCurve*)SelectedKey.Curve)->GetKeyTangentMode(SelectedKey.KeyHandle) != TangentMode))
			{
				return false;
			}
		}
		return true;
	}
	else if (SelectedTangents.Num() > 0)
	{
		for (auto SelectedTangent : SelectedTangents)
		{
			if (SelectedTangent.Key.Curve->GetKeyInterpMode(SelectedTangent.Key.KeyHandle) != InterpMode || ((FRichCurve*)SelectedTangent.Key.Curve)->GetKeyTangentMode(SelectedTangent.Key.KeyHandle) != TangentMode)
			{
				return false;
			}
		}
		return true;
	}
	else
	{
		return false;
	}
}

bool SCurveEditor::HasRichCurves() const
{
	return CurveOwner->HasRichCurves();
}

void SCurveEditor::OnFlattenOrStraightenTangents(bool bFlattenTangents)
{
	if ((SelectedKeys.Num() > 0 || SelectedTangents.Num() > 0) && CurveOwner->HasRichCurves())
	{
		const FScopedTransaction Transaction(LOCTEXT("CurveEditor_FlattenTangents", "Flatten Tangents"));
		CurveOwner->ModifyOwner();

		auto OnFlattenOrStraightenTangents_Internal = [this, bFlattenTangents](FSelectedCurveKey& Key)
		{
			FRichCurve* RichCurve = (FRichCurve*)Key.Curve;
			check(IsValidCurve(RichCurve));

			float LeaveTangent = RichCurve->GetKey(Key.KeyHandle).LeaveTangent;
			float ArriveTangent = RichCurve->GetKey(Key.KeyHandle).ArriveTangent;

			if (bFlattenTangents)
			{
				LeaveTangent = 0;
				ArriveTangent = 0;
			}
			else
			{
				LeaveTangent = (LeaveTangent + ArriveTangent) * 0.5f;
				ArriveTangent = LeaveTangent;
			}

			RichCurve->GetKey(Key.KeyHandle).LeaveTangent = LeaveTangent;
			RichCurve->GetKey(Key.KeyHandle).ArriveTangent = ArriveTangent;
			if (RichCurve->GetKey(Key.KeyHandle).InterpMode == RCIM_Cubic &&
				RichCurve->GetKey(Key.KeyHandle).TangentMode == RCTM_Auto)
			{
				RichCurve->GetKey(Key.KeyHandle).TangentMode = RCTM_User;
			}
		};

		for(auto It = SelectedKeys.CreateIterator();It;++It)
		{
			OnFlattenOrStraightenTangents_Internal(*It);
		}
				
		for(auto It = SelectedTangents.CreateIterator();It;++It)
		{
			OnFlattenOrStraightenTangents_Internal(It->Key);
		}

		TArray<FRichCurveEditInfo> ChangedCurveEditInfos;
		CurveOwner->OnCurveChanged(ChangedCurveEditInfos);
	}
}

void SCurveEditor::OnBakeCurve()
{
	float BakeSampleRate = InputSnap.IsSet() ? InputSnap.Get() : 0.05f;

	// Display dialog and let user enter sample rate.
	GenericTextEntryModeless(
		NSLOCTEXT("CurveEditor.Popups", "BakeSampleRate", "Sample Rate"),
		FText::AsNumber( BakeSampleRate ),
		FOnTextCommitted::CreateSP(this, &SCurveEditor::OnBakeCurveSampleRateCommitted)
		);
}

void SCurveEditor::OnBakeCurveSampleRateCommitted(const FText& InText, ETextCommit::Type CommitInfo)
{
	CloseEntryPopupMenu();
	if (CommitInfo == ETextCommit::OnEnter)
	{
		double NewBakeSampleRate = FCString::Atod(*InText.ToString());
		const bool bIsNumber = InText.IsNumeric(); 
		if (!bIsNumber)
		{
			return;
		}
		if (NewBakeSampleRate <= 0.0)
		{
			UE_LOGF(LogCurveEditor, Error, "Invalid Bake Sample Rate");
			return;
		}

		const float BakeSampleRate = (float)NewBakeSampleRate;

		const FScopedTransaction Transaction(LOCTEXT("CurveEditor_BakeCurve", "Bake Curve"));
		CurveOwner->ModifyOwner();

		bool bAnyCurveViewModelsSelected = AnyCurveViewModelsSelected();

		TArray<FRichCurveEditInfo> ChangedCurveEditInfos;

		// If keys are selected, bake between them
		TMap<FRealCurve*, TInterval<float> > CurveRangeMap;
		for (auto SelectedKey : SelectedKeys)
		{
			float SelectedTime = SelectedKey.Curve->GetKeyTime(SelectedKey.KeyHandle);
			if (CurveRangeMap.Find(SelectedKey.Curve) != nullptr)
			{
				CurveRangeMap[SelectedKey.Curve].Include(SelectedTime);
			}
			else
			{
				CurveRangeMap.Add(SelectedKey.Curve, TInterval<float>(SelectedTime, SelectedTime));
			}
		}

		if (CurveRangeMap.Num())
		{
			for (auto CurveToBake : CurveRangeMap)
			{
				if (CurveToBake.Value.Min != CurveToBake.Value.Max)
				{
					CurveToBake.Key->BakeCurve(BakeSampleRate, CurveToBake.Value.Min, CurveToBake.Value.Max);
					ChangedCurveEditInfos.Add(GetViewModelForCurve(CurveToBake.Key)->CurveInfo);
				}
				else
				{
					UE_LOGF(LogCurveEditor, Warning, "Unable to bake single-point curve. Check if you don't have a single key selected before Baking.");
				}
			}
		}
		else
		{
			for (auto CurveViewModel : CurveViewModels)
			{
				if (!bAnyCurveViewModelsSelected || CurveViewModel->bIsSelected)
				{
					CurveViewModel->CurveInfo.CurveToEdit->BakeCurve(BakeSampleRate);
					ChangedCurveEditInfos.Add(CurveViewModel->CurveInfo);
				}
			}
		}

		if (ChangedCurveEditInfos.Num())
		{
			CurveOwner->OnCurveChanged(ChangedCurveEditInfos);
		}
	}
}

void SCurveEditor::OnReduceCurve()
{
	// Display dialog and let user enter tolerance.
	GenericTextEntryModeless(
		NSLOCTEXT("CurveEditor.Popups", "ReduceCurveTolerance", "Tolerance"),
		FText::AsNumber( ReduceTolerance ),
		FOnTextCommitted::CreateSP(this, &SCurveEditor::OnReduceCurveToleranceCommitted)
		);
}

void SCurveEditor::OnReduceCurveToleranceCommitted(const FText& InText, ETextCommit::Type CommitInfo)
{
	CloseEntryPopupMenu();
	if (CommitInfo == ETextCommit::OnEnter)
	{
		double NewTolerance = FCString::Atod(*InText.ToString());
		const bool bIsNumber = InText.IsNumeric(); 
		if(!bIsNumber)
			return;

		ReduceTolerance = (float)NewTolerance;

		const FScopedTransaction Transaction(LOCTEXT("CurveEditor_ReduceCurve", "Reduce Curve"));
		CurveOwner->ModifyOwner();

		bool bAnyCurveViewModelsSelected = AnyCurveViewModelsSelected();

		TArray<FRichCurveEditInfo> ChangedCurveEditInfos;

		// If keys are selected, bake between them
		TMap<FRealCurve*, TInterval<float> > CurveRangeMap;
		for (auto SelectedKey : SelectedKeys)
		{
			float SelectedTime = SelectedKey.Curve->GetKeyTime(SelectedKey.KeyHandle);
			if (CurveRangeMap.Find(SelectedKey.Curve) != nullptr)
			{
				CurveRangeMap[SelectedKey.Curve].Include(SelectedTime);
			}
			else
			{
				CurveRangeMap.Add(SelectedKey.Curve, TInterval<float>(SelectedTime, SelectedTime));
			}
		}

		if (CurveRangeMap.Num())
		{
			for (auto CurveToBake : CurveRangeMap)
			{
				CurveToBake.Key->RemoveRedundantKeys(ReduceTolerance, CurveToBake.Value.Min, CurveToBake.Value.Max);
				ChangedCurveEditInfos.Add(GetViewModelForCurve(CurveToBake.Key)->CurveInfo);
			}
		}
		else
		{
			for (auto CurveViewModel : CurveViewModels)
			{
				if (!bAnyCurveViewModelsSelected || CurveViewModel->bIsSelected)
				{
					CurveViewModel->CurveInfo.CurveToEdit->RemoveRedundantKeys(ReduceTolerance);
					ChangedCurveEditInfos.Add(CurveViewModel->CurveInfo);
				}
			}
		}

		if (ChangedCurveEditInfos.Num())
		{
			CurveOwner->OnCurveChanged(ChangedCurveEditInfos);
		}
	}
}


void SCurveEditor::OnSelectPreInfinityExtrap(ERichCurveExtrapolation Extrapolation)
{
	const FScopedTransaction Transaction(LOCTEXT("CurveEditor_SetPreInfinityExtrapolation", "Set Pre-Infinity Extrapolation"));
	CurveOwner->ModifyOwner();

	bool bAnyCurveViewModelsSelected = AnyCurveViewModelsSelected();

	TArray<FRichCurveEditInfo> ChangedCurveEditInfos;
	for (auto CurveViewModel : CurveViewModels)
	{
		if (!bAnyCurveViewModelsSelected || CurveViewModel->bIsSelected)
		{
			if (CurveViewModel->CurveInfo.CurveToEdit->PreInfinityExtrap != Extrapolation)
			{
				CurveViewModel->CurveInfo.CurveToEdit->PreInfinityExtrap = Extrapolation;
				ChangedCurveEditInfos.Add(CurveViewModel->CurveInfo);
			}
		}
	}

	if (ChangedCurveEditInfos.Num())
	{
		CurveOwner->OnCurveChanged(ChangedCurveEditInfos);
	}
}

bool SCurveEditor::IsPreInfinityExtrapSelected(ERichCurveExtrapolation Extrapolation)
{
	bool bAnyCurveViewModelsSelected = AnyCurveViewModelsSelected();

	for (auto CurveViewModel : CurveViewModels)
	{
		// If there are any curves selected, the setting must match all of the selected curves
		if (bAnyCurveViewModelsSelected)
		{
			if (CurveViewModel->bIsSelected)
			{
				if (CurveViewModel->CurveInfo.CurveToEdit->PreInfinityExtrap != Extrapolation)
				{
					return false;
				}
			}
		}
		else
		{
			if (CurveViewModel->CurveInfo.CurveToEdit->PreInfinityExtrap != Extrapolation)
			{
				return false;
			}
		}
	}

	return CurveViewModels.Num() > 0;
}

void SCurveEditor::OnSelectPostInfinityExtrap(ERichCurveExtrapolation Extrapolation)
{
	const FScopedTransaction Transaction(LOCTEXT("CurveEditor_SetPostInfinityExtrapolation", "Set Post-Infinity Extrapolation"));
	CurveOwner->ModifyOwner();

	bool bAnyCurveViewModelsSelected = AnyCurveViewModelsSelected();

	TArray<FRichCurveEditInfo> ChangedCurveEditInfos;
	for (auto CurveViewModel : CurveViewModels)
	{
		if (!bAnyCurveViewModelsSelected || CurveViewModel->bIsSelected)
		{
			if (CurveViewModel->CurveInfo.CurveToEdit->PostInfinityExtrap != Extrapolation)
			{
				CurveViewModel->CurveInfo.CurveToEdit->PostInfinityExtrap = Extrapolation;
				ChangedCurveEditInfos.Add(CurveViewModel->CurveInfo);
			}
		}
	}

	if (ChangedCurveEditInfos.Num())
	{
		CurveOwner->OnCurveChanged(ChangedCurveEditInfos);
	}
}

bool SCurveEditor::IsPostInfinityExtrapSelected(ERichCurveExtrapolation Extrapolation)
{
	bool bAnyCurveViewModelsSelected = AnyCurveViewModelsSelected();

	for (auto CurveViewModel : CurveViewModels)
	{
		// If there are any curves selected, the setting must match all of the selected curves
		if (bAnyCurveViewModelsSelected)
		{
			if (CurveViewModel->bIsSelected)
			{
				if (CurveViewModel->CurveInfo.CurveToEdit->PostInfinityExtrap != Extrapolation)
				{
					return false;
				}
			}
		}
		else
		{
			if (CurveViewModel->CurveInfo.CurveToEdit->PostInfinityExtrap != Extrapolation)
			{
				return false;
			}
		}
	}

	return CurveViewModels.Num() > 0;
}

void SCurveEditor::MoveTangents(FTrackScaleInfo& ScaleInfo, FVector2D Delta)
{
	TArray<FRichCurveEditInfo> ChangedCurveEditInfos;
	CurveOwner->ModifyOwnerChange();

	for (const FSelectedTangent& SelectedTangent : SelectedTangents)
	{
		FRichCurveKey& RichKey = ((FRichCurve*)SelectedTangent.Key.Curve)->GetKey(SelectedTangent.Key.KeyHandle);

		const FSelectedCurveKey &Key = SelectedTangent.Key;
		float PreDragArriveTangent = PreDragTangents[SelectedTangent.Key.KeyHandle][0];
		float PreDragLeaveTangent = PreDragTangents[SelectedTangent.Key.KeyHandle][1];

		// Get tangent points in screen space
		FVector2D ArriveTangentDir = CalcTangentDir( PreDragArriveTangent );
		FVector2D LeaveTangentDir = CalcTangentDir( PreDragLeaveTangent );

		FVector2D KeyPosition( Key.Curve->GetKeyTime(Key.KeyHandle),Key.Curve->GetKeyValue(Key.KeyHandle) );

		ArriveTangentDir.Y *= -1.0f;
		LeaveTangentDir.Y *= -1.0f;
		FVector2D ArrivePosition = -ArriveTangentDir + KeyPosition;

		FVector2D LeavePosition = LeaveTangentDir + KeyPosition;

		FVector2D Arrive = FVector2D(ScaleInfo.InputToLocalX(ArrivePosition.X), ScaleInfo.OutputToLocalY(ArrivePosition.Y));
		FVector2D Leave = FVector2D(ScaleInfo.InputToLocalX(LeavePosition.X), ScaleInfo.OutputToLocalY(LeavePosition.Y));

		FVector2D KeyScreenPosition = FVector2D(ScaleInfo.InputToLocalX(KeyPosition.X), ScaleInfo.OutputToLocalY(KeyPosition.Y));

		FVector2D ToArrive = Arrive - KeyScreenPosition;
		ToArrive.Normalize();

		Arrive = KeyScreenPosition + ToArrive*CONST_KeyTangentOffset;

		FVector2D ToLeave = Leave - KeyScreenPosition;
		ToLeave.Normalize();

		Leave = KeyScreenPosition + ToLeave*CONST_KeyTangentOffset;

		// New arrive and leave directions in screen space
		if (SelectedTangent.bIsArrival)
		{
			Arrive += Delta;
			Leave -= Delta;
		}
		else
		{
			Arrive -= Delta;
			Leave += Delta;
		}

		// Convert back to input/output space
		FVector2D NewArriveDir(ScaleInfo.LocalXToInput(Arrive.X), ScaleInfo.LocalYToOutput(Arrive.Y));
		FVector2D NewLeaveDir(ScaleInfo.LocalXToInput(Leave.X), ScaleInfo.LocalYToOutput(Leave.Y));

		// Compute tangents
		float NewArriveTangent = CalcTangent(-1.f*(NewArriveDir - KeyPosition));
		float NewLeaveTangent = CalcTangent(NewLeaveDir - KeyPosition);

		if(RichKey.TangentMode != RCTM_Break)
		{
			RichKey.ArriveTangent = NewArriveTangent;
			RichKey.LeaveTangent = NewLeaveTangent;
		
			RichKey.TangentMode = RCTM_User;
		}
		else
		{
			if(SelectedTangent.bIsArrival)
			{
				RichKey.ArriveTangent = NewArriveTangent;
			}
			else
			{
				RichKey.LeaveTangent = NewLeaveTangent;
			}
		}	

		RichKey.InterpMode = RCIM_Cubic;

		ChangedCurveEditInfos.Add(GetViewModelForCurve(SelectedTangent.Key.Curve)->CurveInfo);
	}

	if (ChangedCurveEditInfos.Num())
	{
		CurveOwner->OnCurveChanged(ChangedCurveEditInfos);
	}
}

void SCurveEditor::GetTangentPoints(  FTrackScaleInfo &ScaleInfo, const FSelectedCurveKey &Key, FVector2D& Arrive, FVector2D& Leave ) const
{
	FVector2D ArriveTangentDir = CalcTangentDir( ((FRichCurve*)Key.Curve)->GetKey(Key.KeyHandle).ArriveTangent);
	FVector2D LeaveTangentDir = CalcTangentDir( ((FRichCurve*)Key.Curve)->GetKey(Key.KeyHandle).LeaveTangent);

	FVector2D KeyPosition( Key.Curve->GetKeyTime(Key.KeyHandle),Key.Curve->GetKeyValue(Key.KeyHandle) );

	ArriveTangentDir.Y *= -1.0f;
	LeaveTangentDir.Y *= -1.0f;
	FVector2D ArrivePosition = -ArriveTangentDir + KeyPosition;

	FVector2D LeavePosition = LeaveTangentDir + KeyPosition;

	Arrive = FVector2D(ScaleInfo.InputToLocalX(ArrivePosition.X), ScaleInfo.OutputToLocalY(ArrivePosition.Y));
	Leave = FVector2D(ScaleInfo.InputToLocalX(LeavePosition.X), ScaleInfo.OutputToLocalY(LeavePosition.Y));

	FVector2D KeyScreenPosition = FVector2D(ScaleInfo.InputToLocalX(KeyPosition.X), ScaleInfo.OutputToLocalY(KeyPosition.Y));

	FVector2D ToArrive = Arrive - KeyScreenPosition;
	ToArrive.Normalize();

	Arrive = KeyScreenPosition + ToArrive*CONST_KeyTangentOffset;

	FVector2D ToLeave = Leave - KeyScreenPosition;
	ToLeave.Normalize();

	Leave = KeyScreenPosition + ToLeave*CONST_KeyTangentOffset;
}

TArray<SCurveEditor::FSelectedCurveKey> SCurveEditor::GetEditableKeysWithinMarquee(const FGeometry& InMyGeometry, FVector2D MarqueeTopLeft, FVector2D MarqueeBottomRight) const
{
	TArray<FSelectedCurveKey> KeysWithinMarquee;
	if (AreCurvesVisible())
	{
		FTrackScaleInfo ScaleInfo(ViewMinInput.Get(), ViewMaxInput.Get(), ViewMinOutput.Get(), ViewMaxOutput.Get(), InMyGeometry.GetLocalSize());
		for (auto CurveViewModel : CurveViewModels)
		{
			if (IsCurveSelectable(CurveViewModel))
			{
				FRealCurve* Curve = CurveViewModel->CurveInfo.CurveToEdit;
				if (Curve != NULL)
				{
					for (auto It(Curve->GetKeyHandleIterator()); It; ++It)
					{
						float KeyScreenX = ScaleInfo.InputToLocalX(Curve->GetKeyTime(*It));
						float KeyScreenY = ScaleInfo.OutputToLocalY(Curve->GetKeyValue(*It));

						if (KeyScreenX >= (MarqueeTopLeft.X - (0.5f * CONST_KeySize.X)) &&
							KeyScreenX <= (MarqueeBottomRight.X + (0.5f * CONST_KeySize.X)) &&
							KeyScreenY >= (MarqueeTopLeft.Y - (0.5f * CONST_KeySize.Y)) &&
							KeyScreenY <= (MarqueeBottomRight.Y + (0.5f * CONST_KeySize.Y)))
						{
							KeysWithinMarquee.Add(FSelectedCurveKey(Curve, *It));
						}
					}
				}
			}
		}
	}

	return KeysWithinMarquee;
}

TArray<SCurveEditor::FSelectedTangent> SCurveEditor::GetEditableTangentsWithinMarquee(const FGeometry& InMyGeometry, FVector2D MarqueeTopLeft, FVector2D MarqueeBottomRight) const
{
	FBox MarqueeBox;
	MarqueeBox.Min = FVector(MarqueeTopLeft.X, MarqueeTopLeft.Y, 0);
	MarqueeBox.Max = FVector(MarqueeBottomRight.X, MarqueeBottomRight.Y, 0);

	TArray<FSelectedTangent> TangentsWithinMarquee;
	if (AreCurvesVisible() && CurveOwner && CurveOwner->HasRichCurves())
	{
		FTrackScaleInfo ScaleInfo(ViewMinInput.Get(), ViewMaxInput.Get(), ViewMinOutput.Get(), ViewMaxOutput.Get(), InMyGeometry.GetLocalSize());
		for (auto CurveViewModel : CurveViewModels)
		{
			if (IsCurveSelectable(CurveViewModel))
			{
				FRichCurve* Curve = (FRichCurve*)CurveViewModel->CurveInfo.CurveToEdit;
				if (Curve != NULL)
				{
					for (auto It(Curve->GetKeyHandleIterator()); It; ++It)
					{
						FKeyHandle KeyHandle = *It;
						FSelectedCurveKey SelectedCurveKey(Curve, KeyHandle);

						if(SelectedCurveKey.IsValid())
						{
							bool bIsTangentSelected = false;
							bool bIsArrivalSelected = false;
							bool bIsLeaveSelected = false;
							bool bIsTangentVisible = IsTangentVisible(Curve, KeyHandle, bIsTangentSelected, bIsArrivalSelected, bIsLeaveSelected);

							if (bIsTangentVisible)
							{
								FVector2D Arrive, Leave;
								GetTangentPoints(ScaleInfo, SelectedCurveKey, Arrive, Leave);

								bool bArriveInside = MarqueeBox.IsInsideOrOn(FVector(Arrive.X, Arrive.Y, 0));
								bool bLeaveInside = MarqueeBox.IsInsideOrOn(FVector(Leave.X, Leave.Y, 0));

								if (bArriveInside || bLeaveInside)
								{
									FSelectedTangent SelectedTangent(SelectedCurveKey);
									SelectedTangent.bIsArrival = bArriveInside;
									TangentsWithinMarquee.Add(SelectedTangent);
								}	
							}
						}
					}
				}
			}
		}
	}
	return TangentsWithinMarquee;
}

void SCurveEditor::BeginDragTransaction()
{
	TransactionIndex = GEditor->BeginTransaction( LOCTEXT("CurveEditor_Drag", "Mouse Drag") );
	CurveOwner->ModifyOwner();
}

void SCurveEditor::EndDragTransaction()
{
	if ( TransactionIndex >= 0 )
	{
		TArray<FRichCurveEditInfo> ChangedCurveEditInfos;

		if (DragState == EDragState::DragKey || DragState == EDragState::FreeDrag)
		{
			for (auto SelectedKey : SelectedKeys)
			{
				ChangedCurveEditInfos.Add(GetViewModelForCurve(SelectedKey.Curve)->CurveInfo);
			}
		}
		else if (DragState == EDragState::DragTangent)
		{
			for (auto SelectedTangent : SelectedTangents)
			{
				ChangedCurveEditInfos.Add(GetViewModelForCurve(SelectedTangent.Key.Curve)->CurveInfo);
			}
		}

		if (ChangedCurveEditInfos.Num())
		{
			CurveOwner->OnCurveChanged(ChangedCurveEditInfos);
		}
		
		GEditor->EndTransaction();
		TransactionIndex = -1;
	}
}

bool SCurveEditor::FSelectedTangent::IsValid() const
{
	return Key.IsValid();
}

void SCurveEditor::UndoAction()
{
	GEditor->UndoTransaction();
}

void SCurveEditor::RedoAction()
{
	GEditor->RedoTransaction();
}

void SCurveEditor::RefreshDetailsView()
{
	check(!bIsPendingRebuilt);

	bIsPendingRebuilt = true;

	//To clean up dangling references
	EmptyAllSelection();
	EmptyAllCurveViewModels();

	// Refresh will still happen in the current frame, but it's important to make it asynchronous because its possible the user sets value through property handle multiple times inside an atomic action
	// Making it synchronous will invalidate property nodes for all the subsequent property handle set value operation.
	RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSPLambda(
		this,
		[this](double, float)
		{
			if(const TSharedPtr<IPropertyUtilities> PropertyUtilities = PropertyUtilitiesWeak.Pin())
			{
				//Rebuild the widget so it will reference to the new correct address of Curves, and show things correctly
				PropertyUtilities->ForceRefresh();
			}

			return EActiveTimerReturnType::Stop;
		}
	));

	ValidateSelection();
}

void SCurveEditor::OnObjectPropertyChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent)
{
	if (CurveOwner && CurveOwner->GetOwners().Contains(Object))
	{
		if (!bIsPendingRebuilt)
		{
			if (GIsTransacting)
			{
				RefreshDetailsView();
			}
		}
	}
}

void SCurveEditor::OnRootPropertyChanged(const FPropertyChangedEvent& PropertyChangedEvent)
{
	if (!bIsPendingRebuilt)
	{
		// CurveEditor will hold dangling references to CurveData and access them in some cases :
		// - Curve Data is inline allocated, or wrapped by InstancedStruct, in a container. And we do Array Remove/Clear op, or Add that causes resize
		// - Curve Data is wrapped by InstancedStruct in a Container, and we do undo/redo(will trigger emptying and refilling the container, causing address changed)
		// - Curve Data is wrapped by InstancedStruct and InstancedStruct is being reset/replaced, causing actual memory address changed
		// todo: should only rebuild when it's the InstanceStruct with ValueSet. But currently there is no way to detect if there is an InstanceStruct in the PropertyNode Hierarchy.
		if (PropertyChangedEvent.ChangeType
			& (EPropertyChangeType::ArrayAdd
			| EPropertyChangeType::ArrayRemove
			| EPropertyChangeType::ArrayClear
			| EPropertyChangeType::ValueSet
			| EPropertyChangeType::ResetToDefault))
		{
			RefreshDetailsView();
		}
	}

	ValidateSelection();
}

void SCurveEditor::HandlePackageReloaded(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent)
{
	if (InPackageReloadPhase == EPackageReloadPhase::OnPackageFixup && CurveOwner)
	{
		// Our curve owner may be an object that has been reloaded, so we need to check that and update the curve editor appropriately
		// We have to do this via the interface as the object addresses stored in the remap table will be offset from the interface pointer due to multiple inheritance
		FCurveOwnerInterface* NewCurveOwner = nullptr;
		if (CurveOwner->RepointCurveOwner(*InPackageReloadedEvent, NewCurveOwner))
		{
			SetCurveOwner(NewCurveOwner, bCanEditTrack);
		}
	}
}

void SCurveEditor::PostUndo(bool bSuccess)
{
	ValidateSelection();
}

bool SCurveEditor::IsLinearColorCurve() const 
{
	return CurveOwner && CurveOwner->IsLinearColorCurve();
}

FVector2D SCurveEditor::SnapLocation(FVector2D InLocation)
{
	if (bInputSnappingEnabled.Get())
	{
		const float InputSnapNow = InputSnap.Get();

		InLocation.X = InputSnapNow != 0 ? FMath::RoundToInt(InLocation.X / InputSnapNow) * InputSnapNow : InLocation.X;
	}

	if (bOutputSnappingEnabled.Get())
	{
		const float OutputSnapNow = OutputSnap.Get();

		InLocation.Y = OutputSnapNow != 0 ? FMath::RoundToInt(InLocation.Y / OutputSnapNow) * OutputSnapNow : InLocation.Y;
	}
	return InLocation;
}

TSharedPtr<FCurveViewModel> SCurveEditor::GetViewModelForCurve(FRealCurve* InCurve)
{
	for (auto CurveViewModel : CurveViewModels)
	{
		if (InCurve == CurveViewModel->CurveInfo.CurveToEdit)
		{
			return CurveViewModel;
		}
	}
	return TSharedPtr<FCurveViewModel>();
}

void SCurveEditor::GenericTextEntryModeless(const FText& DialogText, const FText& DefaultText, FOnTextCommitted OnTextComitted)
{
	TSharedRef<STextEntryPopup> TextEntryPopup = 
		SNew(STextEntryPopup)
		.Label(DialogText)
		.DefaultText(DefaultText)
		.OnTextCommitted(OnTextComitted)
		.ClearKeyboardFocusOnCommit(false)
		.SelectAllTextWhenFocused(true)
		.MaxWidth(1024.0f);

	EntryPopupMenu = FSlateApplication::Get().PushMenu(
		SharedThis(this),
		FWidgetPath(),
		TextEntryPopup,
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup)
	);
}


void SCurveEditor::CloseEntryPopupMenu()
{
	if (EntryPopupMenu.IsValid())
	{
		EntryPopupMenu.Pin()->Dismiss();
	}
}

	
int32 SCurveEditor::TimeToFrame(float InTime) const
{
	const float FrameRate = InputSnap.IsSet() ? 1.0f / InputSnap.Get() : 1.f;
	float Frame = InTime * FrameRate;
	return FMath::RoundToInt(Frame);
}


float SCurveEditor::FrameToTime(int32 InFrame) const
{
	const float FrameRate = InputSnap.IsSet() ? 1.0f / InputSnap.Get() : 1.f;
	return InFrame / FrameRate;
}

#undef LOCTEXT_NAMESPACE
