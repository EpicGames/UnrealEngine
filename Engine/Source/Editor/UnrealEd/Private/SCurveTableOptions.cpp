// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCurveTableOptions.h"

#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "SPrimaryButton.h"
#include "Serialization/Archive.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

DEFINE_LOG_CATEGORY(LogCurveTableOptions);

#define LOCTEXT_NAMESPACE "CurveTableOptions"

SCurveTableOptions::SCurveTableOptions()
{
	// empty	
}

void SCurveTableOptions::Construct(const FArguments& InArgs)
{
	OnCancelClicked = InArgs._OnCancelClicked;
	OnCreateClicked = InArgs._OnCreateClicked;

	// Populate the valid interpolation modes
	{
		CurveInterpModes.Add( MakeShareable( new ERichCurveInterpMode(ERichCurveInterpMode::RCIM_Constant) ) );
		CurveInterpModes.Add( MakeShareable( new ERichCurveInterpMode(ERichCurveInterpMode::RCIM_Linear) ) );
		CurveInterpModes.Add( MakeShareable( new ERichCurveInterpMode(ERichCurveInterpMode::RCIM_Cubic) ) );
	}

	SelectedInterpMode = ERichCurveInterpMode::RCIM_Linear;


	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		.Padding(16.f)
		[
			
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ChooseCurveInterpolationType", "Choose Curve Interpolation Type:"))
			]

			+SVerticalBox::Slot()
			.Padding(0.f, 8.f, 0.f, 0.f)
			.AutoHeight()
			[

				SNew(SComboBox<CurveInterpModePtr>)
				.OptionsSource( &CurveInterpModes )
				.OnGenerateWidget_Lambda( [this] (CurveInterpModePtr Item) { 
					return SNew(STextBlock).Text( this->GetCurveTypeText(*Item) );
				})
				.OnSelectionChanged_Lambda( [this] (CurveInterpModePtr Item, ESelectInfo::Type InSelectInfo)
				{
					if (Item.IsValid())
					{
						SelectedInterpMode = *Item;
					}
				})
				[
					SNew(STextBlock)
					.Text_Lambda([this] () { return GetCurveTypeText(SelectedInterpMode);})
				]
			]

			+SVerticalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Bottom)
			.Padding(0.f, 8.f, 0.f, 0.f)
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SPrimaryButton)
					.Text(LOCTEXT("CreateCurveTable", "Create Curve Table"))
					.OnClicked_Lambda( [this] () {
						if (OnCreateClicked.IsBound())
						{
							OnCreateClicked.Execute(SelectedInterpMode);
						}
						return FReply::Handled();		
					})
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(16.f, 0.f, 0.f, 0.f)
				[
					SNew(SButton)
					.Text(LOCTEXT("Cancel", "Cancel"))
					.HAlign(HAlign_Center)
					.OnClicked_Lambda( [this] () {
						if (OnCancelClicked.IsBound())
						{
							OnCancelClicked.Execute();
						}
						return FReply::Handled();		
					})
				]
			]
		]
	];
}

FText SCurveTableOptions::GetCurveTypeText(const ERichCurveInterpMode& Item) const
{
	FString EnumString;
	switch(Item)
	{
		case ERichCurveInterpMode::RCIM_Constant : 
		EnumString = TEXT("Constant");
		break;

		case ERichCurveInterpMode::RCIM_Linear : 
		EnumString = TEXT("Linear");
		break;

		case ERichCurveInterpMode::RCIM_Cubic : 
		EnumString = TEXT("Cubic");
		break;
	}

	return FText::FromString(EnumString);
}

#undef LOCTEXT_NAMESPACE 
