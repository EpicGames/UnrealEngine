// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Curves/RealCurve.h"
#include "Delegates/Delegate.h"
#include "Internationalization/Text.h"
#include "Logging/LogMacros.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"


DECLARE_DELEGATE_OneParam(FOnCreateCurveTable, ERichCurveInterpMode);

DECLARE_DELEGATE(FOnCancelCurveTable);

DECLARE_LOG_CATEGORY_EXTERN(LogCurveTableOptions, Log, All);

typedef TSharedPtr<ERichCurveInterpMode>		CurveInterpModePtr;
 
/**
 *    UI to allow the user to choose the Interpolation Type when creating a CurveTable.
 */

class SCurveTableOptions: public SCompoundWidget
{
	
public:
	SLATE_BEGIN_ARGS(SCurveTableOptions)
	{}
		SLATE_EVENT(FOnCreateCurveTable, OnCreateClicked)
		SLATE_EVENT(FOnCancelCurveTable, OnCancelClicked)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	SCurveTableOptions();

  protected:

	FText GetCurveTypeText(const ERichCurveInterpMode& InterMode) const;


  private:

	/** All available curve interpolation modes */
	TArray< CurveInterpModePtr >					CurveInterpModes;

	/** The selected curve interpolation type */
	ERichCurveInterpMode							SelectedInterpMode;

	FOnCreateCurveTable								OnCreateClicked;

	FOnCancelCurveTable                       		OnCancelClicked;

};
