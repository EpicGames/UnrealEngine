// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDeleteReferencedActorDialog.h"
#include "HAL/PlatformApplicationMisc.h"
#include "SlateOptMacros.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SDeleteReferencedActorDialog)

#define LOCTEXT_NAMESPACE "DeleteReferencedActorDialog"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SDeleteReferencedActorDialog::Construct(const FArguments& InArgs)
{
	ActorLabel = InArgs._ActorToDeleteLabel;
	ReferenceTypes = InArgs._ReferenceTypes;
	bShowApplyToAll = InArgs._ShowApplyToAll;
	ActorReferencers = InArgs._Referencers;

	constexpr uint32 Width  = 600;
	constexpr uint32 MaxReferencersListHeight = 200;

	CreateMessage();

	const TSharedRef<SScrollBar> HorizontalScrollBar = SNew(SScrollBar)
		.Orientation(Orient_Horizontal);

	const TSharedRef<SScrollBar> VerticalScrollBar = SNew(SScrollBar)
	.Orientation(Orient_Vertical);

	SCustomDialog::Construct(SCustomDialog::FArguments()
		.RootPadding(16.f)
		.HAlignContent(HAlign_Fill)
		.VAlignContent(VAlign_Fill)
		.IconDesiredSizeOverride(FVector2D(24.f, 24.f))
		.HAlignIcon(HAlign_Left)
		.VAlignIcon(VAlign_Top)
		.ContentAreaPadding(FMargin(8.f, 0.f, 0.f, 0.f))
		.ButtonAreaPadding(FMargin(0.f, 32.f, 0.f, 0.f))
		.Icon(FAppStyle::Get().GetBrush("Icons.WarningWithColor.Large"))
		.Title(LOCTEXT("ConfirmDeleteActorMessageTitle", "Delete Referenced Actor"))
		.Buttons({
			SCustomDialog::FButton(LOCTEXT("ConfirmDeleteActorMessage_ButtonDelete", "Delete")).SetPrimary(true)
			, SCustomDialog::FButton(LOCTEXT("ConfirmDeleteActorMessage_ButtonCancel", "Cancel")),
			})
		.Content()
		[
			SNew(SBox)
			.MinDesiredWidth(Width)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(FMargin(0.0f, 0.0f, 0.0f, 4.0f))
				.FillHeight(1.0f)
				[
					SNew(STextBlock)
					.Text(Message)
				]
				+ SVerticalBox::Slot()
				.Padding(FMargin(0.0f, 4.0f))
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.AutoHeight()
				[
					SNew(SBox)
					.MaxDesiredWidth(Width)
					.MaxDesiredHeight(MaxReferencersListHeight)
					.Visibility(this, &SDeleteReferencedActorDialog::GetReferencersListVisibility)
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							[
								SNew(SScrollBox)
								.ExternalScrollbar(HorizontalScrollBar)
								.Orientation(Orient_Horizontal)
								+ SScrollBox::Slot()
								.FillSize(1.0)
								[
									SNew(SScrollBox)
									.ExternalScrollbar(VerticalScrollBar)
									.Orientation(Orient_Vertical)
									+ SScrollBox::Slot()
									.HAlign(HAlign_Fill)
									.FillSize(1.0)
									[
										SNew(SListView<TSharedPtr<FText>>)
										.ListItemsSource(&ActorReferencers)
										.SelectionMode(ESelectionMode::None)
										.OnGenerateRow(this, &SDeleteReferencedActorDialog::OnGenerateRow)
									]
								]
							]
							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								VerticalScrollBar
							]
						]
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							HorizontalScrollBar
						]
					]
				]
				+ SVerticalBox::Slot()
				.Padding(FMargin(0.0f, 8.0f, 0.0f, 2.0f))
				.FillHeight(1.0f)
				.HAlign(HAlign_Fill)
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					[
						SNew(STextBlock)
						.Text(FText::FromString(TEXT("Delete it anyway?")))
					]
				]
			]
		]
		.BeforeButtons()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked(this, &SDeleteReferencedActorDialog::OnCopyMessageClicked)
				.ToolTipText(NSLOCTEXT("SChoiceDialog", "CopyMessageTooltip", "Copy the text in this message to the clipboard (CTRL+C)"))
				.ContentPadding(2.0f)
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Clipboard"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
            .HAlign(HAlign_Left)
            .VAlign(VAlign_Center)
            .Padding(FMargin(16.f, 0.f, 0.f, 0.f))
			[
				SNew(SCheckBox)
				.IsChecked(ECheckBoxState::Unchecked)
				.OnCheckStateChanged(this, &SDeleteReferencedActorDialog::OnApplyToAllCheckboxStateChanged)
				.Visibility(this, &SDeleteReferencedActorDialog::GetApplyToAllCheckboxVisibility)
				[
					SNew(STextBlock)
					.WrapTextAt(615.0f)
					.Text(LOCTEXT("ApplyToAllLabel", "Apply to All"))
				]
			]
		]
	);
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SDeleteReferencedActorDialog::CreateMessage()
{
	Message = FText();

	if (EnumHasAllFlags(ReferenceTypes, EDeletedActorReferenceTypes::All))
	{
		Message = FText::Format(LOCTEXT("ConfirmDeleteActorReferenceByScriptActorAndGroup", "Actor '{0}' is referenced by the Level Blueprint by the following other actors or assets and group:")
				, FText::FromString(ActorLabel));
	}
	else if (EnumHasAllFlags(ReferenceTypes, EDeletedActorReferenceTypes::LevelAndActorOrAsset))
	{
		Message = FText::Format(LOCTEXT("ConfirmDeleteActorReferenceByScriptAndActor", "Actor '{0}' is referenced by the Level Blueprint and by the following other actors or assets:")
				, FText::FromString(ActorLabel));
	}
	else if (EnumHasAllFlags(ReferenceTypes, EDeletedActorReferenceTypes::GroupAndActorOrAsset))
    	{
    		Message = FText::Format(LOCTEXT("ConfirmDeleteActorReferenceByGroupAndActor", "Actor '{0}' is referenced by the following other actors or assets and group:")
    				, FText::FromString(ActorLabel));
    	}
	else if (EnumHasAllFlags(ReferenceTypes, EDeletedActorReferenceTypes::LevelAndGroup))
	{
		Message = FText::Format(LOCTEXT("ConfirmDeleteActorReferenceByGroup", "Actor '{0}' is referenced by the Level Blueprint and by the following group:")
				, FText::FromString(ActorLabel));
	}
	else if (EnumHasAllFlags(ReferenceTypes, EDeletedActorReferenceTypes::ActorOrAsset))
	{
		Message = FText::Format(LOCTEXT("ConfirmDeleteActorReferenceByActor", "Actor '{0}' is referenced by the following other actors or assets:")
					, FText::FromString(ActorLabel));
	}
	else if (EnumHasAllFlags(ReferenceTypes, EDeletedActorReferenceTypes::Group))
	{
		Message = FText::Format(LOCTEXT("ConfirmDeleteActorReferencedByGroup", "Actor '{0}' is in the following group:")
				, FText::FromString(ActorLabel));
	}
	else if (EnumHasAllFlags(ReferenceTypes, EDeletedActorReferenceTypes::LevelBlueprint))
	{
		Message = FText::Format(LOCTEXT("ConfirmDeleteActorReferenceByScript", "Actor '{0}' is referenced by the Level Blueprint.")
				, FText::FromString(ActorLabel));
	}
}

void SDeleteReferencedActorDialog::OnApplyToAllCheckboxStateChanged(ECheckBoxState InCheckBoxState)
{
	bApplyToAll = InCheckBoxState == ECheckBoxState::Checked;
}

EVisibility SDeleteReferencedActorDialog::GetApplyToAllCheckboxVisibility() const
{
	return bShowApplyToAll ? EVisibility::Visible : EVisibility::Hidden; 
}

EVisibility SDeleteReferencedActorDialog::GetReferencersListVisibility() const
{
	return ReferenceTypes == EDeletedActorReferenceTypes::LevelBlueprint ? EVisibility::Collapsed : EVisibility::Visible;
}

FReply SDeleteReferencedActorDialog::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::C && InKeyEvent.IsControlDown())
	{
		CopyMessageToClipboard();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SDeleteReferencedActorDialog::CopyMessageToClipboard()
{
	FString ClipboardMessage = Message.ToString() + LINE_TERMINATOR;
	for (const TSharedPtr<FText>& Referencer : ActorReferencers)
	{
		if (Referencer)
		{
			ClipboardMessage += + TEXT("\t") + Referencer->ToString() + LINE_TERMINATOR;
		}
	}

	ClipboardMessage += LINE_TERMINATOR + LOCTEXT("ConfirmDeleteMessageDeleteAnyway", "Delete it anyway?").ToString();

	FPlatformApplicationMisc::ClipboardCopy(*ClipboardMessage);
}

FReply SDeleteReferencedActorDialog::OnCopyMessageClicked()
{
	CopyMessageToClipboard();
	return FReply::Handled();
}

TSharedRef<ITableRow> SDeleteReferencedActorDialog::OnGenerateRow(TSharedPtr<FText> InText, const TSharedRef<STableViewBase>& InTableView)
{
	return SNew(STableRow<TSharedPtr<FText>>, InTableView)
		.Padding(FMargin(4.0f))
		.Content()
		[
			SNew(SBox)
			.VAlign(VAlign_Center)
			.MaxDesiredHeight(16.0f)
			.Padding(1.0f)
			[
				SNew(STextBlock)
				.Justification(ETextJustify::Left)
				.Text(*InText.Get())
			]
		];
}

#undef LOCTEXT_NAMESPACE
