// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dialog/SMessageDialog.h"
#include "Widgets/SCompoundWidget.h"
#include "SDeleteReferencedActorDialog.generated.h"

UENUM()
enum class EDeletedActorReferenceTypes : uint8
{
	None = 0,
	ActorOrAsset = 1 << 0,
	Group = 1 << 1,
	LevelBlueprint = 1 << 2,

	All = ActorOrAsset | Group | LevelBlueprint,
	LevelAndActorOrAsset = ActorOrAsset | LevelBlueprint,
	GroupAndActorOrAsset = ActorOrAsset | Group,
	LevelAndGroup = Group | LevelBlueprint
};
ENUM_CLASS_FLAGS(EDeletedActorReferenceTypes)

class ITableRow;
class STableViewBase;

/**
 * A dialog to be shown when deleting scene actors which are referenced by one or multiple entities (groups, actors, level BP, etc)
 */
class SDeleteReferencedActorDialog : public SCustomDialog
{
public:
	SLATE_BEGIN_ARGS(SDeleteReferencedActorDialog)
		: _ShowApplyToAll(false)
		, _ReferenceTypes(EDeletedActorReferenceTypes::None)
		{
		}

		SLATE_ARGUMENT(bool, ShowApplyToAll)
		SLATE_ARGUMENT(FString, ActorToDeleteLabel)
		SLATE_ARGUMENT(EDeletedActorReferenceTypes, ReferenceTypes)
		SLATE_ARGUMENT(TArray<TSharedPtr<FText>>, Referencers)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	virtual FReply OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override;

	bool GetApplyToAll() const { return bApplyToAll; }

private:
	void CreateMessage();
	void OnApplyToAllCheckboxStateChanged(ECheckBoxState InCheckBoxState);
	EVisibility GetApplyToAllCheckboxVisibility() const;
	EVisibility GetReferencersListVisibility() const;
	void CopyMessageToClipboard();
	FReply OnCopyMessageClicked();
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FText> InText, const TSharedRef<STableViewBase>& InTableView);

	EDeletedActorReferenceTypes ReferenceTypes = EDeletedActorReferenceTypes::None;
	TArray<TSharedPtr<FText>> ActorReferencers;
	bool bShowApplyToAll = false;
	bool bApplyToAll = false;
	FString ActorLabel;
	FText Message;
};
