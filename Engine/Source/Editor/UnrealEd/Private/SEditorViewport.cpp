// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEditorViewport.h"

#include "EditorInteractiveGizmoManager.h"
#include "Misc/Paths.h"
#include "Framework/Commands/UICommandList.h"
#include "Misc/App.h"
#include "Widgets/Layout/SBorder.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "EngineGlobals.h"
#include "Engine/TextureStreamingTypes.h"
#include "EditorModeManager.h"
#include "Slate/SceneViewport.h"
#include "EditorViewportCommands.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Settings/EditorProjectSettings.h"
#include "Kismet2/DebuggerCommands.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "MaterialShaderQualitySettings.h"
#include "GPUSkinCacheVisualizationMenuCommands.h"
#include "GPUSkinCache.h"
#include "Widgets/Colors/SComplexGradient.h"
#include "Widgets/SBoxPanel.h"
#include "Modules/ModuleManager.h"
#include "IPreviewProfileController.h"
#include "ISettingsModule.h"
#include "ShowFlagMenuCommands.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#if WITH_DUMPGPU
	#include "RenderGraph.h"
#endif
#include "LevelEditorViewport.h"
#include "Viewports/SDepthBar.h"
#include "SEditorViewportGridPanel.h"
#include "ToolMenus.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"
#include "Widgets/Input/SButton.h"
#include "RenderUtils.h"
#include "TransformGizmoEditorSettings.h"

#define LOCTEXT_NAMESPACE "EditorViewport"

static TAutoConsoleVariable<float> CVarEditorViewportDepthBarMinSpace(
	TEXT("r.Editor.DepthBarMinSpace"),
	500.0f,
	TEXT("The minimum vertical space to show the depth bar. Defaults to 500."),
	ECVF_Default);

SEditorViewport::SEditorViewport()
	: LastTickTime(0)
	, bInvalidated(false)
{
}

SEditorViewport::~SEditorViewport()
{
	// Release our reference to the viewport client
	Client.Reset();

	check( SceneViewport.IsUnique() );

	if (const TStrongObjectPtr<UModeManagerInteractiveToolsContext> ModeManagerInteractiveToolsContext =
			ModeManagerInteractiveToolsContextWeak.Pin())
	{
		ModeManagerInteractiveToolsContext->OnBuildViewportInteractions().RemoveAll(this);
	}
}

void SEditorViewport::Construct( const FArguments& InArgs )
{
	CreatePreviewProfileController();

	// Create Viewport Widget

	// clang-format off
	SAssignNew(ViewportWidget, SViewport)
	.ShowEffectWhenDisabled(false)
	.EnableGammaCorrection(false) // Scene rendering handles this
	.AddMetaData(InArgs.MetaData.Num() > 0 ? InArgs.MetaData[0] : MakeShareable(new FTagMetaData(TEXT("LevelEditorViewport"))))
	.ViewportSize(InArgs._ViewportSize)
	[
		SAssignNew(ViewportOverlay, SOverlay)
	];
	// clang-format on

	Client = MakeEditorViewportClient();

	if (!Client->VisibilityDelegate.IsBound())
	{
		Client->VisibilityDelegate.BindSP(this, &SEditorViewport::IsVisible);
	}

	ensure(Client->Viewport == nullptr);
	SceneViewport = FSceneViewport::Create(Client, ViewportWidget);

	if ( Client->IsRealtime() )
	{
		ActiveTimerHandle = RegisterActiveTimer( 0.f, FWidgetActiveTimerDelegate::CreateSP( this, &SEditorViewport::EnsureTick ) );
	}

	CommandList = MakeShareable( new FUICommandList );
	// Ensure the commands are registered
	FEditorViewportCommands::Register();
	BindCommands();

	// clang-format off
	ViewportOverlay->AddSlot()
	[
		SNew(SBorder)
		.BorderImage(this, &SEditorViewport::OnGetViewportBorderBrush)
		.BorderBackgroundColor(this, &SEditorViewport::OnGetViewportBorderColorAndOpacity)
		.Visibility(this, &SEditorViewport::GetActiveBorderVisibility)
		.Padding(0.0f)
		.ShowEffectWhenDisabled(false)
	];
	// clang-format on

	// clang-format off
	ViewportOverlay->AddSlot()
	.VAlign(VAlign_Top)
	[
		SNew(SBox)
		.Visibility(this, &SEditorViewport::OnGetFocusedViewportIndicatorVisibility)
		.MaxDesiredHeight(1.0f)
		.MinDesiredHeight(1.0f)
		[
			CreateViewportIndicatorWidget(
				TAttribute<EVisibility>::CreateSP(this, &SEditorViewport::OnGetFocusedViewportIndicatorVisibility)
				).ToSharedRef()
		]
	];
	// clang-format on

	TSharedPtr<SVerticalBox> VerticalBox;
	// clang-format off
	ChildSlot
	[
		SAssignNew(VerticalBox, SVerticalBox)
	];
	// clang-format on

	// Set up viewport toolbars.
	{
		// Continue to support switching behavior through the deprecation period to aid licensees in their conversion process. 
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		TSharedPtr<SWidget> OldViewportToolbar = MakeViewportToolbar();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		TSharedPtr<SWidget> NewViewportToolbar = BuildViewportToolbar();
		
		// Allow programmatically-migrated viewports (e.g. SCommonEditorViewportToolbarBase) to register themselves on construction as supporting either position
		// This indirection is done to minimize API changes (e.g. using a specific toolbar interface). The goal is that clients do nothing and get upgrades.
		// Overrides of SCommonEditorViewportToolbarBase can construct the full base toolbar and then overwrite the child slot with their own content.
		// This check ensures that the automatic upgrade is only allowed if we end up with a widget containing the child that we expected to have
		// when the upgrade was requested.
		if (TSharedPtr<SWidget> ExpectedChild = AutoUpgradeWidgetChild.Pin())
		{
			if (!NewViewportToolbar && OldViewportToolbar)
			{
				// The auto-upgrade toolbar was created with the legacy function call.
				if (OldViewportToolbar->GetChildren()->Num() == 1 && OldViewportToolbar->GetChildren()->GetChildAt(0) == ExpectedChild)
				{
					bLegacyToolbarIsAutomaticallyUpgradable = true;
					NewViewportToolbar = OldViewportToolbar;
				}
			}
			else if (NewViewportToolbar && !OldViewportToolbar)
			{
				// The auto-upgrade toolbar was created with BuildViewportToolbar()
				// In this case, we still want to be able to swap back and forth.
				if (NewViewportToolbar->GetChildren()->Num() == 1 && NewViewportToolbar->GetChildren()->GetChildAt(0) == ExpectedChild)
				{
					bLegacyToolbarIsAutomaticallyUpgradable = true;
					OldViewportToolbar = NewViewportToolbar;
				}
			}			
		}

		if (OldViewportToolbar)
		{
			const bool bHasNewViewportToolbar = NewViewportToolbar.IsValid();

			// clang-format off
			ViewportOverlay->AddSlot()
				.VAlign(VAlign_Top)
				[
					SNew(SBox)
					.Visibility_Lambda(
						[bHasNewViewportToolbar]() -> EVisibility
						{
							// Always show the old viewport toolbar if there is no new one.
							if (!bHasNewViewportToolbar)
							{
								return EVisibility::Visible;
							}
							
							PRAGMA_DISABLE_DEPRECATION_WARNINGS
							return UE::UnrealEd::ShowOldViewportToolbars() ? EVisibility::Visible: EVisibility::Collapsed;
							PRAGMA_ENABLE_DEPRECATION_WARNINGS
						}
					)
					[
						OldViewportToolbar.ToSharedRef()
					]
				];
			// clang-format on
		}

		// If new toolbar is available, let's add it on top of viewport
		if (NewViewportToolbar)
		{
			const bool bHasOldViewportToolbar = OldViewportToolbar.IsValid();

			// clang-format off
			VerticalBox->AddSlot()
			.AutoHeight()
			[
				SNew(SBox)
				.Visibility_Lambda(
					[bHasOldViewportToolbar, bHasLegacyUpgradedToolbar = bLegacyToolbarIsAutomaticallyUpgradable]() -> EVisibility
					{
						// Always show the new viewport toolbar if there is no old one.
						if (!bHasOldViewportToolbar)
						{
							return EVisibility::Visible;
						}
						// In the case of the old toolbar being upgraded, don't show it twice
						PRAGMA_DISABLE_DEPRECATION_WARNINGS
						return (UE::UnrealEd::ShowNewViewportToolbars() && (!bHasLegacyUpgradedToolbar || !UE::UnrealEd::ShowOldViewportToolbars())) ? EVisibility::Visible: EVisibility::Collapsed;
						PRAGMA_ENABLE_DEPRECATION_WARNINGS
					}
				)
				[
					NewViewportToolbar.ToSharedRef()
				]
			];
			// clang-format on
		}
	}

	// clang-format off
	
	bool bIsMainViewport = false;
	float MainViewportX = 0;
	float MainViewportY = 0;
	TSharedPtr<FEditorViewportClient> EditorViewportClient = GetViewportClient();
	if (EditorViewportClient.IsValid())
	{
		if (EditorViewportClient->IsLevelEditorClient())
		{
			const FLevelEditorViewportClient* LevelClient = static_cast<FLevelEditorViewportClient*>(EditorViewportClient.Get());
			if (LevelClient->IsPerspective())
			{
				bIsMainViewport = true;
			}

			if (LevelClient->Viewport != nullptr)
			{
				MainViewportX = LevelClient->Viewport->GetSizeXY().X;
				MainViewportY = LevelClient->Viewport->GetSizeXY().Y;
			}
		}
	}

	// We wrap the viewports in the middle of a 3x3 grid in order to restrict aspect ratio for preview platforms that request it or for manual aspect ratio restrictions.
	VerticalBox->AddSlot()
		[
			SNew(SGlobalPlayWorldActions)
				[
					SAssignNew(ViewportAspectRatioManager, SEditorViewportGridPanel)
					.ViewportWidget(ViewportWidget)
				]
		];
	// clang-format on
	
	if (TSharedPtr<SWidget> DepthBarWidget = BuildViewportDepthBar())
	{
		ViewportOverlay->AddSlot()
		.HAlign(HAlign_Right)	
		.VAlign(VAlign_Fill)
		[
			SNew(SBox)
			.Padding(0.0f, 100.0f, 6.0f, 100.0f)
			.Visibility(this, &SEditorViewport::OnGetDepthBarVisibility)
			[
				DepthBarWidget.ToSharedRef()
			]
		];
	}
	
	PopulateViewportOverlays(ViewportOverlay.ToSharedRef());

	// Any code retrieving DPI scale before this point might have done that too soon, when the parent Window of the
	// Viewport Widget is not valid yet. This causes the Viewport Client to cache a default DPI scale instead of the
	// correct and expected value.
	// An example of this happening would be FEditorViewportClient::GetPreviewScreenPercentage() which is called when
	// creating Screen Percentage menus in Asset Editors (e.g. Level, Static Mesh, etc.)
	// The following RequestUpdateDPIScale() call marks the DPI value for refresh.
	// This ensures CachedDPIScale value can be properly retrieved at the right time, once the Widget is fully setup and its parent window can be accessed.
	GetViewportClient()->RequestUpdateDPIScale();

	if (UModeManagerInteractiveToolsContext* const ModeManagerInteractiveToolsContext =
			GetModeManagerInteractiveToolsContext())
	{
		// GetModeManagerInteractiveToolsContext is virtual, so we store the context as a weak ptr.
		// This allows us to access it in class destructor
		ModeManagerInteractiveToolsContextWeak = ModeManagerInteractiveToolsContext;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ModeManagerInteractiveToolsContext->OnBuildViewportInteractions().AddSP(
			this, &SEditorViewport::OnRegisterViewportInteractions
		);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

FReply SEditorViewport::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	FReply Reply = FReply::Unhandled();
	if( CommandList->ProcessCommandBindings( InKeyEvent ) )
	{
		Reply = FReply::Handled();
		Client->Invalidate();
	}

	return Reply;
		
}

bool SEditorViewport::SupportsKeyboardFocus() const 
{
	return true;
}

FReply SEditorViewport::OnFocusReceived( const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent )
{
	// forward focus to the viewport
	return FReply::Handled().SetUserFocus(ViewportWidget.ToSharedRef(), InFocusEvent.GetCause());
}

void SEditorViewport::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	LastTickTime = FPlatformTime::Seconds();
	
	if (!bShowDepthBar && Client->IsOrtho() &&
		(Client->GetOrthographicNearPlaneOverride().IsSet() || Client->GetOrthographicFarPlaneOverride().IsSet()))
	{
		bShowDepthBar = true;
	}
	
	bHasSpaceForDepthBar = AllottedGeometry.GetLocalSize().Y >= CVarEditorViewportDepthBarMinSpace.GetValueOnGameThread();
}

void SEditorViewport::AddOverlayWidget(TSharedRef<SWidget> OverlaidWidget, int32 ZOrder)
{
	if (ViewportOverlay.IsValid())
	{
		ViewportOverlay->AddSlot(ZOrder)
		[
			OverlaidWidget
		];
	}
}

void SEditorViewport::RemoveOverlayWidget(TSharedRef<SWidget> OverlaidWidget)
{
	if (ViewportOverlay.IsValid())
	{
		ViewportOverlay->RemoveSlot(OverlaidWidget);	
	}
}

void SEditorViewport::BindCommands()
{
	FUICommandList& CommandListRef = *CommandList;

	const FEditorViewportCommands& Commands = FEditorViewportCommands::Get();

	TSharedRef<FEditorViewportClient> ClientRef = Client.ToSharedRef();

	CommandListRef.MapAction( 
		Commands.ToggleRealTime,
		FExecuteAction::CreateSP( this, &SEditorViewport::OnToggleRealtime ),
		FCanExecuteAction::CreateSP(this, &SEditorViewport::CanToggleRealtime),
		FIsActionChecked::CreateSP(this, &SEditorViewport::IsRealtime));

	CommandListRef.MapAction( 
		Commands.ToggleStats,
		FExecuteAction::CreateSP( this, &SEditorViewport::OnToggleStats ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(ClientRef, &FEditorViewportClient::ShouldShowStats));

	CommandListRef.MapAction( 
		Commands.ToggleFPS,
		FExecuteAction::CreateSP(this, &SEditorViewport::ToggleStatCommand, FString("FPS")),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SEditorViewport::IsStatCommandVisible, FString("FPS")));

	CommandListRef.MapAction(
		Commands.IncrementPositionGridSize,
		FExecuteAction::CreateSP( this, &SEditorViewport::OnIncrementPositionGridSize )
		);

	CommandListRef.MapAction(
		Commands.DecrementPositionGridSize,
		FExecuteAction::CreateSP( this, &SEditorViewport::OnDecrementPositionGridSize )
		);

	CommandListRef.MapAction(
		Commands.IncrementRotationGridSize,
		FExecuteAction::CreateSP( this, &SEditorViewport::OnIncrementRotationGridSize )
		);

	CommandListRef.MapAction(
		Commands.DecrementRotationGridSize,
		FExecuteAction::CreateSP( this, &SEditorViewport::OnDecrementRotationGridSize )
		);

	CommandListRef.MapAction( 
		Commands.Perspective,
		FExecuteAction::CreateSP( ClientRef, &FEditorViewportClient::SetViewportType, LVT_Perspective ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(ClientRef, &FEditorViewportClient::IsActiveViewportType, LVT_Perspective));

	CommandListRef.MapAction( 
		Commands.Front,
		FExecuteAction::CreateSP( ClientRef, &FEditorViewportClient::SetViewportType, LVT_OrthoFront ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(ClientRef, &FEditorViewportClient::IsActiveViewportType, LVT_OrthoFront));

	CommandListRef.MapAction( 
		Commands.Left,
		FExecuteAction::CreateSP(ClientRef, &FEditorViewportClient::SetViewportType, LVT_OrthoLeft),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(ClientRef, &FEditorViewportClient::IsActiveViewportType, LVT_OrthoLeft));

	CommandListRef.MapAction( 
		Commands.Top,
		FExecuteAction::CreateSP( ClientRef, &FEditorViewportClient::SetViewportType, LVT_OrthoTop ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(ClientRef, &FEditorViewportClient::IsActiveViewportType, LVT_OrthoTop));

	CommandListRef.MapAction(
		Commands.Back,
		FExecuteAction::CreateSP(ClientRef, &FEditorViewportClient::SetViewportType, LVT_OrthoBack),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(ClientRef, &FEditorViewportClient::IsActiveViewportType, LVT_OrthoBack));

	CommandListRef.MapAction(
		Commands.Right,
		FExecuteAction::CreateSP(ClientRef, &FEditorViewportClient::SetViewportType, LVT_OrthoRight),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(ClientRef, &FEditorViewportClient::IsActiveViewportType, LVT_OrthoRight));

	CommandListRef.MapAction(
		Commands.Bottom,
		FExecuteAction::CreateSP(ClientRef, &FEditorViewportClient::SetViewportType, LVT_OrthoBottom),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(ClientRef, &FEditorViewportClient::IsActiveViewportType, LVT_OrthoBottom));

	CommandListRef.MapAction(
		Commands.Next,
		FExecuteAction::CreateSP(ClientRef, &FEditorViewportClient::RotateViewportType),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(ClientRef, &FEditorViewportClient::IsActiveViewportTypeInRotation));

	CommandListRef.MapAction(
		Commands.ScreenCapture,
		FExecuteAction::CreateSP( this, &SEditorViewport::OnScreenCapture ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SEditorViewport::DoesAllowScreenCapture)
		);

	CommandListRef.MapAction(
		Commands.ScreenCaptureForProjectThumbnail,
		FExecuteAction::CreateSP( this, &SEditorViewport::OnScreenCaptureForProjectThumbnail ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SEditorViewport::DoesAllowScreenCapture)
		);

	
	CommandListRef.MapAction(
		Commands.SelectMode,
		FExecuteAction::CreateSP(ClientRef, &FEditorViewportClient::SetWidgetMode, UE::Widget::WM_None),
		FCanExecuteAction::CreateSP(ClientRef, &FEditorViewportClient::CanSetWidgetMode, UE::Widget::WM_None),
		FIsActionChecked::CreateSP(this, &SEditorViewport::IsWidgetModeActive, UE::Widget::WM_None)
	);

	CommandListRef.MapAction(
		Commands.TranslateMode,
		FExecuteAction::CreateSP( ClientRef, &FEditorViewportClient::SetWidgetMode, UE::Widget::WM_Translate ),
		FCanExecuteAction::CreateSP( ClientRef, &FEditorViewportClient::CanSetWidgetMode, UE::Widget::WM_Translate ),
		FIsActionChecked::CreateSP( this, &SEditorViewport::IsWidgetModeActive, UE::Widget::WM_Translate ) 
		);

	CommandListRef.MapAction( 
		Commands.RotateMode,
		FExecuteAction::CreateSP( ClientRef, &FEditorViewportClient::SetWidgetMode, UE::Widget::WM_Rotate ),
		FCanExecuteAction::CreateSP( ClientRef, &FEditorViewportClient::CanSetWidgetMode, UE::Widget::WM_Rotate ),
		FIsActionChecked::CreateSP( this, &SEditorViewport::IsWidgetModeActive, UE::Widget::WM_Rotate )
		);
		

	CommandListRef.MapAction( 
		Commands.ScaleMode,
		FExecuteAction::CreateSP( ClientRef, &FEditorViewportClient::SetWidgetMode, UE::Widget::WM_Scale ),
		FCanExecuteAction::CreateSP( ClientRef, &FEditorViewportClient::CanSetWidgetMode, UE::Widget::WM_Scale ),
		FIsActionChecked::CreateSP( this, &SEditorViewport::IsWidgetModeActive, UE::Widget::WM_Scale )
		);

	CommandListRef.MapAction( 
		Commands.TranslateRotateMode,
		FExecuteAction::CreateSP( ClientRef, &FEditorViewportClient::SetWidgetMode, UE::Widget::WM_TranslateRotateZ ),
		FCanExecuteAction::CreateSP( ClientRef, &FEditorViewportClient::CanSetWidgetMode, UE::Widget::WM_TranslateRotateZ ),
		FIsActionChecked::CreateSP( this, &SEditorViewport::IsWidgetModeActive, UE::Widget::WM_TranslateRotateZ ),
		FIsActionButtonVisible::CreateSP( this, &SEditorViewport::IsTranslateRotateModeVisible )
		);

	CommandListRef.MapAction(
		Commands.TranslateRotate2DMode,
		FExecuteAction::CreateSP(ClientRef, &FEditorViewportClient::SetWidgetMode, UE::Widget::WM_2D),
		FCanExecuteAction::CreateSP(ClientRef, &FEditorViewportClient::CanSetWidgetMode, UE::Widget::WM_2D),
		FIsActionChecked::CreateSP(this, &SEditorViewport::IsWidgetModeActive, UE::Widget::WM_2D),
		FIsActionButtonVisible::CreateSP(this, &SEditorViewport::Is2DModeVisible)
		);

	CommandListRef.MapAction(
		Commands.ShrinkTransformWidget,
		FExecuteAction::CreateSP( ClientRef, &FEditorViewportClient::AdjustTransformWidgetSize, -1 )
		);

	CommandListRef.MapAction( 
		Commands.ExpandTransformWidget,
		FExecuteAction::CreateSP( ClientRef, &FEditorViewportClient::AdjustTransformWidgetSize, 1 )
		);

	CommandListRef.MapAction( 
		Commands.RelativeCoordinateSystem_World,
		FExecuteAction::CreateSP( ClientRef, &FEditorViewportClient::SetWidgetCoordSystemSpace, COORD_World ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SEditorViewport::IsCoordSystemActive, COORD_World )
		);

	CommandListRef.MapAction( 
		Commands.RelativeCoordinateSystem_Local,
		FExecuteAction::CreateSP( ClientRef, &FEditorViewportClient::SetWidgetCoordSystemSpace, COORD_Local ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SEditorViewport::IsCoordSystemActive, COORD_Local )
		);

	CommandListRef.MapAction(
		Commands.RelativeCoordinateSystem_Parent,
		FExecuteAction::CreateSP(ClientRef, &FEditorViewportClient::SetWidgetCoordSystemSpace, COORD_Parent),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SEditorViewport::IsCoordSystemActive, COORD_Parent)
	);

	CommandListRef.MapAction(
		Commands.RelativeCoordinateSystem_Explicit,
		FExecuteAction::CreateSP(ClientRef, &FEditorViewportClient::SetWidgetCoordSystemSpace, COORD_Explicit),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SEditorViewport::IsCoordSystemActive, COORD_Explicit)
	);
	
	CommandListRef.MapAction(
		Commands.CycleTransformGizmos,
		FExecuteAction::CreateSP( this, &SEditorViewport::OnCycleWidgetMode ),
		FCanExecuteAction::CreateSP( ClientRef, &FEditorViewportClient::CanCycleWidgetMode )
		);

	CommandListRef.MapAction(
		Commands.CycleTransformGizmoCoordSystem,
		FExecuteAction::CreateSP( this, &SEditorViewport::OnCycleCoordinateSystem )
		);
		
	CommandListRef.MapAction(
		Commands.IncreaseGizmoSize,
		FExecuteAction::CreateSP( this, &SEditorViewport::OnIncreaseGizmoSize ),
		EUIActionRepeatMode::RepeatEnabled
		);
		
	CommandListRef.MapAction(
		Commands.DecreaseGizmoSize,
		FExecuteAction::CreateSP( this, &SEditorViewport::OnDecreaseGizmoSize ),
		EUIActionRepeatMode::RepeatEnabled
		);
		
	CommandListRef.MapAction(
		Commands.ResetGizmoSize,
		FExecuteAction::CreateSP( this, &SEditorViewport::OnResetGizmoSize )
		);

	CommandListRef.MapAction( 
		Commands.FocusViewportToSelection, 
		FExecuteAction::CreateSP( this, &SEditorViewport::OnFocusViewportToSelection )
		//FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("CAMERA ALIGN ACTIVEVIEWPORTONLY") ) )
		);
		
	FUIAction FocusToAndClipAction = FExecuteAction::CreateSP( this, &SEditorViewport::OnFocusAndClipViewportToSelection );
	FocusToAndClipAction.IsActionVisibleDelegate.BindSP( this, &SEditorViewport::CanUseDepthBar );
		
	CommandListRef.MapAction(Commands.FocusAndClipViewportToSelection, FocusToAndClipAction);
		
	CommandListRef.MapAction(
		Commands.UseOrthographicClippingPlanes,
		FExecuteAction::CreateSP( this, &SEditorViewport::OnToggleUseDepthBar ),
		FCanExecuteAction::CreateSP( this, &SEditorViewport::CanUseDepthBar ),
		FIsActionChecked::CreateSP( this, &SEditorViewport::IsDepthBarActive )
		);
		
	CommandListRef.MapAction(
		Commands.DismissOrthographicClippingPlanes,
		FExecuteAction::CreateSP( this, &SEditorViewport::DismissDepthBar )
		);
	
	CommandListRef.MapAction(
		Commands.SurfaceSnapping,
		FExecuteAction::CreateStatic( &SEditorViewport::OnToggleSurfaceSnap ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &SEditorViewport::OnIsSurfaceSnapEnabled ) 
		);

	CommandListRef.MapAction(
		Commands.RotateToSurfaceNormal,
		FExecuteAction::CreateStatic( &SEditorViewport::OnToggleRotateToSurfaceNormal ),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic( &SEditorViewport::IsRotateToSurfaceNormalEnabled ) 
		);

	CommandListRef.MapAction(
		(Client.IsValid() && Client->IsLevelEditorClient()) ? Commands.ToggleInGameExposure : Commands.ToggleAutoExposure,
		FExecuteAction::CreateSP( this, &SEditorViewport::ChangeExposureSetting),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SEditorViewport::IsExposureSettingSelected ) );

	CommandListRef.MapAction(
		Commands.ToggleInViewportContextMenu,
		FExecuteAction::CreateSP(this, &SEditorViewport::ToggleInViewportContextMenu),
		FCanExecuteAction::CreateSP(this, &SEditorViewport::CanToggleInViewportContextMenu)
	);

	CommandListRef.MapAction(
		Commands.ToggleOverrideViewportScreenPercentage,
		FExecuteAction::CreateSP(this, &SEditorViewport::TogglePreviewingScreenPercentage),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SEditorViewport::IsPreviewingScreenPercentage));

	CommandListRef.MapAction(
		Commands.OpenEditorPerformanceProjectSettings,
		FExecuteAction::CreateSP(this, &SEditorViewport::OnOpenViewportPerformanceProjectSettings));

	CommandListRef.MapAction(
		Commands.OpenEditorPerformanceEditorPreferences,
		FExecuteAction::CreateSP(this, &SEditorViewport::OnOpenViewportPerformanceEditorPreferences));

	CommandListRef.MapAction(
		Commands.ToggleDistanceBasedCameraSpeed,
		FExecuteAction::CreateStatic(&SEditorViewport::OnToggleDistanceBasedCameraSpeed),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic(&SEditorViewport::IsDistanceBasedCameraSpeedEnabled));

	// Simple macro for binding many view mode UI commands

#define MAP_VIEWMODEPARAM_ACTION( ViewModeCommand, ViewModeParam ) \
	CommandListRef.MapAction( \
		ViewModeCommand, \
		FExecuteAction::CreateSP( ClientRef, &FEditorViewportClient::SetViewModeParam, ViewModeParam ), \
		FCanExecuteAction(), \
		FIsActionChecked::CreateSP( ClientRef, &FEditorViewportClient::IsViewModeParam, ViewModeParam ) ) 

#define MAP_VIEWMODE_ACTION( ViewModeCommand, ViewModeID ) \
	CommandListRef.MapAction( \
		ViewModeCommand, \
		FExecuteAction::CreateSP( ClientRef, &FEditorViewportClient::SetViewMode, ViewModeID ), \
		FCanExecuteAction(), \
		FIsActionChecked::CreateSP( ClientRef, &FEditorViewportClient::IsViewModeEnabled, ViewModeID ) ) 

	// Map each view mode
	MAP_VIEWMODE_ACTION( Commands.WireframeMode, VMI_BrushWireframe );
	MAP_VIEWMODE_ACTION( Commands.UnlitMode, VMI_Unlit );
	MAP_VIEWMODE_ACTION( Commands.LitMode, VMI_Lit );
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MAP_VIEWMODE_ACTION( Commands.LitWireframeMode, VMI_Lit_Wireframe);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (IsRayTracingAllowed())
	{
		MAP_VIEWMODE_ACTION(Commands.PathTracingMode, VMI_PathTracing);
		MAP_VIEWMODE_ACTION(Commands.RayTracingDebugMode, VMI_RayTracingDebug);
	}

	MAP_VIEWMODE_ACTION( Commands.DetailLightingMode, VMI_Lit_DetailLighting );
	MAP_VIEWMODE_ACTION( Commands.LightingOnlyMode, VMI_LightingOnly );
	MAP_VIEWMODE_ACTION( Commands.LightComplexityMode, VMI_LightComplexity );
	MAP_VIEWMODE_ACTION( Commands.ShaderComplexityMode, VMI_ShaderComplexity );
	MAP_VIEWMODE_ACTION( Commands.QuadOverdrawMode, VMI_QuadOverdraw);
	MAP_VIEWMODE_ACTION( Commands.ShaderComplexityWithQuadOverdrawMode, VMI_ShaderComplexityWithQuadOverdraw );
	MAP_VIEWMODE_ACTION( Commands.TexStreamAccPrimitiveDistanceMode, VMI_PrimitiveDistanceAccuracy );
	MAP_VIEWMODE_ACTION( Commands.TexStreamAccMeshUVDensityMode, VMI_MeshUVDensityAccuracy);
	MAP_VIEWMODE_ACTION( Commands.TexStreamAccMaterialTextureScaleMode, VMI_MaterialTextureScaleAccuracy );
	MAP_VIEWMODE_ACTION( Commands.RequiredTextureResolutionMode, VMI_RequiredTextureResolution);
	MAP_VIEWMODE_ACTION( Commands.StationaryLightOverlapMode, VMI_StationaryLightOverlap );

	if (IsStaticLightingAllowed())
	{
		MAP_VIEWMODE_ACTION(Commands.LightmapDensityMode, VMI_LightmapDensity);
	}

	MAP_VIEWMODE_ACTION( Commands.ReflectionOverrideMode, VMI_ReflectionOverride );
	MAP_VIEWMODE_ACTION( Commands.GroupLODColorationMode, VMI_GroupLODColoration);
	MAP_VIEWMODE_ACTION( Commands.LODColorationMode, VMI_LODColoration );
	MAP_VIEWMODE_ACTION( Commands.HLODColorationMode, VMI_HLODColoration);
	MAP_VIEWMODE_ACTION( Commands.GroupStreamingDeficitMode, VMI_StreamingTextureDeficit);
	MAP_VIEWMODE_ACTION( Commands.StreamingTextureDeficitMode, VMI_StreamingTextureDeficit);
	MAP_VIEWMODE_ACTION( Commands.StreamingTextureResidencyMode, VMI_StreamingTextureResidency);
	MAP_VIEWMODE_ACTION( Commands.StreamingMeshLODDeficitMode, VMI_StreamingMeshLODDeficit);
	MAP_VIEWMODE_ACTION( Commands.StreamingMeshLODResidencyMode, VMI_StreamingMeshLODResidency);
	MAP_VIEWMODE_ACTION( Commands.VisualizeBufferMode, VMI_VisualizeBuffer );
	MAP_VIEWMODE_ACTION( Commands.VisualizeNaniteMode, VMI_VisualizeNanite );
	MAP_VIEWMODE_ACTION( Commands.VisualizeLumenMode, VMI_VisualizeLumen );
	MAP_VIEWMODE_ACTION( Commands.VisualizeMegaLightsMode, VMI_VisualizeMegaLights );
	MAP_VIEWMODE_ACTION( Commands.VisualizeSubstrateMode, VMI_VisualizeSubstrate);
	MAP_VIEWMODE_ACTION( Commands.VisualizeGroomMode, VMI_VisualizeGroom);
	MAP_VIEWMODE_ACTION( Commands.VisualizeVirtualShadowMapMode, VMI_VisualizeVirtualShadowMap );
	MAP_VIEWMODE_ACTION( Commands.VisualizeVirtualTextureMode, VMI_VisualizeVirtualTexture);
	MAP_VIEWMODE_ACTION( Commands.CollisionPawn, VMI_CollisionPawn);
	MAP_VIEWMODE_ACTION( Commands.CollisionVisibility, VMI_CollisionVisibility);

	MAP_VIEWMODE_ACTION( Commands.VisualizeLWCComplexity, VMI_LWCComplexity);

	if (FGPUSkinCache::IsGPUSkinCacheEnabled(GMaxRHIFeatureLevel))
	{
		MAP_VIEWMODE_ACTION(Commands.VisualizeGPUSkinCacheMode, VMI_VisualizeGPUSkinCache);
		FGPUSkinCacheVisualizationMenuCommands::Get().BindCommands(CommandListRef, Client);
	}

	MAP_VIEWMODEPARAM_ACTION( Commands.TexStreamAccMeshUVDensityAll, -1 );
	for (int32 TexCoordIndex = 0; TexCoordIndex < TEXSTREAM_MAX_NUM_UVCHANNELS; ++TexCoordIndex)
	{
		MAP_VIEWMODEPARAM_ACTION( Commands.TexStreamAccMeshUVDensitySingle[TexCoordIndex], TexCoordIndex );
	}

	MAP_VIEWMODEPARAM_ACTION( Commands.TexStreamAccMaterialTextureScaleAll, -1 );

	CommandListRef.MapAction( Commands.GeometryInspectionClayMode, 
		FExecuteAction::CreateSP( ClientRef, &FEditorViewportClient::SetGeometryInspectionViewMode, VMI_Clay ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( ClientRef, &FEditorViewportClient::IsViewModeEnabled, VMI_Clay ));
	CommandListRef.MapAction( Commands.GeometryInspectionFrontBackFaceMode, 
		FExecuteAction::CreateSP( ClientRef, &FEditorViewportClient::SetGeometryInspectionViewMode, VMI_FrontBackFace ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( ClientRef, &FEditorViewportClient::IsViewModeEnabled, VMI_FrontBackFace ));
	CommandListRef.MapAction( Commands.GeometryInspectionRandomColorMode, 
		FExecuteAction::CreateSP( ClientRef, &FEditorViewportClient::SetGeometryInspectionViewMode, VMI_RandomColor ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( ClientRef, &FEditorViewportClient::IsViewModeEnabled, VMI_RandomColor ));
	CommandListRef.MapAction( Commands.GeometryInspectionZebraHorizontalMode,
		FExecuteAction::CreateSP( ClientRef, &FEditorViewportClient::SetZebraViewMode, true ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( ClientRef, &FEditorViewportClient::IsZebraViewModeEnabled, true ) );
	CommandListRef.MapAction( Commands.GeometryInspectionZebraVerticalMode,
		FExecuteAction::CreateSP( ClientRef, &FEditorViewportClient::SetZebraViewMode, false ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( ClientRef, &FEditorViewportClient::IsZebraViewModeEnabled, false ) );
	
	CommandListRef.MapAction(
		Commands.GeometryInspectionShowWireframe,
		FExecuteAction::CreateSP( ClientRef, &FEditorViewportClient::ToggleGeometryInspectionShowWireframe ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( ClientRef, &FEditorViewportClient::IsSetGeometryInspectionShowWireframe )
	);

	CommandListRef.MapAction(
		Commands.AllChannelsMask,
		FExecuteAction::CreateSP( ClientRef, &FEditorViewportClient::SetChannelMask, EColorChannelMask::All ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( ClientRef, &FEditorViewportClient::IsChannelBeingMasked, EColorChannelMask::All )
	);

	CommandListRef.MapAction(
		Commands.RedChannelMask,
		FExecuteAction::CreateSP( ClientRef, &FEditorViewportClient::SetChannelMask, EColorChannelMask::Red ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( ClientRef, &FEditorViewportClient::IsChannelBeingMasked, EColorChannelMask::Red )
	);

	CommandListRef.MapAction(
		Commands.GreenChannelMask,
		FExecuteAction::CreateSP( ClientRef, &FEditorViewportClient::SetChannelMask, EColorChannelMask::Green ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( ClientRef, &FEditorViewportClient::IsChannelBeingMasked, EColorChannelMask::Green )
	);

	CommandListRef.MapAction(
		Commands.BlueChannelMask,
		FExecuteAction::CreateSP( ClientRef, &FEditorViewportClient::SetChannelMask, EColorChannelMask::Blue ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( ClientRef, &FEditorViewportClient::IsChannelBeingMasked, EColorChannelMask::Blue )
	);

	CommandListRef.MapAction(
		Commands.AlphaChannelMask,
		FExecuteAction::CreateSP( ClientRef, &FEditorViewportClient::SetChannelMask, EColorChannelMask::Alpha ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( ClientRef, &FEditorViewportClient::IsChannelBeingMasked, EColorChannelMask::Alpha )
	);

	BindShowCommands( CommandListRef );
}

void SEditorViewport::BindShowCommands( FUICommandList& OutCommandList )
{
	FShowFlagMenuCommands::Get().BindCommands(OutCommandList, Client);
}

EVisibility SEditorViewport::OnGetViewportContentVisibility() const
{
	FEditorModeTools* EditorModeTools = Client->GetModeTools();
	const bool bIsViewportUIHidden = EditorModeTools && EditorModeTools->IsViewportUIHidden();

	return bIsViewportUIHidden ? EVisibility::Collapsed : EVisibility::SelfHitTestInvisible;
}

void SEditorViewport::OnFocusAndClipViewportToSelection()
{
	const bool bPrevious = Client->GetClipToSelectionOnFocus(); 
	Client->SetClipToSelectionOnFocus(true);
	OnFocusViewportToSelection();
	Client->SetClipToSelectionOnFocus(bPrevious);
}

EVisibility SEditorViewport::OnGetDepthBarVisibility() const
{
	if (bHasSpaceForDepthBar && Client->IsOrtho() && bShowDepthBar)
	{
		return EVisibility::Visible;
	}
	
	return EVisibility::Collapsed;
}

SDepthBar::EMode SEditorViewport::GetDepthBarMode() const
{
	return Client->IsOrtho() ? SDepthBar::EMode::Orthographic : SDepthBar::EMode::Perspective;
}

TOptional<SDepthBar::FDepthSpace> SEditorViewport::GetDepthBarSpace() const
{
	FBox WorldBounds = Client->GetWorldBounds();
	if (!WorldBounds.IsValid)
	{
		return TOptional<SDepthBar::FDepthSpace>();
	}
	
	return SDepthBar::FDepthSpace(Client->GetViewLocation(), Client->GetForwardVector(), WorldBounds);
}

TOptional<double> SEditorViewport::GetDepthBarNearPlane() const
{
	if (Client->IsOrtho())
	{
		return Client->GetOrthographicNearPlaneOverride();
	}
	return TOptional<double>();
}

TOptional<double> SEditorViewport::GetDepthBarFarPlane() const
{
	if (Client->IsOrtho())
    {
    	return Client->GetOrthographicFarPlaneOverride();
    }
    return TOptional<double>();
}

void SEditorViewport::OnDepthBarNearPlaneChanged(const TOptional<double>& InNearPlane)
{
	if (Client->IsOrtho())
	{
		Client->SetOrthographicNearPlaneOverride(InNearPlane);
	}
	
	Client->Invalidate();
}

void SEditorViewport::OnDepthBarFarPlaneChanged(const TOptional<double>& InFarPlane)
{
	if (Client->IsOrtho())
	{
		Client->SetOrthographicFarPlaneOverride(InFarPlane);
	}
	
	Client->Invalidate();
}

void SEditorViewport::DismissDepthBar()
{
	Client->SetOrthographicFarPlaneOverride(TOptional<double>());
	Client->SetOrthographicNearPlaneOverride(TOptional<double>());
	bShowDepthBar = false;
}

void SEditorViewport::OnToggleRealtime()
{
	if (Client->IsRealtime())
	{
		Client->SetRealtime( false );
		if ( ActiveTimerHandle.IsValid() )
		{
			UnRegisterActiveTimer( ActiveTimerHandle.Pin().ToSharedRef() );
		}
		
	}
	else
	{
		Client->SetRealtime( true );
		ActiveTimerHandle = RegisterActiveTimer( 0.f, FWidgetActiveTimerDelegate::CreateSP( this, &SEditorViewport::EnsureTick ) );
	}
}


bool SEditorViewport::CanToggleRealtime() const
{
	return !Client->IsRealtimeOverrideSet();
}

void SEditorViewport::SetRenderDirectlyToWindow( const bool bInRenderDirectlyToWindow )
{
	ViewportWidget->SetRenderDirectlyToWindow( bInRenderDirectlyToWindow );
}


void SEditorViewport::EnableStereoRendering( const bool bInEnableStereoRendering )
{
	ViewportWidget->EnableStereoRendering( bInEnableStereoRendering );
}


void SEditorViewport::OnToggleStats()
{
	bool bIsEnabled =  Client->ShouldShowStats();
	Client->SetShowStats( !bIsEnabled );

	if( !bIsEnabled )
	{
		// We cannot show stats unless realtime rendering is enabled
		if ( !Client->IsRealtime() )
		{
			Client->SetRealtime( true );
			ActiveTimerHandle = RegisterActiveTimer( 0.f, FWidgetActiveTimerDelegate::CreateSP( this, &SEditorViewport::EnsureTick ) );
		}

		 // let the user know how they can enable stats via the console
		 FNotificationInfo Info(LOCTEXT("StatsEnableHint", "Stats display can be toggled via the STAT [type] console command"));
		 Info.ExpireDuration = 3.0f;
		 /* Temporarily remove the link until the page is updated
		 Info.HyperlinkText = LOCTEXT("StatsEnableHyperlink", "Learn more");
		 Info.Hyperlink = FSimpleDelegate::CreateStatic([](){ IDocumentation::Get()->Open(TEXT("Engine/Basics/ConsoleCommands#statisticscommands")); });
		 */
		 FSlateNotificationManager::Get().AddNotification(Info);
	}
}

void SEditorViewport::ToggleStatCommand(FString CommandName)
{
	GEngine->ExecEngineStat(GetWorld(), Client.Get(), *CommandName);

	// Invalidate the client to render once in case the click was on the checkbox itself (which doesn't dismiss the menu)
	Client->Invalidate();
}

bool SEditorViewport::IsStatCommandVisible(FString CommandName) const
{
	// Only if realtime and stats are also enabled should we show the stat as visible
	return Client->IsRealtime() && Client->ShouldShowStats() && Client->IsStatEnabled(CommandName);
}

void SEditorViewport::ToggleShowFlag(uint32 EngineShowFlagIndex)
{
	bool bOldState = Client->EngineShowFlags.GetSingleFlag(EngineShowFlagIndex);
	Client->EngineShowFlags.SetSingleFlag(EngineShowFlagIndex, !bOldState);

	// If changing collision flag, need to do special handling for hidden objects
	if (EngineShowFlagIndex == FEngineShowFlags::EShowFlag::SF_Collision)
	{
		Client->UpdateHiddenCollisionDrawing();
	}

	// Invalidate clients which aren't real-time so we see the changes
	Client->Invalidate();
}

bool SEditorViewport::IsShowFlagEnabled(uint32 EngineShowFlagIndex) const
{
	return Client->EngineShowFlags.GetSingleFlag(EngineShowFlagIndex);
}

void SEditorViewport::ChangeExposureSetting()
{
	Client->ExposureSettings.bFixed = !Client->ExposureSettings.bFixed;
	Client->Invalidate();
}

bool SEditorViewport::IsExposureSettingSelected() const
{
	return !Client->ExposureSettings.bFixed;
}

void SEditorViewport::Invalidate()
{
	bInvalidated = true;
	if (!ActiveTimerHandle.IsValid())
	{
		ActiveTimerHandle = RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SEditorViewport::EnsureTick));
	}
}

bool SEditorViewport::IsRealtime() const
{
	return Client->IsRealtime();
}

bool SEditorViewport::IsVisible() const
{
	const float VisibilityTimeThreshold = .25f;
	// The viewport is visible if we don't have a parent layout (likely a floating window) or this viewport is visible in the parent layout.
	// Also, always render the viewport if DumpGPU is active, regardless of tick time threshold -- otherwise these don't show up due to lag
	// caused by the GPU dump being triggered.
	return 
		LastTickTime == 0.0	||	// Never been ticked
		FPlatformTime::Seconds() - LastTickTime <= VisibilityTimeThreshold	// Ticked recently
#if WITH_DUMPGPU
		|| FRDGBuilder::IsDumpingFrame()	// GPU dump in progress
#endif		
		;
}

void SEditorViewport::OnScreenCapture()
{
	Client->TakeScreenshot(Client->Viewport, true);
}

void SEditorViewport::OnScreenCaptureForProjectThumbnail()
{
	if ( FApp::HasProjectName() )
	{
		const FString BaseFilename = FString(FApp::GetProjectName()) + TEXT(".png");
		const FString ScreenshotFilename = FPaths::Combine(*FPaths::ProjectDir(), *BaseFilename);
		UThumbnailManager::CaptureProjectThumbnail(Client->Viewport, ScreenshotFilename, true);
	}
}

EVisibility SEditorViewport::GetTransformToolbarVisibility() const
{
	return (Client->GetWidgetMode() != UE::Widget::WM_None) ? EVisibility::Visible : EVisibility::Hidden;
}

TSharedRef<SWidget> SEditorViewport::BuildFixedEV100Menu()  const
{
	const float EV100Min = -10.f;
	const float EV100Max = 20.f;

	return
		SNew( SBox )
		.HAlign( HAlign_Right )
		[
			SNew( SBox )
			.Padding( FMargin(0.0f, 0.0f, 0.0f, 0.0f) )
			.WidthOverride( 100.0f )
			[
				SNew ( SBorder )
				.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
				.Padding(FMargin(1.0f))
				[
					SNew(SSpinBox<float>)
					.Style(&FAppStyle::Get(), "Menu.SpinBox")
					.Font( FAppStyle::GetFontStyle( TEXT( "MenuItem.Font" ) ) )
					.MinValue(EV100Min)
					.MaxValue(EV100Max)
					.Value( this, &SEditorViewport::OnGetFixedEV100Value )
					.OnValueChanged( const_cast<SEditorViewport*>(this), &SEditorViewport::OnFixedEV100ValueChanged )
					.ToolTipText(LOCTEXT( "EV100ToolTip", "Sets the exposure value of the camera using the specified EV100. Exposure = 1 / (1.2 * 2^EV100)"))
					.IsEnabled( this, &SEditorViewport::IsFixedEV100Enabled )
				]
			]
		];
};

TSharedRef<SWidget> SEditorViewport::BuildWireframeMenu() const
{
	return 
		SNew( SBox )
		.HAlign( HAlign_Right )
		[
			SNew( SBox )
			.Padding( FMargin(0.0f, 0.0f, 0.0f, 0.0f) )
			.WidthOverride( 100.0f )
			[
				SNew ( SBorder )
				.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
				.Padding(FMargin(1.0f))
				[
					SNew(SSpinBox<float>)
					.Style(&FAppStyle::Get(), "Menu.SpinBox")
					.Font( FAppStyle::GetFontStyle( TEXT( "MenuItem.Font" ) ) )
					.MinValue(0.f)
					.MaxValue(1.f)
					.SupportDynamicSliderMaxValue(false)
					.Value( this, &SEditorViewport::OnGetWireframeOpacity )
					.OnValueChanged( const_cast<SEditorViewport*>(this), &SEditorViewport::OnWireframeOpacityChanged )
					.ToolTipText(LOCTEXT("WireframeOpacity_ToolTip", "Adjust opacity of wireframes in view."))
				]
			]
		];
};
				
void SEditorViewport::UpdateInViewportMenuLocation(const FVector2D InLocation)
{
	InViewportContextMenuLocation = InLocation;
	ULevelEditorViewportSettings* LevelEditorViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>();
	LevelEditorViewportSettings->LastInViewportMenuLocation = InLocation;
	LevelEditorViewportSettings->SaveConfig();
}

float SEditorViewport::OnGetFixedEV100Value() const
{
	if( Client.IsValid() )
	{
		return Client->ExposureSettings.FixedEV100;
	}
	return 0;
}

bool SEditorViewport::IsFixedEV100Enabled() const
{
	if( Client.IsValid() )
	{
		return Client->ExposureSettings.bFixed;
	}
	return false;
}


void SEditorViewport::OnFixedEV100ValueChanged(float NewValue)
{
	if( Client.IsValid() )
	{
		Client->ExposureSettings.bFixed = true;
		Client->ExposureSettings.FixedEV100 = NewValue;
		Client->Invalidate();
	}
}

void SEditorViewport::OnWireframeOpacityChanged(float Opacity)
{
	if( Client.IsValid() )
	{
		Client->WireframeOpacity = Opacity;
		Client->Invalidate();
	}
}
float SEditorViewport::OnGetWireframeOpacity() const
{
	if(Client.IsValid())
	{
		return Client->WireframeOpacity;
	}
	
	return 0.8f;
}

bool SEditorViewport::IsWidgetModeActive( UE::Widget::EWidgetMode Mode ) const
{
	return Client->GetWidgetMode() == Mode;
}

bool SEditorViewport::IsTranslateRotateModeVisible() const
{
	return GetDefault<ULevelEditorViewportSettings>()->bAllowTranslateRotateZWidget;
}

bool SEditorViewport::Is2DModeVisible() const
{
	return GetDefault<ULevelEditor2DSettings>()->bEnable2DWidget;
}

bool SEditorViewport::IsCoordSystemActive(ECoordSystem CoordSystem) const
{
	return Client->GetWidgetCoordSystemSpace() == CoordSystem;
}

void SEditorViewport::OnCycleWidgetMode()
{
	UE::Widget::EWidgetMode WidgetMode = Client->GetWidgetMode();

	// Can't cycle the widget mode if we don't currently have a widget
	if (WidgetMode == UE::Widget::WM_None)
	{
		return;
	}

	int32 WidgetModeAsInt = WidgetMode;

	do
	{
		++WidgetModeAsInt;

		if ((WidgetModeAsInt == UE::Widget::WM_TranslateRotateZ) && (!GetDefault<ULevelEditorViewportSettings>()->bAllowTranslateRotateZWidget))
		{
			++WidgetModeAsInt;
		}

		if ((WidgetModeAsInt == UE::Widget::WM_2D) && (!GetDefault<ULevelEditor2DSettings>()->bEnable2DWidget))
		{
			++WidgetModeAsInt;
		}

		if( WidgetModeAsInt == UE::Widget::WM_Max )
		{
			WidgetModeAsInt -= UE::Widget::WM_Max;
		}
	}
	while( !Client->CanSetWidgetMode( (UE::Widget::EWidgetMode)WidgetModeAsInt ) && WidgetModeAsInt != WidgetMode );

	Client->SetWidgetMode( (UE::Widget::EWidgetMode)WidgetModeAsInt );
}

void SEditorViewport::OnCycleCoordinateSystem()
{
	int32 CoordSystemAsInt = Client->GetWidgetCoordSystemSpace();

	++CoordSystemAsInt;

	// parent mode is only supported with new trs gizmos for now
	int CoordMax = COORD_Parent;
	if (UEditorInteractiveGizmoManager::AreEditorGizmosAllowed(Client->GetModeTools()))
	{
		CoordMax = UEditorInteractiveGizmoManager::IsExplicitModeEnabled() ? COORD_Max : COORD_Explicit;
	}
	
	if( CoordSystemAsInt >= CoordMax )
	{
		CoordSystemAsInt = COORD_World;
	}

	Client->SetWidgetCoordSystemSpace( (ECoordSystem)CoordSystemAsInt );
}

void SEditorViewport::OnIncreaseGizmoSize()
{
	UTransformGizmoEditorSettings* GizmoSettings = GetMutableDefault<UTransformGizmoEditorSettings>();
	const float NewValue = GizmoSettings->GetTransformGizmoAxisMultiplier() + 0.1f;
	GizmoSettings->SetTransformGizmoAxisMultiplier(NewValue);
}

void SEditorViewport::OnDecreaseGizmoSize()
{
	UTransformGizmoEditorSettings* GizmoSettings = GetMutableDefault<UTransformGizmoEditorSettings>();
	const float NewValue = FMath::Max(GizmoSettings->GetTransformGizmoAxisMultiplier() - 0.1f, 0.1f);
	GizmoSettings->SetTransformGizmoAxisMultiplier(NewValue);
}

void SEditorViewport::OnResetGizmoSize()
{
	UTransformGizmoEditorSettings* GizmoSettings = GetMutableDefault<UTransformGizmoEditorSettings>();
	GizmoSettings->SetTransformGizmoAxisMultiplier(1.0f);
}

TSharedPtr<SWidget> SEditorViewport::BuildViewportDepthBar()
{
	const FName MenuName = "EditorViewportDepthBarMenu";
	
	if (!UToolMenus::Get()->IsMenuRegistered(MenuName))
	{
		using namespace UE::UnrealEd;
	
		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(MenuName, NAME_None, EMultiBoxType::VerticalToolBar);
		Menu->StyleName = "EditorViewport.DepthBar.Toolbar";
	
		FToolMenuSection& Actions = Menu->AddSection("Actions");
		Actions.AddSeparator("Start");
		
		FToolMenuEntry& FocusEntry = Actions.AddMenuEntry(FEditorViewportCommands::Get().FocusAndClipViewportToSelection);
		FocusEntry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.FrameActor");
		
		Actions.AddEntry(CreateOrthographicClippingPlanesSubmenu(FOrthographicClippingPlanesSubmenuOptions { false } ));
		
		FToolMenuSection& EndSection = Menu->AddSection("End", {}, FToolMenuInsert(NAME_None, EToolMenuInsertType::Last));
		EndSection.AddMenuEntry(FEditorViewportCommands::Get().DismissOrthographicClippingPlanes);
	}
	
	FToolMenuContext Context;
	
	UUnrealEdViewportToolbarContext* ContextObject = UE::UnrealEd::CreateViewportToolbarDefaultContext(SharedThis(this));
	Context.AddObject(ContextObject);
	
	Context.AppendCommandList(GetCommandList());
	
	return SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("EditorViewport.DepthBar.Background"))
		.Padding(3.0f)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SAssignNew(DepthBar, SDepthBar)
				.Style(FAppStyle::Get(), "EditorViewport.DepthBar")
				.Mode(this, &SEditorViewport::GetDepthBarMode)
				.DepthSpace(this, &SEditorViewport::GetDepthBarSpace)
				.NearPlane(this, &SEditorViewport::GetDepthBarNearPlane)
				.FarPlane(this, &SEditorViewport::GetDepthBarFarPlane)
				.OnNearPlaneChanged(this, &SEditorViewport::OnDepthBarNearPlaneChanged)
				.OnFarPlaneChanged(this, &SEditorViewport::OnDepthBarFarPlaneChanged)
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				UToolMenus::Get()->GenerateWidget(MenuName, Context)
			]
		];
}

void SEditorViewport::OnToggleUseDepthBar()
{
	if (Client->IsOrtho())
	{
		if (bShowDepthBar)
		{
			DismissDepthBar();
		}
		else
		{
			bShowDepthBar = true;
		}
	}
}

bool SEditorViewport::CanUseDepthBar() const
{
	return Client->IsOrtho();	
}

bool SEditorViewport::IsDepthBarActive() const
{
	return bShowDepthBar;
}

UWorld* SEditorViewport::GetWorld() const
{
	return Client->GetWorld();
}


void SEditorViewport::OnToggleSurfaceSnap()
{
	auto* Settings = GetMutableDefault<ULevelEditorViewportSettings>();
	Settings->SnapToSurface.bEnabled = !Settings->SnapToSurface.bEnabled;
}

bool SEditorViewport::OnIsSurfaceSnapEnabled()
{
	return GetDefault<ULevelEditorViewportSettings>()->SnapToSurface.bEnabled;
}

void SEditorViewport::OnToggleRotateToSurfaceNormal()
{
	auto& Settings = GetMutableDefault<ULevelEditorViewportSettings>()->SnapToSurface;
	Settings.bSnapRotation = !Settings.bSnapRotation;

	// If user is editing snapping settings, we assume they also want snapping turned on
	if (!Settings.bEnabled)
	{
		Settings.bEnabled = true;
	}
}

bool SEditorViewport::IsRotateToSurfaceNormalEnabled()
{
	const auto& Settings = GetDefault<ULevelEditorViewportSettings>()->SnapToSurface;
	return Settings.bSnapRotation;
}

void SEditorViewport::OnToggleDistanceBasedCameraSpeed()
{
	if (ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>())
	{
		ViewportSettings->bUseDistanceScaledCameraSpeed = !ViewportSettings->bUseDistanceScaledCameraSpeed;
	}
}

bool SEditorViewport::IsDistanceBasedCameraSpeedEnabled()
{
	if (const ULevelEditorViewportSettings* const ViewportSettings = GetDefault<ULevelEditorViewportSettings>())
	{
		return ViewportSettings->bUseDistanceScaledCameraSpeed;
	}

	return false;
}

TSharedPtr<SWidget> SEditorViewport::CreateViewportIndicatorWidget(const TAttribute<EVisibility>& InVisibility)
{
	// This makes a gradient that displays whether a viewport is active
	static FLinearColor ActiveBorderColor =
		FAppStyle::Get().GetSlateColor("EditorViewport.ActiveBorderColor").GetSpecifiedColor();
	static FLinearColor ActiveBorderColorTransparent(ActiveBorderColor.R, ActiveBorderColor.G, ActiveBorderColor.B, 0.0f);
	static TArray<FLinearColor> GradientStops{ ActiveBorderColorTransparent, ActiveBorderColor, ActiveBorderColorTransparent };

	return SNew(SBox)
		.Visibility(InVisibility)
		.MaxDesiredHeight(1.0f)
		.MinDesiredHeight(1.0f)
		[
			SNew(SComplexGradient)
			.GradientColors(GradientStops)
			.Orientation(EOrientation::Orient_Vertical)
		];
}

void SEditorViewport::SetAspectRatio(float AspectRatioIn)
{	
	ViewportAspectRatioManager->SetAspectRatioConstraint(AspectRatioIn);
	ViewportAspectRatioManager->SetAspectRatioMode(SEditorViewportGridPanel::EAspectRatioMode::Constrained);
}

void SEditorViewport::SetUsePlatformAspectRatio()
{
	ViewportAspectRatioManager->SetAspectRatioMode(SEditorViewportGridPanel::EAspectRatioMode::Platform);
}

void SEditorViewport::UnsetAspectRatio()
{
	ViewportAspectRatioManager->SetAspectRatioMode(SEditorViewportGridPanel::EAspectRatioMode::Unconstrained);	
}

bool SEditorViewport::IsPreviewingScreenPercentage() const
{
	return Client->IsPreviewingScreenPercentage();
}

void SEditorViewport::TogglePreviewingScreenPercentage()
{
	Client->SetPreviewingScreenPercentage(!IsPreviewingScreenPercentage());
}

void SEditorViewport::OnOpenViewportPerformanceProjectSettings()
{
	FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Project", "Editor", "EditorPerformanceProjectSettings");
}

void SEditorViewport::OnOpenViewportPerformanceEditorPreferences()
{
	FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Editor", "General", "EditorPerformanceSettings");
}

TSharedPtr<IPreviewProfileController> SEditorViewport::GetPreviewProfileController()
{
	if (!PreviewProfileController)
	{
		PreviewProfileController = CreatePreviewProfileController();
	}

	return PreviewProfileController;
}

void SEditorViewport::MarkLegacyToolbarChildAsAutomaticallyUpgradable(const TSharedRef<SWidget>& ExpectedChild)
{
	AutoUpgradeWidgetChild = ExpectedChild;
}

EActiveTimerReturnType SEditorViewport::EnsureTick( double InCurrentTime, float InDeltaTime )
{
	// Keep the timer going if we're realtime or were invalidated this frame
	const bool bShouldContinue = Client->IsRealtime() || bInvalidated;
	bInvalidated = false;
	return bShouldContinue ? EActiveTimerReturnType::Continue : EActiveTimerReturnType::Stop;
}

EVisibility SEditorViewport::GetActiveBorderVisibility() const
{
	EVisibility BaseVisibility = OnGetViewportContentVisibility();
	if (BaseVisibility != EVisibility::Collapsed)
	{
		// The active border should never be hit testable as it overlays viewport UI but is for display purposes only
		return EVisibility::HitTestInvisible;
	}

	return BaseVisibility;
}

///////////////////////////////////////////////////////////////////////////////
// begin feature level control functions block
///////////////////////////////////////////////////////////////////////////////
EShaderPlatform SEditorViewport::GetShaderPlatformHelper(const ERHIFeatureLevel::Type FeatureLevel) const
{
	UMaterialShaderQualitySettings* MaterialShaderQualitySettings = UMaterialShaderQualitySettings::Get();
	const FName& PreviewPlatform = MaterialShaderQualitySettings->GetPreviewPlatform();

	EShaderPlatform ShaderPlatform = PreviewPlatform != NAME_None ? ShaderFormatToLegacyShaderPlatform(PreviewPlatform) : SP_NumPlatforms;
	if (ShaderPlatform == SP_NumPlatforms)
	{
		ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);
	}

	return ShaderPlatform;
}

// Deprecated in 5.7
TSharedRef<SWidget> SEditorViewport::BuildFeatureLevelWidget() const
{
	return BuildShaderPlatformWidget();
}

TSharedRef<SWidget> SEditorViewport::BuildShaderPlatformWidget() const
{
	TSharedRef<SWidget> BoxWidget = SNew(SHorizontalBox)
		.Visibility(this, &SEditorViewport::GetCurrentShaderPlatformPreviewTextVisibility)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0f, 1.0f, 2.0f, 1.0f)
		[
			SNew(STextBlock)
			.Text(this, &SEditorViewport::GetCurrentShaderPlatformPreviewText, true)
			.ShadowOffset(FVector2D(1, 1))
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(4.0f, 1.0f, 2.0f, 1.0f)
		[
			SNew(STextBlock)
			.Text(this, &SEditorViewport::GetCurrentShaderPlatformPreviewText, false)
			.ShadowOffset(FVector2D(1, 1))
		];

	return BoxWidget;
}

// Deprecated in 5.7
EVisibility SEditorViewport::GetCurrentFeatureLevelPreviewTextVisibility() const
{
	return GetCurrentShaderPlatformPreviewTextVisibility();
}

EVisibility SEditorViewport::GetCurrentShaderPlatformPreviewTextVisibility() const
{
	if (Client->GetWorld() && !GLevelEditorModeTools().IsViewportUIHidden())
	{
		return (GEditor && GEditor->IsPreviewPlatformActive()) ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed;
	}
	else
	{
		return EVisibility::Collapsed;
	}
}

// Deprecated in 5.7
FText SEditorViewport::GetCurrentFeatureLevelPreviewText(bool bDrawOnlyLabel) const
{
	return GetCurrentShaderPlatformPreviewText(bDrawOnlyLabel);
}

FText SEditorViewport::GetCurrentShaderPlatformPreviewText(bool bDrawOnlyLabel) const
{
	FText LabelName;

	if (bDrawOnlyLabel)
	{
		LabelName = LOCTEXT("PreviewPlatformLabel", "Preview Platform:");
	}
	else
	{
		const FText& PlatformText = GEditor->PreviewPlatform.GetFriendlyName();
		LabelName = FText::Format(LOCTEXT("WorldFeatureLevel", "{0}"), PlatformText);
	}

	return LabelName;
}

///////////////////////////////////////////////////////////////////////////////
// end feature level control functions block
///////////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
