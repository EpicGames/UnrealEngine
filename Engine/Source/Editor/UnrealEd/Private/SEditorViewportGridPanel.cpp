// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEditorViewportGridPanel.h"
#include "Framework/Application/SlateApplication.h"

SLATE_IMPLEMENT_WIDGET(SEditorViewportGridPanel)
void SEditorViewportGridPanel::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
}

void SEditorViewportGridPanel::Construct(const FArguments& InArgs)
{
	SGridPanel::FArguments SuperArgs;
	SuperArgs.FillColumn(0, 0.f);
	SuperArgs.FillColumn(1, 1.f);
	SuperArgs.FillColumn(2, 0.f);
	SuperArgs.FillRow(0, 0.f);
	SuperArgs.FillRow(1, 1.f);
	SuperArgs.FillRow(2, 0.f);
	SuperArgs._Visibility = InArgs._Visibility;
	checkf(InArgs._ViewportWidget.IsSet(), TEXT("ViewportWidget must be set for SEditorViewportGridPanel widget!"));
	SuperArgs + SGridPanel::Slot(1, 1)[InArgs._ViewportWidget.Get().ToSharedRef()];
	SGridPanel::Construct(SuperArgs);

	PlatformAspectRatio = 0.f;
	ForcedAspectRatio = 0.f;
	AspectRatioMode = EAspectRatioMode::Platform;	
	FSlateApplication::Get().OnConstrainedAspectRatioChanged.AddSP(this, &SEditorViewportGridPanel::UpdateAspectRatio);
}

void SEditorViewportGridPanel::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	// To make sense of this, look in SEditorViewport::Construct
	// We wrap the main editor viewport in the middle of a 3x3 grid in order to restrict aspect ratio for preview platforms that request it.
	{
		// Default values, fill the middle cell of the grid.
		ColFillCoefficients[0].Set(0.f);
		ColFillCoefficients[1].Set(1.f);
		ColFillCoefficients[2].Set(0.f);
		RowFillCoefficients[0].Set(0.f);
		RowFillCoefficients[1].Set(1.f);
		RowFillCoefficients[2].Set(0.f);

		const float AspectRatio = 
			AspectRatioMode == EAspectRatioMode::Platform ? PlatformAspectRatio : 
			AspectRatioMode == EAspectRatioMode::Constrained ? ForcedAspectRatio :
			0.0;		

		if (AspectRatio != 0.f)
		{
			const float ContainerX = AllottedGeometry.GetLocalSize().X;
			const float ContainerY = AllottedGeometry.GetLocalSize().Y;
			const float NewX = ContainerY * AspectRatio;
			const float NewY = ContainerX / AspectRatio;

			if (NewX < ContainerX)
			{
				const float Offset = (ContainerX - NewX) / 2;
				const float Center = (ContainerX / Offset) - 2;
				ColFillCoefficients[0].Set(1.f);
				ColFillCoefficients[1].Set(Center);
				ColFillCoefficients[2].Set(1.f);
			}

			if (NewY < ContainerY)
			{
				const float Offset = (ContainerY - NewY) / 2;
				const float Center = (ContainerY / Offset) - 2;
				RowFillCoefficients[0].Set(1.f);
				RowFillCoefficients[1].Set(Center);
				RowFillCoefficients[2].Set(1.f);
			}
		}
	}
	SGridPanel::OnArrangeChildren(AllottedGeometry, ArrangedChildren);
}

void SEditorViewportGridPanel::UpdateAspectRatio(const float& AspectRatio)
{
	PlatformAspectRatio = AspectRatio;
}

void SEditorViewportGridPanel::SetAspectRatioMode(EAspectRatioMode ModeIn)
{
	AspectRatioMode = ModeIn;
}

SEditorViewportGridPanel::EAspectRatioMode SEditorViewportGridPanel::GetAspectRatioMode() const
{
	return AspectRatioMode;
}

void SEditorViewportGridPanel::SetAspectRatioConstraint(const float& AspectRatio)
{
	ForcedAspectRatio = AspectRatio;
}