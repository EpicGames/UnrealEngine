// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SWidget.h"

class SEditorViewportGridPanel : public SGridPanel
{
	SLATE_DECLARE_WIDGET_API(SEditorViewportGridPanel, SGridPanel, UNREALED_API)

public:

	enum class EAspectRatioMode
	{
		Platform = 0,
		Unconstrained = 1,
		Constrained = 2
	};

	SLATE_BEGIN_ARGS(SEditorViewportGridPanel)
		{}
		SLATE_ATTRIBUTE(TSharedPtr<SWidget>, ViewportWidget);
	SLATE_END_ARGS()

	UNREALED_API void Construct(const FArguments& InArgs);
	UNREALED_API virtual void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const override;

	// Hook to update the aspect ratio coming from the platform settings - called automatically externally to keep this up to date.
	// Use SetAspectRatioConstraint and SetAspectRatioMode(EAspectRatioMode::Constrained) to set a fixed aspect ratio.
	UNREALED_API void UpdateAspectRatio(const float& AspectRatio);

	UNREALED_API void SetAspectRatioMode(EAspectRatioMode ModeIn);
	UNREALED_API EAspectRatioMode GetAspectRatioMode() const;
	UNREALED_API void SetAspectRatioConstraint(const float& AspectRatio);

private:
	TAttribute<TSharedPtr<SWidget>> ViewportWidget;
	
	EAspectRatioMode AspectRatioMode;
	float PlatformAspectRatio;
	float ForcedAspectRatio;

};