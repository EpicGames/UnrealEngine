// Copyright Epic Games, Inc. All Rights Reserved.


#include "SEditorViewportToolBarButton.h"

#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SNullWidget.h"

class SWidget;


void SEditorViewportToolBarButton::Construct( const FArguments& Declaration)
{
	OnClickedDelegate = Declaration._OnClicked;
	IsChecked = Declaration._IsChecked;
	const TSharedRef<SWidget>& ContentSlotWidget = Declaration._Content.Widget;

	bool bContentOverride = ContentSlotWidget != SNullWidget::NullWidget;

	EUserInterfaceActionType ButtonType = Declaration._ButtonType;
	
	// The style of the image to show in the button
	const FName ImageStyleName = Declaration._Image.Get();

	TSharedPtr<SWidget> ButtonWidget;

	if( ButtonType == EUserInterfaceActionType::Button )
	{
		const FSlateBrush* Brush = FAppStyle::GetBrush( ImageStyleName );

		ButtonWidget =
			SNew( SButton )
			.ButtonStyle(Declaration._ButtonStyle)
			.OnClicked( OnClickedDelegate )
			.HAlign( HAlign_Center )
			.VAlign( VAlign_Center )
			.ContentPadding(FMargin(4.f, 0.f))
			[
				// If we have a content override use it instead of the default image
				bContentOverride
					? ContentSlotWidget
					: TSharedRef<SWidget>( SNew( SImage ) .Image( Brush ) )
			];
	}
	else
	{
		// Cache off checked/unchecked image states
		NormalBrush = FAppStyle::GetBrush( ImageStyleName, ".Normal" );
		CheckedBrush = FAppStyle::GetBrush( ImageStyleName, ".Checked" );

		if( CheckedBrush->GetResourceName() == FName("Default") )
		{
			// A different checked brush was not specified so use the normal image when checked
			CheckedBrush = NormalBrush;
		}

		ButtonWidget = 
			SNew( SCheckBox )
			.Style(Declaration._CheckBoxStyle)
			.OnCheckStateChanged( this, &SEditorViewportToolBarButton::OnCheckStateChanged )
			.IsChecked( this, &SEditorViewportToolBarButton::OnIsChecked )
			[
				bContentOverride ? ContentSlotWidget :
				TSharedRef<SWidget>(
					SNew( SBox )
					.Padding(0.0f)
					.VAlign( VAlign_Center )
					.HAlign( HAlign_Center )
					[
						SNew( SImage )
						.Image( this, &SEditorViewportToolBarButton::OnGetButtonImage )
						.ColorAndOpacity(FSlateColor::UseForeground())
					])
			];
	}

	ChildSlot
	[
		ButtonWidget.ToSharedRef()
	];
}

void SEditorViewportToolBarButton::OnCheckStateChanged( ECheckBoxState NewCheckedState )
{
	// When the check state changes (can only happen during clicking in this case) execute our on clicked delegate
	if(OnClickedDelegate.IsBound() == true)
	{
		OnClickedDelegate.Execute();
	}
}

const FSlateBrush* SEditorViewportToolBarButton::OnGetButtonImage() const
{
	return IsChecked.Get() ? CheckedBrush : NormalBrush;
}

ECheckBoxState SEditorViewportToolBarButton::OnIsChecked() const
{
	return IsChecked.Get() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

