// Copyright Epic Games, Inc. All Rights Reserved.


#include "SEditorViewportToolBarMenu.h"

#include "HAL/PlatformMath.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "SEditorViewportToolBarMenuButton.h"
#include "SViewportToolBar.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Types/SlateEnums.h"
#include "UObject/UnrealNames.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;
struct FGeometry;
struct FPointerEvent;

void SEditorViewportToolbarMenu::Construct( const FArguments& Declaration )
{
	const TAttribute<FText>& Label = Declaration._Label;

	SetToolTipText(FText::GetEmpty());

	const FName ImageName = Declaration._Image;
	const FSlateBrush* ImageBrush = FAppStyle::GetBrush( ImageName );

	LabelIconBrush = Declaration._LabelIcon;

	ParentToolBar = Declaration._ParentToolBar;

	TSharedPtr<SWidget> ButtonContent;

	// Create the content for the button.  We always use an image over a label if it is valid
	if( ImageName != NAME_None )
	{
		ButtonContent = 
			SNew(SBox)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(SImage)	
				.Image(ImageBrush)
				.ColorAndOpacity(FSlateColor::UseForeground())
			];
	}
	else
	{
		if( LabelIconBrush.IsBound() || LabelIconBrush.Get() != NULL )
		{
			// Label with an icon to the left
			ButtonContent = 
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.AutoWidth()
				.Padding(4.0f, 0.0f, 4.f, 0.0f)
				[
					SNew( SImage )
					.Visibility(this, &SEditorViewportToolbarMenu::GetLabelIconVisibility)
					.Image(LabelIconBrush)
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
				+SHorizontalBox::Slot()
				.Padding(0.0f, 0.0f, 4.f, 0.0f)
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(Label)
				];
		}
		else
		{
			// Just the label text, no icon
			ButtonContent = 
				SNew(SBox)
				.VAlign(VAlign_Center)
				.Padding(FMargin(4.f, 0.f))
				[
					SNew( STextBlock )
					.Text(Label)
				];
		}
	}

	ChildSlot
	[
		SAssignNew(MenuAnchor, SMenuAnchor)
		.Placement(Declaration._MenuPlacement)
		.OnGetMenuContent(Declaration._OnGetMenuContent)
	];

	MenuAnchor->SetContent(
		SNew( SEditorViewportToolBarMenuButton, MenuAnchor.ToSharedRef() )
		.ContentPadding( FMargin(0.0f, 0.0f) )
		.VAlign(VAlign_Center)
		.ButtonStyle(Declaration._MenuStyle)
		.ToolTipText(this, &SEditorViewportToolbarMenu::GetFilteredToolTipText, Declaration._ToolTipText)
		.OnClicked( this, &SEditorViewportToolbarMenu::OnMenuClicked )
		.ForegroundColor(Declaration._ForegroundColor)
		[
			ButtonContent.ToSharedRef()
		]
	);
}

FText SEditorViewportToolbarMenu::GetFilteredToolTipText(TAttribute<FText> ToolTipText) const
{
	// If we're part of a toolbar that has a currently open menu, then we suppress our tool-tip
	// as it will just get in the way
	if (TSharedPtr<SViewportToolBar> Toolbar = ParentToolBar.Pin())
	{
		if (TSharedPtr<SMenuAnchor> OpenedMenu = Toolbar->GetOpenMenu().Pin())
		{
			if (OpenedMenu->IsOpen())
			{
				return FText::GetEmpty();
			}
		}
	}

	return ToolTipText.Get();
}

FReply SEditorViewportToolbarMenu::OnMenuClicked()
{
	// If the menu button is clicked toggle the state of the menu anchor which will open or close the menu
	if (MenuAnchor->ShouldOpenDueToClick())
	{
		MenuAnchor->SetIsOpen( true );
		if (TSharedPtr<SViewportToolBar> Toolbar = ParentToolBar.Pin())
		{
			Toolbar->SetOpenMenu(MenuAnchor);
		}
	}
	else
	{
		MenuAnchor->SetIsOpen( false );
		if (TSharedPtr<SViewportToolBar> Toolbar = ParentToolBar.Pin())
		{
			TSharedPtr<SMenuAnchor> NullAnchor;
			Toolbar->SetOpenMenu(NullAnchor);
		}
	}

	return FReply::Handled();
}

void SEditorViewportToolbarMenu::OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	SCompoundWidget::OnMouseEnter(MyGeometry, MouseEvent);

	if (TSharedPtr<SViewportToolBar> Toolbar = ParentToolBar.Pin())
	{
		// See if there is another menu on the same tool bar already open
		if (TSharedPtr<SMenuAnchor> OpenedMenu = Toolbar->GetOpenMenu().Pin())
		{
			if (OpenedMenu != MenuAnchor)
			{
				// There is another menu open so we open this menu and close the other
				Toolbar->SetOpenMenu( MenuAnchor ); 
				MenuAnchor->SetIsOpen( true );	
			}
		}
	}
}

EVisibility SEditorViewportToolbarMenu::GetLabelIconVisibility() const
{
	return LabelIconBrush.Get() != NULL ? EVisibility::Visible : EVisibility::Collapsed;
}

TWeakPtr<class SViewportToolBar> SEditorViewportToolbarMenu::GetParentToolBar() const
{
	return ParentToolBar;
}

bool SEditorViewportToolbarMenu::IsMenuOpen() const
{
	return MenuAnchor->IsOpen();
}
