// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEditorViewportToolBarMenuButton.h"

#include "Delegates/Delegate.h"
#include "Widgets/Input/SMenuAnchor.h"

void SEditorViewportToolBarMenuButton::Construct(const FArguments& InArgs, TSharedRef<SMenuAnchor> InMenuAnchor)
{
	SButton::Construct(SButton::FArguments()
		// Allows users to drag with the mouse to select options after opening the menu
		.ClickMethod(EButtonClickMethod::MouseDown)
		.OnClicked(InArgs._OnClicked)
		.ForegroundColor(InArgs._ForegroundColor)
		.HAlign(InArgs._HAlign)
		.VAlign(InArgs._VAlign)
		.ContentPadding(InArgs._ContentPadding)
		.ButtonStyle(InArgs._ButtonStyle)
		[
			InArgs._Content.Widget
		]
	);

	SetAppearPressed(TAttribute<bool>::CreateSP(this, &SEditorViewportToolBarMenuButton::IsMenuOpen));
	MenuAnchor = InMenuAnchor;
}

bool SEditorViewportToolBarMenuButton::IsMenuOpen() const
{
	if (TSharedPtr<SMenuAnchor> MenuAnchorPinned = MenuAnchor.Pin())
	{
		return MenuAnchorPinned->IsOpen();
	}

	return false;
}
