// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/SlateDelegates.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"

class SMenuAnchor;
class SWidget;

/**
 * Button subclass that appears pressed while its menu anchor has a menu open.
 * Mimics the behavior of the button for pulldown menus.
 */
class SEditorViewportToolBarMenuButton : public SButton
{
public:
	SLATE_BEGIN_ARGS(SEditorViewportToolBarMenuButton)
		: _ForegroundColor(FSlateColor::UseStyle())
		, _HAlign(HAlign_Fill)
		, _VAlign(VAlign_Fill)
		, _ContentPadding(FMargin(4.0, 2.0))
		, _ButtonStyle(&FCoreStyle::Get().GetWidgetStyle< FButtonStyle >("Button"))
	{
	}
		SLATE_ATTRIBUTE(FSlateColor, ForegroundColor)
		SLATE_ARGUMENT(EHorizontalAlignment, HAlign)
		SLATE_ARGUMENT(EVerticalAlignment, VAlign)
		SLATE_ATTRIBUTE(FMargin, ContentPadding)
		SLATE_STYLE_ARGUMENT(FButtonStyle, ButtonStyle)
		SLATE_DEFAULT_SLOT(FArguments, Content)
		SLATE_EVENT(FOnClicked, OnClicked)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<SMenuAnchor> InMenuAnchor);

private:
	bool IsMenuOpen() const;

	TWeakPtr<SMenuAnchor> MenuAnchor;
};
