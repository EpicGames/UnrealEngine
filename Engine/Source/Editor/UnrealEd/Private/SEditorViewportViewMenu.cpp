// Copyright Epic Games, Inc. All Rights Reserved.


#include "SEditorViewportViewMenu.h"
#include "SEditorViewportViewMenuContext.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ToolMenus.h"
#include "Styling/AppStyle.h"
#include "RenderResource.h"
#include "SEditorViewport.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "ViewportToolbar/UnrealEdViewportToolbarContext.h"

#define LOCTEXT_NAMESPACE "EditorViewportViewMenu"

const FName SEditorViewportViewMenu::BaseMenuName("UnrealEd.ViewportToolbar.View");

void SEditorViewportViewMenu::Construct( const FArguments& InArgs, TSharedRef<SEditorViewport> InViewport, TSharedRef<class SViewportToolBar> InParentToolBar )
{
	Viewport = InViewport;
	MenuName = BaseMenuName;
	MenuExtenders = InArgs._MenuExtenders;

	SEditorViewportToolbarMenu::Construct
	(
		SEditorViewportToolbarMenu::FArguments()
			.ParentToolBar( InParentToolBar)
			.Cursor( EMouseCursor::Default )
			.Label(this, &SEditorViewportViewMenu::GetViewMenuLabel)
			.LabelIcon(this, &SEditorViewportViewMenu::GetViewMenuLabelIcon)
			.OnGetMenuContent( this, &SEditorViewportViewMenu::GenerateViewMenuContent )
	);
}

FText SEditorViewportViewMenu::GetViewMenuLabel() const
{
	return UE::UnrealEd::GetViewModesSubmenuLabel(Viewport);
}

const FSlateBrush* SEditorViewportViewMenu::GetViewMenuLabelIcon() const
{

	TSharedPtr< SEditorViewport > PinnedViewport = Viewport.Pin();
	if( PinnedViewport.IsValid() )
	{
		const TSharedPtr<FEditorViewportClient> ViewportClient = PinnedViewport->GetViewportClient();
		check(ViewportClient.IsValid());
		const EViewModeIndex ViewMode = ViewportClient->GetViewMode();

		return UViewModeUtils::GetViewModeDisplayIcon(ViewMode);
	}

	return FStyleDefaults::GetNoBrush();
}

void SEditorViewportViewMenu::RegisterMenus() const
{
	// Use a static bool to track whether or not this menu is registered. Bool instead of checking the registered state
	// with ToolMenus because we want the new viewport toolbar to be able to create this menu without breaking this
	// code. Static because this code can be called multiple times using different instances of this class.
	static bool bDidRegisterMenu = false;
	if (!bDidRegisterMenu)
	{
		bDidRegisterMenu = true;

		// Don't warn here to avoid warnings if the new viewport toolbar already has created an empty version
		// of this menu.
		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(BaseMenuName, NAME_None, EMultiBoxType::Menu, false);
		Menu->AddDynamicSection("BaseSection", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			if (UEditorViewportViewMenuContext* Context = InMenu->FindContext<UEditorViewportViewMenuContext>())
			{
				Context->EditorViewportViewMenu.Pin()->FillViewMenu(InMenu);
			}
		}));
	}
}

TSharedRef<SWidget> SEditorViewportViewMenu::GenerateViewMenuContent() const
{
	RegisterMenus();

	UEditorViewportViewMenuContext* ContextObject = NewObject<UEditorViewportViewMenuContext>();
	ContextObject->EditorViewportViewMenu = SharedThis(this);

	FToolMenuContext MenuContext(Viewport.Pin()->GetCommandList(), MenuExtenders, ContextObject);
	return UToolMenus::Get()->GenerateWidget(MenuName, MenuContext);
}

void SEditorViewportViewMenu::FillViewMenu(UToolMenu* Menu) const
{
	UE::UnrealEd::IsViewModeSupportedDelegate IsViewModeSupported = UE::UnrealEd::IsViewModeSupportedDelegate::CreateLambda(
		[WeakToolBar = ParentToolBar](EViewModeIndex ViewModeIndex) -> bool {
			if (TSharedPtr<SViewportToolBar> ToolBar = WeakToolBar.Pin())
			{
				return ToolBar->IsViewModeSupported(ViewModeIndex);
			}
			return true;
		});

	// Add the UnrealEd viewport toolbar context.
	{
		UUnrealEdViewportToolbarContext* const ContextObject = NewObject<UUnrealEdViewportToolbarContext>();
		ContextObject->Viewport = Viewport;
		Menu->Context.AddObject(ContextObject);
	}

	UE::UnrealEd::PopulateViewModesMenu(Menu);
	UE::UnrealEd::AddExposureSection(Menu, Viewport.Pin());
}

#undef LOCTEXT_NAMESPACE
