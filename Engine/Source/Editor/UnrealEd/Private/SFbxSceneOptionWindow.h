// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateColor.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Fbx/SSceneBaseMeshListView.h"
#include "Framework/Docking/TabManager.h"
#include "Fbx/SSceneMaterialsListView.h"

class IDetailsView;
class SFbxReimportSceneTreeView;
class SFbxSceneSkeletalMeshListView;
class SFbxSceneSkeletalMeshReimportListView;
class SFbxSceneStaticMeshListView;
class SFbxSceneStaticMeshReimportListView;
class SFbxSceneTreeView;
struct FPropertyChangedEvent;

class SFbxSceneOptionWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SFbxSceneOptionWindow)
		: _SceneInfo(nullptr)
		, _SceneInfoOriginal(nullptr)
		, _MeshStatusMap(nullptr)
		, _CanReimportHierarchy(false)
		, _NodeStatusMap(nullptr)
		, _GlobalImportSettings(nullptr)
		, _SceneImportOptionsDisplay(nullptr)
		, _SceneImportOptionsStaticMeshDisplay(nullptr)
		, _OverrideNameOptionsMap(nullptr)
		, _SceneImportOptionsSkeletalMeshDisplay(nullptr)
		, _OwnerWindow()
		, _FullPath(TEXT(""))
		{}

		SLATE_ARGUMENT( TSharedPtr<FFbxSceneInfo>, SceneInfo )
		SLATE_ARGUMENT( TSharedPtr<FFbxSceneInfo>, SceneInfoOriginal)
		SLATE_ARGUMENT( FbxSceneReimportStatusMapPtr, MeshStatusMap)
		SLATE_ARGUMENT( bool, CanReimportHierarchy)
		SLATE_ARGUMENT( FbxSceneReimportStatusMapPtr, NodeStatusMap)
		SLATE_ARGUMENT( UnFbx::FBXImportOptions*, GlobalImportSettings)
		SLATE_ARGUMENT( class UFbxSceneImportOptions*, SceneImportOptionsDisplay)
		SLATE_ARGUMENT( class UFbxSceneImportOptionsStaticMesh*, SceneImportOptionsStaticMeshDisplay)
		SLATE_ARGUMENT( ImportOptionsNameMapPtr, OverrideNameOptionsMap)
		SLATE_ARGUMENT( class UFbxSceneImportOptionsSkeletalMesh*, SceneImportOptionsSkeletalMeshDisplay)
		SLATE_ARGUMENT( TWeakPtr<SWindow>, OwnerWindow)
		SLATE_ARGUMENT( FString, FullPath )
	SLATE_END_ARGS()

public:

	SFbxSceneOptionWindow();
	~SFbxSceneOptionWindow();

	void Construct(const FArguments& InArgs);
	virtual bool SupportsKeyboardFocus() const override { return true; }

	void CloseFbxSceneOption();

	FReply OnImport()
	{
		bShouldImport = true;
		CloseFbxSceneOption();
		return FReply::Handled();
	}

	FReply OnCancel()
	{
		bShouldImport = false;
		CloseFbxSceneOption();
		return FReply::Handled();
	}

	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override
	{
		if( InKeyEvent.GetKey() == EKeys::Escape )
		{
			return OnCancel();
		}

		return FReply::Unhandled();
	}

	bool ShouldImport() const
	{
		return bShouldImport;
	}

	void OnToggleReimportHierarchy(ECheckBoxState CheckType);
	ECheckBoxState IsReimportHierarchyChecked() const;
	void OnToggleBakePivotInVertex(ECheckBoxState CheckType);
	ECheckBoxState IsBakePivotInVertexChecked() const;

	//Material UI
	FText GetMaterialBasePath() const;
	void OnMaterialBasePathCommited(const FText& InText, ETextCommit::Type InCommitType);
	FReply OnMaterialBasePathBrowse();
	FSlateColor GetMaterialBasePathTextColor() const;

	static void CopyFbxOptionsToFbxOptions(UnFbx::FBXImportOptions *SourceOptions, UnFbx::FBXImportOptions *DestinationOptions);

	static void CopyStaticMeshOptionsToFbxOptions(UnFbx::FBXImportOptions *ImportSettings, class UFbxSceneImportOptionsStaticMesh* StaticMeshOptions);
	static void CopyFbxOptionsToStaticMeshOptions(UnFbx::FBXImportOptions *ImportSettings, class UFbxSceneImportOptionsStaticMesh* StaticMeshOptions);

	static void CopySkeletalMeshOptionsToFbxOptions(UnFbx::FBXImportOptions *ImportSettings, class UFbxSceneImportOptionsSkeletalMesh* SkeletalMeshOptions);
	static void CopyFbxOptionsToSkeletalMeshOptions(UnFbx::FBXImportOptions *ImportSettings, class UFbxSceneImportOptionsSkeletalMesh* SkeletalMeshOptions);

	void OnFinishedChangingPropertiesSceneTabDetailView(const FPropertyChangedEvent& PropertyChangedEvent);

private:

	bool CanCloseTab();

	bool CanImport() const;

	void InitAllTabs();
	TSharedPtr<SWidget> SpawnDockTab();
	TSharedRef<SDockTab> SpawnSceneTab(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnStaticMeshTab(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnSkeletalMeshTab(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnMaterialTab(const FSpawnTabArgs& Args);

	TSharedRef<SDockTab> SpawnSceneReimportTab(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnStaticMeshReimportTab(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnSkeletalMeshReimportTab(const FSpawnTabArgs& Args);

private:
	//SFbxSceneOptionWindow Arguments
	
	TSharedPtr< FFbxSceneInfo > SceneInfo;
	TSharedPtr< FFbxSceneInfo > SceneInfoOriginal;
	FbxSceneReimportStatusMapPtr MeshStatusMap;
	FbxSceneReimportStatusMapPtr NodeStatusMap;
	UnFbx::FBXImportOptions *GlobalImportSettings;
	class UFbxSceneImportOptions* SceneImportOptionsDisplay;
	class UFbxSceneImportOptionsStaticMesh* SceneImportOptionsStaticMeshDisplay;
	ImportOptionsNameMapPtr OverrideNameOptionsMap;
	class UFbxSceneImportOptionsSkeletalMesh* SceneImportOptionsSkeletalMeshDisplay;
	TWeakPtr< SWindow > OwnerWindow;
	FString FullPath;

	bool bCanReimportHierarchy;

	
	//Variables

	TSharedPtr<FTabManager> FbxSceneImportTabManager;
	TSharedPtr<FTabManager::FLayout> Layout;
	bool			bShouldImport;
	
	
	//Scene tab variables
	TSharedPtr<SFbxSceneTreeView> SceneTabTreeview;
	TSharedPtr<IDetailsView> SceneTabDetailsView;

	//Material tab Variables
	TSharedPtr<SFbxSceneMaterialsListView> MaterialsTabListView;
	FbxTextureInfoArray TexturesArray;
	FString MaterialBasePath;

	//Shared the options name between staticmesh and skeletalmesh
	FbxOverrideNameOptionsArray OverrideNameOptions;

	//StaticMesh tab Variables
	TSharedPtr<SFbxSceneStaticMeshListView> StaticMeshTabListView;
	TSharedPtr<IDetailsView> StaticMeshTabDetailsView;

	//SkeletalMesh tab Variables
	TSharedPtr<SFbxSceneSkeletalMeshListView> SkeletalMeshTabListView;
	TSharedPtr<IDetailsView> SkeletalMeshTabDetailsView;

	//Scene Reimport tab variables
	TSharedPtr<SFbxReimportSceneTreeView> SceneReimportTreeview;
	TSharedPtr<IDetailsView> SceneReimportTabDetailsView;

	//StaticMesh Reimport tab variables
	TSharedPtr<SFbxSceneStaticMeshReimportListView> StaticMeshReimportListView;
	TSharedPtr<IDetailsView> StaticMeshReimportDetailsView;

	//SkeletalMesh tab Variables
	TSharedPtr<SFbxSceneSkeletalMeshReimportListView> SkeletalMeshReimportListView;
	TSharedPtr<IDetailsView> SkeletalMeshReimportDetailsView;
};
