// Copyright Epic Games, Inc. All Rights Reserved.


#include "SKeySelector.h"

#include "Containers/Map.h"
#include "Delegates/IDelegateInstance.h"
#include "Framework/Views/ITypedTableView.h"
#include "IDocumentation.h"
#include "Input/Events.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "SListViewSelectorDropdownMenu.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/CoreStyle.h"
#include "UObject/NameTypes.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

class ITableRow;
class SWidget;
struct FGeometry;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "KeySelector"

static const FString BigTooltipDocLink = TEXT("Shared/Editor/ProjectSettings");

class FKeyTreeInfo
{
public:
	/** This data item's children */
	TArray< TSharedPtr<FKeyTreeInfo> > Children;

private:
	/** This data item's name */
	FText Name;

	/** The actual key associated with this item */
	TSharedPtr<FKey> Key;

public:
	FKeyTreeInfo(FText InName, TSharedPtr<FKey> InKey)
	: Name(InName)
	, Key(InKey)
	{
	}

	FKeyTreeInfo(TSharedPtr<FKeyTreeInfo> InInfo)
	: Name(InInfo->Name)
	, Key(InInfo->Key)
	{
	}

	FText GetDescription() const
	{
		if (Key.IsValid())
		{
			return Key->GetDisplayName();
		}
		else
		{
			return Name;
		}
	}

	TSharedPtr<FKey> GetKey() const
	{
		return Key;
	}

	bool MatchesSearchTokens(const TArray<FString>& SearchTokens)
	{
		FString Description = GetDescription().ToString();

		for (auto Token : SearchTokens)
		{
			if (!Description.Contains(Token))
			{
				return false;
			}
		}

		return true;
	}
};

void SKeySelector::Construct(const FArguments& InArgs)
{
	SearchText = FText::GetEmpty();

	OnKeyChanged = InArgs._OnKeyChanged;
	CurrentKey = InArgs._CurrentKey;
	AllowChangeKey = InArgs._AllowKeyChange;

	TMap<FName, FKeyTreeItem> TreeRootsForCatgories;

	// Ensure that Gamepad, Keyboard, and Mouse will appear at the top of the list, other categories will dynamically get added as the keys are encountered
	TreeRootsForCatgories.Add(EKeys::NAME_GamepadCategory, *new (KeyTreeRoot) FKeyTreeItem(MakeShareable(new FKeyTreeInfo(EKeys::GetMenuCategoryDisplayName(EKeys::NAME_GamepadCategory), nullptr))));
	TreeRootsForCatgories.Add(EKeys::NAME_KeyboardCategory, *new (KeyTreeRoot) FKeyTreeItem(MakeShareable(new FKeyTreeInfo(EKeys::GetMenuCategoryDisplayName(EKeys::NAME_KeyboardCategory), nullptr))));
	TreeRootsForCatgories.Add(EKeys::NAME_MouseCategory, *new (KeyTreeRoot) FKeyTreeItem(MakeShareable(new FKeyTreeInfo(EKeys::GetMenuCategoryDisplayName(EKeys::NAME_MouseCategory), nullptr))));

	TArray<FKey> AllKeys;
	EKeys::GetAllKeys(AllKeys);

	for (FKey Key : AllKeys)
	{
		if (Key.IsBindableToActions() && (!InArgs._FilterBlueprintBindable || Key.IsBindableInBlueprints()))
		{
			const FName KeyMenuCategory = Key.GetMenuCategory();
			FKeyTreeItem* KeyCategory = TreeRootsForCatgories.Find(KeyMenuCategory);
			if (KeyCategory == nullptr)
			{
				KeyCategory = new (KeyTreeRoot) FKeyTreeItem(MakeShareable(new FKeyTreeInfo(EKeys::GetMenuCategoryDisplayName(KeyMenuCategory), nullptr)));
				TreeRootsForCatgories.Add(KeyMenuCategory, *KeyCategory);
			}
			(*KeyCategory)->Children.Add(MakeShareable(new FKeyTreeInfo(FText(), MakeShareable(new FKey(Key)))));
		}
	}

	// if we allow NoClear, add a "None" option to be able to clear out a binding
	if (InArgs._AllowClear)
	{
		new (KeyTreeRoot) FKeyTreeItem(MakeShareable(new FKeyTreeInfo(FText(), MakeShareable(new FKey(EKeys::Invalid)))));
	}

	TreeViewWidth = InArgs._TreeViewWidth;
	TreeViewHeight = InArgs._TreeViewHeight;
	CategoryFont = FCoreStyle::GetDefaultFontStyle("Bold", 9);
	KeyFont = FCoreStyle::GetDefaultFontStyle("Regular", 10);

	FilteredKeyTreeRoot = KeyTreeRoot;

	this->ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SButton)
			.PressMethod(EButtonPressMethod::DownAndUp)
			.ToolTipText(this, &SKeySelector::GetKeyTooltip)
			.OnClicked(this, &SKeySelector::ListenForInput)
			.IsEnabled(this, &SKeySelector::CanChangeKey)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(SImage)
					.Image(this, &SKeySelector::GetKeyIconImage)
					.ColorAndOpacity(this, &SKeySelector::GetKeyIconColor)
				]
			]
		]
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		[
			SAssignNew(KeyComboButton, SComboButton)
			.OnGetMenuContent(this, &SKeySelector::GetMenuContent)
			.ContentPadding(0.0f)
			.ToolTipText(this, &SKeySelector::GetKeyDescriptionToolTip)	// Longer key descriptions can overrun the visible space in the combo button if the parent width is constrained, so we reflect them in the tooltip too.
			.ButtonContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(2.f, 0.f)
				.AutoWidth()
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				[
					SNew(STextBlock)
					.Text(this, &SKeySelector::GetKeyDescription)
					.Font(InArgs._Font)
				]
			]
		]

	];
}

//=======================================================================
// Attribute Helpers

FText SKeySelector::GetKeyDescription() const
{
	TOptional<FKey> CurrentKeyValue = CurrentKey.Get();
	if (CurrentKeyValue.IsSet())
	{
		return CurrentKeyValue.GetValue().GetDisplayName();
	}
	return LOCTEXT("MultipleValues", "Multiple Values");
}

FText SKeySelector::GetKeyDescriptionToolTip() const
{
	TOptional<FKey> CurrentKeyValue = CurrentKey.Get();
	if (CurrentKeyValue.IsSet())
	{
		return CurrentKeyValue.GetValue().GetDisplayName();
	}
	return LOCTEXT("MultipleValues", "Multiple Values");
}

const FSlateBrush* SKeySelector::GetKeyIconImage() const
{
	TOptional<FKey> CurrentKeyValue = CurrentKey.Get();
	if (CurrentKeyValue.IsSet())
	{
		const FKey& Key = CurrentKeyValue.GetValue();
		if (Key.IsValid() && (Key.IsDeprecated() || !Key.IsBindableToActions()))
		{
			return FAppStyle::GetBrush("Icons.Warning");
		}
		return GetIconFromKey(CurrentKeyValue.GetValue());
	}
	return nullptr;
}

FText SKeySelector::GetKeyTooltip() const
{
	if(!bEnabledFromKeyStructCustomization)
	{
		return DisabledSelectorToolTip;
	}
	
	TOptional<FKey> CurrentKeyValue = CurrentKey.Get();
	if (CurrentKeyValue.IsSet())
	{
		const FKey& Key = CurrentKeyValue.GetValue();
		if (Key.IsValid())
		{
			if (Key.IsDeprecated())
			{
				return LOCTEXT("KeySelectorDeprecated", "The selected key has been deprecated.");
			}
			else if (!Key.IsBindableToActions())
			{
				return LOCTEXT("KeySelectorNotBindable", "The selected key can't be bound to actions.");
			}
		}
	}
	return LOCTEXT("KeySelector", "Select the key value.");
}

FSlateColor SKeySelector::GetKeyIconColor() const
{
	// Orange when listening for input
	return bListenForNextInput ? FLinearColor(0.953f, 0.612f, 0.071f) : FLinearColor::White;
}

//=======================================================================
// Input Capture Helpers

FReply SKeySelector::ListenForInput()
{
	if (!bListenForNextInput && CanChangeKey())
	{
		bListenForNextInput = true;
		return FReply::Handled().CaptureMouse(SharedThis(this)).SetUserFocus(SharedThis(this));
	}
	return FReply::Unhandled();
}

FReply SKeySelector::ProcessHeardInput(FKey KeyHeard)
{
	ensure(CanChangeKey());
	
	if (bListenForNextInput)	// TODO: Unnecessary. Keep it for safety?
	{
		const FScopedTransaction Transaction(LOCTEXT("ChangeKey", "Change Key Value"));
		OnKeyChanged.ExecuteIfBound(MakeShareable(new FKey(KeyHeard)));

		bListenForNextInput = false;
		return FReply::Handled().ReleaseMouseCapture().ClearUserFocus(EFocusCause::Cleared);
	}
	return FReply::Unhandled();
}

FReply SKeySelector::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (bListenForNextInput)
	{
		// In the case of axis inputs emulating button key presses we ignore the key press in favor of the axis. Key variants will need to be set from the combo box if required.
		if (InKeyEvent.GetKey().IsButtonAxis())
		{
			return FReply::Unhandled();
		}

		return ProcessHeardInput(InKeyEvent.GetKey());
	}
	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

FReply SKeySelector::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (bListenForNextInput)
	{
		return ProcessHeardInput(MouseEvent.GetEffectingButton());
	}
	return SCompoundWidget::OnMouseButtonDown(MyGeometry, MouseEvent);
}

FReply SKeySelector::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (bListenForNextInput)
	{
		// We prefer to bind the axis over EKeys::MouseWheelUp/Down.
		return ProcessHeardInput(EKeys::MouseWheelAxis);
	}
	return SCompoundWidget::OnMouseWheel(MyGeometry, MouseEvent);
}

FReply SKeySelector::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// TODO: Support detecting mouse movement?
	// This is difficult to get right for all users and may lead to accidental mouse axis selections, so we'll ignore it for now.
	return SCompoundWidget::OnMouseMove(MyGeometry, MouseEvent);
}

FReply SKeySelector::OnAnalogValueChanged(const FGeometry& MyGeometry, const FAnalogInputEvent& InAnalogInputEvent)
{
	if (bListenForNextInput)
	{
		// We require a (substantial) minimum axis value before triggering the axis to account for dead zones and improve axes selection.
		if (FMath::Abs(InAnalogInputEvent.GetAnalogValue()) < 0.5f)
		{
			return FReply::Handled();
		}

		return ProcessHeardInput(InAnalogInputEvent.GetKey());
	}
	return SCompoundWidget::OnAnalogValueChanged(MyGeometry, InAnalogInputEvent);
}

//=======================================================================
// Key TreeView Support
TSharedRef<ITableRow> SKeySelector::GenerateKeyTreeRow(FKeyTreeItem InItem, const TSharedRef<STableViewBase>& OwnerTree)
{
	const bool bIsCategory = !InItem->GetKey().IsValid();
	const FText Description = InItem->GetDescription();

	// Determine the best icon the to represents this item
	const FSlateBrush* IconBrush = nullptr;
	if (InItem->GetKey().IsValid())
	{
		IconBrush = GetIconFromKey(*InItem->GetKey().Get());
	}

	return SNew(SComboRow<FKeyTreeItem>, OwnerTree)
		.ToolTip(IDocumentation::Get()->CreateToolTip(Description, nullptr, BigTooltipDocLink, Description.ToString()))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(1.f)
			[
				SNew(SImage)
				.Image(IconBrush)
				.Visibility(bIsCategory ? EVisibility::Collapsed : EVisibility::Visible)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(1.f)
			[
				SNew(STextBlock)
				.Text(Description)
				.HighlightText(SearchText)
				.Font(bIsCategory ? CategoryFont : KeyFont)
			]
		];
}

void SKeySelector::OnKeySelectionChanged(FKeyTreeItem Selection, ESelectInfo::Type SelectInfo)
{
	// When the user is navigating, do not act upon the selection change
	if (SelectInfo == ESelectInfo::OnNavigation)
	{
		return;
	}

	// Only handle selection for non-read only items, since STreeViewItem doesn't actually support read-only
	if (Selection.IsValid() && CanChangeKey())
	{
		if (Selection->GetKey().IsValid())
		{
			const FScopedTransaction Transaction(LOCTEXT("ChangeKey", "Change Key Value"));

			KeyComboButton->SetIsOpen(false);

			OnKeyChanged.ExecuteIfBound(Selection->GetKey());
		}
		else
		{
			// Expand / contract the category, if applicable
			if (Selection->Children.Num() > 0)
			{
				const bool bIsExpanded = KeyTreeView->IsItemExpanded(Selection);
				KeyTreeView->SetItemExpansion(Selection, !bIsExpanded);

				if (SelectInfo == ESelectInfo::OnMouseClick)
				{
					KeyTreeView->ClearSelection();
				}
			}
		}
	}
}

void SKeySelector::GetKeyChildren(FKeyTreeItem InItem, TArray<FKeyTreeItem>& OutChildren)
{
	OutChildren = InItem->Children;
}

bool SKeySelector::CanChangeKey() const
{
	return AllowChangeKey.Get();
}

TSharedRef<SWidget>	SKeySelector::GetMenuContent()
{
	if (!MenuContent.IsValid())
	{
		// Pre-build the tree view and search box as it is needed as a parameter for the context menu's container.
		SAssignNew(KeyTreeView, SKeyTreeView)
			.TreeItemsSource(&FilteredKeyTreeRoot)
			.SelectionMode(ESelectionMode::Single)
			.OnGenerateRow(this, &SKeySelector::GenerateKeyTreeRow)
			.OnSelectionChanged(this, &SKeySelector::OnKeySelectionChanged)
			.OnGetChildren(this, &SKeySelector::GetKeyChildren)
			.IsEnabled(this, &SKeySelector::CanChangeKey);

		SAssignNew(FilterTextBox, SSearchBox)
			.OnTextChanged(this, &SKeySelector::OnFilterTextChanged)
			.OnTextCommitted(this, &SKeySelector::OnFilterTextCommitted);

		MenuContent = SNew(SListViewSelectorDropdownMenu<FKeyTreeItem>, FilterTextBox, KeyTreeView)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(4.f, 4.f, 4.f, 4.f)
				[
					FilterTextBox.ToSharedRef()
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(4.f, 4.f, 4.f, 4.f)
				[
					SNew(SBox)
					.HeightOverride(TreeViewHeight)
					.WidthOverride(TreeViewWidth)
					[
						KeyTreeView.ToSharedRef()
					]
				]
			];


		KeyComboButton->SetMenuContentWidgetToFocus(FilterTextBox);
	}
	else
	{
		// Clear the selection in such a way as to also clear the keyboard selector
		KeyTreeView->SetSelection(NULL, ESelectInfo::OnNavigation);
		KeyTreeView->ClearExpandedItems();
	}

	// Clear the filter text box with each opening
	if (FilterTextBox.IsValid())
	{
		FilterTextBox->SetText(FText::GetEmpty());
	}

	return MenuContent.ToSharedRef();
}

//=======================================================================
// Search Support
void SKeySelector::OnFilterTextChanged(const FText& NewText)
{
	SearchText = NewText;
	FilteredKeyTreeRoot.Empty();

	TArray<FString> Tokens;
	GetSearchTokens(SearchText.ToString(), Tokens);

	GetChildrenMatchingSearch(Tokens, KeyTreeRoot, FilteredKeyTreeRoot);
	KeyTreeView->RequestTreeRefresh();

	// Select the first non-category item
	auto SelectedItems = KeyTreeView->GetSelectedItems();
	if (FilteredKeyTreeRoot.Num() > 0)
	{
		// Categories have children, we don't want to select categories
		if (FilteredKeyTreeRoot[0]->Children.Num() > 0)
		{
			KeyTreeView->SetSelection(FilteredKeyTreeRoot[0]->Children[0], ESelectInfo::OnNavigation);
		}
		else
		{
			KeyTreeView->SetSelection(FilteredKeyTreeRoot[0], ESelectInfo::OnNavigation);
		}
	}
}

void SKeySelector::OnFilterTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		auto SelectedItems = KeyTreeView->GetSelectedItems();
		if (SelectedItems.Num() > 0)
		{
			KeyTreeView->SetSelection(SelectedItems[0]);
		}
	}
}

void SKeySelector::GetSearchTokens(const FString& SearchString, TArray<FString>& OutTokens) const
{
	if (SearchString.Contains("\"") && SearchString.ParseIntoArray(OutTokens, TEXT("\""), true) > 0)
	{
		for (auto &TokenIt : OutTokens)
		{
			// we have the token, we don't need the quotes anymore, they'll just confused the comparison later on
			TokenIt = TokenIt.TrimQuotes();
			// We remove the spaces as all later comparison strings will also be de-spaced
			TokenIt = TokenIt.Replace(TEXT(" "), TEXT(""));
		}

		// due to being able to handle multiple quoted blocks like ("Make Epic" "Game Now") we can end up with
		// and empty string between (" ") blocks so this simply removes them
		struct FRemoveMatchingStrings
		{
			bool operator()(const FString& RemovalCandidate) const
			{
				return RemovalCandidate.IsEmpty();
			}
		};
		OutTokens.RemoveAll(FRemoveMatchingStrings());
	}
	else
	{
		// unquoted search equivalent to a match-any-of search
		SearchString.ParseIntoArray(OutTokens, TEXT(" "), true);
	}
}

bool SKeySelector::GetChildrenMatchingSearch(const TArray<FString>& InSearchTokens, const TArray<FKeyTreeItem>& UnfilteredList, TArray<FKeyTreeItem>& OutFilteredList)
{
	bool bReturnVal = false;

	for (auto it = UnfilteredList.CreateConstIterator(); it; ++it)
	{
		FKeyTreeItem Item = *it;
		FKeyTreeItem NewInfo = MakeShareable(new FKeyTreeInfo(Item));
		TArray<FKeyTreeItem> ValidChildren;

		// Have to run GetChildrenMatchingSearch first, so that we can make sure we get valid children for the list!
		if (GetChildrenMatchingSearch(InSearchTokens, Item->Children, ValidChildren)
			|| InSearchTokens.Num() == 0
			|| Item->MatchesSearchTokens(InSearchTokens))
		{
			NewInfo->Children = ValidChildren;
			OutFilteredList.Add(NewInfo);

			KeyTreeView->SetItemExpansion(NewInfo, InSearchTokens.Num() > 0);

			bReturnVal = true;
		}
	}

	return bReturnVal;
}

const FSlateBrush* SKeySelector::GetIconFromKey(FKey Key) const
{
	return FAppStyle::GetBrush(EKeys::GetMenuCategoryPaletteIcon(Key.GetMenuCategory()));
}

#undef LOCTEXT_NAMESPACE
