// Copyright Epic Games, Inc. All Rights Reserved.

#include "SScalabilitySettings.h"
#include "Widgets/SWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSlider.h"
#include "Styling/AppStyle.h"
#include "Editor.h"
#include "Settings/EditorSettings.h"
#include "Editor/EditorPerformanceSettings.h"
#include "SceneUtils.h"
#include "LegacyScreenPercentageDriver.h"
#include "Engine/GameViewportClient.h"
#include "UnrealClient.h"

#define LOCTEXT_NAMESPACE "EngineScalabiltySettings"

// static
bool SScalabilitySettings::IsPlayInEditor()
{
	const TIndirectArray<FWorldContext>&WorldContexts = GEngine->GetWorldContexts();
	for (const FWorldContext& Context : WorldContexts)
	{
		if (Context.WorldType == EWorldType::PIE && GEditor && !GEditor->bIsSimulatingInEditor)
		{
			return true;
		}
	}
	return false;
}

ECheckBoxState SScalabilitySettings::IsGroupQualityLevelSelected(const TCHAR* InGroupName, int32 InQualityLevel) const
{
	int32 QualityLevel = -1;

	if (FCString::Strcmp(InGroupName, TEXT("ResolutionQuality")) == 0) { QualityLevel = static_cast<int32>(CachedQualityLevels.ResolutionQuality); }
	else if (FCString::Strcmp(InGroupName, TEXT("ViewDistanceQuality")) == 0) QualityLevel = CachedQualityLevels.ViewDistanceQuality;
	else if (FCString::Strcmp(InGroupName, TEXT("AntiAliasingQuality")) == 0) QualityLevel = CachedQualityLevels.AntiAliasingQuality;
	else if (FCString::Strcmp(InGroupName, TEXT("PostProcessQuality")) == 0) QualityLevel = CachedQualityLevels.PostProcessQuality;
	else if (FCString::Strcmp(InGroupName, TEXT("ShadowQuality")) == 0) QualityLevel = CachedQualityLevels.ShadowQuality;
	else if (FCString::Strcmp(InGroupName, TEXT("GlobalIlluminationQuality")) == 0) QualityLevel = CachedQualityLevels.GlobalIlluminationQuality;
	else if (FCString::Strcmp(InGroupName, TEXT("ReflectionQuality")) == 0) QualityLevel = CachedQualityLevels.ReflectionQuality;
	else if (FCString::Strcmp(InGroupName, TEXT("TextureQuality")) == 0) QualityLevel = CachedQualityLevels.TextureQuality;
	else if (FCString::Strcmp(InGroupName, TEXT("EffectsQuality")) == 0) QualityLevel = CachedQualityLevels.EffectsQuality;
	else if (FCString::Strcmp(InGroupName, TEXT("FoliageQuality")) == 0) QualityLevel = CachedQualityLevels.FoliageQuality;
	else if (FCString::Strcmp(InGroupName, TEXT("ShadingQuality")) == 0) QualityLevel = CachedQualityLevels.ShadingQuality;
 	else if (FCString::Strcmp(InGroupName, TEXT("LandscapeQuality")) == 0) QualityLevel = CachedQualityLevels.LandscapeQuality;

	return (QualityLevel == InQualityLevel) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SScalabilitySettings::OnGroupQualityLevelChanged(ECheckBoxState NewState, const TCHAR* InGroupName, int32 InQualityLevel)
{
	if (FCString::Strcmp(InGroupName, TEXT("ResolutionQuality")) == 0) CachedQualityLevels.ResolutionQuality = static_cast<float>(InQualityLevel);
	else if (FCString::Strcmp(InGroupName, TEXT("ViewDistanceQuality")) == 0) CachedQualityLevels.ViewDistanceQuality = InQualityLevel;
	else if (FCString::Strcmp(InGroupName, TEXT("AntiAliasingQuality")) == 0) CachedQualityLevels.AntiAliasingQuality = InQualityLevel;
	else if (FCString::Strcmp(InGroupName, TEXT("PostProcessQuality")) == 0) CachedQualityLevels.PostProcessQuality = InQualityLevel;
	else if (FCString::Strcmp(InGroupName, TEXT("ShadowQuality")) == 0) CachedQualityLevels.ShadowQuality = InQualityLevel;
	else if (FCString::Strcmp(InGroupName, TEXT("GlobalIlluminationQuality")) == 0) CachedQualityLevels.GlobalIlluminationQuality = InQualityLevel;
	else if (FCString::Strcmp(InGroupName, TEXT("ReflectionQuality")) == 0) CachedQualityLevels.ReflectionQuality = InQualityLevel;
	else if (FCString::Strcmp(InGroupName, TEXT("TextureQuality")) == 0) CachedQualityLevels.TextureQuality = InQualityLevel;
	else if (FCString::Strcmp(InGroupName, TEXT("EffectsQuality")) == 0) CachedQualityLevels.EffectsQuality = InQualityLevel;
	else if (FCString::Strcmp(InGroupName, TEXT("FoliageQuality")) == 0) CachedQualityLevels.FoliageQuality = InQualityLevel;
	else if (FCString::Strcmp(InGroupName, TEXT("ShadingQuality")) == 0) CachedQualityLevels.ShadingQuality = InQualityLevel;
	else if (FCString::Strcmp(InGroupName, TEXT("LandscapeQuality")) == 0) CachedQualityLevels.LandscapeQuality = InQualityLevel;

	Scalability::SetQualityLevels(CachedQualityLevels);
	Scalability::SaveState(GEditorSettingsIni);
	GEditor->RedrawAllViewports();
	GEditor->RedrawLevelEditingViewports();
}

void SScalabilitySettings::OnResolutionScaleChanged(float InValue)
{
	if (!SScalabilitySettings::IsResolutionScaleEditable())
	{
		return;
	}

	CachedQualityLevels.ResolutionQuality = FMath::Lerp(Scalability::MinResolutionScale, Scalability::MaxResolutionScale, FMath::Clamp(InValue, 0.0f, 1.0f));

	Scalability::SetQualityLevels(CachedQualityLevels);
	Scalability::SaveState(GEditorSettingsIni);
	GEditor->RedrawAllViewports();
	GEditor->RedrawLevelEditingViewports();
}

float SScalabilitySettings::GetResolutionScale() const
{
	return (float)(CachedQualityLevels.ResolutionQuality - Scalability::MinResolutionScale) / (float)(Scalability::MaxResolutionScale - Scalability::MinResolutionScale);
}

FText SScalabilitySettings::GetResolutionScaleString() const
{
	return CachedQualityLevels.ResolutionQuality >= Scalability::MinResolutionScale ? FText::AsPercent(CachedQualityLevels.ResolutionQuality / 100.0f) : FText();
}

FReply SScalabilitySettings::OnResolutionPresetClicked(int32 PresetId)
{
	if (!SScalabilitySettings::IsResolutionScaleEditable())
	{
		return FReply::Handled();
	}

	TArray<Scalability::FResolutionPreset> ResolutionPresets = Scalability::GetResolutionPresets();
	CachedQualityLevels.ResolutionQuality = ResolutionPresets[PresetId].ResolutionQuality;

	Scalability::SetQualityLevels(CachedQualityLevels);
	Scalability::SaveState(GEditorSettingsIni);
	GEditor->RedrawAllViewports();
	GEditor->RedrawLevelEditingViewports();

	return FReply::Handled();
}

// static
bool SScalabilitySettings::IsResolutionScaleEditable()
{
	if (!SScalabilitySettings::IsPlayInEditor())
	{
		return false;
	}

	static IConsoleVariable* CVarScreenPercentage = IConsoleManager::Get().FindConsoleVariable(TEXT("r.ScreenPercentage"));
	if ((uint32(CVarScreenPercentage->GetFlags()) & uint32(ECVF_SetByMask)) > uint32(ECVF_SetByScalability))
	{
		return false;
	}
	
	return true;
}

TSharedRef<SWidget> SScalabilitySettings::MakeButtonWidget(const FText& InName, const TCHAR* InGroupName, int32 InQualityLevel, const FText& InToolTip)
{
	return SNew(SCheckBox)
		.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
		.OnCheckStateChanged(this, &SScalabilitySettings::OnGroupQualityLevelChanged, InGroupName, InQualityLevel)
		.IsChecked(this, &SScalabilitySettings::IsGroupQualityLevelSelected, InGroupName, InQualityLevel)
		.ToolTipText(InToolTip)
		.Content()
		[
			SNew(STextBlock)
			.Text(InName)
		];
}

TSharedRef<SWidget> SScalabilitySettings::MakeHeaderButtonWidget(const FText& InName, int32 InQualityLevel, const FText& InToolTip, Scalability::EQualityLevelBehavior Behavior)
{
	return SNew(SCheckBox)
		.Style(FAppStyle::Get(), "DetailsView.SectionButton")
		.OnCheckStateChanged(this, &SScalabilitySettings::OnHeaderClicked, InQualityLevel, Behavior)
		.IsChecked(this, &SScalabilitySettings::GetHeaderChecked, InQualityLevel, Behavior)
		.ToolTipText(InToolTip)
		.Content()
		[
			SNew(STextBlock)
			.Text(InName)
		];
}

TSharedRef<SWidget> SScalabilitySettings::MakeAutoButtonWidget()
{
	return SNew(SButton)
		.OnClicked(this, &SScalabilitySettings::OnAutoClicked)
		.ToolTipText(LOCTEXT("AutoButtonTooltip", "We test your system and try to find the most suitable settings"))
		.Content()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("AutoLabel", "Auto"))
		];
}

TSharedRef<SWidget> SScalabilitySettings::MakeDefaultButtonWidget()
{
	return SNew(SCheckBox)
		.Style(FAppStyle::Get(), "DetailsView.SectionButton")
		.OnCheckStateChanged(this, &SScalabilitySettings::OnDefaultClicked)
		.IsChecked(this, &SScalabilitySettings::GetDefaultCheckState)
		.ToolTipText_Lambda([]
		{
			if (GEditor && GEditor->IsPreviewPlatformActive())
			{
				return LOCTEXT("DefaultButtonWithProfileTooltip", "Resets the Quality Levels to what the Device Profile sets");
			}
			return LOCTEXT("DefaultButtonTooltip", "Resets the Quality Levels to the default.");
		})
		.Content()
		[
			SNew(STextBlock)
				.Text(LOCTEXT("DefaultLabel", "Default"))
		];
}

void SScalabilitySettings::OnHeaderClicked(ECheckBoxState CheckState, int32 InQualityLevel, Scalability::EQualityLevelBehavior Behavior)
{
	Scalability::FQualityLevels LevelCounts = Scalability::GetQualityLevelCounts();
	switch (Behavior)
	{
		case Scalability::EQualityLevelBehavior::ERelativeToMax:
			CachedQualityLevels.SetFromSingleQualityLevelRelativeToMax(InQualityLevel);
			break;		
		case Scalability::EQualityLevelBehavior::EAbsolute:
		default:
			CachedQualityLevels.SetFromSingleQualityLevel(InQualityLevel);
			break;
	}
	Scalability::SetQualityLevels(CachedQualityLevels);
	Scalability::SaveState(GEditorSettingsIni);
	GEditor->RedrawAllViewports();
	GEditor->RedrawLevelEditingViewports();
}

ECheckBoxState SScalabilitySettings::GetHeaderChecked(int32 InQualityLevel, Scalability::EQualityLevelBehavior Behavior) const
{
	Scalability::FQualityLevels CurrentLevels = Scalability::GetQualityLevels();
	
	Scalability::FQualityLevels TargetLevels;
	switch (Behavior)
	{
	case Scalability::EQualityLevelBehavior::ERelativeToMax:
		TargetLevels.SetFromSingleQualityLevelRelativeToMax(InQualityLevel);
		break;		
	case Scalability::EQualityLevelBehavior::EAbsolute:
	default:
		TargetLevels.SetFromSingleQualityLevel(InQualityLevel);
		break;
	}
	
	if (FMath::IsNearlyZero(CurrentLevels.ResolutionQuality))
	{
		// Resolution quality gets initialized to 0.0 in a fresh project, thereby using project defaults.
		// This should not affect whether a column header is highlighted.
		CurrentLevels.ResolutionQuality = 0.f;
		TargetLevels.ResolutionQuality = 0.f;
	}
	
	return CurrentLevels == TargetLevels ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

FReply SScalabilitySettings::OnAutoClicked()
{
	auto* Settings = GetMutableDefault<UEditorSettings>();
	Settings->AutoApplyScalabilityBenchmark();
	Settings->LoadScalabilityBenchmark();

	CachedQualityLevels = Settings->EngineBenchmarkResult;

	GEditor->RedrawAllViewports();
	GEditor->RedrawLevelEditingViewports();
	return FReply::Handled();
}

void SScalabilitySettings::OnDefaultClicked(ECheckBoxState CheckState)
{
	Scalability::ResetQualityLevelsToDefault();
	CachedQualityLevels = Scalability::GetQualityLevels();
	Scalability::SetQualityLevels(CachedQualityLevels);
	Scalability::SaveState(GEditorSettingsIni);

	GEditor->RedrawAllViewports();
	GEditor->RedrawLevelEditingViewports();
}

ECheckBoxState SScalabilitySettings::GetDefaultCheckState() const
{
	Scalability::FQualityLevels CurrentLevels = Scalability::GetQualityLevels();
	Scalability::FQualityLevels TargetLevels = Scalability::GetDefaultQualityLevels();

	if (FMath::IsNearlyZero(CurrentLevels.ResolutionQuality))
	{
		// Resolution quality gets initialized to 0.0 in a fresh project, thereby using project defaults.
		// This should still show up as "defaulted"
		CurrentLevels.ResolutionQuality = 0.f;
		TargetLevels.ResolutionQuality = 0.f;
	}

	return CurrentLevels == TargetLevels ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

SGridPanel::FSlot::FSlotArguments SScalabilitySettings::MakeGridSlot(int32 InCol, int32 InRow, int32 InColSpan /*= 1*/, int32 InRowSpan /*= 1*/)
{
	float PaddingH = 2.0f;
	float PaddingV = InRow == 0 ? 8.0f : 2.0f;
	return MoveTemp(SGridPanel::Slot(InCol, InRow)
		.Padding(PaddingH, PaddingV)
		.RowSpan(InRowSpan)
		.ColumnSpan(InColSpan));
}

ECheckBoxState SScalabilitySettings::IsMonitoringPerformance() const
{
	const bool bMonitorEditorPerformance = GetDefault<UEditorPerformanceSettings>()->bMonitorEditorPerformance;
	return bMonitorEditorPerformance ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SScalabilitySettings::OnMonitorPerformanceChanged(ECheckBoxState NewState)
{
	const bool bNewEnabledState = ( NewState == ECheckBoxState::Checked );

	auto* Settings = GetMutableDefault<UEditorPerformanceSettings>();
	Settings->bMonitorEditorPerformance = bNewEnabledState;
	Settings->PostEditChange();
	Settings->SaveConfig();
}

void SScalabilitySettings::AddButtonsToGrid(int32 X0, int32 Y0, TSharedRef<SGridPanel> ButtonMatrix, const FText* FiveNameArray, int32 ButtonCount, const TCHAR* GroupName, const FText& TooltipShape)
{
	const int32 ExpectedNamesSize = 5;
	const bool bCanUseNameArray = (FiveNameArray != nullptr);
	const bool bCanAllUseNameArray = (ButtonCount == ExpectedNamesSize) && bCanUseNameArray;
	const int32 CineButtonName = ExpectedNamesSize - 1;
	const int32 CineButtonIndex = ButtonCount - 1;

	for (int32 ButtonIndex = 0; ButtonIndex < ButtonCount; ++ButtonIndex)
	{
		const bool bCineButton = ButtonIndex == CineButtonIndex;
		const bool bUseNameArray = bCanUseNameArray && (bCanAllUseNameArray || bCineButton);
		const int32 ButtonNameIndex = bCineButton ? CineButtonName : ButtonIndex;

		const FText ButtonLabel = bUseNameArray ? FiveNameArray[ButtonNameIndex] : FText::AsNumber(ButtonIndex);
		const FText ButtonTooltip = FText::Format(TooltipShape, ButtonLabel);

		ButtonMatrix->AddSlot(X0 + ButtonIndex, Y0)
		[
			MakeButtonWidget(ButtonLabel, GroupName, ButtonIndex, ButtonTooltip)
		];
	}
}

//static
FFormatNamedArguments SScalabilitySettings::GetScreenPercentageFormatArguments(const UGameViewportClient* ViewportClient)
{
	FFormatNamedArguments FormatArguments;
	if (ViewportClient == nullptr || !SScalabilitySettings::IsPlayInEditor())
	{
		FormatArguments.Add(TEXT("CurrentScreenPercentage"), LOCTEXT("ScreenPercentageNotInPIE", "Not currently in PIE"));

		FormatArguments.Add(TEXT("ResolutionFromTo"), FText());
		FormatArguments.Add(TEXT("ViewportMode"), FText());
		FormatArguments.Add(TEXT("SettingSource"), FText());
		FormatArguments.Add(TEXT("Setting"), FText());

		return FormatArguments;
	}

	const FIntPoint DisplayResolution = ViewportClient->Viewport->GetSizeXY();

	EViewStatusForScreenPercentage ViewportRenderingMode = ViewportClient->GetViewStatusForScreenPercentage();
	FormatArguments.Add(TEXT("ViewportMode"), UEnum::GetDisplayValueAsText(ViewportRenderingMode));

	static IConsoleVariable* CVarScreenPercentage = IConsoleManager::Get().FindConsoleVariable(TEXT("r.ScreenPercentage"));
	const TCHAR* CVarScreenPercentageSetBy = GetConsoleVariableSetByName(CVarScreenPercentage->GetFlags());
	float CVarScreenPercentageValue = CVarScreenPercentage->GetFloat();
	if (EConsoleVariableFlags(uint32(CVarScreenPercentage->GetFlags()) & uint32(ECVF_SetByMask)) > ECVF_SetByScalability)
	{
		if (CVarScreenPercentageValue > 0.0f)
		{
			FormatArguments.Add(TEXT("SettingSource"), FText::Format(
				LOCTEXT("ScreenPercentageCVarSetBy", "r.ScreenPercentage={0} set by {1}"),
				FText::AsNumber(CVarScreenPercentageValue),
				FText::FromString(CVarScreenPercentageSetBy)));
		}
		else
		{
			FormatArguments.Add(TEXT("SettingSource"), FText::Format(
				LOCTEXT("ScreenPercentageCVarSetByUsesProjectDefaults", "Project Settings' default due to r.ScreenPercentage={0} set by {1}"),
				FText::AsNumber(CVarScreenPercentageValue),
				FText::FromString(CVarScreenPercentageSetBy)));
		}
	}
	else
	{
		if (CVarScreenPercentageValue > 0.0f)
		{
			FormatArguments.Add(TEXT("SettingSource"), LOCTEXT("EngineScalabilitySettings", "Engine Scalability Settings"));
		}
		else if (FStaticResolutionFractionHeuristic::FUserSettings::EditorOverridePIESettings())
		{
			FormatArguments.Add(TEXT("SettingSource"), LOCTEXT("ScreenPercentageEditorViewportDefaults", "Editor Preferences > Performance > Viewport Resolution"));
		}
		else
		{
			FormatArguments.Add(TEXT("SettingSource"), LOCTEXT("ScreenPercentageProjectDefaults", "Project Settings > Rendering > Default Screen Percentage"));
		}
	}

	// Get global view fraction.
	const FEngineShowFlags& EngineShowFlags = ViewportClient->EngineShowFlags;
	FStaticResolutionFractionHeuristic StaticHeuristic;
	StaticHeuristic.Settings.PullRunTimeRenderingSettings(ViewportRenderingMode);

	{
		StaticHeuristic.SecondaryViewFraction = 1.0; // TODO
		StaticHeuristic.TotalDisplayedPixelCount = DisplayResolution.X * DisplayResolution.Y;
		StaticHeuristic.DPIScale = ViewportClient->GetDPIScale();
	}

	EScreenPercentageMode FinalScreenPercentageMode = StaticHeuristic.Settings.Mode;
	if (FinalScreenPercentageMode == EScreenPercentageMode::BasedOnDPIScale)
	{
		FormatArguments.Add(TEXT("Setting"), LOCTEXT("ScreenPercentage_Setting_BasedOnDPIScale", "Based on OS's DPI scale"));
	}
	else if (FinalScreenPercentageMode == EScreenPercentageMode::BasedOnDisplayResolution)
	{
		FormatArguments.Add(TEXT("Setting"), LOCTEXT("ScreenPercentage_Setting_BasedOnDisplayResolution", "Based on display resolution"));
	}
	else
	{
		FormatArguments.Add(TEXT("Setting"), LOCTEXT("ScreenPercentage_Setting_Manual", "Manual"));
	}

	float FinalResolutionFraction = StaticHeuristic.ResolveResolutionFraction();
	float FinalScreenPercentage = FinalResolutionFraction * 100.0f;
	FormatArguments.Add(TEXT("CurrentScreenPercentage"), FText::FromString(FString::Printf(TEXT("%3.1f"), FMath::RoundToFloat(FinalScreenPercentage * 10.0f) / 10.0f)));

	{
		FIntPoint RenderingResolution;
		RenderingResolution.X = FMath::CeilToInt(DisplayResolution.X * FinalResolutionFraction);
		RenderingResolution.Y = FMath::CeilToInt(DisplayResolution.Y * FinalResolutionFraction);

		FormatArguments.Add(TEXT("ResolutionFromTo"), FText::FromString(FString::Printf(TEXT("%dx%d -> %dx%d"), RenderingResolution.X, RenderingResolution.Y, DisplayResolution.X, DisplayResolution.Y)));
	}

	return FormatArguments;
}

void SScalabilitySettings::Construct( const FArguments& InArgs )
{
	const float QualityColumnCoeff = 1.0f;

	auto TitleFont = FAppStyle::GetFontStyle(FName("Scalability.TitleFont"));
	auto GroupFont = FAppStyle::GetFontStyle(FName("Scalability.GroupFont"));

	TSharedPtr<SWidget> ScalabilityGroupsWidget;
	{
		const FText NamesLow(LOCTEXT("QualityLowLabel", "Low"));
		const FText NamesMedium(LOCTEXT("QualityMediumLabel", "Medium"));
		const FText NamesHigh(LOCTEXT("QualityHighLabel", "High"));
		const FText NamesEpic(LOCTEXT("QualityEpicLabel", "Epic"));
		const FText NamesCine(LOCTEXT("QualityCineLabel", "Cinematic"));
		const FText NamesAuto(LOCTEXT("QualityAutoLabel", "Auto"));

		const FText DistanceNear = LOCTEXT("ViewDistanceLabel2", "Near");
		const FText DistanceMedium = LOCTEXT("ViewDistanceLabel3", "Medium");
		const FText DistanceFar = LOCTEXT("ViewDistanceLabel4", "Far");
		const FText DistanceEpic = LOCTEXT("ViewDistanceLabel5", "Epic");
		const FText DistanceCinematic = LOCTEXT("ViewDistanceLabel6", "Cinematic");

		const FText FiveNames[5] = { NamesLow, NamesMedium, NamesHigh, NamesEpic, NamesCine };
		const FText FiveDistanceNames[5] = { DistanceNear, DistanceMedium, DistanceFar, DistanceEpic, DistanceCinematic };

		InitialQualityLevels = CachedQualityLevels = Scalability::GetQualityLevels();
		
		Scalability::FQualityLevels LevelCounts = Scalability::GetQualityLevelCounts();
		const int32 MaxLevelCount =
			FMath::Max(LevelCounts.ShadowQuality,
			FMath::Max(LevelCounts.GlobalIlluminationQuality,
			FMath::Max(LevelCounts.ReflectionQuality,
			FMath::Max(LevelCounts.TextureQuality,
			FMath::Max(LevelCounts.ViewDistanceQuality,
			FMath::Max(LevelCounts.EffectsQuality,
			FMath::Max(LevelCounts.FoliageQuality,
			FMath::Max(LevelCounts.ShadingQuality,
			FMath::Max(LevelCounts.PostProcessQuality,
			FMath::Max(LevelCounts.LandscapeQuality,LevelCounts.AntiAliasingQuality)
			)))))))));

		const int32 TotalWidth = MaxLevelCount + 1;
		const int32 ScalabilityFirstRowId = 1;

		ERHIFeatureLevel::Type FeatureLevel = GWorld ? GWorld->GetFeatureLevel() : GMaxRHIFeatureLevel;
		EAntiAliasingMethod AntiAliasingMethod = GetDefaultAntiAliasingMethod(FeatureLevel);
		FText AntiAliasingMethodShortName = FText::FromString(GetShortAntiAliasingName(AntiAliasingMethod));

		TSharedRef<SGridPanel> ButtonMatrix =
			SNew(SGridPanel)
			.FillColumn(0, QualityColumnCoeff)

			+MakeGridSlot(0,0) [ MakeDefaultButtonWidget() ]
			+MakeGridSlot(1,0) [ MakeHeaderButtonWidget(NamesLow, 0, LOCTEXT("QualityLow", "Set all groups to low quality"), Scalability::EQualityLevelBehavior::EAbsolute) ]
			+MakeGridSlot(2,0) [ MakeHeaderButtonWidget(NamesMedium, 3, LOCTEXT("QualityMedium", "Set all groups to medium quality"), Scalability::EQualityLevelBehavior::ERelativeToMax) ]
			+MakeGridSlot(3,0) [ MakeHeaderButtonWidget(NamesHigh, 2, LOCTEXT("QualityHigh", "Set all groups to high quality"), Scalability::EQualityLevelBehavior::ERelativeToMax) ]
			+MakeGridSlot(4,0) [ MakeHeaderButtonWidget(NamesEpic, 1, LOCTEXT("QualityEpic", "Set all groups to epic quality"), Scalability::EQualityLevelBehavior::ERelativeToMax) ]
			+MakeGridSlot(5,0) [ MakeHeaderButtonWidget(NamesCine, 0, LOCTEXT("QualityCinematic", "Set all groups to offline cinematic quality"), Scalability::EQualityLevelBehavior::ERelativeToMax)]
			+MakeGridSlot(6,0) [ MakeAutoButtonWidget() ]

			+MakeGridSlot(0,ScalabilityFirstRowId+0,TotalWidth,1) [ SNew (SBorder).BorderImage(FAppStyle::GetBrush("Scalability.RowBackground")) ]
			+MakeGridSlot(0,ScalabilityFirstRowId+1,TotalWidth,1) [ SNew (SBorder).BorderImage(FAppStyle::GetBrush("Scalability.RowBackground")) ]
			+MakeGridSlot(0,ScalabilityFirstRowId+2,TotalWidth,1) [ SNew (SBorder).BorderImage(FAppStyle::GetBrush("Scalability.RowBackground")) ]
			+MakeGridSlot(0,ScalabilityFirstRowId+3,TotalWidth,1) [ SNew (SBorder).BorderImage(FAppStyle::GetBrush("Scalability.RowBackground")) ]
			+MakeGridSlot(0,ScalabilityFirstRowId+4,TotalWidth,1) [ SNew (SBorder).BorderImage(FAppStyle::GetBrush("Scalability.RowBackground")) ]
			+MakeGridSlot(0,ScalabilityFirstRowId+5,TotalWidth,1) [ SNew (SBorder).BorderImage(FAppStyle::GetBrush("Scalability.RowBackground")) ]
			+MakeGridSlot(0,ScalabilityFirstRowId+6,TotalWidth,1) [ SNew (SBorder).BorderImage(FAppStyle::GetBrush("Scalability.RowBackground")) ]
			+MakeGridSlot(0,ScalabilityFirstRowId+7,TotalWidth,1) [ SNew (SBorder).BorderImage(FAppStyle::GetBrush("Scalability.RowBackground")) ]
			+MakeGridSlot(0,ScalabilityFirstRowId+8,TotalWidth,1) [ SNew (SBorder).BorderImage(FAppStyle::GetBrush("Scalability.RowBackground")) ]
			+MakeGridSlot(0,ScalabilityFirstRowId+9,TotalWidth,1) [ SNew (SBorder).BorderImage(FAppStyle::GetBrush("Scalability.RowBackground")) ]
			+MakeGridSlot(0,ScalabilityFirstRowId+10,TotalWidth,1) [ SNew (SBorder).BorderImage(FAppStyle::GetBrush("Scalability.RowBackground")) ]

			+MakeGridSlot(0,ScalabilityFirstRowId + 0) [ SNew(STextBlock).Text(LOCTEXT("ViewDistanceLabel1", "View Distance")).Font(GroupFont) ]
			+MakeGridSlot(0,ScalabilityFirstRowId + 1) [ SNew(STextBlock).Text(FText::Format(LOCTEXT("AntiAliasingQualityLabel1", "Anti-Aliasing ({0})"), AntiAliasingMethodShortName)).Font(GroupFont) ]
			+MakeGridSlot(0,ScalabilityFirstRowId + 2) [ SNew(STextBlock).Text(LOCTEXT("PostProcessQualityLabel1", "Post Processing")).Font(GroupFont) ]
			+MakeGridSlot(0,ScalabilityFirstRowId + 3) [ SNew(STextBlock).Text(LOCTEXT("ShadowLabel1", "Shadows")).Font(GroupFont) ]
			+MakeGridSlot(0,ScalabilityFirstRowId + 4) [ SNew(STextBlock).Text(LOCTEXT("GlobalIlluminationLabel1", "Global Illumination")).Font(GroupFont) ]
			+MakeGridSlot(0,ScalabilityFirstRowId + 5) [ SNew(STextBlock).Text(LOCTEXT("Reflection1", "Reflections")).Font(GroupFont) ]
			+MakeGridSlot(0,ScalabilityFirstRowId + 6) [ SNew(STextBlock).Text(LOCTEXT("TextureQualityLabel1", "Textures")).Font(GroupFont) ]
			+MakeGridSlot(0,ScalabilityFirstRowId + 7) [ SNew(STextBlock).Text(LOCTEXT("EffectsQualityLabel1", "Effects")).Font(GroupFont) ]
			+MakeGridSlot(0,ScalabilityFirstRowId + 8) [ SNew(STextBlock).Text(LOCTEXT("FoliageQualityLabel1", "Foliage")).Font(GroupFont) ]
			+MakeGridSlot(0,ScalabilityFirstRowId + 9) [ SNew(STextBlock).Text(LOCTEXT("ShadingQualityLabel1", "Shading")).Font(GroupFont) ]
			+MakeGridSlot(0,ScalabilityFirstRowId + 10) [ SNew(STextBlock).Text(LOCTEXT("LandscapeQualityLabel1", "Landscape")).Font(GroupFont) ]
			;

		AddButtonsToGrid(1, ScalabilityFirstRowId + 0, ButtonMatrix, FiveDistanceNames, LevelCounts.ViewDistanceQuality, TEXT("ViewDistanceQuality"), LOCTEXT("ViewDistanceQualityTooltip", "Set view distance to {0}"));
		AddButtonsToGrid(1, ScalabilityFirstRowId + 1, ButtonMatrix, FiveNames, LevelCounts.AntiAliasingQuality, TEXT("AntiAliasingQuality"), LOCTEXT("AntiAliasingQualityTooltip", "Set anti-aliasing quality to {0}"));
		AddButtonsToGrid(1, ScalabilityFirstRowId + 2, ButtonMatrix, FiveNames, LevelCounts.PostProcessQuality, TEXT("PostProcessQuality"), LOCTEXT("PostProcessQualityTooltip", "Set post processing quality to {0}"));
		AddButtonsToGrid(1, ScalabilityFirstRowId + 3, ButtonMatrix, FiveNames, LevelCounts.ShadowQuality, TEXT("ShadowQuality"), LOCTEXT("ShadowQualityTooltip", "Set shadow quality to {0}"));
		AddButtonsToGrid(1, ScalabilityFirstRowId + 4, ButtonMatrix, FiveNames, LevelCounts.GlobalIlluminationQuality, TEXT("GlobalIlluminationQuality"), LOCTEXT("GlobalIlluminationQualityTooltip", "Set Global Illumination quality to {0}"));
		AddButtonsToGrid(1, ScalabilityFirstRowId + 5, ButtonMatrix, FiveNames, LevelCounts.ReflectionQuality, TEXT("ReflectionQuality"), LOCTEXT("ReflectionQualityTooltip", "Set Reflection quality to {0}"));
		AddButtonsToGrid(1, ScalabilityFirstRowId + 6, ButtonMatrix, FiveNames, LevelCounts.TextureQuality, TEXT("TextureQuality"), LOCTEXT("TextureQualityTooltip", "Set texture quality to {0}"));
		AddButtonsToGrid(1, ScalabilityFirstRowId + 7, ButtonMatrix, FiveNames, LevelCounts.EffectsQuality, TEXT("EffectsQuality"), LOCTEXT("EffectsQualityTooltip", "Set effects quality to {0}"));
		AddButtonsToGrid(1, ScalabilityFirstRowId + 8, ButtonMatrix, FiveNames, LevelCounts.FoliageQuality, TEXT("FoliageQuality"), LOCTEXT("FoliageQualityTooltip", "Set foliage quality to {0}"));
		AddButtonsToGrid(1, ScalabilityFirstRowId + 9, ButtonMatrix, FiveNames, LevelCounts.ShadingQuality, TEXT("ShadingQuality"), LOCTEXT("ShadingQualityTooltip", "Set shading quality to {0}"));
		AddButtonsToGrid(1, ScalabilityFirstRowId + 10, ButtonMatrix, FiveNames, LevelCounts.LandscapeQuality, TEXT("LandscapeQuality"), LOCTEXT("LandscapeQualityTooltip", "Set landscape quality to {0}"));
	
		ScalabilityGroupsWidget = ButtonMatrix;
	}

#if WITH_SERVER_CODE
	// PIE 3D resolution.
	TSharedPtr<SWidget> ResolutionSliderWidget;
	{
		auto MakePresetButton = [this](const FText& InName, int32 ResolutionPresetId)
		{
			return SNew(SButton)
				//.ToolTipText(InToolTip)
				.OnClicked(this, &SScalabilitySettings::OnResolutionPresetClicked, ResolutionPresetId)
				.IsEnabled_Lambda([]() {
					return SScalabilitySettings::IsResolutionScaleEditable();
				})
				.Content()
				[
					SNew(STextBlock)
					.Text(InName)
				];
		};

		const TArray<Scalability::FResolutionPreset> ResolutionPresets = Scalability::GetResolutionPresets();

		TSharedRef<SGridPanel> ButtonMatrix =
			SNew(SGridPanel)
			.FillColumn(0, QualityColumnCoeff)

			+MakeGridSlot(0,0)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ScreenPercentage", "Screen Percentage"))
				.Font(GroupFont)
				.IsEnabled_Lambda([this]() {
					return SScalabilitySettings::IsResolutionScaleEditable() && CachedQualityLevels.ResolutionQuality >= Scalability::MinResolutionScale;
				})
			]
			+MakeGridSlot(1,0,ResolutionPresets.Num() - 1,1)
			[
				SNew(SSlider)
				.OnValueChanged(this, &SScalabilitySettings::OnResolutionScaleChanged)
				.Value(this, &SScalabilitySettings::GetResolutionScale)
				.IsEnabled_Lambda([]() {
					return true; // TODO: SScalabilitySettings::IsPlayInEditor(); but looks hugly
				})
			]
			+MakeGridSlot(ResolutionPresets.Num(),0)
			[
				SNew(STextBlock)
				.Text(this, &SScalabilitySettings::GetResolutionScaleString)
				.IsEnabled_Lambda([this]() {
					return SScalabilitySettings::IsResolutionScaleEditable() && CachedQualityLevels.ResolutionQuality >= Scalability::MinResolutionScale;
				})
			]
			;

		for (int32 PresetId = 0; PresetId < ResolutionPresets.Num(); PresetId++)
		{
			const Scalability::FResolutionPreset& Preset = ResolutionPresets[PresetId];
			FText PresetText = FText::FromString(Preset.Name);

			ButtonMatrix->AddSlot(1 + PresetId, 1)
				.Padding(2.f)
			[
				MakePresetButton(PresetText, PresetId)
			];
			//+MakeGridSlot(1, 1)[MakePresetButton(LOCTEXT("SuperResolution_Default", "Default"), 0)]
			//	+ MakeGridSlot(2, 1)[MakePresetButton(LOCTEXT("SuperResolution_Performance", "Performance"), 1)]
			//	+ MakeGridSlot(3, 1)[MakePresetButton(LOCTEXT("SuperResolution_Balanced", "Balanced"), 2)]
			//	+ MakeGridSlot(4, 1)[MakePresetButton(LOCTEXT("SuperResolution_Quality", "Quality"), 3)]
			//	+ MakeGridSlot(5, 1)[MakePresetButton(LOCTEXT("SuperResolution_Native", "Native"), 4)]
			//	;
		}

		ResolutionSliderWidget = ButtonMatrix;
	}
#endif // WITH_SERVER_CODE

	const UGameViewportClient* ViewportClient = GEngine->GameViewport;

	this->ChildSlot
		.HAlign(EHorizontalAlignment::HAlign_Fill)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(5.f, 2.f)
			[
				SNew(STextBlock).Text(LOCTEXT("ScalabilityGroups", "Scalability Groups")).Font(TitleFont)
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(10.f, 2.f)
			[
				ScalabilityGroupsWidget.ToSharedRef()
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(26.f, 3.f)
			[
				SNew(SCheckBox)
				.OnCheckStateChanged(this, &SScalabilitySettings::OnMonitorPerformanceChanged)
				.IsChecked(this, &SScalabilitySettings::IsMonitoringPerformance)
				.Content()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("PerformanceWarningEnableDisableCheckbox", "Monitor Editor Performance?"))
					.ToolTipText_Lambda([this]()
						{
							FProperty* Property = UEditorPerformanceSettings::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UEditorPerformanceSettings, bMonitorEditorPerformance));
							if (Property)
							{
								return Property->GetToolTipText();
							}
							
							return FText::GetEmpty();
						})
				]
			]

#if WITH_SERVER_CODE
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(5.f, 15.f, 5.f, 2.f)
			[
				SNew(STextBlock).Text(LOCTEXT("ResolutionPIE3D", "Play-In-Editor 3D Resolution")).Font(TitleFont)
			]
			
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(12.f, 2.f)
			[
				SNew(STextBlock)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Text_Lambda([ViewportClient]() {
					FFormatNamedArguments FormatArguments = GetScreenPercentageFormatArguments(ViewportClient);
					return FText::Format(LOCTEXT("ScreenPercentageCurrent_Display", "Current Screen Percentage: {CurrentScreenPercentage}"), FormatArguments);
				})
			]
			
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(12.f, 2.f)
			[
				SNew(STextBlock)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Text_Lambda([ViewportClient]() {
					FFormatNamedArguments FormatArguments = GetScreenPercentageFormatArguments(ViewportClient);
					return FText::Format(LOCTEXT("ScreenPercentageResolutions", "Resolution: {ResolutionFromTo}"), FormatArguments);
				})
			]
			
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(12.f, 2.f)
			[
				SNew(STextBlock)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Text_Lambda([ViewportClient]() {
					FFormatNamedArguments FormatArguments = GetScreenPercentageFormatArguments(ViewportClient);
					return FText::Format(LOCTEXT("ScreenPercentageActiveViewport", "Active Viewport: {ViewportMode}"), FormatArguments);
				})
			]
			
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(12.f, 2.f)
			[
				SNew(STextBlock)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Text_Lambda([ViewportClient]() {
					FFormatNamedArguments FormatArguments = GetScreenPercentageFormatArguments(ViewportClient);
					return FText::Format(LOCTEXT("ScreenPercentageSetFrom", "Set From: {SettingSource}"), FormatArguments);
				})
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(12.f, 2.f)
			[
				SNew(STextBlock)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Text_Lambda([ViewportClient]() {
					FFormatNamedArguments FormatArguments = GetScreenPercentageFormatArguments(ViewportClient);
					return FText::Format(LOCTEXT("ScreenPercentageSetting", "Setting: {Setting}"), FormatArguments);
				})
			]
			
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(10.f, 2.f)
			[
				ResolutionSliderWidget.ToSharedRef()
			]
#endif // WITH_SERVER_CODE
		];
}

SScalabilitySettings::~SScalabilitySettings()
{
	// Record any changed quality levels
	if( InitialQualityLevels != CachedQualityLevels )
	{
		const bool bAutoApplied = false;
		Scalability::RecordQualityLevelsAnalytics(bAutoApplied);
	}
}

#undef LOCTEXT_NAMESPACE
