// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSkeletonWidget.h"
#include "Modules/ModuleManager.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/AnimationAsset.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Animation/AnimBlueprint.h"
#include "Editor.h"
#include "Animation/AnimSet.h"
#include "Interfaces/IMainFrameModule.h"
#include "ContentBrowserModule.h"
#include "IDocumentation.h"
#include "AnimPreviewInstance.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AnimationRuntime.h"
#include "Settings/SkeletalMeshEditorSettings.h"
#include "Styling/CoreStyle.h"
#include "Styling/AppStyle.h"
#include "Viewports.h"

#define LOCTEXT_NAMESPACE "SkeletonWidget"

void SSkeletonListWidget::Construct(const FArguments& InArgs)
{
	bShowBones = InArgs._ShowBones;
	InitialViewType = InArgs._InitialViewType;
	CurSelectedSkeleton = nullptr;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.Filter.ClassPaths.Add(USkeleton::StaticClass()->GetClassPathName());
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SSkeletonListWidget::SkeletonSelectionChanged);
	AssetPickerConfig.InitialAssetViewType = InitialViewType;
	AssetPickerConfig.SelectionMode = ESelectionMode::Single;
	AssetPickerConfig.bShowPathInColumnView = true;
	AssetPickerConfig.bShowTypeInColumnView = false;

	TSharedRef<SVerticalBox> ContentBox = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("SelectSkeletonLabel", "Select Skeleton: "))
		]
		+ SVerticalBox::Slot().FillHeight(1).Padding(2)
			[
				SNew(SBorder)
				.Content()
				[
					ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
				]
			];

	if (bShowBones)
	{
		ContentBox->AddSlot()
			.FillHeight(1).Padding(2)
			.Expose(BoneListSlot);
	}

	ChildSlot
		[
			ContentBox
		];

	// Construct the BoneListSlot by clearing the skeleton selection. 
	SkeletonSelectionChanged(FAssetData());
}

void SSkeletonListWidget::SkeletonSelectionChanged(const FAssetData& AssetData)
{
	BoneList.Empty();
	CurSelectedSkeleton = Cast<USkeleton>(AssetData.GetAsset());

	if (bShowBones)
	{
		if (CurSelectedSkeleton != nullptr)
		{
			const FReferenceSkeleton& RefSkeleton = CurSelectedSkeleton->GetReferenceSkeleton();

			for (int32 I = 0; I < RefSkeleton.GetNum(); ++I)
			{
				BoneList.Add(MakeShareable(new FName(RefSkeleton.GetBoneName(I))));
			}

			(*BoneListSlot)
				[
					SNew(SBorder).Padding(2)
					.Content()
				[
					SNew(SListView< TSharedPtr<FName> >)
					.OnGenerateRow(this, &SSkeletonListWidget::GenerateSkeletonBoneRow)
				.ListItemsSource(&BoneList)
				.HeaderRow
				(
					SNew(SHeaderRow)
					+ SHeaderRow::Column(TEXT("Bone Name"))
					.DefaultLabel(NSLOCTEXT("SkeletonWidget", "BoneName", "Bone Name"))
				)
				]
				];
		}
		else
		{
			(*BoneListSlot)
				[
					SNew(SBorder).Padding(2)
					.Content()
				[
					SNew(STextBlock)
					.Text(NSLOCTEXT("SkeletonWidget", "NoSkeletonIsSelected", "No skeleton is selected!"))
				]
				];
		}
	}
}

void SSkeletonCompareWidget::Construct(const FArguments& InArgs)
{
	const UObject* Object = InArgs._Object;

	CurSelectedSkeleton = nullptr;
	BoneNames = *InArgs._BoneNames;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.Filter.ClassPaths.Add(USkeleton::StaticClass()->GetClassPathName());
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SSkeletonCompareWidget::SkeletonSelectionChanged);
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::Column;
	AssetPickerConfig.bShowPathInColumnView = true;
	AssetPickerConfig.bShowTypeInColumnView = false;

	TSharedPtr<SToolTip> SkeletonTooltip = IDocumentation::Get()->CreateToolTip(FText::FromString("Pick a skeleton for this mesh"), nullptr, FString("Shared/Editors/Persona"), FString("Skeleton"));

	this->ChildSlot
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.AutoHeight() 
				.Padding(2) 
				.HAlign(HAlign_Fill)
				[
					SNew(SHorizontalBox)
				 
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					+SHorizontalBox::Slot()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("CurrentlySelectedSkeletonLabel_SelectSkeleton", "Select Skeleton"))
						.Font(FCoreStyle::GetDefaultFontStyle("Regular", 16))
						.ToolTip(SkeletonTooltip)
					]
					+SHorizontalBox::Slot()
					.FillWidth(1)
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					[
						IDocumentation::Get()->CreateAnchor(FString("Engine/Animation/Skeleton"))
					]
				]

				+SVerticalBox::Slot()
				.AutoHeight() 
				.Padding(2, 10)
				.HAlign(HAlign_Fill)
				[
					SNew(SSeparator)
					.Orientation(Orient_Horizontal)
				]

				+SVerticalBox::Slot()
				.AutoHeight() 
				.Padding(2) 
				.HAlign(HAlign_Fill)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("CurrentlySelectedSkeletonLabel", "Currently Selected : "))
				]
				+SVerticalBox::Slot()
				.AutoHeight() 
				.Padding(2) 
				.HAlign(HAlign_Fill)
				[
					SNew(STextBlock)
					.Text(FText::FromString(Object->GetFullName()))
				]
			]

			+SVerticalBox::Slot() .FillHeight(1) .Padding(2)
				[
					SNew(SBorder)
					.Content()
					[
						ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
					]
				]

			+SVerticalBox::Slot() .FillHeight(1) .Padding(2)
				.Expose(BonePairSlot)
		];

	// Construct the BonePairSlot by clearing the skeleton selection. 
	SkeletonSelectionChanged(FAssetData());
}

void SSkeletonCompareWidget::SkeletonSelectionChanged(const FAssetData& AssetData)
{
	BonePairList.Empty();
	CurSelectedSkeleton = Cast<USkeleton>(AssetData.GetAsset());

	if (CurSelectedSkeleton != nullptr)
	{
		for (int32 I=0; I<BoneNames.Num(); ++I)
		{
			if ( CurSelectedSkeleton->GetReferenceSkeleton().FindBoneIndex(BoneNames[I]) != INDEX_NONE )
			{
				BonePairList.Add( MakeShareable(new FBoneTrackPair(BoneNames[I], BoneNames[I])) );
			}
			else
			{
				BonePairList.Add( MakeShareable(new FBoneTrackPair(BoneNames[I], TEXT(""))) );
			}
		}

		(*BonePairSlot)
			[
				SNew(SBorder) .Padding(2)
				.Content()
				[
					SNew(SListView< TSharedPtr<FBoneTrackPair> >)
					.OnGenerateRow(this, &SSkeletonCompareWidget::GenerateBonePairRow)
					.ListItemsSource(&BonePairList)
					.HeaderRow
					(
					SNew(SHeaderRow)
					+SHeaderRow::Column(TEXT("Curretly Selected"))
					.DefaultLabel(NSLOCTEXT("SkeletonWidget", "CurrentlySelected", "Currently Selected"))
					+SHeaderRow::Column(TEXT("Target Skeleton Bone"))
					.DefaultLabel(NSLOCTEXT("SkeletonWidget", "TargetSkeletonBone", "Target Skeleton Bone"))
					)
				]
			];
	}
	else
	{
		(*BonePairSlot)
			[
				SNew(SBorder) .Padding(2)
				.Content()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("NoSkeletonSelectedLabel", "No skeleton is selected!"))
				]
			];
	}
}

void SSkeletonSelectorWindow::Construct(const FArguments& InArgs)
{
	UObject* Object = InArgs._Object;
	WidgetWindow = InArgs._WidgetWindow;
	SelectedSkeleton = nullptr;
	if (Object == nullptr)
	{
		ConstructWindow();
	}
	else if (Object->IsA(USkeletalMesh::StaticClass()))
	{
		ConstructWindowFromMesh(CastChecked<USkeletalMesh>(Object));
	}
	else if (Object->IsA(UAnimSet::StaticClass()))
	{
		ConstructWindowFromAnimSet(CastChecked<UAnimSet>(Object));
	}
	else if (Object->IsA(UAnimBlueprint::StaticClass()))
	{
		ConstructWindowFromAnimBlueprint(CastChecked<UAnimBlueprint>(Object));
	}
}

void SSkeletonSelectorWindow::ConstructWindowFromAnimSet(UAnimSet* InAnimSet)
{
	TArray<FName>  *TrackNames = &InAnimSet->TrackBoneNames;

	TSharedRef<SVerticalBox> ContentBox = SNew(SVerticalBox)
		+SVerticalBox::Slot() 
		.FillHeight(1) 
		.Padding(2)
		[
			SAssignNew(SkeletonWidget, SSkeletonCompareWidget)
			.Object(InAnimSet)
			.BoneNames(TrackNames)	
		];

	ConstructButtons(ContentBox);

	ChildSlot
		[
			ContentBox
		];
}

void SSkeletonSelectorWindow::ConstructWindowFromMesh(USkeletalMesh* InSkeletalMesh)
{
	TArray<FName>  BoneNames;

	for (int32 I=0; I<InSkeletalMesh->GetRefSkeleton().GetRawBoneNum(); ++I)
	{
		BoneNames.Add(InSkeletalMesh->GetRefSkeleton().GetBoneName(I));
	}

	TSharedRef<SVerticalBox> ContentBox = SNew(SVerticalBox)
		+SVerticalBox::Slot() .FillHeight(1) .Padding(2)
		[
			SAssignNew(SkeletonWidget, SSkeletonCompareWidget)
			.Object(InSkeletalMesh)
			.BoneNames(&BoneNames)
		];

	ConstructButtons(ContentBox);

	ChildSlot
		[
			ContentBox
		];
}

void SSkeletonSelectorWindow::ConstructWindowFromAnimBlueprint(UAnimBlueprint* AnimBlueprint)
{
	TSharedRef<SVerticalBox> ContentBox = SNew(SVerticalBox)
		+ SVerticalBox::Slot().FillHeight(1).Padding(2)
		[
			SAssignNew(SkeletonWidget, SSkeletonListWidget)
				.ShowBones(false)
				.InitialViewType(EAssetViewType::List)
		];

	ConstructButtons(ContentBox);

	ChildSlot
		[
			ContentBox
		];
}

void SSkeletonSelectorWindow::ConstructWindow()
{
	TSharedRef<SVerticalBox> ContentBox = SNew(SVerticalBox)
		+SVerticalBox::Slot() .FillHeight(1) .Padding(2)
		[
			SAssignNew(SkeletonWidget, SSkeletonListWidget)
				.ShowBones(true)
				.InitialViewType(EAssetViewType::Column)
		];

	ConstructButtons(ContentBox);

	ChildSlot
		[
			ContentBox
		];
}

void SSkeletonBoneRemoval::Construct( const FArguments& InArgs )
{
	bShouldContinue = false;
	WidgetWindow = InArgs._WidgetWindow;

	for (int32 I=0; I<InArgs._BonesToRemove.Num(); ++I)
	{
		BoneNames.Add( MakeShareable(new FName(InArgs._BonesToRemove[I])) ) ;
	}

	this->ChildSlot
	[
		SNew (SVerticalBox)

		+SVerticalBox::Slot()
		.AutoHeight() 
		.Padding(2) 
		.HAlign(HAlign_Fill)
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			+SHorizontalBox::Slot()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("BoneRemovalLabel", "Bone Removal"))
				.Font(FCoreStyle::GetDefaultFontStyle("Regular", 16))
			]
		]

		+SVerticalBox::Slot()
		.AutoHeight() 
		.Padding(2, 10)
		.HAlign(HAlign_Fill)
		[
			SNew(SSeparator)
			.Orientation(Orient_Horizontal)
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5)
		[
			SNew(STextBlock)
			.WrapTextAt(400.f)
			.Font( FCoreStyle::GetDefaultFontStyle("Regular", 10) )
			.Text( InArgs._WarningMessage )
		]

		+SVerticalBox::Slot()
		.AutoHeight() 
		.Padding(5)
		[
			SNew(SSeparator)
		]

		+SVerticalBox::Slot()
		.MaxHeight(300)
		.Padding(5)
		[
			SNew(SListView< TSharedPtr<FName> >)
			.OnGenerateRow(this, &SSkeletonBoneRemoval::GenerateSkeletonBoneRow)
			.ListItemsSource(&BoneNames)
			.HeaderRow
			(
			SNew(SHeaderRow)
			+SHeaderRow::Column(TEXT("Bone Name"))
			.DefaultLabel(NSLOCTEXT("SkeletonWidget", "BoneName", "Bone Name"))
			)
		]

		+SVerticalBox::Slot()
		.AutoHeight() 
		.Padding(5)
		[
			SNew(SSeparator)
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Bottom)
		[
			SNew(SUniformGridPanel)
			.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
			.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
			.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
			+SUniformGridPanel::Slot(0,0)
			[
				SNew(SButton) .HAlign(HAlign_Center)
				.Text(LOCTEXT("BoneRemoval_Ok", "Ok"))
				.OnClicked(this, &SSkeletonBoneRemoval::OnOk)
				.HAlign(HAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
			]
			+SUniformGridPanel::Slot(1,0)
			[
				SNew(SButton) .HAlign(HAlign_Center)
				.Text(LOCTEXT("BoneRemoval_Cancel", "Cancel"))
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.OnClicked(this, &SSkeletonBoneRemoval::OnCancel)
			]
		]
	];
}

FReply SSkeletonBoneRemoval::OnOk()
{
	bShouldContinue = true;
	CloseWindow();
	return FReply::Handled();
}

FReply SSkeletonBoneRemoval::OnCancel()
{
	CloseWindow();
	return FReply::Handled();
}

void SSkeletonBoneRemoval::CloseWindow()
{
	if ( WidgetWindow.IsValid() )
	{
		WidgetWindow.Pin()->RequestDestroyWindow();
	}
}

bool SSkeletonBoneRemoval::ShowModal(const TArray<FName> BonesToRemove, const FText& WarningMessage)
{
	TSharedPtr<class SSkeletonBoneRemoval> DialogWidget;

	TSharedPtr<SWindow> DialogWindow = SNew(SWindow)
		.Title( LOCTEXT("RemapSkeleton", "Select Skeleton") )
		.SupportsMinimize(false) .SupportsMaximize(false)
		.SizingRule( ESizingRule::Autosized );

	TSharedPtr<SBorder> DialogWrapper = 
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		.Padding(4.0f)
		[
			SAssignNew(DialogWidget, SSkeletonBoneRemoval)
			.WidgetWindow(DialogWindow)
			.BonesToRemove(BonesToRemove)
			.WarningMessage(WarningMessage)
		];

	DialogWindow->SetContent(DialogWrapper.ToSharedRef());

	GEditor->EditorAddModalWindow(DialogWindow.ToSharedRef());

	return DialogWidget.Get()->bShouldContinue;
}

////////////////////////////////////////

class FBasePoseViewportClient: public FEditorViewportClient
{
public:
	FBasePoseViewportClient(FPreviewScene& InPreviewScene, const TSharedRef<SBasePoseViewport>& InBasePoseViewport)
		: FEditorViewportClient(nullptr, &InPreviewScene, StaticCastSharedRef<SEditorViewport>(InBasePoseViewport))
	{
		SetViewMode(VMI_Lit);

		// Always composite editor objects after post processing in the editor
		EngineShowFlags.SetCompositeEditorPrimitives(true);
		EngineShowFlags.DisableAdvancedFeatures();

		UpdateLighting();

		// Setup defaults for the common draw helper.
		DrawHelper.bDrawPivot = false;
		DrawHelper.bDrawWorldBox = false;
		DrawHelper.bDrawKillZ = false;
		DrawHelper.bDrawGrid = true;
		DrawHelper.GridColorAxis = FColor(70, 70, 70);
		DrawHelper.GridColorMajor = FColor(40, 40, 40);
		DrawHelper.GridColorMinor =  FColor(20, 20, 20);
		DrawHelper.PerspectiveGridSize = UE_OLD_HALF_WORLD_MAX1;

		bDisableInput = true;
	}


	// FEditorViewportClient interface
	virtual void Tick(float DeltaTime) override
	{
		if (PreviewScene)
		{
			PreviewScene->GetWorld()->Tick(LEVELTICK_All, DeltaTime);
		}
	}

	virtual FSceneInterface* GetScene() const override
	{
		return PreviewScene->GetScene();
	}

	virtual FLinearColor GetBackgroundColor() const override 
	{ 
		return FLinearColor::White; 
	}

	virtual void SetViewMode(EViewModeIndex Index) override final
	{
		FEditorViewportClient::SetViewMode(Index);
	}

	// End of FEditorViewportClient

	void UpdateLighting()
	{
		const USkeletalMeshEditorSettings* Options = GetDefault<USkeletalMeshEditorSettings>();

		PreviewScene->SetLightDirection(Options->AnimPreviewLightingDirection);
		PreviewScene->SetLightColor(Options->AnimPreviewDirectionalColor);
		PreviewScene->SetLightBrightness(Options->AnimPreviewLightBrightness);
	}
};

////////////////////////////////
// SBasePoseViewport
void SBasePoseViewport::Construct(const FArguments& InArgs)
{
	SEditorViewport::Construct(SEditorViewport::FArguments());

	PreviewComponent = NewObject<UDebugSkelMeshComponent>();
	PreviewComponent->VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones;
	PreviewScene.AddComponent(PreviewComponent, FTransform::Identity);

	SetSkeleton(InArgs._Skeleton);
}

void SBasePoseViewport::SetSkeleton(USkeleton* Skeleton)
{
	if(Skeleton != TargetSkeleton)
	{
		TargetSkeleton = Skeleton;

		if(TargetSkeleton)
		{
			USkeletalMesh* PreviewSkeletalMesh = Skeleton->GetPreviewMesh();
			if(PreviewSkeletalMesh)
			{
				PreviewComponent->SetSkeletalMesh(PreviewSkeletalMesh);
				PreviewComponent->EnablePreview(true, nullptr);
				PreviewComponent->RefreshBoneTransforms(nullptr);

				//Place the camera at a good viewer position
				FVector NewPosition = Client->GetViewLocation();
				NewPosition.Normalize();
				NewPosition *= (PreviewSkeletalMesh->GetImportedBounds().SphereRadius*1.5f);
				Client->SetViewLocation(NewPosition);
			}
			else
			{
				PreviewComponent->SetSkeletalMesh(nullptr);
			}
		}
		else
		{
			PreviewComponent->SetSkeletalMesh(nullptr);
		}

		Client->Invalidate();
	}
}

SBasePoseViewport::SBasePoseViewport()
: PreviewScene(FPreviewScene::ConstructionValues())
{
}

bool SBasePoseViewport::IsVisible() const
{
	return true;
}

TSharedRef<FEditorViewportClient> SBasePoseViewport::MakeEditorViewportClient()
{
	TSharedPtr<FEditorViewportClient> EditorViewportClient = MakeShareable(new FBasePoseViewportClient(PreviewScene, SharedThis(this)));

	EditorViewportClient->ViewportType = LVT_Perspective;
	EditorViewportClient->bSetListenerPosition = false;
	EditorViewportClient->SetViewLocation(EditorViewportDefs::DefaultPerspectiveViewLocation);
	EditorViewportClient->SetViewRotation(EditorViewportDefs::DefaultPerspectiveViewRotation);

	EditorViewportClient->SetRealtime(false);
	EditorViewportClient->VisibilityDelegate.BindSP(this, &SBasePoseViewport::IsVisible);
	EditorViewportClient->SetViewMode(VMI_Lit);

	return EditorViewportClient.ToSharedRef();
}


/////////////////////////////////////////////////
// select folder dialog
//////////////////////////////////////////////////
void SSelectFolderDlg::Construct(const FArguments& InArgs)
{
	AssetPath = FText::FromString(FPackageName::GetLongPackagePath(InArgs._DefaultAssetPath.ToString()));

	if(AssetPath.IsEmpty())
	{
		AssetPath = FText::FromString(TEXT("/Game"));
	}

	FPathPickerConfig PathPickerConfig;
	PathPickerConfig.DefaultPath = AssetPath.ToString();
	PathPickerConfig.OnPathSelected = FOnPathSelected::CreateSP(this, &SSelectFolderDlg::OnPathChange);
	PathPickerConfig.bAddDefaultPath = true;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	SWindow::Construct(SWindow::FArguments()
		.Title(LOCTEXT("SSelectFolderDlg_Title", "Create New Animation Object"))
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		//.SizingRule( ESizingRule::Autosized )
		.ClientSize(FVector2D(450, 450))
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot() // Add user input block
			.Padding(2)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SVerticalBox)

					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("SelectPath", "Select Path"))
						.Font(FCoreStyle::GetDefaultFontStyle("Regular", 14))
					]

					+SVerticalBox::Slot()
					.FillHeight(1)
					.Padding(3)
					[
						ContentBrowserModule.Get().CreatePathPicker(PathPickerConfig)
					]
				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(5)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
				.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
				.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
				+SUniformGridPanel::Slot(0, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.Text(LOCTEXT("OK", "OK"))
					.OnClicked(this, &SSelectFolderDlg::OnButtonClick, EAppReturnType::Ok)
				]
				+SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.Text(LOCTEXT("Cancel", "Cancel"))
					.OnClicked(this, &SSelectFolderDlg::OnButtonClick, EAppReturnType::Cancel)
				]
			]
		]);
}

void SSelectFolderDlg::OnPathChange(const FString& NewPath)
{
	AssetPath = FText::FromString(NewPath);
}

FReply SSelectFolderDlg::OnButtonClick(EAppReturnType::Type ButtonID)
{
	UserResponse = ButtonID;

	RequestDestroyWindow();

	return FReply::Handled();
}

EAppReturnType::Type SSelectFolderDlg::ShowModal()
{
	GEditor->EditorAddModalWindow(SharedThis(this));
	return UserResponse;
}

FString SSelectFolderDlg::GetAssetPath()
{
	return AssetPath.ToString();
}

TSharedRef<FDisplayedAssetEntryInfo> FDisplayedAssetEntryInfo::Make(UObject* InAsset, USkeleton* InNewSkeleton)
{
	return MakeShareable(new FDisplayedAssetEntryInfo(InAsset, InNewSkeleton));
}

FDisplayedAssetEntryInfo::FDisplayedAssetEntryInfo(UObject* InAsset, USkeleton* InNewSkeleton)
	: NewSkeleton(InNewSkeleton)
	, AnimAsset(InAsset)
	, RemapAsset(nullptr)
{
}

void SReplaceMissingSkeletonDialog::Construct(const FArguments& InArgs)
{
	AssetsToReplaceSkeletonOn = InArgs._AnimAssets;
	
	// Load the content browser module to display an asset picker
	const FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	FAssetPickerConfig AssetPickerConfig;

	/** The asset picker will only show skeleton assets */
	AssetPickerConfig.Filter.ClassPaths.Add(USkeleton::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.bRecursiveClasses = true;

	/** The delegate that fires when an asset was selected */
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, & SReplaceMissingSkeletonDialog::OnSkeletonSelected);

	/** The default view mode should be a list view */
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;

	SWindow::Construct(SWindow::FArguments()
		.Title(LOCTEXT("ReplaceSkeletonOptions", "Pick Replacement Skeleton"))
		.ClientSize(FVector2D(500, 600))
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		.ClientSize(FVector2D(450, 450))
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot() // Add user input block
			[
				ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(5)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
				.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
				.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
				+SUniformGridPanel::Slot(0, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.Text(LOCTEXT("OK", "OK"))
					.OnClicked(this, &SReplaceMissingSkeletonDialog::OnButtonClick, EAppReturnType::Ok)
				]
				+SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.Text(LOCTEXT("Cancel", "Cancel"))
					.OnClicked(this, &SReplaceMissingSkeletonDialog::OnButtonClick, EAppReturnType::Cancel)
				]
			]
		]);
}

void SReplaceMissingSkeletonDialog::OnSkeletonSelected(const FAssetData& Replacement)
{
	SelectedAsset = Replacement;
}

FReply SReplaceMissingSkeletonDialog::OnButtonClick(EAppReturnType::Type ButtonID)
{
	UserResponse = ButtonID;
	RequestDestroyWindow();

	if (ButtonID != EAppReturnType::Ok)
	{
		return FReply::Handled();
	}

	const TObjectPtr<UObject> Asset = SelectedAsset.GetAsset();
	if (!Asset)
	{
		return FReply::Handled();
	}
	
	if (const TObjectPtr<USkeleton> ReplacementSkeleton = CastChecked<USkeleton>(Asset))
	{
		constexpr bool bRetargetReferredAssets = true;
		constexpr bool bConvertSpaces = false;
		FAnimationRetargetContext RetargetContext(AssetsToReplaceSkeletonOn, bRetargetReferredAssets, bConvertSpaces);
		// since we are replacing a missing skeleton, we don't want to duplicate the asset
		// setting this to null prevents assets from being duplicated
		const FNameDuplicationRule* NameRule = nullptr;

		EditorAnimUtils::RetargetAnimations(nullptr, ReplacementSkeleton, RetargetContext, bRetargetReferredAssets, NameRule);

		bWasSkeletonReplaced = true;
	}

	return FReply::Handled();
}

bool SReplaceMissingSkeletonDialog::ShowModal()
{
	bWasSkeletonReplaced = false;
	GEditor->EditorAddModalWindow(SharedThis(this));
	return bWasSkeletonReplaced;
}

#undef LOCTEXT_NAMESPACE 
