// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSocketManager.h"
#include "Widgets/Layout/SSplitter.h"
#include "UObject/UnrealType.h"
#include "Engine/StaticMesh.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Developer/ToolWidgets/Public/SPositiveActionButton.h"
#include "Widgets/Views/SListView.h"
#include "Styling/AppStyle.h"
#include "Components/StaticMeshComponent.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/StaticMeshSocket.h"
#include "UnrealEdGlobals.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IStaticMeshEditor.h"

#include "PropertyEditorModule.h"

#include "ScopedTransaction.h"

#include "Interfaces/IAnalyticsProvider.h"
#include "EngineAnalytics.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Framework/Commands/GenericCommands.h"

#define LOCTEXT_NAMESPACE "SSCSSocketManagerEditor"

struct SocketListItem
{
public:
	SocketListItem(UStaticMeshSocket* InSocket)
		: Socket(InSocket)
	{
	}

	/** The static mesh socket this represents */
	UStaticMeshSocket* Socket;

	/** Delegate for when the context menu requests a rename */
	DECLARE_DELEGATE(FOnRenameRequested);
	FOnRenameRequested OnRenameRequested;
};

class SSocketDisplayItem : public SMultiColumnTableRow< TSharedPtr<SocketListItem> >
{

public:
	
	SLATE_BEGIN_ARGS( SSocketDisplayItem ):
		_ReadOnly(false)
		{} 

		/** The socket this item displays. */
		SLATE_ARGUMENT( TWeakPtr< SocketListItem >, SocketItem )

		/** Pointer back to the socket manager */
		SLATE_ARGUMENT( TWeakPtr< SSocketManager >, SocketManagerPtr )

		/** Whether the widget should be editable or not */
		SLATE_ARGUMENT( bool, ReadOnly)
	SLATE_END_ARGS()

	/**
	 * Construct the widget
	 *
	 * @param InArgs   A declaration from which to construct the widget
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView )
	{
		SocketItem = InArgs._SocketItem;
		SocketManagerPtr = InArgs._SocketManagerPtr;
		bReadOnly = InArgs._ReadOnly;

		ImportedSocketBrush = FAppStyle::Get().GetBrush("Icons.Import");
		NotImportedSocketBrush = FAppStyle::Get().GetBrush("NoBrush");

		auto Args = FSuperRowType::FArguments();
			// .Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SceneOutliner.TableViewRow"));

		SMultiColumnTableRow< TSharedPtr<SocketListItem> >::Construct(Args, InOwnerTableView);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override
	{
		// todo make these static id names 
		if( ColumnName == TEXT("Socket"))
		{
			TSharedPtr< SInlineEditableTextBlock > InlineWidget;

			SAssignNew( InlineWidget, SInlineEditableTextBlock )
			.IsReadOnly(bReadOnly)
			.Text( this, &SSocketDisplayItem::GetSocketName )
			.OnVerifyTextChanged( this, &SSocketDisplayItem::OnVerifySocketNameChanged )
			.OnTextCommitted( this, &SSocketDisplayItem::OnCommitSocketName )
			.IsSelected( this, &SMultiColumnTableRow< TSharedPtr<SocketListItem> >::IsSelectedExclusively );

			TSharedPtr<SocketListItem> SocketItemPinned = SocketItem.Pin();
			if (SocketItemPinned.IsValid())
			{
				SocketItemPinned->OnRenameRequested.BindSP(InlineWidget.Get(), &SInlineEditableTextBlock::EnterEditingMode);
			}

			return SNew(SBox)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(FMargin(20.f, 4.f))
			[
				InlineWidget.ToSharedRef()
				];
		}
		else
		{
			return SNew(SBox)
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.ToolTipText(this, &SSocketDisplayItem::GetImportToolTip)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(this, &SSocketDisplayItem::GetImportBrush)
				];
		}
	}

private:
	/** Returns the socket name */
	FText GetSocketName() const
	{
		TSharedPtr<SocketListItem> SocketItemPinned = SocketItem.Pin();
		return SocketItemPinned.IsValid() ? FText::FromName(SocketItemPinned->Socket->SocketName) : FText();
	}

	const FSlateBrush* GetImportBrush() const
	{

		TSharedPtr<SocketListItem> SocketItemPinned = SocketItem.Pin();
		return SocketItemPinned.IsValid() && SocketItemPinned->Socket->bSocketCreatedAtImport ? ImportedSocketBrush : NotImportedSocketBrush;

	}

	FText GetImportToolTip() const
	{
		TSharedPtr<SocketListItem> SocketItemPinned = SocketItem.Pin();
		if (SocketItemPinned.IsValid() && SocketItemPinned->Socket->bSocketCreatedAtImport)
		{
			return LOCTEXT("ImportedSockedTooltip", "Socket was imported from the source mesh.");
		}

		return FText::GetEmpty();
	}

	bool OnVerifySocketNameChanged( const FText& InNewText, FText& OutErrorMessage )
	{
		bool bVerifyName = true;

		FText NewText = FText::TrimPrecedingAndTrailing(InNewText);
		if(NewText.IsEmpty())
		{
			OutErrorMessage = LOCTEXT( "EmptySocketName_Error", "Sockets must have a name!");
			bVerifyName = false;
		}
		else
		{
			TSharedPtr<SocketListItem> SocketItemPinned = SocketItem.Pin();
			TSharedPtr<SSocketManager> SocketManagerPinned = SocketManagerPtr.Pin();

			if (SocketItemPinned.IsValid() && SocketItemPinned->Socket != nullptr && SocketItemPinned->Socket->SocketName.ToString() != NewText.ToString() &&
				SocketManagerPinned.IsValid() && SocketManagerPinned->CheckForDuplicateSocket(NewText.ToString()))
			{
				OutErrorMessage = LOCTEXT("DuplicateSocket_Error", "Socket name in use!");
				bVerifyName = false;
			}
		}

		return bVerifyName;
	}

	void OnCommitSocketName( const FText& InText, ETextCommit::Type CommitInfo )
	{
		FText NewText = FText::TrimPrecedingAndTrailing(InText);

		TSharedPtr<SocketListItem> PinnedSocketItem = SocketItem.Pin();
		if (PinnedSocketItem.IsValid())
		{
			UStaticMeshSocket* SelectedSocket = PinnedSocketItem->Socket;
			if (SelectedSocket != NULL)
			{
				FScopedTransaction Transaction( LOCTEXT("SetSocketName", "Set Socket Name") );
				
				FProperty* ChangedProperty = FindFProperty<FProperty>( UStaticMeshSocket::StaticClass(), "SocketName" );
				
				// Pre edit, calls modify on the object
				SelectedSocket->PreEditChange(ChangedProperty);

				// Edit the property itself
				SelectedSocket->SocketName = FName(*NewText.ToString());

				// Post edit
				FPropertyChangedEvent PropertyChangedEvent( ChangedProperty );
				SelectedSocket->PostEditChangeProperty(PropertyChangedEvent);
			}
		}
	}

private:
	/** The Socket to display. */
	TWeakPtr< SocketListItem > SocketItem;

	/** Pointer back to the socket manager */
	TWeakPtr< SSocketManager > SocketManagerPtr; 

	const FSlateBrush* ImportedSocketBrush;
	const FSlateBrush* NotImportedSocketBrush;

	bool bReadOnly;
};

TSharedPtr<ISocketManager> ISocketManager::CreateSocketManager(TSharedPtr<class IStaticMeshEditor> InStaticMeshEditor, FSimpleDelegate InOnSocketSelectionChanged )
{
	TSharedPtr<SSocketManager> SocketManager;
	SAssignNew(SocketManager, SSocketManager)
		.StaticMeshEditorPtr(InStaticMeshEditor)
		.OnSocketSelectionChanged( InOnSocketSelectionChanged )
		.ReadOnly(InStaticMeshEditor->GetOpenMethod() == EAssetOpenMethod::View);

	TSharedPtr<ISocketManager> ISocket;
	ISocket = StaticCastSharedPtr<ISocketManager>(SocketManager);
	return ISocket;
}

void SSocketManager::Construct(const FArguments& InArgs)
{
	StaticMeshEditorPtr = InArgs._StaticMeshEditorPtr;

	OnSocketSelectionChanged = InArgs._OnSocketSelectionChanged;

	bReadOnly = InArgs._ReadOnly;

	TSharedPtr<IStaticMeshEditor> StaticMeshEditorPinned = StaticMeshEditorPtr.Pin();
	if (!StaticMeshEditorPinned.IsValid())
	{
		return;
	}

	// Register a post undo function which keeps the socket manager list view consistent with the static mesh
	StaticMeshEditorPinned->RegisterOnPostUndo(IStaticMeshEditor::FOnPostUndo::CreateSP(this, &SSocketManager::PostUndo));

	StaticMesh = StaticMeshEditorPinned->GetStaticMesh();

	FDetailsViewArgs Args;
	Args.bHideSelectionTip = true;
	Args.bLockable = false;
	Args.bAllowSearch = false;
	Args.bShowOptions = false;
	Args.NotifyHook = this;
	Args.NameAreaSettings = FDetailsViewArgs::HideNameArea;

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	SocketDetailsView = PropertyModule.CreateDetailView(Args);
	
	SocketDetailsView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateLambda([this]
	{
		if(TSharedPtr<IStaticMeshEditor> StaticMeshEditorPinned = StaticMeshEditorPtr.Pin())
		{
			return StaticMeshEditorPinned->GetOpenMethod() == EAssetOpenMethod::Edit;
		}

		return true;
	}));

	WorldSpaceRotation = FVector::ZeroVector;

	this->ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SSplitter)
			.Orientation(Orient_Vertical)

			+ SSplitter::Slot()
			.Value(.3f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SVerticalBox)
						
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0, 2)
					[
						SNew(SHorizontalBox)
							// Filter entry
							+ SHorizontalBox::Slot()
							.FillWidth(1)
							[
								SAssignNew(NameFilterBox, SSearchBox)
									.SelectAllTextWhenFocused(true)
									.OnTextChanged(this, &SSocketManager::OnFilterTextChanged)
									.OnTextCommitted(this, &SSocketManager::OnFilterTextCommitted)
							]
					]
		
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
							.HAlign(HAlign_Left)
							.Padding(FMargin(12.f, 6.f))
							[
								SNew(STextBlock)
								.TextStyle(FAppStyle::Get(), "ButtonText")
								.Text(LOCTEXT("Sockets", "Sockets"))
								.TransformPolicy(ETextTransformPolicy::ToUpper)	
							]
						]

						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(SPositiveActionButton)
							.ToolTipText(LOCTEXT("CreateSocket", "Create Socket"))
							.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
							.OnClicked(this, &SSocketManager::CreateSocket_Execute)
							.Visibility(this, &SSocketManager::CreateSocket_IsVisible )
						]
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SSeparator)
					]

					+ SVerticalBox::Slot()
					.FillHeight(1.0f)
					[
						SAssignNew(SocketListView, SListView<TSharedPtr< SocketListItem > >)

						.SelectionMode(ESelectionMode::Multi)

						.ListItemsSource(&SocketList)

						// Generates the actual widget for a tree item
						.OnGenerateRow(this, &SSocketManager::MakeWidgetFromOption)

						// Find out when the user selects something in the tree
						.OnSelectionChanged(this, &SSocketManager::SocketSelectionChanged_Execute)

						.OnContextMenuOpening(this, &SSocketManager::OnContextMenuOpening)
						.OnItemScrolledIntoView(this, &SSocketManager::OnItemScrolledIntoView)

						.HeaderRow
						(
							SNew(SHeaderRow)
							.Visibility(EVisibility::Collapsed)

							+ SHeaderRow::Column(TEXT("Socket"))
							.HAlignCell(HAlign_Fill)
							.VAlignCell(VAlign_Center)

							+ SHeaderRow::Column(TEXT("Imported"))
							.HAlignCell(HAlign_Right)
							.VAlignCell(VAlign_Center)
						)
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(FMargin(12.f, 6.f))
					[
						SNew(STextBlock)
						.Text(this, &SSocketManager::GetSocketHeaderText)
					]
				]
			]

			+ SSplitter::Slot()
			.Value(.7f)
			[
				SNew(SOverlay)

				+ SOverlay::Slot()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Visibility(this, &SSocketManager::GetSelectSocketMessageVisibility)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("NoSocketSelected", "Select a Socket"))
					]
				]

				+ SOverlay::Slot()
				[
					SocketDetailsView.ToSharedRef()
				]
			]
		]
	];

	RefreshSocketList();

	AddPropertyChangeListenerToSockets();
}

SSocketManager::~SSocketManager()
{
	TSharedPtr<IStaticMeshEditor> StaticMeshEditorPinned = StaticMeshEditorPtr.Pin();
	if (StaticMeshEditorPinned.IsValid())
	{
		StaticMeshEditorPinned->UnregisterOnPostUndo(this);
	}

	RemovePropertyChangeListenerFromSockets();
}

UStaticMeshSocket* SSocketManager::GetSelectedSocket() const
{
	if( SocketListView->GetSelectedItems().Num())
	{
		return SocketListView->GetSelectedItems()[0]->Socket;
	}

	return nullptr;
}

TArray<UStaticMeshSocket*> SSocketManager::GetSelectedSockets() const
{
	TArray<UStaticMeshSocket*> Sockets;
	Sockets.Reserve(SocketListView->GetNumItemsSelected());

	for (TSharedPtr<SocketListItem>& Socket : SocketListView->GetSelectedItems())
	{
		Sockets.Add(Socket->Socket);
	}

	return Sockets;
}

bool SSocketManager::HasSelectedSockets() const
{
	return SocketListView->GetNumItemsSelected() > 0;
}

EVisibility SSocketManager::GetSelectSocketMessageVisibility() const
{
	return SocketListView->GetSelectedItems().Num() > 0 ? EVisibility::Hidden : EVisibility::Visible;
}

void SSocketManager::SetSelectedSocket(UStaticMeshSocket* InSelectedSocket)
{
	if (InSelectedSocket)
	{
		for( int32 i=0; i < SocketList.Num(); i++)
		{
			if(SocketList[i]->Socket == InSelectedSocket)
			{
				SocketListView->SetSelection(SocketList[i]);

				SocketListView->RequestListRefresh();

				SocketSelectionChanged(InSelectedSocket);

				break;
			}
		}
	}
	else
	{
		SocketListView->ClearSelection();

		SocketListView->RequestListRefresh();

		SocketSelectionChanged(NULL);
	}
}


void SSocketManager::AddSelectedSocket(UStaticMeshSocket* InSelectedSocket)
{
	TArray<UStaticMeshSocket*> SelectedSockets = GetSelectedSockets();
	SelectedSockets.AddUnique(InSelectedSocket);
	SetSelectedSockets(SelectedSockets);
}

void SSocketManager::RemoveSelectedSocket(const UStaticMeshSocket* InSelectedSocket)
{
	TSharedPtr<SocketListItem> SocketItem = nullptr;

	for (int32 i = 0; i < SocketListView->GetItems().Num(); ++i)
	{
		if (SocketListView->GetItems()[i]->Socket == InSelectedSocket)
		{
			SocketItem = SocketListView->GetItems()[i];
			break;
		}
	}

	if (SocketItem == nullptr)
	{
		return;
	}

	SocketListView->SetItemSelection(SocketItem, false);

	SocketListView->RequestListRefresh();
	SocketSelectionChanged(GetSelectedSockets());
}

TSharedRef< ITableRow > SSocketManager::MakeWidgetFromOption( TSharedPtr<SocketListItem> InItem, const TSharedRef< STableViewBase >& OwnerTable )
{
	return SNew( SSocketDisplayItem, OwnerTable )
				.SocketItem(InItem)
				.SocketManagerPtr(SharedThis(this))
				.ReadOnly(bReadOnly);
}

void SSocketManager::CreateSocket()
{
	TSharedPtr<IStaticMeshEditor> StaticMeshEditorPinned = StaticMeshEditorPtr.Pin();
	if (StaticMeshEditorPinned.IsValid())
	{
		UStaticMesh* CurrentStaticMesh = StaticMeshEditorPinned->GetStaticMesh();

		const FScopedTransaction Transaction( LOCTEXT( "CreateSocket", "Create Socket" ) );

		UStaticMeshSocket* NewSocket = NewObject<UStaticMeshSocket>(CurrentStaticMesh);
		check(NewSocket);

		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.CreateSocket"));
		}

		FString SocketNameString = TEXT("Socket");
		FName SocketName = FName(*SocketNameString);

		// Make sure the new name is valid
		int32 Index = 0;
		while (CheckForDuplicateSocket(SocketName.ToString()))
		{
			SocketName = FName(*FString::Printf(TEXT("%s%i"), *SocketNameString, Index));
			++Index;
		}


		NewSocket->SocketName = SocketName;
		NewSocket->SetFlags( RF_Transactional );
		NewSocket->OnPropertyChanged().AddSP( this, &SSocketManager::OnSocketPropertyChanged );

		CurrentStaticMesh->PreEditChange(NULL);
		CurrentStaticMesh->AddSocket(NewSocket);
		CurrentStaticMesh->PostEditChange();
		CurrentStaticMesh->MarkPackageDirty();

		TSharedPtr< SocketListItem > SocketItem = MakeShareable( new SocketListItem(NewSocket) );
		SocketList.Add( SocketItem );
		SocketListView->RequestListRefresh();

		SocketListView->SetSelection(SocketItem);
		RequestRenameSelectedSocket();
	}
}

void SSocketManager::DuplicateSelectedSocket()
{
	TSharedPtr<IStaticMeshEditor> StaticMeshEditorPinned = StaticMeshEditorPtr.Pin();

	UStaticMeshSocket* SelectedSocket = GetSelectedSocket();

	if(StaticMeshEditorPinned.IsValid() && SelectedSocket)
	{
		const FScopedTransaction Transaction( LOCTEXT( "SocketManager_DuplicateSocket", "Duplicate Socket" ) );

		UStaticMesh* CurrentStaticMesh = StaticMeshEditorPinned->GetStaticMesh();

		UStaticMeshSocket* NewSocket = DuplicateObject(SelectedSocket, CurrentStaticMesh);

		// Create a unique name for this socket
		NewSocket->SocketName = MakeUniqueObjectName(CurrentStaticMesh, UStaticMeshSocket::StaticClass(), NewSocket->SocketName);

		// Add the new socket to the static mesh
		CurrentStaticMesh->PreEditChange(NULL);
		CurrentStaticMesh->AddSocket(NewSocket);
		CurrentStaticMesh->PostEditChange();
		CurrentStaticMesh->MarkPackageDirty();

		RefreshSocketList();

		// Select the duplicated socket
		SetSelectedSocket(NewSocket);
	}
}

void SSocketManager::DuplicateSelectedSockets()
{
	TSharedPtr<IStaticMeshEditor> StaticMeshEditorPinned = StaticMeshEditorPtr.Pin();

	TArray<UStaticMeshSocket*> SelectedSockets = GetSelectedSockets();

	if (!StaticMeshEditorPinned.IsValid() || SelectedSockets.Num() == 0)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("SocketManager_DuplicateSocket", "Duplicate Socket"));

	UStaticMesh* CurrentStaticMesh = StaticMeshEditorPinned->GetStaticMesh();

	CurrentStaticMesh->PreEditChange(NULL);

	TArray<UStaticMeshSocket*> NewSockets;
	NewSockets.Reserve(SelectedSockets.Num());
	for (UStaticMeshSocket* SelectedSocket : SelectedSockets)
	{
		UStaticMeshSocket* NewSocket = DuplicateObject(SelectedSocket, CurrentStaticMesh);
		NewSockets.Add(NewSocket);

		// Create a unique name for this socket
		NewSocket->SocketName = MakeUniqueObjectName(CurrentStaticMesh, UStaticMeshSocket::StaticClass(), NewSocket->SocketName);

		// Add the new socket to the static mesh
		CurrentStaticMesh->AddSocket(NewSocket);
	}

	CurrentStaticMesh->PostEditChange();
	CurrentStaticMesh->MarkPackageDirty();

	RefreshSocketList();

	// Select the duplicated sockets
	SetSelectedSockets(NewSockets);
}

void SSocketManager::UpdateStaticMesh()
{
	RefreshSocketList();
}

void SSocketManager::SetSelectedSockets(TArray<UStaticMeshSocket*>& InSelectedSockets)
{
	if (InSelectedSockets.Num())
	{
		TArray<TSharedPtr<SocketListItem>> SocketItems;

		for (const UStaticMeshSocket* Socket : InSelectedSockets)
		{
			for (int32 i = 0; i < SocketListView->GetItems().Num(); ++i)
			{
				if (SocketListView->GetItems()[i]->Socket == Socket)
				{
					SocketItems.Add(SocketListView->GetItems()[i]);
				}
			}
		}

		SocketListView->ClearSelection();
		SocketListView->SetItemSelection(SocketItems, true);

		SocketListView->RequestListRefresh();
		SocketSelectionChanged(InSelectedSockets);
	}
	else
	{
		SocketListView->ClearSelection();

		SocketListView->RequestListRefresh();

		SocketSelectionChanged(NULL);
	}
}

void SSocketManager::RequestRenameSelectedSocket()
{
	if(SocketListView->GetSelectedItems().Num() == 1)
	{
		TSharedPtr< SocketListItem > SocketItem = SocketListView->GetSelectedItems()[0];
		SocketListView->RequestScrollIntoView(SocketItem);
		DeferredRenameRequest = SocketItem;
	}
}

void SSocketManager::DeleteSelectedSocket()
{
	if(SocketListView->GetSelectedItems().Num())
	{
		const FScopedTransaction Transaction( LOCTEXT( "DeleteSocket", "Delete Socket" ) );

		TSharedPtr<IStaticMeshEditor> StaticMeshEditorPinned = StaticMeshEditorPtr.Pin();
		if (StaticMeshEditorPinned.IsValid())
		{
			UStaticMesh* CurrentStaticMesh = StaticMeshEditorPinned->GetStaticMesh();
			CurrentStaticMesh->PreEditChange(NULL);
			UStaticMeshSocket* SelectedSocket = SocketListView->GetSelectedItems()[ 0 ]->Socket;
			SelectedSocket->OnPropertyChanged().RemoveAll( this );
			CurrentStaticMesh->Sockets.Remove(SelectedSocket);
			CurrentStaticMesh->PostEditChange();

			RefreshSocketList();
		}
	}
}

void SSocketManager::DeleteSelectedSockets()
{
	if(SocketListView->GetSelectedItems().Num())
	{
		const FScopedTransaction Transaction(LOCTEXT("DeleteSockets", "Delete Sockets"));

		TSharedPtr<IStaticMeshEditor> StaticMeshEditorPinned = StaticMeshEditorPtr.Pin();
		if (StaticMeshEditorPinned.IsValid())
		{
			UStaticMesh* CurrentStaticMesh = StaticMeshEditorPinned->GetStaticMesh();
			CurrentStaticMesh->PreEditChange(NULL);
			for (int32 i = 0; i < SocketListView->GetSelectedItems().Num(); ++i)
			{
				UStaticMeshSocket* SelectedSocket = SocketListView->GetSelectedItems()[i]->Socket;
				SelectedSocket->OnPropertyChanged().RemoveAll(this);
				CurrentStaticMesh->Sockets.Remove(SelectedSocket);
			}

			CurrentStaticMesh->PostEditChange();

			RefreshSocketList();
		}
	}
}

void SSocketManager::RefreshSocketList()
{
	// The static mesh might not be the same one we built the SocketListView with
	// check it here and update it if necessary.
	TSharedPtr<IStaticMeshEditor> StaticMeshEditorPinned = StaticMeshEditorPtr.Pin();
	if (StaticMeshEditorPinned.IsValid())
	{
		bool bIsSameStaticMesh = true;
		UStaticMesh* CurrentStaticMesh = StaticMeshEditorPinned->GetStaticMesh();
		if (!StaticMesh.IsValid() || CurrentStaticMesh != StaticMesh.Get())
		{
			StaticMesh = CurrentStaticMesh;
			bIsSameStaticMesh = false;
		}
		// Only rebuild the socket list if it differs from the static meshes socket list
		// or the filter text changed since the last refresh 
		// This is done so that an undo on a socket property doesn't cause the selected
		// socket to be de-selected, thus hiding the socket properties on the detail view.
		// NB: Also force a rebuild if the underlying StaticMesh has been changed.
		bool bRefreshList = StaticMesh->Sockets.Num() != SocketList.Num() || !bIsSameStaticMesh || !FilterTextAtLastListRefresh.EqualTo(FilterText);
		if (!bRefreshList)
		{
			//Make sure list is in sync (UObject are the sames)
			for (int32 SocketIndex = 0; SocketIndex < StaticMesh->Sockets.Num(); ++SocketIndex)
			{
				if (StaticMesh->Sockets[SocketIndex].Get() != SocketList[SocketIndex].Get()->Socket)
				{
					bRefreshList = true;
					break;
				}
			}
		}

		if (bRefreshList)
		{
			SocketListView->ClearSelection();
			SocketList.Empty();
			
			const FString FilterString = *FilterText.ToString();

			for (int32 i = 0; i < StaticMesh->Sockets.Num(); i++)
			{
				UStaticMeshSocket* Socket = StaticMesh->Sockets[i];
				const FString SocketString = Socket->SocketName.ToString();

				// See if we pass the search filter
				if (!FilterText.IsEmpty() && !SocketString.Contains(FilterString))
				{
					continue;
				}

				SocketList.Add(MakeShareable(new SocketListItem(Socket)));
			}
			
			SocketListView->RequestListRefresh();
			FilterTextAtLastListRefresh = FilterText;
		}

		// Set the socket on the detail view to keep it in sync with the sockets properties
		if (SocketListView->GetSelectedItems().Num())
		{
			TArray< UObject* > ObjectList;
			ObjectList.Add(SocketListView->GetSelectedItems()[0]->Socket);
			SocketDetailsView->SetObjects(ObjectList, true);
		}

		StaticMeshEditorPinned->RefreshViewport();
	}
	else
	{
		SocketList.Empty();
		SocketListView->ClearSelection();
		SocketListView->RequestListRefresh();
	}
}

bool SSocketManager::CheckForDuplicateSocket(const FString& InSocketName)
{
	for( int32 i=0; i < SocketList.Num(); i++)
	{
		if(SocketList[i]->Socket->SocketName.ToString() == InSocketName)
		{
			return true;
		}
	}

	return false;
}

void SSocketManager::SocketSelectionChanged(UStaticMeshSocket* InSocket)
{
	TArray<UStaticMeshSocket*> SelectedSockets = GetSelectedSockets();
	
	TArray<UObject*> SelectedObjects;
	SelectedObjects.Reserve(SelectedSockets.Num());
	
	for (UStaticMeshSocket* Socket : SelectedSockets)
	{
		SelectedObjects.Add(Socket);
	}

	SocketDetailsView->SetObjects(SelectedObjects);

	// Notify listeners
	OnSocketSelectionChanged.ExecuteIfBound();
}

void SSocketManager::SocketSelectionChanged(TArray<UStaticMeshSocket*> InSockets)
{
	TArray<UObject*> SelectedObjects;
	SelectedObjects.Reserve(InSockets.Num());

	for (UStaticMeshSocket* Socket : InSockets)
	{
		if (Socket)
		{
			SelectedObjects.Add(Socket);
		}
	}

	SocketDetailsView->SetObjects(SelectedObjects);

	// Notify listeners
	OnSocketSelectionChanged.ExecuteIfBound();
}

void SSocketManager::SocketSelectionChanged_Execute(TSharedPtr<SocketListItem> InItem, ESelectInfo::Type /*SelectInfo*/)
{
	if(InItem.IsValid())
	{
		SocketSelectionChanged(InItem->Socket);
	}
	else
	{
		SocketSelectionChanged(NULL);
	}
}

FReply SSocketManager::CreateSocket_Execute()
{
	CreateSocket();

	return FReply::Handled();
}

EVisibility SSocketManager::CreateSocket_IsVisible() const
{
	return bReadOnly ? EVisibility::Collapsed : EVisibility::Visible;
}	

FText SSocketManager::GetSocketHeaderText() const
{
	UStaticMesh* CurrentStaticMesh = nullptr;
	TSharedPtr<IStaticMeshEditor> StaticMeshEditorPinned = StaticMeshEditorPtr.Pin();
	if (StaticMeshEditorPinned.IsValid())
	{
		CurrentStaticMesh = StaticMeshEditorPinned->GetStaticMesh();
	}
	return FText::Format(LOCTEXT("SocketHeader_TotalFmt", "{0} sockets"), FText::AsNumber((CurrentStaticMesh != nullptr) ? CurrentStaticMesh->Sockets.Num() : 0));
}

void SSocketManager::SocketName_TextChanged(const FText& InText)
{
	CheckForDuplicateSocket(InText.ToString());
}

TSharedPtr<SWidget> SSocketManager::OnContextMenuOpening()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;

	TSharedPtr<IStaticMeshEditor> StaticMeshEditorPinned = StaticMeshEditorPtr.Pin();
	if (!StaticMeshEditorPinned.IsValid())
	{
		return TSharedPtr<SWidget>();
	}

	FMenuBuilder MenuBuilder( bShouldCloseWindowAfterMenuSelection, StaticMeshEditorPinned->GetToolkitCommands());

	{
		MenuBuilder.BeginSection("BasicOperations");
		{
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Duplicate);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename);
		}
		MenuBuilder.EndSection();
	}

	return MenuBuilder.MakeWidget();
}

void SSocketManager::NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged )
{
	TArray< TSharedPtr< SocketListItem > > SelectedList = SocketListView->GetSelectedItems();
	if(SelectedList.Num())
	{
		if(PropertyThatChanged->GetName() == TEXT("Pitch") || PropertyThatChanged->GetName() == TEXT("Yaw") || PropertyThatChanged->GetName() == TEXT("Roll"))
		{
			const UStaticMeshSocket* Socket = SelectedList[0]->Socket;
			WorldSpaceRotation.Set( Socket->RelativeRotation.Pitch, Socket->RelativeRotation.Yaw, Socket->RelativeRotation.Roll );
		}
	}
}

void SSocketManager::AddPropertyChangeListenerToSockets()
{
	TSharedPtr<IStaticMeshEditor> StaticMeshEditorPinned = StaticMeshEditorPtr.Pin();
	if (StaticMeshEditorPinned.IsValid())
	{
		UStaticMesh* CurrentStaticMesh = StaticMeshEditorPinned->GetStaticMesh();
		for (int32 i = 0; i < CurrentStaticMesh->Sockets.Num(); ++i)
		{
			CurrentStaticMesh->Sockets[i]->OnPropertyChanged().AddSP(this, &SSocketManager::OnSocketPropertyChanged);
		}
	}
}

void SSocketManager::RemovePropertyChangeListenerFromSockets()
{
	TSharedPtr<IStaticMeshEditor> StaticMeshEditorPinned = StaticMeshEditorPtr.Pin();
	if (StaticMeshEditorPinned.IsValid())
	{
		UStaticMesh* CurrentStaticMesh = StaticMeshEditorPinned->GetStaticMesh();
		if (CurrentStaticMesh)
		{
			for (int32 i = 0; i < CurrentStaticMesh->Sockets.Num(); ++i)
			{
				CurrentStaticMesh->Sockets[i]->OnPropertyChanged().RemoveAll(this);
			}
		}
	}
}

void SSocketManager::OnSocketPropertyChanged( const UStaticMeshSocket* Socket, const FProperty* ChangedProperty )
{
	static FName RelativeRotationName(TEXT("RelativeRotation"));
	static FName RelativeLocationName(TEXT("RelativeLocation"));

	check(Socket != nullptr);

	FName ChangedPropertyName = ChangedProperty->GetFName();

	if ( ChangedPropertyName == RelativeRotationName )
	{
		TArray<UStaticMeshSocket*> SelectedSockets = GetSelectedSockets();
		for (const UStaticMeshSocket* SelectedSocket : SelectedSockets)
		{
			if (Socket == SelectedSocket)
			{
				WorldSpaceRotation.Set(Socket->RelativeRotation.Pitch, Socket->RelativeRotation.Yaw, Socket->RelativeRotation.Roll);
			}
		}
	}

	TSharedPtr<IStaticMeshEditor> StaticMeshEditorPinned = StaticMeshEditorPtr.Pin();
	if (!StaticMeshEditorPinned.IsValid())
	{
		return;
	}

	if (ChangedPropertyName == RelativeRotationName || ChangedPropertyName == RelativeLocationName)
	{
		// If socket location or rotation is changed, update the position of any actors attached to it in instances of this mesh
		UStaticMesh* CurrentStaticMesh = StaticMeshEditorPinned->GetStaticMesh();
		if (CurrentStaticMesh != nullptr)
		{
			bool bUpdatedChild = false;

			for (TObjectIterator<UStaticMeshComponent> It; It; ++It)
			{
				if (It->GetStaticMesh() == CurrentStaticMesh)
				{
					const AActor* Actor = It->GetOwner();
					if (Actor != nullptr)
					{
						const USceneComponent* Root = Actor->GetRootComponent();
						if (Root != nullptr)
						{
							for (USceneComponent* Child : Root->GetAttachChildren())
							{
								if (Child != nullptr && Child->GetAttachSocketName() == Socket->SocketName)
								{
									Child->UpdateComponentToWorld();
									bUpdatedChild = true;
								}
							}
						}
					}
				}
			}

			if (bUpdatedChild)
			{
				GUnrealEd->RedrawLevelEditingViewports();
			}
		}
	}
}

void SSocketManager::OnFilterTextChanged(const FText& SearchText)
{
	FilterText = SearchText;

	RefreshSocketList();
}

void SSocketManager::OnFilterTextCommitted(const FText& SearchText, ETextCommit::Type CommitInfo)
{
	// Just do the same as if the user typed in the box
	OnFilterTextChanged(SearchText);
}

void SSocketManager::PostUndo()
{
	RefreshSocketList();
}

void SSocketManager::OnItemScrolledIntoView( TSharedPtr<SocketListItem> InItem, const TSharedPtr<ITableRow>& InWidget)
{
	TSharedPtr<SocketListItem> DeferredRenameRequestPinned = DeferredRenameRequest.Pin();
	if( DeferredRenameRequestPinned.IsValid() )
	{
		DeferredRenameRequestPinned->OnRenameRequested.ExecuteIfBound();
		DeferredRenameRequest.Reset();
	}
}

#undef LOCTEXT_NAMESPACE
