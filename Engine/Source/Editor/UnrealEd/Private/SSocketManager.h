// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/NotifyHook.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Input/SSpinBox.h"
#include "IDetailsView.h"
#include "ISocketManager.h"

class IStaticMeshEditor;
class UStaticMesh;
class UStaticMeshSocket;
struct FPropertyChangedEvent;
struct SocketListItem;

class SSocketManager : public ISocketManager, public FNotifyHook
{
public:
	SLATE_BEGIN_ARGS( SSocketManager ) :
		 _StaticMeshEditorPtr(),
		_ReadOnly(false)
		{}
		SLATE_ARGUMENT( TSharedPtr< IStaticMeshEditor >, StaticMeshEditorPtr )
		SLATE_ARGUMENT(bool, ReadOnly)
		SLATE_EVENT( FSimpleDelegate, OnSocketSelectionChanged )
	SLATE_END_ARGS()

	virtual void Construct(const FArguments& InArgs);

	virtual ~SSocketManager();

	// ISocketManager interface
	virtual UStaticMeshSocket* GetSelectedSocket() const override;
	virtual TArray<UStaticMeshSocket*> GetSelectedSockets() const override;
	virtual bool HasSelectedSockets() const override;
	virtual void SetSelectedSocket(UStaticMeshSocket* InSelectedSocket) override;
	virtual void AddSelectedSocket(UStaticMeshSocket* InSelectedSocket) override;
	virtual void RemoveSelectedSocket(const UStaticMeshSocket* InSelectedSocket) override;
	virtual void DeleteSelectedSocket() override;
	virtual void DeleteSelectedSockets() override;
	virtual void DuplicateSelectedSocket() override;
	virtual void DuplicateSelectedSockets() override;
	virtual void RequestRenameSelectedSocket() override;
	virtual void UpdateStaticMesh() override;
	// End of ISocketManager

	virtual void SetSelectedSockets(TArray<UStaticMeshSocket*>& InSelectedSockets);

		/**
	 *	Checks for a duplicate socket using the name for comparison.
	 *
	 *	@param InSocketName			The name to compare.
	 *
	 *	@return						TRUE if another socket already exists with that name.
	 */
	bool CheckForDuplicateSocket(const FString& InSocketName);

private:
	/** Creates a widget from the list item. */
	TSharedRef< ITableRow > MakeWidgetFromOption( TSharedPtr< struct SocketListItem> InItem, const TSharedRef< STableViewBase >& OwnerTable );

	/**	Creates a socket with a specified name. */
	void CreateSocket();

	/** Refreshes the socket list. */
	void RefreshSocketList();

	/** Gets the visibility of the select a socket message */
	EVisibility GetSelectSocketMessageVisibility() const;

	/** 
	 *	Updates the details to the selected socket.
	 *
	 *	@param InSocket				The newly selected socket.
	 */
	void SocketSelectionChanged(UStaticMeshSocket* InSocket);
	void SocketSelectionChanged(TArray<UStaticMeshSocket*> InSockets);

	/** Callback for the list view when an item is selected. */
	void SocketSelectionChanged_Execute( TSharedPtr<SocketListItem> InItem, ESelectInfo::Type SelectInfo );

	/** Callback for the Create Socket button. */
	FReply CreateSocket_Execute();

	/** Determines if the Create Socket button is visible. */
	EVisibility CreateSocket_IsVisible() const;

	FText GetSocketHeaderText() const;

	/** Callback for when the socket name textbox is changed, verifies the name is not a duplicate. */
	void SocketName_TextChanged(const FText& InText);

	/** Callback to retrieve the context menu for the list view */
	TSharedPtr<SWidget> OnContextMenuOpening();

	/** FNotifyHook interface */
	virtual void NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged ) override;

	/** Post undo */
	void PostUndo();

	/** Callback when an item is scrolled into view, handling calls to rename items */
	void OnItemScrolledIntoView( TSharedPtr<SocketListItem> InItem, const TSharedPtr<ITableRow>& InWidget);
private:
	/** Add a property change listener to each socket. */
	void AddPropertyChangeListenerToSockets();

	/** Remove the property change listener from the sockets. */
	void RemovePropertyChangeListenerFromSockets();

	/** Called when a socket property has changed. */
	void OnSocketPropertyChanged( const UStaticMeshSocket* Socket, const FProperty* ChangedProperty );

	/**
	* Filters the SListView when the user changes the search text box (NameFilterBox)
	*
	* @param SearchText - The text the user has typed
	*
	*/
	void OnFilterTextChanged(const FText& SearchText);

	/**
	* Filters the SListView when the user hits enter or clears the search box
	* Simply calls OnFilterTextChanged
	*
	* @param SearchText - The text the user has typed
	* @param CommitInfo - Not used
	*
	*/
	void OnFilterTextCommitted(const FText& SearchText, ETextCommit::Type CommitInfo);

	/** Called when socket selection changes */
	FSimpleDelegate OnSocketSelectionChanged;

	/** Pointer back to the static mesh editor, used for */
	TWeakPtr< IStaticMeshEditor > StaticMeshEditorPtr;

	/** Details panel for the selected socket. */
	TSharedPtr<class IDetailsView> SocketDetailsView;

	/** List of sockets for for the associated static mesh or anim set. */
	TArray< TSharedPtr<SocketListItem> > SocketList;

	/** Listview for displaying the sockets. */
	TSharedPtr< SListView<TSharedPtr< SocketListItem > > > SocketListView;

	/** Box to filter to a specific static mesh name */
	TSharedPtr< class SSearchBox > NameFilterBox;

	/** Current text typed into NameFilterBox */
	FText FilterText;

	/** Text typed into NameFilterBox at the time of the last socket list refresh */
	FText FilterTextAtLastListRefresh;

	/** Helper variable for rotating in world space. */
	FVector WorldSpaceRotation;

	/** The static mesh being edited. */
	TWeakObjectPtr< UStaticMesh > StaticMesh;

	/** Widgets for the World Space Rotation */
	TSharedPtr< SSpinBox<float> > PitchRotation;
	TSharedPtr< SSpinBox<float> > YawRotation;
	TSharedPtr< SSpinBox<float> > RollRotation;

	/** Points to an item that is being requested to be renamed */
	TWeakPtr<SocketListItem> DeferredRenameRequest;

	/** true if the owning asset editor is in read only mode*/
	bool bReadOnly;
};
