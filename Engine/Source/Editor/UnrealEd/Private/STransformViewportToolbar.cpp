// Copyright Epic Games, Inc. All Rights Reserved.


#include "STransformViewportToolbar.h"

#include "EditorInteractiveGizmoManager.h"
#include "EngineDefines.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/SlateTypes.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSlider.h"
#include "Styling/AppStyle.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorViewportClient.h"
#include "UnrealEdGlobals.h"
#include "SEditorViewport.h"
#include "EditorViewportCommands.h"
#include "SViewportToolBarIconMenu.h"
#include "SViewportToolBarComboMenu.h"
#include "ISettingsModule.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Settings/EditorProjectSettings.h"
#include "LevelEditor.h"
#include "LevelEditorActions.h"
#include "Styling/ToolBarStyle.h"
#include "SEditorViewportToolBarMenu.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"

#define LOCTEXT_NAMESPACE "TransformToolBar"


void STransformViewportToolBar::Construct( const FArguments& InArgs )
{
	Viewport = InArgs._Viewport;
	CommandList = InArgs._CommandList;
	OnCamSpeedChanged = InArgs._OnCamSpeedChanged;
	OnCamSpeedScalarChanged = InArgs._OnCamSpeedScalarChanged;

	ChildSlot
	[
		MakeTransformToolBar(InArgs._Extenders)
	];

	SViewportToolBar::Construct(SViewportToolBar::FArguments());
}

TSharedRef< SWidget > STransformViewportToolBar::MakeSurfaceSnappingButton()
{
	check(!SurfaceSnappingMenu.IsValid());
	SurfaceSnappingMenu = 
		SNew(SEditorViewportToolbarMenu)
		.ParentToolBar(SharedThis(this))
		.Image("EditorViewport.ToggleSurfaceSnapping")
		.ToolTipText(LOCTEXT("SnapToSurfaceMenu_ToolTip", "Control how objects snap to surfaces"))
		.OnGetMenuContent(this, &STransformViewportToolBar::GenerateSurfaceSnappingMenu)
		.ForegroundColor(this, &STransformViewportToolBar::GetSurfaceSnappingForegroundColor);

	return SurfaceSnappingMenu.ToSharedRef();
}

TSharedRef<SWidget> STransformViewportToolBar::GenerateSurfaceSnappingMenu()
{
	auto IsSnappingEnabled = [] {
		return GetDefault<ULevelEditorViewportSettings>()->SnapToSurface.bEnabled;
	};

	const bool bCloseAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, CommandList);

	MenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().SurfaceSnapping);

	MenuBuilder.BeginSection("SurfaceSnappingSettings", LOCTEXT("SnapToSurfaceSettings", "Settings"));
	{
		MenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().RotateToSurfaceNormal);

		MenuBuilder.AddWidget(

			SNew( SBox )
			.Padding( FMargin(8.0f, 0.0f, 0.0f, 0.0f) )
			.MinDesiredWidth( 100.0f )
			[
				SNew ( SBorder )
				.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
				.Padding(FMargin(1.0f))
				[
					SNew(SNumericEntryBox<float>)
					.IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateStatic(IsSnappingEnabled)))
					.Value(
						TAttribute<TOptional<float>>::Create(TAttribute<TOptional<float>>::FGetter::CreateStatic([] 
						{
							const auto& Settings = GetDefault<ULevelEditorViewportSettings>()->SnapToSurface;
							return TOptional<float>(Settings.SnapOffsetExtent);
						}))
					)
					.OnValueChanged(SNumericEntryBox<float>::FOnValueChanged::CreateStatic([](float Val) 
					{
						GetMutableDefault<ULevelEditorViewportSettings>()->SnapToSurface.SnapOffsetExtent = Val;
					}))
					.MinValue(0.f)
					.MaxValue(static_cast<float>(HALF_WORLD_MAX))
					.MaxSliderValue(1000.f) // 'Sensible' range for the slider (10m)
					.AllowSpin(true)
				]
			],
			LOCTEXT("SnapToSurfaceSettings_Offset", "Surface Offset")
		);

	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

FSlateColor STransformViewportToolBar::GetSurfaceSnappingForegroundColor() const
{
	static const FCheckBoxStyle& ViewportToolbarCheckStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("EditorViewportToolBar").ToggleButton;

	const bool bSurfaceSnappingEnabled = GetDefault<ULevelEditorViewportSettings>()->SnapToSurface.bEnabled;

	bool bShouldAppearHovered = SurfaceSnappingMenu->IsHovered() || SurfaceSnappingMenu->IsMenuOpen();
	// Hovered and checked
	if (bShouldAppearHovered && bSurfaceSnappingEnabled)
	{
		return ViewportToolbarCheckStyle.CheckedHoveredForeground;
	}
	// Not hovered and checked
	else if (bSurfaceSnappingEnabled)
	{
		return ViewportToolbarCheckStyle.CheckedForeground;
	}
	// Hovered not checked
	else if (bShouldAppearHovered)
	{
		return ViewportToolbarCheckStyle.HoveredForeground;
	}
	// Not hovered not checked
	else
	{
		return ViewportToolbarCheckStyle.ForegroundColor;
	}
}

TSharedRef< SWidget > STransformViewportToolBar::MakeTransformToolBar( const TSharedPtr< FExtender > InExtenders )
{
	FSlimHorizontalToolBarBuilder ToolbarBuilder( CommandList, FMultiBoxCustomization::None, InExtenders );

	// Use a custom style
	FName ToolBarStyle = "EditorViewportToolBar";
	ToolbarBuilder.SetStyle(&FAppStyle::Get(), ToolBarStyle);
	ToolbarBuilder.SetLabelVisibility(EVisibility::Collapsed);

	// Transform controls cannot be focusable as it fights with the press space to change transform mode feature
	ToolbarBuilder.SetIsFocusable( false );

	ToolbarBuilder.BeginSection("Transform");
	{
		ToolbarBuilder.BeginBlockGroup();

		// Select Mode
		static FName SelectModeName = FName(TEXT("SelectMode"));
		ToolbarBuilder.AddToolBarButton(FEditorViewportCommands::Get().SelectMode, NAME_None, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>(), SelectModeName);

		// Translate Mode
		static FName TranslateModeName = FName(TEXT("TranslateMode"));
		ToolbarBuilder.AddToolBarButton( FEditorViewportCommands::Get().TranslateMode, NAME_None, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>(), TranslateModeName );
		
		// TranslateRotate Mode
		static FName TranslateRotateModeName = FName(TEXT("TranslateRotateMode"));
		ToolbarBuilder.AddToolBarButton( FEditorViewportCommands::Get().TranslateRotateMode, NAME_None, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>(), TranslateRotateModeName );

		// 2D Mode
		static FName TranslateRotate2DModeName = FName(TEXT("TranslateRotate2DMode"));
		ToolbarBuilder.AddToolBarButton(FEditorViewportCommands::Get().TranslateRotate2DMode, NAME_None, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>(), TranslateRotate2DModeName);

		// Rotate Mode
		static FName RotateModeName = FName(TEXT("RotateMode"));
		ToolbarBuilder.AddToolBarButton( FEditorViewportCommands::Get().RotateMode, NAME_None, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>(), RotateModeName );

		// Scale Mode
		static FName ScaleModeName = FName(TEXT("ScaleMode"));
		ToolbarBuilder.AddToolBarButton( FEditorViewportCommands::Get().ScaleMode, NAME_None, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>(), ScaleModeName );


		ToolbarBuilder.EndBlockGroup();
		ToolbarBuilder.AddSeparator();

		ToolbarBuilder.SetIsFocusable( true );

		TAttribute<FText> CoordSystemToolTip = TAttribute<FText>::CreateLambda([]
		{
			if (UEditorInteractiveGizmoManager::UsesNewTRSGizmos())
			{
				return LOCTEXT(	"CycleTransformGizmoCoordSystemWithParent_ToolTip",
								"Cycles the transform gizmo coordinate systems between world, local, parent and explicit space");
			}
			return FEditorViewportCommands::Get().CycleTransformGizmoCoordSystem->GetDescription();
		});
		
		ToolbarBuilder.AddToolBarButton( FEditorViewportCommands::Get().CycleTransformGizmoCoordSystem,
			NAME_None,
			TAttribute<FText>(),
			CoordSystemToolTip,
			TAttribute<FSlateIcon>(this, &STransformViewportToolBar::GetLocalToWorldIcon),
			FName(TEXT("CycleTransformGizmoCoordSystem")),

			// explictly specify what this widget should look like as a menu item
			FNewMenuDelegate::CreateLambda( []( FMenuBuilder& InMenuBuilder )
			{
				InMenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().RelativeCoordinateSystem_World);
				InMenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().RelativeCoordinateSystem_Local);
			}
		));

	}

	ToolbarBuilder.EndSection();

	ToolbarBuilder.BeginSection("LocationGridSnap");
	{
		static FName SurfaceSnapName = FName(TEXT("SurfaceSnap"));
		ToolbarBuilder.AddWidget( MakeSurfaceSnappingButton(), SurfaceSnapName, false, HAlign_Fill, 
			FNewMenuDelegate::CreateLambda( [this]( FMenuBuilder& InMenuBuilder )
			{
				InMenuBuilder.AddWrapperSubMenu(
					LOCTEXT("SnapToSurfaceMenuSettings", "Surface Snap Settings"),
					LOCTEXT("SnapToSurfaceMenuSettings_Tooltip", "Snap To Surface Settings"),
					FOnGetContent::CreateSP(this, &STransformViewportToolBar::GenerateSurfaceSnappingMenu),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.ToggleSurfaceSnapping")
				);
			}
		));

		ToolbarBuilder.AddSeparator();

		// Grab the existing UICommand 
		TSharedPtr<FUICommandInfo> Command = FEditorViewportCommands::Get().LocationGridSnap;

		static FName PositionSnapName = FName(TEXT("PositionSnap"));

		// Setup a GridSnapSetting with the UICommand
		ToolbarBuilder.AddWidget(
			SNew(SViewportToolBarComboMenu)
				.IsChecked(this, &STransformViewportToolBar::IsLocationGridSnapChecked)
				.OnCheckStateChanged(this, &STransformViewportToolBar::HandleToggleLocationGridSnap)
				.Label(TAttribute<FText>::Create(&UE::UnrealEd::GetLocationGridLabel))
				.OnGetMenuContent(this, &STransformViewportToolBar::FillLocationGridSnapMenu)
				.ToggleButtonToolTip(Command->GetDescription())
				.MenuButtonToolTip(LOCTEXT("LocationGridSnap_ToolTip", "Set the Position Grid Snap value"))
				.Icon(Command->GetIcon())
				.MinDesiredButtonWidth(24.0f)
				.ParentToolBar(SharedThis(this)),
			PositionSnapName,
			false,
			HAlign_Fill,

			// explictly specify what this widget should look like as a menu item
			FNewMenuDelegate::CreateLambda(
				[this, Command](FMenuBuilder& InMenuBuilder)
				{
					FUIAction Action;
					Action.ExecuteAction = FExecuteAction::CreateLambda(
						[this]()
						{
							constexpr ECheckBoxState UnusedCheckedState = ECheckBoxState::Unchecked;
							this->HandleToggleLocationGridSnap(UnusedCheckedState);
						}
					);
					Action.GetActionCheckState =
						FGetActionCheckState::CreateRaw(this, &STransformViewportToolBar::IsLocationGridSnapChecked);

					const FName UnusedExtensionHook = NAME_None;
					InMenuBuilder.AddMenuEntry(
						Command->GetLabel(),
						Command->GetDescription(),
						Command->GetIcon(),
						Action,
						UnusedExtensionHook,
						EUserInterfaceActionType::ToggleButton
					);

					InMenuBuilder.AddWrapperSubMenu(
						LOCTEXT("GridSnapMenuSettings", "Grid Snap Settings"),
						LOCTEXT("GridSnapMenuSettings_ToolTip", "Set the Position Grid Snap value"),
						FOnGetContent::CreateSP(this, &STransformViewportToolBar::FillLocationGridSnapMenu),
						FSlateIcon(Command->GetIcon())
					);
				}
			)
		);
	}
	ToolbarBuilder.EndSection();

	ToolbarBuilder.BeginSection("RotationGridSnap");
	{
		// Grab the existing UICommand 
		TSharedPtr<FUICommandInfo> Command = FEditorViewportCommands::Get().RotationGridSnap;

		static FName RotationSnapName = FName(TEXT("RotationSnap"));

		// Setup a GridSnapSetting with the UICommand
		ToolbarBuilder.AddWidget(
			SNew(SViewportToolBarComboMenu)
				.IsChecked(this, &STransformViewportToolBar::IsRotationGridSnapChecked)
				.OnCheckStateChanged(this, &STransformViewportToolBar::HandleToggleRotationGridSnap)
				.Label(TAttribute<FText>::Create(&UE::UnrealEd::GetRotationGridLabel))
				.OnGetMenuContent(this, &STransformViewportToolBar::FillRotationGridSnapMenu)
				.ToggleButtonToolTip(Command->GetDescription())
				.MenuButtonToolTip(LOCTEXT("RotationGridSnap_ToolTip", "Set the Rotation Grid Snap value"))
				.Icon(Command->GetIcon())
				.ParentToolBar(SharedThis(this)),
			RotationSnapName,
			false,
			HAlign_Fill,

			// explictly specify what this widget should look like as a menu item
			FNewMenuDelegate::CreateLambda(
				[this, Command](FMenuBuilder& InMenuBuilder)
				{
					FUIAction Action;
					Action.ExecuteAction = FExecuteAction::CreateLambda(
						[this]()
						{
							constexpr ECheckBoxState UnusedCheckedState = ECheckBoxState::Unchecked;
							this->HandleToggleRotationGridSnap(UnusedCheckedState);
						}
					);
					Action.GetActionCheckState =
						FGetActionCheckState::CreateRaw(this, &STransformViewportToolBar::IsRotationGridSnapChecked);

					const FName UnusedExtensionHook = NAME_None;
					InMenuBuilder.AddMenuEntry(
						Command->GetLabel(),
						Command->GetDescription(),
						Command->GetIcon(),
						Action,
						UnusedExtensionHook,
						EUserInterfaceActionType::ToggleButton
					);

					InMenuBuilder.AddWrapperSubMenu(
						LOCTEXT("RotationGridSnapMenuSettings", "Rotation Snap Settings"),
						LOCTEXT("RotationGridSnapMenuSettings_ToolTip", "Adjust the Grid Settings for Rotation Snap"),
						FOnGetContent::CreateSP(this, &STransformViewportToolBar::FillRotationGridSnapMenu),
						FSlateIcon(Command->GetIcon())
					);
				}
			)
		);
	}
	ToolbarBuilder.EndSection();

	ToolbarBuilder.BeginSection("Layer2DSnap");
	{
		// Grab the existing UICommand 
		TSharedPtr<FUICommandInfo> Command = FEditorViewportCommands::Get().Layer2DSnap;

		static FName Layer2DSnapName = FName(TEXT("Layer2DSnap"));

		TSharedRef<SWidget> SnapLayerPickerWidget =
			SNew(SViewportToolBarComboMenu)
			.Visibility(this, &STransformViewportToolBar::IsLayer2DSnapVisible)
			.IsChecked(this, &STransformViewportToolBar::IsLayer2DSnapChecked)
			.OnCheckStateChanged(this, &STransformViewportToolBar::HandleToggleLayer2DSnap)
			.Label(this, &STransformViewportToolBar::GetLayer2DLabel)
			.OnGetMenuContent(this, &STransformViewportToolBar::FillLayer2DSnapMenu)
			.ToggleButtonToolTip(Command->GetDescription())
			.MenuButtonToolTip(LOCTEXT("Layer2DSnap_ToolTip", "Set the 2d layer snap value"))
			.Icon(Command->GetIcon())
			.ParentToolBar(SharedThis(this))
			.MinDesiredButtonWidth(88.0f);

		ToolbarBuilder.AddWidget(
			SnapLayerPickerWidget, 
			Layer2DSnapName,
			false, 
			HAlign_Fill, 

			// explictly specify what this widget should look like as a menu item
			FNewMenuDelegate::CreateLambda( [this, Command]( FMenuBuilder& InMenuBuilder )
			{
				if (IsLayer2DSnapVisible() == EVisibility::Visible)
				{
					FUIAction Action;
					Action.ExecuteAction = FExecuteAction::CreateLambda(
						[this]()
						{
							constexpr ECheckBoxState UnusedCheckedState = ECheckBoxState::Unchecked;
							this->HandleToggleLayer2DSnap(UnusedCheckedState);
						}
					);
					Action.GetActionCheckState =
						FGetActionCheckState::CreateRaw(this, &STransformViewportToolBar::IsLayer2DSnapChecked);

					const FName UnusedExtensionHook = NAME_None;
					InMenuBuilder.AddMenuEntry(
						Command->GetLabel(),
						Command->GetDescription(),
						Command->GetIcon(),
						Action,
						UnusedExtensionHook,
						EUserInterfaceActionType::ToggleButton
					);

					InMenuBuilder.AddWrapperSubMenu(
						LOCTEXT("Layer2DSnapMenuSettings", "Layer 2D Snap Settings"),
						LOCTEXT("Layer2DSnapMenuSettings_ToolTip", "Adjust the Grid Settings for Layer 2D Snap"),
						FOnGetContent::CreateSP(this, &STransformViewportToolBar::FillLayer2DSnapMenu),
						FSlateIcon(Command->GetIcon())
					);
				}
			}
		));
	}
	ToolbarBuilder.EndSection();

	ToolbarBuilder.BeginSection("ScaleGridSnap");
	{
		// Grab the existing UICommand
		TSharedPtr<FUICommandInfo> Command = FEditorViewportCommands::Get().ScaleGridSnap;

		static FName ScaleSnapName = FName(TEXT("ScaleSnap"));

		// Setup a GridSnapSetting with the UICommand
		ToolbarBuilder.AddWidget(
			SNew(SViewportToolBarComboMenu)
				.Cursor(EMouseCursor::Default)
				.IsChecked(this, &STransformViewportToolBar::IsScaleGridSnapChecked)
				.OnCheckStateChanged(this, &STransformViewportToolBar::HandleToggleScaleGridSnap)
				.Label(TAttribute<FText>::Create(&UE::UnrealEd::GetScaleGridLabel))
				.OnGetMenuContent(this, &STransformViewportToolBar::FillScaleGridSnapMenu)
				.ToggleButtonToolTip(Command->GetDescription())
				.MenuButtonToolTip(LOCTEXT("ScaleGridSnap_ToolTip", "Set scaling options"))
				.Icon(Command->GetIcon())
				.MinDesiredButtonWidth(24.0f)
				.ParentToolBar(SharedThis(this)),
			ScaleSnapName,
			false,
			HAlign_Fill,

			// explictly specify what this widget should look like as a menu item
			FNewMenuDelegate::CreateLambda(
				[this, Command](FMenuBuilder& InMenuBuilder)
				{
					FUIAction Action;
					Action.ExecuteAction = FExecuteAction::CreateLambda(
						[this]()
						{
							constexpr ECheckBoxState UnusedCheckedState = ECheckBoxState::Unchecked;
							this->HandleToggleScaleGridSnap(UnusedCheckedState);
						}
					);
					Action.GetActionCheckState =
						FGetActionCheckState::CreateRaw(this, &STransformViewportToolBar::IsScaleGridSnapChecked);

					const FName UnusedExtensionHook = NAME_None;
					InMenuBuilder.AddMenuEntry(
						Command->GetLabel(),
						Command->GetDescription(),
						Command->GetIcon(),
						Action,
						UnusedExtensionHook,
						EUserInterfaceActionType::ToggleButton
					);

					InMenuBuilder.AddWrapperSubMenu(
						LOCTEXT("ScaleGridSnapMenuSettings", "Scale Snap Settings"),
						LOCTEXT("ScaleGridSnapMenuSettings_ToolTip", "Adjust the Grid Settings for Scale Snap"),
						FOnGetContent::CreateSP(this, &STransformViewportToolBar::FillScaleGridSnapMenu),
						FSlateIcon(Command->GetIcon())
					);
				}
			)
		);
	}
	ToolbarBuilder.EndSection();

	ToolbarBuilder.BeginSection("CameraSpeed");
	{
		static FName CameraSpeedName = FName(TEXT("CameraSpeed"));

		// Camera speed 
		ToolbarBuilder.AddWidget(	
			SNew(SEditorViewportToolbarMenu)
				.ParentToolBar(SharedThis(this))
				.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("CameraSpeedButton")))
				.ToolTipText(UE::UnrealEd::GetCameraSpeedTooltip())
				.LabelIcon(FAppStyle::Get().GetBrush("EditorViewport.CamSpeedSetting"))
				.Label(this, &STransformViewportToolBar::GetCameraSpeedLabel)
				// Anchor to the right, otherwise the slider in this menu will jitter when the label width changes
				.MenuPlacement(MenuPlacement_BelowRightAnchor)
				.OnGetMenuContent(this, &STransformViewportToolBar::FillCameraSpeedMenu),
			CameraSpeedName,
			false,
			HAlign_Fill,

			// explictly specify what this widget should look like as a menu item
			FNewMenuDelegate::CreateLambda( [this]( FMenuBuilder& InMenuBuilder )
			{
				InMenuBuilder.AddWrapperSubMenu(
					LOCTEXT("CameraSpeedMenuSettings", "Camera Speed Settings"),
					LOCTEXT("CameraSpeedMenuSettings_ToolTip", "Adjust the camera navigation speed"),
					FOnGetContent::CreateSP(this, &STransformViewportToolBar::FillCameraSpeedMenu),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.CamSpeedSetting")
				);
			}
		));


	}
	ToolbarBuilder.EndSection();

	return ToolbarBuilder.MakeWidget();
}


TSharedRef<SWidget> STransformViewportToolBar::FillCameraSpeedMenu()
{
	TSharedRef<SWidget> ReturnWidget = SNew(SBorder)
	.BorderImage(FAppStyle::GetBrush(TEXT("Menu.Background")))
	[
		SNew( SVerticalBox )
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding( FMargin(8.0f, 2.0f, 60.0f, 2.0f) )
		.HAlign( HAlign_Left )
		[
			SNew( STextBlock )
			.Text( LOCTEXT("MouseSettingsCamSpeed", "Camera Speed")  )
			.Font( FAppStyle::GetFontStyle( TEXT( "MenuItem.Font" ) ) )
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding( FMargin(8.0f, 4.0f) )
		[	
			SNew( SHorizontalBox )
			+SHorizontalBox::Slot()
			.FillWidth(1)
			.Padding( FMargin(0.0f, 2.0f) )
			[
				SNew( SBox )
				.MinDesiredWidth(220)
				[
					SNew(SSlider)
					.Value(this, &STransformViewportToolBar::GetCamSpeedSliderPosition)
					.OnValueChanged(this, &STransformViewportToolBar::OnSetCamSpeed)
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding( 8.0f, 2.0f, 0.0f, 2.0f)
			[
				SNew( SBox )
				.WidthOverride(40)
				[
					SNew( STextBlock )
					.Text(this, &STransformViewportToolBar::GetCameraSpeedLabel )
					.Font( FAppStyle::GetFontStyle( TEXT( "MenuItem.Font" ) ) )
				]
			]
		]
	];

	return ReturnWidget;
}

FReply STransformViewportToolBar::OnCycleCoordinateSystem()
{
	if( Viewport.IsValid() )
	{
		Viewport.Pin()->OnCycleCoordinateSystem();
	}

	return FReply::Handled();
}

FSlateIcon STransformViewportToolBar::GetLocalToWorldIcon() const
{
	ECoordSystem CoordSystem = ECoordSystem::COORD_Local;
	if (TSharedPtr<SEditorViewport> PinnedViewport = Viewport.Pin())
	{
		CoordSystem = PinnedViewport->GetViewportClient()->GetWidgetCoordSystemSpace();
	}
	return UE::UnrealEd::GetIconFromCoordSystem(CoordSystem);
}

FText STransformViewportToolBar::GetLayer2DLabel() const
{
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
	const ULevelEditor2DSettings* Settings2D = GetDefault<ULevelEditor2DSettings>();
	if (Settings2D->SnapLayers.IsValidIndex(ViewportSettings->ActiveSnapLayerIndex))
	{
		return FText::FromString(Settings2D->SnapLayers[ViewportSettings->ActiveSnapLayerIndex].Name);
	}

	return FText();
}

FText STransformViewportToolBar::GetCameraSpeedLabel() const
{
	return UE::UnrealEd::GetCameraSpeedLabel(Viewport);
}

float STransformViewportToolBar::GetCamSpeedSliderPosition() const
{
	float SliderPos = 0.f;

	if (TSharedPtr<SEditorViewport> ViewportPin = Viewport.Pin())
	{
		if (TSharedPtr<FEditorViewportClient> Client = ViewportPin->GetViewportClient())
		{
			SliderPos = Client->GetCameraSpeedSettings().GetRelativeUISpeed();
		}
	}
	
	return SliderPos;
}


void STransformViewportToolBar::OnSetCamSpeed(float NewValue)
{
	if (TSharedPtr<SEditorViewport> ViewportPin = Viewport.Pin())
	{
		if (TSharedPtr<FEditorViewportClient> Client = ViewportPin->GetViewportClient())
		{
			FEditorViewportCameraSpeedSettings Settings = Client->GetCameraSpeedSettings();
			Settings.SetRelativeUISpeed(NewValue);
			Client->SetCameraSpeedSettings(Settings);
			OnCamSpeedChanged.ExecuteIfBound(Settings.GetCurrentSpeed());
		}
	}
}

/**
	* Sets our grid size based on what the user selected in the UI
	*
	* @param	InIndex		The new index of the grid size to use
	*/
void STransformViewportToolBar::SetGridSize( int32 InIndex )
{
	GEditor->SetGridSize( InIndex );
}


/**
	* Sets the rotation grid size
	*
	* @param	InIndex		The new index of the rotation grid size to use
	* @param	InGridMode	Controls whether to use Preset or User selected values
	*/
void STransformViewportToolBar::SetRotationGridSize( int32 InIndex, ERotationGridMode InGridMode )
{
	GEditor->SetRotGridSize( InIndex, InGridMode );
}

		
/**
	* Sets the scale grid size
	*
	* @param	InIndex	The new index of the scale grid size to use
	*/
void STransformViewportToolBar::SetScaleGridSize( int32 InIndex )
{
	GEditor->SetScaleGridSize( InIndex );
}

/**
 * Sets the active 2d snap layer 
 *
 * @param	InIndex	The new index of the 2d layer to use
 */
void STransformViewportToolBar::SetLayer2D( int32 Layer2DIndex )
{
	ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>();
	ViewportSettings->bEnableLayerSnap = true;
	ViewportSettings->ActiveSnapLayerIndex = Layer2DIndex;
	ViewportSettings->PostEditChange();
}

		
/**
	* Checks to see if the specified grid size index is the current grid size index
	*
	* @param	GridSizeIndex	The grid size index to test
	*
	* @return	True if the specified grid size index is the current one
	*/
bool STransformViewportToolBar::IsGridSizeChecked( int32 GridSizeIndex )
{
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
	return (ViewportSettings->CurrentPosGridSize == GridSizeIndex);
}


/**
	* Checks to see if the specified rotation grid angle is the current rotation grid angle
	*
	* @param	GridSizeIndex	The grid size index to test
	* @param	InGridMode		Controls whether to use Preset or User selected values
	*
	* @return	True if the specified rotation grid size angle is the current one
	*/
bool STransformViewportToolBar::IsRotationGridSizeChecked( int32 GridSizeIndex, ERotationGridMode GridMode )
{
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
	return (ViewportSettings->CurrentRotGridSize == GridSizeIndex) && (ViewportSettings->CurrentRotGridMode == GridMode);
}


/**
	* Checks to see if the specified scale grid size is the current scale grid size
	*
	* @param	GridSizeIndex	The grid size index to test
	*
	* @return	True if the specified scale grid size is the current one
	*/
bool STransformViewportToolBar::IsScaleGridSizeChecked(int32 GridSizeIndex)
{
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
	return (ViewportSettings->CurrentScalingGridSize == GridSizeIndex);
}

bool STransformViewportToolBar::IsLayer2DSelected(int32 LayerIndex)
{
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
	return (ViewportSettings->ActiveSnapLayerIndex == LayerIndex);
}

void STransformViewportToolBar::TogglePreserveNonUniformScale()
{
	ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>();
	ViewportSettings->PreserveNonUniformScale = !ViewportSettings->PreserveNonUniformScale;
}

bool STransformViewportToolBar::IsPreserveNonUniformScaleChecked()
{
	return GetDefault<ULevelEditorViewportSettings>()->PreserveNonUniformScale;
}

TSharedRef<SWidget> STransformViewportToolBar::FillLocationGridSnapMenu()
{
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
	TArray<float> GridSizes = ViewportSettings->bUsePowerOf2SnapSize ? ViewportSettings->Pow2GridSizes
																	 : ViewportSettings->DecimalGridSizes;

	UE::UnrealEd::FLocationGridCheckboxListExecuteActionDelegate ExecuteDelegate =
		UE::UnrealEd::FLocationGridCheckboxListExecuteActionDelegate::CreateLambda(
			[](int CurrGridSizeIndex)
			{
				STransformViewportToolBar::SetGridSize(CurrGridSizeIndex);
			}
		);

	UE::UnrealEd::FLocationGridCheckboxListIsCheckedDelegate IsCheckedDelegate =
		UE::UnrealEd::FLocationGridCheckboxListIsCheckedDelegate::CreateLambda(
			[](int CurrGridSizeIndex)
			{
				return STransformViewportToolBar::IsGridSizeChecked(CurrGridSizeIndex);
			}
		);

	UE::UnrealEd::FLocationGridSnapMenuOptions MenuOptions;
	MenuOptions.CommandList = CommandList;

	return UE::UnrealEd::CreateLocationGridSnapMenu(MenuOptions);
}

TSharedRef<SWidget> STransformViewportToolBar::FillRotationGridSnapMenu()
{
	UE::UnrealEd::FRotationGridCheckboxListExecuteActionDelegate ExecuteDelegate =
		UE::UnrealEd::FRotationGridCheckboxListExecuteActionDelegate::CreateStatic(&STransformViewportToolBar::SetRotationGridSize
		);

	UE::UnrealEd::FRotationGridCheckboxListIsCheckedDelegate IsCheckedDelegate =
		UE::UnrealEd::FRotationGridCheckboxListIsCheckedDelegate::CreateStatic(
			&STransformViewportToolBar::IsRotationGridSizeChecked
		);

	return UE::UnrealEd::CreateRotationGridSnapMenu(ExecuteDelegate, IsCheckedDelegate, CommandList);
}

TSharedRef<SWidget> STransformViewportToolBar::FillLayer2DSnapMenu()
{
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
	const ULevelEditor2DSettings* Settings2D = GetDefault<ULevelEditor2DSettings>();
	int32 LayerCount = Settings2D->SnapLayers.Num();
	const bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder ShowMenuBuilder(bInShouldCloseWindowAfterMenuSelection, CommandList); // rename
	for (int32 LayerIndex = 0; LayerIndex < LayerCount; ++LayerIndex)
	{
		FName LayerName(*Settings2D->SnapLayers[LayerIndex].Name);

		FUIAction Action(FExecuteAction::CreateStatic(&STransformViewportToolBar::SetLayer2D, LayerIndex),
			FCanExecuteAction(),
			FIsActionChecked::CreateStatic(&STransformViewportToolBar::IsLayer2DSelected, LayerIndex));

		ShowMenuBuilder.AddMenuEntry(FText::FromName(LayerName), FText::GetEmpty(), FSlateIcon(), Action, NAME_None, EUserInterfaceActionType::RadioButton);
	}

	struct FLocalFunctions
	{
		static void ShowSettingsViewer()
		{
			if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
			{
				SettingsModule->ShowViewer("Project", "Editor", "LevelEditor2DSettings");
			}
		}
	};

	FUIAction ShowSettingsAction(FExecuteAction::CreateStatic(&FLocalFunctions::ShowSettingsViewer));
	ShowMenuBuilder.AddMenuEntry(LOCTEXT("2DSnap_EditLayer", "Edit Layers..."), FText::GetEmpty(), FSlateIcon(), ShowSettingsAction, NAME_None, EUserInterfaceActionType::Button);

	// -------------------------------------------------------
	ShowMenuBuilder.AddMenuSeparator();


	FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
	ShowMenuBuilder.AddMenuEntry(LevelEditor.GetLevelEditorCommands().SnapTo2DLayer);

	ShowMenuBuilder.AddMenuSeparator();
	ShowMenuBuilder.AddMenuEntry(LevelEditor.GetLevelEditorCommands().MoveSelectionToTop2DLayer);
	ShowMenuBuilder.AddMenuEntry(LevelEditor.GetLevelEditorCommands().MoveSelectionUpIn2DLayers);
	ShowMenuBuilder.AddMenuEntry(LevelEditor.GetLevelEditorCommands().MoveSelectionDownIn2DLayers);
	ShowMenuBuilder.AddMenuEntry(LevelEditor.GetLevelEditorCommands().MoveSelectionToBottom2DLayer);
	
	ShowMenuBuilder.AddMenuSeparator();
	ShowMenuBuilder.AddMenuEntry(LevelEditor.GetLevelEditorCommands().Select2DLayerAbove);
	ShowMenuBuilder.AddMenuEntry(LevelEditor.GetLevelEditorCommands().Select2DLayerBelow);

	return ShowMenuBuilder.MakeWidget();
}

TSharedRef<SWidget> STransformViewportToolBar::FillScaleGridSnapMenu()
{
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
	TArray<float> GridSizes = ViewportSettings->ScalingGridSizes;

	UE::UnrealEd::FScaleGridCheckboxListExecuteActionDelegate ExecuteDelegate =
		UE::UnrealEd::FScaleGridCheckboxListExecuteActionDelegate::CreateLambda(
			[](int CurrGridScaleIndex)
			{
				STransformViewportToolBar::SetScaleGridSize(CurrGridScaleIndex);
			}
		);

	UE::UnrealEd::FScaleGridCheckboxListIsCheckedDelegate IsCheckedDelegate =
		UE::UnrealEd::FScaleGridCheckboxListIsCheckedDelegate::CreateLambda(
			[](int CurrGridScaleIndex)
		{
				return STransformViewportToolBar::IsScaleGridSizeChecked(CurrGridScaleIndex);
		}
		);

	return UE::UnrealEd::CreateScaleGridSnapMenu(
		ExecuteDelegate,
		IsCheckedDelegate,
		GridSizes,
		true,
		CommandList,
		true,
		FUIAction(
			FExecuteAction::CreateLambda(
				[]()
				{
					ULevelEditorViewportSettings* Settings = GetMutableDefault<ULevelEditorViewportSettings>();
					Settings->PreserveNonUniformScale = !Settings->PreserveNonUniformScale;
				}
			),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda(
				[]()
				{
					return GetDefault<ULevelEditorViewportSettings>()->PreserveNonUniformScale;
				}
			)
		)
	);
}

ECheckBoxState STransformViewportToolBar::IsLocationGridSnapChecked() const
{
	return GetDefault<ULevelEditorViewportSettings>()->GridEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

ECheckBoxState STransformViewportToolBar::IsRotationGridSnapChecked() const
{
	return GetDefault<ULevelEditorViewportSettings>()->RotGridEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

ECheckBoxState STransformViewportToolBar::IsLayer2DSnapChecked() const
{
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
	const ULevelEditor2DSettings* Settings2D = GetDefault<ULevelEditor2DSettings>();
	const bool bChecked = ViewportSettings->bEnableLayerSnap && Settings2D->SnapLayers.IsValidIndex(ViewportSettings->ActiveSnapLayerIndex);
	return bChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

EVisibility STransformViewportToolBar::IsLayer2DSnapVisible() const
{
	const ULevelEditor2DSettings* Settings2D = GetDefault<ULevelEditor2DSettings>();
	return Settings2D->bEnableSnapLayers ? EVisibility::Visible : EVisibility::Collapsed;
}

ECheckBoxState STransformViewportToolBar::IsScaleGridSnapChecked() const
{
	return GetDefault<ULevelEditorViewportSettings>()->SnapScaleEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void STransformViewportToolBar::HandleToggleLocationGridSnap( ECheckBoxState InState )
{
	GUnrealEd->Exec( GEditor->GetEditorWorldContext().World(), *FString::Printf( TEXT("MODE GRID=%d"), !GetDefault<ULevelEditorViewportSettings>()->GridEnabled ? 1 : 0 ) );
}

void STransformViewportToolBar::HandleToggleRotationGridSnap(ECheckBoxState InState)
{
	GUnrealEd->Exec(GEditor->GetEditorWorldContext().World(), *FString::Printf(TEXT("MODE ROTGRID=%d"), !GetDefault<ULevelEditorViewportSettings>()->RotGridEnabled ? 1 : 0));
}

void STransformViewportToolBar::HandleToggleLayer2DSnap(ECheckBoxState InState)
{
	ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>();
	const ULevelEditor2DSettings* Settings2D = GetDefault<ULevelEditor2DSettings>();
	if (!ViewportSettings->bEnableLayerSnap && (Settings2D->SnapLayers.Num() > 0))
	{
		ViewportSettings->bEnableLayerSnap = true;
		ViewportSettings->ActiveSnapLayerIndex = FMath::Clamp(ViewportSettings->ActiveSnapLayerIndex, 0, Settings2D->SnapLayers.Num() - 1);
	}
	else
	{
		ViewportSettings->bEnableLayerSnap = false;
	}
	ViewportSettings->PostEditChange();
}

void STransformViewportToolBar::HandleToggleScaleGridSnap(ECheckBoxState InState)
{
	GUnrealEd->Exec( GEditor->GetEditorWorldContext().World(), *FString::Printf( TEXT("MODE SCALEGRID=%d"), !GetDefault<ULevelEditorViewportSettings>()->SnapScaleEnabled ? 1 : 0 ) );
}

#undef LOCTEXT_NAMESPACE 
