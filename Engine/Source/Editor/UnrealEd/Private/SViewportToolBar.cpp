// Copyright Epic Games, Inc. All Rights Reserved.


#include "SViewportToolBar.h"

#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "Widgets/Input/SMenuAnchor.h"

struct FSlateBrush;

#define LOCTEXT_NAMESPACE "ViewportToolBar"

void SViewportToolBar::Construct(const FArguments& InArgs)
{
}

TWeakPtr<SMenuAnchor> SViewportToolBar::GetOpenMenu() const
{
	return OpenedMenu;
}

void SViewportToolBar::SetOpenMenu( TSharedPtr< SMenuAnchor >& NewMenu )
{
	if( OpenedMenu.IsValid() && OpenedMenu.Pin() != NewMenu )
	{
		// Close any other open menus
		OpenedMenu.Pin()->SetIsOpen( false );
	}
	OpenedMenu = NewMenu;
}

FText SViewportToolBar::GetCameraMenuLabelFromViewportType(const ELevelViewportType ViewportType) const
{
	return UE::UnrealEd::GetCameraSubmenuLabelFromViewportType(ViewportType);
}

const FSlateBrush* SViewportToolBar::GetCameraMenuLabelIconFromViewportType(const ELevelViewportType ViewportType) const
{
	const FName Icon = UE::UnrealEd::GetCameraSubmenuIconFNameFromViewportType(ViewportType);
	return FAppStyle::GetBrush(Icon);
}

bool SViewportToolBar::IsViewModeSupported(EViewModeIndex ViewModeIndex) const 
{
	switch (ViewModeIndex)
	{
	case VMI_PrimitiveDistanceAccuracy:
	case VMI_MaterialTextureScaleAccuracy:
	case VMI_RequiredTextureResolution:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	case VMI_Lit_Wireframe:
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
		return false;
	default:
		return true;
	}
}

#undef LOCTEXT_NAMESPACE
