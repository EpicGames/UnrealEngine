// Copyright Epic Games, Inc. All Rights Reserved.

#include "SViewportToolBarComboMenu.h"

#include "Layout/Children.h"
#include "SEditorViewportToolBarMenuButton.h"
#include "SViewportToolBar.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/ToolBarStyle.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/NameTypes.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

struct FGeometry;
struct FPointerEvent;

void SViewportToolBarComboMenu::Construct( const FArguments& InArgs )
{
	const FButtonStyle& ButtonStyle = FAppStyle::Get().GetWidgetStyle<FButtonStyle>("EditorViewportToolBar.ComboMenu.ButtonStyle");
	const FCheckBoxStyle& CheckBoxStyle = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("EditorViewportToolBar.ToggleButton.Start");
	const FTextBlockStyle& LabelStyle = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("EditorViewportToolBar.ComboMenu.LabelStyle");

	const FSlateIcon& Icon = InArgs._Icon.Get();
	ParentToolBar = InArgs._ParentToolBar;


	TSharedRef<SCheckBox> ToggleControl = SNew(SCheckBox)
		.Style(&CheckBoxStyle)
		.ToolTipText(InArgs._ToggleButtonToolTip)
		.OnCheckStateChanged(InArgs._OnCheckStateChanged)
		.IsChecked(InArgs._IsChecked)
		[
			SNew(SImage)
			.Image(Icon.GetIcon())
			.ColorAndOpacity(FSlateColor::UseForeground())
		];


	{
		MenuAnchor = SNew(SMenuAnchor)
		.Placement( MenuPlacement_BelowAnchor )
		.OnGetMenuContent( InArgs._OnGetMenuContent );

		MenuAnchor->SetContent(
			SNew(SBox)
			.MinDesiredWidth(InArgs._MinDesiredButtonWidth > 0.0f ? InArgs._MinDesiredButtonWidth : FOptionalSize())
			[
				SNew(SEditorViewportToolBarMenuButton, MenuAnchor.ToSharedRef())
				.ButtonStyle(&ButtonStyle)
				.ToolTipText(this, &SViewportToolBarComboMenu::GetFilteredToolTipText, InArgs._MenuButtonToolTip)
				.OnClicked(this, &SViewportToolBarComboMenu::OnMenuClicked)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.TextStyle(&LabelStyle)
					.Text(InArgs._Label)
				]
			]
		);
	}


	ChildSlot
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			ToggleControl
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			MenuAnchor.ToSharedRef()
		]
	];
}

FText SViewportToolBarComboMenu::GetFilteredToolTipText(TAttribute<FText> ToolTipText) const
{
	// If we're part of a toolbar that has a currently open menu, then we suppress our tool-tip
	// as it will just get in the way
	TWeakPtr<SMenuAnchor> OpenedMenu = ParentToolBar.Pin()->GetOpenMenu();
	if (OpenedMenu.IsValid() && OpenedMenu.Pin()->IsOpen())
	{
		return FText::GetEmpty();
	}

	return ToolTipText.Get();
}

FReply SViewportToolBarComboMenu::OnMenuClicked()
{
	// If the menu button is clicked toggle the state of the menu anchor which will open or close the menu
	MenuAnchor->SetIsOpen( !MenuAnchor->IsOpen() );
	ParentToolBar.Pin()->SetOpenMenu( MenuAnchor );
	return FReply::Handled();
}

void SViewportToolBarComboMenu::OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	// See if there is another menu on the same tool bar already open
	TWeakPtr<SMenuAnchor> OpenedMenu = ParentToolBar.Pin()->GetOpenMenu();
	if( OpenedMenu.IsValid() && OpenedMenu.Pin()->IsOpen() && OpenedMenu.Pin() != MenuAnchor )
	{
		// There is another menu open so we open this menu and close the other
		ParentToolBar.Pin()->SetOpenMenu( MenuAnchor ); 
		MenuAnchor->SetIsOpen( true );
	}
}
