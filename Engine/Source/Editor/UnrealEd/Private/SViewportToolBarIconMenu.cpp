// Copyright Epic Games, Inc. All Rights Reserved.

#include "SViewportToolBarIconMenu.h"

#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "SViewportToolBar.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/ToolBarStyle.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/NameTypes.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

struct FGeometry;
struct FPointerEvent;

void SViewportToolBarIconMenu::Construct( const FArguments& InArgs )
{
	ParentToolBar = InArgs._ParentToolBar;

	const FToolBarStyle& ViewportToolbarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>(InArgs._Style.Get());

	const FSlateIcon& Icon = InArgs._Icon.Get();

	SAssignNew( MenuAnchor, SMenuAnchor )
	.Placement( MenuPlacement_BelowAnchor )
	.OnGetMenuContent( InArgs._OnGetMenuContent )
	[
		SNew(SButton)
		.ButtonStyle(&ViewportToolbarStyle.ButtonStyle)
		.ContentPadding( FMargin( 5.0f, 0.0f ) )
		.OnClicked(this, &SViewportToolBarIconMenu::OnMenuClicked)
		[
			SNew(SHorizontalBox)

			// Icon
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(16.0f)
				.HeightOverride(16.0f)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(Icon.GetIcon())
				]
			]

			// Spacer
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding( FMargin( 4.0f, 0.0f ) )

			// Menu dropdown
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Top)
				[
					SNew(STextBlock)
					.TextStyle(&ViewportToolbarStyle.LabelStyle)
					.Text(InArgs._Label)
				]
			]
		]
	];

	ChildSlot
	[
		MenuAnchor.ToSharedRef()
	];
}

FReply SViewportToolBarIconMenu::OnMenuClicked()
{
	// If the menu button is clicked toggle the state of the menu anchor which will open or close the menu
	MenuAnchor->SetIsOpen( !MenuAnchor->IsOpen() );
	ParentToolBar.Pin()->SetOpenMenu( MenuAnchor );
	return FReply::Handled();
}
	
void SViewportToolBarIconMenu::OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	// See if there is another menu on the same tool bar already open
	TWeakPtr<SMenuAnchor> OpenedMenu = ParentToolBar.Pin()->GetOpenMenu();
	if( OpenedMenu.IsValid() && OpenedMenu.Pin()->IsOpen() && OpenedMenu.Pin() != MenuAnchor )
	{
		// There is another menu open so we open this menu and close the other
		ParentToolBar.Pin()->SetOpenMenu( MenuAnchor ); 
		MenuAnchor->SetIsOpen( true );
	}
}
