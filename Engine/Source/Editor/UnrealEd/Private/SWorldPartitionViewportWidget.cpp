// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWorldPartitionViewportWidget.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Framework/Docking/TabManager.h"
#include "LevelEditor.h"
#include "Editor.h"
#include "Engine/World.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/IWorldPartitionEditorModule.h"
#include "Styling/AppStyle.h"
#include "Modules/ModuleManager.h"
#include "GameFramework/WorldSettings.h"

#define LOCTEXT_NAMESPACE "WorldPartitionViewportWidget"

void SWorldPartitionViewportWidget::Construct(const FArguments& InArgs)
{
	bClickable = InArgs._Clickable;
	
	ChildSlot.Padding(2, 2, 2, 2)
	[
		SNew(SButton)
		.ButtonStyle(bClickable? &FCoreStyle::Get().GetWidgetStyle<FButtonStyle>("HoverHintOnly") : &FCoreStyle::Get().GetWidgetStyle<FButtonStyle>("NoBorder"))
		.Cursor(EMouseCursor::Default)
		.IsEnabled_Lambda([this]() { return bClickable; })
		.OnClicked_Lambda([this]()
		{
			FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
			TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();
			LevelEditorModule.GetLevelEditorTabManager()->TryInvokeTab(InvokeTab);
			return FReply::Handled();
		})
		.ToolTipText_Lambda([this]() { return Tooltip; })
		.ContentPadding(0.f)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(FMargin(2, 1, 0, 1))
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.WarningWithColor"))
					.DesiredSizeOverride(FVector2D(16, 16))
				]
				+ SHorizontalBox::Slot()
				.Padding(FMargin(6, 1, 2, 1))
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.FillWidth(1)
				[
					SNew(STextBlock)
					.Text_Lambda([this]() { return Message; })
					.ColorAndOpacity(FAppStyle::Get().GetSlateColor("Colors.Warning"))
				]
			]
		]
	];

	WorldPartitionEditorModule = FModuleManager::GetModulePtr<IWorldPartitionEditorModule>("WorldPartitionEditor");
}

SWorldPartitionViewportWidget::~SWorldPartitionViewportWidget()
{}

EVisibility SWorldPartitionViewportWidget::GetVisibility(UWorld* InWorld)
{
	if (InWorld && InWorld->IsPartitionedWorld())
	{
		UWorldPartition* WorldPartition = InWorld->GetWorldPartition();
		if (WorldPartition->IsStreamingEnabled())
		{
			if (!WorldPartition->HasLoadedUserCreatedRegions() && WorldPartitionEditorModule->GetEnableLoadingInEditor())
			{
				Message = LOCTEXT("NoLoadedRegionsText","No Loaded Region(s)");
				Tooltip = LOCTEXT("NoLoadedRegionsToolTip", "To load a region, drag select an area in the World Partition map and choose 'Load Region From Selection' from the context menu.");
				InvokeTab = TEXT("WorldBrowserPartitionEditor");
				return EVisibility::Visible;
			}
		}
		else
		{
			const AWorldSettings* WorldSettings = InWorld->GetWorldSettings();
			if ((WorldSettings != nullptr && !WorldSettings->bHideEnableStreamingWarning) && WorldPartition->IsEnablingStreamingJustified())
			{
				Message = LOCTEXT("StreamingDisabledText","Streaming Disabled");
				Tooltip = LOCTEXT("StreamingDisabledToolTip", "The size of your world has grown enough to justify enabling streaming.");
				InvokeTab = TEXT("WorldSettingsTab");
				return EVisibility::Visible;
			}
		}
	}

	return EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE