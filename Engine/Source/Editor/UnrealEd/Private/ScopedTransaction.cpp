// Copyright Epic Games, Inc. All Rights Reserved.


#include "ScopedTransaction.h"
#include "Framework/Application/SlateApplication.h"
#include "Editor.h"

FScopedTransaction::FScopedTransaction(const FText& SessionName, const bool bShouldActuallyTransact )
{
	Construct( TEXT(""), SessionName, NULL, bShouldActuallyTransact);
}

FScopedTransaction::FScopedTransaction(const TCHAR* TransactionContext, const FText& SessionName, UObject* PrimaryObject, const bool bShouldActuallyTransact)
{
	Construct( TransactionContext, SessionName, PrimaryObject, bShouldActuallyTransact);
}

void FScopedTransaction::Construct (const TCHAR* TransactionContext, const FText& SessionName, UObject* PrimaryObject, const bool bShouldActuallyTransact)
{
	if( bShouldActuallyTransact && GEditor && GEditor->CanTransact() && ensure(!GIsTransacting))
	{
		Index = GEditor->BeginTransaction( TransactionContext, SessionName, PrimaryObject );
		check( IsOutstanding() );
	}
	else
	{
		Index = -1;
	}
}

FScopedTransaction::~FScopedTransaction()
{
	if ( IsOutstanding() )
	{
		GEditor->EndTransaction();
	}
}

/**
 * Cancels the transaction.  Reentrant.
 */
void FScopedTransaction::Cancel()
{
	if ( IsOutstanding() )
	{
		GEditor->CancelTransaction( Index );
		Index = -1;
	}
}

/**
 * @return	true if the transaction is still outstanding (that is, has not been cancelled).
 */
bool FScopedTransaction::IsOutstanding() const
{
	return Index >= 0;
}
