// Copyright Epic Games, Inc. All Rights Reserved.

#include "Selection.h"

#include "Containers/ContainerAllocationPolicies.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "Elements/Framework/TypedElementListObjectUtil.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Interfaces/TypedElementObjectInterface.h"
#include "Elements/Interfaces/TypedElementSelectionInterface.h"
#include "GameFramework/Actor.h"
#include "Logging/LogMacros.h"
#include "Templates/UnrealTemplate.h"
#include "Editor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Selection)

DEFINE_LOG_CATEGORY_STATIC(LogSelection, Log, All);

class ISelectionElementBridge
{
public:
	virtual ~ISelectionElementBridge() = default;
	virtual bool IsValidObjectType(const UObject* InObject) const = 0;
	virtual FTypedElementHandle GetElementHandleForObject(const UObject* InObject, const bool bAllowCreate = true) const = 0;
};

class FObjectSelectionElementBridge : public ISelectionElementBridge
{
public:
	virtual bool IsValidObjectType(const UObject* InObject) const override
	{
		return IsValidChecked(InObject);
	}

	virtual FTypedElementHandle GetElementHandleForObject(const UObject* InObject, const bool bAllowCreate = true) const override
	{
		check(InObject);
		return UEngineElementsLibrary::AcquireEditorObjectElementHandle(InObject, bAllowCreate);
	}
};

class FActorSelectionElementBridge : public ISelectionElementBridge
{
public:
	virtual bool IsValidObjectType(const UObject* InObject) const override
	{
		return IsValidChecked(InObject) && InObject->IsA<AActor>();
	}

	virtual FTypedElementHandle GetElementHandleForObject(const UObject* InObject, const bool bAllowCreate = true) const override
	{
		check(InObject);
		return UEngineElementsLibrary::AcquireEditorActorElementHandle(CastChecked<AActor>(InObject), bAllowCreate);
	}
};

class FComponentSelectionElementBridge : public ISelectionElementBridge
{
public:
	virtual bool IsValidObjectType(const UObject* InObject) const override
	{
		return IsValidChecked(InObject) && InObject->IsA<UActorComponent>();
	}

	virtual FTypedElementHandle GetElementHandleForObject(const UObject* InObject, const bool bAllowCreate = true) const override
	{
		check(InObject);
		return UEngineElementsLibrary::AcquireEditorComponentElementHandle(CastChecked<UActorComponent>(InObject), bAllowCreate);
	}
};

USelection::FOnSelectionChanged						USelection::SelectionChangedEvent;
USelection::FOnSelectionChanged						USelection::SelectObjectEvent;
FSimpleMulticastDelegate							USelection::SelectNoneEvent;
USelection::FOnSelectionElementSelectionPtrChanged	USelection::SelectionElementSelectionPtrChanged;

USelection* USelection::CreateObjectSelection(UObject* InOuter, FName InName, EObjectFlags InFlags)
{
	USelection* Selection = NewObject<USelection>(InOuter, InName, InFlags);
	Selection->Initialize(MakeShared<FObjectSelectionElementBridge>());
	return Selection;
}

USelection* USelection::CreateActorSelection(UObject* InOuter, FName InName, EObjectFlags InFlags)
{
	USelection* Selection = NewObject<USelection>(InOuter, InName, InFlags);
	Selection->Initialize(MakeShared<FActorSelectionElementBridge>());
	return Selection;
}

USelection* USelection::CreateComponentSelection(UObject* InOuter, FName InName, EObjectFlags InFlags)
{
	USelection* Selection = NewObject<USelection>(InOuter, InName, InFlags);
	Selection->Initialize(MakeShared<FComponentSelectionElementBridge>());
	return Selection;
}

void USelection::Initialize(TSharedRef<ISelectionElementBridge>&& InSelectionElementBridge)
{
	SelectionElementBridge = MoveTemp(InSelectionElementBridge);
}

void USelection::SetElementSelectionSet(UTypedElementSelectionSet* InElementSelectionSet)
{
	if (ElementSelectionSet)
	{
		ElementSelectionSet->Legacy_GetElementListSync().OnSyncEvent().RemoveAll(this);
	}

	UTypedElementSelectionSet* OldElementSelectionSet = ElementSelectionSet;
	ElementSelectionSet = InElementSelectionSet;

	if (ElementSelectionSet)
	{
		ElementSelectionSet->Legacy_GetElementListSync().OnSyncEvent().AddUObject(this, &USelection::OnElementListSyncEvent);
	}
	
	USelection::SelectionElementSelectionPtrChanged.Broadcast(this, OldElementSelectionSet, ElementSelectionSet);
}

UTypedElementSelectionSet* USelection::GetElementSelectionSet() const
{
	return ElementSelectionSet;
}

int32 USelection::Num() const
{
	return ElementSelectionSet
		? ElementSelectionSet->GetElementList()->Num()
		: 0;
}

UObject* USelection::GetSelectedObject(const int32 InIndex) const
{
	if (ElementSelectionSet)
	{
		FTypedElementListConstRef ElementList = ElementSelectionSet->GetElementList();
		if (ElementList->IsValidIndex(InIndex))
		{
			const FTypedElementHandle ElementHandle = ElementList->GetElementHandleAt(InIndex);
			return GetObjectForElementHandle(ElementHandle);
		}
	}

	return nullptr;
}

void USelection::BeginBatchSelectOperation()
{
	if (ElementSelectionSet)
	{
		ElementSelectionSet->Legacy_GetElementListSync().BeginBatchOperation();
	}
}

void USelection::EndBatchSelectOperation(bool bNotify)
{
	if (ElementSelectionSet)
	{
		ElementSelectionSet->Legacy_GetElementListSync().EndBatchOperation(bNotify);
	}
}

bool USelection::IsBatchSelecting() const
{
	return ElementSelectionSet && ElementSelectionSet->Legacy_GetElementListSync().IsRunningBatchOperation();
}

bool USelection::IsValidObjectToSelect(const UObject* InObject) const
{
	bool bIsValid = SelectionElementBridge->IsValidObjectType(InObject);

	if (bIsValid && ElementSelectionSet)
	{
		TTypedElement<ITypedElementObjectInterface> ObjectElement = ElementSelectionSet->GetElementList()->GetElement<ITypedElementObjectInterface>(SelectionElementBridge->GetElementHandleForObject(InObject));
		if (ObjectElement)
		{
			bIsValid &= ObjectElement.GetObject() == InObject;
		}
		else
		{
			// Elements must implement the object interface in order to be selected!
			bIsValid = false;
		}
	}

	return bIsValid;
}

UObject* USelection::GetObjectForElementHandle(const FTypedElementHandle& InElementHandle) const
{
	check(InElementHandle && ElementSelectionSet);

	// Defensive guard against stale handles
	if (InElementHandle.GetId().GetTypeId() == 0)
	{
		return nullptr;
	}

	if (TTypedElement<ITypedElementObjectInterface> ObjectElement = ElementSelectionSet->GetElementList()->GetElement<ITypedElementObjectInterface>(InElementHandle))
	{
		UObject* Object = ObjectElement.GetObject();
		if (Object && SelectionElementBridge->IsValidObjectType(Object))
		{
			return Object;
		}
	}

	return nullptr;
}

void USelection::OnElementListSyncEvent(const FTypedElementList& InElementList, FTypedElementList::FLegacySync::ESyncType InSyncType, const FTypedElementHandle& InElementHandle, bool bIsWithinBatchOperation)
{
	check(&InElementList == &ElementSelectionSet->GetElementList().Get());

	const bool bNotify = !bIsWithinBatchOperation;
	if (bNotify)
	{
		switch (InSyncType)
		{
		// Emit a general object selection changed notification for the following events
		case FTypedElementList::FLegacySync::ESyncType::Added:
		case FTypedElementList::FLegacySync::ESyncType::Removed:
			if (UObject* Object = GetObjectForElementHandle(InElementHandle))
			{
				USelection::SelectObjectEvent.Broadcast(Object);
			}
			break;

		// Emit a general selection changed notification for the following events
		case FTypedElementList::FLegacySync::ESyncType::Modified:
		case FTypedElementList::FLegacySync::ESyncType::Cleared:
		case FTypedElementList::FLegacySync::ESyncType::BatchComplete:
			USelection::SelectionChangedEvent.Broadcast(this);
			break;

		default:
			break;
		}
	}
}

void USelection::Select(UObject* InObject)
{
	check(InObject);
	if (IsValidObjectToSelect(InObject) && ElementSelectionSet)
	{
		FTypedElementHandle ElementHandle = SelectionElementBridge->GetElementHandleForObject(InObject);
		check(ElementHandle);
		ElementSelectionSet->SelectElement(ElementHandle, FTypedElementSelectionOptions().SetAllowHidden(true).SetAllowGroups(false).SetWarnIfLocked(false));
	}
}

void USelection::Deselect(UObject* InObject)
{
	check(InObject);
	if (ElementSelectionSet)
	{
		if (const FTypedElementHandle ElementHandle = SelectionElementBridge->GetElementHandleForObject(InObject, /*bAllowCreate*/false))
		{
			ElementSelectionSet->DeselectElement(ElementHandle, FTypedElementSelectionOptions().SetAllowHidden(true).SetAllowGroups(false).SetWarnIfLocked(false));
		}
	}
}

void USelection::Select(UObject* InObject, bool bSelect)
{
	if (bSelect)
	{
		Select(InObject);
	}
	else
	{
		Deselect(InObject);
	}
}

void USelection::ToggleSelect(UObject* InObject)
{
	Select(InObject, InObject && !InObject->IsSelected());
}

void USelection::DeselectAll(UClass* InClass)
{
	if (!ElementSelectionSet)
	{
		return;
	}

	UClass* ClassToDeselect = InClass;
	if (!ClassToDeselect)
	{
		ClassToDeselect = UObject::StaticClass();
	}

	TArray<UObject*> ObjectsToDeselect;
	GetSelectedObjects(ClassToDeselect, ObjectsToDeselect);

	TArray<FTypedElementHandle, TInlineAllocator<256>> ElementsToDeselect;
	ElementsToDeselect.Reserve(ObjectsToDeselect.Num());
	for (UObject* ObjectToDeselect : ObjectsToDeselect)
	{
		if (FTypedElementHandle ElementHandle = SelectionElementBridge->GetElementHandleForObject(ObjectToDeselect, /*bAllowCreate*/false))
		{
			ElementsToDeselect.Add(MoveTemp(ElementHandle));
		}
	}

	if (ElementsToDeselect.Num() > 0)
	{
		ElementSelectionSet->DeselectElements(ElementsToDeselect, FTypedElementSelectionOptions().SetAllowHidden(true).SetAllowGroups(false).SetWarnIfLocked(false));
	}
}

void USelection::ForceBatchDirty()
{
	if (IsBatchSelecting())
	{
		check(ElementSelectionSet);
		ElementSelectionSet->Legacy_GetElementListSync().ForceBatchOperationDirty();
	}
}

void USelection::NoteSelectionChanged()
{
	USelection::SelectionChangedEvent.Broadcast(this);
}

void USelection::NoteUnknownSelectionChanged()
{
	USelection::SelectionChangedEvent.Broadcast(nullptr);
}

bool USelection::IsSelected(const UObject* InObject) const
{
	if (InObject && ElementSelectionSet)
	{
		if (const FTypedElementHandle ElementHandle = SelectionElementBridge->GetElementHandleForObject(InObject, /*bAllowCreate*/false))
		{
			return ElementSelectionSet->IsElementSelected(ElementHandle, FTypedElementIsSelectedOptions());
		}
	}

	return false;
}

bool USelection::IsClassSelected(UClass* Class) const
{
	return ElementSelectionSet
		&& TypedElementListObjectUtil::HasObjectsOfExactClass(ElementSelectionSet->GetElementList(), Class);
}

void USelection::Serialize(FArchive& Ar)
{
	if (ElementSelectionSet)
	{
		ElementSelectionSet->Serialize(Ar);
	}
}

bool USelection::Modify(bool bAlwaysMarkDirty)
{
	return ElementSelectionSet 
		&& ElementSelectionSet->Modify(bAlwaysMarkDirty);
}

FDeselectedActorsEvent::~FDeselectedActorsEvent()
{
	if (GEditor && GEditor->GetSelectedActors())
	{
		if (UTypedElementSelectionSet* ElementSelectionSet = GEditor->GetSelectedActors()->GetElementSelectionSet())
		{
			bool bSelectionChanged = false;
			FTypedElementList::FLegacySyncScopedBatch LegacySyncBatch(*ElementSelectionSet->GetElementList());

			for (AActor* Actor : DeselectedActors)
			{
				if (!Actor->GetRootSelectionParent())
				{
					if (FTypedElementHandle ActorHandle = UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor, /*bAllowCreate*/false))
					{
						static const FTypedElementSelectionOptions SelectionOptions = FTypedElementSelectionOptions()
							.SetAllowHidden(true)
							.SetAllowGroups(false)
							.SetWarnIfLocked(false)
							.SetChildElementInclusionMethod(ETypedElementChildInclusionMethod::Recursive);

						bSelectionChanged |= ElementSelectionSet->DeselectElement(ActorHandle, SelectionOptions);
					}
				}
			}

			if (bSelectionChanged)
			{
				GEditor->NoteSelectionChange();
			}
		}
	}
}
