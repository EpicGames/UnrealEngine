// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/DefaultCookedFilePackageWriter.h"

#include "AssetRegistry/AssetRegistryState.h"
#include "Misc/Paths.h"
#include "Serialization/FilePackageWriterUtil.h"

FDefaultCookedFilePackageWriter::FDefaultCookedFilePackageWriter(const FString& InBaseFilename)
	: TPackageWriterToSharedBuffer<FBasePackageWriter>()
	, BaseFilename(InBaseFilename)
{}

void FDefaultCookedFilePackageWriter::WritePackageData(const IPackageWriter::FPackageInfo& Info, FLargeMemoryWriter& ExportsArchive, const TArray<FFileRegion>& FileRegions)
{
	//This is working because Super::WritePackageData will store a copy of InfoCopy. But it's sketchy!
	IPackageWriter::FPackageInfo InfoCopy = Info;

	FString Extension = FPaths::GetExtension(Info.LooseFilePath, false);
	InfoCopy.LooseFilePath = FPaths::SetExtension(BaseFilename, Extension);
	
	Super::WritePackageData(InfoCopy, ExportsArchive, FileRegions);
}

void FDefaultCookedFilePackageWriter::WriteBulkData(const IPackageWriter::FBulkDataInfo& Info, const FIoBuffer& BulkData, const TArray<FFileRegion>& FileRegions)
{
	//This is working because Super::WriteBulkData will store a copy of InfoCopy. But it's sketchy!
	IPackageWriter::FBulkDataInfo InfoCopy = Info;

	FString Extension = FPaths::GetExtension(Info.LooseFilePath, false);
	InfoCopy.LooseFilePath = FPaths::SetExtension(BaseFilename, Extension);

	Super::WriteBulkData(InfoCopy, BulkData, FileRegions);
}

void FDefaultCookedFilePackageWriter::CommitPackageInternal(FPackageWriterRecords::FPackage&& BaseRecord, const FCommitPackageInfo& Info)
{
	FFilePackageWriterUtil::FRecord& Record = static_cast<FFilePackageWriterUtil::FRecord&>(BaseRecord);
	FFilePackageWriterUtil::FWritePackageParameters Parameters(Record, Info);
	FFilePackageWriterUtil::WritePackage(Parameters);
}

FPackageWriterRecords::FPackage* FDefaultCookedFilePackageWriter::ConstructRecord()
{
	return new FFilePackageWriterUtil::FRecord();
}
