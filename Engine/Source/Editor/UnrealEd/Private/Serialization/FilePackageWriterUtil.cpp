// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/FilePackageWriterUtil.h"

#include "HAL/FileManager.h"
#include "Misc/PackagePath.h"
#include "Misc/Paths.h"
#include "Serialization/MemoryWriter.h"
#include "UObject/SavePackage.h"

FFilePackageWriterUtil::FWritePackageParameters::FWritePackageParameters(FRecord& InRecord,
	const IPackageWriter::FCommitPackageInfo& InInfo,
	TMap<FName, TRefCountPtr<FPackageHashes>>* InAllPackageHashes,
	FCriticalSection* InPackageHashesLock,
	bool bInProvidePerPackageResult)
	: FWritePackageParameters(InRecord, InInfo)
{
	AllPackageHashes = InAllPackageHashes;
	PackageHashesLock = InPackageHashesLock;
	bProvidePerPackageResult = bInProvidePerPackageResult;
}

FFilePackageWriterUtil::FWritePackageParameters::FWritePackageParameters(FRecord& InRecord,
	const IPackageWriter::FCommitPackageInfo& InInfo)
	: Record(InRecord)
	, Info(InInfo)
{
}

void FFilePackageWriterUtil::WritePackage(FWritePackageParameters& Parameters)
{
	if (Parameters.Info.Status == IPackageWriter::ECommitStatus::Success)
	{
		AsyncSave(Parameters);
	}
}

void FFilePackageWriterUtil::AsyncSave(FWritePackageParameters& Parameters)
{
	FCommitContext Context{ Parameters.Info };
	Context.Cooker = Parameters.Cooker;

	// The order of these collection calls is important, both for ExportsBuffers (affects the meaning of offsets
	// to those buffers) and for OutputFiles (affects the calculation of the Hash for the set of PackageData)
	// The order of ExportsBuffers must match CompleteExportsArchiveForDiff.
	CollectForSavePackageData(Parameters.Record, Context);
	CollectForSaveBulkData(Parameters.Record, Context);
	CollectForSaveLinkerAdditionalDataRecords(Parameters.Record, Context);
	CollectForSaveAdditionalFileRecords(Parameters.Record, Context);
	CollectForSaveExportsFooter(Parameters.Record, Context);
	CollectForSaveExportsPackageTrailer(Parameters.Record, Context);
	CollectForSaveExportsBuffers(Parameters.Record, Context);

	AsyncSaveOutputFiles(Context, Parameters.AllPackageHashes, Parameters.PackageHashesLock, Parameters.bProvidePerPackageResult);
}

void FFilePackageWriterUtil::CollectForSavePackageData(FRecord& Record, FCommitContext& Context)
{
	Context.ExportsBuffers.AddDefaulted(Record.Packages.Num());
	for (FPackageWriterRecords::FWritePackage& Package : Record.Packages)
	{
		Context.ExportsBuffers[Package.Info.MultiOutputIndex].Add(FExportBuffer{ Package.Buffer, MoveTemp(Package.Regions) });
	}
}

void FFilePackageWriterUtil::CollectForSaveBulkData(FRecord& Record, FCommitContext& Context)
{
	for (FPackageWriterRecords::FBulkData& BulkRecord : Record.BulkDatas)
	{
		if (BulkRecord.Info.BulkDataType == IPackageWriter::FBulkDataInfo::AppendToExports)
		{
			if (Record.bCompletedExportsArchiveForDiff)
			{
				// Already Added in CompleteExportsArchiveForDiff
				continue;
			}
			Context.ExportsBuffers[BulkRecord.Info.MultiOutputIndex].Add(FExportBuffer{ BulkRecord.Buffer, MoveTemp(BulkRecord.Regions) });
		}
		else
		{
			UE::PackageWriter::Private::FWriteFileData& OutputFile = Context.OutputFiles.Emplace_GetRef();
			OutputFile.Filename = BulkRecord.Info.LooseFilePath;
			OutputFile.Buffer = FCompositeBuffer(BulkRecord.Buffer);
			OutputFile.Regions = MoveTemp(BulkRecord.Regions);
			OutputFile.bIsSidecar = true;
			// Only calculate the CookHash for files in the main package output. Ignore it for the Optional package output.
			OutputFile.bContributeToHash = BulkRecord.Info.MultiOutputIndex == 0;
			OutputFile.bPackageSpecificFilename = true;
			OutputFile.ChunkId = BulkRecord.Info.ChunkId;
		}
	}
}

void FFilePackageWriterUtil::CollectForSaveLinkerAdditionalDataRecords(FRecord& Record, FCommitContext& Context)
{
	if (Record.bCompletedExportsArchiveForDiff)
	{
		// Already Added in CompleteExportsArchiveForDiff
		return;
	}

	for (FPackageWriterRecords::FLinkerAdditionalData& AdditionalRecord : Record.LinkerAdditionalDatas)
	{
		Context.ExportsBuffers[AdditionalRecord.Info.MultiOutputIndex].Add(FExportBuffer{ AdditionalRecord.Buffer, MoveTemp(AdditionalRecord.Regions) });
	}
}

void FFilePackageWriterUtil::CollectForSaveAdditionalFileRecords(FRecord& Record, FCommitContext& Context)
{
	for (FPackageWriterRecords::FAdditionalFile& AdditionalRecord : Record.AdditionalFiles)
	{
		UE::PackageWriter::Private::FWriteFileData& OutputFile = Context.OutputFiles.Emplace_GetRef();
		OutputFile.Filename = AdditionalRecord.Info.Filename;
		OutputFile.Buffer = FCompositeBuffer(AdditionalRecord.Buffer);
		OutputFile.bIsSidecar = true;
		// Only calculate the CookHash for files in the main package output. Ignore it for the Optional package output.
		OutputFile.bContributeToHash = AdditionalRecord.Info.MultiOutputIndex == 0;
		// For robustness, we handle all AdditionalFiles reported by UObject::CookAdditionalFiles as if they might be
		// written by multiple packages, without regard for their filename.
		OutputFile.bPackageSpecificFilename = false;
		OutputFile.ChunkId = AdditionalRecord.Info.ChunkId;
	}
}

void FFilePackageWriterUtil::CollectForSaveExportsFooter(FRecord& Record, FCommitContext& Context)
{
	if (Record.bCompletedExportsArchiveForDiff)
	{
		// Already Added in CompleteExportsArchiveForDiff
		return;
	}

	uint32 FooterData = PACKAGE_FILE_TAG;
	FSharedBuffer Buffer = FSharedBuffer::Clone(&FooterData, sizeof(FooterData));
	for (FPackageWriterRecords::FWritePackage& Package : Record.Packages)
	{
		Context.ExportsBuffers[Package.Info.MultiOutputIndex].Add(FExportBuffer{ Buffer, TArray<FFileRegion>() });
	}
}

void FFilePackageWriterUtil::CollectForSaveExportsPackageTrailer(FRecord& Record, FCommitContext& Context)
{
	if (Record.bCompletedExportsArchiveForDiff)
	{
		// Already Added in CompleteExportsArchiveForDiff
		return;
	}

	for (FPackageWriterRecords::FPackageTrailer& PackageTrailer : Record.PackageTrailers)
	{
		Context.ExportsBuffers[PackageTrailer.Info.MultiOutputIndex].Add(
			FExportBuffer{ PackageTrailer.Buffer, TArray<FFileRegion>() });
	}
}

void FFilePackageWriterUtil::CollectForSaveExportsBuffers(FRecord& Record, FCommitContext& Context)
{
	check(Context.ExportsBuffers.Num() == Record.Packages.Num());
	for (FPackageWriterRecords::FWritePackage& Package : Record.Packages)
	{
		TArray<FExportBuffer>& ExportsBuffers = Context.ExportsBuffers[Package.Info.MultiOutputIndex];
		check(ExportsBuffers.Num() > 0);

		// Split the ExportsBuffer into (1) Header and (2) Exports + AllAppendedData
		int64 HeaderSize = Package.Info.HeaderSize;
		FExportBuffer& HeaderAndExportsBuffer = ExportsBuffers[0];
		FSharedBuffer& HeaderAndExportsData = HeaderAndExportsBuffer.Buffer;

	
		// Header (.uasset/.umap)
		{
			UE::PackageWriter::Private::FWriteFileData& OutputFile = Context.OutputFiles.Emplace_GetRef();
			OutputFile.Filename = Package.Info.LooseFilePath;
			OutputFile.Buffer = FCompositeBuffer(
				FSharedBuffer::MakeView(HeaderAndExportsData.GetData(), HeaderSize, HeaderAndExportsData));
			OutputFile.bIsSidecar = false;
			// Only calculate the CookHash for files in the main package output. Ignore it for the Optional package output.
			OutputFile.bContributeToHash = Package.Info.MultiOutputIndex == 0;
			OutputFile.bPackageSpecificFilename = true;
		}

		// Exports + AllAppendedData (.uexp)
		{
			UE::PackageWriter::Private::FWriteFileData& OutputFile = Context.OutputFiles.Emplace_GetRef();
			OutputFile.Filename = FPaths::ChangeExtension(Package.Info.LooseFilePath, LexToString(EPackageExtension::Exports));
			OutputFile.bIsSidecar = false;
			// Only calculate the CookHash for files in the main package output. Ignore it for the Optional package output.
			OutputFile.bContributeToHash = Package.Info.MultiOutputIndex == 0;
			OutputFile.bPackageSpecificFilename = true;

			int32 NumBuffers = ExportsBuffers.Num();
			TArray<FSharedBuffer> BuffersForComposition;
			BuffersForComposition.Reserve(NumBuffers);

			const uint8* ExportsStart = static_cast<const uint8*>(HeaderAndExportsData.GetData()) + HeaderSize;
			BuffersForComposition.Add(FSharedBuffer::MakeView(ExportsStart, HeaderAndExportsData.GetSize() - HeaderSize,
				HeaderAndExportsData));
			OutputFile.Regions.Append(MoveTemp(HeaderAndExportsBuffer.Regions));

			for (FExportBuffer& ExportsBuffer : TArrayView<FExportBuffer>(ExportsBuffers).Slice(1, NumBuffers - 1))
			{
				BuffersForComposition.Add(ExportsBuffer.Buffer);
				OutputFile.Regions.Append(MoveTemp(ExportsBuffer.Regions));
			}
			OutputFile.Buffer = FCompositeBuffer(BuffersForComposition);

			// Adjust regions so they are relative to the start of the uexp file
			for (FFileRegion& Region : OutputFile.Regions)
			{
				Region.Offset -= HeaderSize;
			}
		}
	}
}

void FFilePackageWriterUtil::AsyncSaveOutputFiles(FCommitContext& Context,
	TMap<FName, TRefCountPtr<FPackageHashes>>* AllPackageHashes, FCriticalSection* PackageHashesLock,
	bool bProvidePerPackageResult)
{
	if (bProvidePerPackageResult && !AllPackageHashes)
	{
		UE_LOGF(LogSavePackage, Error, "FFilePackageWriterUtil::AsyncSaveOutputFiles : if bProvidePerPackageResult is true then AllPackageHashes can't be null.");
		return;
	}

	if (AllPackageHashes && !PackageHashesLock)
	{
		UE_LOGF(LogSavePackage, Error, "FFilePackageWriterUtil::AsyncSaveOutputFiles : if AllPackageHashes is provided, then PackageHashesLock can't be null.");
		return;
	}

	if (!EnumHasAnyFlags(Context.Info.WriteOptions, IPackageWriter::EWriteOptions::Write | IPackageWriter::EWriteOptions::ComputeHash))
	{
		return;
	}

	UE::SavePackageUtilities::IncrementOutstandingAsyncWrites();

	TRefCountPtr<FPackageHashes> ThisPackageHashes;
	TUniquePtr<TPromise<int>> PackageHashesCompletionPromise;

	if (EnumHasAnyFlags(Context.Info.WriteOptions, IPackageWriter::EWriteOptions::ComputeHash))
	{
		ThisPackageHashes = new FPackageHashes();
		
		if (bProvidePerPackageResult)
		{
			PackageHashesCompletionPromise.Reset(new TPromise<int>());
			ThisPackageHashes->CompletionFuture = PackageHashesCompletionPromise->GetFuture();
		}

		if (AllPackageHashes)
		{
			FScopeLock PackageHashesScopeLock(PackageHashesLock);
			// Store a RefCount to the FPackageHashes in AllPackageHashes so e.g. replication of the package's save
			// results from CookWorker to CookDirector can be delayed until the Future is ready. Give an error if we
			// try to assign the future twice, it is only supposed to happen once, when the package is saved, and we
			// don't want the burden of supporting multiple Futures
			TRefCountPtr<FPackageHashes>& ExistingPackageHashes = AllPackageHashes->FindOrAdd(Context.Info.PackageName);
			if (ExistingPackageHashes.IsValid())
			{
				UE_LOGF(LogSavePackage, Error,
					"FCookedFilePackageWriter encountered the same package twice in a cook! (%ls)",
					*Context.Info.PackageName.ToString());
			}
			ExistingPackageHashes = ThisPackageHashes;
		}
	}

	UE::Tasks::Launch(TEXT("HashAndWriteCookedFile"),
		[OutputFiles = MoveTemp(Context.OutputFiles), WriteOptions = Context.Info.WriteOptions,
		ThisPackageHashes = MoveTemp(ThisPackageHashes),
		PackageHashesCompletionPromise = MoveTemp(PackageHashesCompletionPromise),
		Cooker = Context.Cooker]
		() mutable
		{
			FMD5 AccumulatedHash;
			for (const UE::PackageWriter::Private::FWriteFileData& OutputFile : OutputFiles)
			{
				if (!OutputFile.bPackageSpecificFilename && Cooker)
				{
					Cooker->WriteFileOnCookDirector(OutputFile, AccumulatedHash, ThisPackageHashes, WriteOptions);
				}
				else
				{
					UE::PackageWriter::Private::HashAndWrite(
						OutputFile, AccumulatedHash, ThisPackageHashes, WriteOptions);
				}
			}

			if (EnumHasAnyFlags(WriteOptions, IPackageWriter::EWriteOptions::ComputeHash))
			{
				ThisPackageHashes->PackageHash.Set(AccumulatedHash);
			}

			if (PackageHashesCompletionPromise)
			{
				// Note that setting this Promise might call arbitrary code from anything that subscribed
				// to ThisPackageHashes->CompletionFuture.Then(). So don't call it inside a lock.
				PackageHashesCompletionPromise->SetValue(0);
			}

			// This is used to release the game thread to access the hashes
			UE::SavePackageUtilities::DecrementOutstandingAsyncWrites();
		},
		UE::Tasks::ETaskPriority::BackgroundNormal
	);
}