// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings/BlueprintEditorProjectSettings.h"
#include "Modules/ModuleManager.h"
#include "BlueprintEditorModule.h"
#include "Editor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintEditorProjectSettings)

/* UBlueprintEditorProjectSettings */

UBlueprintEditorProjectSettings::UBlueprintEditorProjectSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, DefaultChildActorTreeViewMode(EChildActorComponentTreeViewVisualizationMode::ComponentOnly)
	, bDisallowEditorUtilityBlueprintFunctionsInDetailsView(false)
{
}

void UBlueprintEditorProjectSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName Name = PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	if (Name == GET_MEMBER_NAME_CHECKED(UBlueprintEditorProjectSettings, bEnableChildActorExpansionInTreeView))
	{
		// Find open blueprint editors and refresh them
		FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>("Kismet");
		for (const TSharedRef<IBlueprintEditor>& BlueprintEditor : BlueprintEditorModule.GetBlueprintEditors())
		{
			BlueprintEditor->RefreshEditors();
		}

		// Deselect actors so we are forced to clear the current tree view
		// @todo - Figure out how to update the tree view directly instead?
		if (GEditor && GEditor->GetSelectedActorCount() > 0)
		{
			const bool bNoteSelectionChange = true;
			const bool bDeselectBSPSurfaces = true;
			GEditor->SelectNone(bNoteSelectionChange, bDeselectBSPSurfaces);
		}
	}
	else if (Name == GET_MEMBER_NAME_CHECKED(UBlueprintEditorProjectSettings, NamespacesToAlwaysInclude))
	{
		// Close any open Blueprint editor windows so that we have a chance to reload them with the updated import set.
		FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>("Kismet");
		for (const TSharedRef<IBlueprintEditor>& BlueprintEditor : BlueprintEditorModule.GetBlueprintEditors())
		{
			BlueprintEditor->CloseWindow(EAssetEditorCloseReason::AssetUnloadingOrInvalid);
		}
	}
}
