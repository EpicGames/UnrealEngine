// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings/BlueprintEditorProjectSettingsCustomization.h"
#include "BlueprintEditorSettings.h"
#include "Settings/BlueprintEditorProjectSettings.h"
#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"
#include "IDetailPropertyRow.h"
#include "DetailWidgetRow.h"
#include "PropertyCustomizationHelpers.h"
#include "BlueprintManagedListDetails.h"
#include "SBlueprintNamespaceEntry.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "FBlueprintEditorProjectSettingsCustomization"

namespace UE::Editor::BlueprintEditorProjectSettingsCustomization::Private
{
	class FBlueprintGlobalProjectImportsLayout : public FBlueprintManagedListDetails, public TSharedFromThis<FBlueprintGlobalProjectImportsLayout>
	{
	public:
		FBlueprintGlobalProjectImportsLayout(TSharedRef<IPropertyHandle> InPropertyHandle)
			: FBlueprintManagedListDetails()
			, PropertyHandle(InPropertyHandle)
		{
			DisplayOptions.TitleText = PropertyHandle->GetPropertyDisplayName();
			DisplayOptions.TitleTooltipText = PropertyHandle->GetToolTipText();

			DisplayOptions.NoItemsLabelText = LOCTEXT("NoGlobalImports", "None");

			// Add a custom edit condition to link it to the editor's namespace feature toggle flag (for consistency w/ the editor-specific set).
			DisplayOptions.EditCondition = TAttribute<bool>::CreateLambda([]()
			{
				return GetDefault<UBlueprintEditorSettings>()->bEnableNamespaceEditorFeatures;
			});
		}

	protected:
		/** FBlueprintManagedListDetails interface*/
		virtual TSharedPtr<SWidget> MakeAddItemWidget() override
		{
			return SNew(SBlueprintNamespaceEntry)
				.AllowTextEntry(false)
				.OnNamespaceSelected(this, &FBlueprintGlobalProjectImportsLayout::OnNamespaceSelected)
				.OnGetNamespacesToExclude(this, &FBlueprintGlobalProjectImportsLayout::OnGetNamespacesToExclude)
				.ExcludedNamespaceTooltipText(LOCTEXT("CannotSelectNamespace", "This namespace is already included in the list."))
				.ButtonContent()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("BlueprintAddGlobalImportButton", "Add"))
					.ToolTipText(LOCTEXT("BlueprintAddGlobalImportButton_Tooltip", "Choose a namespace that all Blueprints in this project should import by default (applies to all users)."))
				];
		}

		virtual void GetManagedListItems(TArray<FManagedListItem>& OutListItems) const override
		{
			for(const FString& GlobalNamespace : GetDefault<UBlueprintEditorProjectSettings>()->NamespacesToAlwaysInclude)
			{
				FManagedListItem ItemDesc;
				ItemDesc.ItemName = GlobalNamespace;
				ItemDesc.DisplayName = FText::FromString(GlobalNamespace);
				ItemDesc.bIsRemovable = true;

				OutListItems.Add(MoveTemp(ItemDesc));
			}
		}

		virtual void OnRemoveItem(const FManagedListItem& Item)
		{
			FScopedTransaction Transaction(LOCTEXT("RemoveGlobalImport_Transaction", "Remove Global Import"));

			UBlueprintEditorProjectSettings* BlueprintEditorProjectSettings = GetMutableDefault<UBlueprintEditorProjectSettings>();
			check(BlueprintEditorProjectSettings);

			PropertyHandle->NotifyPreChange();
			BlueprintEditorProjectSettings->NamespacesToAlwaysInclude.Remove(Item.ItemName);
			PropertyHandle->NotifyPostChange(EPropertyChangeType::ArrayRemove);
			PropertyHandle->NotifyFinishedChangingProperties();

			RegenerateChildContent();
		}
		/** END FBlueprintManagedListDetails interface */

		void OnNamespaceSelected(const FString& InNamespace)
		{
			FScopedTransaction Transaction(LOCTEXT("AddGlobalImport_Transaction", "Add Global Import"));

			UBlueprintEditorProjectSettings* BlueprintEditorProjectSettings = GetMutableDefault<UBlueprintEditorProjectSettings>();
			check(BlueprintEditorProjectSettings);

			PropertyHandle->NotifyPreChange();
			BlueprintEditorProjectSettings->NamespacesToAlwaysInclude.AddUnique(InNamespace);
			PropertyHandle->NotifyPostChange(EPropertyChangeType::ArrayAdd);
			PropertyHandle->NotifyFinishedChangingProperties();

			RegenerateChildContent();
		}

		void OnGetNamespacesToExclude(TSet<FString>& OutNamespacesToExclude) const
		{
			// Exclude namespaces that are already listed.
			OutNamespacesToExclude.Append(GetDefault<UBlueprintEditorProjectSettings>()->NamespacesToAlwaysInclude);
		}

	private:
		TSharedRef<IPropertyHandle> PropertyHandle;
	};
}

TSharedRef<IDetailCustomization> FBlueprintEditorProjectSettingsCustomization::MakeInstance()
{
	return MakeShared<FBlueprintEditorProjectSettingsCustomization>();
}

void FBlueprintEditorProjectSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& LayoutBuilder)
{
	static FName PropertyName_NamespacesToAlwaysInclude = GET_MEMBER_NAME_CHECKED(UBlueprintEditorProjectSettings, NamespacesToAlwaysInclude);
	TSharedRef<IPropertyHandle> PropertyHandle_NamespacesToAlwaysInclude = LayoutBuilder.GetProperty(PropertyName_NamespacesToAlwaysInclude);

	PropertyHandle_NamespacesToAlwaysInclude->MarkHiddenByCustomization();

	IDetailCategoryBuilder& CategoryBuilder = LayoutBuilder.EditCategory(PropertyHandle_NamespacesToAlwaysInclude->GetDefaultCategoryName());
	CategoryBuilder.AddCustomBuilder(MakeShared<UE::Editor::BlueprintEditorProjectSettingsCustomization::Private::FBlueprintGlobalProjectImportsLayout>(PropertyHandle_NamespacesToAlwaysInclude));
}

#undef LOCTEXT_NAMESPACE
