// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"

/**
 * Implements a details customization for UBlueprintEditorProjectSettings.
 */
class FBlueprintEditorProjectSettingsCustomization : public IDetailCustomization
{
public:
	// Factory creation.
	static TSharedRef<IDetailCustomization> MakeInstance();

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& LayoutBuilder) override;
	// End IDetailCustomization interface
};