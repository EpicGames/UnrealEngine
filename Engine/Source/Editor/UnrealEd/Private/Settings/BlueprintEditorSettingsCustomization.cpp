// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings/BlueprintEditorSettingsCustomization.h"
#include "BlueprintEditorSettings.h"
#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"
#include "IDetailPropertyRow.h"
#include "DetailWidgetRow.h"
#include "PropertyCustomizationHelpers.h"
#include "BlueprintManagedListDetails.h"
#include "SBlueprintNamespaceEntry.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "FBlueprintEditorSettingsCustomization"

namespace UE::Editor::BlueprintEditorSettingsCustomization::Private
{
	class FBlueprintGlobalEditorImportsLayout : public FBlueprintManagedListDetails, public TSharedFromThis<FBlueprintGlobalEditorImportsLayout>
	{
	public:
		FBlueprintGlobalEditorImportsLayout(TSharedRef<IPropertyHandle> InPropertyHandle)
			: FBlueprintManagedListDetails()
			, PropertyHandle(InPropertyHandle)
		{
			DisplayOptions.TitleText = PropertyHandle->GetPropertyDisplayName();
			DisplayOptions.TitleTooltipText = PropertyHandle->GetToolTipText();

			DisplayOptions.NoItemsLabelText = LOCTEXT("NoGlobalImports", "None");

			// Add an edit condition to link it to the namespace feature toggle flag.
			DisplayOptions.EditCondition = TAttribute<bool>::CreateLambda([]()
			{
				return GetDefault<UBlueprintEditorSettings>()->bEnableNamespaceEditorFeatures;
			});
		}

	protected:
		/** FBlueprintManagedListDetails interface*/
		virtual TSharedPtr<SWidget> MakeAddItemWidget() override
		{
			return SNew(SBlueprintNamespaceEntry)
				.AllowTextEntry(false)
				.OnNamespaceSelected(this, &FBlueprintGlobalEditorImportsLayout::OnNamespaceSelected)
				.OnGetNamespacesToExclude(this, &FBlueprintGlobalEditorImportsLayout::OnGetNamespacesToExclude)
				.ExcludedNamespaceTooltipText(LOCTEXT("CannotSelectNamespace", "This namespace is already included in the list."))
				.ButtonContent()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("BlueprintAddGlobalImportButton", "Add"))
					.ToolTipText(LOCTEXT("BlueprintAddGlobalImportButton_Tooltip", "Choose a namespace that Blueprint editors should always import by default (applies only to you as the current local user)."))
				];
		}

		virtual void GetManagedListItems(TArray<FManagedListItem>& OutListItems) const override
		{
			for (const FString& GlobalNamespace : GetDefault<UBlueprintEditorSettings>()->NamespacesToAlwaysInclude)
			{
				FManagedListItem ItemDesc;
				ItemDesc.ItemName = GlobalNamespace;
				ItemDesc.DisplayName = FText::FromString(GlobalNamespace);
				ItemDesc.bIsRemovable = true;

				OutListItems.Add(MoveTemp(ItemDesc));
			}
		}

		virtual void OnRemoveItem(const FManagedListItem& Item)
		{
			FScopedTransaction Transaction(LOCTEXT("RemoveGlobalImport_Transaction", "Remove Global Import"));

			UBlueprintEditorSettings* BlueprintEditorSettings = GetMutableDefault<UBlueprintEditorSettings>();
			check(BlueprintEditorSettings);

			PropertyHandle->NotifyPreChange();
			BlueprintEditorSettings->NamespacesToAlwaysInclude.Remove(Item.ItemName);
			PropertyHandle->NotifyPostChange(EPropertyChangeType::ArrayRemove);
			PropertyHandle->NotifyFinishedChangingProperties();

			RegenerateChildContent();
		}
		/** END FBlueprintManagedListDetails interface */

		void OnNamespaceSelected(const FString& InNamespace)
		{
			FScopedTransaction Transaction(LOCTEXT("AddGlobalImport_Transaction", "Add Global Import"));

			UBlueprintEditorSettings* BlueprintEditorSettings = GetMutableDefault<UBlueprintEditorSettings>();
			check(BlueprintEditorSettings);

			PropertyHandle->NotifyPreChange();
			BlueprintEditorSettings->NamespacesToAlwaysInclude.AddUnique(InNamespace);
			PropertyHandle->NotifyPostChange(EPropertyChangeType::ArrayAdd);
			PropertyHandle->NotifyFinishedChangingProperties();

			RegenerateChildContent();
		}

		void OnGetNamespacesToExclude(TSet<FString>& OutNamespacesToExclude) const
		{
			// Exclude namespaces that are already listed.
			OutNamespacesToExclude.Append(GetDefault<UBlueprintEditorSettings>()->NamespacesToAlwaysInclude);
		}

	private:
		TSharedRef<IPropertyHandle> PropertyHandle;
	};
}

TSharedRef<IDetailCustomization> FBlueprintEditorSettingsCustomization::MakeInstance()
{
	return MakeShared<FBlueprintEditorSettingsCustomization>();
}

void FBlueprintEditorSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& LayoutBuilder)
{
	// Get the property name and handle for the namespace feature toggle setting. Mark it hidden because we will add it via customization below.
	static FName PropertyName_bEnableNamespaceEditorFeatures = GET_MEMBER_NAME_CHECKED(UBlueprintEditorSettings, bEnableNamespaceEditorFeatures);
	TSharedRef<IPropertyHandle> PropertyHandle_bEnableNamespaceEditorFeatures = LayoutBuilder.GetProperty(PropertyName_bEnableNamespaceEditorFeatures);
	PropertyHandle_bEnableNamespaceEditorFeatures->MarkHiddenByCustomization();

	// Get the property name and handle for the global namespace list setting. Mark it hidden because we will add it via customization below.
	static FName PropertyName_NamespacesToAlwaysInclude = GET_MEMBER_NAME_CHECKED(UBlueprintEditorSettings, NamespacesToAlwaysInclude);
	TSharedRef<IPropertyHandle> PropertyHandle_NamespacesToAlwaysInclude = LayoutBuilder.GetProperty(PropertyName_NamespacesToAlwaysInclude);
	PropertyHandle_NamespacesToAlwaysInclude->MarkHiddenByCustomization();

	// List the edit condition first, followed by the customization for the global namespace list (i.e. ensure they are adjacent within the same category).
	IDetailCategoryBuilder& CategoryBuilder = LayoutBuilder.EditCategory(PropertyHandle_bEnableNamespaceEditorFeatures->GetDefaultCategoryName());
	CategoryBuilder.AddProperty(PropertyHandle_bEnableNamespaceEditorFeatures);
	CategoryBuilder.AddCustomBuilder(MakeShared<UE::Editor::BlueprintEditorSettingsCustomization::Private::FBlueprintGlobalEditorImportsLayout>(PropertyHandle_NamespacesToAlwaysInclude));
}

#undef LOCTEXT_NAMESPACE
