// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"

#include "Settings/EditorSettings.h"

#define LOCTEXT_NAMESPACE "FEditorLoadingSavingSettingsCustomization"

/**
 * Implements a details view customization for UGameMapsSettingsCustomization objects.
 */
class FEditorLoadingSavingSettingsCustomization
	: public IDetailCustomization
{
public:

	/** Virtual destructor. */
	virtual ~FEditorLoadingSavingSettingsCustomization( ) = default;

public:

	// IDetailCustomization interface

	virtual void CustomizeDetails( IDetailLayoutBuilder& LayoutBuilder ) override
	{
		CustomizeStartupCategory(LayoutBuilder);
	}

public:

	/**
	 * Creates a new instance.
	 *
	 * @return A new struct customization for game maps settings.
	 */
	static TSharedRef<IDetailCustomization> MakeInstance( )
	{
		return MakeShareable(new FEditorLoadingSavingSettingsCustomization());
	}

protected:

	/**
	 * Customizes the Startup property category.
	 *
	 * This customization pulls in a setting from the game agnostic HomeScreenSettings in the MainFrame module, which
	 * are stored in a different UObject, but which we would like to show in this section.
	 *
	 * @param LayoutBuilder The layout builder.
	 */
	void CustomizeStartupCategory( IDetailLayoutBuilder& LayoutBuilder )
	{
		IDetailCategoryBuilder& StartupCategory = LayoutBuilder.EditCategory("Startup");
		{
			if (const UClass* SettingsClass = FindObject<UClass>(nullptr, TEXT("/Script/MainFrame.HomeScreenSettings")))
			{
				TArray<UObject*> HomeScreenSettingObjectList;
				HomeScreenSettingObjectList.Add(SettingsClass->GetDefaultObject());
				StartupCategory.AddExternalObjectProperty(HomeScreenSettingObjectList, "LoadAtStartup");
			}
		}
	}
};


#undef LOCTEXT_NAMESPACE
