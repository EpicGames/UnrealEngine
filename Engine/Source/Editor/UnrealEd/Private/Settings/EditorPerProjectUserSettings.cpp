// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/EditorPerProjectUserSettings.h"
#include "Misc/Paths.h"
#include "HAL/IConsoleManager.h"
#include "RHI.h"
#include "ToolMenus.h"
#include "UnrealEdMisc.h"
#include "BlueprintPaletteFavorites.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorPerProjectUserSettings)

#define LOCTEXT_NAMESPACE "EditorPerProjectUserSettings"

static TAutoConsoleVariable<int32> CVarNeverStartInPreviewMode(
	TEXT("r.Editor.NeverStartInPreviewMode"),
	0,
	TEXT("0: Editor can start in preview mode, 1: Editor never starts in preview mode"),
	ECVF_Default);

/// @cond DOXYGEN_WARNINGS

UEditorPerProjectUserSettings::UEditorPerProjectUserSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	//Default to high quality
	BlueprintFavorites = CreateDefaultSubobject<UBlueprintPaletteFavorites>(TEXT("BlueprintFavorites"));
	SCSViewportCameraSpeed_DEPRECATED = 4;
	AssetViewerProfileIndex = 0;
	bAnimationReimportWarnings = false;

	bDisplayBlackboardKeysInAlphabeticalOrder = false;

	SimplygonServerIP = "127.0.0.1";
	SimplygonSwarmDelay = 5000;
	bEnableSwarmDebugging = false;
	SwarmNumOfConcurrentJobs = 16;
	SwarmMaxUploadChunkSizeInMB = 100;
	SwarmIntermediateFolder = FPaths::ConvertRelativePathToFull(FPaths::ProjectIntermediateDir() + TEXT("Simplygon/"));
	PreviewFeatureLevel = GMaxRHIFeatureLevel;
	PreviewPlatformName = NAME_None;
	PreviewShaderFormatName = NAME_None;
	bPreviewShaderPlatformActive = false;
	bPreviewFeatureLevelWasDefault = true;
	PreviewDeviceProfileName = NAME_None;
	bShowSelectionSubcomponents = true;

	bShowEnsuresInMessageLog = true;
}

void UEditorPerProjectUserSettings::PostInitProperties()
{
	Super::PostInitProperties();

	// if we last saved as the default or we somehow are loading a preview feature level higher than we can support or we explictly disabled it with r.Editor.NeverStartInPreviewMode,
	// fall back to the current session's maximum feature level
	if (bPreviewFeatureLevelWasDefault || PreviewFeatureLevel > GMaxRHIFeatureLevel || (CVarNeverStartInPreviewMode.GetValueOnAnyThread() != 0))
	{
		PreviewFeatureLevel = GMaxRHIFeatureLevel;
		PreviewShaderPlatformName = NAME_None;
		PreviewShaderFormatName = NAME_None;
        PreviewPlatformName = NAME_None;
		bPreviewShaderPlatformActive = false;
		bPreviewFeatureLevelWasDefault = true;
		PreviewDeviceProfileName = NAME_None;
	}
}

#if WITH_EDITOR
void UEditorPerProjectUserSettings::PostEditChangeProperty( FPropertyChangedEvent& PropertyChangedEvent )
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName Name = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if (Name == FName(TEXT("bUseCurvesForDistributions")))
	{
		extern ENGINE_API uint32 GDistributionType;
		//GDistributionType == 0 for curves
		GDistributionType = (bUseCurvesForDistributions) ? 0 : 1;
	}
	else if (Name == "bDisplayUIExtensionPoints")
	{
		// Refresh all ToolMenu widgets to make them reflect this setting's new value.
		if (UToolMenus* ToolMenus = UToolMenus::TryGet())
		{
			ToolMenus->RefreshAllWidgets();
		}
	}

	if (!FUnrealEdMisc::Get().IsDeletePreferences())
	{
		SaveConfig();
	}

	UserSettingChangedEvent.Broadcast(Name);
}
#endif

/// @endcond

#undef LOCTEXT_NAMESPACE
