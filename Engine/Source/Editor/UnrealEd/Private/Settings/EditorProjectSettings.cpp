// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings/EditorProjectSettings.h"
#include "UObject/UnrealType.h"
#include "Toolkits/ToolkitManager.h"
#include "BlueprintEditor.h"
#include "Editor/EditorPerformanceSettings.h"
#include "EditorViewportClient.h"
#include "Editor.h"
#include "Engine/GameViewportClient.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorProjectSettings)


TAutoConsoleVariable<int32> CVarEditorViewportDefaultScreenPercentageMode(
	TEXT("r.Editor.Viewport.ScreenPercentageMode.RealTime"), int32(EScreenPercentageMode::BasedOnDisplayResolution),
	TEXT("Controls the default screen percentage mode for realtime editor viewports using desktop renderer."),
	ECVF_Default);

TAutoConsoleVariable<int32> CVarEditorViewportDefaultMobileScreenPercentageMode(
	TEXT("r.Editor.Viewport.ScreenPercentageMode.Mobile"), int32(EScreenPercentageMode::BasedOnDPIScale),
	TEXT("Controls the default screen percentage mode for realtime editor viewports using mobile renderer."),
	ECVF_Default);

TAutoConsoleVariable<int32> CVarEditorViewportDefaultVRScreenPercentageMode(
	TEXT("r.Editor.Viewport.ScreenPercentageMode.VR"), int32(EScreenPercentageMode::Manual),
	TEXT("Controls the default screen percentage mode for VR editor viewports."),
	ECVF_Default);

TAutoConsoleVariable<int32> CVarEditorViewportDefaultPathTracerScreenPercentageMode(
	TEXT("r.Editor.Viewport.ScreenPercentageMode.PathTracer"), int32(EScreenPercentageMode::Manual),
	TEXT("Controls the default screen percentage mode for path-traced viewports."),
	ECVF_Default);

TAutoConsoleVariable<int32> CVarEditorViewportNonRealtimeDefaultScreenPercentageMode(
	TEXT("r.Editor.Viewport.ScreenPercentageMode.NonRealTime"), int32(EScreenPercentageMode::BasedOnDPIScale),
	TEXT("Controls the default screen percentage mode for non-realtime editor viewports."),
	ECVF_Default);

TAutoConsoleVariable<float> CVarEditorViewportDefaultScreenPercentage(
	TEXT("r.Editor.Viewport.ScreenPercentage"), 100,
	TEXT("Controls the editor viewports' default screen percentage when using r.Editor.Viewport.ScreenPercentageMode=0."),
	ECVF_Default);

TAutoConsoleVariable<int32> CVarEditorViewportDefaultMinRenderingResolution(
	TEXT("r.Editor.Viewport.MinRenderingResolution"), 720,
	TEXT("Controls the minimum number of rendered pixel by default in editor viewports."),
	ECVF_Default);

TAutoConsoleVariable<int32> CVarEditorViewportDefaultMaxRenderingResolution(
	TEXT("r.Editor.Viewport.MaxRenderingResolution"), 2160,
	TEXT("Controls the absolute maximum number of rendered pixel in editor viewports."),
	ECVF_Default);


EUnit ConvertDefaultInputUnits(EDefaultLocationUnit In)
{
	typedef EDefaultLocationUnit EDefaultLocationUnit;

	switch(In)
	{
	case EDefaultLocationUnit::Micrometers:		return EUnit::Micrometers;
	case EDefaultLocationUnit::Millimeters:		return EUnit::Millimeters;
	case EDefaultLocationUnit::Centimeters:		return EUnit::Centimeters;
	case EDefaultLocationUnit::Meters:			return EUnit::Meters;
	case EDefaultLocationUnit::Kilometers:		return EUnit::Kilometers;
	case EDefaultLocationUnit::Inches:			return EUnit::Inches;
	case EDefaultLocationUnit::Feet:			return EUnit::Feet;
	case EDefaultLocationUnit::Yards:			return EUnit::Yards;
	case EDefaultLocationUnit::Miles:			return EUnit::Miles;
	default:									return EUnit::Centimeters;
	}
}

UEditorProjectAppearanceSettings::UEditorProjectAppearanceSettings(const FObjectInitializer& Initializer)
	: Super(Initializer)
	, bDisplayUnitsOnComponentTransforms(false)
	, UnitDisplay_DEPRECATED(EUnitDisplay::Invalid)
	, DefaultInputUnits_DEPRECATED(EDefaultLocationUnit::Invalid)
{
}

void UEditorProjectAppearanceSettings::PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent )
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName Name = PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	auto& Settings = FUnitConversion::Settings();
	if (Name == GET_MEMBER_NAME_CHECKED(UEditorProjectAppearanceSettings, DistanceUnits))
	{
		Settings.SetDisplayUnits(EUnitType::Distance, DistanceUnits);
	}
	else if (Name == GET_MEMBER_NAME_CHECKED(UEditorProjectAppearanceSettings, MassUnits))
	{
		Settings.SetDisplayUnits(EUnitType::Mass, MassUnits);
	}
	else if (Name == GET_MEMBER_NAME_CHECKED(UEditorProjectAppearanceSettings, TimeUnits))
	{
		Settings.SetDisplayUnits(EUnitType::Time, TimeUnits);
	}
	else if (Name == GET_MEMBER_NAME_CHECKED(UEditorProjectAppearanceSettings, AngleUnits))
	{
		Settings.SetDisplayUnits(EUnitType::Angle, AngleUnits);
	}
	else if (Name == GET_MEMBER_NAME_CHECKED(UEditorProjectAppearanceSettings, SpeedUnits))
	{
		Settings.SetDisplayUnits(EUnitType::Speed, SpeedUnits);
	}
	else if (Name == GET_MEMBER_NAME_CHECKED(UEditorProjectAppearanceSettings, AngularSpeedUnits))
	{
		Settings.SetDisplayUnits(EUnitType::AngularSpeed, AngularSpeedUnits);
	}
	else if (Name == GET_MEMBER_NAME_CHECKED(UEditorProjectAppearanceSettings, AccelerationUnits))
	{
		Settings.SetDisplayUnits(EUnitType::Acceleration, AccelerationUnits);
	}
	else if (Name == GET_MEMBER_NAME_CHECKED(UEditorProjectAppearanceSettings, TemperatureUnits))
	{
		Settings.SetDisplayUnits(EUnitType::Temperature, TemperatureUnits);
	}
	else if (Name == GET_MEMBER_NAME_CHECKED(UEditorProjectAppearanceSettings, ForceUnits))
	{
		Settings.SetDisplayUnits(EUnitType::Force, ForceUnits);
	}
	else if (Name == GET_MEMBER_NAME_CHECKED(UEditorProjectAppearanceSettings, TorqueUnits))
	{
		Settings.SetDisplayUnits(EUnitType::Torque, TorqueUnits);
	}
	else if (Name == GET_MEMBER_NAME_CHECKED(UEditorProjectAppearanceSettings, ImpulseUnits))
	{
		Settings.SetDisplayUnits(EUnitType::Impulse, ImpulseUnits);
	}
	else if (Name == GET_MEMBER_NAME_CHECKED(UEditorProjectAppearanceSettings, PositionalImpulseUnits))
	{
		Settings.SetDisplayUnits(EUnitType::PositionalImpulse, PositionalImpulseUnits);
	}
	else if (Name == GET_MEMBER_NAME_CHECKED(UEditorProjectAppearanceSettings, bDisplayUnits))
	{
		Settings.SetShouldDisplayUnits(bDisplayUnits);
	}

	DefaultInputUnits_DEPRECATED = EDefaultLocationUnit::Invalid;
	UnitDisplay_DEPRECATED = EUnitDisplay::Invalid;

	SaveConfig();
}

void SetupEnumMetaData(UClass* Class, const FName& MemberName, const TCHAR* Values)
{
	FArrayProperty* Array = CastField<FArrayProperty>(Class->FindPropertyByName(MemberName));
	if (Array && Array->Inner)
	{
		Array->Inner->SetMetaData(TEXT("ValidEnumValues"), Values);
	}
}

void UEditorProjectAppearanceSettings::PostInitProperties()
{
	Super::PostInitProperties();

	/** Setup the meta data for the array properties */
	SetupEnumMetaData(GetClass(), GET_MEMBER_NAME_CHECKED(UEditorProjectAppearanceSettings, DistanceUnits), TEXT("Micrometers, Millimeters, Centimeters, Meters, Kilometers, Inches, Feet, Yards, Miles"));
	SetupEnumMetaData(GetClass(), GET_MEMBER_NAME_CHECKED(UEditorProjectAppearanceSettings, MassUnits), TEXT("Micrograms, Milligrams, Grams, Kilograms, MetricTons,	Ounces, Pounds, Stones"));
	SetupEnumMetaData(GetClass(), GET_MEMBER_NAME_CHECKED(UEditorProjectAppearanceSettings, TimeUnits), TEXT("Milliseconds, Seconds, Minutes, Hours, Days, Months, Years"));

	if (UnitDisplay_DEPRECATED != EUnitDisplay::Invalid)
	{
		bDisplayUnits = UnitDisplay_DEPRECATED != EUnitDisplay::None;
	}

	if (DefaultInputUnits_DEPRECATED != EDefaultLocationUnit::Invalid)
	{
		DistanceUnits.Empty();
		DistanceUnits.Add(ConvertDefaultInputUnits(DefaultInputUnits_DEPRECATED));
	}

	auto& Settings = FUnitConversion::Settings();

	Settings.SetDisplayUnits(EUnitType::Distance, DistanceUnits);
	Settings.SetDisplayUnits(EUnitType::Mass, MassUnits);
	Settings.SetDisplayUnits(EUnitType::Time, TimeUnits);
	Settings.SetDisplayUnits(EUnitType::Angle, AngleUnits);
	Settings.SetDisplayUnits(EUnitType::Speed, SpeedUnits);
	Settings.SetDisplayUnits(EUnitType::AngularSpeed, AngularSpeedUnits);
	Settings.SetDisplayUnits(EUnitType::Acceleration, AccelerationUnits);
	Settings.SetDisplayUnits(EUnitType::Temperature, TemperatureUnits);
	Settings.SetDisplayUnits(EUnitType::Force, ForceUnits);
	Settings.SetDisplayUnits(EUnitType::Torque, TorqueUnits);
	Settings.SetDisplayUnits(EUnitType::Impulse, ImpulseUnits);
	Settings.SetDisplayUnits(EUnitType::PositionalImpulse, PositionalImpulseUnits);

	Settings.SetShouldDisplayUnits(bDisplayUnits);
}


/* ULevelEditor2DSettings
*****************************************************************************/

ULevelEditor2DSettings::ULevelEditor2DSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, SnapAxis(ELevelEditor2DAxis::Y)
{
	SnapLayers.Emplace(TEXT("Foreground"), 100.0f);
	SnapLayers.Emplace(TEXT("Default"), 0.0f);
	SnapLayers.Emplace(TEXT("Background"), -100.0f);
}

void ULevelEditor2DSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	// Sort the snap layers
	SnapLayers.Sort([](const FMode2DLayer& LHS, const FMode2DLayer& RHS){ return LHS.Depth > RHS.Depth; });

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

/* ULevelEditorProjectSettings
*****************************************************************************/

ULevelEditorProjectSettings::ULevelEditorProjectSettings( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
	, bEnableViewportSMInstanceSelection(true)
{
}

void ULevelEditorProjectSettings::PostInitProperties()
{
	Super::PostInitProperties();
	if (FProperty* EnableSMInstanceSelectionProperty = GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ULevelEditorProjectSettings, bEnableViewportSMInstanceSelection)))
	{
		ExportValuesToConsoleVariables(EnableSMInstanceSelectionProperty);
	}
}

void ULevelEditorProjectSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	if (!PropertyChangedEvent.Property)
	{
		return;
	}
	ExportValuesToConsoleVariables(PropertyChangedEvent.Property);
}


/* UEditorPerformanceProjectSettings
*****************************************************************************/

UEditorPerformanceProjectSettings::UEditorPerformanceProjectSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, RealtimeScreenPercentageMode(EScreenPercentageMode::BasedOnDisplayResolution)
	, MobileScreenPercentageMode(EScreenPercentageMode::BasedOnDPIScale)
	, VRScreenPercentageMode(EScreenPercentageMode::Manual)
	, PathTracerScreenPercentageMode(EScreenPercentageMode::Manual)
	, NonRealtimeScreenPercentageMode(EScreenPercentageMode::BasedOnDPIScale)
	, ManualScreenPercentage(100.0f)
	, MinViewportRenderingResolution(720)
	, MaxViewportRenderingResolution(2160)
{
}

void UEditorPerformanceProjectSettings::PostInitProperties()
{
	Super::PostInitProperties();
	ExportResolutionValuesToConsoleVariables();
}

void UEditorPerformanceProjectSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	if (!PropertyChangedEvent.Property)
	{
		return;
	}

	if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UEditorPerformanceProjectSettings, RealtimeScreenPercentageMode) ||
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UEditorPerformanceProjectSettings, MobileScreenPercentageMode) ||
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UEditorPerformanceProjectSettings, VRScreenPercentageMode) ||
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UEditorPerformanceProjectSettings, PathTracerScreenPercentageMode) ||
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UEditorPerformanceProjectSettings, NonRealtimeScreenPercentageMode) ||
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UEditorPerformanceProjectSettings, ManualScreenPercentage) ||
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UEditorPerformanceProjectSettings, MinViewportRenderingResolution) ||
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UEditorPerformanceProjectSettings, MaxViewportRenderingResolution))
	{
		ExportResolutionValuesToConsoleVariables();
	}
	else
	{
		ExportValuesToConsoleVariables(PropertyChangedEvent.Property);
	}
}

// static
void UEditorPerformanceProjectSettings::ExportResolutionValuesToConsoleVariables()
{
	// Apply project settings settings
	{
		const UEditorPerformanceProjectSettings* EditorProjectSettings = GetDefault<UEditorPerformanceProjectSettings>();

		CVarEditorViewportDefaultScreenPercentageMode->Set(int32(EditorProjectSettings->RealtimeScreenPercentageMode), ECVF_SetByProjectSetting);
		CVarEditorViewportDefaultMobileScreenPercentageMode->Set(int32(EditorProjectSettings->MobileScreenPercentageMode), ECVF_SetByProjectSetting);
		CVarEditorViewportDefaultVRScreenPercentageMode->Set(int32(EditorProjectSettings->VRScreenPercentageMode), ECVF_SetByProjectSetting);
		CVarEditorViewportDefaultPathTracerScreenPercentageMode->Set(int32(EditorProjectSettings->PathTracerScreenPercentageMode), ECVF_SetByProjectSetting);
		CVarEditorViewportNonRealtimeDefaultScreenPercentageMode->Set(int32(EditorProjectSettings->NonRealtimeScreenPercentageMode), ECVF_SetByProjectSetting);
		CVarEditorViewportDefaultScreenPercentage->Set(EditorProjectSettings->ManualScreenPercentage, ECVF_SetByProjectSetting);
		CVarEditorViewportDefaultMinRenderingResolution->Set(EditorProjectSettings->MinViewportRenderingResolution, ECVF_SetByProjectSetting);
		CVarEditorViewportDefaultMaxRenderingResolution->Set(EditorProjectSettings->MaxViewportRenderingResolution, ECVF_SetByProjectSetting);
	}


	// Override with the per editor user settings
	{
		const UEditorPerformanceSettings* EditorUserSettings = GetDefault<UEditorPerformanceSettings>();

		// Real-time viewports
		auto OverrideCVarModeWithEditorUserSettings = [](TAutoConsoleVariable<int32>& CVar, EEditorUserScreenPercentageModeOverride EditorUserSettings)
		{
			if (EditorUserSettings == EEditorUserScreenPercentageModeOverride::Manual)
			{
				CVar->Set(int32(EScreenPercentageMode::Manual), ECVF_SetByProjectSetting);
			}
			else if (EditorUserSettings == EEditorUserScreenPercentageModeOverride::BasedOnDisplayResolution)
			{
				CVar->Set(int32(EScreenPercentageMode::BasedOnDisplayResolution), ECVF_SetByProjectSetting);
			}
			else if (EditorUserSettings == EEditorUserScreenPercentageModeOverride::BasedOnDPIScale)
			{
				CVar->Set(int32(EScreenPercentageMode::BasedOnDPIScale), ECVF_SetByProjectSetting);
			}
		};

		OverrideCVarModeWithEditorUserSettings(CVarEditorViewportDefaultScreenPercentageMode, EditorUserSettings->RealtimeScreenPercentageMode);
		OverrideCVarModeWithEditorUserSettings(CVarEditorViewportDefaultMobileScreenPercentageMode, EditorUserSettings->MobileScreenPercentageMode);
		OverrideCVarModeWithEditorUserSettings(CVarEditorViewportDefaultVRScreenPercentageMode, EditorUserSettings->VRScreenPercentageMode);
		OverrideCVarModeWithEditorUserSettings(CVarEditorViewportDefaultPathTracerScreenPercentageMode, EditorUserSettings->PathTracerScreenPercentageMode);
		OverrideCVarModeWithEditorUserSettings(CVarEditorViewportNonRealtimeDefaultScreenPercentageMode, EditorUserSettings->NonRealtimeScreenPercentageMode);

		if (EditorUserSettings->bOverrideManualScreenPercentage)
		{
			CVarEditorViewportDefaultScreenPercentage->Set(EditorUserSettings->ManualScreenPercentage, ECVF_SetByProjectSetting);
		}

		if (EditorUserSettings->bOverrideMinViewportRenderingResolution)
		{
			CVarEditorViewportDefaultMinRenderingResolution->Set(EditorUserSettings->MinViewportRenderingResolution, ECVF_SetByProjectSetting);
		}

		if (EditorUserSettings->bOverrideMaxViewportRenderingResolution)
		{
			CVarEditorViewportDefaultMaxRenderingResolution->Set(EditorUserSettings->MaxViewportRenderingResolution, ECVF_SetByProjectSetting);
		}
	}

	// Tell all viewports to refresh their screen percentage when the dpi scaling override changes
	if (GEngine && GEditor)
	{
		for (FEditorViewportClient* Client : GEditor->GetAllViewportClients())
		{
			if (Client)
			{
				Client->RequestUpdateDPIScale();
				Client->Invalidate();
			}
		}

		if (GEngine->GameViewport)
		{
			GEngine->GameViewport->RequestUpdateDPIScale();
		}
	}
}
