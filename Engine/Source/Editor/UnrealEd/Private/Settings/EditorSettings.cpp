// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings/EditorSettings.h"

#include "CoreGlobals.h"
#include "HAL/FileManager.h"
#include "HAL/Platform.h"
#include "HAL/PlatformMisc.h"
#include "Interfaces/IProjectManager.h"
#include "Internationalization/Internationalization.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "IDesktopPlatform.h"
#include "DesktopPlatformModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorSettings)

UEditorSettings::UEditorSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	AutoScalabilityWorkScaleAmount = 1;

	// Read the current state of the environment variables and cache it.
	GlobalLocalDDCPath.Path = FPlatformMisc::GetEnvironmentVariable(TEXT("UE-LocalDataCachePath"));
	GlobalSharedDDCPath.Path = FPlatformMisc::GetEnvironmentVariable(TEXT("UE-SharedDataCachePath"));
	GlobalS3DDCPath.Path = FPlatformMisc::GetEnvironmentVariable(TEXT("UE-S3DataCachePath"));

	// If the user has set the stored value we'll stomp the environmental variable's influence, otherwise the environment variable option reigns.
	FPlatformMisc::GetStoredValue(TEXT("Epic Games"), TEXT("GlobalDataCachePath"), TEXT("UE-LocalDataCachePath"), GlobalLocalDDCPath.Path);
	FPlatformMisc::GetStoredValue(TEXT("Epic Games"), TEXT("GlobalDataCachePath"), TEXT("UE-SharedDataCachePath"), GlobalSharedDDCPath.Path);
	FPlatformMisc::GetStoredValue(TEXT("Epic Games"), TEXT("GlobalDataCachePath"), TEXT("UE-S3DataCachePath"), GlobalS3DDCPath.Path);

	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::TryGet();
	if (DesktopPlatform != nullptr)
	{
		DesktopPlatform->GetHordeUrl(HordeUrl);
	}
}

bool UEditorSettings::CanEditChange(const FProperty* InProperty) const
{
	if (InProperty->GetFName().IsEqual(GET_MEMBER_NAME_CHECKED(UEditorSettings, bEnableS3DDC)))
	{
		bool bValue = false;
		if (!GConfig->GetBool(TEXT("EditorSettings"), TEXT("bShowEnableS3DDC"), bValue, GEditorIni) || !bValue)
		{
			return false;
		}
	}

	return Super::CanEditChange(InProperty);
}

void UEditorSettings::PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FProperty* PropertyThatChanged = PropertyChangedEvent.MemberProperty;
	const FName PropertyName = PropertyThatChanged ? PropertyThatChanged->GetFName() : NAME_None;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FName DeprecatedName = GET_MEMBER_NAME_CHECKED(UEditorSettings, bLoadTheMostRecentlyLoadedProjectAtStartup);
	if (PropertyName == DeprecatedName)
	{
		const FString& AutoLoadProjectFileName = IProjectManager::Get().GetAutoLoadProjectFileName();
		if ( bLoadTheMostRecentlyLoadedProjectAtStartup )
		{
			// Form or overwrite the file that is read at load to determine the most recently loaded project file
			FFileHelper::SaveStringToFile(FPaths::GetProjectFilePath(), *AutoLoadProjectFileName);
		}
		else
		{
			// Remove the file. It's possible for bLoadTheMostRecentlyLoadedProjectAtStartup to be set before FPaths::GetProjectFilePath() is valid, so we need to distinguish the two cases.
			IFileManager::Get().Delete(*AutoLoadProjectFileName);
		}

		// Broadcast that this setting changed
		OnMostRecentProjectSettingChanged().Broadcast(bLoadTheMostRecentlyLoadedProjectAtStartup);
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UEditorSettings, GlobalLocalDDCPath))
	{
		if (GlobalLocalDDCPath.Path.IsEmpty())
		{
			FPlatformMisc::DeleteStoredValue(TEXT("Epic Games"), TEXT("GlobalDataCachePath"), TEXT("UE-LocalDataCachePath"));
			
			// empty registry key means use environment, so re-fetch it now
			GlobalLocalDDCPath.Path = FPlatformMisc::GetEnvironmentVariable(TEXT("UE-LocalDataCachePath"));
		}
		else
		{
			FPlatformMisc::SetStoredValue(TEXT("Epic Games"), TEXT("GlobalDataCachePath"), TEXT("UE-LocalDataCachePath"), GlobalLocalDDCPath.Path);
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UEditorSettings, GlobalSharedDDCPath))
	{
		if (GlobalSharedDDCPath.Path.IsEmpty())
		{
			FPlatformMisc::DeleteStoredValue(TEXT("Epic Games"), TEXT("GlobalDataCachePath"), TEXT("UE-SharedDataCachePath"));
			
			// empty registry key means use environment, so re-fetch it now
			GlobalSharedDDCPath.Path = FPlatformMisc::GetEnvironmentVariable(TEXT("UE-SharedDataCachePath"));
		}
		else
		{
			FPlatformMisc::SetStoredValue(TEXT("Epic Games"), TEXT("GlobalDataCachePath"), TEXT("UE-SharedDataCachePath"), GlobalSharedDDCPath.Path);
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UEditorSettings, GlobalS3DDCPath))
	{
		if (GlobalS3DDCPath.Path.IsEmpty())
		{
			FPlatformMisc::DeleteStoredValue(TEXT("Epic Games"), TEXT("GlobalDataCachePath"), TEXT("UE-S3DataCachePath"));
			
			// empty registry key means use environment, so re-fetch it now
			GlobalS3DDCPath.Path = FPlatformMisc::GetEnvironmentVariable(TEXT("UE-S3DataCachePath"));
		}
		else
		{
			FPlatformMisc::SetStoredValue(TEXT("Epic Games"), TEXT("GlobalDataCachePath"), TEXT("UE-S3DataCachePath"), GlobalS3DDCPath.Path);
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UEditorSettings, HordeUrl))
	{
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::TryGet();
		if (DesktopPlatform != nullptr)
		{
			DesktopPlatform->SetHordeUrl(HordeUrl);
		}
	}

	SaveConfig(CPF_Config);
}

void UEditorSettings::LoadScalabilityBenchmark()
{
	check(!GEditorSettingsIni.IsEmpty());

	const TCHAR* Section = TEXT("EngineBenchmarkResult");

	Scalability::FQualityLevels Temporary;

	if (IsScalabilityBenchmarkValid())
	{
		GConfig->GetFloat(Section, TEXT("ResolutionQuality"), Temporary.ResolutionQuality, GEditorSettingsIni);
		GConfig->GetInt(Section, TEXT("ViewDistanceQuality"), Temporary.ViewDistanceQuality, GEditorSettingsIni);
		GConfig->GetInt(Section, TEXT("AntiAliasingQuality"), Temporary.AntiAliasingQuality, GEditorSettingsIni);
		GConfig->GetInt(Section, TEXT("ShadowQuality"), Temporary.ShadowQuality, GEditorSettingsIni);
		GConfig->GetInt(Section, TEXT("GlobalIlluminationQuality"), Temporary.GlobalIlluminationQuality, GEditorSettingsIni);
		GConfig->GetInt(Section, TEXT("ReflectionQuality"), Temporary.ReflectionQuality, GEditorSettingsIni);
		GConfig->GetInt(Section, TEXT("PostProcessQuality"), Temporary.PostProcessQuality, GEditorSettingsIni);
		GConfig->GetInt(Section, TEXT("TextureQuality"), Temporary.TextureQuality, GEditorSettingsIni);
		GConfig->GetInt(Section, TEXT("EffectsQuality"), Temporary.EffectsQuality, GEditorSettingsIni);
		GConfig->GetInt(Section, TEXT("FoliageQuality"), Temporary.FoliageQuality, GEditorSettingsIni);
		GConfig->GetInt(Section, TEXT("ShadingQuality"), Temporary.ShadingQuality, GEditorSettingsIni);
		GConfig->GetInt(Section, TEXT("LandscapeQuality"), Temporary.LandscapeQuality, GEditorSettingsIni);
		EngineBenchmarkResult = Temporary;
	}
}

void UEditorSettings::AutoApplyScalabilityBenchmark()
{
	const TCHAR* Section = TEXT("EngineBenchmarkResult");

	FScopedSlowTask SlowTask(0, NSLOCTEXT("UnrealEd", "RunningEngineBenchmark", "Running engine benchmark..."));
	SlowTask.MakeDialog();


	Scalability::FQualityLevels Temporary = Scalability::BenchmarkQualityLevels(AutoScalabilityWorkScaleAmount);

	GConfig->SetBool(Section, TEXT("Valid"), true, GEditorSettingsIni);
	GConfig->SetFloat(Section, TEXT("ResolutionQuality"), Temporary.ResolutionQuality, GEditorSettingsIni);
	GConfig->SetInt(Section, TEXT("ViewDistanceQuality"), Temporary.ViewDistanceQuality, GEditorSettingsIni);
	GConfig->SetInt(Section, TEXT("AntiAliasingQuality"), Temporary.AntiAliasingQuality, GEditorSettingsIni);
	GConfig->SetInt(Section, TEXT("ShadowQuality"), Temporary.ShadowQuality, GEditorSettingsIni);
	GConfig->SetInt(Section, TEXT("GlobalIlluminationQuality"), Temporary.GlobalIlluminationQuality, GEditorSettingsIni);
	GConfig->SetInt(Section, TEXT("ReflectionQuality"), Temporary.ReflectionQuality, GEditorSettingsIni);
	GConfig->SetInt(Section, TEXT("PostProcessQuality"), Temporary.PostProcessQuality, GEditorSettingsIni);
	GConfig->SetInt(Section, TEXT("TextureQuality"), Temporary.TextureQuality, GEditorSettingsIni);
	GConfig->SetInt(Section, TEXT("EffectsQuality"), Temporary.EffectsQuality, GEditorSettingsIni);
	GConfig->SetInt(Section, TEXT("FoliageQuality"), Temporary.FoliageQuality, GEditorSettingsIni);
	GConfig->SetInt(Section, TEXT("ShadingQuality"), Temporary.ShadingQuality, GEditorSettingsIni);
	GConfig->SetInt(Section, TEXT("LandscapeQuality"), Temporary.LandscapeQuality, GEditorSettingsIni);

	Scalability::SetQualityLevels(Temporary);
	Scalability::SaveState(GEditorSettingsIni);
}

bool UEditorSettings::IsScalabilityBenchmarkValid() const
{
	const TCHAR* Section = TEXT("EngineBenchmarkResult");

	bool bIsValid = false;
	GConfig->GetBool(Section, TEXT("Valid"), bIsValid, GEditorSettingsIni);

	return bIsValid;
}
