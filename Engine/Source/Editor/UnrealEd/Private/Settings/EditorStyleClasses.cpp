// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreGlobals.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/IConsoleManager.h"
#include "HAL/Platform.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ConfigCacheIni.h"
#include "Settings/EditorStyleSettings.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "Styling/StyleColors.h"
#include "HAL/FileManager.h"
#include "EditorStyleSettingsCustomization.h"

class FObjectInitializer;


/* UEditorStyleSettings interface
 *****************************************************************************/

UEditorStyleSettings::UEditorStyleSettings( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
	, AdditionalSelectionColors{
		FStyleColors::AccentBlue.GetSpecifiedColor(),
		FStyleColors::AccentPurple.GetSpecifiedColor(),
		FStyleColors::AccentPink.GetSpecifiedColor(),
		FStyleColors::AccentRed.GetSpecifiedColor(),
		FStyleColors::AccentYellow.GetSpecifiedColor(),
		FStyleColors::AccentGreen.GetSpecifiedColor(),
	}
	, ViewportToolOverlayColor(FLinearColor::White)
{
	bEnableUserEditorLayoutManagement = true;

	SelectionColor = FLinearColor(0.828f, 0.364f, 0.003f);

	EditorWindowBackgroundColor = FLinearColor::White;

	AssetEditorOpenLocation = EAssetEditorOpenLocation::Default;
	bEnableColorizedEditorTabs = true;
	
	bUseGrid = true;
	bAntiAliasGrid = true;

	RegularColor = FLinearColor(0.024f, 0.024f, 0.024f);
	RuleColor = FLinearColor(0.010f, 0.010f, 0.010f);
	CenterColor = FLinearColor(0.005f, 0.005f, 0.005f) ;

	GridSnapSize = 16;

	bShowFriendlyNames = true;
	bShowNativeComponentNames = true;
}

void UEditorStyleSettings::Init()
{
	// if it's a valid id, set current theme ID in USlateThemeManager to CurrentAppliedTheme. 
	if (CurrentAppliedTheme.IsValid())
	{
		USlateThemeManager::Get().ApplyTheme(CurrentAppliedTheme); 
	}
	// if it's invalid id, set current theme to default. 
	else
	{
		CurrentAppliedTheme = USlateThemeManager::Get().GetCurrentThemeID();
		SaveConfig(); 
	}
	
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().SetApplicationScale(ApplicationScale);
	}

	// Set from CVar 
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("EnableHighDPIAwareness"));
	bEnableHighDPIAwareness = CVar->GetInt() != 0;
}

FLinearColor UEditorStyleSettings::GetSubduedSelectionColor() const
{
	FLinearColor SubduedSelectionColor = SelectionColor.LinearRGBToHSV();
	SubduedSelectionColor.G *= 0.55f;		// take the saturation 
	SubduedSelectionColor.B *= 0.8f;		// and brightness down

	return SubduedSelectionColor.HSVToLinearRGB();
}

void UEditorStyleSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	OnPropertyChanged(PropertyName);

	SettingChangedEvent.Broadcast(PropertyName);
}

bool UEditorStyleSettings::OnImportBegin(const FString& ImportFromPath)
{
	LoadConfig(GetClass(), *ImportFromPath, UE::LCPF_PropagateToInstances);

	// if theme exists, it would already be applied by import. 
	// if theme doesn't exist, prompt to open to import a theme. 
	if (!USlateThemeManager::Get().DoesThemeExist(CurrentAppliedTheme))
	{
		FEditorStyleSettingsCustomization::PromptToImportTheme(ImportFromPath); 
	}

	OnPropertyChanged(NAME_None);

	return true; 
}

void UEditorStyleSettings::OnPropertyChanged(FName InPropertyName)
{
	// This property is intentionally not per project so it must be manually written to the correct config file
	if (InPropertyName.IsNone() || InPropertyName == GET_MEMBER_NAME_CHECKED(UEditorStyleSettings, bEnableHighDPIAwareness))
	{
		GConfig->SetBool(TEXT("HDPI"), TEXT("EnableHighDPIAwareness"), bEnableHighDPIAwareness, GEditorSettingsIni);
	}
	
	if (InPropertyName.IsNone() || InPropertyName == GET_MEMBER_NAME_CHECKED(UEditorStyleSettings, ApplicationScale))
	{
		if (FSlateApplication::IsInitialized())
		{
			FSlateApplication::Get().SetApplicationScale(ApplicationScale);
		}
	}
	
	SaveConfig();
}

bool UEditorStyleSettings::OnExportBegin(const FString& ExportToPath)
{
	SaveConfig(CPF_Config, *ExportToPath);
	if (!IPlatformFile::GetPlatformPhysical().FileExists(*ExportToPath))
	{
		return false;
	}

	const FStyleTheme& CurrentTheme = USlateThemeManager::Get().GetCurrentTheme();

	FString PathPart;
	FString Extension;
	FString FilenameWithoutExtension;
	FPaths::Split(ExportToPath, PathPart, FilenameWithoutExtension, Extension);

	const FString DestinationPath = PathPart / CurrentTheme.DisplayName.ToString() + TEXT(".json");

	// Export the theme file to the same destination, replacing any existing file at that path.
	return USlateThemeManager::Get().ExportCurrentThemeTo(DestinationPath);
}
