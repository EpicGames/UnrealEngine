// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings/EditorStyleSettingsCustomization.h"
#include "DetailCategoryBuilder.h"
#include "Styling/StyleColors.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/Input/STextComboBox.h"
#include "DetailWidgetRow.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Framework/Notifications/NotificationManager.h"
#include "SSimpleButton.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Misc/FileHelper.h"


#if ALLOW_THEMES
#include "IDesktopPlatform.h"
#include "DesktopPlatformModule.h"
#include "SPrimaryButton.h"
#include "HAL/FileManager.h"
#include "Misc/MessageDialog.h"
#include "Settings/EditorStyleSettings.h"

#define LOCTEXT_NAMESPACE "ThemeEditor"

TWeakPtr<SWindow> ThemeEditorWindow;
FString CurrentActiveThemeDisplayName;
FString OriginalThemeName;


class SThemeEditor : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SThemeEditor)
	{}
		SLATE_EVENT(FOnThemeEditorClosed, OnThemeEditorClosed);
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, TSharedRef<SWindow> InParentWindow)
	{
		OnThemeEditorClosed = InArgs._OnThemeEditorClosed;

		ParentWindow = InParentWindow;
		InParentWindow->SetOnWindowClosed(FOnWindowClosed::CreateSP(this, &SThemeEditor::OnParentWindowClosed));

		FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;

		TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

		DetailsView->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateLambda([](const FPropertyAndParent& PropertyAndParent)
			{
				static const FName CurrentThemeIdName("CurrentThemeId");

				return PropertyAndParent.Property.GetFName() != CurrentThemeIdName;
			})
		);

		DetailsView->SetObject(&USlateThemeManager::Get());
		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(6.0f, 3.0f)
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(0.6f)
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.Padding(5.0f, 2.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ThemeName", "Name"))
					]
					+ SHorizontalBox::Slot()
					.FillWidth(2.0f)
					.VAlign(VAlign_Center)
					.Padding(5.0f, 2.0f)
					[
						SAssignNew(EditableThemeName, SEditableTextBox)
						.Text(this, &SThemeEditor::GetThemeName)
						.OnTextChanged(this, &SThemeEditor::OnThemeNameChanged)
						.OnTextCommitted(this, &SThemeEditor::OnThemeNameCommitted)
						.SelectAllTextWhenFocused(true)
						//.IsReadOnly(true)
					]
				]
				/*+ SVerticalBox::Slot()
				.Padding(6.0f, 3.0f)
				.AutoHeight()
				[

					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(0.6f)
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.Padding(5.0f, 2.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ThemeDescription", "Description"))
					]
					+ SHorizontalBox::Slot()
					.FillWidth(2.0f)
					.VAlign(VAlign_Center)
					.Padding(5.0f, 2.0f)
					[
						SNew(SEditableTextBox)
						.Text(FText::FromString("Test Theme Description"))
						.IsReadOnly(true)
					]
				]*/
				+ SVerticalBox::Slot()
				.Padding(6.0f, 3.0f)
				[
					DetailsView
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Bottom)
				.Padding(6.0f, 3.0f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Bottom)
					.Padding(4, 3)
					[
						SNew(SPrimaryButton)
						.Text(LOCTEXT("SaveThemeButton", "Save"))
						.OnClicked(this, &SThemeEditor::OnSaveClicked)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Bottom)
					.Padding(4, 3)
					[
						SNew(SButton)
						.Text(LOCTEXT("CancelThemeEditingButton", "Cancel"))
						.OnClicked(this, &SThemeEditor::OnCancelClicked)
					]
				]
			]
		];

	}

private:
	FText GetThemeName() const
	{
		return USlateThemeManager::Get().GetCurrentTheme().DisplayName;
	}

	// Validate new theme name against existing theme names to avoid duplicate. 
	bool ValidateThemeName(const FText& ThemeName)
	{
		FText OutErrorMessage;
		const TArray<FStyleTheme> ThemeOptions = USlateThemeManager::Get().GetThemes();
		
		if (ThemeName.IsEmpty())
		{
			OutErrorMessage = LOCTEXT("ThemeNameEmpty", "The theme name cannot be empty.");
			EditableThemeName->SetError(OutErrorMessage);
			return false;
		}

		for (const FStyleTheme& Theme : ThemeOptions)
		{
			// show error message whenever there's duplicate (and different from the previous name) 
			if (Theme.DisplayName.EqualTo(ThemeName) && !CurrentActiveThemeDisplayName.Equals(ThemeName.ToString()))
			{
				OutErrorMessage = FText::Format(LOCTEXT("RenameThemeAlreadyExists", "A theme already exists with the name '{0}'."), ThemeName);
				EditableThemeName->SetError(OutErrorMessage);
				return false;
			}
		}
		EditableThemeName->SetError(FText::GetEmpty());
		return true; 
	}

	void OnThemeNameChanged(const FText& NewName)
	{
		// verify duplicates before setting the display name. 
		ValidateThemeName(NewName);
	}

	void OnThemeNameCommitted(const FText& NewName, ETextCommit::Type = ETextCommit::Default)
	{
		if (!ValidateThemeName(NewName))
		{
			const FText OriginalTheme = FText::FromString(OriginalThemeName); 
			EditableThemeName->SetText(OriginalTheme);
			EditableThemeName->SetError(FText::GetEmpty());
		}
		else
		{
			EditableThemeName->SetText(NewName);
		}
	}

	FReply OnSaveClicked()
	{
		FString Filename;
		bool bSuccess = true; 
		FString PreviousFilename; 

		const FStyleTheme& Theme = USlateThemeManager::Get().GetCurrentTheme();

		// updated name is taken: DO NOT SAVE. 
		if (!ValidateThemeName(EditableThemeName->GetText()))
		{
			bSuccess = false;
		}
		// Duplicating a theme: 
		else if (Theme.Filename.IsEmpty())
		{
			// updated name is not taken: SAVE. 
			USlateThemeManager::Get().SetCurrentThemeDisplayName(EditableThemeName->GetText());
			Filename = USlateThemeManager::Get().GetUserThemeDir() / Theme.DisplayName.ToString() + TEXT(".json");
			EditableThemeName->SetError(FText::GetEmpty()); 
		}
		// Modifying a theme: would only be here if the user is modifying a user-specific theme. 
		else
		{
			// updated name is not taken: SAVE. 
			PreviousFilename = Theme.Filename; 
			USlateThemeManager::Get().SetCurrentThemeDisplayName(EditableThemeName->GetText());
			Filename = USlateThemeManager::Get().GetUserThemeDir() / Theme.DisplayName.ToString() + TEXT(".json");
			EditableThemeName->SetError(FText::GetEmpty());
		}

		if (!Filename.IsEmpty() && bSuccess)
		{
			USlateThemeManager::Get().SaveCurrentThemeAs(Filename); 
			// if user modified an existing user-specific theme name, delete the old one. 
			if (!PreviousFilename.IsEmpty() && !PreviousFilename.Equals(Filename))
			{
				IPlatformFile::GetPlatformPhysical().DeleteFile(*PreviousFilename);
			}
			EditableThemeName->SetError(FText::GetEmpty()); 

			ParentWindow.Pin()->SetOnWindowClosed(FOnWindowClosed());
			ParentWindow.Pin()->RequestDestroyWindow();

			OnThemeEditorClosed.ExecuteIfBound(true);
		}
		return FReply::Handled();
	}

	FReply OnCancelClicked()
	{
		ParentWindow.Pin()->SetOnWindowClosed(FOnWindowClosed());
		ParentWindow.Pin()->RequestDestroyWindow();

		OnThemeEditorClosed.ExecuteIfBound(false);
		return FReply::Handled();
	}

	void OnParentWindowClosed(const TSharedRef<SWindow>&)
	{
		OnCancelClicked();
	}

private:
	FOnThemeEditorClosed OnThemeEditorClosed;
	TSharedPtr<SEditableTextBox> EditableThemeName; 
	TWeakPtr<SWindow> ParentWindow;
};

#undef LOCTEXT_NAMESPACE

#define LOCTEXT_NAMESPACE "EditorStyleSettingsCustomization"

TSharedRef<IPropertyTypeCustomization> FStyleColorListCustomization::MakeInstance()
{
	return MakeShared<FStyleColorListCustomization>();
}

void FStyleColorListCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{

}

void FStyleColorListCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	uint32 NumChildren = 0;
	TSharedPtr<IPropertyHandle> ColorArrayProperty = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStyleColorList, StyleColors));

	ColorArrayProperty->GetNumChildren(NumChildren);
	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		FResetToDefaultOverride ResetToDefaultOverride =
			FResetToDefaultOverride::Create(
				FIsResetToDefaultVisible::CreateSP(this, &FStyleColorListCustomization::IsResetToDefaultVisible, (EStyleColor)ChildIndex),
				FResetToDefaultHandler::CreateSP(this, &FStyleColorListCustomization::OnResetColorToDefault, (EStyleColor)ChildIndex));

		if (ChildIndex < (uint32)EStyleColor::User1)
		{
			IDetailPropertyRow& Row = ChildBuilder.AddProperty(ColorArrayProperty->GetChildHandle(ChildIndex).ToSharedRef());
			Row.OverrideResetToDefault(ResetToDefaultOverride);
		}
		else
		{
			// user colors are added if they have been customized with a display name
			FText DisplayName = USlateThemeManager::Get().GetColorDisplayName((EStyleColor)ChildIndex);
			if (!DisplayName.IsEmpty())
			{
				IDetailPropertyRow& Row = ChildBuilder.AddProperty(ColorArrayProperty->GetChildHandle(ChildIndex).ToSharedRef());
				Row.DisplayName(DisplayName);
				Row.OverrideResetToDefault(ResetToDefaultOverride);
			}
		}
	}
}


void FStyleColorListCustomization::OnResetColorToDefault(TSharedPtr<IPropertyHandle> Handle, EStyleColor Color)
{
	FLinearColor CurrentColor = USlateThemeManager::Get().GetColor(Color);
	const FStyleTheme& Theme = USlateThemeManager::Get().GetCurrentTheme();
	if (Theme.LoadedDefaultColors.Num())
	{
		USlateThemeManager::Get().ResetActiveColorToDefault(Color);
	}
}

bool FStyleColorListCustomization::IsResetToDefaultVisible(TSharedPtr<IPropertyHandle> Handle, EStyleColor Color)
{
	FLinearColor CurrentColor = USlateThemeManager::Get().GetColor(Color);
	const FStyleTheme& Theme = USlateThemeManager::Get().GetCurrentTheme();
	if (Theme.LoadedDefaultColors.Num())
	{
		return Theme.LoadedDefaultColors[(int32)Color] != CurrentColor;
	}

	return false;
}

TSharedRef<IDetailCustomization> FEditorStyleSettingsCustomization::MakeInstance()
{
	TSharedRef<FEditorStyleSettingsCustomization> Instance = MakeShared<FEditorStyleSettingsCustomization>();
	// unable to perform this operation in FEditorStyleSettingsCustomization's constructor since by then the shared ref 
	// controller has not been created yet
	USlateThemeManager::Get().OnThemeChanged().AddSP(Instance, &FEditorStyleSettingsCustomization::OnThemeChanged);

	return Instance;
}

FEditorStyleSettingsCustomization::~FEditorStyleSettingsCustomization()
{
	USlateThemeManager& ThemeManager = USlateThemeManager::Get();
	ThemeManager.OnThemeChanged().RemoveAll(this);
}

void FEditorStyleSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	IDetailCategoryBuilder& ColorCategory = DetailLayout.EditCategory("Theme");

	TArray<UObject*> Objects = { &USlateThemeManager::Get() };

	if (IDetailPropertyRow* ThemeRow = ColorCategory.AddExternalObjectProperty(Objects, "CurrentThemeId"))
	{
		MakeThemePickerRow(*ThemeRow);
	}
}

void FEditorStyleSettingsCustomization::RefreshComboBox()
{
	TSharedPtr<FString> SelectedTheme;
	GenerateThemeOptions(SelectedTheme);
	ComboBox->RefreshOptions();
	ComboBox->SetSelectedItem(SelectedTheme);
}

void FEditorStyleSettingsCustomization::GenerateThemeOptions(TSharedPtr<FString>& OutSelectedTheme)
{
	const TArray<FStyleTheme>& Themes = USlateThemeManager::Get().GetThemes();

	ThemeOptions.Empty(Themes.Num());
	int32 Index = 0;
	for (const FStyleTheme& Theme : Themes)
	{
		TSharedRef<FString> ThemeString = MakeShared<FString>(FString::FromInt(Index));

		if (USlateThemeManager::Get().GetCurrentTheme() == Theme)
		{
			OutSelectedTheme = ThemeString;
		}

		ThemeOptions.Add(ThemeString);
		++Index;
	}

}

void FEditorStyleSettingsCustomization::MakeThemePickerRow(IDetailPropertyRow& PropertyRow)
{

	TSharedPtr<FString> SelectedItem;
	GenerateThemeOptions(SelectedItem);

	// Make combo choices
	ComboBox =
		SNew(STextComboBox)
		.OptionsSource(&ThemeOptions)
		.InitiallySelectedItem(SelectedItem)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.OnGetTextLabelForItem(this, &FEditorStyleSettingsCustomization::GetTextLabelForThemeEntry)
		.OnSelectionChanged(this, &FEditorStyleSettingsCustomization::OnThemePicked);


	FDetailWidgetRow& CustomWidgetRow = PropertyRow.CustomWidget(false);

	CustomWidgetRow
	.NameContent()
	[
		PropertyRow.GetPropertyHandle()->CreatePropertyNameWidget(LOCTEXT("ActiveThemeDisplayName", "Active Theme"))
	]
	.ValueContent()
	.MaxDesiredWidth(350.f)
	[
		SNew(SHorizontalBox)
		.IsEnabled(this, &FEditorStyleSettingsCustomization::IsThemeEditingEnabled)
		+SHorizontalBox::Slot()
		[
			SNew(SBox)
			.WidthOverride(125.f)
			[
				ComboBox.ToSharedRef()
			]
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		.AutoWidth()
		[
			SNew(SSimpleButton)
			.Icon(FAppStyle::Get().GetBrush("Icons.Edit"))
			.IsEnabled_Lambda(
				[]()
				{
					return !(USlateThemeManager::Get().IsEngineTheme() || USlateThemeManager::Get().IsProjectTheme());
				})
			.ToolTipText_Lambda(
				[]()
				{	
					if (USlateThemeManager::Get().IsEngineTheme())
					{
						return LOCTEXT("CannotEditEngineThemeToolTip", "Engine themes can't be edited");
					}
					else if (USlateThemeManager::Get().IsProjectTheme())
					{
						return LOCTEXT("CannotEditProjectThemeToolTip", "Project themes can't be edited");
					}
					return LOCTEXT("EditThemeToolTip", "Edit this theme");
				 })
			.OnClicked(this, &FEditorStyleSettingsCustomization::OnEditThemeClicked)
		]
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		.AutoWidth()
		[
			SNew(SSimpleButton)
			.Icon(FAppStyle::Get().GetBrush("Icons.Duplicate"))
			.ToolTipText(LOCTEXT("DuplicateThemeToolTip", "Duplicate this theme and edit it"))
			.OnClicked(this, &FEditorStyleSettingsCustomization::OnDuplicateAndEditThemeClicked)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(8.0f, 0.0f, 0.0f, 0.0f)
		[
			// export button
			SNew(SSimpleButton)
			.Icon(FAppStyle::Get().GetBrush("Themes.Export"))
			.ToolTipText(LOCTEXT("ExportButtonTooltip", "Export this theme to a file on your computer"))
			.OnClicked(this, &FEditorStyleSettingsCustomization::OnExportThemeClicked)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(8.0f, 0.0f, 0.0f, 0.0f)
		[
			// import button
			SNew(SSimpleButton)
			.Icon(FAppStyle::Get().GetBrush("Themes.Import"))
			.ToolTipText(LOCTEXT("ImportButtonTooltip", "Import a theme from a file on your computer"))
			.OnClicked(this, &FEditorStyleSettingsCustomization::OnImportThemeClicked)
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		.AutoWidth()
		[
			// delete button
			SNew(SSimpleButton)
			.Icon(FAppStyle::Get().GetBrush("Icons.Delete"))
			.IsEnabled_Lambda(
				[]()
				{
					return !(USlateThemeManager::Get().IsEngineTheme() || USlateThemeManager::Get().IsProjectTheme()); 
				})
			.ToolTipText_Lambda(
				[]()
				{
					if (USlateThemeManager::Get().IsEngineTheme())
					{
						return LOCTEXT("CannotDeleteEngineThemeToolTip", "Engine themes can't be deleted");
					}
					else if (USlateThemeManager::Get().IsProjectTheme())
					{
						return LOCTEXT("CannotDeleteProjectThemeToolTip", "Project themes can't be deleted");
					}
					return LOCTEXT("DeleteThemeToolTip", "Delete this theme");
				})
			.OnClicked(this, &FEditorStyleSettingsCustomization::OnDeleteThemeClicked)
		]
	];
}

static void OnThemeEditorClosed(bool bSaved, TWeakPtr<FEditorStyleSettingsCustomization> ActiveCustomization, FGuid CreatedThemeId, FGuid PreviousThemeId)
{
	if (!bSaved)
	{
		if (PreviousThemeId.IsValid())
		{
	
			USlateThemeManager::Get().ApplyTheme(PreviousThemeId);

			if (CreatedThemeId.IsValid())
			{
				USlateThemeManager::Get().RemoveTheme(CreatedThemeId);

			}
			if (ActiveCustomization.IsValid())
			{

				ActiveCustomization.Pin()->RefreshComboBox();
			}
		}
		else
		{
			USlateThemeManager::Get().ResetColorsToDefault();
		}
	}
}

FReply FEditorStyleSettingsCustomization::OnExportThemeClicked()
{
	TArray<FString> OutFiles; 
	const void* ParentWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);
	const FString ExportPath = FPlatformProcess::UserDir();
	const FString DefaultFileName = USlateThemeManager::Get().GetCurrentTheme().DisplayName.ToString();

	if (FDesktopPlatformModule::Get()->SaveFileDialog(ParentWindowHandle, LOCTEXT("ExportThemeDialogTitle", "Export current theme...").ToString(), FPaths::GetPath(ExportPath), DefaultFileName, TEXT("JSON files (*.json)|*.json"), EFileDialogFlags::None, OutFiles))
	{
		const FString DestPath = OutFiles[0];

		if (USlateThemeManager::Get().ExportCurrentThemeTo(DestPath))
		{
			ShowNotification(LOCTEXT("ExportThemeSuccess", "Export theme succeeded"), SNotificationItem::CS_Success);
		}
		else
		{
			ShowNotification(LOCTEXT("ExportThemeFailure", "Export theme failed"), SNotificationItem::CS_Fail);
		}
	}
	return FReply::Handled();
}

// validate theme name without error messages: 
bool IsThemeNameValid(const FString& ThemeName)
{
	const TArray<FStyleTheme> ThemeOptions = USlateThemeManager::Get().GetThemes();

	for (const FStyleTheme& Theme : ThemeOptions)
	{
		// show error message whenever there's duplicate (and different from the previous name) 
		if (Theme.DisplayName.ToString().Equals(ThemeName))
		{
			return false;
		}
	}
	return true;
}


void GetThemeIdFromPath(FString& ThemePath, FString& ImportedThemeID)
{
	FString ThemeData;

	if (FFileHelper::LoadFileToString(ThemeData, *ThemePath))
	{
		TSharedRef<TJsonReader<>> ReaderRef = TJsonReaderFactory<>::Create(ThemeData);
		TJsonReader<>& Reader = ReaderRef.Get();

		TSharedPtr<FJsonObject> ObjectPtr = MakeShareable(new FJsonObject()); 

		if (FJsonSerializer::Deserialize(Reader, ObjectPtr) && ObjectPtr.IsValid())
		{
			// Just check that the theme has Id. We won't load them unless the theme is used
			ObjectPtr->TryGetStringField(TEXT("Id"), ImportedThemeID); 
		}
	}
}

FReply FEditorStyleSettingsCustomization::OnImportThemeClicked()
{
	PromptToImportTheme(FPlatformProcess::UserDir());
	return FReply::Handled();
}

FReply FEditorStyleSettingsCustomization::OnDeleteThemeClicked()
{
	const FStyleTheme PreviouslyActiveTheme = USlateThemeManager::Get().GetCurrentTheme();

	// Are you sure you want to do this?
	const FText FileNameToRemove = FText::FromString(PreviouslyActiveTheme.DisplayName.ToString());
	const FText TextBody = FText::Format(LOCTEXT("ActionRemoveMsg", "Are you sure you want to permanently delete the theme \"{0}\"? This action cannot be undone."), FileNameToRemove);
	const FText TextTitle = FText::Format(LOCTEXT("RemoveTheme_Title", "Remove Theme \"{0}\"?"), FileNameToRemove);

	// If user select "OK"...
	if (EAppReturnType::Ok == FMessageDialog::Open(EAppMsgType::OkCancel, TextBody, TextTitle))
	{
		// apply default theme
		USlateThemeManager::Get().ApplyDefaultTheme();

		// remove previously active theme
		const FString Filename = USlateThemeManager::Get().GetUserThemeDir() / PreviouslyActiveTheme.DisplayName.ToString() + TEXT(".json");
		IFileManager::Get().Delete(*Filename);
		USlateThemeManager::Get().RemoveTheme(PreviouslyActiveTheme.Id);
		RefreshComboBox();
	}
	// Else, do nothing. 
	return FReply::Handled();
}

FReply FEditorStyleSettingsCustomization::OnDuplicateAndEditThemeClicked()
{
	FGuid PreviouslyActiveTheme = USlateThemeManager::Get().GetCurrentTheme().Id;

	FGuid NewThemeId = USlateThemeManager::Get().DuplicateActiveTheme();
	USlateThemeManager::Get().ApplyTheme(NewThemeId);
	// Set the new theme name to empty FText, to avoid a generated name collision or needing to delete a template name
	USlateThemeManager::Get().SetCurrentThemeDisplayName(FText::GetEmpty());
	CurrentActiveThemeDisplayName = USlateThemeManager::Get().GetCurrentTheme().DisplayName.ToString();
	OriginalThemeName = USlateThemeManager::Get().GetCurrentTheme().DisplayName.ToString(); 

	RefreshComboBox();

	OpenThemeEditorWindow(FOnThemeEditorClosed::CreateStatic(&OnThemeEditorClosed, TWeakPtr<FEditorStyleSettingsCustomization>(SharedThis(this)), NewThemeId, PreviouslyActiveTheme));

	return FReply::Handled();
}

FReply FEditorStyleSettingsCustomization::OnEditThemeClicked()
{
	FGuid CurrentlyActiveTheme = USlateThemeManager::Get().GetCurrentTheme().Id;

	CurrentActiveThemeDisplayName = USlateThemeManager::Get().GetCurrentTheme().DisplayName.ToString();
	OriginalThemeName = CurrentActiveThemeDisplayName; 
	
	// There is no new theme created, so just pass in the current active theme ID
	OpenThemeEditorWindow(FOnThemeEditorClosed::CreateStatic(&OnThemeEditorClosed, TWeakPtr<FEditorStyleSettingsCustomization>(SharedThis(this)), FGuid(), CurrentlyActiveTheme));

	return FReply::Handled();
}

FString FEditorStyleSettingsCustomization::GetTextLabelForThemeEntry(TSharedPtr<FString> Entry)
{
	const TArray<FStyleTheme>& Themes = USlateThemeManager::Get().GetThemes();
	return Themes[TCString<TCHAR>::Atoi(**Entry)].DisplayName.ToString();
}

void FEditorStyleSettingsCustomization::OnThemePicked(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	UEditorStyleSettings* StyleSetting = GetMutableDefault<UEditorStyleSettings>();

	// set current applied theme to selected theme. 
	const TArray<FStyleTheme>& Themes = USlateThemeManager::Get().GetThemes();
	StyleSetting->CurrentAppliedTheme = Themes[TCString<TCHAR>::Atoi(**NewSelection)].Id;
	
	// If set directly in code, the theme was already applied
	if(SelectInfo != ESelectInfo::Direct)
	{
		StyleSetting->SaveConfig();
		USlateThemeManager::Get().ApplyTheme(StyleSetting->CurrentAppliedTheme);
	}
}

void FEditorStyleSettingsCustomization::OpenThemeEditorWindow(FOnThemeEditorClosed OnThemeEditorClosed)
{
	if(!ThemeEditorWindow.IsValid())
	{
		TSharedRef<SWindow> NewWindow = SNew(SWindow)
			.Title(LOCTEXT("ThemeEditorWindowTitle", "Theme Editor"))
			.SizingRule(ESizingRule::UserSized)
			.ClientSize(FVector2D(600, 600))
			.SupportsMaximize(false)
			.SupportsMinimize(false);

		TSharedRef<SThemeEditor> ThemeEditor =
			SNew(SThemeEditor, NewWindow)
			.OnThemeEditorClosed(OnThemeEditorClosed);
			
		NewWindow->SetContent(
			ThemeEditor
		);

		if (TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(ComboBox.ToSharedRef()))
		{
			FSlateApplication::Get().AddWindowAsNativeChild(NewWindow, ParentWindow.ToSharedRef());
		}
		else
		{
			FSlateApplication::Get().AddWindow(NewWindow);
		}

		ThemeEditorWindow = NewWindow;
	}


}

bool FEditorStyleSettingsCustomization::IsThemeEditingEnabled() const
{
	// Don't allow changing themes while editing them
	return !ThemeEditorWindow.IsValid();
}

void FEditorStyleSettingsCustomization::ShowNotification(const FText& Text, SNotificationItem::ECompletionState CompletionState)
{
	FNotificationInfo Notification(Text);
	Notification.ExpireDuration = 3.f;
	Notification.bUseSuccessFailIcons = false;

	FSlateNotificationManager::Get().AddNotification(Notification)->SetCompletionState(CompletionState);
}

void FEditorStyleSettingsCustomization::PromptToImportTheme(const FString& ImportPath)
{
	TArray<FString> OutFiles;
	const void* ParentWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);

	if (FDesktopPlatformModule::Get()->OpenFileDialog(ParentWindowHandle, LOCTEXT("ImportThemeDialogTitle", "Import theme...").ToString(), FPaths::GetPath(ImportPath), TEXT(""), TEXT("JSON files (*.json)|*.json"), EFileDialogFlags::None, OutFiles))
	{
		FString SourcePath = OutFiles[0];
		const FString DestPath = USlateThemeManager::Get().GetUserThemeDir() / FPaths::GetCleanFilename(SourcePath);

		FString PathPart;
		FString Extension;
		FString FilenameWithoutExtension;
		FPaths::Split(SourcePath, PathPart, FilenameWithoutExtension, Extension);

		// if theme name exists, don't import (to prevent from overwriting existing theme files)
		if (!IsThemeNameValid(FilenameWithoutExtension))
		{
			ShowNotification(LOCTEXT("ImportThemeFailureNameExists", "Import theme failed: Theme name already exists"), SNotificationItem::CS_Fail);
		}
		// if theme name is valid: copying the file is safe (as it will not overwrite existing theme files)
		else
		{
			const int32 NumOfThemesBefore = USlateThemeManager::Get().GetThemes().Num();

			if (IPlatformFile::GetPlatformPhysical().CopyFile(*DestPath, *SourcePath))
			{
				// update the number of valid themes: 
				USlateThemeManager::Get().LoadThemes();

				// if valid theme: the theme Num will be updated. 
				if (USlateThemeManager::Get().GetThemes().Num() != NumOfThemesBefore)
				{
					// Extract ID as a FString directly from a JSON file. 
					FString ImportedThemeID;
					GetThemeIdFromPath(SourcePath, ImportedThemeID);

					// convert FString ID to a FGuid
					FGuid ImportedThemeGUID = FGuid(ImportedThemeID);
					USlateThemeManager::Get().ApplyTheme(ImportedThemeGUID);

					ShowNotification(LOCTEXT("ImportThemeSuccess", "Import theme succeeded"), SNotificationItem::CS_Success);
				}
				// if invalid theme: delete the copied file. 
				else
				{
					// incomplete themes will not reach here. 
					IPlatformFile::GetPlatformPhysical().DeleteFile(*DestPath);
					ShowNotification(LOCTEXT("ImportThemeFailureInvalidName", "Import theme failed: Invalid theme"), SNotificationItem::CS_Fail);
				}
			}
			// if unable to copy the file to user-specific theme location, do nothing. 
			else
			{
				ShowNotification(LOCTEXT("ImportThemeFailure", "Import theme failed"), SNotificationItem::CS_Fail);
			}
		}
	}
}


#undef LOCTEXT_NAMESPACE

#endif // ALLOW_THEMES
