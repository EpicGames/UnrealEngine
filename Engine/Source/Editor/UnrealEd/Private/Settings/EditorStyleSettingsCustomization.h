// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "Styling/StyleColors.h"
#include "Widgets/Notifications/SNotificationList.h"

class IDetailLayoutBuilder;
class STextComboBox;
class IDetailPropertyRow;

#if ALLOW_THEMES

DECLARE_DELEGATE_OneParam(FOnThemeEditorClosed, bool)

class FStyleColorListCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
private:
	void OnResetColorToDefault(TSharedPtr<IPropertyHandle> Handle, EStyleColor Color);
	bool IsResetToDefaultVisible(TSharedPtr<IPropertyHandle> Handle, EStyleColor Color);
};

class FEditorStyleSettingsCustomization : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	virtual ~FEditorStyleSettingsCustomization(); 

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	
	void RefreshComboBox();

	/** Import theme from OS */
	static void PromptToImportTheme(const FString& ImportPath);

private:
	void GenerateThemeOptions(TSharedPtr<FString>& OutSelectedTheme);

	void MakeThemePickerRow(IDetailPropertyRow& PropertyRow);
	FReply OnExportThemeClicked(); 
	FReply OnImportThemeClicked(); 
	FReply OnDeleteThemeClicked();
	FReply OnDuplicateAndEditThemeClicked();
	FReply OnEditThemeClicked();
	FString GetTextLabelForThemeEntry(TSharedPtr<FString> Entry);
	void OnThemePicked(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);
	void OpenThemeEditorWindow(FOnThemeEditorClosed OnThemeEditorClosed);
	bool IsThemeEditingEnabled() const;

	/** Delegate for when theme gets changed by another class */
	void OnThemeChanged(const FGuid ThemeId) { RefreshComboBox(); }

	/** Show import status (success/failure) when users import a theme */
	static void ShowNotification(const FText& Text, SNotificationItem::ECompletionState CompletionState);

private:
	TArray<TSharedPtr<FString>> ThemeOptions;
	TSharedPtr<STextComboBox> ComboBox;
};
#endif
