// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings/EditorViewportSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorViewportSettings)

namespace UE::Editor::Private
{
	constexpr float SMALLEST_SPEED = 0.0001f;
}

FEditorViewportCameraSpeedSettings::FEditorViewportCameraSpeedSettings(float Speed)
{
	if (Speed < UE::Editor::Private::SMALLEST_SPEED)
	{
		return;
	}
	
	if (Speed < AbsoluteMinSpeed)
	{
		AbsoluteMinSpeed = Speed;
	}
	if (Speed > AbsoluteMaxSpeed)
	{
		AbsoluteMaxSpeed = Speed;
	}
	
	CurrentSpeed = Speed;
}

FEditorViewportCameraSpeedSettings::FEditorViewportCameraSpeedSettings(float Speed, float Min, float Max)
{
	SetAbsoluteSpeedRange(Min, Max);
	SetCurrentSpeed(Speed);
}

FEditorViewportCameraSpeedSettings FEditorViewportCameraSpeedSettings::FromUIRange(float Speed, float UIMin, float UIMax)
{
	FEditorViewportCameraSpeedSettings NewSettings(Speed);
	
	NewSettings.SetAbsoluteSpeedRange(
		FMath::Min(NewSettings.AbsoluteMinSpeed, UIMin),
		FMath::Max(NewSettings.AbsoluteMaxSpeed, UIMax)
	);
	
	NewSettings.SetUISpeedRange(UIMin, UIMax);
	
	return NewSettings;
}

float FEditorViewportCameraSpeedSettings::GetCurrentSpeed() const
{
	return CurrentSpeed;
}

void FEditorViewportCameraSpeedSettings::SetCurrentSpeed(float NewSpeed)
{
	CurrentSpeed = FMath::Clamp(NewSpeed, AbsoluteMinSpeed, AbsoluteMaxSpeed);
}

float FEditorViewportCameraSpeedSettings::GetAbsoluteMinSpeed() const
{
	return AbsoluteMinSpeed;
}

float FEditorViewportCameraSpeedSettings::GetMinUISpeed() const
{
	return MinUISpeed;
}

float FEditorViewportCameraSpeedSettings::GetAbsoluteMaxSpeed() const
{
	return AbsoluteMaxSpeed;
}

float FEditorViewportCameraSpeedSettings::GetMaxUISpeed() const
{
	return MaxUISpeed;
}

void FEditorViewportCameraSpeedSettings::SetAbsoluteSpeedRange(float NewMinSpeed, float NewMaxSpeed)
{
	AbsoluteMinSpeed = FMath::Max(NewMinSpeed, UE::Editor::Private::SMALLEST_SPEED);
	AbsoluteMaxSpeed = FMath::Max(NewMaxSpeed, AbsoluteMinSpeed);
	
	if (NewMinSpeed == NewMaxSpeed)
	{
		AbsoluteMaxSpeed = NewMinSpeed + 1.0f;
	}
	
	SetUISpeedRange(MinUISpeed, MaxUISpeed);
	SetCurrentSpeed(CurrentSpeed);
}

void FEditorViewportCameraSpeedSettings::SetUISpeedRange(float NewMinSpeed, float NewMaxSpeed)
{
	if (NewMinSpeed > NewMaxSpeed)
	{
		const float Swap = NewMaxSpeed;
		NewMaxSpeed = NewMinSpeed;
		NewMinSpeed = Swap; 
	}

	MinUISpeed = FMath::Max(AbsoluteMinSpeed, FMath::Min(NewMinSpeed, AbsoluteMaxSpeed));
	MaxUISpeed = FMath::Min(AbsoluteMaxSpeed, FMath::Max(NewMaxSpeed, AbsoluteMinSpeed));
	
	if (MinUISpeed == MaxUISpeed)
	{
		if (MinUISpeed >= AbsoluteMaxSpeed - 1.0f)
		{
			MinUISpeed = FMath::Max(AbsoluteMinSpeed, AbsoluteMaxSpeed - 1.0f);
		}
		else
		{
			MaxUISpeed = FMath::Min(AbsoluteMaxSpeed, MinUISpeed + 1.0f);
		}
	}
}

float FEditorViewportCameraSpeedSettings::GetRelativeSpeed() const
{
	return (CurrentSpeed - AbsoluteMinSpeed) / (AbsoluteMaxSpeed - AbsoluteMinSpeed);
}

void FEditorViewportCameraSpeedSettings::SetRelativeSpeed(float RelativeSpeed)
{
	CurrentSpeed = FMath::Lerp(AbsoluteMinSpeed, AbsoluteMaxSpeed, RelativeSpeed);
}

float FEditorViewportCameraSpeedSettings::GetRelativeUISpeed() const
{
	return (CurrentSpeed - MinUISpeed) / (MaxUISpeed - MinUISpeed);
}

void FEditorViewportCameraSpeedSettings::SetRelativeUISpeed(float RelativeUISpeed)
{
	CurrentSpeed = FMath::Lerp(MinUISpeed, MaxUISpeed, RelativeUISpeed);
}

float FEditorViewportCameraSpeedSettings::GetLowSpeedWarningThreshold() const
{
	return LowSpeedWarningThreshold;
}
