// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/PackageName.h"
#include "Layout/Margin.h"
#include "Styling/SlateColor.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "Textures/SlateIcon.h"
#include "Styling/AppStyle.h"
#include "Framework/Commands/UIAction.h"
#include "IDetailCustomization.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SComboButton.h"
#include "GameModeInfoCustomizer.h"

#define LOCTEXT_NAMESPACE "FLevelEditorPlaySettingsCustomization"

/**
 * Implements a details view customization for UGameMapsSettingsCustomization objects.
 */
class FGameMapsSettingsCustomization
	: public IDetailCustomization
{
public:

	/** Virtual destructor. */
	virtual ~FGameMapsSettingsCustomization( ) = default;

public:

	// IDetailCustomization interface

	virtual void CustomizeDetails( IDetailLayoutBuilder& LayoutBuilder ) override
	{
		// Add extra info around 'Global Default Game Mode'
		IDetailCategoryBuilder& DefaultModesCategory = LayoutBuilder.EditCategory("DefaultModes");
		{
			// Get the object that we are viewing details of. Expect to only edit one GameMapSettings object at a time!
			TArray< TWeakObjectPtr<UObject> > ObjectsCustomized;
			LayoutBuilder.GetObjectsBeingCustomized(ObjectsCustomized);
			UObject* ObjectCustomized = (ObjectsCustomized.Num() > 0) ? ObjectsCustomized[0].Get() : NULL;
			// Get name of GameMode property
			static FName GlobalDefaultGameModeName(TEXT("GlobalDefaultGameMode"));

			// Allocate customizer object
			GameInfoModeCustomizer = MakeShareable(new FGameModeInfoCustomizer(ObjectCustomized, GlobalDefaultGameModeName));

			// Then use it to customize
			GameInfoModeCustomizer->CustomizeGameModeSetting(LayoutBuilder, DefaultModesCategory);
		}
	}

public:

	/**
	 * Creates a new instance.
	 *
	 * @return A new struct customization for game maps settings.
	 */
	static TSharedRef<IDetailCustomization> MakeInstance( )
	{
		return MakeShareable(new FGameMapsSettingsCustomization());
	}

protected:

	/**
	 * Customizes the property row for a map setting.
	 *
	 * @param LayoutBuilder The layout builder.
	 * @param CategoryBuilder The builder for the detail category that the setting belongs to.
	 * @param PropertyName The name of the property that holds the map setting.
	 */
	void CustomizeMapSetting( IDetailLayoutBuilder& LayoutBuilder, IDetailCategoryBuilder& CategoryBuilder, const FName& PropertyName )
	{
		TSharedPtr<IPropertyHandle> PropertyHandle = LayoutBuilder.GetProperty(PropertyName);
		IDetailPropertyRow& PropertyRow = CategoryBuilder.AddProperty(PropertyHandle);
		
		PropertyRow.CustomWidget()
			.NameContent()
			[
				PropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.MaxDesiredWidth(0)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						SNew(SEditableTextBox)
							.ForegroundColor(this, &FGameMapsSettingsCustomization::HandleMapTextBoxForegroundColor, PropertyHandle)
							.OnTextChanged(this, &FGameMapsSettingsCustomization::HandleMapTextBoxTextChanged, PropertyHandle)
							.OnTextCommitted(this, &FGameMapsSettingsCustomization::HandleMapTextBoxTextCommitted, PropertyHandle)
							.Text(this, &FGameMapsSettingsCustomization::HandleMapTextBoxText, PropertyHandle)
							.ToolTipText(PropertyHandle->GetToolTipText())
					]

				+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2.0f, 0.0f, 0.0f, 0.0f)
					[
						SNew(SComboButton)
							.ButtonContent()
							[
								SNullWidget::NullWidget
							]
							.ContentPadding(FMargin(6.0f, 1.0f))
							.MenuContent()
							[
								MakeMapMenu(PropertyHandle)
							]
							.ToolTipText(LOCTEXT("AvailableMapsButtonTooltip", "Pick from the list of available maps"))
					]
			];
	}

	/**
	 * Checks whether the specified map name is valid.
	 *
	 * @param MapName The map name to validate.
	 * @return true if the map name is valid, false otherwise.
	 */
	bool IsValidMapName( const FString& MapName ) const
	{
		if (!FPackageName::IsValidLongPackageName(MapName, true))
		{
			return false;
		}

		return (FPaths::FileExists(FPackageName::LongPackageNameToFilename(MapName, FPackageName::GetMapPackageExtension())));
	}

	/**
	 * Creates a widget for the map picker.
	 *
	 * @param PropertyRow The property row to create the widget for.
	 * @return The widget.
	 */
	TSharedRef<SWidget> MakeMapMenu( const TSharedPtr<IPropertyHandle>& PropertyHandle )
	{
		FMenuBuilder MenuBuilder(true, NULL);

		const FString MapFileWildCard = FString::Printf(TEXT("*%s"), *FPackageName::GetMapPackageExtension());
		TArray<FString> MapNames;

		// maps that belong to the project
		MenuBuilder.BeginSection(NAME_None, LOCTEXT("ProjectMapsSectionHeader", "Project"));
		{
			IFileManager::Get().FindFilesRecursive(MapNames, *FPaths::ProjectContentDir(), *MapFileWildCard, true, false);
			MapNames.Sort([](const FString& A, const FString& B) { return FPaths::GetBaseFilename(A) < FPaths::GetBaseFilename(B); });

			for (int32 i = 0; i < MapNames.Num(); i++)
			{
				FUIAction Action(FExecuteAction::CreateRaw(this, &FGameMapsSettingsCustomization::HandleMapSelected, MapNames[i], PropertyHandle));
				MenuBuilder.AddMenuEntry( 
					FText::FromString( FPaths::GetBaseFilename(MapNames[i]) ), 
					FText::FromString( MapNames[i] ), 
					FSlateIcon(), 
					Action
				);
			}
		}
		MenuBuilder.EndSection();

		// maps that belong to the Engine
		MenuBuilder.BeginSection(NAME_None, LOCTEXT("EngineMapsSectionHeader", "Engine"));
		{
			IFileManager::Get().FindFilesRecursive(MapNames, *FPaths::EngineContentDir(), *MapFileWildCard, true, false);
			MapNames.Sort([](const FString& A, const FString& B) { return FPaths::GetBaseFilename(A) < FPaths::GetBaseFilename(B); });

			for (int32 i = 0; i < MapNames.Num(); i++)
			{
				FUIAction Action(FExecuteAction::CreateRaw(this, &FGameMapsSettingsCustomization::HandleMapSelected, MapNames[i], PropertyHandle));
				MenuBuilder.AddMenuEntry(
					FText::FromString( FPaths::GetBaseFilename(MapNames[i]) ), 
					FText::FromString( MapNames[i] ), 
					FSlateIcon(), 
					Action
				);
			}
		}
		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}

private:

	// Handles selecting a map from a map picker.
	void HandleMapSelected( FString MapName, TSharedPtr<IPropertyHandle> PropertyHandle )
	{
		FString PackageName;

		if (FPackageName::TryConvertFilenameToLongPackageName(MapName, PackageName))
		{
			PropertyHandle->SetValue(PackageName);
		}
	}

	// Handles getting the text color of a map text block.
	FSlateColor HandleMapTextBoxForegroundColor( TSharedPtr<IPropertyHandle> PropertyHandle ) const
	{
		FString Value;

		if (PropertyHandle->GetValue(Value) == FPropertyAccess::Success)
		{
			if (Value.IsEmpty() || IsValidMapName(Value))
			{
				static const FName InvertedForegroundName("InvertedForeground");

				return FAppStyle::GetSlateColor(InvertedForegroundName);
			}
		}

		return FLinearColor::Red;
	}

	// Handles getting the text of a map text block.
	FText HandleMapTextBoxText( TSharedPtr<IPropertyHandle> PropertyHandle ) const
	{
		FString Value;

		if (PropertyHandle->GetValue(Value) == FPropertyAccess::Success)
		{
			return FText::FromString(Value);
		}

		return FText::GetEmpty();
	}

	// Handles text changes in a map text block.
	void HandleMapTextBoxTextChanged( const FText& InText, TSharedPtr<IPropertyHandle> PropertyHandle )
	{
		PropertyHandle->SetValue(InText.ToString());
	}

	// Handles committing changes in a map text block.
	void HandleMapTextBoxTextCommitted( const FText& InText, ETextCommit::Type /*CommitType*/, TSharedPtr<IPropertyHandle> PropertyHandle )
	{
		FString Value;

		if ((PropertyHandle->GetValue(Value) != FPropertyAccess::Success) || !IsValidMapName(Value))
		{
			PropertyHandle->SetValue(FString());
		}
	}

private:

	// Helper class to customizer GameMode property.
	TSharedPtr<FGameModeInfoCustomizer>	GameInfoModeCustomizer;
};


#undef LOCTEXT_NAMESPACE
