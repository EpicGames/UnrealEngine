// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings/LevelEditorPlayNetworkEmulationSettings.h"

#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "Engine/NetDriver.h"
#include "Engine/NetworkSettings.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#include "Settings/LevelEditorPlaySettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelEditorPlayNetworkEmulationSettings)

namespace NetworkEmulationSettingsHelper
{
	const FString& GetCustomProfileName()
	{
		static const FString CustomProfileName = TEXT("Custom");
		return CustomProfileName;
	}

	void ConvertNetDriverSettingsToLevelEditorSettings(const FPacketSimulationSettings& NetDriverSettings, FNetworkEmulationPacketSettings& OutgoingTrafficPieSettings, FNetworkEmulationPacketSettings& IncomingTrafficPieSettings)
	{
		if (NetDriverSettings.PktLag > 0)
		{
			OutgoingTrafficPieSettings.MinLatency = FMath::Max(NetDriverSettings.PktLag - NetDriverSettings.PktLagVariance, 0);
			OutgoingTrafficPieSettings.MaxLatency = FMath::Max(OutgoingTrafficPieSettings.MinLatency, NetDriverSettings.PktLag + NetDriverSettings.PktLagVariance); ;
		}
		else if (NetDriverSettings.PktLagMin > 0 || NetDriverSettings.PktLagMax > 0)
		{
			OutgoingTrafficPieSettings.MinLatency = NetDriverSettings.PktLagMin;
			OutgoingTrafficPieSettings.MaxLatency = NetDriverSettings.PktLagMax;
		}

		OutgoingTrafficPieSettings.PacketLossPercentage = NetDriverSettings.PktLoss;

		IncomingTrafficPieSettings.MinLatency = NetDriverSettings.PktIncomingLagMin;
		IncomingTrafficPieSettings.MaxLatency = NetDriverSettings.PktIncomingLagMax;
		IncomingTrafficPieSettings.PacketLossPercentage = NetDriverSettings.PktIncomingLoss;
	}

	FNetworkEmulationPacketSettings* GetPacketSettingsFromHandle(const TSharedPtr<IPropertyHandle>& PropertyHandle)
	{
		void* ValueData(nullptr);
		FPropertyAccess::Result Result = PropertyHandle->GetValueData(ValueData);
		if (Result != FPropertyAccess::Success)
		{
			return nullptr;
		}

		return (FNetworkEmulationPacketSettings*)ValueData;
	}
}

//-------------------------------------------------------------------------------------------------
// FLevelEditorPlayNetworkEmulationSettingsDetail

TSharedRef<IPropertyTypeCustomization> FLevelEditorPlayNetworkEmulationSettingsDetail::MakeInstance()
{
	return MakeShareable(new FLevelEditorPlayNetworkEmulationSettingsDetail());
}

void FLevelEditorPlayNetworkEmulationSettingsDetail::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	IsNetworkEmulationEnabledHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLevelEditorPlayNetworkEmulationSettings, bIsNetworkEmulationEnabled));

	HeaderRow.NameContent()
	[
		IsNetworkEmulationEnabledHandle->CreatePropertyNameWidget()
		
	]
	.ValueContent()
	[
		IsNetworkEmulationEnabledHandle->CreatePropertyValueWidget()
	]
	.IsEnabled(TAttribute<bool>(this, &FLevelEditorPlayNetworkEmulationSettingsDetail::IsNetworkEmulationAvailable));
}


void FLevelEditorPlayNetworkEmulationSettingsDetail::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// Grab the widget handles
	NetworkEmulationSettingsHandle = StructPropertyHandle;
	CurrentProfileHandle = NetworkEmulationSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLevelEditorPlayNetworkEmulationSettings, CurrentProfile));
	OutPacketsHandle = NetworkEmulationSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLevelEditorPlayNetworkEmulationSettings, OutPackets));
	InPacketsHandle = NetworkEmulationSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLevelEditorPlayNetworkEmulationSettings, InPackets));

	StructBuilder.AddProperty(NetworkEmulationSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLevelEditorPlayNetworkEmulationSettings, EmulationTarget)).ToSharedRef())
		.IsEnabled(TAttribute<bool>(this, &FLevelEditorPlayNetworkEmulationSettingsDetail::HandleAreSettingsEnabled));

	StructBuilder.AddCustomRow(NSLOCTEXT("FLevelEditorPlayNetworkEmulationSettings", "NetworkEmulationProfileName", "Network Emulation Profile"))
	.NameContent()
	[
		SNew(STextBlock)
		.Text(NSLOCTEXT("FLevelEditorPlayNetworkEmulationSettings", "NetworkEmulationProfileName", "Network Emulation Profile"))
		.Font(IDetailLayoutBuilder::GetDetailFont())

	]
	.ValueContent()
	[
		SNew(SComboButton)
		.OnGetMenuContent(this, &FLevelEditorPlayNetworkEmulationSettingsDetail::OnGetProfilesMenuContent)
		.ButtonContent()
		[
			SNew(STextBlock)
			.Text(this, &FLevelEditorPlayNetworkEmulationSettingsDetail::GetSelectedNetworkEmulationProfile)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
	]
	.IsEnabled(TAttribute<bool>(this, &FLevelEditorPlayNetworkEmulationSettingsDetail::HandleAreSettingsEnabled));

	

	StructBuilder.AddProperty(InPacketsHandle.ToSharedRef())
		.IsEnabled(TAttribute<bool>(this, &FLevelEditorPlayNetworkEmulationSettingsDetail::HandleAreSettingsEnabled));
	
	StructBuilder.AddProperty(OutPacketsHandle.ToSharedRef())
		.IsEnabled(TAttribute<bool>(this, &FLevelEditorPlayNetworkEmulationSettingsDetail::HandleAreSettingsEnabled));
}

bool FLevelEditorPlayNetworkEmulationSettingsDetail::HandleAreSettingsEnabled() const
{
	if (IsNetworkEmulationEnabledHandle.IsValid())
	{
		bool bIsEnabled(false);
		IsNetworkEmulationEnabledHandle->GetValue(bIsEnabled);
		return bIsEnabled;
	}

	return false;
}

TSharedRef<SWidget> FLevelEditorPlayNetworkEmulationSettingsDetail::OnGetProfilesMenuContent() const
{
	FMenuBuilder MenuBuilder(true, nullptr);

	if (const UNetworkSettings* NetworkSettings = GetDefault<UNetworkSettings>())
	{
		// Add preconfigured profiles
		for (int32 Index=0; Index < NetworkSettings->NetworkEmulationProfiles.Num(); Index++ )
		{
			const FNetworkEmulationProfileDescription& ProfileDescription = NetworkSettings->NetworkEmulationProfiles[Index];
			
			MenuBuilder.AddMenuEntry(
				FText::FromString(ProfileDescription.ProfileName),
				FText::FromString(ProfileDescription.ToolTip),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &FLevelEditorPlayNetworkEmulationSettingsDetail::OnEmulationProfileChanged, Index),
					FCanExecuteAction()
				)
			);
		}

		// Add the custom entry
		MenuBuilder.AddMenuEntry(
			NSLOCTEXT("FLevelEditorPlayNetworkEmulationSettings", "NetworkEmulationCustomProfile", "Custom"),
			NSLOCTEXT("FLevelEditorPlayNetworkEmulationSettings", "NetworkEmulationCustomProfileToolTip", "Customizable profile"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FLevelEditorPlayNetworkEmulationSettingsDetail::OnEmulationProfileChanged, -1),
				FCanExecuteAction()
			)
		);
	}

	return MenuBuilder.MakeWidget();
}

void FLevelEditorPlayNetworkEmulationSettingsDetail::OnEmulationProfileChanged(int32 Index) const
{
#if DO_ENABLE_NET_TEST
	FString Selection;
	if (const UNetworkSettings* NetworkSettings = GetDefault<UNetworkSettings>())
	{
		if(NetworkSettings->NetworkEmulationProfiles.IsValidIndex(Index))
		{
			Selection = NetworkSettings->NetworkEmulationProfiles[Index].ProfileName;
		}
		else
		{
			Selection = NetworkEmulationSettingsHelper::GetCustomProfileName();
		}
	}

	// Get the current settings
	FNetworkEmulationPacketSettings* OutSettings = NetworkEmulationSettingsHelper::GetPacketSettingsFromHandle(OutPacketsHandle);
	FNetworkEmulationPacketSettings* InSettings = NetworkEmulationSettingsHelper::GetPacketSettingsFromHandle(InPacketsHandle);

	// Reload the settings when switching to an official profile
	if (Selection != NetworkEmulationSettingsHelper::GetCustomProfileName())
	{
		FPacketSimulationSettings NetDriverSettings;
		NetDriverSettings.LoadEmulationProfile(*Selection);

		FNetworkEmulationPacketSettings OutgoingTrafficPIESettings;
		FNetworkEmulationPacketSettings IncomingTrafficPIESettings;
		NetworkEmulationSettingsHelper::ConvertNetDriverSettingsToLevelEditorSettings(NetDriverSettings, OutgoingTrafficPIESettings, IncomingTrafficPIESettings);

		*OutSettings = OutgoingTrafficPIESettings;
		*InSettings = IncomingTrafficPIESettings;
	}

	CurrentProfileHandle->SetValue(Selection);
#endif
}

FText FLevelEditorPlayNetworkEmulationSettingsDetail::GetSelectedNetworkEmulationProfile() const
{
	FString SelectedProfile;
	CurrentProfileHandle->GetValue(SelectedProfile);

	return FText::FromString(SelectedProfile);
}

bool FLevelEditorPlayNetworkEmulationSettingsDetail::IsNetworkEmulationAvailable() const
{
	const ULevelEditorPlaySettings* PlayerLevelEditorSettings = GetDefault<ULevelEditorPlaySettings>();

	int32 NumberOfClients;
	PlayerLevelEditorSettings->GetPlayNumberOfClients(NumberOfClients);

	EPlayNetMode NetMode;
	PlayerLevelEditorSettings->GetPlayNetMode(NetMode);

	return (NumberOfClients > 1) || NetMode != PIE_Standalone || PlayerLevelEditorSettings->bLaunchSeparateServer;
}

//-------------------------------------------------------------------------------------------------
// FLevelEditorPlayNetworkEmulationSettings

bool FLevelEditorPlayNetworkEmulationSettings::IsCustomProfile() const
{
	return CurrentProfile == NetworkEmulationSettingsHelper::GetCustomProfileName();
}

bool FLevelEditorPlayNetworkEmulationSettings::IsEmulationEnabledForTarget(NetworkEmulationTarget CurrentTarget) const
{
	// Settings are applied to everyone
	if (EmulationTarget == NetworkEmulationTarget::Any)
	{
		return true;
	}

	// Otherwise check if target matches the wanted setting
	return EmulationTarget == CurrentTarget;
}

void FLevelEditorPlayNetworkEmulationSettings::ConvertToNetDriverSettings(FPacketSimulationSettings& OutNetDriverSettings) const
{
	OutNetDriverSettings.ResetSettings();

	bool bProfileFound(false);

	if ( !IsCustomProfile() )
	{
		// Load hardcoded profiles from the config since they might have advanced settings not found in PIE settings
		bProfileFound = OutNetDriverSettings.LoadEmulationProfile(*CurrentProfile);
	}

	if (IsCustomProfile() || !bProfileFound)
	{
		// For custom set the settings manually from the PIE variables
		OutNetDriverSettings.PktLagMin = OutPackets.MinLatency;
		OutNetDriverSettings.PktLagMax = OutPackets.MaxLatency;
		OutNetDriverSettings.PktLoss = OutPackets.PacketLossPercentage;
		OutNetDriverSettings.PktIncomingLagMin = InPackets.MinLatency;
		OutNetDriverSettings.PktIncomingLagMax = InPackets.MaxLatency;
		OutNetDriverSettings.PktIncomingLoss = InPackets.PacketLossPercentage;
	}
}

FString FLevelEditorPlayNetworkEmulationSettings::BuildPacketSettingsForCmdLine() const
{
	// Empty string when disabled
	if (!bIsNetworkEmulationEnabled)
	{
		return FString();
	}

	FString CmdLine;

	if (IsCustomProfile())
	{
		// Set each setting manually for user-edited profiles
		CmdLine += FString::Printf(TEXT(" -PktLagMin=%d"), OutPackets.MinLatency);
		CmdLine += FString::Printf(TEXT(" -PktLagMax=%d"), OutPackets.MaxLatency);
		CmdLine += FString::Printf(TEXT(" -PktLoss=%d"), OutPackets.PacketLossPercentage);

		CmdLine += FString::Printf(TEXT(" -PktIncomingLagMin=%d"), InPackets.MinLatency);
		CmdLine += FString::Printf(TEXT(" -PktIncomingLagMax=%d"), InPackets.MaxLatency);
		CmdLine += FString::Printf(TEXT(" -PktIncomingLoss=%d"), InPackets.PacketLossPercentage);
	}
	else
	{
		CmdLine += FString::Printf(TEXT(" -PktEmulationProfile=%s"), *CurrentProfile);
	}
	
	return CmdLine;
}

FString FLevelEditorPlayNetworkEmulationSettings::BuildPacketSettingsForURL() const
{
	// Empty string when disabled
	if (!bIsNetworkEmulationEnabled)
	{
		return FString();
	}

	FString CmdLine;

	if (IsCustomProfile())
	{
		// Set each setting manually for user-edited profiles
		CmdLine += FString::Printf(TEXT("?PktLagMin=%d"), OutPackets.MinLatency);
		CmdLine += FString::Printf(TEXT("?PktLagMax=%d"), OutPackets.MaxLatency);
		CmdLine += FString::Printf(TEXT("?PktLoss=%d"), OutPackets.PacketLossPercentage);

		CmdLine += FString::Printf(TEXT("?PktIncomingLagMin=%d"), InPackets.MinLatency);
		CmdLine += FString::Printf(TEXT("?PktIncomingLagMax=%d"), InPackets.MaxLatency);
		CmdLine += FString::Printf(TEXT("?PktIncomingLoss=%d"), InPackets.PacketLossPercentage);
	}
	else
	{
		CmdLine += FString::Printf(TEXT("?PktEmulationProfile=%s"), *CurrentProfile);
	}

	return CmdLine;
}

void FLevelEditorPlayNetworkEmulationSettings::OnPostInitProperties()
{
	if (CurrentProfile.IsEmpty())
	{
		CurrentProfile = NetworkEmulationSettingsHelper::GetCustomProfileName();
	}
	else if (!IsCustomProfile())
	{
		// For official profiles reload the settings from the config in case some values got changed
		FPacketSimulationSettings NetDriverSettings;

		bool bProfileFound = NetDriverSettings.LoadEmulationProfile(*CurrentProfile);

		if (bProfileFound)
		{
			NetworkEmulationSettingsHelper::ConvertNetDriverSettingsToLevelEditorSettings(NetDriverSettings, OutPackets, InPackets);
		}
		else
		{
			CurrentProfile = NetworkEmulationSettingsHelper::GetCustomProfileName();
		}
	}
}

void FLevelEditorPlayNetworkEmulationSettings::OnPostEditChange(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property)
	{
		if (GET_MEMBER_NAME_CHECKED(FLevelEditorPlayNetworkEmulationSettings, CurrentProfile) == PropertyChangedEvent.Property->GetFName() ||
			GET_MEMBER_NAME_CHECKED(FLevelEditorPlayNetworkEmulationSettings, bIsNetworkEmulationEnabled) == PropertyChangedEvent.Property->GetFName() ||
			GET_MEMBER_NAME_CHECKED(FLevelEditorPlayNetworkEmulationSettings, EmulationTarget) == PropertyChangedEvent.Property->GetFName())
		{
			// Don't consider the settings dirty when these properties change
			return;
		}

		// Set profile to Custom when the user dirties any other property
		CurrentProfile = NetworkEmulationSettingsHelper::GetCustomProfileName();
	}
}

