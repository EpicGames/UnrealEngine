// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings/LevelEditorPlaySettingsCustomization.h"

#include "Misc/Attribute.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Editor.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "IDetailPropertyRow.h"
#include "DetailWidgetRow.h"
#include "Widgets/Input/SComboBox.h"
#include "Sound/AudioSettings.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "FLevelEditorPlaySettingsCustomization"

void SScreenPositionCustomization::Construct( const FArguments& InArgs, IDetailLayoutBuilder* LayoutBuilder, const TSharedRef<IPropertyHandle>& InWindowPositionProperty, const TSharedRef<IPropertyHandle>& InCenterWindowProperty )
{
	check( LayoutBuilder != NULL );

	CenterWindowProperty = InCenterWindowProperty;

	ChildSlot
		[
			SNew( SVerticalBox )
			+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew( SHorizontalBox )
			+ SHorizontalBox::Slot()
		[
			SNew( SVerticalBox )
			.IsEnabled( this, &SScreenPositionCustomization::HandleNewWindowPositionPropertyIsEnabled )
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			InWindowPositionProperty->CreatePropertyNameWidget( LOCTEXT( "WindowPosXLabel", "Left Position" ) )
		]
	+ SVerticalBox::Slot()
		.AutoHeight()
		[
			InWindowPositionProperty->GetChildHandle( 0 )->CreatePropertyValueWidget()
		]
		]

	+ SHorizontalBox::Slot()
		.Padding( 8.0f, 0.0f, 0.0f, 0.0f )
		[
			SNew( SVerticalBox )
			.IsEnabled( this, &SScreenPositionCustomization::HandleNewWindowPositionPropertyIsEnabled )

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			InWindowPositionProperty->CreatePropertyNameWidget( LOCTEXT( "TopPositionLabel", "Top Position" ) )
		]

	+ SVerticalBox::Slot()
		.AutoHeight()
		[
			InWindowPositionProperty->GetChildHandle( 1 )->CreatePropertyValueWidget()
		]
		]
		]
	+ SVerticalBox::Slot()
		.Padding( 0.0f, 2.0f )
		.AutoHeight()
		[
			SNew( SHorizontalBox )
			+ SHorizontalBox::Slot()
		.VAlign( VAlign_Center )
		.AutoWidth()
		[
			InCenterWindowProperty->CreatePropertyValueWidget()
		]
	+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding( 4.0f, 0.0f, 0.0f, 0.0f )
		.VAlign( VAlign_Bottom )
		[
			InWindowPositionProperty->CreatePropertyNameWidget( LOCTEXT( "CenterWindowLabel", "Always center first viewport window to screen" ) )
		]
		]
		];
}

bool SScreenPositionCustomization::HandleNewWindowPositionPropertyIsEnabled() const
{
	bool CenterNewWindow;
	CenterWindowProperty->GetValue( CenterNewWindow );
	return !CenterNewWindow;
}

void SScreenResolutionCustomization::Construct( const FArguments& InArgs, IDetailLayoutBuilder* LayoutBuilder, const TSharedRef<IPropertyHandle>& InWindowHeightProperty, const TSharedRef<IPropertyHandle>& InWindowWidthProperty )
{
	check( LayoutBuilder != NULL );

	WindowHeightProperty = InWindowHeightProperty;
	WindowWidthProperty = InWindowWidthProperty;

	FSimpleDelegate SizeChangeDelegate = FSimpleDelegate::CreateSP( this, &SScreenResolutionCustomization::OnSizeChanged );
	WindowHeightProperty->SetOnPropertyValueChanged( SizeChangeDelegate );
	WindowWidthProperty->SetOnPropertyValueChanged( SizeChangeDelegate );

	ChildSlot
		[
			SNew( SVerticalBox )
			+ SVerticalBox::Slot()
		.VAlign( VAlign_Bottom )
		[
			SNew( SHorizontalBox )
			+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign( VAlign_Center )
		[
			SNew( SComboButton )
			.VAlign( VAlign_Center )
		.ButtonContent()
		[
			SNew( STextBlock )
			.Font( LayoutBuilder->GetDetailFont() )
		.Text( LOCTEXT( "CommonResolutionsButtonText", "Common Resolutions" ) )
		]
	.ContentPadding(FMargin(6, 2))
		.OnGetMenuContent(this, &SScreenResolutionCustomization::GetResolutionsMenu)
	.ToolTipText( LOCTEXT( "CommonResolutionsButtonTooltip", "Pick from a list of common screen resolutions" ) )
		]
	+ SHorizontalBox::Slot()
		.Padding( 0, 0, 6, 0 )
		.AutoWidth()
		.VAlign( VAlign_Center )
		[
			SNew( SButton )
			.OnClicked( this, &SScreenResolutionCustomization::HandleSwapAspectRatioClicked )
		.ContentPadding( FMargin( 3, 0, 3, 1 ) )
		.Content()
		[
			SNew( SImage )
			.Image( this, &SScreenResolutionCustomization::GetAspectRatioSwitchImage )
		]
	.ToolTipText( LOCTEXT( "SwapAspectRatioTooltip", "Swap between portrait and landscape orientation." ) )
		]
		]
	+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew( SHorizontalBox )
			+ SHorizontalBox::Slot()
		[
			SNew( SVerticalBox )
			+ SVerticalBox::Slot()
		.AutoHeight()
		[
			WindowWidthProperty->CreatePropertyNameWidget( LOCTEXT( "ViewportWidthLabel", "Viewport Width" ) )
		]
	+ SVerticalBox::Slot()
		[
			WindowWidthProperty->CreatePropertyValueWidget()
		]
		]
	+ SHorizontalBox::Slot()
		.Padding( 8, 0, 0, 0 )
		[
			SNew( SVerticalBox )
			+ SVerticalBox::Slot()
		.AutoHeight()
		[
			WindowHeightProperty->CreatePropertyNameWidget( LOCTEXT( "ViewportHeightLabel", "Viewport Height" ) )
		]
	+ SVerticalBox::Slot()
		.AutoHeight()
		[
			WindowHeightProperty->CreatePropertyValueWidget()
		]
		]
		]
		];
}

FReply SScreenResolutionCustomization::HandleSwapAspectRatioClicked()
{
	FString HeightString;
	WindowHeightProperty->GetValueAsDisplayString( HeightString );
	FString WidthString;
	WindowWidthProperty->GetValueAsDisplayString( WidthString );
	int32 NewHeight = FCString::Atoi( *WidthString );
	int32 NewWidth = FCString::Atoi( *HeightString );

	ULevelEditorPlaySettings* PlayInSettings = GetMutableDefault<ULevelEditorPlaySettings>();

	const UDeviceProfile* DeviceProfile = UDeviceProfileManager::Get().FindProfile(PlayInSettings->DeviceToEmulate, false);
	if ( DeviceProfile )
	{
		// Rescale the swapped sizes if we are on Android
		if ( DeviceProfile && DeviceProfile->DeviceType == TEXT( "Android" ) )
		{
			float ScaleFactor = 1.0f;
			PlayInSettings->RescaleForMobilePreview( DeviceProfile, NewWidth, NewHeight, ScaleFactor );
		}
	}

	WindowHeightProperty->SetValue(NewHeight);
	WindowWidthProperty->SetValue(NewWidth);

	return FReply::Handled();
}

void SScreenResolutionCustomization::HandleCommonResolutionSelected( const FPlayScreenResolution Resolution )
{
	int32 Width = Resolution.LogicalWidth;
	int32 Height = Resolution.LogicalHeight;
	ULevelEditorPlaySettings* PlayInSettings = GetMutableDefault<ULevelEditorPlaySettings>();
	// Maintain previous orientation (i.e., swap Width and Height if required)
	int32 PreviousWidth = -1;
	int32 PreviousHeight = -1;
	if ( WindowWidthProperty->GetValue( PreviousWidth ) == FPropertyAccess::Success && WindowHeightProperty->GetValue( PreviousHeight ) == FPropertyAccess::Success )
	{
		const bool bIsOrientationPreserved = (PreviousWidth < PreviousHeight) != (Width < Height);
		if ( bIsOrientationPreserved )
		{
			Width = Resolution.LogicalHeight;
			Height = Resolution.LogicalWidth;
		}
	}

	UDeviceProfile* DeviceProfile = UDeviceProfileManager::Get().FindProfile( Resolution.ProfileName, false );
	if ( DeviceProfile )
	{
		PlayInSettings->DeviceToEmulate = Resolution.ProfileName;
	}
	else
	{
		PlayInSettings->DeviceToEmulate = FString();
	}

	if ( (PreviousWidth == Width) && (PreviousHeight == Height) )
	{
		// Since the values are identical, the SetValue on the property handle will not actually change the value and trigger OnPropertyValueChanged, so call our handler directly
		OnSizeChanged();
	}
	else
	{
		WindowHeightProperty->SetValue(Height);
		WindowWidthProperty->SetValue(Width);
	}
}

const FSlateBrush* SScreenResolutionCustomization::GetAspectRatioSwitchImage() const
{
	FString HeightString;
	WindowHeightProperty->GetValueAsDisplayString( HeightString );
	int32 Height = FCString::Atoi( *HeightString );
	FString WidthString;
	WindowWidthProperty->GetValueAsDisplayString( WidthString );
	int32 Width = FCString::Atoi( *WidthString );
	if ( Height > Width )
	{
		return FAppStyle::Get().GetBrush( "UMGEditor.OrientPortrait" );
	}
	return FAppStyle::Get().GetBrush( "UMGEditor.OrientLandscape" );
}

void SScreenResolutionCustomization::OnSizeChanged()
{
	GetMutableDefault<ULevelEditorPlaySettings>()->UpdateCustomSafeZones();
}

FUIAction SScreenResolutionCustomization::GetResolutionMenuAction( const FPlayScreenResolution& ScreenResolution )
{
	return FUIAction( FExecuteAction::CreateRaw( this, &SScreenResolutionCustomization::HandleCommonResolutionSelected, ScreenResolution ) );
}

TSharedRef<SWidget> SScreenResolutionCustomization::GetResolutionsMenu()
{
	UCommonResolutionMenuContext* CommonResolutionMenuContext = NewObject<UCommonResolutionMenuContext>();
	CommonResolutionMenuContext->GetUIActionFromLevelPlaySettings = UCommonResolutionMenuContext::FGetUIActionFromLevelPlaySettings::CreateRaw(this, &SScreenResolutionCustomization::GetResolutionMenuAction);

	return UToolMenus::Get()->GenerateWidget(ULevelEditorPlaySettings::GetCommonResolutionsMenuName(), CommonResolutionMenuContext);
}

/** Virtual destructor. */
FLevelEditorPlaySettingsCustomization::~FLevelEditorPlaySettingsCustomization()
{
}

void FLevelEditorPlaySettingsCustomization::CustomizeDetails( IDetailLayoutBuilder& LayoutBuilder )
{
	const float MaxPropertyWidth = 400.0f;

	// play in editor settings
	IDetailCategoryBuilder& PlayInEditorCategory = LayoutBuilder.EditCategory( "PlayInEditor" );
	{
		TArray<TSharedRef<IPropertyHandle>> PIECategoryProperties;
		PlayInEditorCategory.GetDefaultProperties( PIECategoryProperties, true, false );

		TSharedPtr<IPropertyHandle> PIEEnableSoundHandle = LayoutBuilder.GetProperty( GET_MEMBER_NAME_CHECKED( ULevelEditorPlaySettings, EnableGameSound ) );
		PIESoundQualityLevelHandle = LayoutBuilder.GetProperty( GET_MEMBER_NAME_CHECKED( ULevelEditorPlaySettings, PlayInEditorSoundQualityLevel ) );
		PIESoundQualityLevelHandle->MarkHiddenByCustomization();

		for ( TSharedRef<IPropertyHandle>& PropertyHandle : PIECategoryProperties )
		{
			if ( PropertyHandle->GetProperty() != PIESoundQualityLevelHandle->GetProperty() )
			{
				PlayInEditorCategory.AddProperty( PropertyHandle );
			}

			if ( PropertyHandle->GetProperty() == PIEEnableSoundHandle->GetProperty() )
			{
				PlayInEditorCategory.AddCustomRow( PIESoundQualityLevelHandle->GetPropertyDisplayName(), false )
					.NameContent()
					[
						PIESoundQualityLevelHandle->CreatePropertyNameWidget()
					]
				.ValueContent()
					.MaxDesiredWidth( MaxPropertyWidth )
					[
						SAssignNew( QualityLevelComboBox, SComboBox<TSharedPtr<FString>> )
						.OptionsSource( &AvailableQualityLevels )
					.OnComboBoxOpening( this, &FLevelEditorPlaySettingsCustomization::HandleQualityLevelComboBoxOpening )
					.OnGenerateWidget( this, &FLevelEditorPlaySettingsCustomization::HandleQualityLevelComboBoxGenerateWidget )
					.OnSelectionChanged( this, &FLevelEditorPlaySettingsCustomization::HandleQualityLevelSelectionChanged )
					[
						SNew( STextBlock )
						.Text( this, &FLevelEditorPlaySettingsCustomization::GetSelectedQualityLevelName )
					]
					];
			}
		}


	}
	IDetailCategoryBuilder& GameViewportSettings = LayoutBuilder.EditCategory( "GameViewportSettings" );
	{
		// new window resolution
		TSharedRef<IPropertyHandle> WindowHeightHandle = LayoutBuilder.GetProperty( GET_MEMBER_NAME_CHECKED( ULevelEditorPlaySettings, NewWindowHeight ) );
		TSharedRef<IPropertyHandle> WindowWidthHandle = LayoutBuilder.GetProperty( GET_MEMBER_NAME_CHECKED( ULevelEditorPlaySettings, NewWindowWidth ) );
		TSharedRef<IPropertyHandle> WindowPositionHandle = LayoutBuilder.GetProperty( GET_MEMBER_NAME_CHECKED( ULevelEditorPlaySettings, NewWindowPosition ) );
		TSharedRef<IPropertyHandle> CenterNewWindowHandle = LayoutBuilder.GetProperty( GET_MEMBER_NAME_CHECKED( ULevelEditorPlaySettings, CenterNewWindow ) );
		TSharedRef<IPropertyHandle> EmulatedDeviceHandle = LayoutBuilder.GetProperty( GET_MEMBER_NAME_CHECKED( ULevelEditorPlaySettings, DeviceToEmulate ) );

		WindowHeightHandle->MarkHiddenByCustomization();
		WindowWidthHandle->MarkHiddenByCustomization();
		WindowPositionHandle->MarkHiddenByCustomization();
		CenterNewWindowHandle->MarkHiddenByCustomization();
		EmulatedDeviceHandle->MarkHiddenByCustomization();

		GameViewportSettings.AddCustomRow( LOCTEXT( "NewViewportResolutionRow", "New Viewport Resolution" ), false )
			.NameContent()
			[
				SNew( STextBlock )
				.Font( LayoutBuilder.GetDetailFont() )
			.Text( LOCTEXT( "NewViewportResolutionName", "New Viewport Resolution" ) )
			.ToolTipText( LOCTEXT( "NewWindowSizeTooltip", "Sets the width and height of floating PIE windows (in pixels)" ) )
			]
		.ValueContent()
			.MaxDesiredWidth( MaxPropertyWidth )
			[
				SNew( SScreenResolutionCustomization, &LayoutBuilder, WindowHeightHandle, WindowWidthHandle )
			];

		GameViewportSettings.AddCustomRow( LOCTEXT( "NewWindowPositionRow", "New Window Position" ), false )
			.NameContent()
			[
				SNew( STextBlock )
				.Font( LayoutBuilder.GetDetailFont() )
			.Text( LOCTEXT( "NewWindowPositionName", "New Window Position" ) )
			.ToolTipText( LOCTEXT( "NewWindowPositionTooltip", "Sets the screen coordinates for the top-left corner of floating PIE windows (in pixels)" ) )
			]
		.ValueContent()
			.MaxDesiredWidth( MaxPropertyWidth )
			[
				SNew( SScreenPositionCustomization, &LayoutBuilder, WindowPositionHandle, CenterNewWindowHandle )
			];

		GameViewportSettings.AddCustomRow( LOCTEXT( "SafeZonePreviewName", "Safe Zone Preview" ), false )
			.NameContent()
			[
				SNew( STextBlock )
				.Font( LayoutBuilder.GetDetailFont() )
			.Text( LOCTEXT( "SafeZonePreviewName", "Safe Zone Preview" ) )
			]
		.ValueContent()
			[
				SNew( STextBlock )
				.Font( LayoutBuilder.GetDetailFont() )
				.Text( this, &FLevelEditorPlaySettingsCustomization::GetPreviewText )
				.ToolTipText(this, &FLevelEditorPlaySettingsCustomization::GetPreviewTextToolTipText)
			];
	}

	// play in new window settings
	IDetailCategoryBuilder& PlayInNewWindowCategory = LayoutBuilder.EditCategory( "PlayInNewWindow" );
	{
		// Mac does not support parenting, do not show
#if PLATFORM_MAC
		PlayInNewWindowCategory.AddProperty( "PIEAlwaysOnTop" )
			.DisplayName( LOCTEXT( "PIEAlwaysOnTop", "Always On Top" ) )
			.IsEnabled( false );
#else
		PlayInNewWindowCategory.AddProperty( "PIEAlwaysOnTop" )
			.DisplayName( LOCTEXT( "PIEAlwaysOnTop", "Always On Top" ) );
#endif
	}


	// play in standalone game settings
	IDetailCategoryBuilder& PlayInStandaloneCategory = LayoutBuilder.EditCategory( "PlayInStandaloneGame" );
	{
		// command line options
		TSharedPtr<IPropertyHandle> DisableStandaloneSoundProperty = LayoutBuilder.GetProperty( GET_MEMBER_NAME_CHECKED( ULevelEditorPlaySettings, DisableStandaloneSound ) );

		DisableStandaloneSoundProperty->MarkHiddenByCustomization();

		PlayInStandaloneCategory.AddCustomRow( LOCTEXT( "AdditionalStandaloneDetails", "Additional Options" ), false )
			.NameContent()
			[
				SNew( STextBlock )
				.Font( LayoutBuilder.GetDetailFont() )
			.Text( LOCTEXT( "ClientCmdLineName", "Command Line Options" ) )
			.ToolTipText( LOCTEXT( "ClientCmdLineTooltip", "Generates a command line for additional settings that will be passed to the game clients." ) )
			]
		.ValueContent()
			.MaxDesiredWidth( MaxPropertyWidth )
			[
				SNew( SHorizontalBox )

				+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				DisableStandaloneSoundProperty->CreatePropertyValueWidget()
			]

		+ SHorizontalBox::Slot()
			.Padding( 0.0f, 2.5f )
			.VAlign( VAlign_Center )
			.AutoWidth()
			[
				DisableStandaloneSoundProperty->CreatePropertyNameWidget( LOCTEXT( "DisableStandaloneSoundLabel", "Disable Sound (-nosound)" ) )
			]

			];
	}

	// multi-player options
	IDetailCategoryBuilder& NetworkCategory = LayoutBuilder.EditCategory( "Multiplayer Options" );
	TArray<TSharedRef<IPropertyHandle>> AllProperties;
	NetworkCategory.GetDefaultProperties( AllProperties );

	{
		// Add all the default properties in before we add custom rows at the bottom.
		for ( TSharedRef<IPropertyHandle> DefaultProperty : AllProperties )
		{
			NetworkCategory.AddProperty( DefaultProperty );
		}


		// client window size
		TSharedRef<IPropertyHandle> WindowHeightHandle = LayoutBuilder.GetProperty( "ClientWindowHeight" );
		TSharedRef<IPropertyHandle> WindowWidthHandle = LayoutBuilder.GetProperty( "ClientWindowWidth" );

		WindowHeightHandle->MarkHiddenByCustomization();
		WindowWidthHandle->MarkHiddenByCustomization();



		NetworkCategory.AddCustomRow( LOCTEXT( "PlayInNetworkViewportSize", "Viewport Size\n(Additional Clients)" ), false )
			.NameContent()
			[
				WindowHeightHandle->CreatePropertyNameWidget( LOCTEXT( "ClientViewportSizeName", "Multiplayer Viewport Size (in pixels)" ), LOCTEXT( "ClientWindowSizeTooltip", "Width and Height to use when spawning additional clients. Useful when you need multiple clients connected but only interact with one window." ) )
			]
		.ValueContent()
			.MaxDesiredWidth( MaxPropertyWidth )
			[
				SNew( SScreenResolutionCustomization, &LayoutBuilder, WindowHeightHandle, WindowWidthHandle )
			]
		.IsEnabled( TAttribute<bool>( this, &FLevelEditorPlaySettingsCustomization::HandleClientWindowSizePropertyIsEnabled ) );

		NetworkCategory.AddCustomRow( LOCTEXT( "AdditionalMultiplayerDetails", "Additional Options" ), true )
			.NameContent()
			[
				SNew( STextBlock )
				.Font( LayoutBuilder.GetDetailFont() )
			.Text( LOCTEXT( "PlainTextName", "Play In Editor Description" ) )
			.ToolTipText( LOCTEXT( "PlainTextToolTip", "A brief description of the multiplayer settings and what to expect if you play with them in the editor." ) )
			]
		.ValueContent()
			.MaxDesiredWidth( MaxPropertyWidth )
			[
				SNew( STextBlock )
				.Font( LayoutBuilder.GetDetailFont() )
			.Text( this, &FLevelEditorPlaySettingsCustomization::HandleMultiplayerOptionsDescription )
			.WrapTextAt( MaxPropertyWidth )
			];
	}
}

TSharedRef<IDetailCustomization> FLevelEditorPlaySettingsCustomization::MakeInstance()
{
	return MakeShareable( new FLevelEditorPlaySettingsCustomization() );
}

FText FLevelEditorPlaySettingsCustomization::HandleMultiplayerOptionsDescription() const
{
	const ULevelEditorPlaySettings* PlayInSettings = GetDefault<ULevelEditorPlaySettings>();
	const bool bLaunchSeparateServer = PlayInSettings->bLaunchSeparateServer;
	const bool CanRunUnderOneProcess = [&PlayInSettings] { bool RunUnderOneProcess( false ); return (PlayInSettings->GetRunUnderOneProcess( RunUnderOneProcess ) && RunUnderOneProcess); }();
	const int32 PlayNumberOfClients = [&PlayInSettings] { int32 NumClients( 0 ); return (PlayInSettings->GetPlayNumberOfClients( NumClients ) ? NumClients : 0); }();
	const EPlayNetMode PlayNetMode = [&PlayInSettings] { EPlayNetMode NetMode( PIE_Standalone ); return (PlayInSettings->GetPlayNetMode( NetMode ) ? NetMode : PIE_Standalone); }();
	FString Desc;
	if ( CanRunUnderOneProcess )
	{
		Desc += LOCTEXT( "MultiplayerDescription_OneProcess", "The following will all run under one UE instance:\n" ).ToString();
		if ( PlayNetMode == EPlayNetMode::PIE_Client )
		{
			Desc += LOCTEXT( "MultiplayerDescription_DedicatedServerHidden", "A hidden dedicated server instance will run in editor. " ).ToString();
			if ( PlayNumberOfClients == 1 )
			{
				Desc += LOCTEXT( "MultiplayerDescription_EditorClient", "The editor will connect as a client. " ).ToString();
			}
			else
			{
				Desc += FText::Format( LOCTEXT( "MultiplayerDescription_EditorAndClients", "The editor will connect as a client and {0} additional client window(s) will also connect. " ), FText::AsNumber( PlayNumberOfClients - 1 ) ).ToString();
			}
		}
		else if ( PlayNetMode == EPlayNetMode::PIE_ListenServer )
		{
			if ( PlayNumberOfClients == 1 )
			{
				Desc += LOCTEXT( "MultiplayerDescription_EditorListenServer", "The editor will run as a listen server. " ).ToString();
			}
			else
			{
				Desc += FText::Format( LOCTEXT( "MultiplayerDescription_EditorListenServerAndClients", "The editor will run as a listen server and {0} additional client window(s) will also connect to it. " ), FText::AsNumber( PlayNumberOfClients - 1 ) ).ToString();
			}
		}
		else
		{
			if ( PlayNumberOfClients == 1 )
			{
				Desc += LOCTEXT( "MultiplayerDescription_EditorStandalone", "The editor will run in offline mode. " ).ToString();
			}
			else
			{
				Desc += FText::Format( LOCTEXT( "MultiplayerDescription_StandaloneAndClients", "The editor will run offline and {0} additional offline mode window(s) will also open. " ), FText::AsNumber( PlayNumberOfClients - 1 ) ).ToString();
			}

			if ( bLaunchSeparateServer )
			{
				Desc += LOCTEXT( "MultiplayerDescription_StandaloneSeparateServer", "\nAn additional server instance will be launched but not connected to. Use \"open 127.0.0.1:<port>\" to connect. " ).ToString();
			}
		}
	}
	else
	{
		Desc += LOCTEXT( "MultiplayerDescription_MultiProcess", "The following will run with multiple UE instances:\n" ).ToString();
		if ( PlayNetMode == PIE_Standalone )
		{
			if ( PlayNumberOfClients == 1 )
			{
				Desc += LOCTEXT( "MultiplayerDescription_EditorStandalone", "The editor will run in offline mode. " ).ToString();
			}
			else
			{
				Desc += FText::Format( LOCTEXT( "MultiplayerDescription_StandaloneAndClients", "The editor will run offline and {0} additional offline mode window(s) will also open. " ), FText::AsNumber( PlayNumberOfClients - 1 ) ).ToString();
			}

			if ( bLaunchSeparateServer )
			{
				Desc += LOCTEXT( "MultiplayerDescription_StandaloneSeparateServer", "\nAn additional server instance will be launched but not connected to. Use \"open 127.0.0.1:<port>\" to connect. " ).ToString();
			}
		}
		else if ( PlayNetMode == PIE_ListenServer )
		{
			if ( PlayNumberOfClients == 1 )
			{
				Desc += LOCTEXT( "MultiplayerDescription_EditorListenServer", "The editor will run as a listen server. " ).ToString();
			}
			else
			{
				Desc += FText::Format( LOCTEXT( "MultiplayerDescription_EditorListenServerAndClients", "The editor will run as a listen server and {0} additional client window(s) will also connect to it. " ), FText::AsNumber( PlayNumberOfClients - 1 ) ).ToString();
			}
		}
		else
		{
			// Client requires additional dedicated server instance
			Desc += LOCTEXT( "MultiplayerDescription_DedicatedServerNewWindow", "A dedicated server will open in a new window. " ).ToString();
			if ( PlayNumberOfClients == 1 )
			{
				Desc += LOCTEXT( "MultiplayerDescription_EditorClient", "The editor will connect as a client. " ).ToString();
			}
			else
			{
				Desc += FText::Format( LOCTEXT( "MultiplayerDescription_EditorAndClients", "The editor will connect as a client and {0} additional client window(s) will also connect. " ), FText::AsNumber( PlayNumberOfClients - 1 ) ).ToString();
			}

		}
	}
	return FText::FromString( Desc );
}

bool FLevelEditorPlaySettingsCustomization::HandleClientWindowSizePropertyIsEnabled() const
{
	return GetDefault<ULevelEditorPlaySettings>()->IsClientWindowSizeActive();
}

bool FLevelEditorPlaySettingsCustomization::HandleGameOptionsIsEnabled() const
{
	return GetDefault<ULevelEditorPlaySettings>()->IsAdditionalServerGameOptionsActive();
}

bool FLevelEditorPlaySettingsCustomization::HandleRerouteInputToSecondWindowEnabled() const
{
	return GetDefault<ULevelEditorPlaySettings>()->IsRouteGamepadToSecondWindowActive();
}

EVisibility FLevelEditorPlaySettingsCustomization::HandleRerouteInputToSecondWindowVisibility() const
{
	return GetDefault<ULevelEditorPlaySettings>()->GetRouteGamepadToSecondWindowVisibility();
}

void FLevelEditorPlaySettingsCustomization::HandleQualityLevelComboBoxOpening()
{
	const UAudioSettings* AudioSettings = GetDefault<UAudioSettings>();
	AvailableQualityLevels.Empty( AudioSettings->QualityLevels.Num() );
	for ( const FAudioQualitySettings& AQSettings : AudioSettings->QualityLevels )
	{
		AvailableQualityLevels.Add( MakeShareable( new FString( AQSettings.DisplayName.ToString() ) ) );
	}
	QualityLevelComboBox->RefreshOptions();
}

TSharedRef<SWidget> FLevelEditorPlaySettingsCustomization::HandleQualityLevelComboBoxGenerateWidget( TSharedPtr<FString> InItem )
{
	return SNew( STextBlock )
		.Text( FText::FromString( *InItem ) );
}

void FLevelEditorPlaySettingsCustomization::HandleQualityLevelSelectionChanged( TSharedPtr<FString> InSelection, ESelectInfo::Type SelectInfo )
{
	if ( InSelection.IsValid() )
	{
		const UAudioSettings* AudioSettings = GetDefault<UAudioSettings>();
		for ( int32 QualityLevel = 0; QualityLevel < AudioSettings->QualityLevels.Num(); ++QualityLevel )
		{
			if ( AudioSettings->QualityLevels[QualityLevel].DisplayName.ToString() == *InSelection )
			{
				PIESoundQualityLevelHandle->SetValue( QualityLevel );
				break;
			}
		}
	}
}

FText FLevelEditorPlaySettingsCustomization::GetSelectedQualityLevelName() const
{
	int32 QualityLevel = 0;
	PIESoundQualityLevelHandle->GetValue( QualityLevel );
	const UAudioSettings* AudioSettings = GetDefault<UAudioSettings>();
	return (QualityLevel >= 0 && QualityLevel < AudioSettings->QualityLevels.Num() ? AudioSettings->QualityLevels[QualityLevel].DisplayName : FText::GetEmpty());
}

FText FLevelEditorPlaySettingsCustomization::GetPreviewText() const
{
	const ULevelEditorPlaySettings* LevelEditorPlaySettings = GetDefault<ULevelEditorPlaySettings>();
	float SafeZone = FDisplayMetrics::GetDebugTitleSafeZoneRatio();
	if ( FMath::IsNearlyEqual( SafeZone, 1.0f ) )
	{
		if ( LevelEditorPlaySettings->DeviceToEmulate.IsEmpty() || LevelEditorPlaySettings->PIESafeZoneOverride.GetDesiredSize().IsZero() )
		{
			return LOCTEXT( "NoSafeZoneSet", "No Device Safe Zone Set" );
		}
		else
		{
			return FText::FromString( LevelEditorPlaySettings->DeviceToEmulate );
		}
	}
	else
	{
		return FText::Format( LOCTEXT( "UniformSafeZone", "Uniform Safe Zone: {0}" ), FText::AsNumber( SafeZone ) );
	}
}

FText FLevelEditorPlaySettingsCustomization::GetPreviewTextToolTipText() const
{
	if (FDisplayMetrics::GetDebugTitleSafeZoneRatio() < 1.0f)
	{
		return LOCTEXT("SafeZoneSetFromRatioCvarToolTip", "Uniform Safe Zone was set from cvar (r.DebugSafeZone.TitleRatio).");
	}

	const ULevelEditorPlaySettings* LevelEditorPlaySettings = GetDefault<ULevelEditorPlaySettings>();
	if (!LevelEditorPlaySettings->DeviceToEmulate.IsEmpty() && !LevelEditorPlaySettings->PIESafeZoneOverride.GetDesiredSize().IsZero() )
	{
		FMargin PIESafeZoneOverride = LevelEditorPlaySettings->PIESafeZoneOverride;
		FNumberFormattingOptions FormattingOptions;
		FormattingOptions.SetMaximumFractionalDigits(2);
		FormattingOptions.SetMinimumFractionalDigits(2);

		return FText::Format(
			LOCTEXT("CustomSafeZoneSetFromDevice", "PIE safe zone override set from device {0} profile.\nLeft: {1}, Top: {2}, Right: {3}, Bottom: {4}"),
			FText::FromString(LevelEditorPlaySettings->DeviceToEmulate),
			FText::AsNumber(PIESafeZoneOverride.Left, &FormattingOptions),
			FText::AsNumber(PIESafeZoneOverride.Top, &FormattingOptions),
			FText::AsNumber(PIESafeZoneOverride.Right, &FormattingOptions),
			FText::AsNumber(PIESafeZoneOverride.Bottom, &FormattingOptions)
		);
	}

	return FText();
}

#undef LOCTEXT_NAMESPACE
