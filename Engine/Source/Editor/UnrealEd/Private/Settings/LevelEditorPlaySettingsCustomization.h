// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "Settings/LevelEditorPlaySettings.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FMenuBuilder;
struct FSlateBrush;
class IDetailCustomization;
class IDetailLayoutBuilder;
class IPropertyHandle;
template <typename OptionType = TSharedPtr<FString>>
class SComboBox;
class SWidget;

class SScreenPositionCustomization
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SScreenPositionCustomization) { }
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param LayoutBuilder The layout builder to use for generating property widgets.
	 * @param InWindowPositionProperty The handle to the window position property.
	 * @param InCenterWindowProperty The handle to the center window property.
	 */
	void Construct( const FArguments& InArgs, IDetailLayoutBuilder* LayoutBuilder, const TSharedRef<IPropertyHandle>& InWindowPositionProperty, const TSharedRef<IPropertyHandle>& InCenterWindowProperty );

private:

	// Callback for checking whether the window position properties are enabled.
	bool HandleNewWindowPositionPropertyIsEnabled() const;

private:

	// Holds the 'Center window' property
	TSharedPtr<IPropertyHandle> CenterWindowProperty;
};


/**
 * Implements a screen resolution picker widget.
 */
class SScreenResolutionCustomization
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SScreenResolutionCustomization) { }
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param LayoutBuilder The layout builder to use for generating property widgets.
	 * @param InWindowHeightProperty The handle to the window height property.
	 * @param InWindowWidthProperty The handle to the window width property.
	 */
	void Construct( const FArguments& InArgs, IDetailLayoutBuilder* LayoutBuilder, const TSharedRef<IPropertyHandle>& InWindowHeightProperty, const TSharedRef<IPropertyHandle>& InWindowWidthProperty );

protected:

	// Resets LevelEditorPlaySettings that relate to the title safe zone debug draw.
	void ResetCustomTitleSafeZoneSettings(ULevelEditorPlaySettings* PlayInSettings, const int32 Width, const int32 Height);

	// Called when the PIE safe zone override is changed. Performs final safe zone calculations and broadcasts the result.
	void BroadcastSafeZoneChanged(const FMargin& SafeZoneRatio, const int32 Width, const int32 Height);

	// Handles swapping the current aspect ratio.
	// Also will set a custom safe zone matching the device profile, if r.DebugSafeZone.TitleRatio is set to 1.0
	FReply HandleSwapAspectRatioClicked();

private:

	// Handles selecting a common screen resolution.
	// Also will set a custom safe zone matching the device profile, if r.DebugSafeZone.TitleRatio is set to 1.0
	void HandleCommonResolutionSelected( const FPlayScreenResolution Resolution );

	const FSlateBrush* GetAspectRatioSwitchImage() const;

	void OnSizeChanged();

	FUIAction GetResolutionMenuAction( const FPlayScreenResolution& ScreenResolution );

	TSharedRef<SWidget> GetResolutionsMenu();

private:

	// Holds the handle to the window height property.
	TSharedPtr<IPropertyHandle> WindowHeightProperty;

	// Holds the handle to the window width property.
	TSharedPtr<IPropertyHandle> WindowWidthProperty;

	// True if a property was set from the resolution menu
	bool bSetFromMenu;
};


/**
 * Implements a details view customization for ULevelEditorPlaySettings objects.
 */
class FLevelEditorPlaySettingsCustomization
	: public IDetailCustomization
{
public:

	/** Virtual destructor. */
	virtual ~FLevelEditorPlaySettingsCustomization();

public:

	// IDetailCustomization interface
	virtual void CustomizeDetails( IDetailLayoutBuilder& LayoutBuilder ) override;
	// End IDetailCustomization interface

public:

	/**
	 * Creates a new instance.
	 *
	 * @return A new struct customization for play-in settings.
	 */
	static TSharedRef<IDetailCustomization> MakeInstance();

private:

	// Callback for getting the description of the settings
	FText HandleMultiplayerOptionsDescription() const;

	// Callback for checking whether the ClientWindowHeight and ClientWindowWidth properties are enabled.
	bool HandleClientWindowSizePropertyIsEnabled() const;

	// Callback for checking whether the AdditionalServerGameOptions is enabled.;
	bool HandleGameOptionsIsEnabled( ) const;

	// Callback for getting the enabled state of the RerouteInputToSecondWindow property.;
	bool HandleRerouteInputToSecondWindowEnabled( ) const;
	
	// Callback for getting the visibility of the RerouteInputToSecondWindow property.
	EVisibility HandleRerouteInputToSecondWindowVisibility() const;

	void HandleQualityLevelComboBoxOpening();

	TSharedRef<SWidget> HandleQualityLevelComboBoxGenerateWidget( TSharedPtr<FString> InItem );

	void HandleQualityLevelSelectionChanged( TSharedPtr<FString> InSelection, ESelectInfo::Type SelectInfo );

	FText GetSelectedQualityLevelName() const;

	FText GetPreviewText() const;

	FText GetPreviewTextToolTipText() const;
private:

	/** Collection of possible quality levels we can use as a parent for this profile */
	TArray<TSharedPtr<FString>> AvailableQualityLevels;
	TSharedPtr<IPropertyHandle> PIESoundQualityLevelHandle;
	TSharedPtr<SComboBox<TSharedPtr<FString>>> QualityLevelComboBox;

};
