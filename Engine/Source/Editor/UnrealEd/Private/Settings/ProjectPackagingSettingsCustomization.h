// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/Attribute.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Misc/NotifyHook.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SComboButton.h"
#include "Styling/SlateTypes.h"
#include "Misc/Paths.h"
#include "Styling/CoreStyle.h"
#include "HAL/FileManager.h"
#include "Styling/AppStyle.h"
#include "IDetailCustomization.h"
#include "IPropertyUtilities.h"
#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "Widgets/Text/STextBlock.h"
#include "DetailWidgetRow.h"
#include "Internationalization/Culture.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Settings/ProjectPackagingSettings.h"
#include "Settings/EditorExperimentalSettings.h"
#include "PropertyRestriction.h"
#include "Widgets/Views/SMultipleOptionTable.h"
#include "DesktopPlatformModule.h"
#include "ILauncherServicesModule.h"


#define LOCTEXT_NAMESPACE "FProjectPackagingSettingsCustomization"

class SCulturePickerRowWidget : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SCulturePickerRowWidget){}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, FCulturePtr InCulture, TAttribute<bool> InIsFilteringCultures)
	{
		Culture = InCulture;
		IsFilteringCultures = InIsFilteringCultures;

		// Identify if this culture has localization data.
		{
			const TArray<FString> LocalizedCultureNames = FTextLocalizationManager::Get().GetLocalizedCultureNames(ELocalizationLoadFlags::Game);
			const TArray<FCultureRef> LocalizedCultures = FInternationalization::Get().GetAvailableCultures(LocalizedCultureNames, true);
			HasLocalizationData = LocalizedCultures.Contains(Culture.ToSharedRef());
		}

		ChildSlot
			[
				SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(3.0, 2.0))
					.VAlign(VAlign_Center)
					[
						// Warning Icon for whether or not this culture has localization data.
						SNew(SImage)
						.Image( FCoreStyle::Get().GetBrush("Icons.Warning") )
						.Visibility(this, &SCulturePickerRowWidget::HandleWarningImageVisibility)
						.ToolTipText(LOCTEXT("NotLocalizedWarning", "This project does not have localization data (translations) for this culture."))
					]
					+SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.VAlign(VAlign_Center)
					[
						// Display name of culture.
						SNew(STextBlock)
						.Text(FText::FromString(Culture->GetDisplayName()))
						.ToolTipText(FText::FromString(Culture->GetName()))
					]
			];
	}

	EVisibility HandleWarningImageVisibility() const
	{
		// Don't show the warning image if this culture has localization data.
		// Collapse the widget entirely if we are filtering to only show cultures that have it - gets rid of awkward empty column of space.
		bool bIsFilteringCultures = IsFilteringCultures.IsBound() ? IsFilteringCultures.Get() : false;
		return bIsFilteringCultures ? EVisibility::Collapsed : (HasLocalizationData ? EVisibility::Hidden : EVisibility::Visible);
	}

private:
	FCulturePtr Culture;
	TAttribute<bool> IsFilteringCultures;
	bool HasLocalizationData;
};

/**
 * Implements a details view customization for UProjectPackagingSettingsCustomization objects.
 */
class FProjectPackagingSettingsCustomization
	: public IDetailCustomization
{
public:

	// IDetailCustomization interface
	virtual void CustomizeDetails( IDetailLayoutBuilder& LayoutBuilder ) override
	{
		CustomizeProjectCategory(LayoutBuilder);
		CustomizePackagingCategory(LayoutBuilder);
		CustomizeCustomBuildsCategory(LayoutBuilder);
	}

public:

	/**
	 * Creates a new instance.
	 *
	 * @return A new struct customization for play-in settings.
	 */
	static TSharedRef<IDetailCustomization> MakeInstance( )
	{
		return MakeShareable(new FProjectPackagingSettingsCustomization());
	}

protected:

	enum class EFilterCulturesChoices
	{
		/**
		 * Only show cultures that have localization data.
		 */
		OnlyLocalizedCultures,

		/**
		 * Show all available cultures.
		 */
		AllAvailableCultures
	};

	FProjectPackagingSettingsCustomization()
		: FilterCulturesChoice(EFilterCulturesChoices::AllAvailableCultures)
		, IsInBatchSelectOperation(false)
	{

	}

	/**
	 * Customizes the Project property category.
	 *
	 * @param LayoutBuilder The layout builder.
	 */
	void CustomizeProjectCategory( IDetailLayoutBuilder& LayoutBuilder )
	{
		TArray<EProjectPackagingBuildConfigurations> PackagingConfigurations = UProjectPackagingSettings::GetValidPackageConfigurations();

		TSharedRef<FPropertyRestriction> BuildConfigurationRestriction = MakeShareable(new FPropertyRestriction(LOCTEXT("ConfigurationRestrictionReason", "This configuration is not valid for this project. DebugGame configurations are not available in Content-Only or Launcher projects, and client/server configurations require the appropriate targets..")));

		const UEnum* const ProjectPackagingBuildConfigurationsEnum = StaticEnum<EProjectPackagingBuildConfigurations>();
		for (int Idx = 0; Idx < (int)EProjectPackagingBuildConfigurations::PPBC_MAX; Idx++)
		{
			EProjectPackagingBuildConfigurations Configuration = (EProjectPackagingBuildConfigurations)Idx;
			if (!PackagingConfigurations.Contains(Configuration))
			{
				BuildConfigurationRestriction->AddDisabledValue(ProjectPackagingBuildConfigurationsEnum->GetNameStringByValue(Idx));
			}
		}

		TSharedRef<IPropertyHandle> BuildConfigurationHandle = LayoutBuilder.GetProperty("BuildConfiguration");
		BuildConfigurationHandle->AddRestriction(BuildConfigurationRestriction);
	}

	/**
	 * Customizes the Packaging property category.
	 *
	 * @param LayoutBuilder The layout builder.
	 */
	void CustomizePackagingCategory( IDetailLayoutBuilder& LayoutBuilder )
	{
		IDetailCategoryBuilder& PackagingCategory = LayoutBuilder.EditCategory("Packaging");
		{
			CulturesPropertyHandle = LayoutBuilder.GetProperty("CulturesToStage", UProjectPackagingSettings::StaticClass());
			CulturesPropertyHandle->MarkHiddenByCustomization();
			CulturesPropertyArrayHandle = CulturesPropertyHandle->AsArray();

			PopulateCultureList();

			PackagingCategory.AddCustomRow(LOCTEXT("CulturesToStageLabel", "Languages To Package"), true)
				.NameContent()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Top)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						CulturesPropertyHandle->CreatePropertyNameWidget()
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush(TEXT("Icons.Error")))
						.ToolTipText(LOCTEXT("NoCulturesToStageSelectedError", "At least one language must be selected or fatal errors may occur when launching games."))
						.Visibility(this, &FProjectPackagingSettingsCustomization::HandleNoCulturesErrorIconVisibility)
					]
				]
				.ValueContent()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 4.0f)
					.VAlign(VAlign_Center)
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							// all cultures radio button
							SNew(SCheckBox)
							.IsChecked(this, &FProjectPackagingSettingsCustomization::HandleShowCulturesCheckBoxIsChecked, EFilterCulturesChoices::AllAvailableCultures)
							.OnCheckStateChanged(this, &FProjectPackagingSettingsCustomization::HandleShowCulturesCheckBoxCheckStateChanged, EFilterCulturesChoices::AllAvailableCultures)
							.Style(FAppStyle::Get(), "RadioButton")
							[
								SNew(STextBlock)
								.Text(LOCTEXT("AllCulturesCheckBoxText", "Show All"))
							]
						]

						+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						.Padding(8.0f, 0.0f, 0.0f, 0.0f)
						[
							// localized cultures radio button
							SNew(SCheckBox)
							.IsChecked(this, &FProjectPackagingSettingsCustomization::HandleShowCulturesCheckBoxIsChecked, EFilterCulturesChoices::OnlyLocalizedCultures)
							.OnCheckStateChanged(this, &FProjectPackagingSettingsCustomization::HandleShowCulturesCheckBoxCheckStateChanged, EFilterCulturesChoices::OnlyLocalizedCultures)
							.Style(FAppStyle::Get(), "RadioButton")
							[
								SNew(STextBlock)
								.Text(LOCTEXT("CookedCulturesCheckBoxText", "Show Localized"))
							]
						]
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SAssignNew(Table, SMultipleOptionTable<FCulturePtr>, &CultureList)
						.OnPreBatchSelect(this, &FProjectPackagingSettingsCustomization::OnPreBatchSelect)
						.OnPostBatchSelect(this, &FProjectPackagingSettingsCustomization::OnPostBatchSelect)
						.OnGenerateOptionWidget(this, &FProjectPackagingSettingsCustomization::GenerateWidgetForCulture)
						.OnOptionSelectionChanged(this, &FProjectPackagingSettingsCustomization::OnCultureSelectionChanged)
						.IsOptionSelected(this, &FProjectPackagingSettingsCustomization::IsCultureSelected)
						.ListHeight(100.0f)
					]
				];
		}
	}

	void PopulateCultureList()
	{
		switch(FilterCulturesChoice)
		{
		case EFilterCulturesChoices::AllAvailableCultures:
			{
				TArray<FString> CultureNames;
				FInternationalization::Get().GetCultureNames(CultureNames);
				
				CultureList.Reset();
				for(const FString& CultureName : CultureNames)
				{
					CultureList.Add(FInternationalization::Get().GetCulture(CultureName));
				}
			}
			break;

		case EFilterCulturesChoices::OnlyLocalizedCultures:
			{
				const TArray<FString> LocalizedCultureNames = FTextLocalizationManager::Get().GetLocalizedCultureNames(ELocalizationLoadFlags::Game);
				const TArray<FCultureRef> LocalizedCultureList = FInternationalization::Get().GetAvailableCultures(LocalizedCultureNames, true);

				CultureList.Reset();
				CultureList.Append(LocalizedCultureList);
			}
			break;

		default:
			checkf(false, TEXT("Unknown EFilterCulturesChoices"));
			break;
		}
	}

	EVisibility HandleNoCulturesErrorIconVisibility() const
	{
		TArray<void*> RawData;
		CulturesPropertyHandle->AccessRawData(RawData);
		TArray<FString>* RawCultureStringArray = reinterpret_cast<TArray<FString>*>(RawData[0]);

		return RawCultureStringArray->Num() ? EVisibility::Hidden : EVisibility::Visible;
	}

	ECheckBoxState HandleShowCulturesCheckBoxIsChecked( EFilterCulturesChoices Choice ) const
	{
		if (FilterCulturesChoice == Choice)
		{
			return ECheckBoxState::Checked;
		}

		return ECheckBoxState::Unchecked;
	}

	void HandleShowCulturesCheckBoxCheckStateChanged( ECheckBoxState NewState, EFilterCulturesChoices Choice )
	{
		if (NewState == ECheckBoxState::Checked)
		{
			FilterCulturesChoice = Choice;
		}

		PopulateCultureList();
		Table->RequestTableRefresh();
	}

	void AddCulture(FString CultureName)
	{
		if(!IsInBatchSelectOperation)
		{
			CulturesPropertyHandle->NotifyPreChange();
		}
		TArray<void*> RawData;
		CulturesPropertyHandle->AccessRawData(RawData);
		TArray<FString>* RawCultureStringArray = reinterpret_cast<TArray<FString>*>(RawData[0]);
		RawCultureStringArray->Add(CultureName);
		if(!IsInBatchSelectOperation)
		{
			CulturesPropertyHandle->NotifyPostChange(EPropertyChangeType::ArrayAdd);
		}
	}

	void RemoveCulture(FString CultureName)
	{
		if(!IsInBatchSelectOperation)
		{
			CulturesPropertyHandle->NotifyPreChange();
		}
		TArray<void*> RawData;
		CulturesPropertyHandle->AccessRawData(RawData);
		TArray<FString>* RawCultureStringArray = reinterpret_cast<TArray<FString>*>(RawData[0]);
		RawCultureStringArray->Remove(CultureName);
		if(!IsInBatchSelectOperation)
		{
			CulturesPropertyHandle->NotifyPostChange(EPropertyChangeType::ArrayRemove);
		}
	}

	bool IsFilteringCultures() const
	{
		return FilterCulturesChoice == EFilterCulturesChoices::OnlyLocalizedCultures;
	}

	void OnPreBatchSelect()
	{
		IsInBatchSelectOperation = true;
		CulturesPropertyHandle->NotifyPreChange();
	}

	void OnPostBatchSelect()
	{
		CulturesPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		IsInBatchSelectOperation = false;
	}

	TSharedRef<SWidget> GenerateWidgetForCulture(FCulturePtr Culture)
	{
		return SNew(SCulturePickerRowWidget, Culture, TAttribute<bool>(this, &FProjectPackagingSettingsCustomization::IsFilteringCultures));
	}

	void OnCultureSelectionChanged(bool IsSelected, FCulturePtr Culture)
	{
		if(IsSelected)
		{
			AddCulture(Culture->GetName());
		}
		else
		{
			RemoveCulture(Culture->GetName());
		}
		
	}

	bool IsCultureSelected(FCulturePtr Culture)
	{
		FString CultureName = Culture->GetName();

		uint32 ElementCount;
		CulturesPropertyArrayHandle->GetNumElements(ElementCount);
		for(uint32 Index = 0; Index < ElementCount; ++Index)
		{
			const TSharedRef<IPropertyHandle> PropertyHandle = CulturesPropertyArrayHandle->GetElement(Index);
			FString CultureNameAtIndex;
			PropertyHandle->GetValue(CultureNameAtIndex);
			if(CultureNameAtIndex == CultureName)
			{
				return true;
			}
		}

		return false;
	}


	/**
	 * Customizes the Custom Builds property category.
	 *
	 * @param LayoutBuilder The layout builder.
	 */
	void CustomizeCustomBuildsCategory( IDetailLayoutBuilder& LayoutBuilder )
	{
		TSharedPtr<IPropertyUtilities> PropertyUtilities = LayoutBuilder.GetPropertyUtilities();
		ILauncherServicesModule& LauncherServicesModule = FModuleManager::LoadModuleChecked<ILauncherServicesModule>("LauncherServices");
		ILauncherProfileManagerRef LauncherProfileManager = LauncherServicesModule.GetProfileManager();

		IDetailCategoryBuilder& CustomBuildsCategory = LayoutBuilder.EditCategory("CustomBuilds", FText::GetEmpty(), ECategoryPriority::Important);
		{
			CustomBuildsCategory.AddCustomRow(FText::GetEmpty(), false)
				.WholeRowWidget
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					[
						SNew(SHorizontalBox)

						// combo button to import from project launcher. hidden if there are no custom laucher profiles
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(8.4f)
						[
							SNew(SComboButton)
							.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
							.OnGetMenuContent_Lambda( [this, PropertyUtilities] { return CreateImportFromProjectLauncherMenu(PropertyUtilities); } )
							.Visibility_Lambda( [LauncherProfileManager] { return LauncherProfileManager->GetAllProfiles().IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible; })
							.ButtonContent()
							[
								SNew(SHorizontalBox)

								+SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								.HAlign(HAlign_Center)
								[
									SNew(SImage)
									.Image(FAppStyle::GetBrush("Launcher.TabIcon"))
								]

								+SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								.Padding(4,0,0,0)
								[
									SNew(STextBlock)
									.Text(LOCTEXT("ImportProjectLauncher", "Import From Project Launcher"))
								]
							]
						]
					]
				]
				.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda([]() 
				{
					// only visible if the user has enabled this in the experimental settings
					return GetDefault<UEditorExperimentalSettings>()->bProjectCustomBuildTools ? EVisibility::Visible : EVisibility::Collapsed; 
				})))
				;				
		}
	}

	TSharedRef<SWidget> CreateImportFromProjectLauncherMenu( TSharedPtr<IPropertyUtilities> PropertyUtilities )
	{
		const bool bCloseAfterSelection = true;
		FMenuBuilder MenuBuilder(bCloseAfterSelection, nullptr, nullptr, true);

		ILauncherServicesModule& LauncherServicesModule = FModuleManager::LoadModuleChecked<ILauncherServicesModule>("LauncherServices");
		for (const ILauncherProfilePtr& LauncherProfilePtr : LauncherServicesModule.GetProfileManager()->GetAllProfiles())
		{
			if (!LauncherProfilePtr->GetFirstBuildCookRun().IsValid())
			{
				continue;
			}

			MenuBuilder.AddMenuEntry(
				FText::FromString(LauncherProfilePtr->GetName()),
				FText::FromString(LauncherProfilePtr->GetDescription()),
				FSlateIcon(),
				FUIAction
				(
					FExecuteAction::CreateRaw( this, &FProjectPackagingSettingsCustomization::ImportFromLauncherProfile, LauncherProfilePtr, PropertyUtilities )
				),
				NAME_None,
				EUserInterfaceActionType::Button);
		}

		return MenuBuilder.MakeWidget();
	}

	void ImportFromLauncherProfile( const ILauncherProfilePtr LauncherProfilePtr, TSharedPtr<IPropertyUtilities> PropertyUtilities )
	{
		ILauncherServicesModule& LauncherServicesModule = FModuleManager::LoadModuleChecked<ILauncherServicesModule>("LauncherServices");

		const ILauncherProfileBuildCookRunPtr BuildCookRunPtr = LauncherProfilePtr->GetFirstBuildCookRun();
		if (BuildCookRunPtr == nullptr)
		{
			return;
		}

		// grab the project packaging settings
		UProjectPackagingSettings* ProjectPackagingSettings = UProjectPackagingSettings::StaticClass()->GetDefaultObject<UProjectPackagingSettings>();
		if (ProjectPackagingSettings == nullptr)
		{
			return;
		}

		// ensure the name is unique (Turnkey builds a dictionary using the Name)
		FString ProfileName = LauncherProfilePtr->GetName();
		int UniqueId = 1;
		while (ProjectPackagingSettings->ProjectCustomBuilds.ContainsByPredicate( [ProfileName](const FProjectBuildSettings& Other) { return ProfileName == Other.Name; } ) )
		{
			ProfileName = FString::Printf( TEXT("%s %d"), *LauncherProfilePtr->GetName(), UniqueId++ );
		}

		// add a new item
		FProjectBuildSettings& ProjectBuildSettings = ProjectPackagingSettings->ProjectCustomBuilds.AddDefaulted_GetRef();
		ProjectBuildSettings.Name               = ProfileName;
		ProjectBuildSettings.HelpText           = LauncherProfilePtr->GetDescription().IsEmpty() ? LauncherProfilePtr->GetName() : LauncherProfilePtr->GetDescription();
		ProjectBuildSettings.SpecificPlatforms  = BuildCookRunPtr->GetCookedPlatforms();
		if (ProjectBuildSettings.SpecificPlatforms.Num() == 0 && !BuildCookRunPtr->GetDefaultDeployPlatform().IsNone())
		{
			ProjectBuildSettings.SpecificPlatforms.Add(BuildCookRunPtr->GetDefaultDeployPlatform().ToString());
		}
		ProjectBuildSettings.BuildCookRunParams = LauncherServicesModule.GetProfileManager()->MakeBuildCookRunParamsForProjectCustomBuild(LauncherProfilePtr.ToSharedRef(), ProjectBuildSettings.SpecificPlatforms);


		// signal that the property has changed
		FNotifyHook* NotifyHook = PropertyUtilities->GetNotifyHook();
		FProperty* Property = UProjectPackagingSettings::StaticClass()->FindPropertyByName("ProjectCustomBuilds");
		if (NotifyHook != nullptr && Property != nullptr)
		{
			TArray<const UObject*> NotifyTopLevelObjects;
			NotifyTopLevelObjects.Add(ProjectPackagingSettings);

			FEditPropertyChain PropertyChain;
			PropertyChain.AddHead(Property);

			FPropertyChangedEvent ChangeEvent(Property, EPropertyChangeType::ValueSet, MakeArrayView(NotifyTopLevelObjects));
			NotifyHook->NotifyPostChange(ChangeEvent, &PropertyChain);
		}
	}


private:
	TArray<FCulturePtr> CultureList;
	TSharedPtr<IPropertyHandle> CulturesPropertyHandle;
	TSharedPtr<IPropertyHandleArray> CulturesPropertyArrayHandle;
	EFilterCulturesChoices FilterCulturesChoice;
	TSharedPtr< SMultipleOptionTable<FCulturePtr> > Table;
	bool IsInBatchSelectOperation;
};


#undef LOCTEXT_NAMESPACE
