// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Misc/PackageName.h"
#include "InputCoreTypes.h"
#include "Settings/EditorStyleSettings.h"
#include "AI/NavigationSystemBase.h"
#include "Model.h"
#include "ISourceControlModule.h"
#include "GameFramework/InputSettings.h"
#include "Settings/ContentBrowserSettings.h"
#include "Settings/LevelEditorPlaySettings.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Settings/EditorProjectSettings.h"
#include "Settings/ClassViewerSettings.h"
#include "Settings/StructViewerSettings.h"
#include "Settings/EditorExperimentalSettings.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "Settings/EditorMiscSettings.h"
#include "Settings/LevelEditorMiscSettings.h"
#include "Engine/GameViewportClient.h"
#include "EngineGlobals.h"
#include "Components/ArrowComponent.h"
#include "Components/BillboardComponent.h"
#include "UnrealWidgetFwd.h"
#include "EditorModeManager.h"
#include "UnrealEdMisc.h"
#include "CrashReporterSettings.h"
#include "AutoReimport/AutoReimportUtilities.h"
#include "Misc/ConfigCacheIni.h" // for FConfigCacheIni::GetString()
#include "Misc/AssertionMacros.h"
#include "SourceCodeNavigation.h"
#include "Interfaces/IProjectManager.h"
#include "ProjectDescriptor.h"
#include "Settings/SkeletalMeshEditorSettings.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "DesktopPlatformModule.h"
#include "InstalledPlatformInfo.h"
#include "DrawDebugHelpers.h"
#include "LevelEditorActions.h"
#include "ToolMenus.h"
#include "TransformGizmoEditorSettings.h"
#include "Subsystems/AssetEditorSubsystem.h"

#define LOCTEXT_NAMESPACE "SettingsClasses"

DEFINE_LOG_CATEGORY_STATIC(LogSettingsClasses, Log, All);

/* UContentBrowserSettings interface
 *****************************************************************************/

UContentBrowserSettings::FSettingChangedEvent UContentBrowserSettings::SettingChangedEvent;

void UContentBrowserSettings::PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent )
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName Name = (PropertyChangedEvent.Property != nullptr)
		? PropertyChangedEvent.Property->GetFName()
		: NAME_None;

	if (!FUnrealEdMisc::Get().IsDeletePreferences())
	{
		SaveConfig();
	}

	SettingChangedEvent.Broadcast(Name);
}

/* UClassViewerSettings interface
*****************************************************************************/

UClassViewerSettings::FSettingChangedEvent UClassViewerSettings::SettingChangedEvent;

UClassViewerSettings::UClassViewerSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UClassViewerSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName Name = (PropertyChangedEvent.Property != nullptr)
		? PropertyChangedEvent.Property->GetFName()
		: NAME_None;

	if (PropertyChangedEvent.MemberProperty && PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UClassViewerSettings, AllowedClasses))
	{
		FixupShortNames();
	}

	if (!FUnrealEdMisc::Get().IsDeletePreferences())
	{
		SaveConfig();
	}

	SettingChangedEvent.Broadcast(Name);
}

void UClassViewerSettings::PostInitProperties()
{
	Super::PostInitProperties();
	FixupShortNames();
}

void UClassViewerSettings::PostLoad()
{
	Super::PostLoad();
	FixupShortNames();
}

void UClassViewerSettings::FixupShortNames()
{
	for (FString& ClassName : AllowedClasses)
	{
		// Empty string may represent a new entry that's just been added in the editor
		if (ClassName.Len() && FPackageName::IsShortPackageName(ClassName))
		{
			FTopLevelAssetPath ClassPathName = UClass::TryConvertShortTypeNameToPathName<UStruct>(ClassName, ELogVerbosity::Warning, TEXT("ClassViewerSettings"));
			if (!ClassPathName.IsNull())
			{
				ClassName = ClassPathName.ToString();
			}
			else
			{
				UE_LOGF(LogSettingsClasses, Warning, "Unable to convert short class name \"%ls\" to path names for %ls.AllowedClasses. Please update it manually", *ClassName, *GetPathName());
			}
		}
	}
}

/* UStructViewerSettings interface
*****************************************************************************/

UStructViewerSettings::FSettingChangedEvent UStructViewerSettings::SettingChangedEvent;

void UStructViewerSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName Name = (PropertyChangedEvent.Property != nullptr)
		? PropertyChangedEvent.Property->GetFName()
		: NAME_None;

	if (!FUnrealEdMisc::Get().IsDeletePreferences())
	{
		SaveConfig();
	}

	SettingChangedEvent.Broadcast(Name);
}

/* USkeletalMeshEditorSettings interface
*****************************************************************************/

USkeletalMeshEditorSettings::USkeletalMeshEditorSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	AnimPreviewLightingDirection = FRotator(-45.0f, 45.0f, 0);
	AnimPreviewSkyColor = FColor::Blue;
	AnimPreviewFloorColor = FColor(51, 51, 51);
	AnimPreviewSkyBrightness = 0.2f * PI;
	AnimPreviewDirectionalColor = FColor::White;
	AnimPreviewLightBrightness = 1.0f * PI;
}

/* UEditorExperimentalSettings interface
 *****************************************************************************/

static TAutoConsoleVariable<int32> CVarEditorHDRSupport(
	TEXT("Editor.HDRSupport"),
	0,
	TEXT("Sets whether or not we should allow the editor to run on HDR monitors"),
	ECVF_Default);

static TAutoConsoleVariable<float> CVarEditorHDRNITLevel(
	TEXT("Editor.HDRNITLevel"),
	160.0f,
	TEXT("Sets The desired NIT level of the editor when running on HDR"),
	ECVF_Default);

UEditorExperimentalSettings::UEditorExperimentalSettings( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
	, bEnableAsyncTextureCompilation(false)
	, bEnableAsyncStaticMeshCompilation(false)
	, bEnableAsyncSkeletalMeshCompilation(true)	// This was false and set to True in /Engine/Config/BaseEditorPerProjectUserSettings.ini. The setting is removed from .ini so change it to default True.
	, bEnableAsyncSkinnedAssetCompilation(false)
	, bEnableAsyncSoundWaveCompilation(false)
	, bEnableAsyncGroomBindingCompilation(false)
	, bHDREditor(false)
	, HDREditorNITLevel(160.0f)
	, bUseOpenCLForConvexHullDecomp(false)
	, bAllowPotentiallyUnsafePropertyEditing(false)
	, bEnableLevelInstanceLandscapeEdit(true)
{
}

void UEditorExperimentalSettings::PostInitProperties()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// bEnableAsyncSkeletalMeshCompilation's default to True (see comment in constructor above).
	// To be backwards compatible, if a user project overrides it to False, pass on the value to bEnableAsyncSkinnedAssetCompilation.
	if (!bEnableAsyncSkeletalMeshCompilation)
	{
		UE_LOGF(LogSettingsClasses, Warning, "bEnableAsyncSkeletalMeshCompilation is deprecated and replaced with bEnableAsyncSkinnedAssetCompilation. Please update the config. Setting bEnableAsyncSkinnedAssetCompilation to False.");
		bEnableAsyncSkinnedAssetCompilation = false;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	CVarEditorHDRSupport->Set(bHDREditor ? 1 : 0, ECVF_SetByProjectSetting);
	CVarEditorHDRNITLevel->Set(HDREditorNITLevel, ECVF_SetByProjectSetting);
	Super::PostInitProperties();
}

void UEditorExperimentalSettings::PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent )
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName Name = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if (Name == GET_MEMBER_NAME_CHECKED(UEditorExperimentalSettings, ConsoleForGamepadLabels))
	{
		EKeys::SetConsoleForGamepadLabels(ConsoleForGamepadLabels);
	}
	else if (Name == GET_MEMBER_NAME_CHECKED(UEditorExperimentalSettings, bHDREditor))
	{
		CVarEditorHDRSupport->Set(bHDREditor ? 1 : 0, ECVF_SetByProjectSetting);
	}
	else if (Name == GET_MEMBER_NAME_CHECKED(UEditorExperimentalSettings, HDREditorNITLevel))
	{
		CVarEditorHDRNITLevel->Set(HDREditorNITLevel, ECVF_SetByProjectSetting);
	}

	if (!FUnrealEdMisc::Get().IsDeletePreferences())
	{
		SaveConfig();
	}

	SettingChangedEvent.Broadcast(Name);
}


/* UEditorLoadingSavingSettings interface
 *****************************************************************************/

UEditorLoadingSavingSettings::UEditorLoadingSavingSettings( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
	, bMonitorContentDirectories(true)
	, AutoReimportThreshold(3.f)
	, bAutoCreateAssets(true)
	, bAutoDeleteAssets(true)
	, bDetectChangesOnStartup(true)
	, bDeleteSourceFilesWithAssets(false)
{
	TextDiffToolPath.FilePath = TEXT("P4Merge.exe");

	FAutoReimportDirectoryConfig Default;
	Default.SourceDirectory = TEXT("/Game/");
	AutoReimportDirectorySettings.Add(Default);

	bPromptBeforeAutoImporting = true;
}

// @todo thomass: proper settings support for source control module
void UEditorLoadingSavingSettings::SccHackInitialize()
{
	bSCCUseGlobalSettings = ISourceControlModule::Get().GetUseGlobalSettings();
}

void UEditorLoadingSavingSettings::PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent )
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Use MemberProperty here so we report the correct member name for nested changes
	const FName Name = (PropertyChangedEvent.MemberProperty != nullptr) ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;

	if (Name == FName(TEXT("bSCCUseGlobalSettings")))
	{
		// unfortunately we cant use UserSettingChangedEvent here as the source control module cannot depend on the editor
		ISourceControlModule::Get().SetUseGlobalSettings(bSCCUseGlobalSettings);
	}

	if (!FUnrealEdMisc::Get().IsDeletePreferences())
	{
		SaveConfig();
	}

	SettingChangedEvent.Broadcast(Name);
}

void UEditorLoadingSavingSettings::PostInitProperties()
{
	if (AutoReimportDirectories_DEPRECATED.Num() != 0)
	{
		AutoReimportDirectorySettings.Empty();
		for (const auto& String : AutoReimportDirectories_DEPRECATED)
		{
			FAutoReimportDirectoryConfig Config;
			Config.SourceDirectory = String;
			AutoReimportDirectorySettings.Add(Config);
		}
		AutoReimportDirectories_DEPRECATED.Empty();
	}
	Super::PostInitProperties();
}

bool UEditorLoadingSavingSettings::GetAutomaticallyCheckoutOnAssetModification() const
{
	if (bAutomaticallyCheckoutOnAssetModificationOverride.IsSet())
	{
		return bAutomaticallyCheckoutOnAssetModificationOverride.GetValue();
	}
	else
	{
		return bAutomaticallyCheckoutOnAssetModification;
	}
}

void UEditorLoadingSavingSettings::SetAutomaticallyCheckoutOnAssetModificationOverride(bool InValue)
{
	bAutomaticallyCheckoutOnAssetModificationOverride = InValue;
}

void UEditorLoadingSavingSettings::ResetAutomaticallyCheckoutOnAssetModificationOverride()
{
	bAutomaticallyCheckoutOnAssetModificationOverride.Reset();
}

FAutoReimportDirectoryConfig::FParseContext::FParseContext(bool bInEnableLogging)
	: bEnableLogging(bInEnableLogging)
{
	TArray<FString> RootContentPaths;
	FPackageName::QueryRootContentPaths( RootContentPaths );
	for (FString& RootPath : RootContentPaths)
	{
		FString ContentFolder = FPaths::ConvertRelativePathToFull(FPackageName::LongPackageNameToFilename(RootPath));
		MountedPaths.Emplace( MoveTemp(ContentFolder), MoveTemp(RootPath) );
	}
}

bool FAutoReimportDirectoryConfig::ParseSourceDirectoryAndMountPoint(FString& SourceDirectory, FString& MountPoint, const FParseContext& InContext)
{
	SourceDirectory.ReplaceInline(TEXT("\\"), TEXT("/"));
	MountPoint.ReplaceInline(TEXT("\\"), TEXT("/"));

	// Check if starts with relative path.
	if (SourceDirectory.StartsWith("../"))
	{
		// Normalize. Interpret setting as a relative path from the Game User directory (Named after the Game)
		SourceDirectory = FPaths::ConvertRelativePathToFull(FPaths::ProjectUserDir() / SourceDirectory);
	}

	// Check if the source directory is actually a mount point
	const FName SourceDirectoryMountPointName = FPackageName::GetPackageMountPoint(SourceDirectory);
	if (!SourceDirectoryMountPointName.IsNone())
	{
		FString SourceDirectoryMountPoint = SourceDirectoryMountPointName.ToString();
		if (SourceDirectoryMountPoint.Len() + 2 == SourceDirectory.Len())
		{
			// Mount point name + 2 for the directory slashes is the equal, this is exactly a mount point
			MountPoint = SourceDirectory;
			SourceDirectory = FPackageName::LongPackageNameToFilename(MountPoint);
		}
		else
		{
			// Starts off with a mount point (not case sensitive)
			FString SourceMountPoint = TEXT("/") + SourceDirectoryMountPoint + TEXT("/");
			if (MountPoint.IsEmpty() || FPackageName::GetPackageMountPoint(MountPoint).IsNone())
			{
				//Set the mountPoint
				MountPoint = SourceMountPoint;
			}
			FString SourceDirectoryLeftChop = SourceDirectory.Left(SourceMountPoint.Len());
			FString SourceDirectoryRightChop = SourceDirectory.RightChop(SourceMountPoint.Len());
			// Resolve mount point on file system (possibly case sensitive, so re-use original source path)
			SourceDirectory = FPaths::ConvertRelativePathToFull(
				FPackageName::LongPackageNameToFilename(SourceDirectoryLeftChop) / SourceDirectoryRightChop);
		}
	}

	if (!SourceDirectory.IsEmpty() && !MountPoint.IsEmpty())
	{
		// We have both a source directory and a mount point. Verify that the source dir exists, and that the mount point is valid.
		if (!IFileManager::Get().DirectoryExists(*SourceDirectory))
		{
			UE_CLOGF(InContext.bEnableLogging, LogAutoReimportManager, Warning, "Unable to watch directory %ls as it doesn't exist.", *SourceDirectory);
			return false;
		}

		if (FPackageName::GetPackageMountPoint(MountPoint).IsNone())
		{
			UE_CLOGF(InContext.bEnableLogging, LogAutoReimportManager, Warning, "Unable to setup directory %ls to map to %ls, as it's not a valid mounted path. Continuing without mounted path (auto reimports will still work, but auto add won't).", *SourceDirectory, *MountPoint);
			MountPoint = FString();
			return false; // Return false when unable to determine mount point.
		}
	}
	else if(!MountPoint.IsEmpty())
	{
		// We have just a mount point - validate it, and find its source directory
		if (FPackageName::GetPackageMountPoint(MountPoint).IsNone())
		{
			UE_CLOGF(InContext.bEnableLogging, LogAutoReimportManager, Warning, "Unable to setup directory monitor for %ls, as it's not a valid mounted path.", *MountPoint);
			return false;
		}

		SourceDirectory = FPackageName::LongPackageNameToFilename(MountPoint);
	}
	else if(!SourceDirectory.IsEmpty())
	{
		// We have just a source directory - verify whether it's a mounted path, and set up the mount point if so
		if (!IFileManager::Get().DirectoryExists(*SourceDirectory))
		{
			UE_CLOGF(InContext.bEnableLogging, LogAutoReimportManager, Warning, "Unable to watch directory %ls as it doesn't exist.", *SourceDirectory);
			return false;
		}

		// Set the mounted path if necessary
		auto* Pair = InContext.MountedPaths.FindByPredicate([&](const TPair<FString, FString>& InPair){
			return SourceDirectory.StartsWith(InPair.Key);
		});
		if (Pair)
		{
			// Resolve source directory by replacing mount point with actual path
			MountPoint = Pair->Value / SourceDirectory.RightChop(Pair->Key.Len());
			MountPoint.ReplaceInline(TEXT("\\"), TEXT("/"));
		}
		else
		{
			UE_CLOGF(InContext.bEnableLogging, LogAutoReimportManager, Warning, "Unable to watch directory %ls as not associated with mounted path.", *SourceDirectory);
			return false;
		}
	}
	else
	{
		// Don't have any valid settings
		return false;
	}

	return true;
}

/* UEditorMiscSettings interface
 *****************************************************************************/

UEditorMiscSettings::UEditorMiscSettings( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{
}


/* ULevelEditorMiscSettings interface
 *****************************************************************************/

ULevelEditorMiscSettings::ULevelEditorMiscSettings( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{
	bAutoApplyLightingEnable = true;
	SectionName = TEXT("Misc");
	CategoryName = TEXT("LevelEditor");
	EditorScreenshotSaveDirectory.Path = FPaths::ScreenShotDir();
	bPromptWhenAddingToLevelBeforeCheckout = true;
	bPromptWhenAddingToLevelOutsideBounds = true;
	PercentageThresholdForPrompt = 20.0f;
	MinimumBoundsForCheckingSize = FVector(500.0f, 500.0f, 50.0f);
	bCreateNewAudioDeviceForPlayInEditor = true;
	bAvoidRelabelOnPasteSelected = false;
}

void ULevelEditorMiscSettings::PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent )
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName Name = (PropertyChangedEvent.Property != nullptr)
		? PropertyChangedEvent.Property->GetFName()
		: NAME_None;

	if (Name == FName(TEXT("bNavigationAutoUpdate")))
	{
		FWorldContext &EditorContext = GEditor->GetEditorWorldContext();
		FNavigationSystem::SetNavigationAutoUpdateEnabled(bNavigationAutoUpdate, EditorContext.World()->GetNavigationSystem());
	}

	if (!FUnrealEdMisc::Get().IsDeletePreferences())
	{
		SaveConfig();
	}
}


/* ULevelEditorPlaySettings interface
 *****************************************************************************/

ULevelEditorPlaySettings::ULevelEditorPlaySettings( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{
	ClientWindowWidth = 640;
	ClientWindowHeight = 480;
	PlayNetMode = EPlayNetMode::PIE_Standalone;
	bLaunchSeparateServer = false;
	PlayNumberOfClients = 1;
	ServerPort = 17777;
	RunUnderOneProcess = true;
	RouteGamepadToSecondWindow = false;
	BuildGameBeforeLaunch = EPlayOnBuildMode::PlayOnBuild_Default;
	LaunchConfiguration = EPlayOnLaunchConfiguration::LaunchConfig_Default;
	bAutoCompileBlueprintsOnLaunch = true;
	CenterNewWindow = false;
	NewWindowPosition = FIntPoint::NoneValue; // It will center PIE to the middle of the screen the first time it is run (until the user drag the window somewhere else)

	EnablePIEEnterAndExitSounds = false;

	bShowServerDebugDrawingByDefault = true;
	ServerDebugDrawingColorTintStrength = 0.0f;
	ServerDebugDrawingColorTint = FLinearColor(0.0f, 0.0f, 0.0f, 1.0f);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bOneHeadsetEachProcess_DEPRECATED = false;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	bHMDForPrimaryProcessOnly = true;
}

void ULevelEditorPlaySettings::PushDebugDrawingSettings()
{
#if ENABLE_DRAW_DEBUG
	extern ENGINE_API float GServerDrawDebugColorTintStrength;
	extern ENGINE_API FLinearColor GServerDrawDebugColorTint;

	GServerDrawDebugColorTintStrength = ServerDebugDrawingColorTintStrength;
	GServerDrawDebugColorTint = ServerDebugDrawingColorTint;
#endif
}

void ULevelEditorPlaySettings::ClampPIEClientValues()
{
	PlayNumberOfClients = FMath::Clamp(PlayNumberOfClients, 1, MAX_PIE_INSTANCES - 1);
	PrimaryPIEClientIndex = FMath::Clamp(PrimaryPIEClientIndex, INDEX_NONE, PlayNumberOfClients - 1);
}

void FPlayScreenResolution::PostInitProperties()
{
	ScaleFactor = 1.0f;
	LogicalHeight = Height;
	LogicalWidth = Width;

	UDeviceProfile* DeviceProfile = UDeviceProfileManager::Get().FindProfile(ProfileName, false);
	if (DeviceProfile)
	{
		GetMutableDefault<ULevelEditorPlaySettings>()->RescaleForMobilePreview(DeviceProfile, LogicalWidth, LogicalHeight, ScaleFactor);
	}
}

void ULevelEditorPlaySettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if (BuildGameBeforeLaunch != EPlayOnBuildMode::PlayOnBuild_Always && !FSourceCodeNavigation::IsCompilerAvailable() && !PLATFORM_LINUX)
	{
		BuildGameBeforeLaunch = EPlayOnBuildMode::PlayOnBuild_Never;
	}

	const FName PropertyName = PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if (PropertyName == GET_MEMBER_NAME_CHECKED(ULevelEditorPlaySettings, bOnlyLoadVisibleLevelsInPIE))
	{
		for (TObjectIterator<UWorld> WorldIt; WorldIt; ++WorldIt)
		{
			WorldIt->PopulateStreamingLevelsToConsider();
		}
	}

	if (PropertyChangedEvent.MemberProperty && PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(ULevelEditorPlaySettings, NetworkEmulationSettings))
	{
		NetworkEmulationSettings.OnPostEditChange(PropertyChangedEvent);
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(ULevelEditorPlaySettings, SimulatedDeviceMappingPolicy) || PropertyName == GET_MEMBER_NAME_CHECKED(ULevelEditorPlaySettings, bSimulateDeviceMappingPolicy))
	{
		SimulatedDeviceMappingPolicyPostEditChange.Broadcast();
	}
	
	PushDebugDrawingSettings();
	if (PropertyChangedEvent.MemberProperty && PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(ULevelEditorPlaySettings, bShowServerDebugDrawingByDefault))
	{
		// If the show option is turned on or off, force it on or off in any active PIE instances too as a QOL aid so they don't have to stop and restart PIE again for it to take effect
		for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
		{
			if ((WorldContext.WorldType == EWorldType::PIE) &&
				(WorldContext.World() != nullptr) &&
				(WorldContext.World()->GetNetMode() == NM_Client) &&
				(WorldContext.GameViewport != nullptr))
			{
				WorldContext.GameViewport->EngineShowFlags.SetServerDrawDebug(bShowServerDebugDrawingByDefault);
			}
		}
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(ULevelEditorPlaySettings, PlayNumberOfClients) || PropertyName == GET_MEMBER_NAME_CHECKED(ULevelEditorPlaySettings, PrimaryPIEClientIndex))
	{
		ClampPIEClientValues();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void ULevelEditorPlaySettings::PostInitProperties()
{
	Super::PostInitProperties();
	ClampPIEClientValues();

	// If the deprecated bOneHeadsetEachProcess setting is on, translate it to bHMDForPrimaryProcessOnly value
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (bOneHeadsetEachProcess_DEPRECATED && !GetClass()->HasAnyClassFlags(EClassFlags::CLASS_Abstract))
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		bHMDForPrimaryProcessOnly = false;
		UpdateSinglePropertyInConfigFile(GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ULevelEditorPlaySettings, bHMDForPrimaryProcessOnly)), GEditorPerProjectIni);
	};

	NewWindowWidth = FMath::Max(0, NewWindowWidth);
	NewWindowHeight = FMath::Max(0, NewWindowHeight);

	NetworkEmulationSettings.OnPostInitProperties();

#if WITH_EDITOR
	FCoreDelegates::OnSafeFrameChangedEvent.AddUObject(this, &ULevelEditorPlaySettings::UpdateCustomSafeZones);
#endif

	for (FPlayScreenResolution& Resolution : LaptopScreenResolutions)
	{
		Resolution.PostInitProperties();
	}
	for (FPlayScreenResolution& Resolution : MonitorScreenResolutions)
	{
		Resolution.PostInitProperties();
	}
	for (FPlayScreenResolution& Resolution : PhoneScreenResolutions)
	{
		Resolution.PostInitProperties();
	}
	for (FPlayScreenResolution& Resolution : TabletScreenResolutions)
	{
		Resolution.PostInitProperties();
	}
	for (FPlayScreenResolution& Resolution : TelevisionScreenResolutions)
	{
		Resolution.PostInitProperties();
	}

	PushDebugDrawingSettings();

	ClampPIEClientValues();
}

bool ULevelEditorPlaySettings::CanEditChange(const FProperty* InProperty) const
{
	const bool ParentVal = Super::CanEditChange(InProperty);
	FName PropertyName = InProperty->GetFName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(ULevelEditorPlaySettings, AdditionalServerLaunchParameters))
	{
		return ParentVal && (!RunUnderOneProcess && (PlayNetMode == EPlayNetMode::PIE_Client || bLaunchSeparateServer));
	}

	return ParentVal;
}

#if WITH_EDITOR
void ULevelEditorPlaySettings::UpdateCustomSafeZones()
{
	const bool bResetCustomSafeZone = FDisplayMetrics::GetDebugTitleSafeZoneRatio() < 1.f;

	// Prefer to use r.DebugSafeZone.TitleRatio if it is set
	if (bResetCustomSafeZone)
	{
		PIESafeZoneOverride = FMargin();
	}
	else
	{
		PIESafeZoneOverride = CalculateCustomUnsafeZones(CustomUnsafeZoneStarts, CustomUnsafeZoneDimensions, DeviceToEmulate, FVector2D(NewWindowWidth, NewWindowHeight));
	}

	if (FSlateApplication::IsInitialized())
	{
		if (bResetCustomSafeZone)
		{
			FSlateApplication::Get().ResetCustomSafeZone();
		}
		else
		{
			FMargin SafeZoneRatio = PIESafeZoneOverride;
			SafeZoneRatio.Left /= (NewWindowWidth / 2.0f);
			SafeZoneRatio.Right /= (NewWindowWidth / 2.0f);
			SafeZoneRatio.Bottom /= (NewWindowHeight / 2.0f);
			SafeZoneRatio.Top /= (NewWindowHeight / 2.0f);
			FSlateApplication::Get().OnDebugSafeZoneChanged.Broadcast(SafeZoneRatio, true);
		}
	}
}
#endif

FMargin ULevelEditorPlaySettings::CalculateCustomUnsafeZones(TArray<FVector2D>& CustomSafeZoneStarts, TArray<FVector2D>& CustomSafeZoneDimensions, FString& DeviceType, FVector2D PreviewSize)
{
	double PreviewHeight = PreviewSize.Y;
	double PreviewWidth = PreviewSize.X;
	bool bPreviewIsPortrait = PreviewHeight > PreviewWidth;
	FMargin CustomSafeZoneOverride = FMargin();
	CustomSafeZoneStarts.Empty();
	CustomSafeZoneDimensions.Empty();
	UDeviceProfile* DeviceProfile = UDeviceProfileManager::Get().FindProfile(DeviceType, false);
	if (DeviceProfile)
	{
		FString CVarUnsafeZonesString;
		if (DeviceProfile->GetConsolidatedCVarValue(TEXT("r.CustomUnsafeZones"), CVarUnsafeZonesString))
		{
			TArray<FString> UnsafeZones;
			CVarUnsafeZonesString.ParseIntoArray(UnsafeZones, TEXT(";"), true);
			for (FString UnsafeZone : UnsafeZones)
			{
				FString Orientation;
				FString FixedState;
				FString TempString;
				FVector2D Start;
				FVector2D Dimensions;
				bool bAdjustsToDeviceRotation = false;
				UnsafeZone.Split(TEXT("("), &TempString, &UnsafeZone);
				Orientation = UnsafeZone.Left(1);
				UnsafeZone.Split(TEXT("["), &TempString, &UnsafeZone);
				if (TempString.Contains(TEXT("free")))
				{
					bAdjustsToDeviceRotation = true;
				}

				UnsafeZone.Split(TEXT(","), &TempString, &UnsafeZone);
				Start.X = FCString::Atof(*TempString);
				UnsafeZone.Split(TEXT("]"), &TempString, &UnsafeZone);
				Start.Y = FCString::Atof(*TempString);
				UnsafeZone.Split(TEXT("["), &TempString, &UnsafeZone);
				UnsafeZone.Split(TEXT(","), &TempString, &UnsafeZone);
				Dimensions.X = FCString::Atof(*TempString);
				Dimensions.Y = FCString::Atof(*UnsafeZone);

				bool bShouldScale = false;
				float CVarMobileContentScaleFactor = FCString::Atof(*DeviceProfile->GetCVarValue(TEXT("r.MobileContentScaleFactor")));
				if (CVarMobileContentScaleFactor != 0)
				{
					bShouldScale = true;
				}
				else
				{
					if (DeviceProfile->GetConsolidatedCVarValue(TEXT("r.MobileContentScaleFactor"), CVarMobileContentScaleFactor, true))
					{
						bShouldScale = true;
					}
				}
				if (bShouldScale)
				{
					Start *= CVarMobileContentScaleFactor;
					Dimensions *= CVarMobileContentScaleFactor;
				}

				if (!bAdjustsToDeviceRotation && ((Orientation.Contains(TEXT("L")) && bPreviewIsPortrait) ||
					(Orientation.Contains(TEXT("P")) && !bPreviewIsPortrait)))
				{
					double Placeholder = Start.X;
					Start.X = Start.Y;
					Start.Y = Placeholder;

					Placeholder = Dimensions.X;
					Dimensions.X = Dimensions.Y;
					Dimensions.Y = Placeholder;
				}

				if (Start.X < 0)
				{
					Start.X += PreviewWidth;
				}
				if (Start.Y < 0)
				{
					Start.Y += PreviewHeight;
				}

				// Remove any overdraw if this is an unsafe zone that could adjust with device rotation
				if (bAdjustsToDeviceRotation)
				{
					if (Dimensions.X + Start.X > PreviewWidth)
					{
						Dimensions.X = PreviewWidth - Start.X;
					}
					if (Dimensions.Y + Start.Y > PreviewHeight)
					{
						Dimensions.Y = PreviewHeight - Start.Y;
					}
				}

				CustomSafeZoneStarts.Add(Start);
				CustomSafeZoneDimensions.Add(Dimensions);

				if (Start.X + Dimensions.X == PreviewWidth && !FMath::IsNearlyZero(Start.X))
				{
					CustomSafeZoneOverride.Right = FMath::Max(CustomSafeZoneOverride.Right, Dimensions.X);
				}
				else if (Start.X == 0.0f && Start.X + Dimensions.X != PreviewWidth)
				{
					CustomSafeZoneOverride.Left = FMath::Max(CustomSafeZoneOverride.Left, Dimensions.X);
				}
				if (Start.Y + Dimensions.Y == PreviewHeight && !FMath::IsNearlyZero(Start.Y))
				{
					CustomSafeZoneOverride.Bottom = FMath::Max(CustomSafeZoneOverride.Bottom, Dimensions.Y);
				}
				else if (Start.Y == 0.0f && Start.Y + Dimensions.Y != PreviewHeight)
				{
					CustomSafeZoneOverride.Top = FMath::Max(CustomSafeZoneOverride.Top, Dimensions.Y);
				}
			}
		}
	}
	return CustomSafeZoneOverride;
}

FMargin ULevelEditorPlaySettings::FlipCustomUnsafeZones(TArray<FVector2D>& CustomSafeZoneStarts, TArray<FVector2D>& CustomSafeZoneDimensions, FString& DeviceType, FVector2D PreviewSize)
{
	FMargin CustomSafeZoneOverride = CalculateCustomUnsafeZones(CustomSafeZoneStarts, CustomSafeZoneDimensions, DeviceType, PreviewSize);
	for (FVector2D& CustomSafeZoneStart : CustomSafeZoneStarts)
	{
		CustomSafeZoneStart.X = PreviewSize.X - CustomSafeZoneStart.X;
	}
	for (FVector2D& CustomSafeZoneDimension : CustomSafeZoneDimensions)
	{
		CustomSafeZoneDimension.X *= -1.0f;
	}
	float Placeholder = CustomSafeZoneOverride.Left;
	CustomSafeZoneOverride.Left = CustomSafeZoneOverride.Right;
	CustomSafeZoneOverride.Right = Placeholder;
	return CustomSafeZoneOverride;
}

void ULevelEditorPlaySettings::RescaleForMobilePreview(const UDeviceProfile* DeviceProfile, int32 &PreviewWidth, int32 &PreviewHeight, float &ScaleFactor)
{
	bool bShouldScale = false;
	float CVarMobileContentScaleFactor = 0.0f;
	const FString ScaleFactorString = DeviceProfile->GetCVarValue(TEXT("r.MobileContentScaleFactor"));
	if (ScaleFactorString != FString())
	{
		CVarMobileContentScaleFactor = FCString::Atof(*ScaleFactorString);

		if (!FMath::IsNearlyEqual(CVarMobileContentScaleFactor, 0.0f))
		{
			bShouldScale = true;
			ScaleFactor = CVarMobileContentScaleFactor;
		}
	}
	else
	{
		TMap<FString, FString> ParentValues;
		DeviceProfile->GatherParentCVarInformationRecursively(ParentValues);
		const FString* ParentScaleFactorPtr = ParentValues.Find(TEXT("r.MobileContentScaleFactor"));
		if (ParentScaleFactorPtr != nullptr)
		{
			FString CompleteString = *ParentScaleFactorPtr;
			FString DiscardString;
			FString ValueString;
			CompleteString.Split(TEXT("="), &DiscardString, &ValueString);
			CVarMobileContentScaleFactor = FCString::Atof(*ValueString);
			if (!FMath::IsNearlyEqual(CVarMobileContentScaleFactor, 0.0f))
			{
				bShouldScale = true;
				ScaleFactor = CVarMobileContentScaleFactor;
			}
		}
	}
	if (bShouldScale)
	{
		if (DeviceProfile->DeviceType == TEXT("Android"))
		{
			const float OriginalPreviewWidth = PreviewWidth;
			const float OriginalPreviewHeight = PreviewHeight;
			float TempPreviewHeight = 0.0f;
			float TempPreviewWidth = 0.0f;
			// Portrait
			if (PreviewHeight > PreviewWidth)
			{
				TempPreviewHeight = 1280 * ScaleFactor;
			
			}
			// Landscape
			else
			{
				TempPreviewHeight = 720 * ScaleFactor;
			}
			TempPreviewWidth = TempPreviewHeight * OriginalPreviewWidth / OriginalPreviewHeight + 0.5f;
			PreviewHeight = (int32)FMath::GridSnap(TempPreviewHeight, 8.0f);
			PreviewWidth = (int32)FMath::GridSnap(TempPreviewWidth, 8.0f);
		}
		else
		{
			PreviewWidth *= ScaleFactor;
			PreviewHeight *= ScaleFactor;
		}

	}
}

void ULevelEditorPlaySettings::RegisterCommonResolutionsMenu()
{
	UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(GetCommonResolutionsMenuName());
	check(Menu);

	FToolMenuSection& ResolutionsSection = Menu->AddSection("CommonResolutions");
	const ULevelEditorPlaySettings* PlaySettings = GetDefault<ULevelEditorPlaySettings>();

	auto AddSubMenuToSection = [&ResolutionsSection](const FString& SectionName, const FText& SubMenuTitle, const TArray<FPlayScreenResolution>& Resolutions)
	{
		ResolutionsSection.AddSubMenu(
			FName(*SectionName),
			SubMenuTitle,
			FText(),
			FNewToolMenuChoice(FNewToolMenuDelegate::CreateStatic(&ULevelEditorPlaySettings::AddScreenResolutionSection, &Resolutions, SectionName))
		);
	};

	AddSubMenuToSection(FString("Phones"), LOCTEXT("CommonPhonesSectionHeader", "Phones"), PlaySettings->PhoneScreenResolutions);
	AddSubMenuToSection(FString("Tablets"), LOCTEXT("CommonTabletsSectionHeader", "Tablets"), PlaySettings->TabletScreenResolutions);
	AddSubMenuToSection(FString("Laptops"), LOCTEXT("CommonLaptopsSectionHeader", "Laptops"), PlaySettings->LaptopScreenResolutions);
	AddSubMenuToSection(FString("Monitors"), LOCTEXT("CommonMonitorsSectionHeader", "Monitors"), PlaySettings->MonitorScreenResolutions);
	AddSubMenuToSection(FString("Televisions"), LOCTEXT("CommonTelevesionsSectionHeader", "Televisions"), PlaySettings->TelevisionScreenResolutions);
}

FName ULevelEditorPlaySettings::GetCommonResolutionsMenuName()
{
	const static FName MenuName("EditorSettingsViewer.LevelEditorPlaySettings");
	return MenuName;
}

void ULevelEditorPlaySettings::AddScreenResolutionSection(UToolMenu* InToolMenu, const TArray<FPlayScreenResolution>* Resolutions, const FString SectionName)
{
	check(Resolutions);
	for (const FPlayScreenResolution& Resolution : *Resolutions)
	{
		FInternationalization& I18N = FInternationalization::Get();

		FFormatNamedArguments Args;
		Args.Add(TEXT("Width"), FText::AsNumber(Resolution.Width, NULL, I18N.GetInvariantCulture()));
		Args.Add(TEXT("Height"), FText::AsNumber(Resolution.Height, NULL, I18N.GetInvariantCulture()));
		Args.Add(TEXT("AspectRatio"), FText::FromString(Resolution.AspectRatio));

		FText ToolTip;
		if (!Resolution.ProfileName.IsEmpty())
		{
			Args.Add(TEXT("LogicalWidth"), FText::AsNumber(Resolution.LogicalWidth, NULL, I18N.GetInvariantCulture()));
			Args.Add(TEXT("LogicalHeight"), FText::AsNumber(Resolution.LogicalHeight, NULL, I18N.GetInvariantCulture()));
			Args.Add(TEXT("ScaleFactor"), FText::AsNumber(Resolution.ScaleFactor, NULL, I18N.GetInvariantCulture()));
			ToolTip = FText::Format(LOCTEXT("CommonResolutionFormatWithContentScale", "{Width} x {Height} ({AspectRatio}, Logical Res: {LogicalWidth} x {LogicalHeight}, Content Scale: {ScaleFactor})"), Args);
		}
		else
		{
			ToolTip = FText::Format(LOCTEXT("CommonResolutionFormat", "{Width} x {Height} ({AspectRatio})"), Args);
		}

		UCommonResolutionMenuContext* Context = InToolMenu->FindContext<UCommonResolutionMenuContext>();
		check(Context);
		check(Context->GetUIActionFromLevelPlaySettings.IsBound());

		FUIAction Action = Context->GetUIActionFromLevelPlaySettings.Execute(Resolution);
		InToolMenu->AddMenuEntry(FName(*SectionName), FToolMenuEntry::InitMenuEntry(FName(*Resolution.Description), FText::FromString(Resolution.Description), ToolTip, FSlateIcon(), Action));
	}
}

/* ULevelEditorViewportSettings interface
 *****************************************************************************/

ULevelEditorViewportSettings::ULevelEditorViewportSettings( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{
	MinimumOrthographicZoom = 250.0f;
	bLevelStreamingVolumePrevis = false;
	BillboardScale = 1.0f;
	MeasuringToolUnits = MeasureUnits_Centimeters;
	bAllowArcballRotate = false;
	bAllowScreenRotate = false;
	bShowActorEditorContext = true;
	bAllowEditWidgetAxisDisplay = true;
	MouseSensitivty = .2f;
	bUseLegacyCameraMovementNotifications = false;
	// Set a default preview mesh
	PreviewMeshes.Add(FSoftObjectPath("/Engine/EditorMeshes/ColorCalibrator/SM_ColorCalibrator.SM_ColorCalibrator"));
	LastInViewportMenuLocation = FVector2D(EForceInit::ForceInitToZero);
	PreserveNonUniformScale = true;
	bUseLODViewLocking = false;
}

void ULevelEditorViewportSettings::PostInitProperties()
{
	Super::PostInitProperties();
	UBillboardComponent::SetEditorScale(BillboardScale);
	UArrowComponent::SetEditorScale(BillboardScale);

	// Make sure the mouse sensitivity is not set below a valid value somehow. This can cause weird viewport interactions.
	MouseSensitivty = FMath::Max(MouseSensitivty, .01f);
}

void ULevelEditorViewportSettings::PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent )
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName Name = (PropertyChangedEvent.Property != nullptr)
		? PropertyChangedEvent.Property->GetFName()
		: NAME_None;

	if (Name == GET_MEMBER_NAME_CHECKED(ULevelEditorViewportSettings, bAllowTranslateRotateZWidget))
	{
		if (bAllowTranslateRotateZWidget)
		{
			GLevelEditorModeTools().SetWidgetMode(UE::Widget::WM_TranslateRotateZ);
		}
		else if (GLevelEditorModeTools().GetWidgetMode() == UE::Widget::WM_TranslateRotateZ)
		{
			GLevelEditorModeTools().SetWidgetMode(UE::Widget::WM_Translate);
		}
	}
	else if (Name == GET_MEMBER_NAME_CHECKED(ULevelEditorViewportSettings, bHighlightWithBrackets))
	{
		GEngine->SetSelectedMaterialColor(bHighlightWithBrackets
			? FLinearColor::Black
			: GetDefault<UEditorStyleSettings>()->SelectionColor);
	}
	else if (Name == GET_MEMBER_NAME_CHECKED(ULevelEditorViewportSettings, SelectionHighlightIntensity))
	{
		GEngine->SelectionHighlightIntensity = SelectionHighlightIntensity;
	}
	else if (Name == GET_MEMBER_NAME_CHECKED(ULevelEditorViewportSettings, BSPSelectionHighlightIntensity))
	{
		GEngine->BSPSelectionHighlightIntensity = BSPSelectionHighlightIntensity;
	}
	else if ((Name == FName(TEXT("UserDefinedPosGridSizes"))) || (Name == FName(TEXT("UserDefinedRotGridSizes"))) || (Name == FName(TEXT("ScalingGridSizes"))) || (Name == FName(TEXT("GridIntervals")))) //@TODO: This should use GET_MEMBER_NAME_CHECKED
	{
		const float MinGridSize = (Name == FName(TEXT("GridIntervals"))) ? 4.0f : 0.0001f; //@TODO: This should use GET_MEMBER_NAME_CHECKED
		TArray<float>* ArrayRef = nullptr;
		int32* IndexRef = nullptr;

		if (Name == GET_MEMBER_NAME_CHECKED(ULevelEditorViewportSettings, ScalingGridSizes))
		{
			ArrayRef = &(ScalingGridSizes);
			IndexRef = &(CurrentScalingGridSize);
		}

		if (ArrayRef && IndexRef)
		{
			// Don't allow an empty array of grid sizes
			if (ArrayRef->Num() == 0)
			{
				ArrayRef->Add(MinGridSize);
			}

			// Don't allow negative numbers
			for (int32 SizeIdx = 0; SizeIdx < ArrayRef->Num(); ++SizeIdx)
			{
				if ((*ArrayRef)[SizeIdx] < MinGridSize)
				{
					(*ArrayRef)[SizeIdx] = MinGridSize;
				}
			}
		}
	}
	else if (Name == GET_MEMBER_NAME_CHECKED(ULevelEditorViewportSettings, bUsePowerOf2SnapSize))
	{
		const float BSPSnapSize = bUsePowerOf2SnapSize ? 128.0f : 100.0f;
		UModel::SetGlobalBSPTexelScale(BSPSnapSize);
	}
	else if (Name == GET_MEMBER_NAME_CHECKED(ULevelEditorViewportSettings, BillboardScale))
	{
		UBillboardComponent::SetEditorScale(BillboardScale);
		UArrowComponent::SetEditorScale(BillboardScale);
	}
	else if (Name == GET_MEMBER_NAME_CHECKED(ULevelEditorViewportSettings, bEnableLayerSnap))
	{
		ULevelEditor2DSettings* Settings2D = GetMutableDefault<ULevelEditor2DSettings>();
		if (bEnableLayerSnap && !Settings2D->bEnableSnapLayers)
		{
			Settings2D->bEnableSnapLayers = true;
		}
	}
	else if (Name == GET_MEMBER_NAME_CHECKED(ULevelEditorViewportSettings, bUseLODViewLocking))
	{
		GEditor->RedrawLevelEditingViewports();
	}

	if (!FUnrealEdMisc::Get().IsDeletePreferences())
	{
		SaveConfig();
	}

	GEditor->RedrawAllViewports();

	SettingChangedEvent.Broadcast(Name);
}


/* UCrashReporterSettings interface
*****************************************************************************/
UCrashReporterSettings::UCrashReporterSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#undef LOCTEXT_NAMESPACE
