// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderLibraryChunkDataGenerator.h"

#include "CookOnTheSide/CookLog.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "Interfaces/ITargetPlatform.h"
#include "IPlatformFileSandboxWrapper.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "ShaderCodeLibrary.h"

class FName;

FShaderLibraryChunkDataGenerator::FShaderLibraryChunkDataGenerator(UCookOnTheFlyServer& InCOTFS, const ITargetPlatform* TargetPlatform)
	: COTFS(InCOTFS)
{
	// Find out if this platform requires stable shader keys, by reading the platform setting file.
	bOptedOut = false;
	PlatformNameUsedForIni = TargetPlatform->IniPlatformName();

	bool bChunkShadersWhenCookingDLC = false;

	FConfigFile PlatformIniFile;
	FConfigCacheIni::LoadLocalIniFile(PlatformIniFile, TEXT("Engine"), true, *PlatformNameUsedForIni);
	PlatformIniFile.GetBool(TEXT("DevOptions.Shaders"), TEXT("bDoNotChunkShaderLib"), bOptedOut);
	PlatformIniFile.GetBool(TEXT("DevOptions.Shaders"), TEXT("bChunkShaderWhenCookingDLC"), bChunkShadersWhenCookingDLC);

	// Make sure that chunking DLC plugins is only allowed when -OverrideShaderLibraryName is set. As of Oct 2023 chunking DLC shader libraries is used as a part of bespoke build process
	// that splits a monolithic cook into a number of "fake" DLC cooks, but overrides the library name to match the game in the end. This flow is not intended to be supported for true DLC plugins
	// and as such shader code library is not atm prepared to look for chunked parts when trying to open a DLC plugin library
	if (bChunkShadersWhenCookingDLC)
	{
		FString OverrideShaderLibraryName;
		if (!FParse::Value(FCommandLine::Get(), TEXT("OverrideShaderLibraryName="), OverrideShaderLibraryName))
		{
			UE_LOGF(LogCook, Error, "bChunkShaderWhenCookingDLC is true, but -OverrideShaderLibraryName is not set. Chunk a real DLC plugin content isn't currently supported. bChunkShaderWhenCookingDLC is only usable in conjuction with -OverrideShaderLibraryName for a bespoke build process.");
		}
	}

	// Disable chunking for DLC - this causes problems as the main game can be optionally (for faster iteration) cooked with -fastcook. Fastcook disables chunking,
	// so the game has no idea about ChunkIDs and cannot find DLC's chunked libs. If DLC lib is monolithic, both monolithic and chunked games will try to open it.
	if (!bChunkShadersWhenCookingDLC && COTFS.IsCookingDLC())
	{
		bOptedOut = true;
	}
}

void FShaderLibraryChunkDataGenerator::BeginGenerateChunkDataFiles(FBeginGenerateContext& Context)
{
	const ITargetPlatform* TargetPlatform = Context.GetTargetPlatform();
	FString ShaderCodeDir;
	FString MetaDataPath;
	COTFS.GetShaderLibraryPaths(TargetPlatform, ShaderCodeDir, MetaDataPath, true /* bUseProjectDirForDLC */);
	FShaderLibraryCooker::DeleteShaderLibraryChunks(TargetPlatform, ShaderCodeDir, MetaDataPath);
}

void FShaderLibraryChunkDataGenerator::GenerateChunkDataFiles(const int32 InChunkId, const TSet<FName>& InPackagesInChunk,
	const ITargetPlatform* TargetPlatform, FSandboxPlatformFile* InSandboxFile, TArray<FString>& OutChunkFilenames)
{
	if (!bOptedOut && InPackagesInChunk.Num() > 0)
	{
		checkf(PlatformNameUsedForIni == TargetPlatform->IniPlatformName(),
			TEXT("Mismatch between platform names in shaderlib chunk generator. Ini settings might have been applied incorrectly."));

		// get the sandbox content directory here, to relieve shaderlib from including the wrapper
		FString ShaderCodeDir;
		FString MetaDataPath;
		COTFS.GetShaderLibraryPaths(TargetPlatform, ShaderCodeDir, MetaDataPath, true /* bUseProjectDirForDLC */);

		bool bHasData;
		UE::Cook::EPackagingSystem PackagingSystem = COTFS.IsUsingIoStore() ? UE::Cook::EPackagingSystem::IoStore
			: UE::Cook::EPackagingSystem::Loose;
		FShaderLibraryCooker::SaveShaderLibraryChunk(InChunkId, InPackagesInChunk, TargetPlatform, ShaderCodeDir,
			MetaDataPath, PackagingSystem, OutChunkFilenames, bHasData);
	}
}
	