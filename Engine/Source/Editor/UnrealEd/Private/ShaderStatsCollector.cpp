// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderStatsCollector.h"
#include "AnalyticsEventAttribute.h"
#include "CookOnTheSide/CookLog.h"
#include "Serialization/CompactBinaryWriter.h"
#include "ShaderStats.h"

static TAutoConsoleVariable<int> CVarShaderCompilerStatsPrintoutInterval(
	TEXT("r.ShaderCompiler.StatsPrintoutInterval"),
	180,
	TEXT("Minimum interval (in seconds) between printing out debugging stats (by default, no closer than once per minute)."),
	ECVF_Default
);

static TAutoConsoleVariable<bool> CVarShaderCompilerPrintIndividualProcessStats(
	TEXT("r.ShaderCompiler.PrintIndividualProcessStats"),
	false,
	TEXT("If true, in a multiprocess cook, stats will be printed for each individual process (as well as aggregated across all)."),
	ECVF_Default
);

class FShaderStatsReporter : public FTickableEditorObject, FTickableCookObject
{
public:
	FShaderStatsReporter() : FTickableEditorObject(), FTickableCookObject() {}
	virtual void Tick(float DeltaTime) override;
	virtual void TickCook(float DeltaTime, bool bCookComplete) override;
	virtual bool IsTickable() const override;
	virtual TStatId GetStatId() const override;
	
	inline void SetAggregator(FShaderStatsAggregator* InAggregator)
	{
		Aggregator = InAggregator;
	}

private:
	friend class FShaderStatsFunctions;
	void InternalTick(float DeltaTime, bool bForceLog = false);
	void LogStats();
	void AggregateStats(FShaderCompilerStats& OutStats) const;

	FShaderStatsAggregator* Aggregator;
	double LastLogTime = 0.0f;
	double LastSynchTime = 0.0f;
};

static FShaderStatsReporter GShaderStatsReporter;

void FShaderStatsFunctions::GatherShaderAnalytics(TArray<FAnalyticsEventAttribute>& Attributes)
{
	FShaderCompilerStats AggregatedCompilerStats;
	GShaderCompilingManager->GetLocalStats(AggregatedCompilerStats);
	GShaderStatsReporter.AggregateStats(AggregatedCompilerStats);
	AggregatedCompilerStats.GatherAnalytics(TEXT("Shaders_"), Attributes);

	int32 TotalShaderTypePermutations = 0;
	for (TLinkedList<FShaderType*>::TIterator ShaderTypeIt(FShaderType::GetTypeList()); ShaderTypeIt; ShaderTypeIt.Next())
	{
		TotalShaderTypePermutations += ShaderTypeIt->GetPermutationCount();
	}
	Attributes.Emplace(TEXT("Shaders_NumShaderTypePermutations"), TotalShaderTypePermutations);

	int32 TotalVFTypePermutations = 0;
	for (TLinkedList<FVertexFactoryType*>::TIterator VFTypeIt(FVertexFactoryType::GetTypeList()); VFTypeIt; VFTypeIt.Next())
	{
		TotalVFTypePermutations += 1;
	}
	Attributes.Emplace(TEXT("Shaders_NumVertexFactoryTypePermutations"), TotalVFTypePermutations);
}

void FShaderStatsFunctions::WriteShaderStats()
{
	FShaderCompilerStats AggregatedCompilerStats;
	GShaderCompilingManager->GetLocalStats(AggregatedCompilerStats);
	GShaderStatsReporter.AggregateStats(AggregatedCompilerStats);
	AggregatedCompilerStats.WriteStats();
}

void FShaderStatsReporter::Tick(float DeltaTime)
{
	InternalTick(DeltaTime);
}

void FShaderStatsReporter::TickCook(float DeltaTime, bool bCookComplete)
{
	// force log output at completion of cook
	InternalTick(DeltaTime, bCookComplete);
}

void FShaderStatsReporter::InternalTick(float DeltaTime, bool bForceLog)
{
	const int PrintInterval = CVarShaderCompilerStatsPrintoutInterval.GetValueOnAnyThread();
	if (bForceLog || ((PrintInterval > 0) && (FPlatformTime::Seconds() - LastLogTime) >= PrintInterval))
	{
		LastLogTime = FPlatformTime::Seconds();
		LogStats();
	}
}

bool FShaderStatsReporter::IsTickable() const
{
	// the stats should be reported (and so stats reporter tickable) in the case where we're the cook director process in
	// a multiprocess cook, running a single process cook/editor, or if the "print individual process stats" cvar is enabled.
	// the Aggregator will have Mode==Director the first case, and be unset in the second.
	return !Aggregator || (Aggregator->Mode == FShaderStatsAggregator::EMode::Director) || CVarShaderCompilerPrintIndividualProcessStats.GetValueOnAnyThread();
}

TStatId FShaderStatsReporter::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FShaderStatsReporter, STATGROUP_Tickables);
}

void FShaderStatsReporter::LogStats()
{
	// only ever called on director process in MP cook, or single process in normal cook/editor
	FShaderCompilerStats LocalStats;
	GShaderCompilingManager->GetLocalStats(LocalStats);

	if (Aggregator && Aggregator->Mode == FShaderStatsAggregator::EMode::Director && CVarShaderCompilerPrintIndividualProcessStats.GetValueOnAnyThread())
	{
		// if we're the director in a MP cook, and the "individual process stats" are requested, print them before aggregating and printing the aggregated results
		LocalStats.WriteStatSummary();
	}
	
	AggregateStats(LocalStats);

	static uint32 PreviousTotalShadersCompiled = 0;
	const uint32 CurrentTotalShadersCompiled = LocalStats.GetTotalShadersCompiled();
	if (CurrentTotalShadersCompiled > PreviousTotalShadersCompiled)
	{
		LocalStats.WriteStatSummary();
		PreviousTotalShadersCompiled = CurrentTotalShadersCompiled;
	}
}

void FShaderStatsReporter::AggregateStats(FShaderCompilerStats& OutStats) const
{
	if (Aggregator && Aggregator->Mode == FShaderStatsAggregator::EMode::Director)
	{
		OutStats.SetMultiProcessAggregated();
		for (FShaderCompilerStats& WorkerStats : Aggregator->WorkerCompilerStats)
		{
			OutStats.Aggregate(WorkerStats);
		}
	}
}

FGuid FShaderStatsAggregator::MessageType(TEXT("F8D2E291D9A44F999D79E3839091BF25"));

FShaderStatsAggregator::FShaderStatsAggregator(EMode Mode) : Mode(Mode)
{
	GShaderStatsReporter.SetAggregator(this);
}

void FShaderStatsAggregator::ClientTick(UE::Cook::FMPCollectorClientTickContext& Context)
{
	const int PrintInterval = CVarShaderCompilerStatsPrintoutInterval.GetValueOnAnyThread();
	// synch stats twice as often as they are printed to ensure each stats print on the cook director
	// has at least one update from all workers.
	const int SynchInterval = PrintInterval / 2;

	if (Context.IsFlush() || ((SynchInterval > 0) && (FPlatformTime::Seconds() - LastSynchTime) >= PrintInterval))
	{
		LastSynchTime = FPlatformTime::Seconds();
		FCbWriter Writer;
		Writer.BeginObject();

		FShaderCompilerStats LocalStats;
		GShaderCompilingManager->GetLocalStats(LocalStats);
		LocalStats.WriteToCompactBinary(Writer);

		Writer.EndObject();
		UE_LOGF(LogCook, Verbose, "Synching shader compile stats to cook director; message size %lld bytes", Writer.GetSaveSize());

		Context.AddMessage(Writer.Save().AsObject());
	}
}

void FShaderStatsAggregator::ServerReceiveMessage(UE::Cook::FMPCollectorServerMessageContext& Context, FCbObjectView Message)
{
	int32 WorkerId = Context.GetProfileId();

	if (WorkerCompilerStats.Num() < (WorkerId + 1))
	{
		WorkerCompilerStats.SetNum(WorkerId + 1);
	}
	WorkerCompilerStats[WorkerId].ReadFromCompactBinary(Message);
}
