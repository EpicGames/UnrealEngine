// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Cooker/MPCollector.h"
#include "ShaderCompiler.h"
#include "TickableEditorObject.h"

class FShaderStatsAggregator : public UE::Cook::IMPCollector{
public:
	enum class EMode
	{
		Worker,
		Director,
	};
	FShaderStatsAggregator(EMode Mode);
	virtual FGuid GetMessageType() const override { return MessageType; }
	virtual const TCHAR* GetDebugName() const override { return TEXT("FShaderStatsAggregator"); }

	virtual void ClientTick(UE::Cook::FMPCollectorClientTickContext& Context) override;
	virtual void ServerReceiveMessage(UE::Cook::FMPCollectorServerMessageContext& Context, FCbObjectView Message) override;

	static FGuid MessageType;

private:
	friend class FShaderStatsFunctions;
	friend class FShaderStatsReporter;
	EMode Mode;
	TArray<FShaderCompilerStats> WorkerCompilerStats;
	float LastSynchTime = 0.0f;
};
