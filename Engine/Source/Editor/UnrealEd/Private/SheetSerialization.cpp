// Copyright Epic Games, Inc. All Rights Reserved.

#include "SheetSerialization.h"

#include "DataTableEditorUtils.h"
#include "Editor.h"
#include "Misc/OutputDeviceNull.h"
#include "UObject/StrProperty.h"

constexpr TCHAR TabCharacter = TCHAR('\t');
constexpr TCHAR NewlineCharacter = TCHAR('\n');
constexpr TCHAR CarriageReturnCharacter = TCHAR('\r');
constexpr TCHAR DoubleQuoteCharacter = TCHAR('"');

int32 FSheetData::GetRowCount() const
{
	if (Cells.Num() == 0)
	{
		return 0;
	}

	return Cells.Num() / ColumnCount;
}

FString& FSheetData::GetStringForCell(const int32 Column, const int32 Row)
{
	const int32 Index = Column + Row * ColumnCount;
	const int32 ReserveSize = (Row + 1) * ColumnCount;

	// Ensure there are sufficient cells
	if (Cells.Num() <= ReserveSize)
	{
		Cells.AddDefaulted(ReserveSize - Cells.Num());
	}

	return Cells[Index];
}

int32 FSheetData::GetObjectRowCount(const int32 StartingColumn, const int32 StartingRow, const int32 MaxRows)
{
	check(ColumnCount);
	const int32 EndRow = StartingRow + MaxRows;

	for (int32 Row = StartingRow + 1; Row < EndRow; ++Row)
	{
		if (!GetStringForCell(StartingColumn, Row).IsEmpty())
		{
			return Row - StartingRow;
		}
	}

	return EndRow - StartingRow;
}

bool FSheetData::FillFromSheet(const FString& InSheetText, FOutputDevice* ErrorText)
{
	FOutputDeviceNull NullErrorText;
	if (!ErrorText)
	{
		ErrorText = &NullErrorText;
	}

	if (InSheetText.IsEmpty())
	{
		ErrorText->Log(TEXT("Sheet data is empty."));
		return false;
	}

	uint32 QuoteCount = 0;
	FString CellBuffer;

	for (const TCHAR& Character : InSheetText)
	{
		// Only count quotes if we're at the start of the cell or are already tracking
		if (Character == DoubleQuoteCharacter && (CellBuffer.IsEmpty() || QuoteCount > 0))
		{
			++QuoteCount;
		}

		// Only end the cell if we're at even or no quotes
		if ((Character == TabCharacter || Character == NewlineCharacter) && QuoteCount % 2 == 0)
		{
			// Strip the now trailing carriage return if it was part of a line terminator
			if (Character == NewlineCharacter && !CellBuffer.IsEmpty() && CellBuffer[CellBuffer.Len() - 1] == CarriageReturnCharacter)
			{
				CellBuffer.RemoveAt(CellBuffer.Len() - 1, 1, EAllowShrinking::No);
			}

			QuoteCount = 0;
			Cells.Add(USheetSerializationSubSystem::UnEscapeIfNecessary(CellBuffer));
			CellBuffer.Reset();

			if (Character == NewlineCharacter)
			{
				if (ColumnCount == 0)
				{
					ColumnCount = Cells.Num();
				}
				else if (Cells.Num() % ColumnCount != 0)
				{
					ErrorText->Log(TEXT("Invalid sheet data format. Rows much each have the same number of columns."));
					return false;
				}
			}
		}
		else
		{
			CellBuffer.AppendChar(Character);
		}
	}

	// Add the final cell
	if (!CellBuffer.IsEmpty())
	{
		Cells.Add(USheetSerializationSubSystem::UnEscapeIfNecessary(CellBuffer));
	}

	if (ColumnCount == 0)
	{
		ColumnCount = Cells.Num();
	}
	else if (Cells.Num() < ColumnCount)
	{
		ErrorText->Log(TEXT("Invalid sheet data format. Rows must have the appropriate number of columns."));
		return false;
	}

	return true;
}

USheetSerializationSubSystem* USheetSerializationSubSystem::Get()
{
	check(GEditor);
	return GEditor->GetEditorSubsystem<USheetSerializationSubSystem>();
}

void USheetSerializationSubSystem::Initialize(FSubsystemCollectionBase& Collection)
{
	HeaderGetters.Add(FStructProperty::StaticClass()->GetName(), FFillHeadersFromProperty::CreateUObject(this, &USheetSerializationSubSystem::FillHeadersFromStructProperty));
	HeaderGetters.Add(FArrayProperty::StaticClass()->GetName(), FFillHeadersFromProperty::CreateUObject(this, &USheetSerializationSubSystem::FillHeadersFromArrayProperty));
	HeaderGetters.Add(FSetProperty::StaticClass()->GetName(), FFillHeadersFromProperty::CreateUObject(this, &USheetSerializationSubSystem::FillHeadersFromSetProperty));
	HeaderGetters.Add(FMapProperty::StaticClass()->GetName(), FFillHeadersFromProperty::CreateUObject(this, &USheetSerializationSubSystem::FillHeadersFromMapProperty));

	CellGetters.Add(FStructProperty::StaticClass()->GetName(), FFillCellsFromProperty::CreateUObject(this, &USheetSerializationSubSystem::FillCellsFromStructProperty));
	CellGetters.Add(FArrayProperty::StaticClass()->GetName(), FFillCellsFromProperty::CreateUObject(this, &USheetSerializationSubSystem::FillCellsFromArrayProperty));
	CellGetters.Add(FSetProperty::StaticClass()->GetName(), FFillCellsFromProperty::CreateUObject(this, &USheetSerializationSubSystem::FillCellsFromSetProperty));
	CellGetters.Add(FMapProperty::StaticClass()->GetName(), FFillCellsFromProperty::CreateUObject(this, &USheetSerializationSubSystem::FillCellsFromMapProperty));
	CellGetters.Add(FStrProperty::StaticClass()->GetName(), FFillCellsFromProperty::CreateUObject(this, &USheetSerializationSubSystem::FillCellsFromStringProperty));

	PropertySetters.Add(FStructProperty::StaticClass()->GetName(), FSetPropertyFromCells::CreateUObject(this, &USheetSerializationSubSystem::SetStructPropertyFromCells));
	PropertySetters.Add(FArrayProperty::StaticClass()->GetName(), FSetPropertyFromCells::CreateUObject(this, &USheetSerializationSubSystem::SetArrayPropertyFromCells));
	PropertySetters.Add(FSetProperty::StaticClass()->GetName(), FSetPropertyFromCells::CreateUObject(this, &USheetSerializationSubSystem::SetSetPropertyFromCells));
	PropertySetters.Add(FMapProperty::StaticClass()->GetName(), FSetPropertyFromCells::CreateUObject(this, &USheetSerializationSubSystem::SetMapPropertyFromCells));
	PropertySetters.Add(FStrProperty::StaticClass()->GetName(), FSetPropertyFromCells::CreateUObject(this, &USheetSerializationSubSystem::SetStringPropertyFromCells));
}

void USheetSerializationSubSystem::Deinitialize()
{
	Super::Deinitialize();
}

void USheetSerializationSubSystem::RegisterHeaderGetter(const FString& PropertyClassName, const FFillHeadersFromProperty& GetHeaderFromProperty)
{
	HeaderGetters.Add(PropertyClassName, GetHeaderFromProperty);
}

void USheetSerializationSubSystem::RegisterCellGetter(const FString& PropertyClassName, const FFillCellsFromProperty& GetCellsFromProperty)
{
	CellGetters.Add(PropertyClassName, GetCellsFromProperty);
}

void USheetSerializationSubSystem::RegisterPropertySetter(const FString& PropertyClassName, const FSetPropertyFromCells& SetPropertyFromCells)
{
	PropertySetters.Add(PropertyClassName, SetPropertyFromCells);
}

void USheetSerializationSubSystem::UnregisterHeaderGetter(const FString& PropertyClassName)
{
	HeaderGetters.Remove(PropertyClassName);
}

void USheetSerializationSubSystem::UnregisterCellGetter(const FString& PropertyClassName)
{
	CellGetters.Remove(PropertyClassName);
}

void USheetSerializationSubSystem::UnregisterPropertySetter(const FString& PropertyClassName)
{
	PropertySetters.Remove(PropertyClassName);
}

FString USheetSerializationSubSystem::ExportForSheet(TArray<FDataTableRowData>& Rows, UDataTable* DataTable, const bool IncludeHeaders /*= false*/)
{
	FSheetData SheetData;

	// Get the headers to at least figure out how many columns each should use
	SheetData.Cells.Add(TEXT("Name"));
	FillHeadersFromStruct(SheetData, DataTable->RowStruct.Get());
	SheetData.ColumnCount = SheetData.Cells.Num();

	FIntPoint StartingPosition = FIntPoint{ 1, IncludeHeaders ? 1 : 0 };

	if (!IncludeHeaders)
	{
		SheetData.Cells.Reset();
	}

	int32 FinalRowCount = StartingPosition.Y;
	for (const FDataTableRowData& Row : Rows)
	{
		FString& CellString = SheetData.GetStringForCell(0, StartingPosition.Y);
		CellString = Row.Name.ToString();

		const int32 RowCount = FillCellsFromObject(SheetData, StartingPosition, Row.DataPtr, DataTable, DataTable->RowStruct.Get());
		FinalRowCount += RowCount;
		StartingPosition.Y += RowCount;
	}

	int32 FinalLength = 0;

	// Escape any cells that need escaping
	for (FString& Cell : SheetData.Cells)
	{
		FinalLength += EscapeIfNecessary(Cell).Len();
	}

	FString Result;

	// Reserve enough space for the text + cell-dividing characters
	Result.Reserve(FinalLength + SheetData.Cells.Num() + SheetData.GetRowCount());

	for (int32 Y = 0; Y < FinalRowCount; ++Y)
	{
		for (int32 X = 0; X < SheetData.ColumnCount; ++X)
		{
			if (X > 0)
			{
				Result.AppendChar(TabCharacter);
			}

			Result.Append(SheetData.Cells[X + Y * SheetData.ColumnCount]);
		}

		Result.Append(LINE_TERMINATOR);
	}

	return Result;
}

bool USheetSerializationSubSystem::ImportFromSheet(const FString& SheetText, UDataTable* DataTable, FOutputDevice* ErrorText)
{
	const FString NameHeader = TEXT("Name");

	FOutputDeviceNull NullErrorText;
	if (!ErrorText)
	{
		ErrorText = &NullErrorText;
	}

	FSheetData SheetData;

	// Get the headers to figure out how many columns each should use
	SheetData.Cells.Add(NameHeader);
	FillHeadersFromStruct(SheetData, DataTable->RowStruct.Get());
	SheetData.ColumnCount = SheetData.Cells.Num();
	SheetData.Cells.Reset();

	if (!SheetData.FillFromSheet(SheetText, ErrorText))
	{
		return false;
	}

	DataTable->EmptyTable();

	const int32 SheetRowCount = SheetData.GetRowCount();

	// Skip the first row if it contains headers (we'll just check that it starts with "Name")
	int32 SheetRow = SheetData.Cells[0] == NameHeader ? 1 : 0;
	while (SheetRow < SheetRowCount)
	{
		const int32 ObjectRowCount = SheetData.GetObjectRowCount(0, SheetRow, SheetRowCount - SheetRow);
		const FName RowName = FName{ SheetData.GetStringForCell(0, SheetRow) };

		// Allocate a temp to store data
		uint8* TempRowData = (uint8*)FMemory::Malloc(DataTable->RowStruct->GetStructureSize());
		DataTable->RowStruct->InitializeStruct(TempRowData);

		SetObjectFromCells(SheetData, FIntPoint{ 1, SheetRow }, ObjectRowCount, TempRowData, DataTable, DataTable->RowStruct.Get());

		// Add the row to the data table
		DataTable->AddRow(RowName, TempRowData, DataTable->RowStruct.Get());
		DataTable->RowStruct->DestroyStruct(TempRowData);
		FMemory::Free(TempRowData);

		SheetRow += ObjectRowCount;
	}

	return true;
}

bool USheetSerializationSubSystem::ImportRowFromSheet(const FString& SheetText, UDataTable* DataTable, FName& OutRowName, uint8* RowPtr, FOutputDevice* ErrorText)
{
	const FString NameHeader = TEXT("Name");

	FOutputDeviceNull NullErrorText;
	if (!ErrorText)
	{
		ErrorText = &NullErrorText;
	}

	FSheetData SheetData;

	// Get the headers to figure out how many columns each should use
	SheetData.Cells.Add(NameHeader);
	FillHeadersFromStruct(SheetData, DataTable->RowStruct.Get());
	SheetData.ColumnCount = SheetData.Cells.Num();
	SheetData.Cells.Reset();

	if (!SheetData.FillFromSheet(SheetText, ErrorText))
	{
		return false;
	}

	// Skip the first row if it contains headers (we'll just check that it starts with "Name")
	const int32 SheetRow = SheetData.Cells[0] == NameHeader ? 1 : 0;
	const int32 ObjectRowCount = SheetData.GetObjectRowCount(0, SheetRow, SheetData.GetRowCount());

	OutRowName = FName{ SheetData.GetStringForCell(0, SheetRow) };
	SetObjectFromCells(SheetData, FIntPoint{ 1, SheetRow }, ObjectRowCount, RowPtr, DataTable, DataTable->RowStruct.Get());

	return true;
}

FString& USheetSerializationSubSystem::EscapeIfNecessary(FString& CellData, bool* OutModified)
{
	bool Modified = false;
	if (CellData.Contains(TEXT("\n")) || CellData.Contains(TEXT("\t")))
	{
		CellData.ReplaceInline(TEXT("\""), TEXT("\"\""));

		CellData.Reserve(CellData.Len() + 2);
		CellData.InsertAt(0, DoubleQuoteCharacter);
		CellData.AppendChar(DoubleQuoteCharacter);

		Modified = true;
	}

	if (OutModified)
	{
		*OutModified = Modified;
	}

	return CellData;
}

FString& USheetSerializationSubSystem::UnEscapeIfNecessary(FString& CellData, bool* OutModified)
{
	bool Modified = false;
	if (CellData.Contains(TEXT("\n")) || CellData.Contains(TEXT("\t")))
	{
		bool Trimmed;
		CellData.TrimCharInline('"', &Trimmed);

		Modified = CellData.ReplaceInline(TEXT("\"\""), TEXT("\"")) || Trimmed;
	}

	if (OutModified)
	{
		*OutModified = Modified;
	}

	return CellData;
}

int32 USheetSerializationSubSystem::FillHeadersFromStruct(FSheetData& SheetData, const UStruct* InStruct, const FString* ParentName)
{
	int32 ColumnCount = 0;

	for (TFieldIterator<FProperty> It{ InStruct }; It; ++It)
	{
		ColumnCount += FillHeadersFromProperty(SheetData, *It, nullptr, ParentName);
	}

	return ColumnCount;
}

int32 USheetSerializationSubSystem::FillHeadersFromProperty(FSheetData& SheetData, const FProperty* InProperty, const FProperty* ParentProperty, const FString* ParentName)
{
	if (!InProperty || !InProperty->ShouldPort(PPF_Copy))
	{
		return 0;
	}

	FString PropName;
	if (ParentProperty && ParentProperty->GetName() == InProperty->GetName())
	{
		PropName = *ParentName;
	}
	else if (ParentName)
	{
		PropName = FString::Printf(TEXT("%s.%s"), **ParentName, *InProperty->GetDisplayNameText().ToString());
	}
	else
	{
		PropName = InProperty->GetDisplayNameText().ToString();
	}

	int32 ColumnCount;
	const FFillHeadersFromProperty* PropertyHeaderGetter = HeaderGetters.Find(InProperty->GetClass()->GetName());
	if (PropertyHeaderGetter && PropertyHeaderGetter->IsBound())
	{
		ColumnCount = PropertyHeaderGetter->Execute(SheetData, InProperty, PropName);
	}
	else
	{
		ColumnCount = 1;
		SheetData.Cells.Add(PropName);
	}

	SheetData.ColumnsPerProperty.Add(InProperty, ColumnCount);
	return ColumnCount;
}

int32 USheetSerializationSubSystem::FillCellsFromObject(FSheetData& SheetData, const FIntPoint StartingPosition, void* InObject, UObject* Parent, const UStruct* RowStruct)
{
	int32 RowCount = 1;

	int32 CurrentColumnPosition = StartingPosition.X;
	for (TFieldIterator<FProperty> It{ RowStruct }; It; ++It)
	{
		if (!It->ShouldPort(PPF_Copy))
		{
			continue;
		}

		const int32 PropertyHeight = FillCellsFromProperty(SheetData, FIntPoint{ CurrentColumnPosition, StartingPosition.Y }, InObject, Parent, *It);
		RowCount = FMath::Max(RowCount, PropertyHeight);

		// Move to the next column after the columns added by this struct
		CurrentColumnPosition += SheetData.ColumnsPerProperty[*It];
	}

	return RowCount;
}

int32 USheetSerializationSubSystem::FillCellsFromProperty(FSheetData& SheetData, const FIntPoint StartingPosition, void* InObject, UObject* Parent, const FProperty* InProperty)
{
	int32 RowCount = 1;

	FFillCellsFromProperty* PropertyCellGetter = CellGetters.Find(InProperty->GetClass()->GetName());
	if (PropertyCellGetter && PropertyCellGetter->IsBound())
	{
		RowCount = PropertyCellGetter->Execute(SheetData, StartingPosition, InObject, Parent, InProperty);
	}
	else
	{
		FString& CellString = SheetData.GetStringForCell(StartingPosition.X, StartingPosition.Y);
		InProperty->ExportText_InContainer(0, CellString, InObject, InObject, Parent, PPF_Delimited | PPF_Copy);
	}

	return RowCount;
}

void USheetSerializationSubSystem::SetObjectFromCells(FSheetData& SheetData, const FIntPoint StartingPosition, int32 ObjectRowCount, void* InObject, UObject* Parent, const UStruct* RowStruct)
{
	int32 CurrentColumnPosition = StartingPosition.X;
	for (TFieldIterator<FProperty> It{ RowStruct }; It; ++It)
	{
		if (!It->ShouldPort(PPF_Copy))
		{
			continue;
		}

		SetPropertyFromCells(SheetData, FIntPoint{ CurrentColumnPosition, StartingPosition.Y }, ObjectRowCount, InObject, Parent, *It);

		// Move to the next column after the columns added by this struct
		CurrentColumnPosition += SheetData.ColumnsPerProperty[*It];
	}
}

void USheetSerializationSubSystem::SetPropertyFromCells(FSheetData& SheetData, const FIntPoint StartingPosition, int32 ObjectRowCount, void* InObject, UObject* Parent, const FProperty* InProperty)
{
	FSetPropertyFromCells* PropertySetter = PropertySetters.Find(InProperty->GetClass()->GetName());
	if (PropertySetter && PropertySetter->IsBound())
	{
		PropertySetter->Execute(SheetData, StartingPosition, ObjectRowCount, InObject, Parent, InProperty);
	}
	else
	{
		FString& CellString = SheetData.GetStringForCell(StartingPosition.X, StartingPosition.Y);
		InProperty->ImportText_InContainer(*CellString, InObject, Parent, PPF_Delimited | PPF_Copy);
	}
}

int32 USheetSerializationSubSystem::FillHeadersFromStructProperty(FSheetData& SheetData, const FProperty* InProperty, const FString& DefaultPropName)
{
	int32 Count = 0;

	if (const FStructProperty* StructProp = CastField<FStructProperty>(InProperty))
	{
		FString PropName = DefaultPropName;
		if (InProperty->ArrayDim > 1)
		{
			++Count;
			SheetData.Cells.Add(FString::Printf(TEXT("%s.Index"), *PropName));
			PropName = FString::Printf(TEXT("%s.Value"), *PropName);
		}

		Count += FillHeadersFromStruct(SheetData, StructProp->Struct.Get(), &PropName);
	}

	return Count;
}

int32 USheetSerializationSubSystem::FillHeadersFromArrayProperty(FSheetData& SheetData, const FProperty* InProperty, const FString& DefaultPropName)
{
	int32 Count = 0;

	if (const FArrayProperty* ArrayProp = CastField<FArrayProperty>(InProperty))
	{
		++Count;
		SheetData.Cells.Add(FString::Printf(TEXT("%s.Index"), *DefaultPropName));

		FString ValuePropName = FString::Printf(TEXT("%s.Value"), *DefaultPropName);
		if (ArrayProp->Inner->IsA<FStructProperty>())
		{
			Count += FillHeadersFromProperty(SheetData, ArrayProp->Inner, InProperty, &ValuePropName);
		}
		else
		{
			++Count;
			SheetData.Cells.Add(ValuePropName);
			SheetData.ColumnsPerProperty.Add(ArrayProp->Inner, 1);
		}
	}

	return Count;
}

int32 USheetSerializationSubSystem::FillHeadersFromSetProperty(FSheetData& SheetData, const FProperty* InProperty, const FString& DefaultPropName)
{
	int32 Count = 0;

	if (const FSetProperty* SetProp = CastField<FSetProperty>(InProperty))
	{
		++Count;
		SheetData.Cells.Add(FString::Printf(TEXT("%s.Index"), *DefaultPropName));

		FString ValuePropName = FString::Printf(TEXT("%s.Value"), *DefaultPropName);
		if (SetProp->ElementProp->IsA<FStructProperty>())
		{
			Count += FillHeadersFromProperty(SheetData, SetProp->ElementProp, InProperty, &ValuePropName);
		}
		else
		{
			++Count;
			SheetData.Cells.Add(ValuePropName);
			SheetData.ColumnsPerProperty.Add(SetProp->ElementProp, 1);
		}
	}

	return Count;
}

int32 USheetSerializationSubSystem::FillHeadersFromMapProperty(FSheetData& SheetData, const FProperty* InProperty, const FString& DefaultPropName)
{
	int32 Count = 0;

	if (const FMapProperty* MapProp = CastField<FMapProperty>(InProperty))
	{
		const FString KeyPropName = FString::Printf(TEXT("%s.Key"), *DefaultPropName);
		if (MapProp->KeyProp->IsA<FStructProperty>())
		{
			Count += FillHeadersFromProperty(SheetData, MapProp->KeyProp, InProperty, &KeyPropName);
		}
		else
		{
			++Count;
			SheetData.Cells.Add(KeyPropName);
			SheetData.ColumnsPerProperty.Add(MapProp->KeyProp, 1);
		}

		const FString ValuePropName = FString::Printf(TEXT("%s.Value"), *DefaultPropName);
		if (MapProp->ValueProp->IsA<FStructProperty>())
		{
			Count += FillHeadersFromProperty(SheetData, MapProp->ValueProp, InProperty, &ValuePropName);
		}
		else
		{
			++Count;
			SheetData.Cells.Add(ValuePropName);
			SheetData.ColumnsPerProperty.Add(MapProp->ValueProp, 1);
		}
	}

	return Count;
}

int32 USheetSerializationSubSystem::FillCellsFromStructProperty(FSheetData& SheetData, const FIntPoint StartingPosition, void* InObject, UObject* Parent, const FProperty* InProperty)
{
	int32 RowCount = 1;

	if (const FStructProperty* StructProp = CastField<FStructProperty>(InProperty))
	{
		RowCount = 0;
		const int32 ArrayAdjustment = InProperty->ArrayDim > 1 ? 1 : 0;

		for (int32 Index = 0; Index < InProperty->ArrayDim; ++Index)
		{
			if (ArrayAdjustment)
			{
				// Add index to top-left
				FString& CellString = SheetData.GetStringForCell(StartingPosition.X, StartingPosition.Y + RowCount);
				CellString = FString::FromInt(Index);
			}

			RowCount += FillCellsFromObject(SheetData, FIntPoint{ StartingPosition.X + ArrayAdjustment, StartingPosition.Y + RowCount }, InProperty->ContainerPtrToValuePtr<void>(InObject, Index), Parent, StructProp->Struct.Get());
		}
	}

	return RowCount;
}

int32 USheetSerializationSubSystem::FillCellsFromArrayProperty(FSheetData& SheetData, const FIntPoint StartingPosition, void* InObject, UObject* Parent, const FProperty* InProperty)
{
	int32 RowCount = 1;

	if (const FArrayProperty* ArrayProp = CastField<FArrayProperty>(InProperty))
	{
		FScriptArrayHelper ArrayHelper{ ArrayProp, InProperty->ContainerPtrToValuePtr<void>(InObject, 0) };
		if (ArrayHelper.Num() <= 0)
		{
			return RowCount;
		}

		RowCount = 0;
		uint8* PropData = ArrayHelper.GetRawPtr(0);

		int32 Index = 0;
		for (int32 Count = ArrayHelper.Num(); Count; PropData += ArrayProp->Inner->GetElementSize(), ++Index)
		{
			if (!ArrayHelper.IsValidIndex(Index))
			{
				continue;
			}

			// Add index to top-left
			FString& IndexCell = SheetData.GetStringForCell(StartingPosition.X, StartingPosition.Y + RowCount);
			IndexCell = FString::FromInt(Index);

			RowCount += FillCellsFromProperty(SheetData, FIntPoint{ StartingPosition.X + 1, StartingPosition.Y + RowCount }, PropData, Parent, ArrayProp->Inner);

			--Count;
		}
	}

	return RowCount;
}

int32 USheetSerializationSubSystem::FillCellsFromSetProperty(FSheetData& SheetData, const FIntPoint StartingPosition, void* InObject, UObject* Parent, const FProperty* InProperty)
{
	int32 RowCount = 1;

	if (const FSetProperty* SetProp = CastField<FSetProperty>(InProperty))
	{
		FScriptSetHelper SetHelper{ SetProp, InProperty->ContainerPtrToValuePtr<void>(InObject, 0) };
		if (SetHelper.Num() <= 0)
		{
			return RowCount;
		}

		RowCount = 0;
		uint8* PropData = SetHelper.GetElementPtr(0);

		int32 Index = 0;
		for (int32 Count = SetHelper.Num(); Count; PropData += SetProp->SetLayout.Size, ++Index)
		{
			if (!SetHelper.IsValidIndex(Index))
			{
				continue;
			}

			// Add index to top-left
			FString& IndexCell = SheetData.GetStringForCell(StartingPosition.X, StartingPosition.Y + RowCount);
			IndexCell = FString::FromInt(Index);

			RowCount += FillCellsFromProperty(SheetData, FIntPoint{ StartingPosition.X + 1, StartingPosition.Y + RowCount }, PropData, Parent, SetProp->ElementProp);

			--Count;
		}
	}

	return RowCount;
}

int32 USheetSerializationSubSystem::FillCellsFromMapProperty(FSheetData& SheetData, const FIntPoint StartingPosition, void* InObject, UObject* Parent, const FProperty* InProperty)
{
	int32 RowCount = 1;

	if (const FMapProperty* MapProp = CastField<FMapProperty>(InProperty))
	{
		FScriptMapHelper MapHelper{ MapProp, InProperty->ContainerPtrToValuePtr<void>(InObject, 0) };
		if (MapHelper.Num() <= 0)
		{
			return RowCount;
		}

		RowCount = 0;
		uint8* PropData = MapHelper.GetPairPtr(0);

		int32 Index = 0;
		for (int32 Count = MapHelper.Num(); Count; PropData += MapProp->MapLayout.SetLayout.Size, ++Index)
		{
			if (!MapHelper.IsValidIndex(Index))
			{
				continue;
			}

			const int32 KeyHeight = FillCellsFromProperty(SheetData, FIntPoint{ StartingPosition.X, StartingPosition.Y + RowCount }, PropData, Parent, MapProp->KeyProp);
			const int32 ValueHeight = FillCellsFromProperty(SheetData, FIntPoint{ StartingPosition.X + SheetData.ColumnsPerProperty[MapProp->KeyProp], StartingPosition.Y + RowCount }, PropData, Parent, MapProp->ValueProp);

			RowCount += FMath::Max(KeyHeight, ValueHeight);

			--Count;
		}
	}

	return RowCount;
}

int32 USheetSerializationSubSystem::FillCellsFromStringProperty(FSheetData& SheetData, const FIntPoint StartingPosition, void* InObject, UObject* Parent, const FProperty* InProperty)
{
	if (const FStrProperty* StrProp = CastField<FStrProperty>(InProperty))
	{
		FString& CellString = SheetData.GetStringForCell(StartingPosition.X, StartingPosition.Y);
		CellString = StrProp->GetPropertyValue_InContainer(InObject);
	}

	return 1;
}

void USheetSerializationSubSystem::SetStructPropertyFromCells(FSheetData& SheetData, const FIntPoint StartingPosition, int32 ObjectRowCount, void* InObject, UObject* Parent, const FProperty* InProperty)
{
	if (const FStructProperty* StructProp = CastField<FStructProperty>(InProperty))
	{
		int32 RowCount = 0;
		const int32 ArrayAdjustment = InProperty->ArrayDim > 1 ? 1 : 0;

		for (int32 Index = 0; Index < InProperty->ArrayDim; ++Index)
		{
			const int32 ChildRowCount = ArrayAdjustment ? SheetData.GetObjectRowCount(StartingPosition.X, StartingPosition.Y + RowCount, ObjectRowCount - RowCount) : ObjectRowCount;
			SetObjectFromCells(SheetData, FIntPoint{ StartingPosition.X + ArrayAdjustment, StartingPosition.Y + RowCount }, ChildRowCount, InProperty->ContainerPtrToValuePtr<void>(InObject, Index), Parent, StructProp->Struct.Get());
			RowCount += ChildRowCount;
		}
	}
}

void USheetSerializationSubSystem::SetArrayPropertyFromCells(FSheetData& SheetData, const FIntPoint StartingPosition, int32 ObjectRowCount, void* InObject, UObject* Parent, const FProperty* InProperty)
{
	if (const FArrayProperty* ArrayProp = CastField<FArrayProperty>(InProperty))
	{
		FScriptArrayHelper ArrayHelper{ ArrayProp, InProperty->ContainerPtrToValuePtr<void>(InObject, 0) };
		ArrayHelper.EmptyValues();

		int32 RowCount = 0;
		while (ObjectRowCount - RowCount > 0)
		{
			const FString& IndexCell = SheetData.GetStringForCell(StartingPosition.X, StartingPosition.Y + RowCount);
			if (IndexCell.IsEmpty())
			{
				break;
			}

			const int32 ChildRowCount = SheetData.GetObjectRowCount(StartingPosition.X, StartingPosition.Y + RowCount, ObjectRowCount - RowCount);

			int32 Index = 0;
			if (!LexTryParseString(Index, *IndexCell) || Index < 0)
			{
				RowCount += ChildRowCount;
				continue;
			}

			ArrayHelper.ExpandForIndex(Index);
			uint8* PropData = ArrayHelper.GetRawPtr(Index);

			SetPropertyFromCells(SheetData, FIntPoint{ StartingPosition.X + 1, StartingPosition.Y + RowCount }, ChildRowCount, PropData, Parent, ArrayProp->Inner);
			RowCount += ChildRowCount;
		}
	}
}

void USheetSerializationSubSystem::SetSetPropertyFromCells(FSheetData& SheetData, const FIntPoint StartingPosition, int32 ObjectRowCount, void* InObject, UObject* Parent, const FProperty* InProperty)
{
	if (const FSetProperty* SetProp = CastField<FSetProperty>(InProperty))
	{
		FScriptSetHelper SetHelper{ SetProp, InProperty->ContainerPtrToValuePtr<void>(InObject, 0) };
		SetHelper.EmptyElements();

		uint8* TempElementStorage = (uint8*)FMemory::Malloc(SetProp->ElementProp->GetElementSize());

		bool bSuccess = false;
		ON_SCOPE_EXIT
		{
			FMemory::Free(TempElementStorage);

			// If we are returning because of an error, remove any already-added elements from the map before returning
			// to ensure we're not left with a partial state.
			if (!bSuccess)
			{
				SetHelper.EmptyElements();
			}
		};

		int32 RowCount = 0;
		while (ObjectRowCount - RowCount > 0)
		{
			// The actual contents of the index cell doesn't matter in a Set.
			// It just needs to be there to understand where each element begins.
			const FString& IndexCell = SheetData.GetStringForCell(StartingPosition.X, StartingPosition.Y + RowCount);
			if (IndexCell.IsEmpty())
			{
				break;
			}

			SetProp->ElementProp->InitializeValue(TempElementStorage);
			ON_SCOPE_EXIT
			{
				SetProp->ElementProp->DestroyValue(TempElementStorage);
			};

			const int32 ChildRowCount = SheetData.GetObjectRowCount(StartingPosition.X, StartingPosition.Y + RowCount, ObjectRowCount - RowCount);

			SetPropertyFromCells(SheetData, FIntPoint{ StartingPosition.X + 1, StartingPosition.Y + RowCount }, ChildRowCount, TempElementStorage, Parent, SetProp->ElementProp);

			if (SetHelper.FindElementIndex(TempElementStorage) == INDEX_NONE)
			{
				const int32 NewElementIndex = SetHelper.AddDefaultValue_Invalid_NeedsRehash();
				uint8* NewElementPtr = SetHelper.GetElementPtr(NewElementIndex);

				// Copy over imported key and value from temporary storage
				SetProp->ElementProp->CopyCompleteValue_InContainer(NewElementPtr, TempElementStorage);
			}

			RowCount += ChildRowCount;
		}

		SetHelper.Rehash();
		bSuccess = true;
	}
}

void USheetSerializationSubSystem::SetMapPropertyFromCells(FSheetData& SheetData, const FIntPoint StartingPosition, int32 ObjectRowCount, void* InObject, UObject* Parent, const FProperty* InProperty)
{
	if (const FMapProperty* MapProp = CastField<FMapProperty>(InProperty))
	{
		FScriptMapHelper MapHelper{ MapProp, InProperty->ContainerPtrToValuePtr<void>(InObject, 0) };
		MapHelper.EmptyValues();

		uint8* TempPairStorage = (uint8*)FMemory::Malloc(MapProp->MapLayout.ValueOffset + MapProp->ValueProp->GetElementSize());

		bool bSuccess = false;
		ON_SCOPE_EXIT
		{
			FMemory::Free(TempPairStorage);

			// If we are returning because of an error, remove any already-added elements from the map before returning
			// to ensure we're not left with a partial state.
			if (!bSuccess)
			{
				MapHelper.EmptyValues();
			}
		};

		int32 RowCount = 0;
		while (ObjectRowCount - RowCount > 0)
		{
			if (SheetData.GetStringForCell(StartingPosition.X, StartingPosition.Y + RowCount).IsEmpty())
			{
				break;
			}

			MapProp->KeyProp->InitializeValue(TempPairStorage);
			MapProp->ValueProp->InitializeValue(TempPairStorage + MapProp->MapLayout.ValueOffset);
			ON_SCOPE_EXIT
			{
				MapProp->ValueProp->DestroyValue(TempPairStorage + MapProp->MapLayout.ValueOffset);
				MapProp->KeyProp->DestroyValue(TempPairStorage);
			};

			const int32 ChildRowCount = SheetData.GetObjectRowCount(StartingPosition.X, StartingPosition.Y + RowCount, ObjectRowCount - RowCount);
			SetPropertyFromCells(SheetData, FIntPoint{ StartingPosition.X, StartingPosition.Y + RowCount }, ChildRowCount, TempPairStorage, Parent, MapProp->KeyProp);

			if (MapHelper.FindMapIndexWithKey(TempPairStorage) == INDEX_NONE)
			{
				SetPropertyFromCells(SheetData, FIntPoint{ StartingPosition.X + SheetData.ColumnsPerProperty[MapProp->KeyProp], StartingPosition.Y + RowCount }, ChildRowCount, TempPairStorage, Parent, MapProp->ValueProp);

				const int32 Index = MapHelper.AddDefaultValue_Invalid_NeedsRehash();
				uint8* PairPtr = MapHelper.GetPairPtr(Index);

				// Copy over imported key and value from temporary storage
				MapProp->KeyProp->CopyCompleteValue_InContainer(PairPtr, TempPairStorage);
				MapProp->ValueProp->CopyCompleteValue_InContainer(PairPtr, TempPairStorage);
			}

			RowCount += ChildRowCount;
		}

		MapHelper.Rehash();
		bSuccess = true;
	}
}

void USheetSerializationSubSystem::SetStringPropertyFromCells(FSheetData& SheetData, const FIntPoint StartingPosition, int32 ObjectRowCount, void* InObject, UObject* Parent, const FProperty* InProperty)
{
	if (const FStrProperty* StrProp = CastField<FStrProperty>(InProperty))
	{
		FString& CellString = SheetData.GetStringForCell(StartingPosition.X, StartingPosition.Y);
		StrProp->SetPropertyValue_InContainer(InObject, CellString);
	}
}
