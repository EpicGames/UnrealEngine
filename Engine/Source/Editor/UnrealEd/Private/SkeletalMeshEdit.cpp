// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SkeletalMeshEdit.cpp: Unreal editor skeletal mesh/anim support
=============================================================================*/

#include "Animation/AnimationSettings.h"
#include "Animation/AnimCurveTypes.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimTypes.h"
#include "Animation/Skeleton.h"
#include "Animation/SmartName.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ComponentReregisterContext.h"
#include "Components/SkeletalMeshComponent.h"
#include "CoreMinimal.h"
#include "Curves/KeyHandle.h"
#include "Editor/EditorEngine.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Factories/Factory.h"
#include "Factories/FbxAnimSequenceImportData.h"
#include "Factories/FbxImportUI.h"
#include "Factories/FbxSkeletalMeshImportData.h"
#include "FbxAnimUtils.h"
#include "FbxImporter.h"
#include "Logging/TokenizedMessage.h"
#include "Misc/FbxErrors.h"
#include "Misc/FeedbackContext.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "ObjectTools.h"
#include "Rendering/SkeletalMeshLODImporterData.h"
#include "UObject/UObjectIterator.h"
#include "UObject/Object.h"
#include "ComponentReregisterContext.h"
#include "Components/SkeletalMeshComponent.h"
#include "FbxAnimUtils.h"
#include "Animation/AnimSequenceHelpers.h"
#include "Animation/BuiltInAttributeTypes.h"

#define LOCTEXT_NAMESPACE "SkeletalMeshEdit"

//The max reference rate is use to cap the maximum rate we support.
//It must be base on DEFAULT_SAMPLERATE*2ExpX where X is a integer with range [1 to 6] because we use KINDA_SMALL_NUMBER(0.0001) we do not want to pass 1920Hz 1/1920 = 0.0005
#define MaxReferenceRate 1920.0f

UAnimSequence * UEditorEngine::ImportFbxAnimation( USkeleton* Skeleton, UObject* Outer, UFbxAnimSequenceImportData* TemplateImportData, const TCHAR* InFilename, const TCHAR* AnimName, bool bImportMorphTracks )
{
	check(Skeleton);

	UAnimSequence * NewAnimation=nullptr;

	UnFbx::FFbxImporter* FFbxImporter = UnFbx::FFbxImporter::GetInstance();

	const bool bPrevImportMorph = FFbxImporter->ImportOptions->bImportMorph;
	FFbxImporter->ImportOptions->bImportMorph = bImportMorphTracks;
	if ( !FFbxImporter->ImportFromFile( InFilename, FPaths::GetExtension( InFilename ), true ) )
	{
		// Log the error message and fail the import.
		FFbxImporter->FlushToTokenizedErrorMessage(EMessageSeverity::Error);
	}
	else
	{
		// Log the import message and import the mesh.
		FFbxImporter->FlushToTokenizedErrorMessage(EMessageSeverity::Warning);

		const FString Filename( InFilename );

		// Get Mesh nodes array that bind to the skeleton system, then morph animation is imported.
		TArray<FbxNode*> FBXMeshNodeArray;
		FbxNode* SkeletonRoot = FFbxImporter->FindFBXMeshesByBone(Skeleton->GetReferenceSkeleton().GetBoneName(0), true, FBXMeshNodeArray);

		if (!SkeletonRoot)
		{
			FFbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("Error_CouldNotFindFbxTrack", "Mesh contains {0} bone as root but animation doesn't contain the root track.\nImport failed."), FText::FromName(Skeleton->GetReferenceSkeleton().GetBoneName(0)))), FFbxErrors::Animation_CouldNotFindRootTrack);

			FFbxImporter->ReleaseScene();
			return nullptr;
		}

		// Check for blend shape curves that are not skinned.  Unskinned geometry can still contain morph curves
		if( bImportMorphTracks )
		{
			TArray<FbxNode*> MeshNodes;
			FFbxImporter->FillFbxMeshArray( FFbxImporter->Scene->GetRootNode(), MeshNodes, FFbxImporter );

			for( int32 NodeIndex = 0; NodeIndex < MeshNodes.Num(); ++NodeIndex )
			{
				// Its possible the nodes already exist so make sure they are only added once
				FBXMeshNodeArray.AddUnique( MeshNodes[NodeIndex] );
			}
		}

		TArray<FbxNode*> SortedLinks;
		FFbxImporter->RecursiveBuildSkeleton(SkeletonRoot, SortedLinks);

		if(SortedLinks.Num() == 0)
		{
			FFbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, LOCTEXT("Error_CouldNotBuildValidSkeleton", "Could not create a valid skeleton from the import data that matches the given Skeletal Mesh.  Check the bone names of both the Skeletal Mesh for this AnimSet and the animation data you are trying to import.")), 
				FFbxErrors::Animation_CouldNotBuildSkeleton);
		}
		else
		{
			NewAnimation = FFbxImporter->ImportAnimations( Skeleton, Outer, SortedLinks, AnimName, TemplateImportData, FBXMeshNodeArray);

			if( NewAnimation )
			{
				// since to know full path, reimport will need to do same
				UFbxAnimSequenceImportData* ImportData = UFbxAnimSequenceImportData::GetImportDataForAnimSequence(NewAnimation, TemplateImportData);
				ImportData->Update(UFactory::GetCurrentFilename(), &(FFbxImporter->Md5Hash));
			}
		}
	}

	FFbxImporter->ImportOptions->bImportMorph = bPrevImportMorph;
	FFbxImporter->ReleaseScene();

	return NewAnimation;
}

bool UEditorEngine::ReimportFbxAnimation( USkeleton* Skeleton, UAnimSequence* AnimSequence, UFbxAnimSequenceImportData* ImportData, const TCHAR* InFilename, bool& bOutImportAll, const bool bFactoryShowOptions, UFbxImportUI* ReimportUI)
{
	check(Skeleton);
	bool bResult = true;
	GWarn->BeginSlowTask( LOCTEXT("ImportingFbxAnimations", "Importing FBX animations"), true );

	UnFbx::FFbxImporter* FbxImporter = UnFbx::FFbxImporter::GetInstance();
	
	const bool bPrevImportMorph = (AnimSequence->GetDataModel()->GetNumberOfFloatCurves() > 0);

	const bool bOverrideImportSettings = ReimportUI != nullptr;
	
	if (!ReimportUI)
	{
		ReimportUI = NewObject<UFbxImportUI>();
	}

	ReimportUI->MeshTypeToImport = FBXIT_Animation;
	ReimportUI->bOverrideFullName = false;
	ReimportUI->bImportAnimations = true;

	const bool ShowImportDialogAtReimport = GetDefault<UEditorPerProjectUserSettings>()->bShowImportDialogAtReimport && !GIsAutomationTesting && bFactoryShowOptions;
	if (ImportData && !ShowImportDialogAtReimport)
	{
		// Prepare the import options
		if (!bOverrideImportSettings)
		{
			//Use the asset import data setting only if there was no ReimportUI parameter
			ReimportUI->AnimSequenceImportData = ImportData;
		}
		else
		{
			ImportData->CopyAnimationValues(ReimportUI->AnimSequenceImportData);
		}
		ReimportUI->SkeletalMeshImportData->bImportMeshesInBoneHierarchy = ImportData->bImportMeshesInBoneHierarchy;

		ApplyImportUIToImportOptions(ReimportUI, *FbxImporter->ImportOptions);
	}
	else if(ShowImportDialogAtReimport)
	{
		if (ImportData == nullptr)
		{
			// An existing import data object was not found, make one here and show the options dialog
			ImportData = UFbxAnimSequenceImportData::GetImportDataForAnimSequence(AnimSequence, ReimportUI->AnimSequenceImportData);
			AnimSequence->AssetImportData = ImportData;
		}
		ReimportUI->bIsReimport = true;
		ReimportUI->ReimportMesh = AnimSequence;
		ReimportUI->AnimSequenceImportData = ImportData;

		bool bImportOperationCanceled = false;
		bool bShowOptionDialog = true;
		bool bForceImportType = true;
		bool bIsObjFormat = false;
		bool bIsAutomated = false;

		// @hack to make sure skeleton is set before opening the dialog
		FbxImporter->ImportOptions->SkeletonForAnimation = Skeleton;

		GetImportOptions(FbxImporter, ReimportUI, bShowOptionDialog, bIsAutomated, AnimSequence->GetPathName(), bImportOperationCanceled, bOutImportAll, bIsObjFormat, InFilename, bForceImportType, FBXIT_Animation);

		if (bImportOperationCanceled)
		{
			//User cancel the re-import
			bResult = false;
			GWarn->EndSlowTask();
			return bResult;
		}
	}
	else
	{
		FbxImporter->ImportOptions->ResetForReimportAnimation();	
	}

	if ( !FbxImporter->ImportFromFile( InFilename, FPaths::GetExtension( InFilename ), true ) )
	{
		// Log the error message and fail the import.
		FbxImporter->FlushToTokenizedErrorMessage(EMessageSeverity::Error);
		bResult = false;
	}
	else
	{
		// Log the import message and import the mesh.
		FbxImporter->FlushToTokenizedErrorMessage(EMessageSeverity::Warning);


		const FString Filename( InFilename );

		// Get Mesh nodes array that bind to the skeleton system, then morph animation is imported.
		TArray<FbxNode*> FBXMeshNodeArray;
		FbxNode* SkeletonRoot = FbxImporter->FindFBXMeshesByBone(Skeleton->GetReferenceSkeleton().GetBoneName(0), true, FBXMeshNodeArray);

		if (!SkeletonRoot)
		{
			FbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("Error_CouldNotFindFbxTrack", "Mesh contains {0} bone as root but animation doesn't contain the root track.\nImport failed."), FText::FromName(Skeleton->GetReferenceSkeleton().GetBoneName(0)))), FFbxErrors::Animation_CouldNotFindTrack);
			bResult = false;
		}

		if (bResult)
		{
			// for now import all the time?
			bool bImportMorphTracks = true;
			// Check for blend shape curves that are not skinned.  Unskinned geometry can still contain morph curves
			if (bImportMorphTracks)
			{
				TArray<FbxNode*> MeshNodes;
				FbxImporter->FillFbxMeshArray(FbxImporter->Scene->GetRootNode(), MeshNodes, FbxImporter);

				for (int32 NodeIndex = 0; NodeIndex < MeshNodes.Num(); ++NodeIndex)
				{
					// Its possible the nodes already exist so make sure they are only added once
					FBXMeshNodeArray.AddUnique(MeshNodes[NodeIndex]);
				}
			}

			TArray<FbxNode*> SortedLinks;
			FbxImporter->RecursiveBuildSkeleton(SkeletonRoot, SortedLinks);

			if (SortedLinks.Num() == 0)
			{
				FbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, LOCTEXT("Error_CouldNotBuildValidSkeleton", "Could not create a valid skeleton from the import data that matches the given Skeletal Mesh.  Check the bone names of both the Skeletal Mesh for this AnimSet and the animation data you are trying to import.")), FFbxErrors::Animation_CouldNotBuildSkeleton);
			}
			else
			{
				check(ImportData);

				// find the correct animation based on import data
				FbxAnimStack* CurAnimStack = nullptr;

				//ignore the source animation name if there's only one animation in the file.
				//this is to make it easier for people who use content creation programs that only export one animation and/or ones that don't allow naming animations			
				if (FbxImporter->Scene->GetSrcObjectCount(FbxCriteria::ObjectType(FbxAnimStack::ClassId)) > 1 && !ImportData->SourceAnimationName.IsEmpty())
				{
					CurAnimStack = FbxCast<FbxAnimStack>(FbxImporter->Scene->FindSrcObject(FbxCriteria::ObjectType(FbxAnimStack::ClassId), TCHAR_TO_UTF8(*ImportData->SourceAnimationName), 0));
				}
				else
				{
					CurAnimStack = FbxCast<FbxAnimStack>(FbxImporter->Scene->GetSrcObject(FbxCriteria::ObjectType(FbxAnimStack::ClassId), 0));
				}

				if (CurAnimStack)
				{
					// set current anim stack
					int32 ResampleRate = static_cast<int32>(DEFAULT_SAMPLERATE);
					if (FbxImporter->ImportOptions->bResample)
					{
						if(FbxImporter->ImportOptions->ResampleRate > 0)
						{
							ResampleRate = FbxImporter->ImportOptions->ResampleRate;
						}
						else
						{
							int32 BestResampleRate = FbxImporter->GetMaxSampleRate(SortedLinks);
							if(BestResampleRate > 0)
							{
								ResampleRate = BestResampleRate;
							}
						}
					}
					FbxTimeSpan AnimTimeSpan = FbxImporter->GetAnimationTimeSpan(SortedLinks[0], CurAnimStack);
					// for now it's not importing morph - in the future, this should be optional or saved with asset
					if (FbxImporter->ValidateAnimStack(SortedLinks, FBXMeshNodeArray, CurAnimStack, ResampleRate, bImportMorphTracks, FbxImporter->ImportOptions->bSnapToClosestFrameBoundary, AnimTimeSpan))
					{
						AnimSequence->ImportResampleFramerate = ResampleRate;
						FbxImporter->ImportAnimation(Skeleton, AnimSequence, Filename, SortedLinks, FBXMeshNodeArray, CurAnimStack, ResampleRate, AnimTimeSpan, true);
					}
				}
				else
				{
					// no track is found

					FbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("Error_CouldNotFindTrack", "Could not find needed track.")), FFbxErrors::Animation_CouldNotFindTrack);

					bResult = false;
				}
			}
		}
	}

	FbxImporter->ImportOptions->bImportMorph = bPrevImportMorph;
	FbxImporter->ReleaseScene();
	GWarn->EndSlowTask();

	return bResult;
}

// The Unroll filter expects only rotation curves, we need to walk the scene and extract the
// rotation curves from the nodes property. This can become time consuming but we have no choice.
static void ApplyUnroll(FbxNode *pNode, FbxAnimLayer* pLayer, FbxAnimCurveFilterUnroll* pUnrollFilter)
{
	if (!pNode || !pLayer || !pUnrollFilter)
	{
		return;
	}

	FbxAnimCurveNode* lCN = pNode->LclRotation.GetCurveNode(pLayer);
	if (lCN)
	{
		FbxAnimCurve* lRCurve[3];
		lRCurve[0] = lCN->GetCurve(0);
		lRCurve[1] = lCN->GetCurve(1);
		lRCurve[2] = lCN->GetCurve(2);


		// Set bone rotation order
		EFbxRotationOrder RotationOrder = eEulerXYZ;
		pNode->GetRotationOrder(FbxNode::eSourcePivot, RotationOrder);
		pUnrollFilter->SetRotationOrder((FbxEuler::EOrder)(RotationOrder));

		pUnrollFilter->Apply(lRCurve, 3);
	}

	for (int32 i = 0; i < pNode->GetChildCount(); i++)
	{
		ApplyUnroll(pNode->GetChild(i), pLayer, pUnrollFilter);
	}
}

void UnFbx::FFbxImporter::MergeAllLayerAnimation(FbxAnimStack* AnimStack, int32 ResampleRate)
{
	FbxTime lFramePeriod;
	lFramePeriod.SetSecondDouble(1.0 / ResampleRate);

	FbxTimeSpan lTimeSpan = AnimStack->GetLocalTimeSpan();
	AnimStack->BakeLayers(Scene->GetAnimationEvaluator(), lTimeSpan.GetStart(), lTimeSpan.GetStop(), lFramePeriod);

	// always apply unroll filter
	FbxAnimCurveFilterUnroll UnrollFilter;

	FbxAnimLayer* lLayer = AnimStack->GetMember<FbxAnimLayer>(0);
	UnrollFilter.Reset();
	ApplyUnroll(Scene->GetRootNode(), lLayer, &UnrollFilter);
}

bool UnFbx::FFbxImporter::IsValidAnimationData(TArray<FbxNode*>& SortedLinks, TArray<FbxNode*>& NodeArray, int32& ValidTakeCount)
{
	// If there are no valid links, then we cannot import the anim set
	if(SortedLinks.Num() == 0)
	{
		return false;
	}

	ValidTakeCount = 0;

	int32 AnimStackCount = Scene->GetSrcObjectCount<FbxAnimStack>();

	int32 AnimStackIndex;
	for (AnimStackIndex = 0; AnimStackIndex < AnimStackCount; AnimStackIndex++ )
	{
		FbxAnimStack* CurAnimStack = Scene->GetSrcObject<FbxAnimStack>(AnimStackIndex);
		// set current anim stack
		Scene->SetCurrentAnimationStack(CurAnimStack);

		// debug purpose
		for (int32 BoneIndex = 0; BoneIndex < SortedLinks.Num(); BoneIndex++)
		{
			FString BoneName = MakeName(SortedLinks[BoneIndex]->GetName());
			UE_LOGF(LogFbx, Log, "SortedLinks :(%d) %ls", BoneIndex, *BoneName );
		}

		//The animation timespan must use the original fbx framerate so the frame number match the DCC frame number
		FbxTimeSpan AnimTimeSpan = GetAnimationTimeSpan(SortedLinks[0], CurAnimStack);
		if (AnimTimeSpan.GetDuration() <= 0)
		{
			AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("FBXImport_ZeroLength", "Animation Stack {0} does not contain any valid key. Try different time options when import."), FText::FromString(UTF8_TO_TCHAR(CurAnimStack->GetName())))), FFbxErrors::Animation_ZeroLength);
			continue;
		}

		ValidTakeCount++;
		{
			bool bBlendCurveFound = false;

			for ( int32 NodeIndex = 0; !bBlendCurveFound && NodeIndex < NodeArray.Num(); NodeIndex++ )
			{
				// consider blendshape animation curve
				FbxGeometry* Geometry = (FbxGeometry*)NodeArray[NodeIndex]->GetNodeAttribute();
				if (Geometry)
				{
					int32 BlendShapeDeformerCount = Geometry->GetDeformerCount(FbxDeformer::eBlendShape);
					for(int32 BlendShapeIndex = 0; BlendShapeIndex<BlendShapeDeformerCount; ++BlendShapeIndex)
					{
						FbxBlendShape* BlendShape = (FbxBlendShape*)Geometry->GetDeformer(BlendShapeIndex, FbxDeformer::eBlendShape);

						int32 BlendShapeChannelCount = BlendShape->GetBlendShapeChannelCount();
						for(int32 ChannelIndex = 0; ChannelIndex<BlendShapeChannelCount; ++ChannelIndex)
						{
							FbxBlendShapeChannel* Channel = BlendShape->GetBlendShapeChannel(ChannelIndex);

							if(Channel)
							{
								// Get the percentage of influence of the shape.
								FbxAnimCurve* Curve = Geometry->GetShapeChannel(BlendShapeIndex, ChannelIndex, (FbxAnimLayer*)CurAnimStack->GetMember(0));
								if (Curve && Curve->KeyGetCount() > 0)
								{
									bBlendCurveFound = true;
									break;
								}
							}
						}
					}
				}
			}
		}
	}		

	return ( ValidTakeCount != 0 );
}

void UnFbx::FFbxImporter::FillAndVerifyBoneNames(USkeleton* Skeleton, TArray<FbxNode*>& SortedLinks, TArray<FName>& OutRawBoneNames, FString Filename)
{
	int32 TrackNum = SortedLinks.Num();

	OutRawBoneNames.AddUninitialized(TrackNum);
	// copy to the data
	for (int32 BoneIndex = 0; BoneIndex < TrackNum; BoneIndex++)
	{
		OutRawBoneNames[BoneIndex] = FName(*FSkeletalMeshImportData::FixupBoneName( MakeName(SortedLinks[BoneIndex]->GetName()) ));
	}

	const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();

	// make sure at least root bone matches
	if ( OutRawBoneNames[0] != RefSkeleton.GetBoneName(0) )
	{
		AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("FBXImport_RootMatchFail", "Root bone name does not match (FBX: {0} | Skeleton: {1})"), FText::FromName(OutRawBoneNames[0]), FText::FromName(RefSkeleton.GetBoneName(0)))), FFbxErrors::Animation_RootTrackMismatch);

		return;
	}

	// ensure there are no duplicated names
	for (int32 I = 0; I < TrackNum; I++)
	{
		for ( int32 J = I+1; J < TrackNum; J++ )
		{
			if (OutRawBoneNames[I] == OutRawBoneNames[J])
			{
				FString RawBoneName = OutRawBoneNames[J].ToString();
				AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("FBXImport_DupeBone", "Could not import {0}.\nDuplicate bone name found ('{1}'). Each bone must have a unique name."), FText::FromString(Filename), FText::FromString(RawBoneName))), FFbxErrors::Animation_DuplicatedBone);
			}
		}
	}

	// make sure all bone names are included, if not warn user
	FString BoneNames;
	for (int32 I = 0; I < TrackNum; ++I)
	{
		FName RawBoneName = OutRawBoneNames[I];
		if (RefSkeleton.FindBoneIndex(RawBoneName) == INDEX_NONE && !IsUnrealTransformAttribute(SortedLinks[I]))
		{
			BoneNames += RawBoneName.ToString();
			BoneNames += TEXT("  \n");
		}
	}

	if (BoneNames.IsEmpty() == false)
	{
		// warn user
		AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("FBXImport_MissingBone", "The following bones exist in the imported animation, but not in the Skeleton asset {0}.  Any animation on these bones will not be imported: \n\n {1}"), FText::FromString(Skeleton->GetName()), FText::FromString(BoneNames) )), FFbxErrors::Animation_MissingBones);
	}
}
//-------------------------------------------------------------------------
//
//-------------------------------------------------------------------------

FbxTimeSpan UnFbx::FFbxImporter::GetAnimationTimeSpan(FbxNode* RootNode, FbxAnimStack* AnimStack)
{
	FBXImportOptions* ImportOption = GetImportOptions();
	FbxTimeSpan AnimTimeSpan(FBXSDK_TIME_INFINITE, FBXSDK_TIME_MINUS_INFINITE);
	if (ImportOption)
	{
		bool bUseDefault = ImportOption->AnimationLengthImportType == FBXALIT_ExportedTime || FMath::IsNearlyZero(OriginalFbxFramerate, KINDA_SMALL_NUMBER);
		if  (bUseDefault)
		{
			AnimTimeSpan = AnimStack->GetLocalTimeSpan();
		}
		else if (ImportOption->AnimationLengthImportType == FBXALIT_AnimatedKey)
		{
			RootNode->GetAnimationInterval(AnimTimeSpan, AnimStack);
		}
		else // then it's range 
		{
			AnimTimeSpan = AnimStack->GetLocalTimeSpan();

			FbxTimeSpan AnimatedInterval(FBXSDK_TIME_INFINITE, FBXSDK_TIME_MINUS_INFINITE);
			RootNode->GetAnimationInterval(AnimatedInterval, AnimStack);

			// find the most range that covers by both method, that'll be used for clamping
			FbxTime StartTime = FMath::Min<FbxTime>(AnimTimeSpan.GetStart(), AnimatedInterval.GetStart());
			FbxTime StopTime = FMath::Max<FbxTime>(AnimTimeSpan.GetStop(),AnimatedInterval.GetStop());

			// make inclusive time between localtimespan and animation interval
			AnimTimeSpan.SetStart(StartTime);
			AnimTimeSpan.SetStop(StopTime);

			FbxTime EachFrame = FBXSDK_TIME_ONE_SECOND/OriginalFbxFramerate;
			int32 StartFrame = StartTime.Get()/EachFrame.Get();
			int32 StopFrame = StopTime.Get()/EachFrame.Get();
			if (StartFrame != StopFrame)
			{
				FbxTime Duration = AnimTimeSpan.GetDuration();

				int32 AnimRangeX = FMath::Clamp<int32>(ImportOption->AnimationRange.X, StartFrame, StopFrame);
				int32 AnimRangeY = FMath::Clamp<int32>(ImportOption->AnimationRange.Y, StartFrame, StopFrame);

				FbxLongLong Interval = EachFrame.Get();

				// now set new time
				if (StartFrame != AnimRangeX)
				{
					FbxTime NewTime(AnimRangeX*Interval);
					AnimTimeSpan.SetStart(NewTime);
				}

				if (StopFrame != AnimRangeY)
				{
					FbxTime NewTime(AnimRangeY*Interval);
					AnimTimeSpan.SetStop(NewTime);
				}
			}
		}
	}

	return AnimTimeSpan;
}

void UnFbx::FFbxImporter::GetAnimationIntervalMultiLayer(FbxNode* RootNode, FbxAnimStack* AnimStack, FbxTimeSpan& AnimTimeSpan)
{
	int NumAnimLayers = AnimStack != nullptr ? AnimStack->GetMemberCount() : 0;
	for (int AnimLayerIndex = 0; AnimLayerIndex < NumAnimLayers; ++AnimLayerIndex)
	{
		FbxTimeSpan LayerAnimTimeSpan(FBXSDK_TIME_INFINITE, FBXSDK_TIME_MINUS_INFINITE);
		RootNode->GetAnimationInterval(LayerAnimTimeSpan, AnimStack, AnimLayerIndex);
		if (LayerAnimTimeSpan.GetStart() < AnimTimeSpan.GetStart())
		{
			AnimTimeSpan.SetStart(LayerAnimTimeSpan.GetStart());
		}
		if (LayerAnimTimeSpan.GetStop() > AnimTimeSpan.GetStop())
		{
			AnimTimeSpan.SetStop(LayerAnimTimeSpan.GetStop());
		}
	}
}

/**
* Add to the animation set, the animations contained within the FBX document, for the given skeleton
*/
UAnimSequence * UnFbx::FFbxImporter::ImportAnimations(USkeleton* Skeleton, UObject* Outer, TArray<FbxNode*>& SortedLinks, const FString& Name, UFbxAnimSequenceImportData* TemplateImportData, TArray<FbxNode*>& NodeArray)
{
	// we need skeleton to create animsequence
	if (Skeleton == nullptr || !CanImportClass(UAnimSequence::StaticClass()))
	{
		return nullptr;
	}

	int32 ValidTakeCount = 0;
	if (IsValidAnimationData(SortedLinks, NodeArray, ValidTakeCount) == false)
	{
		AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, LOCTEXT("FBXImport_InvalidAnimationData", "This does not contain any valid animation takes.")), FFbxErrors::Animation_InvalidData);
		return nullptr;
	}

	UAnimSequence* LastCreatedAnim = nullptr;

	int32 ResampleRate = DEFAULT_SAMPLERATE;
	if ( ImportOptions->bResample )
	{
		if (ImportOptions->ResampleRate > 0)
		{
			ResampleRate = ImportOptions->ResampleRate;
		}
		else
		{
			// For FBX data, "Frame Rate" is just the speed at which the animation is played back.  It can change
			// arbitrarily, and the underlying data can stay the same.  What we really want here is the Sampling Rate,
			// ie: the number of animation keys per second.  These are the individual animation curve keys
			// on the FBX nodes of the skeleton.  So we loop through the nodes of the skeleton and find the maximum number 
			// of keys that any node has, then divide this by the total length (in seconds) of the animation to find the 
			// sampling rate of this set of data 

			// we want the maximum resample rate, so that we don't lose any precision of fast anims,
			// and don't mind creating lerped frames for slow anims
			int32 BestResampleRate = GetMaxSampleRate(SortedLinks);

			if (BestResampleRate > 0)
			{
				ResampleRate = BestResampleRate;
			}
		}
	}

	int32 AnimStackCount = Scene->GetSrcObjectCount<FbxAnimStack>();
	for( int32 AnimStackIndex = 0; AnimStackIndex < AnimStackCount; AnimStackIndex++ )
	{
		FbxAnimStack* CurAnimStack = Scene->GetSrcObject<FbxAnimStack>(AnimStackIndex);

		FbxTimeSpan AnimTimeSpan = GetAnimationTimeSpan(SortedLinks[0], CurAnimStack);
		bool bValidAnimStack = ValidateAnimStack(SortedLinks, NodeArray, CurAnimStack, ResampleRate, ImportOptions->bImportMorph, ImportOptions->bSnapToClosestFrameBoundary, AnimTimeSpan);
		// no animation
		if (!bValidAnimStack)
		{
			continue;
		}
		
		FString SequenceName = Name;
		FString SourceAnimationName = UTF8_TO_TCHAR(CurAnimStack->GetName());
		if (ValidTakeCount > 1)
		{
			SequenceName += "_";
			SequenceName += SourceAnimationName;
		}

		// See if this sequence already exists.
		SequenceName = ObjectTools::SanitizeObjectName(SequenceName);

		FString 	ParentPath = FString::Printf(TEXT("%s/%s"), *FPackageName::GetLongPackagePath(*Outer->GetName()), *SequenceName);
		UObject* 	ParentPackage = CreatePackage( *ParentPath);
		UObject* Object = LoadObject<UObject>(ParentPackage, *SequenceName, nullptr, (LOAD_Quiet | LOAD_NoWarn), nullptr);
		UAnimSequence * DestSeq = Cast<UAnimSequence>(Object);
		// if object with same name exists, warn user
		if (Object && !DestSeq)
		{
			AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("Error_AssetExist", "Asset with same name exists. Can't overwrite another asset")), FFbxErrors::Generic_SameNameAssetExists);
			continue; // Move on to next sequence...
		}

		// If not, create new one now.

		const bool bCreateAsset = !DestSeq;
		if(bCreateAsset)
		{
			DestSeq = NewObject<UAnimSequence>(ParentPackage, *SequenceName, RF_Public | RF_Standalone);
			CreatedObjects.Add(DestSeq);
		}

		DestSeq->SetSkeleton(Skeleton);

		// since to know full path, reimport will need to do same
		UFbxAnimSequenceImportData* ImportData = UFbxAnimSequenceImportData::GetImportDataForAnimSequence(DestSeq, TemplateImportData);
		ImportData->Update(UFactory::GetCurrentFilename(), &Md5Hash);
		ImportData->SourceAnimationName = SourceAnimationName;
		DestSeq->ImportFileFramerate = GetOriginalFbxFramerate();
		DestSeq->ImportResampleFramerate = ResampleRate;

		DestSeq->GetController().InitializeModel();

		ImportAnimation(Skeleton, DestSeq, Name, SortedLinks, NodeArray, CurAnimStack, ResampleRate, AnimTimeSpan, false);

		if (bCreateAsset)
		{			
			// Notify the asset registry
			FAssetRegistryModule::AssetCreated(DestSeq);
		}

		LastCreatedAnim = DestSeq;
	}

	return LastCreatedAnim;
}

//Get the smallest sample rate(integer) representing the DeltaTime(time between 0.0f and 1.0f).
//@DeltaTime: the time to find the rate between 0.0f and 1.0f
int32 GetTimeSampleRate(const float DeltaTime)
{
	float OriginalSampleRateDivider = 1.0f / DeltaTime;
	float SampleRateDivider = OriginalSampleRateDivider;
	float SampleRemainder = FPlatformMath::Fractional(SampleRateDivider);
	float Multiplier = 2.0f;
	float IntegerPrecision = FMath::Min(FMath::Max(KINDA_SMALL_NUMBER*SampleRateDivider, KINDA_SMALL_NUMBER), 0.1f); //The precision is limit between KINDA_SMALL_NUMBER and 0.1f
	while (!FMath::IsNearlyZero(SampleRemainder, IntegerPrecision) && !FMath::IsNearlyEqual(SampleRemainder, 1.0f, IntegerPrecision))
	{
		SampleRateDivider = OriginalSampleRateDivider * Multiplier;
		SampleRemainder = FPlatformMath::Fractional(SampleRateDivider);
		if (SampleRateDivider > MaxReferenceRate)
		{
			SampleRateDivider = DEFAULT_SAMPLERATE;
			break;
		}
		Multiplier += 1.0f;
	}
	return FMath::Min(FPlatformMath::RoundToInt(SampleRateDivider), FPlatformMath::RoundToInt(MaxReferenceRate));
}

int32 GetAnimationCurveRate(FbxAnimCurve* CurrentCurve)
{
	if (CurrentCurve == nullptr)
		return 0;

	int32 KeyCount = CurrentCurve->KeyGetCount();
	
	FbxTimeSpan TimeInterval(FBXSDK_TIME_INFINITE, FBXSDK_TIME_MINUS_INFINITE);
	bool bValidTimeInterval = CurrentCurve->GetTimeInterval(TimeInterval);
	if (KeyCount > 1 && bValidTimeInterval)
	{
		double KeyAnimLength = TimeInterval.GetDuration().GetSecondDouble();
		if (KeyAnimLength != 0.0)
		{
			//////////////////////////////////////////////////////////////////////////
			// 1. Look if we have high frequency keys(resampling).

			//Basic sample rate is compute by dividing the KeyCount by the anim length. This is valid only if
			//all keys are time equidistant. But if we find a rate over DEFAULT_SAMPLERATE, we can estimate that
			//there is a constant frame rate between the key and simply return the rate.
			int32 SampleRate = FPlatformMath::RoundToInt((KeyCount - 1) / KeyAnimLength);
			if (SampleRate >= DEFAULT_SAMPLERATE)
			{
				//We import a curve with more then 30 keys per frame
				return SampleRate;
			}
			
			//////////////////////////////////////////////////////////////////////////
			// 2. Compute the sample rate of every keys with there time. Use the
			//    least common multiplier to get a sample rate that go through all keys.

			SampleRate = 1;
			double OldKeyTime = 0.0f;
			TSet<int32> DeltaComputed;
			//Reserve some space
			DeltaComputed.Reserve(30);
			const double KeyMultiplier = (1.0f / KINDA_SMALL_NUMBER);
			//Find also the smallest delta time between keys
			for (int32 KeyIndex = 0; KeyIndex < KeyCount; ++KeyIndex)
			{
				double KeyTime = (CurrentCurve->KeyGet(KeyIndex).GetTime().GetSecondDouble());
				//Collect the smallest delta time, there is no delta in case the first animation key time is negative
				double Delta = (KeyTime < 0 && KeyIndex == 0) ? 0.0 : KeyTime - OldKeyTime;
				//use the fractional part of the delta to have the delta between 0.0f and 1.0f
				Delta = FPlatformMath::Fractional(Delta);
				int32 DeltaKey = FPlatformMath::RoundToInt(Delta*KeyMultiplier);
				if (!FMath::IsNearlyZero((float)Delta, KINDA_SMALL_NUMBER) && !DeltaComputed.Contains(DeltaKey))
				{
					int32 ComputeSampleRate = GetTimeSampleRate(Delta);
					DeltaComputed.Add(DeltaKey);
					//Use the least common multiplier with the new delta entry
					int32 LeastCommonMultiplier = FMath::Min(FMath::LeastCommonMultiplier(SampleRate, ComputeSampleRate), FPlatformMath::RoundToInt(MaxReferenceRate));
					SampleRate = LeastCommonMultiplier != 0 ? LeastCommonMultiplier : FMath::Max3(FPlatformMath::RoundToInt(DEFAULT_SAMPLERATE), SampleRate, ComputeSampleRate);
				}
				OldKeyTime = KeyTime;
			}
			return SampleRate;
		}
	}

	return 0;
}

void GetNodeSampleRate(FbxNode* Node, FbxAnimLayer* AnimLayer, TArray<int32>& NodeAnimSampleRates)
{
	const int32 MaxElement = 9;
	FbxAnimCurve* Curves[MaxElement];

	Curves[0] = Node->LclTranslation.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_X, false);
	Curves[1] = Node->LclTranslation.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_Y, false);
	Curves[2] = Node->LclTranslation.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_Z, false);
	Curves[3] = Node->LclRotation.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_X, false);
	Curves[4] = Node->LclRotation.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_Y, false);
	Curves[5] = Node->LclRotation.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_Z, false);
	Curves[6] = Node->LclScaling.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_X, false);
	Curves[7] = Node->LclScaling.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_Y, false);
	Curves[8] = Node->LclScaling.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_Z, false);


	for (int32 CurveIndex = 0; CurveIndex < MaxElement; ++CurveIndex)
	{
		FbxAnimCurve* CurrentCurve = Curves[CurveIndex];
		if (CurrentCurve)
		{
			int32 CurveAnimRate = GetAnimationCurveRate(CurrentCurve);
			if (CurveAnimRate != 0)
			{
				NodeAnimSampleRates.AddUnique(CurveAnimRate);
			}
		}
	}
}

int32 UnFbx::FFbxImporter::GetGlobalAnimStackSampleRate(FbxAnimStack* CurAnimStack)
{
	int32 ResampleRate = DEFAULT_SAMPLERATE;
	if (ImportOptions->bResample)
	{
		TArray<int32> CurveAnimSampleRates;
		int32 MaxStackResampleRate = 0;
		int32 AnimStackLayerCount = CurAnimStack->GetMemberCount();
		for (int32 LayerIndex = 0; LayerIndex < AnimStackLayerCount; ++LayerIndex)
		{
			FbxAnimLayer* AnimLayer = (FbxAnimLayer*)CurAnimStack->GetMember(LayerIndex);
			for (int32 NodeIndex = 0; NodeIndex < Scene->GetNodeCount(); ++NodeIndex)
			{
				FbxNode* Node = Scene->GetNode(NodeIndex);
				//Get both the transform properties curve and the blend shape animation sample rate
				GetNodeSampleRate(Node, AnimLayer, CurveAnimSampleRates);
			}
		}

		MaxStackResampleRate = CurveAnimSampleRates.Num() > 0 ? 1 : MaxStackResampleRate;
		//Find the lowest sample rate that will pass by all the keys from all curves
		for (int32 CurveSampleRate : CurveAnimSampleRates)
		{
			if (CurveSampleRate >= MaxReferenceRate && MaxStackResampleRate < CurveSampleRate)
			{
				MaxStackResampleRate = CurveSampleRate;
			}
			else if (MaxStackResampleRate < MaxReferenceRate)
			{
				int32 LeastCommonMultiplier = FMath::LeastCommonMultiplier(MaxStackResampleRate, CurveSampleRate);
				MaxStackResampleRate = LeastCommonMultiplier != 0 ? LeastCommonMultiplier : FMath::Max3(FPlatformMath::RoundToInt(DEFAULT_SAMPLERATE), MaxStackResampleRate, CurveSampleRate);
				if (MaxStackResampleRate >= MaxReferenceRate)
				{
					MaxStackResampleRate = MaxReferenceRate;
				}
			}
		}

		// Make sure we're not hitting 0 for samplerate
		if (MaxStackResampleRate != 0)
		{
			//Make sure the resample rate is positive
			if (!ensure(MaxStackResampleRate >= 0))
			{
				MaxStackResampleRate *= -1;
			}
			ResampleRate = MaxStackResampleRate;
		}
	}
	return ResampleRate;
}

int32 UnFbx::FFbxImporter::GetMaxSampleRate(TArray<FbxNode*>& SortedLinks)
{
	
	int32 MaxStackResampleRate = 0;
	TArray<int32> CurveAnimSampleRates;
	const FBXImportOptions* ImportOption = GetImportOptions();
	int32 AnimStackCount = Scene->GetSrcObjectCount<FbxAnimStack>();
	for( int32 AnimStackIndex = 0; AnimStackIndex < AnimStackCount; AnimStackIndex++)
	{
		FbxAnimStack* CurAnimStack = Scene->GetSrcObject<FbxAnimStack>(AnimStackIndex);

		FbxTimeSpan AnimStackTimeSpan = GetAnimationTimeSpan(SortedLinks[0], CurAnimStack);

		double AnimStackStart = AnimStackTimeSpan.GetStart().GetSecondDouble();
		double AnimStackStop = AnimStackTimeSpan.GetStop().GetSecondDouble();

		int32 AnimStackLayerCount = CurAnimStack->GetMemberCount();
		for (int32 LayerIndex = 0; LayerIndex < AnimStackLayerCount; ++LayerIndex)
		{
			FbxAnimLayer* AnimLayer = (FbxAnimLayer*)CurAnimStack->GetMember(LayerIndex);
			for (int32 LinkIndex = 0; LinkIndex < SortedLinks.Num(); ++LinkIndex)
			{
				FbxNode* CurrentLink = SortedLinks[LinkIndex];
				GetNodeSampleRate(CurrentLink, AnimLayer, CurveAnimSampleRates);
			}
		}
	}

	MaxStackResampleRate = CurveAnimSampleRates.Num() > 0 ? 1 : MaxStackResampleRate;
	//Find the lowest sample rate that will pass by all the keys from all curves
	for (int32 CurveSampleRate : CurveAnimSampleRates)
	{
		if (CurveSampleRate >= MaxReferenceRate && MaxStackResampleRate < CurveSampleRate)
		{
			MaxStackResampleRate = CurveSampleRate;
		}
		else if (MaxStackResampleRate < MaxReferenceRate)
		{
			int32 LeastCommonMultiplier = FMath::LeastCommonMultiplier(MaxStackResampleRate, CurveSampleRate);
			MaxStackResampleRate = LeastCommonMultiplier != 0 ? LeastCommonMultiplier : FMath::Max3(FPlatformMath::RoundToInt(DEFAULT_SAMPLERATE), MaxStackResampleRate, CurveSampleRate);
			if (MaxStackResampleRate >= MaxReferenceRate)
			{
				MaxStackResampleRate = MaxReferenceRate;
			}
		}
	}

	// Make sure we're not hitting 0 for samplerate
	if ( MaxStackResampleRate != 0 )
	{
		//Make sure the resample rate is positive
		if (!ensure(MaxStackResampleRate >= 0))
		{
			MaxStackResampleRate *= -1;
		}
		return MaxStackResampleRate;
	}

	return DEFAULT_SAMPLERATE;
}

bool UnFbx::FFbxImporter::ValidateAnimStack(TArray<FbxNode*>& SortedLinks, TArray<FbxNode*>& NodeArray, FbxAnimStack* CurAnimStack, int32 ResampleRate, bool bImportMorph, bool bSnapToClosestFrameBoundary, FbxTimeSpan &AnimTimeSpan)
{
	// set current anim stack
	Scene->SetCurrentAnimationStack(CurAnimStack);

	UE_LOGF(LogFbx, Log, "Parsing AnimStack %ls",UTF8_TO_TCHAR(CurAnimStack->GetName()));

	bool bValidAnimStack = true;

	AnimTimeSpan = GetAnimationTimeSpan(SortedLinks[0], CurAnimStack);
	
	// if no duration is found, return false
	if (AnimTimeSpan.GetDuration() <= 0)
	{
		return false;
	}

	const double SequenceLengthInSeconds = FGenericPlatformMath::Max<double>(AnimTimeSpan.GetDuration().GetSecondDouble(), MINIMUM_ANIMATION_LENGTH);
	const FFrameRate TargetFrameRate(ResampleRate, 1);
	const FFrameTime LengthInFrameTime = TargetFrameRate.AsFrameTime(SequenceLengthInSeconds);
	const float SubFrame = LengthInFrameTime.GetSubFrame();
	if (!FMath::IsNearlyZero(SubFrame, KINDA_SMALL_NUMBER) && !FMath::IsNearlyEqual(SubFrame, 1.0f, KINDA_SMALL_NUMBER))
	{
		if (bSnapToClosestFrameBoundary)
		{
			// Figure out whether start or stop has to be adjusted
			const FbxTime StartTime = AnimTimeSpan.GetStart();
			const FbxTime StopTime = AnimTimeSpan.GetStop();
			FbxTime NewStartTime;
			FbxTime NewStopTime;

			const FFrameTime StartFrameTime = TargetFrameRate.AsFrameTime(StartTime.GetSecondDouble());
			const FFrameTime StopFrameTime = TargetFrameRate.AsFrameTime(StopTime.GetSecondDouble());
			FFrameNumber StartFrameNumber, StopFrameNumber;

			if (!FMath::IsNearlyZero(StartFrameTime.GetSubFrame()))
			{
				StartFrameNumber = StartFrameTime.RoundToFrame();
				NewStartTime.SetSecondDouble(TargetFrameRate.AsSeconds(StartFrameNumber));
				AnimTimeSpan.SetStart(NewStartTime);
			}

			if (!FMath::IsNearlyZero(StopFrameTime.GetSubFrame()))
			{
				StopFrameNumber = StopFrameTime.RoundToFrame();
				NewStopTime.SetSecondDouble(TargetFrameRate.AsSeconds(StopFrameNumber));
				AnimTimeSpan.SetStop(NewStopTime);
			}
			
			AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Info, FText::Format(LOCTEXT("Info_ImportLengthSnap", "Animation length has been adjusted to align with frame borders using import frame-rate {0}.\n\nOriginal timings:\n\t\tStart: {1} ({2})\n\t\tStop: {3} ({4})\nAligned timings:\n\t\tStart: {5} ({6})\n\t\tStop: {7} ({8})"),
				TargetFrameRate.ToPrettyText(),
				FText::AsNumber(StartTime.GetSecondDouble()),
				FText::AsNumber(StartFrameTime.AsDecimal()),
				FText::AsNumber(StopTime.GetSecondDouble()),
				FText::AsNumber(StopFrameTime.AsDecimal()),
				FText::AsNumber(NewStartTime.GetSecondDouble()),
				FText::AsNumber(StartFrameNumber.Value),
				FText::AsNumber(NewStopTime.GetSecondDouble()),
				FText::AsNumber(StopFrameNumber.Value))), FFbxErrors::Animation_InvalidData);
		}
		else
		{
			AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("Error_InvalidImportLength", "Animation length {0} is not compatible with import frame-rate {1} (sub frame {2}), animation has to be frame-border aligned. Either re-export animation or enable snap to closest frame boundary import option."), FText::AsNumber(SequenceLengthInSeconds), TargetFrameRate.ToPrettyText(), FText::AsNumber(SubFrame))), FFbxErrors::Animation_InvalidData);
			return false;
		}
	}

	const FBXImportOptions* ImportOption = GetImportOptions();
	// only add morph time if not setrange. If Set Range there is no reason to override time
	if ( bImportMorph && ImportOption->AnimationLengthImportType != FBXALIT_SetRange)
	{
		for ( int32 NodeIndex = 0; NodeIndex < NodeArray.Num(); NodeIndex++ )
		{
			// consider blendshape animation curve
			FbxGeometry* Geometry = (FbxGeometry*)NodeArray[NodeIndex]->GetNodeAttribute();
			if (Geometry)
			{
				int32 BlendShapeDeformerCount = Geometry->GetDeformerCount(FbxDeformer::eBlendShape);
				for(int32 BlendShapeIndex = 0; BlendShapeIndex<BlendShapeDeformerCount; ++BlendShapeIndex)
				{
					FbxBlendShape* BlendShape = (FbxBlendShape*)Geometry->GetDeformer(BlendShapeIndex, FbxDeformer::eBlendShape);

					int32 BlendShapeChannelCount = BlendShape->GetBlendShapeChannelCount();
					for(int32 ChannelIndex = 0; ChannelIndex<BlendShapeChannelCount; ++ChannelIndex)
					{
						FbxBlendShapeChannel* Channel = BlendShape->GetBlendShapeChannel(ChannelIndex);

						if(Channel)
						{
							// Get the percentage of influence of the shape.
							FbxAnimCurve* Curve = Geometry->GetShapeChannel(BlendShapeIndex, ChannelIndex, (FbxAnimLayer*)CurAnimStack->GetMember(0));
							if (Curve && Curve->KeyGetCount() > 0)
							{
								FbxTimeSpan TmpAnimSpan;

								if (Curve->GetTimeInterval(TmpAnimSpan))
								{
									bValidAnimStack = true;
									// update animation interval to include morph target range
									AnimTimeSpan.UnionAssignment(TmpAnimSpan);
								}
							}
						}
					}
				}
			}
		}
	}

	return bValidAnimStack;
}

bool UnFbx::FFbxImporter::ImportCurve(const FbxAnimCurve* FbxCurve, FRichCurve& RichCurve, const FbxTimeSpan &AnimTimeSpan, const bool bNegative/*=false*/, const float ValueScale/*=1.f*/, const bool bAutoSetTangents /*= true*/)
{
	const float DefaultCurveWeight = FbxAnimCurveDef::sDEFAULT_WEIGHT;

	if ( FbxCurve )
	{
		//We use the non const to query the left and right derivative of the key, for whatever reason those FBX API functions are not const
		FbxAnimCurve* NonConstFbxCurve = const_cast<FbxAnimCurve*>(FbxCurve);
		int32 KeyCount = FbxCurve->KeyGetCount();
		const float AdjustedValueScale = (bNegative ? -ValueScale : ValueScale);
		for ( int32 KeyIndex=0; KeyIndex < KeyCount; ++KeyIndex )
		{
			FbxAnimCurveKey Key = FbxCurve->KeyGet(KeyIndex);
			FbxTime KeyTime = Key.GetTime() - AnimTimeSpan.GetStart();
			const float KeyTimeValue = static_cast<float>(KeyTime.GetSecondDouble());
			float Value = Key.GetValue() * AdjustedValueScale;
			FKeyHandle NewKeyHandle = RichCurve.AddKey(KeyTimeValue, Value, false);

			const bool bIncludeOverrides = true;
			FbxAnimCurveDef::ETangentMode KeyTangentMode = Key.GetTangentMode(bIncludeOverrides);
			FbxAnimCurveDef::EInterpolationType KeyInterpMode = Key.GetInterpolation();
			FbxAnimCurveDef::EWeightedMode KeyTangentWeightMode = Key.GetTangentWeightMode();

			ERichCurveInterpMode NewInterpMode = RCIM_Linear;
			ERichCurveTangentMode NewTangentMode = RCTM_Auto;
			ERichCurveTangentWeightMode NewTangentWeightMode = RCTWM_WeightedNone;

			float RightTangent = NonConstFbxCurve->KeyGetRightDerivative(KeyIndex) * AdjustedValueScale;
			float LeftTangent = NonConstFbxCurve->KeyGetLeftDerivative(KeyIndex) * AdjustedValueScale;
			float RightTangentWeight = 0.0f;
			float LeftTangentWeight = 0.0f; //This one is dependent on the previous key.
			bool bLeftWeightActive = false;
			bool bRightWeightActive = false;

			const bool bPreviousKeyValid = KeyIndex > 0;
			const bool bNextKeyValid = KeyIndex < KeyCount - 1;
			float PreviousValue = 0.0f;
			float PreviousKeyTimeValue = 0.0f;
			float NextValue = 0.0f;
			float NextKeyTimeValue = 0.0f;
			if (bPreviousKeyValid)
			{
				FbxAnimCurveKey PreviousKey = FbxCurve->KeyGet(KeyIndex - 1);
				FbxTime PreviousKeyTime = PreviousKey.GetTime() - AnimTimeSpan.GetStart();
				PreviousKeyTimeValue = static_cast<float>(PreviousKeyTime.GetSecondDouble());
				PreviousValue = PreviousKey.GetValue() * AdjustedValueScale;
				//The left tangent is driven by the previous key. If the previous key have a the NextLeftweight or both flag weighted mode, it mean the next key is weighted on the left side
				bLeftWeightActive = (PreviousKey.GetTangentWeightMode() & FbxAnimCurveDef::eWeightedNextLeft) > 0;
				if (bLeftWeightActive)
				{
					LeftTangentWeight = PreviousKey.GetDataFloat(FbxAnimCurveDef::eNextLeftWeight);
				}
			}
			if (bNextKeyValid)
			{
				FbxAnimCurveKey NextKey = FbxCurve->KeyGet(KeyIndex + 1);
				FbxTime NextKeyTime = NextKey.GetTime() - AnimTimeSpan.GetStart();
				NextKeyTimeValue = static_cast<float>(NextKeyTime.GetSecondDouble());
				NextValue = NextKey.GetValue() * AdjustedValueScale;

				bRightWeightActive = (KeyTangentWeightMode & FbxAnimCurveDef::eWeightedRight) > 0;
				if (bRightWeightActive)
				{
					//The right tangent weight should be use only if we are not the last key since the last key do not have a right tangent.
					//Use the current key to gather the right tangent weight
					RightTangentWeight = Key.GetDataFloat(FbxAnimCurveDef::eRightWeight);
				}
			}

			// When this flag is true, the tangent is flat if the value has the same value as the previous or next key.
			const bool bTangentGenericClamp = (KeyTangentMode & FbxAnimCurveDef::eTangentGenericClamp);

			//Time independent tangent this is consider has a spline tangent key
			const bool bTangentGenericTimeIndependent = (KeyTangentMode & FbxAnimCurveDef::ETangentMode::eTangentGenericTimeIndependent);
			
			// When this flag is true, the tangent is flat if the value is outside of the [previous key, next key] value range.
			//Clamp progressive is (eTangentGenericClampProgressive |eTangentGenericTimeIndependent)
			const bool bTangentGenericClampProgressive = (KeyTangentMode & FbxAnimCurveDef::ETangentMode::eTangentGenericClampProgressive) == FbxAnimCurveDef::ETangentMode::eTangentGenericClampProgressive;

			if (KeyTangentMode & FbxAnimCurveDef::eTangentGenericBreak)
			{
				NewTangentMode = RCTM_Break;
			}
			else if (KeyTangentMode & FbxAnimCurveDef::eTangentUser)
			{
				NewTangentMode = RCTM_User;
			}

			switch (KeyInterpMode)
			{
			case FbxAnimCurveDef::eInterpolationConstant://! Constant value until next key.
				NewInterpMode = RCIM_Constant;
				break;
			case FbxAnimCurveDef::eInterpolationLinear://! Linear progression to next key.
				NewInterpMode = RCIM_Linear;
				break;
			case FbxAnimCurveDef::eInterpolationCubic://! Cubic progression to next key.
				NewInterpMode = RCIM_Cubic;
				// get tangents
				{
					bool bIsFlatTangent = false;
					bool bIsComputedTangent = false;
					if (bTangentGenericClampProgressive)
					{
						if (bPreviousKeyValid && bNextKeyValid)
						{
							const float PreviousNextHalfDelta = (NextValue - PreviousValue) * 0.5f;
							const float PreviousNextAverage = PreviousValue + PreviousNextHalfDelta;
							// If the value is outside of the previous-next value range, the tangent is flat.
							bIsFlatTangent = FMath::Abs(Value - PreviousNextAverage) >= FMath::Abs(PreviousNextHalfDelta);
						}
						else
						{
							//Start/End tangent with the ClampProgressive flag are flat.
							bIsFlatTangent = true;
						}
					}
					else if (bTangentGenericClamp && (bPreviousKeyValid || bNextKeyValid))
					{
						if (bPreviousKeyValid && PreviousValue == Value)
						{
							bIsFlatTangent = true;
						}
						if (bNextKeyValid)
						{
							bIsFlatTangent |= Value == NextValue;
						}
					}
					else if (bTangentGenericTimeIndependent)
					{
						//Spline tangent key, because bTangentGenericClampProgressive include bTangentGenericTimeIndependent, we must treat this case after bTangentGenericClampProgressive
						if (KeyCount == 1)
						{
							bIsFlatTangent = true;
						}
						else
						{
							//Spline tangent key must be User mode since we want to keep the tangents provide by the fbx key left and right derivatives
							NewTangentMode = RCTM_User;
						}
					}
					
					if (bIsFlatTangent)
					{
						RightTangent = 0;
						LeftTangent = 0;
						//To force flat tangent we need to set the tangent mode to user
						NewTangentMode = RCTM_User;
					}

				}
				break;
			}

			//auto with weighted give the wrong result, so when auto is weighted we set user mode and set the Right tangent equal to the left tangent.
			//Auto has only the left tangent set
			if (NewTangentMode == RCTM_Auto && (bLeftWeightActive || bRightWeightActive))
			{
				
				NewTangentMode = RCTM_User;
				RightTangent = LeftTangent;
			}

			if (NewTangentMode != RCTM_Auto)
			{
				const bool bEqualTangents = FMath::IsNearlyEqual(LeftTangent, RightTangent);
				//If tangents are different then broken.
				if (bEqualTangents)
				{
					NewTangentMode = RCTM_User;
				}
				else
				{
					NewTangentMode = RCTM_Break;
				}
			}

			//Only cubic interpolation allow weighted tangents
			if (KeyInterpMode == FbxAnimCurveDef::eInterpolationCubic)
			{
				if (bLeftWeightActive && bRightWeightActive)
				{
					NewTangentWeightMode = RCTWM_WeightedBoth;
				}
				else if (bLeftWeightActive)
				{
					NewTangentWeightMode = RCTWM_WeightedArrive;
					RightTangentWeight = DefaultCurveWeight;
				}
				else if (bRightWeightActive)
				{
					NewTangentWeightMode = RCTWM_WeightedLeave;
					LeftTangentWeight = DefaultCurveWeight;
				}
				else
				{
					NewTangentWeightMode = RCTWM_WeightedNone;
					LeftTangentWeight = DefaultCurveWeight;
					RightTangentWeight = DefaultCurveWeight;
				}

				auto ComputeWeightInternal = [](float TimeA, float TimeB, const float TangentSlope, const float TangentWeight)
				{
					const float X = TimeA - TimeB;
					const float Y = TangentSlope * X;
					return FMath::Sqrt(X * X + Y * Y) * TangentWeight;
				};

				if (!FMath::IsNearlyZero(LeftTangentWeight))
				{
					if (bPreviousKeyValid)
					{
						LeftTangentWeight = ComputeWeightInternal(KeyTimeValue, PreviousKeyTimeValue, LeftTangent, LeftTangentWeight);
					}
					else
					{
						LeftTangentWeight = 0.0f;
					}
				}

				if (!FMath::IsNearlyZero(RightTangentWeight))
				{
					if (bNextKeyValid)
					{
						RightTangentWeight = ComputeWeightInternal(NextKeyTimeValue, KeyTimeValue, RightTangent, RightTangentWeight);
					}
					else
					{
						RightTangentWeight = 0.0f;
					}
				}
			}

			const bool bForceDisableTangentRecompute = false; //No need to recompute all the tangents of the curve every time we change de key.
			RichCurve.SetKeyInterpMode(NewKeyHandle, NewInterpMode, bForceDisableTangentRecompute);
			RichCurve.SetKeyTangentMode(NewKeyHandle, NewTangentMode, bForceDisableTangentRecompute);
			RichCurve.SetKeyTangentWeightMode(NewKeyHandle, NewTangentWeightMode, bForceDisableTangentRecompute);

			FRichCurveKey& NewKey = RichCurve.GetKey(NewKeyHandle);
			NewKey.ArriveTangent = LeftTangent;
			NewKey.LeaveTangent = RightTangent;
			NewKey.ArriveTangentWeight = LeftTangentWeight;
			NewKey.LeaveTangentWeight = RightTangentWeight;
		}

		if (bAutoSetTangents)
		{
			RichCurve.AutoSetTangents();
		}

		return true;
	}

	return false;
}

bool UnFbx::FFbxImporter::ImportCurveToAnimSequence(class UAnimSequence * TargetSequence, const FString& CurveName, const FbxAnimCurve* FbxCurve, int32 CurveFlags,const FbxTimeSpan& AnimTimeSpan, const bool bReimport, float ValueScale/*=1.f*/) const
{
	if (TargetSequence && FbxCurve)
	{
		FName Name = *CurveName;

		FAnimationCurveIdentifier FloatCurveId(Name, ERawCurveTrackTypes::RCT_Float);

		const bool bShouldTransact = bReimport;
		IAnimationDataModel* DataModel = TargetSequence->GetDataModel();
		IAnimationDataController& Controller = TargetSequence->GetController();

		const FFloatCurve* TargetCurve = DataModel->FindFloatCurve(FloatCurveId);
		if (TargetCurve == nullptr)
		{
			// Need to add the curve first
			Controller.AddCurve(FloatCurveId, AACF_DefaultCurve | CurveFlags, bShouldTransact);
			TargetCurve = DataModel->FindFloatCurve(FloatCurveId);
		}
		else
		{
			// Need to update any of the flags
			Controller.SetCurveFlags(FloatCurveId, CurveFlags | TargetCurve->GetCurveTypeFlags(), bShouldTransact);
		}
		
		// Should be valid at this point
		ensure(TargetCurve);

		FRichCurve RichCurve;
		constexpr bool bNegative = false;
		if (ImportCurve(FbxCurve, RichCurve, AnimTimeSpan, bNegative, ValueScale))
		{
			if (ImportOptions->bRemoveRedundantKeys)
			{
				RichCurve.RemoveRedundantAutoTangentKeys(SMALL_NUMBER);
			}

			// Set actual keys on curve within the model
			Controller.SetCurveKeys(FloatCurveId, RichCurve.GetConstRefOfKeys(), bShouldTransact);

			return true;
		}
	}

	return false;
}


bool UnFbx::FFbxImporter::ImportRichCurvesToAnimSequence(UAnimSequence* TargetSequence, const TArray<FString>& CurveNames, const TArray<FRichCurve> RichCurves, int32 CurveFlags, const bool bReimport) const
{
	if (TargetSequence && CurveNames.Num() > 0 && CurveNames.Num() == RichCurves.Num())
	{
		const bool bShouldTransact = bReimport;
		for (int32 CurveIndex = 0; CurveIndex < CurveNames.Num(); ++CurveIndex)
		{
			FName Name = *CurveNames[CurveIndex];
			
			FAnimationCurveIdentifier FloatCurveId(Name, ERawCurveTrackTypes::RCT_Float);
			const IAnimationDataModel* DataModel = TargetSequence->GetDataModel();
			IAnimationDataController& Controller = TargetSequence->GetController();

			const FFloatCurve* TargetCurve = DataModel->FindFloatCurve(FloatCurveId);
			
			if (TargetCurve == nullptr)
			{
				// Need to add the curve first
				Controller.AddCurve(FloatCurveId, AACF_DefaultCurve | CurveFlags, bShouldTransact);
				TargetCurve = DataModel->FindFloatCurve(FloatCurveId);
			}
			else
			{
				// Need to update any of the flags
				Controller.SetCurveFlags(FloatCurveId, CurveFlags | TargetCurve->GetCurveTypeFlags(), bShouldTransact);
			}
		
			// Should be valid at this point
			ensure(TargetCurve);

			// Set actual keys on curve within the model
			Controller.SetCurveKeys(FloatCurveId, RichCurves[CurveIndex].GetConstRefOfKeys(), bShouldTransact);

		}
		
		return true;
	}

	return false;
}

TArray<FRichCurve> UnFbx::FFbxImporter::ResolveWeightsForBlendShapeCurve(FRichCurve& ChannelWeightCurve, const TArray<float>& InbetweenFullWeights) const
{
	int32 NumInbetweens = InbetweenFullWeights.Num();
	if (NumInbetweens == 0)
	{
		return { ChannelWeightCurve };
	}

	TArray<FRichCurve> Result;
	Result.SetNum( NumInbetweens + 1 );

	TArray<float> ResolvedInbetweenWeightsSample;
	ResolvedInbetweenWeightsSample.SetNum( NumInbetweens );

	for ( const FRichCurveKey& SourceKey : ChannelWeightCurve.Keys )
	{
		const float SourceTime = SourceKey.Time;
		const float SourceValue = SourceKey.Value;

		float ResolvedPrimarySample = 0.0f;

		ResolveWeightsForBlendShape(InbetweenFullWeights,SourceValue, ResolvedPrimarySample, ResolvedInbetweenWeightsSample);

		FRichCurve& PrimaryCurve = Result[ 0 ];
		FKeyHandle PrimaryHandle = PrimaryCurve.AddKey( SourceTime, ResolvedPrimarySample );
		PrimaryCurve.SetKeyInterpMode( PrimaryHandle, SourceKey.InterpMode );

		for ( int32 InbetweenIndex = 0; InbetweenIndex < NumInbetweens; ++InbetweenIndex )
		{
			FRichCurve& InbetweenCurve = Result[ InbetweenIndex + 1 ];
			FKeyHandle InbetweenHandle = InbetweenCurve.AddKey( SourceTime, ResolvedInbetweenWeightsSample[ InbetweenIndex ] );
			InbetweenCurve.SetKeyInterpMode( InbetweenHandle, SourceKey.InterpMode );
		}
	}

	return Result;
}

void UnFbx::FFbxImporter::ResolveWeightsForBlendShape(const TArray<float>& InbetweenFullWeights , float InWeight, float& OutMainWeight, TArray<float>& OutInbetweenWeights) const
{
	int32 NumInbetweens = InbetweenFullWeights.Num();
	if ( NumInbetweens == 0 )
	{
		OutMainWeight = InWeight;
		return;
	}

	OutInbetweenWeights.SetNumUninitialized( NumInbetweens );
	for ( float& OutInbetweenWeight : OutInbetweenWeights )
	{
		OutInbetweenWeight = 0.0f;
	}

	if ( FMath::IsNearlyEqual( InWeight, 0.0f ) )
	{
		OutMainWeight = 0.0f;
		return;
	}
	else if ( FMath::IsNearlyEqual( InWeight, 1.0f ) )
	{
		OutMainWeight = 1.0f;
		return;
	}

	// Note how we don't care if UpperIndex/LowerIndex are beyond the bounds of the array here,
	// as that signals when we're above/below all inbetweens
	int32 UpperIndex = Algo::UpperBoundBy( InbetweenFullWeights, InWeight, []( const double& InbetweenWeight )
	{
		return InbetweenWeight;
	} );
	int32 LowerIndex = UpperIndex - 1;

	float UpperWeight = 1.0f;
	if ( UpperIndex <= NumInbetweens - 1 )
	{
		UpperWeight = InbetweenFullWeights[ UpperIndex ];
	}

	float LowerWeight = 0.0f;
	if ( LowerIndex >= 0 )
	{
		LowerWeight = InbetweenFullWeights[ LowerIndex ];
	}

	UpperWeight = ( InWeight - LowerWeight ) / ( UpperWeight - LowerWeight );
	LowerWeight = ( 1.0f - UpperWeight );

	// We're between upper inbetween and the 1.0 weight
	if ( UpperIndex > NumInbetweens - 1 )
	{
		OutMainWeight = UpperWeight;
		OutInbetweenWeights[ NumInbetweens - 1 ] = LowerWeight;
	}
	// We're between 0.0 and the first inbetween weight
	else if ( LowerIndex < 0 )
	{
		OutMainWeight = 0;
		OutInbetweenWeights[ 0 ] = UpperWeight;
	}
	// We're between two inbetweens
	else
	{
		OutInbetweenWeights[ UpperIndex ] = UpperWeight;
		OutInbetweenWeights[ LowerIndex ] = LowerWeight;
	}
}

template<typename AttributeType>
void FillCurveAttributeToBone(TArray<float>& OutFrameTimes, TArray<AttributeType>& OutFrameValues, const FbxAnimCurve* FbxCurve, const FbxTimeSpan& AnimTimeSpan, TFunctionRef<AttributeType(const FbxAnimCurveKey*, const FbxTime*)> EvaluationFunction)
{
	const int32 KeyCount = FbxCurve ? FbxCurve->KeyGetCount() : 0;

	if (KeyCount > 0)
	{
		OutFrameTimes.Reserve(KeyCount);
		OutFrameValues.Reserve(KeyCount);
		const FbxTime StartTime = AnimTimeSpan.GetStart();

		for (int32 KeyIndex = 0; KeyIndex < KeyCount; ++KeyIndex)
		{
			FbxAnimCurveKey Key = FbxCurve->KeyGet(KeyIndex);
			FbxTime KeyTime = Key.GetTime() - StartTime;

			OutFrameTimes.Add(KeyTime.GetSecondDouble());
			OutFrameValues.Add(EvaluationFunction(&Key, &KeyTime));
		}
	}
	else
	{
		OutFrameTimes.Add(0);
		OutFrameValues.Add(EvaluationFunction(nullptr, nullptr));
	}
}

bool UnFbx::FFbxImporter::ImportCustomAttributeToBone(UAnimSequence* TargetSequence, FbxProperty& InProperty, FName BoneName, const FString& CurveName, const FbxAnimCurve* FbxCurve, const FbxTimeSpan& AnimTimeSpan, const bool bReimport, float ValueScale/*=1.f*/)
{
	if (TargetSequence)
	{
		TArray<float> TimeArray;

		switch (InProperty.GetPropertyDataType().GetType())
		{
			case EFbxType::eFbxHalfFloat:
			case EFbxType::eFbxFloat:
			case EFbxType::eFbxDouble:
			{
				TArray<float> FloatValues;
				FillCurveAttributeToBone<float>(TimeArray, FloatValues, FbxCurve, AnimTimeSpan, 
					[&ValueScale, &InProperty](const FbxAnimCurveKey* Key, const FbxTime* KeyTime){
					if (Key)
					{
						return Key->GetValue() * ValueScale;
					}
					else
					{
						return InProperty.Get<float>() * ValueScale;
					}
				});

				UE::Anim::AddTypedCustomAttribute<FFloatAnimationAttribute, float>(FName(CurveName), BoneName, TargetSequence, MakeArrayView(TimeArray), MakeArrayView(FloatValues));
				break;
			}
			case EFbxType::eFbxBool:
			case EFbxType::eFbxShort:
			case EFbxType::eFbxUShort:
			case EFbxType::eFbxInt:
			case EFbxType::eFbxUInt:
			case EFbxType::eFbxLongLong:
			case EFbxType::eFbxULongLong:
			case EFbxType::eFbxChar:
			{
				TArray<int32> IntValues;
				FillCurveAttributeToBone<int32>(TimeArray, IntValues, FbxCurve, AnimTimeSpan,
					[&ValueScale, &InProperty](const FbxAnimCurveKey* Key, const FbxTime* KeyTime) {
					if (Key)
					{
						return static_cast<int32>(Key->GetValue() * ValueScale);
					}
					else
					{
						return static_cast<int32>(InProperty.Get<int32>() * ValueScale);
					}
				});

				UE::Anim::AddTypedCustomAttribute<FIntegerAnimationAttribute, int32>(FName(CurveName), BoneName, TargetSequence, MakeArrayView(TimeArray), MakeArrayView(IntValues));
				break;
			}
			case EFbxType::eFbxString:
			{
				TArray<FString> StringValues;
				FillCurveAttributeToBone<FString>(TimeArray, StringValues, FbxCurve, AnimTimeSpan,
					[&ValueScale, &InProperty](const FbxAnimCurveKey* Key, const FbxTime* KeyTime) {
					if (KeyTime)
					{
						FbxPropertyValue& EvaluatedValue = InProperty.EvaluateValue(*KeyTime);
						FbxString StringValue;
						EvaluatedValue.Get(&StringValue, EFbxType::eFbxString);
						return FString(UTF8_TO_TCHAR(StringValue));
					}
					else
					{
						return FString(UTF8_TO_TCHAR(InProperty.Get<FbxString>()));
					}
				});

				UE::Anim::AddTypedCustomAttribute<FStringAnimationAttribute, FString>(FName(CurveName), BoneName, TargetSequence, MakeArrayView(TimeArray), MakeArrayView(StringValues));
				break;
			}
			case EFbxType::eFbxEnum:
			{
				// Enum-typed properties in FBX are converted to string-typed custom attributes using the string value
				// that corresponds to the enum index.
				TArray<FString> StringValues;
				FillCurveAttributeToBone<FString>(TimeArray, StringValues, FbxCurve, AnimTimeSpan,
					[&ValueScale, &InProperty](const FbxAnimCurveKey* Key, const FbxTime* KeyTime) {
					int32 EnumIndex = -1;

					if (KeyTime)
					{
						FbxPropertyValue& EvaluatedValue = InProperty.EvaluateValue(*KeyTime);
						EvaluatedValue.Get(&EnumIndex, EFbxType::eFbxEnum);
					}
					else
					{
						EnumIndex = InProperty.Get<FbxEnum>();
					}

					if (EnumIndex < 0 || EnumIndex >= InProperty.GetEnumCount())
					{
						return FString();
					}

					const char* EnumValue = InProperty.GetEnumValue(EnumIndex);
					return FString(UTF8_TO_TCHAR(EnumValue));
				});

				UE::Anim::AddTypedCustomAttribute<FStringAnimationAttribute, FString>(FName(CurveName), BoneName, TargetSequence, MakeArrayView(TimeArray), MakeArrayView(StringValues));
				break;
			}
			default:
			{
				AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("Warning_CustomAttributeTypeNotSupported", "Custom Attribute ({0}) could not be imported on bone, as its type {1} is not supported."), FText::FromString(CurveName), FText::AsNumber((int32)InProperty.GetPropertyDataType().GetType()))), FFbxErrors::Animation_InvalidData);
				return false;
			}
		}

		return true;
	}

	return false;
}

bool ShouldImportCurve(FbxAnimCurve* Curve, bool bDoNotImportWithZeroValues)
{
	if (Curve && Curve->KeyGetCount() > 0)
	{
		if (bDoNotImportWithZeroValues)
		{
			for (int32 KeyIndex = 0; KeyIndex < Curve->KeyGetCount(); ++KeyIndex)
			{
				if (!FMath::IsNearlyZero(Curve->KeyGetValue(KeyIndex)))
				{
					return true;
				}
			}
		}
		else
		{
			return true;
		}
	}

	return false;
}

bool UnFbx::FFbxImporter::ImportAnimation(USkeleton* Skeleton, UAnimSequence * DestSeq, const FString& FileName, TArray<FbxNode*>& SortedLinks, TArray<FbxNode*>& NodeArray, FbxAnimStack* CurAnimStack, const int32 ResampleRate, const FbxTimeSpan AnimTimeSpan, const bool bReimport)
{
	// @todo : the length might need to change w.r.t. sampling keys
	FbxTime SequenceLength = AnimTimeSpan.GetDuration();
	float PreviousSequenceLength = DestSeq->GetPlayLength();

	const bool bShouldTransact = bReimport;
	IAnimationDataModel::FReimportScope ReimportScope(DestSeq->GetDataModel());
	IAnimationDataController& Controller = DestSeq->GetController();
	Controller.OpenBracket(LOCTEXT("ImportAnimation_Bracket", "Importing Animation"), bShouldTransact);

	//This destroy all previously imported animation raw data
	Controller.RemoveAllBoneTracks(bShouldTransact);

	// First set frame rate
	const FFrameRate ResampleFrameRate(ResampleRate, 1);
	Controller.SetFrameRate(ResampleFrameRate, bShouldTransact);

	// if you have one pose(thus 0.f duration), it still contains animation, so we'll need to consider that as MINIMUM_ANIMATION_LENGTH time length
	const FFrameNumber NumberOfFrames = ResampleFrameRate.AsFrameNumber(SequenceLength.GetSecondDouble());
	Controller.SetNumberOfFrames(FGenericPlatformMath::Max<int32>(NumberOfFrames.Value, 1), bShouldTransact);

	if(PreviousSequenceLength > MINIMUM_ANIMATION_LENGTH && DestSeq->GetDataModel()->GetNumberOfFloatCurves() > 0)
	{
		// The sequence already existed when we began the import. We need to scale the key times for all curves to match the new 
		// duration before importing over them. This is to catch any user-added curves
		float ScaleFactor = DestSeq->GetPlayLength() / PreviousSequenceLength;
		if (!FMath::IsNearlyEqual(ScaleFactor, 1.f))
		{
			for (const FFloatCurve& Curve : DestSeq->GetDataModel()->GetFloatCurves())
			{
				const FAnimationCurveIdentifier CurveId(Curve.GetName(), ERawCurveTrackTypes::RCT_Float);
				Controller.ScaleCurve(CurveId, 0.f, ScaleFactor, bShouldTransact);
			}
		}
	}

	USkeleton* MySkeleton = DestSeq->GetSkeleton();
	check(MySkeleton);

	if (ImportOptions->bDeleteExistingMorphTargetCurves || ImportOptions->bDeleteExistingCustomAttributeCurves)
	{
		TArray<FName> CurveNamesToRemove;
		for (const FFloatCurve& Curve : DestSeq->GetDataModel()->GetFloatCurves())
		{
			const FCurveMetaData* MetaData = MySkeleton->GetCurveMetaData(Curve.GetName());
			if (MetaData)
			{
				bool bDeleteCurve = MetaData->Type.bMorphtarget ? ImportOptions->bDeleteExistingMorphTargetCurves : ImportOptions->bDeleteExistingCustomAttributeCurves;
				if (bDeleteCurve)
				{
					CurveNamesToRemove.Add(Curve.GetName());
				}
			}
		}

		for (auto CurveName : CurveNamesToRemove)
		{
			const FAnimationCurveIdentifier CurveId(CurveName, ERawCurveTrackTypes::RCT_Float);
			Controller.RemoveCurve(CurveId, bShouldTransact);
		}
	}

	if (ImportOptions->bDeleteExistingNonCurveCustomAttributes)
	{
		Controller.RemoveAllAttributes(bShouldTransact);
	}
	
	const bool bReimportWarnings = GetDefault<UEditorPerProjectUserSettings>()->bAnimationReimportWarnings;
	
	if (bReimportWarnings && !FMath::IsNearlyZero(PreviousSequenceLength) && !FMath::IsNearlyEqual(DestSeq->GetPlayLength(), PreviousSequenceLength))
	{
		AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("Warning_SequenceLengthChanged", "Animation Sequence ({0}) length {1} is different from previous {2}."), FText::FromName(DestSeq->GetFName()), DestSeq->GetPlayLength(), PreviousSequenceLength)), FFbxErrors::Animation_DifferentLength);
	}

	TArray<FName> FbxRawBoneNames;
	FillAndVerifyBoneNames(Skeleton, SortedLinks, FbxRawBoneNames, FileName);
	FAnimCurveImportSettings AnimImportSettings(DestSeq, NodeArray, SortedLinks, FbxRawBoneNames, AnimTimeSpan);

	//
	// import blend shape curves
	//
	int32 CurveAttributeKeyCount = 0;
	ImportBlendShapeCurves(AnimImportSettings, CurAnimStack, CurveAttributeKeyCount, bReimport);

	// importing custom attribute START
	TArray<FString> CurvesNotFound;
	if (ImportOptions->bImportCustomAttribute)
	{
		int CustomAttributeKeyCount = 0;
		
		ImportAnimationCustomAttribute(AnimImportSettings, CustomAttributeKeyCount, CurvesNotFound, bReimport);

		CurveAttributeKeyCount = FMath::Max(CurveAttributeKeyCount, CustomAttributeKeyCount);
	}
	else
	{
		// Store float curve tracks which use to exist on the animation
		for (const FFloatCurve& Curve : DestSeq->GetDataModel()->GetFloatCurves())
		{
			const FCurveMetaData* MetaData = MySkeleton->GetCurveMetaData(Curve.GetName());
			if (MetaData && !MetaData->Type.bMorphtarget)
			{
				CurvesNotFound.Add(Curve.GetName().ToString());
			}
		}
	}

	if (bReimportWarnings && CurvesNotFound.Num())
	{
		for (const FString& CurveName : CurvesNotFound)
		{
			AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("Warning_NonExistingCurve", "Curve ({0}) was not found in the new Animation."), FText::FromString(CurveName))), FFbxErrors::Animation_CurveNotFound);
		}
	}
	// importing custom attribute END
	
	int32 TotalNumKeys = 0;
	const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();

	// import animation
	if (ImportOptions->bImportBoneTracks)
	{
		FbxNode* SkeletalMeshRootNode = NodeArray.Num() > 0 ? NodeArray[0] : nullptr;
		ImportBoneTracks(Skeleton, AnimImportSettings, SkeletalMeshRootNode, ResampleRate, TotalNumKeys, bReimport);
	}
	else if (CurveAttributeKeyCount > 0)
	{
		Controller.SetFrameRate(FFrameRate(ResampleRate, 1), bShouldTransact);
	}

	// Import bone metadata to AnimSequence
	for (FbxNode* SkeletonNode : SortedLinks)
	{
		ImportNodeCustomProperties(DestSeq, SkeletonNode, true);
	}

	Controller.NotifyPopulated();
	Controller.CloseBracket(bShouldTransact);

	// Reregister skeletal mesh components so they reflect the updated animation
	for (TObjectIterator<USkeletalMeshComponent> Iter; Iter; ++Iter)
	{
		FComponentReregisterContext ReregisterContext(*Iter);
	}
	return true;
}

void UnFbx::FFbxImporter::ImportBlendShapeCurves(FAnimCurveImportSettings& AnimImportSettings, FbxAnimStack* CurAnimStack, int32& OutKeyCount, const bool bReimport)
{
	FText CurrentExportMessage = LOCTEXT("BeginImportMorphTargetCurves", "Importing Morph Target Curves");
	FScopedSlowTask SlowTaskNode(AnimImportSettings.NodeArray.Num(), CurrentExportMessage);
	SlowTaskNode.MakeDialog();

	OutKeyCount = 0;
	USkeleton* MySkeleton = AnimImportSettings.DestSeq->GetSkeleton();
	for (int32 NodeIndex = 0; NodeIndex < AnimImportSettings.NodeArray.Num(); NodeIndex++)
	{
		SlowTaskNode.EnterProgressFrame(1, CurrentExportMessage);

		// consider blendshape animation curve
		FbxGeometry* Geometry = (FbxGeometry*)AnimImportSettings.NodeArray[NodeIndex]->GetNodeAttribute();
		if (Geometry)
		{
			int32 BlendShapeDeformerCount = Geometry->GetDeformerCount(FbxDeformer::eBlendShape);
			FScopedSlowTask SlowTaskBlendShape(BlendShapeDeformerCount);

			for (int32 BlendShapeIndex = 0; BlendShapeIndex < BlendShapeDeformerCount; ++BlendShapeIndex)
			{
				SlowTaskBlendShape.EnterProgressFrame(1);
				FbxBlendShape* BlendShape = (FbxBlendShape*)Geometry->GetDeformer(BlendShapeIndex, FbxDeformer::eBlendShape);

				const int32 BlendShapeChannelCount = BlendShape->GetBlendShapeChannelCount();

				FString BlendShapeName = MakeName(BlendShape->GetName());

				// see below where this is used for explanation...
				const bool bMightBeBadMAXFile = (BlendShapeName == FString("Morpher"));
				FScopedSlowTask SlowTaskChannel(BlendShapeChannelCount);
				for (int32 ChannelIndex = 0; ChannelIndex < BlendShapeChannelCount; ++ChannelIndex)
				{
					FbxBlendShapeChannel* Channel = BlendShape->GetBlendShapeChannel(ChannelIndex);
					bool bUpdatedProgress = false;

					if (Channel)
					{
						FString ChannelName = MakeName(Channel->GetName());
						// Maya adds the name of the blendshape and an underscore or point to the front of the channel name, so remove it
						// Also avoid to endup with a empty name, we prefer having the Blendshapename instead of nothing
						if (ChannelName.StartsWith(BlendShapeName) && ChannelName.Len() > BlendShapeName.Len())
						{
							ChannelName.RightInline(ChannelName.Len() - (BlendShapeName.Len() + 1), EAllowShrinking::No);
						}

						if (bMightBeBadMAXFile)
						{
							FbxShape* TargetShape = Channel->GetTargetShapeCount() > 0 ? Channel->GetTargetShape(0) : nullptr;
							if (TargetShape)
							{
								FString TargetShapeName = MakeName(TargetShape->GetName());
								ChannelName = TargetShapeName.IsEmpty() ? ChannelName : TargetShapeName;
							}
						}

						FbxAnimCurve* Curve = Geometry->GetShapeChannel(BlendShapeIndex, ChannelIndex, (FbxAnimLayer*)CurAnimStack->GetMember(0));
						if (FbxAnimUtils::ShouldImportCurve(Curve, ImportOptions->bDoNotImportCurveWithZero))
						{
							FFormatNamedArguments Args;
							Args.Add(TEXT("BlendShape"), FText::FromString(ChannelName));
							CurrentExportMessage = FText::Format(LOCTEXT("ImportingMorphTargetCurvesDetail", "Importing Morph Target Curves [{BlendShape}]"), Args);
							SlowTaskNode.FrameMessage = CurrentExportMessage;
							SlowTaskChannel.EnterProgressFrame(1);
							bUpdatedProgress = true;

							const int32 TargetShapeCount = Channel->GetTargetShapeCount();

							if (ensure(TargetShapeCount > 0))
							{
								if (TargetShapeCount == 1)
								{
									// now see if we have one already exists. If so, just overwrite that. if not, add new one. 

									if (ImportCurveToAnimSequence(AnimImportSettings.DestSeq, *ChannelName, Curve, 0, AnimImportSettings.AnimTimeSpan, bReimport, 0.01f /** for some reason blend shape values are coming as 100 scaled **/))
									{
										OutKeyCount = FMath::Max(OutKeyCount, Curve->KeyGetCount());
										if(ImportOptions->bAddCurveMetadataToSkeleton)
										{
											// this one doesn't reset Material curve to false, it just accumulate if true. 
											MySkeleton->AccumulateCurveMetaData(*ChannelName, false, true);
										}
									}
								}
								else
								{
									// the blend shape channel can have multiple inbetween target shapes
									// since the engine does not directly support inbetween morphs at runtime,
									// each inbetween is imported as a standalone blendshape.
									// as a result we have to create a curve for each one of those inbetweens
									// and modify the primary channel curve such that their combined effect preserves
									// the original animation
							
									TArray<FString> CurveNames;
									CurveNames.Reserve(TargetShapeCount);

									// in fbx the primary shape is the last shape, however to make
									// the code more similar to usd importer, we deal with the primary shape separately
									CurveNames.Add(ChannelName);

									// ignoring the last shape because it is not a inbetween, i.e. it is the primary shape
									int32 InbetweenCount = TargetShapeCount - 1;
							
									TArrayView<double> FbxInbetweenFullWeights = {Channel->GetTargetShapeFullWeights(), InbetweenCount};

									TArray<float> InbetweenFullWeights;
									InbetweenFullWeights.Reserve(InbetweenCount);
									/** for some reason blend shape values are coming as 100 scaled, so a transform is needed to scale it to 0-1 **/
									Algo::Transform(FbxInbetweenFullWeights, InbetweenFullWeights, [](double Input){ return Input * 0.01f; });

									// collect inbetween shape names
									for (int32 InbetweenIndex = 0; InbetweenIndex < InbetweenCount; ++InbetweenIndex)
									{
										FbxShape* Shape = Channel->GetTargetShape(InbetweenIndex);
										CurveNames.Add(MakeName(Shape->GetName()));
									}
						
									// first convert fbx curve to rich curve
									FRichCurve ChannelWeightCurve;
									ImportCurve(Curve, ChannelWeightCurve,  AnimImportSettings.AnimTimeSpan, false, 0.01f /** for some reason blend shape values are coming as 100 scaled **/);
									if (ensure(AnimImportSettings.DestSeq))
									{
#if WITH_EDITORONLY_DATA
										ChannelWeightCurve.BakeCurve(1.0f / AnimImportSettings.DestSeq->ImportResampleFramerate);
#endif
									}
									
									// use the primary curve to generate inbetween shape curves + a modified primary curve
									TArray<FRichCurve> Results = ResolveWeightsForBlendShapeCurve(ChannelWeightCurve, InbetweenFullWeights);

									for (FRichCurve& Result : Results)
									{
										if (ImportOptions->bRemoveRedundantKeys)
										{
											Result.RemoveRedundantAutoTangentKeys(SMALL_NUMBER);
										}
									}

									if (ImportRichCurvesToAnimSequence(AnimImportSettings.DestSeq, CurveNames, Results, 0, bReimport))
									{
										OutKeyCount = FMath::Max(OutKeyCount, Curve->KeyGetCount());
										if(ImportOptions->bAddCurveMetadataToSkeleton)
										{
											for (const FString& CurveName : CurveNames)
											{
												// this one doesn't reset Material curve to false, it just accumulate if true. 
												MySkeleton->AccumulateCurveMetaData(*CurveName, false, true);
											}
										}
									}
								}
							}
						}
						else
						{
							UE_LOGF(LogFbx, Warning, "CurveName(%ls) is skipped because it only contains invalid values.", *ChannelName);
						}
					}

					if (!bUpdatedProgress)
					{
						SlowTaskChannel.EnterProgressFrame(1);
					}
				}
			}
		}
	}
}

void UnFbx::FFbxImporter::ImportAnimationCustomAttribute(FAnimCurveImportSettings& AnimImportSettings, int32& OutKeyCount, TArray<FString>& OutCurvesNotFound, const bool bReimport)
{
	FScopedSlowTask SlowTask(AnimImportSettings.SortedLinks.Num(), LOCTEXT("BeginImportCustomAttributeCurves", "Importing Custom Attribute Curves"), true);
	SlowTask.MakeDialog();

	// Store float curve tracks which use to exist on the animation
	UAnimSequence* DestSeq = AnimImportSettings.DestSeq;
	USkeleton* MySkeleton = DestSeq->GetSkeleton();

	const IAnimationDataModel* DataModel = DestSeq->GetDataModel();
	const int32 NumFloatCurves = DataModel->GetNumberOfFloatCurves();
	const FAnimationCurveData& CurveData = DataModel->GetCurveData();

	OutCurvesNotFound.Reset(NumFloatCurves);

	for (const FFloatCurve& FloatCurve : CurveData.FloatCurves)
	{
		const FCurveMetaData* MetaData = MySkeleton->GetCurveMetaData(FloatCurve.GetName());

		if (MetaData && !MetaData->Type.bMorphtarget)
		{
			OutCurvesNotFound.Add(FloatCurve.GetName().ToString());
		}
	}

	OutKeyCount = 0;
	for (int32 LinkIndex = 0; LinkIndex < AnimImportSettings.SortedLinks.Num(); ++LinkIndex)
	{
		FbxNode* Node = AnimImportSettings.SortedLinks[LinkIndex];
		FName BoneName = AnimImportSettings.FbxRawBoneNames[LinkIndex];
		bool bImportAllAttributesOnBone = UAnimationSettings::Get()->BoneNamesWithCustomAttributes.Contains(BoneName.ToString());
		SlowTask.EnterProgressFrame(1);

		if (!bImportAllAttributesOnBone)
		{
			FbxAnimUtils::ExtractAttributeCurves(Node, ImportOptions->bDoNotImportCurveWithZero,
				[this, &LinkIndex, &DestSeq, &AnimImportSettings, &OutCurvesNotFound, &OutKeyCount, &SlowTask, bReimport](FbxAnimCurve* InCurve, const FString& InCurveName)
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("CurveName"), FText::FromString(InCurveName));
				const FText StatusUpate = FText::Format(LOCTEXT("ImportingCustomAttributeCurvesDetail", "Importing Custom Attribute [{CurveName}]"), Args);
				SlowTask.EnterProgressFrame(0, StatusUpate);

				int32 CurveFlags = AACF_DefaultCurve;
				if (ImportCurveToAnimSequence(DestSeq, InCurveName, InCurve, CurveFlags, AnimImportSettings.AnimTimeSpan, bReimport))
				{
					OutKeyCount = FMath::Max(OutKeyCount, InCurve->KeyGetCount());

					if(ImportOptions->bAddCurveMetadataToSkeleton)
					{
						USkeleton* SeqSkeleton = DestSeq->GetSkeleton();

						// first let them override material curve if required
						if (ImportOptions->bSetMaterialDriveParameterOnCustomAttribute)
						{
							// now mark this curve as material curve
							SeqSkeleton->AccumulateCurveMetaData(FName(*InCurveName), true, false);
						}
						else
						{
							// if not material set by default, apply naming convention for material
							for (const auto& Suffix : ImportOptions->MaterialCurveSuffixes)
							{
								int32 TotalSuffix = Suffix.Len();
								if (InCurveName.Right(TotalSuffix) == Suffix)
								{
									SeqSkeleton->AccumulateCurveMetaData(FName(*InCurveName), true, false);
									break;
								}
							}
						}
					}

					OutCurvesNotFound.Remove(InCurveName);
				}
			});
		}

		FbxAnimUtils::ExtractNodeAttributes(Node, ImportOptions->bDoNotImportCurveWithZero, bImportAllAttributesOnBone,
			[this, &OutKeyCount, &DestSeq, &AnimImportSettings, &BoneName, bReimport](FbxProperty& InProperty, FbxAnimCurve* InCurve, const FString& InCurveName)
		{
			if (ImportCustomAttributeToBone(DestSeq, InProperty, BoneName, InCurveName, InCurve, AnimImportSettings.AnimTimeSpan, bReimport))
			{
				OutKeyCount = FMath::Max(OutKeyCount, InCurve ? InCurve->KeyGetCount() : 1);
			}
		});
	}
}

void UnFbx::FFbxImporter::ImportBoneTracks(USkeleton* Skeleton, FAnimCurveImportSettings& AnimImportSettings, FbxNode* SkeletalMeshRootNode, const int32 ResampleRate, int32& OutTotalNumKeys, const bool bReimport)
{
	OutTotalNumKeys = 0;
	UnFbx::FFbxImporter* FbxImporter = UnFbx::FFbxImporter::GetInstance();
	const bool bPreserveLocalTransform = FbxImporter->GetImportOptions()->bPreserveLocalTransform;

	UAnimSequence* DestSeq = AnimImportSettings.DestSeq;
	const FbxTimeSpan& AnimTimeSpan = AnimImportSettings.AnimTimeSpan;
	const bool bShouldTransact = bReimport;

	// Build additional transform matrix
	UFbxAnimSequenceImportData* TemplateData = Cast<UFbxAnimSequenceImportData>(DestSeq->AssetImportData);
	FbxAMatrix FbxAddedMatrix;
	BuildFbxMatrixForImportTransform(FbxAddedMatrix, TemplateData);
	FMatrix AddedMatrix = Converter.ConvertMatrix(FbxAddedMatrix);
	FTransform AddedTransform(AddedMatrix);

	bool bIsRigidMeshAnimation = false;
	if (ImportOptions->bImportScene && AnimImportSettings.SortedLinks.Num() > 0)
	{
		for (int32 BoneIdx = 0; BoneIdx < AnimImportSettings.SortedLinks.Num(); ++BoneIdx)
		{
			FbxNode* Link = AnimImportSettings.SortedLinks[BoneIdx];
			if (Link->GetMesh() && Link->GetMesh()->GetDeformerCount(FbxDeformer::eSkin) == 0)
			{
				bIsRigidMeshAnimation = true;
				break;
			}
		}
	}

	// Prepare per-track data

	const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();

	int32 SourceTrackNum = AnimImportSettings.FbxRawBoneNames.Num();
	int32 KeyTotal = FMath::RoundToInt((AnimTimeSpan.GetDuration().GetSecondDouble() * ResampleRate)) + 1; // Expected Number of Keys

	TArray<int32> BoneTreeIndices;
	TArray<bool> SourceTrackValid;
	TArray<FRawAnimSequenceTrack> RawTracks;
	TArray<TArray<float>> TimeKeys;

	BoneTreeIndices.SetNum(SourceTrackNum);
	SourceTrackValid.SetNum(SourceTrackNum);
	RawTracks.SetNum(SourceTrackNum);
	TimeKeys.SetNum(SourceTrackNum);

	for (int32 SourceTrackIdx = 0; SourceTrackIdx < SourceTrackNum; ++SourceTrackIdx)
	{
		BoneTreeIndices[SourceTrackIdx] = RefSkeleton.FindBoneIndex(AnimImportSettings.FbxRawBoneNames[SourceTrackIdx]);
		SourceTrackValid[SourceTrackIdx] = BoneTreeIndices[SourceTrackIdx] != INDEX_NONE || IsUnrealTransformAttribute(AnimImportSettings.SortedLinks[SourceTrackIdx]);
		RawTracks[SourceTrackIdx].PosKeys.Reserve(KeyTotal);
		RawTracks[SourceTrackIdx].RotKeys.Reserve(KeyTotal);
		RawTracks[SourceTrackIdx].ScaleKeys.Reserve(KeyTotal);
		TimeKeys[SourceTrackIdx].Reserve(KeyTotal);
	}

	// Set the time increment from the re-sample rate

	FbxTime TimeIncrement = 0;
	TimeIncrement.SetSecondDouble(1.0 / ((double)(ResampleRate)));

	// Add a threshold when we compare if we have reach the end of the animation
	const FbxTime TimeComparisonThreshold = (KINDA_SMALL_NUMBER * static_cast<float>(FBXSDK_TC_SECOND));

	// Loop Over Key Times

	FScopedSlowTask SlowTask(KeyTotal, LOCTEXT("BeginImportAnimation", "Importing Animation"), true);
	SlowTask.MakeDialog();

	int32 KeyNum = 0;
	for (FbxTime CurTime = AnimTimeSpan.GetStart(); CurTime < (AnimTimeSpan.GetStop() + TimeComparisonThreshold); CurTime += TimeIncrement)
	{
		// Update Status

		FFormatNamedArguments Args;
		Args.Add(TEXT("KeyNum"), FText::AsNumber(KeyNum + 1));
		Args.Add(TEXT("KeyTotal"), FText::AsNumber(KeyTotal));
		const FText StatusUpate = FText::Format(LOCTEXT("ImportingAnimTrackDetail", "Importing Animation Keys {KeyNum} of {KeyTotal}"), Args);
		SlowTask.EnterProgressFrame(1.0f, StatusUpate);

		// Loop Over Tracks

		for (int32 SourceTrackIdx = 0; SourceTrackIdx < SourceTrackNum; ++SourceTrackIdx)
		{
			// Skip Invalid Tracks
			if (!SourceTrackValid[SourceTrackIdx])
			{
				continue;
			}

			// Get Bone name and index.
			FName BoneName = AnimImportSettings.FbxRawBoneNames[SourceTrackIdx];
			int32 BoneTreeIndex = BoneTreeIndices[SourceTrackIdx];

			// Get Links
			FbxNode* Link = AnimImportSettings.SortedLinks[SourceTrackIdx];
			FbxNode* LinkParent = Link->GetParent();

			FTransform LocalTransform;
			if (!bPreserveLocalTransform && LinkParent)
			{
				// get global transform
				FbxAMatrix GlobalMatrix = Link->EvaluateGlobalTransform(CurTime) * FFbxDataConverter::GetJointPostConversionMatrix();

				// we'd like to verify this before going to Transform. 
				// currently transform has tons of NaN check, so it will crash there
				FMatrix GlobalUEMatrix = Converter.ConvertMatrix(GlobalMatrix);
				if (GlobalUEMatrix.ContainsNaN())
				{
					SourceTrackValid[SourceTrackIdx] = false;
					AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("Error_InvalidTransform", "Track {0} contains invalid transform. Could not import the track."), FText::FromName(BoneName))), FFbxErrors::Animation_TransformError);
					continue;
				}

				FTransform GlobalTransform = Converter.ConvertTransform(GlobalMatrix);
				if (GlobalTransform.ContainsNaN())
				{
					SourceTrackValid[SourceTrackIdx] = false;
					AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("Error_InvalidUnrealTransform", "Track {0} has invalid transform(NaN). Zero scale transform can cause this issue."), FText::FromName(BoneName))), FFbxErrors::Animation_TransformError);
					continue;
				}

				// I can't rely on LocalMatrix. I need to recalculate quaternion/scale based on global transform if Parent exists
				FbxAMatrix ParentGlobalMatrix = Link->GetParent()->EvaluateGlobalTransform(CurTime);
				if (BoneTreeIndex != 0)
				{
					ParentGlobalMatrix = ParentGlobalMatrix * FFbxDataConverter::GetJointPostConversionMatrix();
				}
				FTransform ParentGlobalTransform = Converter.ConvertTransform(ParentGlobalMatrix);
				//In case we do a scene import we need to add the skeletal mesh root node matrix to the parent link.
				if (ImportOptions->bImportScene && !ImportOptions->bTransformVertexToAbsolute && BoneTreeIndex == 0 && SkeletalMeshRootNode != nullptr)
				{
					//In the case of a rigidmesh animation we have to use the skeletalMeshRootNode position at zero since the mesh can be animate.
					FbxAMatrix GlobalSkeletalNodeFbx = bIsRigidMeshAnimation ? SkeletalMeshRootNode->EvaluateGlobalTransform(0) : SkeletalMeshRootNode->EvaluateGlobalTransform(CurTime);
					FTransform GlobalSkeletalNode = Converter.ConvertTransform(GlobalSkeletalNodeFbx);
					ParentGlobalTransform = ParentGlobalTransform * GlobalSkeletalNode;
				}

				LocalTransform = GlobalTransform.GetRelativeTransform(ParentGlobalTransform);
			}
			else
			{
				FbxAMatrix& LocalMatrix = Link->EvaluateLocalTransform(CurTime);
				FbxVector4 NewLocalT = LocalMatrix.GetT();
				FbxVector4 NewLocalS = LocalMatrix.GetS();
				FbxQuaternion NewLocalQ = LocalMatrix.GetQ();

				LocalTransform.SetTranslation(Converter.ConvertPos(NewLocalT));
				LocalTransform.SetScale3D(Converter.ConvertScale(NewLocalS));
				LocalTransform.SetRotation(Converter.ConvertRotToQuat(NewLocalQ));
			}

			if (TemplateData && BoneTreeIndex == 0)
			{
				// If we found template data earlier, apply the import transform matrix to
				// the root track.
				LocalTransform.SetFromMatrix(LocalTransform.ToMatrixWithScale() * AddedMatrix);
			}

			if (LocalTransform.ContainsNaN())
			{
				SourceTrackValid[SourceTrackIdx] = false;
				AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("Error_InvalidUnrealLocalTransform", "Track {0} has invalid transform(NaN). If you have zero scale transform, that can cause this."), FText::FromName(BoneName))), FFbxErrors::Animation_TransformError);
				continue;
			}

			// Add data to RawTracks, TimeKeys

			RawTracks[SourceTrackIdx].ScaleKeys.Add(FVector3f(LocalTransform.GetScale3D()));
			RawTracks[SourceTrackIdx].PosKeys.Add(FVector3f(LocalTransform.GetTranslation()));
			RawTracks[SourceTrackIdx].RotKeys.Add(FQuat4f(LocalTransform.GetRotation()));
			TimeKeys[SourceTrackIdx].Add((CurTime - AnimTimeSpan.GetStart()).GetSecondDouble());
		}

		// Update Key Num

		KeyNum++;
	}

	// Compute Total Number of Keys

	for (int32 SourceTrackIdx = 0; SourceTrackIdx < SourceTrackNum; ++SourceTrackIdx)
	{
		OutTotalNumKeys = FMath::Max(OutTotalNumKeys, TimeKeys[SourceTrackIdx].Num());
	}

	// Put raw track data into animation
	
	IAnimationDataController& Controller = DestSeq->GetController();
	Controller.SetFrameRate(FFrameRate(ResampleRate, 1), bShouldTransact);
	
	for (int32 SourceTrackIdx = 0; SourceTrackIdx < SourceTrackNum; ++SourceTrackIdx)
	{
		if (!SourceTrackValid[SourceTrackIdx])
		{
			continue;
		}

		FName BoneName = AnimImportSettings.FbxRawBoneNames[SourceTrackIdx];
		int32 BoneTreeIndex = BoneTreeIndices[SourceTrackIdx];

		FbxNode* Link = AnimImportSettings.SortedLinks[SourceTrackIdx];
		FbxNode* LinkParent = Link->GetParent();

		const int32 SourceTrackKeyNum = TimeKeys[SourceTrackIdx].Num();

		if (BoneTreeIndex != INDEX_NONE)
		{
			//add new track
			if (BoneName.GetStringLength() > 92)
			{
				//The bone name exceed the maximum length supported by the animation system
				//The animation system is adding _CONTROL to the bone name to name the animation controller and
				//the maximum total length is cap at 100, so user should not import bone name longer then 92 characters
				AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("Error_BoneNameExceed92Characters", "Bone with animation cannot have a name exceeding 92 characters: {0}"), FText::FromName(BoneName))), FFbxErrors::Animation_InvalidData);
				continue;
			}
			Controller.AddBoneCurve(BoneName, bShouldTransact);
			Controller.SetBoneTrackKeys(BoneName, RawTracks[SourceTrackIdx].PosKeys, RawTracks[SourceTrackIdx].RotKeys, RawTracks[SourceTrackIdx].ScaleKeys, bShouldTransact);
		}
		else if (SourceTrackKeyNum > 0) // add transform attribute
		{
			FbxNode* TargetBoneLink = LinkParent;
			while (TargetBoneLink != nullptr && !IsUnrealBone(TargetBoneLink))
			{
				TargetBoneLink = TargetBoneLink->GetParent();
			}

			if (TargetBoneLink)
			{
				int32 TargetBoneTrackIndex = AnimImportSettings.SortedLinks.Find(TargetBoneLink);
				if (TargetBoneTrackIndex != INDEX_NONE)
				{
					FName TargetBoneName = AnimImportSettings.FbxRawBoneNames[TargetBoneTrackIndex];
					if (RefSkeleton.FindBoneIndex(TargetBoneName) != INDEX_NONE)
					{
						FAnimationAttributeIdentifier AttributeIdentifier = UAnimationAttributeIdentifierExtensions::CreateAttributeIdentifier(DestSeq, FName(BoneName), TargetBoneName, FTransformAnimationAttribute::StaticStruct());
						if (AttributeIdentifier.IsValid())
						{
							// remove any existing attribute with the same identifier
							if (const IAnimationDataModel* Model = Controller.GetModel())
							{
								if (Model->FindAttribute(AttributeIdentifier))
								{
									Controller.RemoveAttribute(AttributeIdentifier, bShouldTransact);
								}
							}

							// pack the separate rot/pos/scale key arrays into a single array
							TArray<FTransform> TransformValues;
							TransformValues.Reserve(SourceTrackKeyNum);
							for (int32 KeyIndex = 0; KeyIndex < SourceTrackKeyNum; ++KeyIndex)
							{
								const FQuat Q(RawTracks[SourceTrackIdx].RotKeys[KeyIndex]);
								const FVector T(RawTracks[SourceTrackIdx].PosKeys[KeyIndex]);
								const FVector S(RawTracks[SourceTrackIdx].ScaleKeys[KeyIndex]);
								TransformValues.Add(FTransform(Q, T, S));
							}

							// reduce keys for the common case where all of the keys have the same values
							bool bReduceKeys = true;
							for (int32 KeyIndex = 1; KeyIndex < SourceTrackKeyNum; ++KeyIndex)
							{
								if (!TransformValues[KeyIndex].Equals(TransformValues[0]))
								{
									bReduceKeys = false;
									break;
								}
							}

							// create the attribute and add the transform keys
							const int32 NumAttributeKeys = (bReduceKeys) ? 1 : SourceTrackKeyNum;
							UE::Anim::AddTypedCustomAttribute<FTransformAnimationAttribute, FTransform>(FName(BoneName), TargetBoneName, DestSeq, MakeArrayView(TimeKeys[SourceTrackIdx].GetData(), NumAttributeKeys), MakeArrayView(TransformValues.GetData(), NumAttributeKeys), bShouldTransact);
						}
					}
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
