// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkinWeightsUtilities.h"
#include "LODUtilities.h"
#include "Modules/ModuleManager.h"
#include "Components/SkinnedMeshComponent.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "EditorFramework/AssetImportData.h"
#include "MeshUtilities.h"

#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "Factories/FbxFactory.h"
#include "Factories/FbxAnimSequenceImportData.h"
#include "Factories/FbxSkeletalMeshImportData.h"
#include "Factories/FbxStaticMeshImportData.h"
#include "Factories/FbxTextureImportData.h"
#include "Factories/FbxImportUI.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ObjectTools.h"
#include "AssetImportTask.h"
#include "FbxImporter.h"
#include "ScopedTransaction.h"

#include "ComponentReregisterContext.h"
#include "Rendering/SkinWeightProfile.h"

#include "IDesktopPlatform.h"
#include "DesktopPlatformModule.h"
#include "EditorDirectories.h"
#include "Framework/Application/SlateApplication.h"
#include "InterchangeAssetImportData.h"
#include "InterchangeFilePickerBase.h"
#include "InterchangePipelineBase.h"
#include "InterchangeManager.h"
#include "InterchangeProjectSettings.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Misc/CoreMisc.h"

DEFINE_LOG_CATEGORY_STATIC(LogSkinWeightsUtilities, Log, All);

bool FSkinWeightsUtilities::ImportAlternateSkinWeight(USkeletalMesh* SkeletalMesh, const FString& Path, int32 TargetLODIndex, const FName& ProfileName, const bool bIsReimport)
{
	check(SkeletalMesh);
	FSkeletalMeshLODInfo* LODInfo = SkeletalMesh->GetLODInfo(TargetLODIndex);

	if (!LODInfo)
	{
		UE_LOGF(LogSkinWeightsUtilities, Error, "Cannot import Skin Weight Profile. No valid LOD info.");
		return false;
	}

	if (LODInfo->bHasBeenSimplified && LODInfo->ReductionSettings.BaseLOD != TargetLODIndex)
	{
		//We cannot import alternate skin weights profile for a generated LOD
		UE_LOGF(LogSkinWeightsUtilities, Error, "Cannot import Skin Weight Profile for a generated LOD. Skin weight profile are transfert from the source LOD during the simplification process.");
		return false;
	}

	FString AbsoluteFilePath = UAssetImportData::ResolveImportFilename(Path, SkeletalMesh->GetOutermost());
	if (!FPaths::FileExists(AbsoluteFilePath))
	{
		UE_LOGF(LogSkinWeightsUtilities, Error, "Path containing Skin Weight Profile data does not exist (%ls).", *Path);
		return false;
	}

	FString Action = bIsReimport ? TEXT("Reimport") : TEXT("Import");
	UE_ASSET_LOG(LogSkinWeightsUtilities, Display, SkeletalMesh, TEXT("%s Alternate skin weight Begin [LodIndex: %d] [Profile: %s] [file: %s]."), *Action, TargetLODIndex, *ProfileName.ToString(), *Path);

	FScopedSuspendAlternateSkinWeightPreview ScopedSuspendAlternateSkinnWeightPreview(SkeletalMesh);
	FScopedSkeletalMeshPostEditChange ScopePostEditChange(SkeletalMesh);

	const FString FileExtension = FPaths::GetExtension(AbsoluteFilePath);

	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	FString ImportAssetPath = TEXT("/Engine/TempEditor/SkeletalMeshTool");
	//Empty the temporary path
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

	auto DeletePathAssets = [&AssetRegistryModule, &ImportAssetPath]()
	{
		TArray<FAssetData> AssetsToDelete;
		AssetRegistryModule.Get().GetAssetsByPath(FName(*ImportAssetPath), AssetsToDelete, true);
		for (FAssetData AssetData : AssetsToDelete)
		{
			UObject* ObjToDelete = AssetData.GetAsset();
			if (ObjToDelete)
			{
				//Avoid temporary package to be saved
				UPackage* Package = ObjToDelete->GetOutermost();
				Package->SetDirtyFlag(false);
				//Avoid gc, use keep flags
				ObjToDelete->ClearFlags(RF_Standalone);
				ObjToDelete->ClearInternalFlags(EInternalObjectFlags::Async);
				//Make the object transient to prevent saving
				ObjToDelete->SetFlags(RF_Transient);
			}
		}
	};

	DeletePathAssets();

	UObject* ImportedObject = nullptr;

	bool bCreateTransaction = false;

	UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
	const UInterchangeSourceData* SourceData = InterchangeManager.CreateSourceData(AbsoluteFilePath);
	const bool bInterchangeCanImportSourceData = InterchangeManager.CanTranslateSourceData(SourceData);
	if (bInterchangeCanImportSourceData)
	{
		UInterchangeAssetImportData* SelectedInterchangeAssetImportData = Cast<UInterchangeAssetImportData>(SkeletalMesh->GetAssetImportData());
		if (!SelectedInterchangeAssetImportData)
		{
			//Try to convert the asset import data
			InterchangeManager.ConvertImportData(SkeletalMesh->GetAssetImportData(), UInterchangeAssetImportData::StaticClass(), reinterpret_cast<UObject**>(&SelectedInterchangeAssetImportData));
		}

		UE::Interchange::FScopedSourceData ScopedSourceData(AbsoluteFilePath);
		const UInterchangeProjectSettings* InterchangeProjectSettings = GetDefault<UInterchangeProjectSettings>();
		FImportAssetParameters ImportAssetParameters;
		ImportAssetParameters.bIsAutomated = true; // From the InterchangeManager point of view, this is considered an automated import

		if (const UClass* GenericPipelineClass = InterchangeProjectSettings->GenericPipelineClass.LoadSynchronous())
		{
			UInterchangePipelineBase* GenericPipeline = nullptr;
			if (SelectedInterchangeAssetImportData)
			{
				for (UObject* PipelineObject : SelectedInterchangeAssetImportData->GetPipelines())
				{
					if (PipelineObject->GetClass()->IsChildOf(GenericPipelineClass))
					{
						if (UInterchangePipelineBase* ImportPipeline = Cast<UInterchangePipelineBase>(PipelineObject))
						{
							GenericPipeline = Cast<UInterchangePipelineBase>(StaticDuplicateObject(ImportPipeline, GetTransientPackage()));
							break;
						}
					}
				}
			}
			
			if (!GenericPipeline)
			{
				GenericPipeline = NewObject<UInterchangePipelineBase>(GetTransientPackage(), GenericPipelineClass);
			}

			if (GenericPipeline)
			{
				GenericPipeline->ClearFlags(EObjectFlags::RF_Standalone | EObjectFlags::RF_Public);
				FInterchangePipelineContextParams ContextParams;
				ContextParams.ContextType = bIsReimport ? EInterchangePipelineContext::AssetAlternateSkinningReimport : EInterchangePipelineContext::AssetAlternateSkinningImport;
				ContextParams.ImportObjectType = USkeletalMesh::StaticClass();
				GenericPipeline->AdjustSettingsForContext(ContextParams);
				ImportAssetParameters.OverridePipelines.Add(GenericPipeline);
			}
		}

		ImportAssetParameters.DestinationName = FGuid::NewGuid().ToString(EGuidFormats::Digits);
		//TODO create a pipeline that set all the proper skeletalmesh options (look at the legacy system setup)
		UE::Interchange::FAssetImportResultRef AssetImportResult = InterchangeManager.ImportAssetAsync(ImportAssetPath, ScopedSourceData.GetSourceData(), ImportAssetParameters);
		AssetImportResult->WaitUntilDone(); //TODO, do not stall the main thread here, WaitUntilDone will tick taskgraph so the job can complete even if we wait on the game thread.
		if (USkeletalMesh* ImportedSkeletalMesh = Cast< USkeletalMesh >(AssetImportResult->GetFirstAssetOfClass(USkeletalMesh::StaticClass())))
		{
			ImportedObject = ImportedSkeletalMesh;
		}
	}
	else if(FileExtension.Equals(TEXT("fbx"), ESearchCase::IgnoreCase))
	{
		UFbxSkeletalMeshImportData* OriginalSkeletalMeshImportData = Cast<UFbxSkeletalMeshImportData>(SkeletalMesh->GetAssetImportData());
		if (!OriginalSkeletalMeshImportData)
		{
			//Convert the data if its Interchange import data
			if (UInterchangeAssetImportData* InterchangeAssetImportData = Cast<UInterchangeAssetImportData>(SkeletalMesh->GetAssetImportData()))
			{
				InterchangeManager.ConvertImportData(InterchangeAssetImportData, UFbxSkeletalMeshImportData::StaticClass(), reinterpret_cast<UObject**>(&OriginalSkeletalMeshImportData));
			}

			if (!OriginalSkeletalMeshImportData)
			{
				//This will reset the import data
				OriginalSkeletalMeshImportData = UFbxSkeletalMeshImportData::GetImportDataForSkeletalMesh(SkeletalMesh, nullptr);
			}
		}

		//Import the alternate fbx into a temporary skeletal mesh using the same import options
		UFbxFactory* FbxFactory = NewObject<UFbxFactory>(UFbxFactory::StaticClass());
		FbxFactory->AddToRoot();

		FbxFactory->ImportUI = NewObject<UFbxImportUI>(FbxFactory);
		if (OriginalSkeletalMeshImportData != nullptr)
		{
			//Copy the skeletal mesh import data options
			FbxFactory->ImportUI->SkeletalMeshImportData = DuplicateObject<UFbxSkeletalMeshImportData>(OriginalSkeletalMeshImportData, FbxFactory);
		}
		//Skip the auto detect type on import, the test set a specific value
		FbxFactory->SetDetectImportTypeOnImport(false);
		FbxFactory->ImportUI->bImportAsSkeletal = true;
		FbxFactory->ImportUI->MeshTypeToImport = FBXIT_SkeletalMesh;
		FbxFactory->ImportUI->bIsReimport = false;
		FbxFactory->ImportUI->ReimportMesh = nullptr;
		FbxFactory->ImportUI->bAllowContentTypeImport = true;
		FbxFactory->ImportUI->bImportAnimations = false;
		FbxFactory->ImportUI->bAutomatedImportShouldDetectType = false;
		FbxFactory->ImportUI->bCreatePhysicsAsset = false;
		FbxFactory->ImportUI->bImportMaterials = false;
		FbxFactory->ImportUI->bImportTextures = false;
		FbxFactory->ImportUI->bImportMesh = true;
		FbxFactory->ImportUI->bImportRigidMesh = false;
		FbxFactory->ImportUI->bIsObjImport = false;
		FbxFactory->ImportUI->bOverrideFullName = true;
		FbxFactory->ImportUI->Skeleton = nullptr;

		//Force some skeletal mesh import options
		if (FbxFactory->ImportUI->SkeletalMeshImportData)
		{
			FbxFactory->ImportUI->SkeletalMeshImportData->bImportMeshLODs = false;
			FbxFactory->ImportUI->SkeletalMeshImportData->bImportMorphTargets = false;
			FbxFactory->ImportUI->SkeletalMeshImportData->bUpdateSkeletonReferencePose = false;
			FbxFactory->ImportUI->SkeletalMeshImportData->ImportContentType = EFBXImportContentType::FBXICT_All; //We need geo and skinning, so we can match the weights
		}
		//Force some material options
		if (FbxFactory->ImportUI->TextureImportData)
		{
			FbxFactory->ImportUI->TextureImportData->MaterialSearchLocation = EMaterialSearchLocation::DoNotSearch;
			FbxFactory->ImportUI->TextureImportData->BaseMaterialName.Reset();
		}

		TArray<FString> ImportFilePaths;
		ImportFilePaths.Add(AbsoluteFilePath);

		UAssetImportTask* Task = NewObject<UAssetImportTask>();
		Task->AddToRoot();
		Task->bAutomated = true;
		Task->bReplaceExisting = true;
		Task->DestinationPath = ImportAssetPath;
		Task->bSave = false;
		Task->DestinationName = FGuid::NewGuid().ToString(EGuidFormats::Digits);
		Task->Options = FbxFactory->ImportUI;
		Task->Filename = AbsoluteFilePath;
		Task->Factory = FbxFactory;
		FbxFactory->SetAssetImportTask(Task);
		TArray<UAssetImportTask*> Tasks;
		Tasks.Add(Task);
		AssetToolsModule.Get().ImportAssetTasks(Tasks);

		for (FString AssetPath : Task->ImportedObjectPaths)
		{
			FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(AssetPath));
			ImportedObject = AssetData.GetAsset();
			if (ImportedObject != nullptr)
			{
				break;
			}
		}

		//Factory and task can now be garbage collected
		Task->RemoveFromRoot();
		FbxFactory->RemoveFromRoot();
		bCreateTransaction = true;
	}

	USkeletalMesh* TmpSkeletalMesh = Cast<USkeletalMesh>(ImportedObject);
	if (TmpSkeletalMesh == nullptr || TmpSkeletalMesh->GetSkeleton() == nullptr)
	{
		UE_LOGF(LogSkinWeightsUtilities, Error, "Failed to import Skin Weight Profile from provided file (%ls).", *Path);
		DeletePathAssets();
		return false;
	}

	//The LOD index of the source is always 0, 
	const int32 SrcLodIndex = 0;
	bool bResult = false;

	if (SkeletalMesh && TmpSkeletalMesh)
	{
		if (FSkeletalMeshModel* TargetModel = SkeletalMesh->GetImportedModel())
		{
			if (TargetModel->LODModels.IsValidIndex(TargetLODIndex))
			{
				//Prepare the profile data
				FSkeletalMeshLODModel& TargetLODModel = TargetModel->LODModels[TargetLODIndex];

				// Prepare the profile data
				FSkinWeightProfileInfo* Profile = SkeletalMesh->GetSkinWeightProfiles().FindByPredicate([ProfileName](FSkinWeightProfileInfo Profile) { return Profile.Name == ProfileName; });

				const bool bIsReimportLocal = Profile != nullptr;
				if (bCreateTransaction)
				{
					FText TransactionName = bIsReimportLocal ? NSLOCTEXT("UnrealEd", "UpdateAlternateSkinningWeight", "Update Alternate Skinning Weight")
						: NSLOCTEXT("UnrealEd", "ImportAlternateSkinningWeight", "Import Alternate Skinning Weight");
					FScopedTransaction ScopedTransaction(TransactionName);
					SkeletalMesh->Modify();
				}

				if (bIsReimportLocal)
				{
					// Update source file path
					FString& StoredPath = Profile->PerLODSourceFiles.FindOrAdd(TargetLODIndex);
					StoredPath = UAssetImportData::SanitizeImportFilename(AbsoluteFilePath, SkeletalMesh->GetOutermost());
					Profile->PerLODSourceFiles.KeySort([](int32 A, int32 B) { return A < B; });
				}
				
				// Clear profile data before import
				FImportedSkinWeightProfileData& ProfileData = TargetLODModel.SkinWeightProfiles.FindOrAdd(ProfileName);
				ProfileData.SkinWeights.Empty();
				ProfileData.SourceModelInfluences.Empty();

				FImportedSkinWeightProfileData PreviousProfileData = ProfileData;
				
				IMeshUtilities::MeshBuildOptions BuildOptions;
				//Use the Lod info build settings
				BuildOptions.OverlappingThresholds.ThresholdPosition = LODInfo->BuildSettings.ThresholdPosition;
				BuildOptions.OverlappingThresholds.ThresholdTangentNormal = LODInfo->BuildSettings.ThresholdTangentNormal;
				BuildOptions.OverlappingThresholds.ThresholdUV = LODInfo->BuildSettings.ThresholdUV;
				BuildOptions.OverlappingThresholds.MorphThresholdPosition = LODInfo->BuildSettings.MorphThresholdPosition;
				BuildOptions.bComputeNormals = LODInfo->BuildSettings.bRecomputeNormals;
				BuildOptions.bComputeTangents = LODInfo->BuildSettings.bRecomputeTangents;
				BuildOptions.bUseMikkTSpace = LODInfo->BuildSettings.bUseMikkTSpace;
				BuildOptions.bComputeWeightedNormals = LODInfo->BuildSettings.bComputeWeightedNormals;
				// There's currently no import option for this. We could add one if needed.
				BuildOptions.BoneInfluenceLimit = 0;

				bResult = FLODUtilities::UpdateAlternateSkinWeights(SkeletalMesh, ProfileName, TmpSkeletalMesh, TargetLODIndex, SrcLodIndex, BuildOptions);
				
				if (!bResult)
				{
					// Remove invalid profile data due to failed import
					if (!bIsReimportLocal)
					{
						TargetLODModel.SkinWeightProfiles.Remove(ProfileName);
					}
					else
					{
						// Otherwise restore previous data
						ProfileData = PreviousProfileData;
					}
				}

				// Only add if it is an initial import and it was successful 
				if (!bIsReimportLocal && bResult)
				{
					TArray<FSkinWeightProfileInfo>& SkinWeightProfileInfos = SkeletalMesh->GetSkinWeightProfiles();

					for (size_t SkinWeightProfileIndex = 0; SkinWeightProfileIndex < SkinWeightProfileInfos.Num(); SkinWeightProfileIndex++)
					{
						FSkinWeightProfileInfo& SkinWeightProfileInfo = SkinWeightProfileInfos[SkinWeightProfileInfos.Num() - 1 - SkinWeightProfileIndex];
						if (SkinWeightProfileInfo.Name == ProfileName)
						{
							SkinWeightProfileInfo.PerLODSourceFiles.Add(TargetLODIndex, UAssetImportData::SanitizeImportFilename(AbsoluteFilePath, SkeletalMesh->GetOutermost()));

							Profile = &SkinWeightProfileInfo;

							break;
						}
					}
				}
			}
		}
	}
	
	//Make sure all created objects are gone
	DeletePathAssets();

	return bResult;
}

bool FSkinWeightsUtilities::ReimportAlternateSkinWeight(USkeletalMesh* SkeletalMesh, int32 TargetLODIndex)
{
	bool bResult = false;

	const TArray<FSkinWeightProfileInfo>& SkinWeightProfiles = SkeletalMesh->GetSkinWeightProfiles();
	if (SkinWeightProfiles.Num() <= 0)
	{
		return bResult;
	}
	FScopedSuspendAlternateSkinWeightPreview ScopedSuspendAlternateSkinnWeightPreview(SkeletalMesh);
	FScopedSkeletalMeshPostEditChange ScopePostEditChange(SkeletalMesh);

	for (int32 ProfileIndex = 0; ProfileIndex < SkinWeightProfiles.Num(); ++ProfileIndex)
	{
		const FSkinWeightProfileInfo& ProfileInfo = SkinWeightProfiles[ProfileIndex];

		const FString* PathNamePtr = ProfileInfo.PerLODSourceFiles.Find(TargetLODIndex);
		//Skip profile that do not have data for TargetLODIndex
		if (!PathNamePtr)
		{
			continue;
		}

		const FString& PathName = *PathNamePtr;
		FString AbsoluteFilePath = UAssetImportData::ResolveImportFilename(PathName, SkeletalMesh->GetOutermost());
		if (FPaths::FileExists(AbsoluteFilePath))
		{
			bResult |= FSkinWeightsUtilities::ImportAlternateSkinWeight(SkeletalMesh, AbsoluteFilePath, TargetLODIndex, ProfileInfo.Name, true);
		}
		else
		{
			const FString PickedFileName = FSkinWeightsUtilities::PickSkinWeightPath(TargetLODIndex, SkeletalMesh);
			if (!PickedFileName.IsEmpty() && FPaths::FileExists(PickedFileName))
			{
				bResult |= FSkinWeightsUtilities::ImportAlternateSkinWeight(SkeletalMesh, PickedFileName, TargetLODIndex, ProfileInfo.Name, true);
			}
		}
	}

	
	if (bResult)
	{
		FLODUtilities::RegenerateDependentLODs(SkeletalMesh, TargetLODIndex, GetTargetPlatformManagerRef().GetRunningTargetPlatform());
	}
	
	return bResult;
}

bool FSkinWeightsUtilities::RemoveSkinnedWeightProfileData(USkeletalMesh* SkeletalMesh, const FName& ProfileName, int32 LODIndex)
{
	check(SkeletalMesh);
	check(SkeletalMesh->GetImportedModel());
	check(SkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(LODIndex));
	FSkeletalMeshLODModel& LODModelDest = SkeletalMesh->GetImportedModel()->LODModels[LODIndex];
	LODModelDest.SkinWeightProfiles.Remove(ProfileName);

	FSkeletalMeshImportData ImportDataDest;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SkeletalMesh->LoadLODImportedData(LODIndex, ImportDataDest);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	for (size_t AlternateInfluenceIndex = 0; AlternateInfluenceIndex < ImportDataDest.AlternateInfluenceProfileNames.Num(); AlternateInfluenceIndex++)
	{
		if (ImportDataDest.AlternateInfluenceProfileNames[AlternateInfluenceIndex] == ProfileName)
		{
			ImportDataDest.AlternateInfluenceProfileNames.RemoveAt(AlternateInfluenceIndex);
			ImportDataDest.AlternateInfluences.RemoveAt(AlternateInfluenceIndex);
			break;
		}
	}

	//If we have a LOD info we use the build settings to be sure we are rechunking the LOD with the existing options
	IMeshUtilities::MeshBuildOptions BuildOptions;
	if (FSkeletalMeshLODInfo* LODInfo = SkeletalMesh->GetLODInfo(LODIndex))
	{
		BuildOptions.OverlappingThresholds.ThresholdPosition = LODInfo->BuildSettings.ThresholdPosition;
		BuildOptions.OverlappingThresholds.ThresholdTangentNormal = LODInfo->BuildSettings.ThresholdTangentNormal;
		BuildOptions.OverlappingThresholds.ThresholdUV = LODInfo->BuildSettings.ThresholdUV;
		BuildOptions.OverlappingThresholds.MorphThresholdPosition = LODInfo->BuildSettings.MorphThresholdPosition;
		BuildOptions.bComputeNormals = LODInfo->BuildSettings.bRecomputeNormals;
		BuildOptions.bComputeTangents = LODInfo->BuildSettings.bRecomputeTangents;
		BuildOptions.bUseMikkTSpace = LODInfo->BuildSettings.bUseMikkTSpace;
		BuildOptions.bComputeWeightedNormals = LODInfo->BuildSettings.bComputeWeightedNormals;
	}

	BuildOptions.bRemoveDegenerateTriangles = false;
	BuildOptions.TargetPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
	// There's currently no import option for this. We could add one if needed.
	BuildOptions.BoneInfluenceLimit = 0;

	TArray<FVector3f> LODPointsDest;
	TArray<SkeletalMeshImportData::FMeshWedge> LODWedgesDest;
	TArray<SkeletalMeshImportData::FMeshFace> LODFacesDest;
	TArray<SkeletalMeshImportData::FVertInfluence> LODInfluencesDest;
	TArray<int32> LODPointToRawMapDest;
	ImportDataDest.CopyLODImportData(LODPointsDest, LODWedgesDest, LODFacesDest, LODInfluencesDest, LODPointToRawMapDest);

	//Build the skeletal mesh asset
	IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshUtilities>("MeshUtilities");
	TArray<FText> WarningMessages;
	TArray<FName> WarningNames;

	//BaseLOD need to make sure the source data fit with the skeletalmesh materials array before using meshutilities.BuildSkeletalMesh
	FLODUtilities::AdjustImportDataFaceMaterialIndex(SkeletalMesh->GetMaterials(), ImportDataDest.Materials, LODFacesDest, LODIndex);

	//Build the destination mesh with the Alternate influences, so the chunking is done properly.
	const bool bBuildSuccess = MeshUtilities.BuildSkeletalMesh(LODModelDest, SkeletalMesh->GetPathName(), SkeletalMesh->GetRefSkeleton(), LODInfluencesDest, LODWedgesDest, LODFacesDest, LODPointsDest, LODPointToRawMapDest, BuildOptions, &WarningMessages, &WarningNames);
	FLODUtilities::RegenerateAllImportSkinWeightProfileData(LODModelDest, BuildOptions.BoneInfluenceLimit, BuildOptions.TargetPlatform);

	//Resave the bulk data with the new or refreshed data
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SkeletalMesh->SaveLODImportedData(LODIndex, ImportDataDest);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	return bBuildSuccess;
}

FString FSkinWeightsUtilities::PickSkinWeightPath(int32 LODIndex, USkeletalMesh* SkeletalMesh)
{
	FString PickedFileName("");
	
	bool bUseInterchangeFramework = UInterchangeManager::IsInterchangeImportEnabled();
	const UInterchangeAssetImportData* SelectedInterchangeAssetImportData = Cast<UInterchangeAssetImportData>(SkeletalMesh->GetAssetImportData());

	if (bUseInterchangeFramework && SelectedInterchangeAssetImportData)
	{
		const FString& FirstSourceFile = SelectedInterchangeAssetImportData->ScriptGetFirstFilename();
		FString DefaultPath = FPaths::GetPath(FirstSourceFile);
		// Otherwise resort back to last imported directory
		if (!FPaths::DirectoryExists(DefaultPath))
		{
			DefaultPath = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_IMPORT);
		}

		//Ask the user for a file path
		const UInterchangeProjectSettings* InterchangeProjectSettings = GetDefault<UInterchangeProjectSettings>();
		UInterchangeFilePickerBase* FilePicker = nullptr;

		//In runtime we do not have any pipeline configurator
#if WITH_EDITORONLY_DATA
		TSoftClassPtr <UInterchangeFilePickerBase> FilePickerClass = InterchangeProjectSettings->FilePickerClass;
		if (FilePickerClass.IsValid())
		{
			UClass* FilePickerClassLoaded = FilePickerClass.LoadSynchronous();
			if (FilePickerClassLoaded)
			{
				FilePicker = NewObject<UInterchangeFilePickerBase>(GetTransientPackage(), FilePickerClassLoaded, NAME_None, RF_NoFlags);
			}
		}
#endif
		if (FilePicker)
		{
			FInterchangeFilePickerParameters Parameters;
			Parameters.bAllowMultipleFiles = false;
			Parameters.DefaultPath = DefaultPath;
			Parameters.Title = FText::Format(NSLOCTEXT("FSkinWeightsUtilities", "PickSkinWeightPath_Title", "Choose a file to import alternate skinning for LOD {0}"), FText::AsNumber(LODIndex));
			TArray<FString> Filenames;

			if (FilePicker->ScriptedFilePickerForTranslatorAssetType(EInterchangeTranslatorAssetType::Meshes, Parameters, Filenames))
			{
				ensure(Filenames.Num() == 1);
				PickedFileName = Filenames[0];
				FEditorDirectories::Get().SetLastDirectory(ELastDirectory::GENERIC_IMPORT, FPaths::GetPath(PickedFileName));
			}
			else
			{
				// Error
			}
		}
	}
	else
	{
		bool bOpen = false;
		TArray<FString> OpenFilenames;
		FString ExtensionStr;
		ExtensionStr += TEXT("FBX files|*.fbx|");

		// First, display the file open dialog for selecting the file.
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
		if (DesktopPlatform)
		{
			// Try and retrieve the path containing the original skeletal mesh source data, and set it as default path for the file dialog
			UFbxSkeletalMeshImportData* ImportData = SkeletalMesh ? Cast<UFbxSkeletalMeshImportData>(SkeletalMesh->GetAssetImportData()) : nullptr;
			FString DefaultPath;
			FString TempString;
			if (ImportData)
			{
				ImportData->GetImportContentFilename(DefaultPath, TempString);
				DefaultPath = FPaths::GetPath(DefaultPath);
			}
		
			// Otherwise resort back to last FBX directory
			if(!FPaths::DirectoryExists(DefaultPath))
			{
				DefaultPath = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::FBX);
			}		

			const FString DialogTitle = TEXT("Pick FBX file containing Skin Weight data for LOD ") + FString::FormatAsNumber(LODIndex);
			bOpen = DesktopPlatform->OpenFileDialog(
				FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
				DialogTitle,
				*DefaultPath,
				TEXT(""),
				*ExtensionStr,
				EFileDialogFlags::None,
				OpenFilenames
			);
			if (bOpen)
			{
				if (OpenFilenames.Num() == 1)
				{
					PickedFileName = OpenFilenames[0];
					// Set last directory path for FBX files
					FEditorDirectories::Get().SetLastDirectory(ELastDirectory::FBX, FPaths::GetPath(PickedFileName));
				}
				else
				{
					// Error
				}
			}
		}
	}

	return PickedFileName;
}