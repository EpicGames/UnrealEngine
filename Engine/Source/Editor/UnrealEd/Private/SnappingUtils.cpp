// Copyright Epic Games, Inc. All Rights Reserved.

#include "SnappingUtils.h"
#include "Model.h"
#include "Modules/ModuleManager.h"
#include "GameFramework/Actor.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Editor/GroupActor.h"
#include "GameFramework/PhysicsVolume.h"
#include "Engine/PostProcessVolume.h"
#include "GameFramework/WorldSettings.h"
#include "EngineUtils.h"
#include "LevelEditorViewport.h"
#include "Engine/Selection.h"
#include "EditorModeManager.h"
#include "ScopedTransaction.h"
#include "EdMode.h"
#include "LevelEditor.h"
#include "LevelEditorActions.h"
#include "VertexSnapping.h"
#include "ISnappingPolicy.h"
#include "ViewportSnappingModule.h"
#include "ActorGroupingUtils.h"
#include "Elements/Actor/ActorElementData.h"

namespace SnappingUtilsLocals
{
	static bool bAlwaysSnapPercentageBasedScaleToGrid = true;
	static FAutoConsoleVariableRef CVarAlwaysSnapPercentageBasedScaleToGrid(
		TEXT("Editor.AlwaysSnapPercentageBasedScale"),
		bAlwaysSnapPercentageBasedScaleToGrid,
		TEXT("When true (default), grid snapping is always used when scaling an object with Percentage based scaling, regardless of snap state."));
}

//////////////////////////////////////////////////////////////////////////
// FEditorViewportSnapping

class FEditorViewportSnapping : public ISnappingPolicy
{
public:
	// FEditorViewportSnapping interface
	virtual void SnapScale(FVector& Point, const FVector& GridBase) override;
	virtual void SnapPointToGrid(FVector& Point, const FVector& GridBase) override;
	virtual void SnapRotatorToGrid(FRotator& Rotation) override;
	virtual void ClearSnappingHelpers(bool bClearImmediately = false) override;
	virtual void DrawSnappingHelpers(const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	virtual bool IsTranslationSnappingEnabled() const override;
	virtual bool IsRotationSnappingEnabled() const override;
	virtual bool IsScaleSnappingEnabled() const override;
	// End of FEditorViewportSnapping interface

	bool IsSnappingActive() const;

	bool IsSnapToGridEnabled();
	bool IsSnapRotationEnabled();
	bool IsSnapScaleEnabled();

	/**
	 * @return true if snapping to vertices is enabled
	 */
	bool IsSnapToVertexEnabled(bool bIsPivot = false);

	/**
	 * @return true if snapping actors to other actors is enabled
	 */
	bool IsSnapToActorEnabled();

	/** Set user setting for actor snap. */
	void EnableActorSnap(bool bEnable);

	/** Access user setting for distance. Fractional 0.0->100.0 */
	float GetActorSnapDistance(bool bScalar = false);

	/** Set user setting for distance. Fractional 0.0->100.0 */
	void SetActorSnapDistance(float Distance);

	/**
	 * Attempts to snap the selected actors to the nearest other actor
	 *
	 * @param DragDelta			The current world space drag amount
	 * @param ViewportClient	The viewport client the user is dragging in
	 */
	bool SnapActorsToNearestActor( FVector& DragDelta, FLevelEditorViewportClient* ViewportClient );

	/**
     * Attempts to snap the selected actors to the nearest other actor
     *
     * @param DragDelta			The current world space drag amount
     * @param InWorld			The world the user is dragging in
     * @return the location delta to the nearest actor, if any
     */
    TOptional<FVector> GetDeltaToNearestActor( FVector& DragDelta, const UWorld* InWorld );

	/**
	 * Snaps actors to the nearest vertex on another actor
	 *
	 * @param DragDelta			The current world space drag amount that will be modified to account for snapping to a vertex
	 * @param ViewportClient	The viewport client the user is dragging in
	 * @return true if anything was snapped
	 */
	bool SnapDraggedActorsToNearestVertex( FVector& DragDelta, TNotNull<FEditorViewportClient*> ViewportClient );

	/**
	 * Snaps a delta drag movement to the nearest vertex
	 *
	 * @param BaseLocation		Location that should be snapped before any drag is applied
	 * @param DragDelta			Delta drag movement that should be snapped.  This value will be updated such that BaseLocation+DragDelta is the nearest snapped verted
	 * @param ViewportClient	The viewport client being dragged in.
	 * @return true if anything was snapped
	 */
	bool SnapDragLocationToNearestVertex( const FVector& BaseLocation, FVector& DragDelta, TNotNull<FEditorViewportClient*> ViewportClient, bool bIsPivot = false );

	/**
	 * Snaps a location to the nearest vertex
	 *
	 * @param Location			The location to snap
	 * @param MouseLocation		The current 2d mouse location.  Vertices closer to the mouse are favored
	 * @param ViewportClient	The viewport client being used
	 * @param OutVertexNormal	The normal at the closest vertex
	 * @return true if anything was snapped
	 */
	bool SnapLocationToNearestVertex( FVector& Location, const FVector2D& MouseLocation, TNotNull<FEditorViewportClient*> ViewportClient, FVector& OutVertexNormal, bool bDrawVertHelpers, const TSet<TWeakObjectPtr<AActor>>& ActorsToIgnore );

	bool SnapToBSPVertex( FVector& Location, FVector GridBase, FRotator& Rotation );

private:
	bool IsSnapRotationEnabledInternal() const;
	bool IsSnapScaleEnabledInternal() const;

	/**
	 * @return true if snapping to vertices is enabled
	 */
	bool IsSnapToVertexEnabledInternal(bool bIsPivot = false) const;

	/**
	 * @return true if snapping actors to other actors is enabled
	 */
	bool IsSnapToActorEnabledInternal() const;
	
private:

	/** Vertex snapping implementation */
	FVertexSnappingImpl VertexSnappingImpl;
};

//////////////////////////////////////////////////////////////////////////
// FEditorViewportSnapping

bool FEditorViewportSnapping::IsSnapToGridEnabled()
{
	return IsSnappingActive() && GetDefault<ULevelEditorViewportSettings>()->GridEnabled && !IsSnapToVertexEnabledInternal();
}

bool FEditorViewportSnapping::IsSnapRotationEnabled()
{	
	return IsSnapRotationEnabledInternal();
}

bool FEditorViewportSnapping::IsSnapScaleEnabled()
{
	return IsSnapScaleEnabledInternal();
}

bool FEditorViewportSnapping::IsSnapRotationEnabledInternal() const
{
	// Ask Current Editor Mode if Rotation Snap is enabled
	const ULevelEditorViewportSettings* Settings = GetDefault<ULevelEditorViewportSettings>();
	return GLevelEditorModeTools().IsSnapRotationEnabled() || (Settings->SnapToSurface.bEnabled && Settings->SnapToSurface.bSnapRotation);
}

bool FEditorViewportSnapping::IsSnapScaleEnabledInternal() const
{
	return IsSnappingActive() && GetDefault<ULevelEditorViewportSettings>()->SnapScaleEnabled;
}

bool FEditorViewportSnapping::IsSnapToVertexEnabled(bool bIsPivot)
{
	return IsSnapToVertexEnabledInternal(bIsPivot);
}

bool FEditorViewportSnapping::IsSnapToVertexEnabledInternal(bool bIsPivot) const
{
	if (!IsSnappingActive())
	{
		return false;
	}

	if( GetDefault<ULevelEditorViewportSettings>()->bSnapVertices )
	{
		return true;
	}
	else if( GCurrentLevelEditingViewportClient )
	{
		FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>( TEXT("LevelEditor") );
		TSharedPtr<FUICommandInfo> Command = bIsPivot ? LevelEditor.GetLevelEditorCommands().HoldToEnablePivotVertexSnapping : LevelEditor.GetLevelEditorCommands().HoldToEnableVertexSnapping;
		return GCurrentLevelEditingViewportClient->IsCommandChordPressed(Command);
	}
	else
	{
		return false;
	}
}

bool FEditorViewportSnapping::IsSnapToActorEnabled()
{
	return IsSnapToActorEnabledInternal();
}

bool FEditorViewportSnapping::IsSnapToActorEnabledInternal() const
{
	return IsSnappingActive() && (GetDefault<ULevelEditorViewportSettings>()->bEnableActorSnap && !IsSnapToVertexEnabledInternal());
}

void FEditorViewportSnapping::EnableActorSnap(bool bEnable)
{
	ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>();
	ViewportSettings->bEnableActorSnap = bEnable;
	ViewportSettings->PostEditChange();
}

float FEditorViewportSnapping::GetActorSnapDistance(bool bScalar)
{
	ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>();

	// If the user has purposefully exceeded the maximum scale, convert this to the range so that it can be more easily adjusted in the editor
	if (ViewportSettings->ActorSnapScale > 1.0f)
	{
		ViewportSettings->ActorSnapDistance *= ViewportSettings->ActorSnapScale;
		ViewportSettings->ActorSnapScale = 1.0f;
		ViewportSettings->PostEditChange();
	}

	if (bScalar)
	{
		// Clamp to within range (just so slider looks correct)
		return FMath::Clamp(ViewportSettings->ActorSnapScale, 0.0f, 1.0f);
	}

	// Multiply by the max distance allowed to convert to range
	return FMath::Max(0.0f, ViewportSettings->ActorSnapScale) * ViewportSettings->ActorSnapDistance;
}

void FEditorViewportSnapping::SetActorSnapDistance(float Distance)
{
	ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>();
	ViewportSettings->ActorSnapScale = Distance;
	ViewportSettings->PostEditChange();
}

bool FEditorViewportSnapping::SnapActorsToNearestActor( FVector& Drag, FLevelEditorViewportClient* ViewportClient )
{
	const TOptional<FVector> DeltaToNearestActor = GetDeltaToNearestActor(Drag, ViewportClient->GetWorld());
	const bool bWasSnapped = DeltaToNearestActor.IsSet();
	if (bWasSnapped)
	{
		ViewportClient->ApplyDeltaToActors(DeltaToNearestActor.GetValue(), FRotator::ZeroRotator, FVector::ZeroVector);
	}

	return bWasSnapped;
}

TOptional<FVector> FEditorViewportSnapping::GetDeltaToNearestActor(FVector& Drag, const UWorld* InWorld)
{
	TOptional<FVector> DeltaToNearestActor;

	FEditorModeTools& Tools = GLevelEditorModeTools();

	// Does the user have actor snapping enabled?
	bool bSnapped = false;
	if ( IsSnapToActorEnabled() )
	{		
		// Are there selected actors?
		USelection* Selection = GEditor->GetSelectedActors();
		if ( Selection->Num() > 0 )
		{
			// Nearest results
			const AActor* BestActor = NULL;
			FVector BestPoint = FVector::ZeroVector;
			double BestSqrdDist = 0.0f;

			// Find the nearest actor to the pivot point that isn't part of the selection
			const FVector PivotLocation = Tools.PivotLocation;
			for (FActorIterator It(InWorld); It; ++It)	// Actor iterator :( [Note: Also, can't use BoxOverlapMulti to as the pivot may lie outside the bounds of the actor!]
			{
				AActor* Actor = static_cast<AActor*>( *It );
				checkSlow( Actor->IsA(AActor::StaticClass()) );

				// Make sure this isn't an invalid actor type or one of the selected actors
				const FString tmp = Actor->GetActorLabel();
				if ( !Actor->IsA(AWorldSettings::StaticClass()) 
					&& !Actor->IsA(APhysicsVolume::StaticClass()) 
					&& !Actor->IsA(APostProcessVolume::StaticClass()) 
					&& !Selection->IsSelected( Actor ) )
				{
					// Group Actors don't appear in the selected actors list!
					if (UActorGroupingUtils::IsGroupingActive())
					{
						// Valid snaps: locked groups (not self or actors within locked groups), actors within unlocked groups (not the group itself), other actors
						const AGroupActor* GroupActor = Cast<AGroupActor>( Actor ); // AGroupActor::GetRootForActor( Actor );
						if ( GroupActor && ( !GroupActor->IsLocked() || GroupActor->HasSelectedActors() ) )
						{
							continue;
						}
					}

					// Is this the nearest actor to the pivot?
					const FVector Point = Actor->GetActorLocation();
					const double SqrdDist = FVector::DistSquared( PivotLocation, Point );
					if ( BestActor == nullptr || SqrdDist < BestSqrdDist )
					{
						BestActor = Actor;
						BestPoint = Point;
						BestSqrdDist = SqrdDist;
					}
				}
			}

			// Did we find an actor?
			const FString tmp = BestActor ? BestActor->GetActorLabel() : TEXT( "None" );
			if ( BestActor )
			{				
				// Are we within the threshold or exitting it?
				const float Dist = GetActorSnapDistance();
				if ( BestSqrdDist < FMath::Square( Dist )  )
				{
					bSnapped = true;

					// Are we no already snapped, or is it different to our current location
					if ( !Tools.SnappedActor || !Tools.CachedLocation.Equals( BestPoint ) )
					{
						// Calculate the delta between the snapped location and the current pivot and apply to all the selected actors
						const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "SnapActorsToActor", "Snap Actors To Actor") );
						const FVector PivotDelta = ( BestPoint - PivotLocation );
						DeltaToNearestActor = PivotDelta;
						Tools.SetPivotLocation( BestPoint, false );	// Overwrite the location for next time we check
						Drag = FVector::ZeroVector;	// Reset the drag so the pivot doesn't jump
					}
				}
				else if ( Tools.SnappedActor && !Tools.CachedLocation.Equals( PivotLocation ) )
				{
					const FVector PivotDelta = ( PivotLocation - BestPoint );
					DeltaToNearestActor = PivotDelta;
					//GUnrealEd->UpdatePivotLocationForSelection();	// Calling this ends up forcing the pivot back inside the threshold?!
					Tools.SetPivotLocation( PivotLocation, false );	// Overwrite the location for next time we check
					Drag = FVector::ZeroVector;	// Reset the drag so the pivot doesn't jump
				}
			}
		}
	}

	Tools.SnappedActor = bSnapped;
	return DeltaToNearestActor;	// Returns the location delta to the nearest actor, if snapped
}

void FEditorViewportSnapping::SnapPointToGrid(FVector& Point, const FVector& GridBase)
{
	if( IsSnapToGridEnabled() )
	{
		Point = (Point - GridBase).GridSnap( GEditor->GetGridSize() ) + GridBase;
	}
}

void FEditorViewportSnapping::SnapRotatorToGrid(FRotator& Rotation)
{
	if( IsSnapRotationEnabled() )
	{		
		if (GLevelEditorModeTools().SnapRotatorToGridOverride(Rotation))
		{
			return;
		}

		Rotation = Rotation.GridSnap( GEditor->GetRotGridSize() );		
	}
}

void FEditorViewportSnapping::SnapScale(FVector& Point, const FVector& GridBase)
{
	if( IsSnapScaleEnabled() )
	{
		if( GEditor->UsePercentageBasedScaling() )
		{
			Point = (Point - GridBase).GridSnap( GEditor->GetGridSize() ) + GridBase;
		}
		else
		{
			if (GetDefault<ULevelEditorViewportSettings>()->PreserveNonUniformScale)
			{
				// when using 'auto-precision', we take the max component & snap its scale, then proportionally scale the other components
				double MaxComponent = Point.GetAbsMax();
				if(MaxComponent == 0.0f)
				{
					MaxComponent = 1.0f;
				}
				const double SnappedMaxComponent = FMath::GridSnap(MaxComponent, GEditor->GetScaleGridSize());
				Point = Point * (SnappedMaxComponent / MaxComponent);
			}
			else
			{
				Point = Point.GridSnap( GEditor->GetScaleGridSize() );
			}
		}
	}
}


bool FEditorViewportSnapping::SnapToBSPVertex(FVector& Location, FVector GridBase, FRotator& Rotation)
{
	bool bSnapped = false;
	SnapRotatorToGrid( Rotation );
	if( IsSnapToVertexEnabled() )
	{
		FVector3f	SrcPoint = (FVector3f)Location;
		FVector3f	DestPoint;
		int32 Temp;
		if( GWorld && GWorld->GetModel() && GWorld->GetModel()->FindNearestVertex(SrcPoint, DestPoint, GetDefault<ULevelEditorViewportSettings>()->SnapDistance, Temp ) >= 0.0)
		{
			Location = (FVector)DestPoint;
			bSnapped = true;
		}
	}

	if( !bSnapped )
	{
		SnapPointToGrid( Location, GridBase );
	}

	return bSnapped;
}

void FEditorViewportSnapping::ClearSnappingHelpers( bool bClearImmediately )
{
	VertexSnappingImpl.ClearSnappingHelpers(bClearImmediately);
}

void FEditorViewportSnapping::DrawSnappingHelpers(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	VertexSnappingImpl.DrawSnappingHelpers( View, PDI );
}

bool FEditorViewportSnapping::IsSnappingActive() const
{
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
	return ViewportSettings->bSnappingActive;
}

bool FEditorViewportSnapping::IsTranslationSnappingEnabled() const
{
	const ULevelEditorViewportSettings* Settings = GetDefault<ULevelEditorViewportSettings>();
	return Settings->GridEnabled || Settings->SnapToSurface.bEnabled || IsSnapToVertexEnabledInternal() || IsSnapToActorEnabledInternal();
}

bool FEditorViewportSnapping::IsRotationSnappingEnabled() const
{
	return IsSnapRotationEnabledInternal();
}

bool FEditorViewportSnapping::IsScaleSnappingEnabled() const
{
	return IsSnapScaleEnabledInternal();
}

bool FEditorViewportSnapping::SnapLocationToNearestVertex( FVector& Location, const FVector2D& MouseLocation, TNotNull<FEditorViewportClient*> ViewportClient, FVector& OutVertexNormal, bool bDrawVertHelpers, const TSet<TWeakObjectPtr<AActor>>& ActorsToIgnore )
{
	bool bSnapped = false;
	if( IsSnapToVertexEnabled() )
	{
		bSnapped = VertexSnappingImpl.SnapLocationToNearestVertex( Location, MouseLocation, ViewportClient, OutVertexNormal, bDrawVertHelpers, ActorsToIgnore );
	}
	else
	{
		OutVertexNormal = FVector(ForceInitToZero);
	}

	return bSnapped;
}

bool FEditorViewportSnapping::SnapDraggedActorsToNearestVertex( FVector& DragDelta, TNotNull<FEditorViewportClient*> ViewportClient )
{
	bool bSnapped = false;
	if( IsSnapToVertexEnabled() && !DragDelta.IsNearlyZero() )
	{
		bSnapped = VertexSnappingImpl.SnapDraggedActorsToNearestVertex( DragDelta, ViewportClient );
	}
	return bSnapped;
}

bool FEditorViewportSnapping::SnapDragLocationToNearestVertex( const FVector& BaseLocation, FVector& DragDelta, TNotNull<FEditorViewportClient*> ViewportClient, bool bIsPivot )
{
	bool bSnapped = false;
	if( IsSnapToVertexEnabled(bIsPivot) && !DragDelta.IsNearlyZero() )
	{
		bSnapped = VertexSnappingImpl.SnapDragLocationToNearestVertex( BaseLocation, DragDelta, ViewportClient );
	}
	return bSnapped;
}

//////////////////////////////////////////////////////////////////////////
// FSnappingUtils

TSharedPtr<FEditorViewportSnapping> FSnappingUtils::EditorViewportSnapper;

bool FSnappingUtils::IsSnappingActive()
{
	return EditorViewportSnapper.IsValid() && EditorViewportSnapper->IsSnappingActive();
}

bool FSnappingUtils::IsSnapToGridEnabled()
{
	return EditorViewportSnapper.IsValid() && EditorViewportSnapper->IsSnapToGridEnabled();
}

bool FSnappingUtils::IsTranslationSnappingEnabled(const bool bInRequireSnappingActive)
{
	return EditorViewportSnapper.IsValid()
		&& (!bInRequireSnappingActive || EditorViewportSnapper->IsSnappingActive()) 
		&& EditorViewportSnapper->IsTranslationSnappingEnabled();
}

bool FSnappingUtils::IsRotationSnapEnabled()
{
	return EditorViewportSnapper.IsValid() && EditorViewportSnapper->IsSnapRotationEnabled();
}

bool FSnappingUtils::IsRotationSnappingEnabled(const bool bInRequireSnappingActive)
{
	return EditorViewportSnapper.IsValid() 
		&& (!bInRequireSnappingActive || EditorViewportSnapper->IsSnappingActive()) 
		&& EditorViewportSnapper->IsRotationSnappingEnabled();
}

bool FSnappingUtils::IsScaleSnapEnabled()
{
	return EditorViewportSnapper.IsValid() && EditorViewportSnapper->IsSnapScaleEnabled();
}

bool FSnappingUtils::IsScaleSnappingEnabled(const bool bInRequireSnappingActive)
{
	return EditorViewportSnapper.IsValid()
		&& (!bInRequireSnappingActive || EditorViewportSnapper->IsSnappingActive()) 
		&& EditorViewportSnapper->IsScaleSnappingEnabled();
}

bool FSnappingUtils::IsSnapToActorEnabled()
{
	return EditorViewportSnapper.IsValid() && EditorViewportSnapper->IsSnapToActorEnabled();
}

void FSnappingUtils::EnableActorSnap(bool bEnable)
{
	if (EditorViewportSnapper.IsValid())
	{
		EditorViewportSnapper->EnableActorSnap(bEnable);
	}
}

float FSnappingUtils::GetActorSnapDistance(bool bScalar)
{
	return EditorViewportSnapper.IsValid() ? EditorViewportSnapper->GetActorSnapDistance(bScalar) : 0.0f;
}

void FSnappingUtils::SetActorSnapDistance(float Distance)
{
	if (EditorViewportSnapper.IsValid())
	{
		EditorViewportSnapper->SetActorSnapDistance(Distance);
	}
}

bool FSnappingUtils::SnapActorsToNearestActor(FVector& DragDelta, FLevelEditorViewportClient* ViewportClient)
{
	return EditorViewportSnapper.IsValid() ? EditorViewportSnapper->SnapActorsToNearestActor(DragDelta, ViewportClient) : false;
}

TOptional<FVector> FSnappingUtils::GetDeltaToNearestActor(FVector& DragDelta, const UWorld* InWorld)
{
	return EditorViewportSnapper.IsValid() ? EditorViewportSnapper->GetDeltaToNearestActor(DragDelta, InWorld) : TOptional<FVector>();
}

bool FSnappingUtils::SnapDraggedActorsToNearestVertex(FVector& DragDelta, FEditorViewportClient* ViewportClient)
{
	return EditorViewportSnapper.IsValid() ? EditorViewportSnapper->SnapDraggedActorsToNearestVertex(DragDelta, ViewportClient) : false;
}

bool FSnappingUtils::SnapDragLocationToNearestVertex(const FVector& BaseLocation, FVector& DragDelta, FEditorViewportClient* ViewportClient, bool bIsPivot)
{
	return EditorViewportSnapper.IsValid() ? EditorViewportSnapper->SnapDragLocationToNearestVertex(BaseLocation, DragDelta, ViewportClient, bIsPivot) : false;
}

bool FSnappingUtils::SnapLocationToNearestVertex(FVector& Location, const FVector2D& MouseLocation, FEditorViewportClient* ViewportClient, FVector& OutVertexNormal, bool bDrawVertHelpers, const TSet<TWeakObjectPtr<AActor>>& ActorsToIgnore )
{
	return EditorViewportSnapper.IsValid() ? EditorViewportSnapper->SnapLocationToNearestVertex(Location, MouseLocation, ViewportClient, OutVertexNormal, bDrawVertHelpers, ActorsToIgnore) : false;
}

bool FSnappingUtils::SnapLocationToNearestVertex(FVector& Location, const FVector2D& MouseLocation, FLevelEditorViewportClient* ViewportClient, FVector& OutVertexNormal, bool bDrawVertHelpers )
{
	TSet<TWeakObjectPtr<AActor>> ActorsToIgnore;
	if (const TSharedPtr<FTypedElementList>& Elements = ViewportClient->GetDropPreviewElements())
	{
		Elements->ForEachElementHandle([&ActorsToIgnore](const FTypedElementHandle& Element)
			{
				if (AActor* Actor = ActorElementDataUtil::GetActorFromHandle(Element, true))
				{
					ActorsToIgnore.Add(Actor);
				}
				return true;
			});
	}
	return SnapLocationToNearestVertex(Location, MouseLocation, ViewportClient, OutVertexNormal, bDrawVertHelpers, MoveTemp(ActorsToIgnore));
}

void FSnappingUtils::SnapScale(FVector& Point, const FVector& GridBase)
{
	IViewportSnappingModule::GetSnapManager()->SnapScale(Point, GridBase);
}

void FSnappingUtils::AdjustPercentageBasedScale(FVector& InOutScale)
{
	// If we are scaling, we need to change the scaling factor a bit to properly align to grid
	if (!InOutScale.IsNearlyZero() && GEditor->UsePercentageBasedScaling())
	{
		// In both cases, the scale is assumed to be in the 0-100 range and is adjusted to 0-1
		if (SnappingUtilsLocals::bAlwaysSnapPercentageBasedScaleToGrid)
		{
			InOutScale *= ((GEditor->GetScaleGridSize() / 100.0f) / GEditor->GetGridSize());
		}
		else
		{
			InOutScale /= 100.0f;
		}
	}
}

void FSnappingUtils::SnapPointToGrid(FVector& Point, const FVector& GridBase)
{
	IViewportSnappingModule::GetSnapManager()->SnapPointToGrid(Point, GridBase);
}

void FSnappingUtils::SnapRotatorToGrid(FRotator& Rotation)
{
	IViewportSnappingModule::GetSnapManager()->SnapRotatorToGrid(Rotation);
}

bool FSnappingUtils::SnapToBSPVertex(FVector& Location, FVector GridBase, FRotator& Rotation)
{
	return EditorViewportSnapper.IsValid() ? EditorViewportSnapper->SnapToBSPVertex(Location, GridBase, Rotation) : false;
}

void FSnappingUtils::ClearSnappingHelpers(bool bClearImmediately)
{
	IViewportSnappingModule::GetSnapManager()->ClearSnappingHelpers(bClearImmediately);
}

void FSnappingUtils::DrawSnappingHelpers(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	IViewportSnappingModule::GetSnapManager()->DrawSnappingHelpers(View, PDI);
}

void FSnappingUtils::InitEditorSnappingTools()
{
	EditorViewportSnapper = MakeShareable(new FEditorViewportSnapping);

	IViewportSnappingModule& Module = FModuleManager::LoadModuleChecked<IViewportSnappingModule>("ViewportSnapping");
	Module.RegisterSnappingPolicy(EditorViewportSnapper);
}
