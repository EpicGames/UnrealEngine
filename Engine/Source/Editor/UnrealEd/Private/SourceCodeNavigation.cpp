// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceCodeNavigation.h"

#include "Async/Async.h"
#include "Async/AsyncWork.h"
#include "Async/TaskGraphInterfaces.h"
#include "Editor/UnrealEdEngine.h"
#include "GameFramework/Actor.h"
#include "GameFramework/Pawn.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformStackWalk.h"
#include "HttpModule.h"
#include "Interfaces/IHttpRequest.h"
#include "Interfaces/IHttpResponse.h"
#include "ISourceCodeAccessModule.h"
#include "ISourceCodeAccessor.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "Misc/EngineBuildSettings.h"
#include "Misc/FileHelper.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Modules/ModuleManager.h"
#include "Preferences/UnrealEdOptions.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Styling/AppStyle.h"
#include "TickableEditorObject.h"
#include "UnrealEdGlobals.h"
#include "UnrealEdMisc.h"
#include "UObject/Class.h"
#include "UObject/MetaData.h"
#include "UObject/Package.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
	#include <DbgHelp.h>				
	#include <TlHelp32.h>		
	#include <psapi.h>
#include "Windows/HideWindowsPlatformTypes.h"
#elif PLATFORM_MAC
#include "Apple/PreAppleSystemHeaders.h"
#include <mach-o/dyld.h>
#include <mach-o/nlist.h>
#include <mach-o/stab.h>
#include <cxxabi.h>
#include "Apple/PostAppleSystemHeaders.h"
#include "Apple/ApplePlatformSymbolication.h"
#endif
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "DesktopPlatformModule.h"

DEFINE_LOG_CATEGORY(LogSelectionDetails);

static bool GAsyncSourceCodeNavigation = true;
static FAutoConsoleVariableRef CVarAsyncSourceCodeNavigation(
	TEXT("SourceCodeNavigation.EnableAsync"),
	GAsyncSourceCodeNavigation,
	TEXT("When true, opening source files in the IDE is deferred so the editor stays responsive. Set to false to revert to synchronous (blocking) behavior."),
	ECVF_Default
);

#define LOCTEXT_NAMESPACE "SourceCodeNavigation"

#define SOURCECODENAVIGATOR_SHOW_CONSTRUCTOR_AND_DESTRUCTOR 0 // @todo editcode: Not sure if we want this by default (make user-configurable?)
#define SOURCECODENAVIGATOR_GATHER_LOW_LEVEL_CLASSES        0 // @todo editcode: Always skip these?  Make optional in UI?

namespace SourceCodeNavigationDefs
{
	FString IDEInstallerFilename("UE4_SuggestedIDEInstaller");
}

/**
 * Caches information about source symbols for fast look-up
 */
class FSourceSymbolDatabase
{

public:

	/**
	 * Attempts to locate function symbols for the specified module and class name
	 *
	 * @param	ModuleName	Module name to search
	 * @param	ClassName	Class name to search
	 * @param	OutFunctionSymbolNames	Functions that we found (if return value was true)
	 * @param	OutIsCompleteList	True if the list returned contains all known functions, or false if the list is known to be incomplete (e.g., we're still actively digesting functions and updating the list, so you should call this again to get the full list later.)
	 *
	 * @return	True if functions were found, otherwise false
	 */
	bool QueryFunctionsForClass( const FString& ModuleName, const FString& ClassName, TArray< FString >& OutFunctionSymbolNames, bool& OutIsCompleteList );


	/**
	 * Sets the function names for the specified module and class name
	 *
	 * @param	ModuleName	Module name to set functions for
	 * @param	ClassName	Class name to set functions for
	 * @param	FunctionSymbolNames	Functions to set for this module and class
	 */
	void SetFunctionsForClass( const FString& ModuleName, const FString& ClassName, const TArray< FString >& FunctionSymbolNames );


private:

	struct FClass
	{
		/** List of function symbols within the class */
		TArray< FString > FunctionSymbolNames;

		/** True if all functions have been gathered for this class */
		bool bIsCompleteList;
	};

	struct FModule
	{
		/** Maps class names to functions in that class */
		TMap< FString, FClass > Classes;
	};


	/** Maps module names to classes in that module */
	TMap< FString, FModule > Modules;
};



bool FSourceSymbolDatabase::QueryFunctionsForClass( const FString& ModuleName, const FString& ClassName, TArray< FString >& OutFunctionSymbolNames, bool& OutIsCompleteList )
{
	OutIsCompleteList = false;

	FModule* FoundModule = Modules.Find( ModuleName );
	bool bWasFound = false;
	if( FoundModule != NULL )
	{
		FClass* FoundClass = FoundModule->Classes.Find( ClassName );
		if( FoundClass != NULL )
		{
			// Copy function list into the output array
			OutFunctionSymbolNames = FoundClass->FunctionSymbolNames;
			OutIsCompleteList = FoundClass->bIsCompleteList;
			bWasFound = true;
		}
	}

	return bWasFound;
}


void FSourceSymbolDatabase::SetFunctionsForClass( const FString& ModuleName, const FString& ClassName, const TArray< FString >& FunctionSymbolNames )
{
	FModule& Module = Modules.FindOrAdd( ModuleName );
	FClass& Class = Module.Classes.FindOrAdd( ClassName );

	// Copy function list into our array
	Class.FunctionSymbolNames = FunctionSymbolNames;
	Class.bIsCompleteList = true;
}





/**
 * Async task for gathering symbols
 */
class FAsyncSymbolGatherer : public FNonAbandonableTask
{

public:

	/** Constructor */
	FAsyncSymbolGatherer( const FString& InitModuleName, const FString& InitClassName )
		: AskedToAbortCount( 0 ),
		  ModuleName( InitModuleName ),
		  ClassName( InitClassName )
	{
	}


	/** Performs work on thread */
	void DoWork();

	/** Returns true if the task should be aborted.  Called from within the task processing code itself via delegate */
	bool ShouldAbort() const
	{
		return AskedToAbortCount.GetValue() > 0;
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FAsyncSymbolGatherer, STATGROUP_ThreadPoolAsyncTasks);
	}

private:

	/** True if we've been asked to abort work in progress at the next opportunity */
	FThreadSafeCounter AskedToAbortCount;

	/** Module name we're looking for symbols in */
	FString ModuleName;

	/** Class name we're looking for symbols in */
	FString ClassName;
};



FSourceFileDatabase::FSourceFileDatabase()
	: bIsDirty(true)
{
	// Register to be notified when new .Build.cs files are added to the project
	FSourceCodeNavigation::AccessOnNewModuleAdded().AddRaw(this, &FSourceFileDatabase::OnNewModuleAdded);
}

FSourceFileDatabase::~FSourceFileDatabase()
{
	FSourceCodeNavigation::AccessOnNewModuleAdded().RemoveAll(this);
}

void FSourceFileDatabase::UpdateIfNeeded()
{
	if (!bIsDirty)
	{
		return;
	}

	bIsDirty = false;

	ModuleNames.Reset();
	DisallowedHeaderNames.Empty();

	auto AppendConvertedPaths = [](TArray<FString>& OutDirs, TArrayView<const FString> BaseDirs, FPaths::EPathConversion Method, const TCHAR* ExtraData = nullptr)
	{
		OutDirs.Reserve(OutDirs.Num() + BaseDirs.Num());
		for (const FString& BaseDir : BaseDirs)
		{
			OutDirs.Add(FPaths::ConvertPath(BaseDir, Method, ExtraData));
		}
	};

	auto AppendEnginePaths = [&AppendConvertedPaths](TArray<FString>& OutDirs, TArrayView<const FString> BaseDirs)
	{
		OutDirs.Append(BaseDirs);

		for (const FString& PlatformName : FDataDrivenPlatformInfoRegistry::GetPlatformDirectoryNames(false))
		{
			AppendConvertedPaths(OutDirs, BaseDirs, FPaths::EPathConversion::Engine_PlatformExtension, *PlatformName);
		}

		if (FEngineBuildSettings::IsInternalBuild())
		{
			AppendConvertedPaths(OutDirs, BaseDirs, FPaths::EPathConversion::Engine_NoRedist);
			AppendConvertedPaths(OutDirs, BaseDirs, FPaths::EPathConversion::Engine_NotForLicensees);
		}
	};

	auto AppendProjectPaths = [&AppendConvertedPaths](TArray<FString>& OutDirs, TArrayView<const FString> BaseDirs)
	{
		OutDirs.Append(BaseDirs);

		for (const FString& PlatformName : FDataDrivenPlatformInfoRegistry::GetPlatformDirectoryNames(false))
		{
			AppendConvertedPaths(OutDirs, BaseDirs, FPaths::EPathConversion::Project_PlatformExtension, *PlatformName);
		}

		if (FEngineBuildSettings::IsInternalBuild())
		{
			AppendConvertedPaths(OutDirs, BaseDirs, FPaths::EPathConversion::Project_NoRedist);
			AppendConvertedPaths(OutDirs, BaseDirs, FPaths::EPathConversion::Project_NotForLicensees);
		}
	};

	// Find all the non-plugin modules
	{
		TArray<FString> ModuleSearchDirs;

		// Build the list of Engine directories to search for modules
		{
			const FString EngineModuleSearchDirs[] = {
				FPaths::Combine(FPaths::EngineDir(), TEXT("Source"), TEXT("Developer")),
				FPaths::Combine(FPaths::EngineDir(), TEXT("Source"), TEXT("Editor")),
				FPaths::Combine(FPaths::EngineDir(), TEXT("Source"), TEXT("Runtime")),
			};
			AppendEnginePaths(ModuleSearchDirs, EngineModuleSearchDirs);
		}

		// Build the list of Project directories to search for modules
		if (FApp::HasProjectName())
		{
			const FString ProjectModuleSearchDirs[] = {
				FPaths::Combine(FPaths::ProjectDir(), TEXT("Source")),
			};
			AppendProjectPaths(ModuleSearchDirs, ProjectModuleSearchDirs);
		}

		// Find all the build rules within the module search directories
		for (const FString& ModuleSearchDir : ModuleSearchDirs)
		{
			FindRootFilesRecursive(ModuleNames, ModuleSearchDir, TEXT("*.Build.cs"));
		}
	}

	// Find list of disallowed header names in native (non-plugin) directories
	{
		TArray<FString> HeaderFiles;
		for (const FString& ModuleName : ModuleNames)
		{
			IFileManager::Get().FindFilesRecursive(HeaderFiles, *FPaths::Combine(FPaths::GetPath(ModuleName), TEXT("Classes")), TEXT("*.h"), true, false, false);
			IFileManager::Get().FindFilesRecursive(HeaderFiles, *FPaths::Combine(FPaths::GetPath(ModuleName), TEXT("Public")), TEXT("*.h"), true, false, false);
		}

		for (const FString& HeaderFile : HeaderFiles)
		{
			DisallowedHeaderNames.Add(FPaths::GetBaseFilename(HeaderFile));
		}

		for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
		{
			DisallowedHeaderNames.Remove(ClassIt->GetName());
		}
	}

	// Find all the plugin modules
	{
		TArray<FString> PluginSearchDirs;

		// Build the list of Engine directories to search for plugins
		{
			const FString EnginePluginSearchDirs[] = {
				FPaths::Combine(FPaths::EngineDir(), TEXT("Plugins")),
			};
			AppendEnginePaths(PluginSearchDirs, EnginePluginSearchDirs);
		}

		// Build the list of Project directories to search for plugins
		if (FApp::HasProjectName())
		{
			const FString ProjectPluginSearchDirs[] = {
				FPaths::Combine(FPaths::ProjectDir(), TEXT("Plugins")),
			};
			AppendProjectPaths(PluginSearchDirs, ProjectPluginSearchDirs);
		}

		// Find all the plugin directories
		TArray<FString> PluginNames;
		for (const FString& PluginSearchDir : PluginSearchDirs)
		{
			FindRootFilesRecursive(PluginNames, PluginSearchDir, TEXT("*uplugin"));
		}

		// Add all the files within plugin directories
		for (const FString& PluginName : PluginNames)
		{
			FindRootFilesRecursive(ModuleNames, FPaths::Combine(FPaths::GetPath(PluginName), TEXT("Source")), TEXT("*.Build.cs"));
		}
	}
}

void FSourceFileDatabase::FindRootFilesRecursive(TArray<FString> &FileNames, const FString &BaseDirectory, const FString &Wildcard)
{
	// Find all the files within this directory
	TArray<FString> BasedFileNames;
	IFileManager::Get().FindFiles(BasedFileNames, *(BaseDirectory / Wildcard), true, false);

	// Append to the result if we have any, otherwise recurse deeper
	if (BasedFileNames.Num() == 0)
	{
		TArray<FString> DirectoryNames;
		IFileManager::Get().FindFiles(DirectoryNames, *(BaseDirectory / TEXT("*")), false, true);

		for (int32 Idx = 0; Idx < DirectoryNames.Num(); Idx++)
		{
			FindRootFilesRecursive(FileNames, BaseDirectory / DirectoryNames[Idx], Wildcard);
		}
	}
	else
	{
		for (int32 Idx = 0; Idx < BasedFileNames.Num(); Idx++)
		{
			FileNames.Add(BaseDirectory / BasedFileNames[Idx]);
		}
	}
}

void FSourceFileDatabase::OnNewModuleAdded(FName InModuleName)
{
	bIsDirty = true;
}


DECLARE_DELEGATE_RetVal( bool, FShouldAbortDelegate );


class FSourceCodeNavigationImpl
	: public FTickableEditorObject
{

public:

	/**
	 * Static: Queries singleton instance
	 *
	 * @return	Singleton instance of FSourceCodeNavigationImpl
	 */
	static FSourceCodeNavigationImpl& Get()
	{
		static FSourceCodeNavigationImpl SingletonInstance;
		return SingletonInstance;
	}

	FSourceCodeNavigationImpl()
		: bAsyncWorkIsInProgress(false)
	{
	}

	/** Destructor */
	~FSourceCodeNavigationImpl();

	/**
	 * Locates the source file and line for a specific function in a specific module and navigates an external editing to that source line
	 *
	 * @param	FunctionSymbolName	The function to navigate tool (e.g. "MyClass::MyFunction")
	 * @param	FunctionModuleName	The module to search for this function's symbols (e.g. "GameName-Win64-Debug")
	 * @param	bIgnoreLineNumber	True if we should just go to the source file and not a specific line within the file
	 */
	void NavigateToFunctionSource( const FString& FunctionSymbolName, const FString& FunctionModuleName, const bool bIgnoreLineNumber );

	/**
	 * Gathers all functions within a C++ class using debug symbols
	 *
	 * @param	ModuleName	The module name to search for symbols in
	 * @param	ClassName	The name of the C++ class to find functions in
	 * @param	ShouldAbortDelegate		Called frequently to check to see if the task should be aborted
	 */
	void GatherFunctions( const FString& ModuleName, const FString& ClassName, const FShouldAbortDelegate& ShouldAbortDelegate );

	/** Makes sure that debug symbols are loaded */
	void SetupModuleSymbols();

	/**
	 * Returns any function symbols that we've cached that match the request, and if possible, queues asynchronous task to gather symbols that are not yet cached.
	 *
	 * @param	ModuleName	The module name to search for symbols in
	 * @param	ClassName	The name of the C++ class to find functions in
	 * @param	OutFunctionSymbolNames	List of function symbols found for this module and class combination.  May not be a complete list.
	 * @param	OutIsCompleteList	True if the returned list of functions is complete, or false if we're still processing data.
	 */
	void TryToGatherFunctions( const FString& ModuleName, const FString& ClassName, TArray< FString >& OutFunctionSymbolNames, bool& OutIsCompleteList );

	/** A batch of symbol queries have started */
	void SymbolQueryStarted();

	/** The final symbol query in a batch completed */
	void SymbolQueryFinished();

	/** Handler called when the installer for the suggested IDE has finished downloading */
	void OnSuggestedIDEInstallerDownloadComplete(FHttpRequestPtr Request, FHttpResponsePtr Response, bool bWasSuccessful, FOnIDEInstallerDownloadComplete OnDownloadComplete);

	/** Launches the IDE installer process */
	void LaunchIDEInstaller(const FString& Filepath);

	/** @return The name of the IDE installer file for the platform */
	FString GetSuggestedIDEInstallerFileName();

protected:
	/** FTickableEditorObject interface */
	virtual void Tick( float DeltaTime ) override;
	virtual ETickableTickType GetTickableTickType() const override
	{
		return ETickableTickType::Always;
	}
	virtual TStatId GetStatId() const override;

private:

	/** Source symbol database.  WARNING: This is accessed by multiple threads and requires a mutex to read/write! */
	FSourceSymbolDatabase SourceSymbolDatabase;

	/** Async task that gathers symbols */
	TSharedPtr< FAsyncTask< FAsyncSymbolGatherer > > AsyncSymbolGatherer;

	/** Object used for synchronization via a scoped lock **/
	FCriticalSection SynchronizationObject;


	/** Describes a list element for a pending symbol gather request */
	struct FSymbolGatherRequest
	{
		/** Name of module */
		FString ModuleName;

		/** Name of the C++ class */
		FString ClassName;

		
		/** Equality operator (case sensitive!) */
		inline bool operator==( const FSymbolGatherRequest& RHS ) const
		{
			return FCString::Strcmp( *ModuleName, *RHS.ModuleName ) == 0 &&
				   FCString::Strcmp( *ClassName, *RHS.ClassName ) == 0;
		}
	};

	/** List of classes that are enqueued for symbol harvesting, as soon as the current gather finishes */
	TArray< FSymbolGatherRequest > ClassesToGatherSymbolsFor;

	/** The AsyncSymbolGatherer is working */
	bool bAsyncWorkIsInProgress;

	/** The source code symbol query in progress message */
	TWeakPtr<SNotificationItem> SymbolQueryNotificationPtr;

	/** Multi-cast delegate that fires after any symbols have finished digesting */
	FSourceCodeNavigation::FOnSymbolQueryFinished OnSymbolQueryFinished;

	/** Multi-cast delegate that fires after a compiler is not found. */
	FSourceCodeNavigation::FOnCompilerNotFound OnCompilerNotFound;

	/** Multi-cast delegate that fires after a new module (.Build.cs file) has been added */
	FSourceCodeNavigation::FOnNewModuleAdded OnNewModuleAdded;

	friend class FSourceCodeNavigation;
};



/** Performs work on thread */
void FAsyncSymbolGatherer::DoWork()
{
	FSourceCodeNavigationImpl::Get().GatherFunctions(
		ModuleName, ClassName,
		FShouldAbortDelegate::CreateRaw( this, &FAsyncSymbolGatherer::ShouldAbort ) );
}


FSourceCodeNavigationImpl::~FSourceCodeNavigationImpl()
{
	// Make sure async tasks are completed before we exit
	// @todo editcode: These could take awhile to finish.  Can we kill them in progress?
	if( AsyncSymbolGatherer.IsValid() )
	{
		AsyncSymbolGatherer->EnsureCompletion();
		AsyncSymbolGatherer.Reset();
	}
}



void FSourceCodeNavigationImpl::SetupModuleSymbols()
{
	// Initialize stack walking as it loads up symbol information which we require.
	// @todo editcode: For modules that were loaded after the first time appInitStackWalking is called, we may need to load those modules into DbgHelp (see LoadProcessModules)
	FPlatformStackWalk::InitStackWalking();
}


void FSourceCodeNavigationImpl::NavigateToFunctionSource( const FString& FunctionSymbolName, const FString& FunctionModuleName, const bool bIgnoreLineNumber )
{
	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
	ISourceCodeAccessor& SourceCodeAccessor = SourceCodeAccessModule.GetAccessor();

#if PLATFORM_WINDOWS
	FString SourceFileName;
	uint32 SourceLineNumber = 1;
	uint32 SourceColumnNumber = 0;
	if (FPlatformStackWalk::GetFunctionDefinitionLocation(FunctionSymbolName, FunctionModuleName, SourceFileName, SourceLineNumber, SourceColumnNumber))
	{
		// If the file cannot be found, we are likely in a case of an Installed Build or a Project that was relocated post compilation. Try to rebuild the path from known roots.
		if (FPaths::FileExists(SourceFileName) == false)
		{
			bool bFoundSource = false;
			TArray<FString> Tokens;

			//Split path from the PDB on backslashes 
			SourceFileName.ParseIntoArray(Tokens, TEXT("\\"));

			auto PathSearch = [&Tokens, &SourceFileName](const FString& BasePath) {
				FString PathTail;
				PathTail.Reserve(SourceFileName.Len());
				FString TempPath;
				TempPath.Reserve(SourceFileName.Len());

				int32 index = Tokens.Num() - 1;

				//Add the file name
				PathTail = Tokens[index--];

				//Successively prepend the folders until the file is found on disk or the full path is consumed.
				do
				{
					TempPath = BasePath + PathTail;
						
					if (FPaths::FileExists(TempPath) == true)
					{
						//Resolve the path to an absolute path.
						SourceFileName = IFileManager::Get().GetFilenameOnDisk(*TempPath);
						return true;
					}

					//Prepend another folder to the current tail
					PathTail = Tokens[index] + TEXT("\\") + PathTail;
				} while (--index > 0);

				return false;
			};

			const FString EnginePath = FPaths::EngineDir();
			const FString ProjectPath = FPaths::ProjectDir();
				
			if((PathSearch(EnginePath) == false) && (PathSearch(ProjectPath) == false))
			{
				UE_LOGF(LogSelectionDetails, Warning, "NavigateToFunctionSource:  Didn't find source file for [%ls] - File [%ls], Line [%i], Column [%i]",
					*FunctionSymbolName,
					*SourceFileName,
					SourceLineNumber,
					SourceColumnNumber);
			}
			else
			{
				UE_LOGF(LogSelectionDetails, Verbose, "NavigateToFunctionSource:  Found symbol file in renamed or moved code base.");
			}
		}

		UE_LOGF(LogSelectionDetails, Verbose, "NavigateToFunctionSource:  Found symbols for [%ls] - File [%ls], Line [%i], Column [%i]",
			*FunctionSymbolName,
			*SourceFileName,
			SourceLineNumber,
			SourceColumnNumber );

		if (bIgnoreLineNumber)
		{
			SourceLineNumber = 1;
			SourceColumnNumber = 1;
		}

		// Open this source file in our IDE and take the user right to the line number
		SourceCodeAccessor.OpenFileAtLine( SourceFileName, SourceLineNumber, SourceColumnNumber );
	}
#if !NO_LOGGING
	else
	{
		TCHAR ErrorBuffer[ MAX_SPRINTF ];
		UE_LOGF(LogSelectionDetails, Warning, "NavigateToFunctionSource:  Unable to find source file and line number for '%ls' [%ls]",
			*FunctionSymbolName,
			FPlatformMisc::GetSystemErrorMessage( ErrorBuffer, MAX_SPRINTF, 0 ) );
	}
#endif // !NO_LOGGING

#elif PLATFORM_MAC
    
    int32 ReturnCode = 0;
    FString Results;
    FString Errors;
	for(uint32 Index = 0; Index < _dyld_image_count(); Index++)
	{
		char const* IndexName = _dyld_get_image_name(Index);
		FString FullModulePath(IndexName);
		FString Name = FPaths::GetBaseFilename(FullModulePath);
		if(Name == FunctionModuleName)
		{
            FString FullDsymPath = FPaths::ChangeExtension(FullModulePath, TEXT("dsym"));
            const FString SourceCodeLookupCommand = FString::Printf(TEXT("%sBuild/BatchFiles/Mac/SourceCodeLookup.sh \"%s\" \"%s\" \"%s\""), *FPaths::EngineDir(), *FunctionSymbolName, *FullModulePath, *FullDsymPath);
            FPlatformProcess::ExecProcess( TEXT("/bin/sh"), *SourceCodeLookupCommand, &ReturnCode, &Results, &Errors );
            if(ReturnCode == 0 && !Results.IsEmpty())
            {
                // find the last occurance of (filepath:linenumber)
                int32 OpenIndex = -1;
                int32 ColonIndex = -1;
                int32 CloseIndex = -1;
                if(Results.FindLastChar(TCHAR('('), OpenIndex) && Results.FindLastChar(TCHAR(':'), ColonIndex) && Results.FindLastChar(TCHAR(')'), CloseIndex)
                   && CloseIndex > ColonIndex && OpenIndex < ColonIndex)
                {
                    int32 FileNamePos = OpenIndex + 1;
                    int32 FileNameLen = ColonIndex - FileNamePos;
                    FString FileName = Results.Mid(FileNamePos, FileNameLen);
                    int32 LineNumberPos = ColonIndex + 1;
                    int32 LineNumberLen = CloseIndex - LineNumberPos;
                    FString LineNumber = Results.Mid(LineNumberPos, LineNumberLen);
                    
                    if (!FPaths::FileExists(FileName))
                    {
                        // could be using dsym which has fullpath of source file on Horde build machine
                        // fix it from /Users/build/Build/++UE5/Sync/Engine/Source/** to FPaths::EngineDir()/Source/**
                        int pos = FileName.Find(TEXT("/Engine/Source/"));
                        if (pos != INDEX_NONE)
                        {
                            FileName.RemoveAt(0, pos + 8 /* length of "/Engine/" */ );
                            FileName.InsertAt(0, FPaths::EngineDir());
                        }
                    }
                    
                    if (SourceCodeAccessor.OpenFileAtLine( FileName, FCString::Atoi(*LineNumber), 0 ))
                    {
                        return;
                    }
                    else
                    {
                        UE_LOGF(LogSelectionDetails, Warning, "NavigateToFunctionSource: Unable to open file %ls and line number %ls", *FileName, *LineNumber);
                    }
                }
                else
                {
                    if (!FPaths::FileExists(FullDsymPath))
                    {
                        UE_LOGF(LogSelectionDetails, Warning, "NavigateToFunctionSource: Missing dSYM files, please install Editor symbols for debugging");
                    }
                    else
                    {
                        UE_LOGF(LogSelectionDetails, Warning, "NavigateToFunctionSource: Unexpected SourceCodeLookup.sh output: %ls", *Results);
                    }
                }
            }
            else if (ReturnCode != 0 || !Errors.IsEmpty())
            {
                UE_LOGF(LogSelectionDetails, Warning, "NavigateToFunctionSource: SourceCodeLookup.sh failed with code: %d\n%ls", ReturnCode, *Errors);
            }
		}
	}
    
    UE_LOGF(LogSelectionDetails, Warning, "NavigateToFunctionSource: Unable to look up symbol: %ls in module:%ls", *FunctionSymbolName, *FunctionModuleName);
    
#endif	// PLATFORM_WINDOWS
}


FCriticalSection FSourceCodeNavigation::CriticalSection;
FSourceFileDatabase FSourceCodeNavigation::Instance;
bool FSourceCodeNavigation::bCachedIsCompilerAvailable = false;

void FSourceCodeNavigation::Initialize()
{
	class FAsyncInitializeSourceFileDatabase : public FNonAbandonableTask
	{
	public:
		/** Performs work on thread */
		void DoWork()
		{
			FSourceCodeNavigation::GetSourceFileDatabase();
		}

		/** Returns true if the task should be aborted.  Called from within the task processing code itself via delegate */
		bool ShouldAbort() const
		{
			return false;
		}

		FORCEINLINE TStatId GetStatId() const
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(FAsyncInitializeSourceFileDatabase, STATGROUP_ThreadPoolAsyncTasks);
		}
	};

	RefreshCompilerAvailability();

	// Initialize SourceFileDatabase instance asynchronously
	(new FAutoDeleteAsyncTask<FAsyncInitializeSourceFileDatabase>)->StartBackgroundTask();
}


const FSourceFileDatabase& FSourceCodeNavigation::GetSourceFileDatabase()
{
	// Lock so that nothing may proceed while the AsyncTask is constructing the FSourceFileDatabase for the first time
	FScopeLock Lock(&CriticalSection);
	Instance.UpdateIfNeeded();

	return Instance;
}


void FSourceCodeNavigation::NavigateToFunctionSourceAsync( const FString& FunctionSymbolName, const FString& FunctionModuleName, const bool bIgnoreLineNumber )
{
	if ( !IsCompilerAvailable() )
	{
		// Let others know that we've failed to open a source file.
		AccessOnCompilerNotFound().Broadcast();
		return;
	}

	// @todo editcode: This will potentially take a long time to execute.  We need a way to tell the async task
	//				   system that this may block internally and it should always have it's own thread.  Also
	//				   we may need a way to kill these tasks on shutdown, or when an action is cancelled/overridden
	//				   by the user interactively	
	struct FNavigateFunctionParams
	{
		FString FunctionSymbolName;
		FString FunctionModuleName;
		bool bIgnoreLineNumber;
	};

	TSharedRef< FNavigateFunctionParams > NavigateFunctionParams( new FNavigateFunctionParams() );
	NavigateFunctionParams->FunctionSymbolName = FunctionSymbolName;
	NavigateFunctionParams->FunctionModuleName = FunctionModuleName;
	NavigateFunctionParams->bIgnoreLineNumber = bIgnoreLineNumber;

	struct FLocal
	{
		/** Wrapper functions to asynchronously look-up symbols and navigate to source code.  We do this
	        asynchronously because symbol look-up can often take some time */

		static void PreloadSymbolsTaskWrapper( ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent )
		{
			// Make sure debug symbols are loaded and ready
			FSourceCodeNavigationImpl::Get().SetupModuleSymbols();
		}

		static void NavigateToFunctionSourceTaskWrapper( ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent, TSharedRef< FNavigateFunctionParams > Params, TSharedPtr< SNotificationItem > CompileNotificationPtr )
		{
			// Call the navigate function!
			FSourceCodeNavigationImpl::Get().NavigateToFunctionSource( Params->FunctionSymbolName, Params->FunctionModuleName, Params->bIgnoreLineNumber );

			// clear the notification
			CompileNotificationPtr->SetCompletionState( SNotificationItem::CS_Success );
			CompileNotificationPtr->ExpireAndFadeout();
		}
	};

	FNotificationInfo Info( LOCTEXT("ReadingSymbols", "Reading C++ Symbols") );
	Info.Image = FAppStyle::GetBrush(TEXT("LevelEditor.RecompileGameCode"));
	Info.ExpireDuration = 2.0f;
	Info.bFireAndForget = false;

	TSharedPtr< SNotificationItem > CompileNotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
	if (CompileNotificationPtr.IsValid())
	{
		CompileNotificationPtr->SetCompletionState(SNotificationItem::CS_Pending);
	}

	// Kick off asynchronous task to load symbols
	DECLARE_CYCLE_STAT(TEXT("FDelegateGraphTask.EditorSourceCodeNavigation"),
		STAT_FDelegateGraphTask_EditorSourceCodeNavigation,
		STATGROUP_TaskGraphTasks);

	FGraphEventRef PreloadSymbolsAsyncResult(
		FDelegateGraphTask::CreateAndDispatchWhenReady(FDelegateGraphTask::FDelegate::CreateStatic(&FLocal::PreloadSymbolsTaskWrapper), GET_STATID(STAT_FDelegateGraphTask_EditorSourceCodeNavigation)));

	// add a dependent task to run on the main thread when symbols are loaded
	FGraphEventRef UnusedAsyncResult(
		FDelegateGraphTask::CreateAndDispatchWhenReady(
			FDelegateGraphTask::FDelegate::CreateStatic(
				&FLocal::NavigateToFunctionSourceTaskWrapper, NavigateFunctionParams, CompileNotificationPtr), GET_STATID(STAT_FDelegateGraphTask_EditorSourceCodeNavigation),
				PreloadSymbolsAsyncResult, ENamedThreads::GameThread, ENamedThreads::GameThread
			)
		);
}



void FSourceCodeNavigationImpl::GatherFunctions( const FString& ModuleName, const FString& ClassName, const FShouldAbortDelegate& ShouldAbortDelegate )
{
	TArray< FString > FunctionSymbolNames;


#if PLATFORM_WINDOWS

	// Initialize stack walking as it loads up symbol information which we require.
	SetupModuleSymbols();

	struct FCallbackUserData
	{
		FCallbackUserData( TArray< FString >& InitFunctionSymbolNames, const FShouldAbortDelegate& InitShouldAbortDelegate )
			: FunctionSymbolNames( InitFunctionSymbolNames ),
			  ShouldAbortDelegate( InitShouldAbortDelegate )
		{
		}

		TArray< FString >& FunctionSymbolNames;
		const FShouldAbortDelegate& ShouldAbortDelegate;
	};


	struct Local
	{
		static BOOL CALLBACK EnumSymbolsCallback( PSYMBOL_INFO pSymInfo, ULONG SymbolSize, PVOID UserContext )
		{
			FCallbackUserData& CallbackUserData = *static_cast< FCallbackUserData* >( UserContext );

			ANSICHAR SymbolBuffer[ MAX_SYM_NAME ];
			FCStringAnsi::Strncpy( SymbolBuffer, pSymInfo->Name, pSymInfo->NameLen + 1 );
			
			FString FunctionSymbolName( SymbolBuffer );
			
			// Strip off the class name if we have one
			FString FoundClassName;
			FString FunctionName = FunctionSymbolName;
			const int32 ClassDelimeterPos = FunctionSymbolName.Find( TEXT( "::" ) );
			if( ClassDelimeterPos != INDEX_NONE )
			{
				FoundClassName = FunctionSymbolName.Mid( 0, ClassDelimeterPos );
				FunctionName = FunctionSymbolName.Mid( ClassDelimeterPos + 2 );
			}

			// Filter out symbols that aren't pretty to look at
			bool bPassedFilter = true;
			{
				// Filter compiler-generated functions
				if( FunctionName.StartsWith( TEXT( "`" ) ) )
				{
					// e.g.
					// `scalar deleting destructor'
					// `vector deleting destructor'
					// `vftable'
					bPassedFilter = false;
				}

				// Filter out operators
				else if( FunctionName.StartsWith( TEXT( "operator " ) ) )
				{
					// e.g.
					// operator new
					bPassedFilter = false;
				}

				// Filter out member functions of inner class/struct types
				else if( FunctionName.Contains( TEXT( "::" ) ) )
				{
					// e.g.
					// FStateEvent::FStateEvent (UObject)
					bPassedFilter = false;
				}

#if !SOURCECODENAVIGATOR_SHOW_CONSTRUCTOR_AND_DESTRUCTOR
				// Filter class constructor
				else if( FunctionName == FoundClassName )
				{
					// <class>
					bPassedFilter = false;
				}

				// Filter class destructor
				else if( FunctionName.StartsWith( TEXT( "~" ) ) )
				{
					// ~<class>
					bPassedFilter = false;
				}
#endif

				// Filter various macro-generated Unreal methods and static member functions
				else if( FunctionName == TEXT( "Default" ) ||
						 FunctionName == TEXT( "GetPrivateStaticClass" ) ||
						 FunctionName == TEXT( "StaticClass" ) ||
						 FunctionName.StartsWith( TEXT( "StaticRegisterNatives" ), ESearchCase::CaseSensitive ) ||
						 FunctionName.StartsWith( TEXT( "exec" ), ESearchCase::CaseSensitive ) ||
						 FunctionName.StartsWith( TEXT( "event" ), ESearchCase::CaseSensitive ) )
				{
					bPassedFilter = false;
				}

			}


			if( bPassedFilter )
			{
				// Don't add duplicates (overloads, filter mangling, various other reasons for this.)
				// @todo editcode: O(N) look-up here, could slow down symbol gathering!
				if( !CallbackUserData.FunctionSymbolNames.Contains( FunctionSymbolName ) )
				{
					// Add it to the list
					new( CallbackUserData.FunctionSymbolNames ) FString( FunctionSymbolName );
				}
			}

			bool bShouldAbort = false;
			if( CallbackUserData.ShouldAbortDelegate.IsBound() )
			{
				bShouldAbort = CallbackUserData.ShouldAbortDelegate.Execute();
			}

			// Return true to continue searching, otherwise false
			return !bShouldAbort;
		}
	};


	// Build a search string that finds any method with the specified class, in any loaded module
	check( !ClassName.IsEmpty() && !ModuleName.IsEmpty() );
	const FString SearchMask( FString::Printf( TEXT( "%s!%s::*" ), *ModuleName, *ClassName ) );

	FCallbackUserData CallbackUserData( FunctionSymbolNames, ShouldAbortDelegate );

	// We'll need the current process handle in order to call into DbgHelp.  This must be the same
	// process handle that was passed to SymInitialize() earlier.
	const HANDLE ProcessHandle = ::GetCurrentProcess();

	// NOTE: This function sometimes takes a VERY long time to complete (multiple seconds!)
	const bool bSuccessful = !!SymEnumSymbols(
		ProcessHandle,					// Process handle
		0,								// DLL base address (or zero, to search multiple modules)
		TCHAR_TO_ANSI( *SearchMask ),	// Search mask string (see docs)
		&Local::EnumSymbolsCallback,	// Callback function
		&CallbackUserData );			// User data pointer for callback

	if( bSuccessful )
	{
		FScopeLock ScopeLock( &SynchronizationObject );

		// Update our symbol cache
		SourceSymbolDatabase.SetFunctionsForClass( ModuleName, ClassName, FunctionSymbolNames );
	}
#if !NO_LOGGING
	else
	{
		TCHAR ErrorBuffer[ MAX_SPRINTF ];
		UE_LOGF(LogSelectionDetails, Warning, "GatherFunctions:  Unable to enumerate symbols for module '%ls', search mask '%ls' [%ls]",
			*ModuleName,
			*SearchMask,
			FPlatformMisc::GetSystemErrorMessage( ErrorBuffer, MAX_SPRINTF, 0 ) );
	}
#endif
#elif PLATFORM_MAC
	// Build a search string that finds any method with the specified class, in any loaded module
	check( !ClassName.IsEmpty() && !ModuleName.IsEmpty() );
	
	for(uint32 Index = 0; Index < _dyld_image_count(); Index++)
	{
		char const* IndexName = _dyld_get_image_name(Index);
		FString FullModulePath(IndexName);
		FString Name = FPaths::GetBaseFilename(FullModulePath);
		if(Name == ModuleName)
		{
			bool bSucceeded = true;
			struct mach_header_64 const* IndexModule64 = NULL;
			struct load_command const* LoadCommands = NULL;
			
			struct mach_header const* IndexModule32 = _dyld_get_image_header(Index);
			check(IndexModule32->magic == MH_MAGIC_64);
			
			IndexModule64 = (struct mach_header_64 const*)IndexModule32;
			LoadCommands = (struct load_command const*)(IndexModule64 + 1);
			struct load_command const* Command = LoadCommands;
			struct symtab_command const* SymbolTable = NULL;
			struct dysymtab_command const* DsymTable = NULL;
			for(uint32 CommandIndex = 0; CommandIndex < IndexModule32->ncmds; CommandIndex++)
			{
				if (Command && Command->cmd == LC_SYMTAB)
				{
					SymbolTable = (struct symtab_command const*)Command;
				}
				else if(Command && Command->cmd == LC_DYSYMTAB)
				{
					DsymTable = (struct dysymtab_command const*)Command;
				}
				Command = (struct load_command const*)(((char const*)Command) + Command->cmdsize);
			}
			
			check(SymbolTable && DsymTable);
			
			IPlatformFile& PlatformFile = IPlatformFile::GetPlatformPhysical();
			IFileHandle* File = PlatformFile.OpenRead(*FullModulePath);
			if(File)
			{
				struct nlist_64* SymbolEntries = new struct nlist_64[SymbolTable->nsyms];
				check(SymbolEntries);
				char* StringTable = new char[SymbolTable->strsize];
				check(StringTable);
				
				bool FileOK = File->Seek(SymbolTable->symoff+(DsymTable->iextdefsym*sizeof(struct nlist_64)));
				FileOK &= File->Read((uint8*)SymbolEntries, DsymTable->nextdefsym*sizeof(struct nlist_64));
				
				FileOK &= File->Seek(SymbolTable->stroff);
				FileOK &= File->Read((uint8*)StringTable, SymbolTable->strsize);
				
				delete File;
				
				bSucceeded = FileOK;
				
				for(uint32 SymbolIndex = 0; FileOK && SymbolIndex < DsymTable->nextdefsym; SymbolIndex++)
				{
					struct nlist_64 const& SymbolEntry = SymbolEntries[SymbolIndex];
					// All the entries in the mach-o external table are functions.
					// The local table contains the minimal debug stabs used by dsymutil to create the DWARF dsym.
					if(SymbolEntry.n_un.n_strx)
					{
						if (SymbolEntry.n_value)
						{
							char const* MangledSymbolName = (StringTable+SymbolEntry.n_un.n_strx);
							if(FString(MangledSymbolName).Contains(ClassName))
							{
								// Remove leading '_'
								MangledSymbolName += 1;
								
								int32 Status = 0;
								char* DemangledName = abi::__cxa_demangle(MangledSymbolName, NULL, 0, &Status);
								
								FString FunctionSymbolName;
								if (DemangledName)
								{
									// C++ function
									FunctionSymbolName = DemangledName;
									free(DemangledName);
									
									// This contains return & arguments, it would seem that the DbgHelp API doesn't.
									// So we shall strip them.
									int32 ArgumentIndex = -1;
									if(FunctionSymbolName.FindLastChar(TCHAR('('), ArgumentIndex))
									{
										FunctionSymbolName.LeftInline(ArgumentIndex, EAllowShrinking::No);
										int32 TemplateNesting = 0;
										
										int32 Pos = FunctionSymbolName.Len();
										// Cast operators are special & include spaces, whereas normal functions don't.
										int32 OperatorIndex = FunctionSymbolName.Find("operator");
										if(OperatorIndex >= 0)
										{
											// Trim from before the 'operator'
											Pos = OperatorIndex;
										}
										
										for(; Pos > 0; --Pos)
										{
											TCHAR Character = FunctionSymbolName[Pos - 1];
											if(Character == TCHAR(' ') && TemplateNesting == 0)
											{
												FunctionSymbolName.MidInline(Pos, MAX_int32, EAllowShrinking::No);
												break;
											}
											else if(Character == TCHAR('>'))
											{
												TemplateNesting++;
											}
											else if(Character == TCHAR('<'))
											{
												TemplateNesting--;
											}
										}
									}
								}
								else
								{
									// C function
									FunctionSymbolName = MangledSymbolName;
								}
								
								// Strip off the class name if we have one
								FString FunctionClassName;
								FString FunctionName = FunctionSymbolName;
								const int32 ClassDelimeterPos = FunctionSymbolName.Find( TEXT( "::" ) );
								if( ClassDelimeterPos != INDEX_NONE )
								{
									FunctionClassName = FunctionSymbolName.Mid( 0, ClassDelimeterPos );
									FunctionName = FunctionSymbolName.Mid( ClassDelimeterPos + 2 );
								}
								
								// Filter out symbols that aren't pretty to look at
								bool bPassedFilter = true;
								{
									// @todo editcode: Not sure if we want this by default (make user-configurable?)
									const bool bShowConstructorAndDestructor = false;
									
									if (ClassName != FunctionClassName)
									{
										bPassedFilter = false;
									}
									
									// Filter compiler-generated functions
									if( FunctionName.StartsWith( TEXT( "`" ) ) )
									{
										// e.g.
										// `scalar deleting destructor'
										// `vector deleting destructor'
										// `vftable'
										bPassedFilter = false;
									}
									
									else if( FunctionName.StartsWith( TEXT( "vtable for" ) ) || FunctionName.StartsWith( TEXT( "scalar deleting" ) ) || FunctionName.StartsWith( TEXT( "vector deleting" ) ) )
									{
										// e.g.
										// `scalar deleting destructor'
										// `vector deleting destructor'
										// `vftable'
										bPassedFilter = false;
									}
									
									// Filter out operators
									else if( FunctionName.StartsWith( TEXT( "operator " ) ) )
									{
										// e.g.
										// operator new
										bPassedFilter = false;
									}
									
									// Filter out member functions of inner class/struct types
									else if( FunctionName.Contains( TEXT( "::" ) ) )
									{
										// e.g.
										// FStateEvent::FStateEvent (UObject)
										bPassedFilter = false;
									}
									
									// Filter class constructor
									else if( !bShowConstructorAndDestructor && FunctionName == FunctionClassName )
									{
										// <class>
										bPassedFilter = false;
									}
									
									// Filter class destructor
									else if( !bShowConstructorAndDestructor && FunctionName.StartsWith( TEXT( "~" ) ) )
									{
										// ~<class>
										bPassedFilter = false;
									}
									
									// Filter various macro-generated Unreal methods and static member functions
									else if( FunctionName == TEXT( "Default" ) ||
											FunctionName == TEXT( "GetPrivateStaticClass" ) ||
											FunctionName == TEXT( "StaticClass" ) ||
											FunctionName.StartsWith( TEXT( "StaticRegisterNatives" ), ESearchCase::CaseSensitive ) ||
											FunctionName.StartsWith( TEXT( "exec" ), ESearchCase::CaseSensitive ) ||
											FunctionName.StartsWith( TEXT( "event" ), ESearchCase::CaseSensitive ) )
									{
										bPassedFilter = false;
									}
									
								}
								
								if( bPassedFilter )
								{
									// Don't add duplicates (overloads, filter mangling, various other reasons for this.)
									// @todo editcode: O(N) look-up here, could slow down symbol gathering!
									if( !FunctionSymbolNames.Contains( FunctionSymbolName ) )
									{
										// Add it to the list
										new( FunctionSymbolNames ) FString( FunctionSymbolName );
									}
								}
								
								if( ShouldAbortDelegate.IsBound() && ShouldAbortDelegate.Execute())
								{
									bSucceeded = false;
									break;
								}
							}
						}
					}
				}
				
				delete [] StringTable;
				delete [] SymbolEntries;
			}
			else
			{
				bSucceeded = false;
			}
			
			if(bSucceeded)
			{
				FScopeLock ScopeLock( &SynchronizationObject );
				
				// Update our symbol cache
				SourceSymbolDatabase.SetFunctionsForClass( ModuleName, ClassName, FunctionSymbolNames );
			}
			break;
		}
	}
#endif	// PLATFORM_WINDOWS
	
}

void FSourceCodeNavigation::GatherFunctionsForActors( TArray< AActor* >& Actors, const EGatherMode::Type GatherMode, TArray< FEditCodeMenuClass >& Classes )
{
	// NOTE: It's important for this function to execute very quickly, especially when GatherMode is "ClassesOnly".  This
	// is because the code may execute every time the user right clicks on an actor in the level editor, before the menu
	// is able to be summoned.  We need the UI to be responsive!

	struct Local
	{
		static FEditCodeMenuClass& GetClassInfo( TArray< FEditCodeMenuClass >& InClasses, const FString& ModuleName, const FString& ClassName, UObject* ReferencedObject = NULL )
		{
			// We're expecting all functions to have a class here
			check( !ClassName.IsEmpty() );

			// Check to see if we already have this class name in our list
			FEditCodeMenuClass* FoundClass = NULL;
			for( int32 CurClassIndex = 0; CurClassIndex < InClasses.Num(); ++CurClassIndex )
			{
				FEditCodeMenuClass& CurClass = InClasses[ CurClassIndex ];
				if( CurClass.Name == ClassName )
				{
					FoundClass = &CurClass;
					break;
				}
			}

			// Add a new class to our list if we need to
			if( FoundClass == NULL )
			{
				FoundClass = new( InClasses ) FEditCodeMenuClass();
				FoundClass->Name = ClassName;
				FoundClass->bIsCompleteList = true;	// Until proven otherwise!
				FoundClass->ReferencedObject = ReferencedObject;
				FoundClass->ModuleName = ModuleName;
			}
			else
			{
				check(FoundClass->ReferencedObject.Get() == ReferencedObject);
			}

			return *FoundClass;
		}


		static void AddFunction( TArray< FEditCodeMenuClass >& InClasses, const FFunctionSymbolInfo& FunctionSymbolInfo, UObject* ReferencedObject = NULL )
		{
			// We're expecting all functions to have a class here
			if( ensure( !FunctionSymbolInfo.ClassName.IsEmpty() ) )
			{
				// Keep track of the current function
				FEditCodeMenuClass& ClassInfo = GetClassInfo(InClasses, FunctionSymbolInfo.ModuleName, FunctionSymbolInfo.ClassName, ReferencedObject);
				ClassInfo.Functions.Add( FunctionSymbolInfo );
			}
			else
			{
				// No class for this function.  We'll ignore it as we only want to show functions for this class
			}
		}
	};

	// Skip low-level classes that we never want users to see.  These usually have a lot of symbols
	// that slow down digestion times and clutter the UI too.

	TSet< FString > ClassesWithIncompleteFunctionLists;

	for( TArray< AActor* >::TIterator It( Actors ); It; ++It )
	{
		AActor* Actor = static_cast<AActor*>( *It );
		checkSlow( Actor->IsA(AActor::StaticClass()) );

		// Grab the class of this actor
		UClass* ActorClass = Actor->GetClass();
		check( ActorClass != NULL );

		// Walk the inheritance hierarchy for this class
		for( UClass* CurClass = ActorClass; CurClass != NULL; CurClass = CurClass->GetSuperClass() )
		{
			// Skip low-level classes if we were asked to do that.  Here, we'll require the class to have
			// been derived from a low level actor/pawn class.
#if !SOURCECODENAVIGATOR_GATHER_LOW_LEVEL_CLASSES
			if( !CurClass->IsChildOf( AActor::StaticClass() ) || // @todo editcode: A bit hacky here, hard-coding types
				  CurClass == AActor::StaticClass() ||
				  CurClass == APawn::StaticClass() )
			{
				continue;
			}
#endif

			const FString CPlusPlusClassName( FString( CurClass->GetPrefixCPP() ) + CurClass->GetName() );

			// Figure out the module file name that this class' C++ code lives in
			FString ModuleName = FPaths::GetBaseFilename(FPlatformProcess::ExecutableName());	// Default to the executable module
			
			// Only bother getting the correct module if we're gathering functions, too, since it can slow down the process a bit.
			if( GatherMode == EGatherMode::ClassesAndFunctions )
			{
				FindClassModuleName( CurClass, ModuleName );
			}

			{
				// Assume there are always C++ functions to gather.  This isn't necessarily correct but it's too
				// slow to check to be sure when only asked to gather classes.  Besides, currently we display
				// functions for UObject which everything derives from, so there are always *some* functions, just
				// not necessarily for every class we report.
				bool bIsCompleteList = false;

				// True to gather functions from the symbol database (slow, but has every function.)
				// False to gather script-exposed native functions from our UObject class data (fast, but only has script-exposed functions.)
				const bool bGetFunctionsFromSymbolDatabase = false;			// @todo editcode: Should we get rid of the unused code path here?

				if( GatherMode == EGatherMode::ClassesAndFunctions )
				{
					if( bGetFunctionsFromSymbolDatabase )
					{
						// Gather functions from symbol database (slow, but has every function.)
						TArray< FString > GatheredFunctionSymbolNames;
						FSourceCodeNavigationImpl::Get().TryToGatherFunctions( ModuleName, CPlusPlusClassName, GatheredFunctionSymbolNames, bIsCompleteList );

						for( int32 CurFunctionIndex = 0; CurFunctionIndex < GatheredFunctionSymbolNames.Num(); ++CurFunctionIndex )
						{
							const FString& FunctionSymbolName = GatheredFunctionSymbolNames[ CurFunctionIndex ];

							FFunctionSymbolInfo SymbolInfo;
							SymbolInfo.SymbolName = FunctionSymbolName;
							SymbolInfo.ClassName = CPlusPlusClassName;
							SymbolInfo.ModuleName = ModuleName;

							Local::AddFunction( Classes, SymbolInfo );
						}
					}
					else
					{
						// Gather script-exposed native functions from our UObject class data (fast, but only has script-exposed functions.)

						// Find all of the editable functions in this class
						for( int32 CurFunctionIndex = 0; CurFunctionIndex < CurClass->NativeFunctionLookupTable.Num(); ++CurFunctionIndex )
						{
							// Convert the function name (e.g., "execOnTouched") to an FString so we can manipulate it easily
							const FString ImplFunctionName = CurClass->NativeFunctionLookupTable[ CurFunctionIndex ].Name.ToString();

							// Create a fully-qualified symbol name for this function that includes the class
							const FString FunctionSymbolName =
								CPlusPlusClassName +						// The C++ class name (e.g. "APawn")
								FString( TEXT( "::" ) ) +					// C++ class scoping
								ImplFunctionName;							// The function name (e.g. "OnTouched")
				
							FFunctionSymbolInfo SymbolInfo;
							SymbolInfo.SymbolName = FunctionSymbolName;
							SymbolInfo.ClassName = CPlusPlusClassName;
							SymbolInfo.ModuleName = ModuleName;

							Local::AddFunction( Classes, SymbolInfo );
						}

						// We always have complete data when gathering directly from the native function table
						bIsCompleteList = true;
					}
				}

				if( !bIsCompleteList )
				{
					// Find the class and mark it incomplete
					FEditCodeMenuClass& ClassInfo = Local::GetClassInfo( Classes, ModuleName, CPlusPlusClassName );
					ClassInfo.bIsCompleteList = false;
				}
			}
		}
	}


	if( GatherMode == EGatherMode::ClassesAndFunctions )
	{
		// Sort function lists
		for( int32 CurClassIndex = 0; CurClassIndex < Classes.Num(); ++CurClassIndex )
		{
			FEditCodeMenuClass& CurClass = Classes[ CurClassIndex ];

			struct FCompareSymbolName
			{
				FORCEINLINE bool operator()( const FFunctionSymbolInfo& A, const FFunctionSymbolInfo& B ) const
				{
					return A.SymbolName < B.SymbolName;
				}
			};
			CurClass.Functions.Sort( FCompareSymbolName() );
		}
	}
}

bool FSourceCodeNavigation::NavigateToFunctionAsync(UFunction* InFunction)
{
	return NavigateToFunction(InFunction);
}

static TArray<ISourceCodeNavigationHandler*> SourceCodeNavigationHandlers;

void FSourceCodeNavigation::AddNavigationHandler(ISourceCodeNavigationHandler* Handler)
{
	SourceCodeNavigationHandlers.Add(Handler);
}

void FSourceCodeNavigation::RemoveNavigationHandler(ISourceCodeNavigationHandler* Handler)
{
	SourceCodeNavigationHandlers.Remove(Handler);
}

void FSourceCodeNavigation::SetPreferredAccessor(const TCHAR* Name)
{
	GConfig->SetString(TEXT("/Script/SourceCodeAccess.SourceCodeAccessSettings"), TEXT("PreferredAccessor"), Name, GEditorSettingsIni);

	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>(TEXT("SourceCodeAccess"));
	SourceCodeAccessModule.SetAccessor(Name);

	RefreshCompilerAvailability();
}

bool FSourceCodeNavigation::CanNavigateToClass(const UClass* InClass)
{
	if (!InClass)
	{
		return false;
	}

	if (ensure(GUnrealEd) && !GUnrealEd->GetUnrealEdOptions()->IsCPPAllowed())
	{
		return false;
	}

	for (int32 i = 0; i < SourceCodeNavigationHandlers.Num(); ++i)
	{
		ISourceCodeNavigationHandler* Handler = SourceCodeNavigationHandlers[i];
		if (Handler->CanNavigateToClass(InClass))
		{
			return true;
		}
	}

	return InClass->HasAllClassFlags(CLASS_Native) && FSourceCodeNavigation::IsCompilerAvailable();
}

bool FSourceCodeNavigation::NavigateToClass(const UClass* InClass)
{
	if (!InClass)
	{
		return false;
	}

	for (int32 i = 0; i < SourceCodeNavigationHandlers.Num(); ++i)
	{
		ISourceCodeNavigationHandler* Handler = SourceCodeNavigationHandlers[i];
		if (Handler->NavigateToClass(InClass))
		{
			return true;
		}
	}

	FString ClassHeaderPath;
	if (FSourceCodeNavigation::FindClassHeaderPath(InClass, ClassHeaderPath) && IFileManager::Get().FileSize(*ClassHeaderPath) != INDEX_NONE)
	{
		FString AbsoluteHeaderPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*ClassHeaderPath);
		FSourceCodeNavigation::OpenSourceFileAsync(AbsoluteHeaderPath);
		return true;
	}
	return false;
}

bool FSourceCodeNavigation::CanNavigateToStruct(const UScriptStruct* InStruct)
{
	if (!InStruct)
	{
		return false;
	}

	if (ensure(GUnrealEd) && !GUnrealEd->GetUnrealEdOptions()->IsCPPAllowed())
	{
		return false;
	}

	for (int32 i = 0; i < SourceCodeNavigationHandlers.Num(); ++i)
	{
		ISourceCodeNavigationHandler* Handler = SourceCodeNavigationHandlers[i];
		if (Handler->CanNavigateToStruct(InStruct))
		{
			return true;
		}
	}

	return (InStruct->StructFlags & STRUCT_Native) && FSourceCodeNavigation::IsCompilerAvailable();
}

bool FSourceCodeNavigation::NavigateToStruct(const UScriptStruct* InStruct)
{
	if (!InStruct)
	{
		return false;
	}

	for (int32 i = 0; i < SourceCodeNavigationHandlers.Num(); ++i)
	{
		ISourceCodeNavigationHandler* Handler = SourceCodeNavigationHandlers[i];
		if (Handler->NavigateToStruct(InStruct))
		{
			return true;
		}
	}

	FString StructHeaderPath;
	if (FSourceCodeNavigation::FindClassHeaderPath(InStruct, StructHeaderPath) && IFileManager::Get().FileSize(*StructHeaderPath) != INDEX_NONE)
	{
		FString AbsoluteHeaderPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*StructHeaderPath);
		FSourceCodeNavigation::OpenSourceFileAsync(AbsoluteHeaderPath);
		return true;
	}
	return false;
}

bool FSourceCodeNavigation::CanNavigateToFunction(const UFunction* InFunction)
{
	if (!InFunction)
	{
		return false;
	}

	if (ensure(GUnrealEd) && !GUnrealEd->GetUnrealEdOptions()->IsCPPAllowed())
	{
		return false;
	}

	for (int32 i = 0; i < SourceCodeNavigationHandlers.Num(); ++i)
	{
		ISourceCodeNavigationHandler* Handler = SourceCodeNavigationHandlers[i];
		if (Handler->CanNavigateToFunction(InFunction))
		{
			return true;
		}
	}

	UClass* OwningClass = InFunction->GetOwnerClass();

	return OwningClass->HasAllClassFlags(CLASS_Native) && FSourceCodeNavigation::IsCompilerAvailable();
}

bool FSourceCodeNavigation::NavigateToFunction(const UFunction* InFunction)
{
	if (!InFunction)
	{
		return false;
	}

	for (int32 i = 0; i < SourceCodeNavigationHandlers.Num(); ++i)
	{
		ISourceCodeNavigationHandler* Handler = SourceCodeNavigationHandlers[i];
		if (Handler->NavigateToFunction(InFunction))
		{
			return true;
		}
	}

	UClass* OwningClass = InFunction->GetOwnerClass();

	if(  OwningClass->HasAllClassFlags( CLASS_Native ))
	{
		FString ModuleName;
		// Find module name for class
		if( FindClassModuleName( OwningClass, ModuleName ))
		{
			const FString SymbolName = FString::Printf( TEXT( "%s%s::%s" ), OwningClass->GetPrefixCPP(), *OwningClass->GetName(), *InFunction->GetName() );
			NavigateToFunctionSourceAsync( SymbolName, ModuleName, false );
			return true;
		}
	}

	return false;
}

bool FSourceCodeNavigation::CanNavigateToProperty(const FProperty* InProperty)
{
	if (!InProperty)
	{
		return false;
	}

	if (ensure(GUnrealEd) && !GUnrealEd->GetUnrealEdOptions()->IsCPPAllowed())
	{
		return false;
	}

	for (int32 i = 0; i < SourceCodeNavigationHandlers.Num(); ++i)
	{
		ISourceCodeNavigationHandler* Handler = SourceCodeNavigationHandlers[i];
		if (Handler->CanNavigateToProperty(InProperty))
		{
			return true;
		}
	}

	return InProperty->IsNative() && IsCompilerAvailable();
}

bool FSourceCodeNavigation::NavigateToProperty(const FProperty* InProperty)
{
	if (!InProperty)
	{
		return false;
	}

	for (int32 i = 0; i < SourceCodeNavigationHandlers.Num(); ++i)
	{
		ISourceCodeNavigationHandler* Handler = SourceCodeNavigationHandlers[i];
		if (Handler->NavigateToProperty(InProperty))
		{
			return true;
		}
	}

	if (InProperty && InProperty->IsNative())
	{
		FString SourceFilePath;
		const bool bFileLocated = FindClassHeaderPath(InProperty->GetOwnerUField(), SourceFilePath) &&
			IFileManager::Get().FileSize(*SourceFilePath) != INDEX_NONE;

		if (bFileLocated)
		{
			const FString AbsoluteSourcePath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*SourceFilePath);
			OpenSourceFileAsync(AbsoluteSourcePath);
			return true;
		}
	}
	return false;
}

bool FSourceCodeNavigation::CanNavigateToStruct(const UStruct* InStruct)
{
	if (!InStruct)
	{
		return false;
	}

	if (ensure(GUnrealEd) && !GUnrealEd->GetUnrealEdOptions()->IsCPPAllowed())
	{
		return false;
	}

	for (int32 i = 0; i < SourceCodeNavigationHandlers.Num(); ++i)
	{
		ISourceCodeNavigationHandler* handler = SourceCodeNavigationHandlers[i];
		if (handler->CanNavigateToStruct(InStruct))
		{
			return true;
		}
	}

	return InStruct->IsNative() && IsCompilerAvailable();
}

bool FSourceCodeNavigation::NavigateToStruct(const UStruct* InStruct)
{
	if (!InStruct)
	{
		return false;
	}

	for (int32 i = 0; i < SourceCodeNavigationHandlers.Num(); ++i)
	{
		ISourceCodeNavigationHandler* handler = SourceCodeNavigationHandlers[i];
		if (handler->NavigateToStruct(InStruct))
		{
			return true;
		}
	}

	FString ClassHeaderPath;
	if (FSourceCodeNavigation::FindClassHeaderPath(InStruct, ClassHeaderPath) && IFileManager::Get().FileSize(*ClassHeaderPath) != INDEX_NONE)
	{
		FString AbsoluteHeaderPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*ClassHeaderPath);
		FSourceCodeNavigation::OpenSourceFileAsync(AbsoluteHeaderPath);
		return true;
	}

	return false;
}

bool FSourceCodeNavigation::FindClassModuleName( UClass* InClass, FString& ModuleName )
{
	bool bResult = false;
	// Find module name from class
	if( InClass )
	{
		UPackage* ClassPackage = InClass->GetOuterUPackage();

		if( ClassPackage )
		{
			//@Package name transition
			FName ShortClassPackageName = FPackageName::GetShortFName(ClassPackage->GetFName());

			// Is this module loaded?  In many cases, we may not have a loaded module for this class' package,
			// as it might be statically linked into the executable, etc.
			if( FModuleManager::Get().IsModuleLoaded( ShortClassPackageName ) )
			{
				// Because the module loaded into memory may have a slightly mutated file name (for
				// hot reload, etc), we ask the module manager for the actual file name being used.  This
				// is important as we need to be sure to get the correct symbols.
				FModuleStatus ModuleStatus;
				if( ensure( FModuleManager::Get().QueryModule( ShortClassPackageName, ModuleStatus ) ) )
				{
					// Use the base file name (no path, no extension) as the module name for symbol look up!
					ModuleName = FPaths::GetBaseFilename(ModuleStatus.FilePath);
					bResult = true;
				}
				else
				{
					// This module should always be known.  Should never happen.
				}
			}
		}
	}
	return bResult;
}

/** Call this to access the multi-cast delegate that you can register a callback with */
FSourceCodeNavigation::FOnSymbolQueryFinished& FSourceCodeNavigation::AccessOnSymbolQueryFinished()
{
	return FSourceCodeNavigationImpl::Get().OnSymbolQueryFinished;
}

/** Returns the name of the selected IDE */
FText FSourceCodeNavigation::GetSelectedSourceCodeIDE()
{
	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
	return SourceCodeAccessModule.GetAccessor().GetNameText();
}

FText FSourceCodeNavigation::GetSuggestedSourceCodeIDE(bool bShortIDEName)
{
#if PLATFORM_WINDOWS
	if ( bShortIDEName )
	{
		return LOCTEXT("SuggestedCodeIDE_ShortWindows", "Visual Studio");
	}
	else
	{
		return LOCTEXT("SuggestedCodeIDE_Windows", "Visual Studio 2022");
	}
#elif PLATFORM_MAC
	return LOCTEXT("SuggestedCodeIDE_Mac", "Xcode");
#elif PLATFORM_LINUX
	return LOCTEXT("SuggestedCodeIDE_Linux", "VS Code");
#else
	return LOCTEXT("SuggestedCodeIDE_Generic", "an IDE to edit source code");
#endif
}


FSlateIcon FSourceCodeNavigation::GetOpenSourceCodeIDEIcon()
{
	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
	return FSlateIcon(SourceCodeAccessModule.GetAccessor().GetStyleSet(), SourceCodeAccessModule.GetAccessor().GetOpenIconName());
}


FSlateIcon FSourceCodeNavigation::GetRefreshSourceCodeIDEIcon()
{
	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
	return FSlateIcon(SourceCodeAccessModule.GetAccessor().GetStyleSet(), SourceCodeAccessModule.GetAccessor().GetRefreshIconName());
}

FString FSourceCodeNavigation::GetSuggestedSourceCodeIDEDownloadURL()
{
	FString SourceCodeIDEURL;
#if PLATFORM_WINDOWS
	// Visual Studio
	FUnrealEdMisc::Get().GetURL( TEXT("SourceCodeIDEURL_Windows"), SourceCodeIDEURL );
#elif PLATFORM_MAC
	// Xcode
	FUnrealEdMisc::Get().GetURL( TEXT("SourceCodeIDEURL_Mac"), SourceCodeIDEURL );
#elif PLATFORM_LINUX
	// VSCode
	FUnrealEdMisc::Get().GetURL( TEXT("SourceCodeIDEURL_Linux"), SourceCodeIDEURL );
#else
	// Unknown platform, just link to wikipedia page on IDEs
	FUnrealEdMisc::Get().GetURL( TEXT("SourceCodeIDEURL_Other"), SourceCodeIDEURL );
#endif
	return SourceCodeIDEURL;
}

bool FSourceCodeNavigation::GetCanDirectlyInstallSourceCodeIDE()
{
#if PLATFORM_WINDOWS
	return true;
#else
	return false;
#endif
}

void FSourceCodeNavigation::DownloadAndInstallSuggestedIDE(FOnIDEInstallerDownloadComplete OnDownloadComplete)
{
	FSourceCodeNavigationImpl& SourceCodeNavImpl = FSourceCodeNavigationImpl::Get();

	// Check to see if the file exists first
	auto UserTempDir = FPaths::ConvertRelativePathToFull(FDesktopPlatformModule::Get()->GetUserTempPath());
	FString InstallerFullPath = FString::Printf(TEXT("%s%s"), *UserTempDir, *SourceCodeNavImpl.GetSuggestedIDEInstallerFileName());

	if (!IPlatformFile::GetPlatformPhysical().FileExists(*InstallerFullPath))
	{
		FString DownloadURL;
		TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();

		// Download the installer for the suggested IDE
		HttpRequest->OnProcessRequestComplete().BindRaw(&SourceCodeNavImpl, &FSourceCodeNavigationImpl::OnSuggestedIDEInstallerDownloadComplete, OnDownloadComplete);
		HttpRequest->SetVerb(TEXT("GET"));

		HttpRequest->SetURL(GetSuggestedSourceCodeIDEDownloadURL());
		HttpRequest->ProcessRequest();
	}
	else
	{
		SourceCodeNavImpl.LaunchIDEInstaller(InstallerFullPath);
		OnDownloadComplete.ExecuteIfBound(true);
	}
}

void FSourceCodeNavigation::RefreshCompilerAvailability()
{
	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
	SourceCodeAccessModule.GetAccessor().RefreshAvailability();

	bCachedIsCompilerAvailable = SourceCodeAccessModule.CanCompileSourceCode();
}

bool FSourceCodeNavigation::OpenSourceFile( const FString& AbsoluteSourcePath, int32 LineNumber, int32 ColumnNumber )
{
	if ( IsCompilerAvailable() )
	{
		ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
		return SourceCodeAccessModule.GetAccessor().OpenFileAtLine(AbsoluteSourcePath, LineNumber, ColumnNumber);
	}

	// Let others know that we've failed to open a source file.
	AccessOnCompilerNotFound().Broadcast();

	return false;
}

bool FSourceCodeNavigation::OpenSourceFiles(const TArray<FString>& AbsoluteSourcePaths)
{
	if ( IsCompilerAvailable() )
	{
		ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
		return SourceCodeAccessModule.GetAccessor().OpenSourceFiles(AbsoluteSourcePaths);
	}

	// Let others know that we've failed to open some source files.
	AccessOnCompilerNotFound().Broadcast();

	return false;
}

void FSourceCodeNavigation::OpenSourceFileAsync(const FString& AbsoluteSourcePath, int32 LineNumber, int32 ColumnNumber)
{
	if (!GAsyncSourceCodeNavigation)
	{
		OpenSourceFile(AbsoluteSourcePath, LineNumber, ColumnNumber);
		return;
	}

	if (!IsCompilerAvailable())
	{
		AccessOnCompilerNotFound().Broadcast();
		return;
	}

	FNotificationInfo Info(LOCTEXT("OpeningSourceFile", "Opening source file..."));
	Info.Image = FAppStyle::GetBrush(TEXT("LevelEditor.RecompileGameCode"));
	Info.bFireAndForget = false;
	Info.ExpireDuration = 2.0f;
	Info.bUseThrobber = true;

	TSharedPtr<SNotificationItem> NotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
	if (NotificationPtr.IsValid())
	{
		NotificationPtr->SetCompletionState(SNotificationItem::CS_Pending);
	}

	// Defer to the next game thread tick so the calling UI event handler returns immediately
	// and Slate can render a frame before the (potentially blocking) accessor call executes.
	// NOTE: We must stay on the game thread because some accessors (e.g. Rider) may show
	// Slate dialogs internally, which requires game thread access.
	FString PathCopy = AbsoluteSourcePath;
	AsyncTask(ENamedThreads::GameThread, [PathCopy, LineNumber, ColumnNumber, NotificationPtr]()
	{
		ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
		const bool bSuccess = SourceCodeAccessModule.GetAccessor().OpenFileAtLine(PathCopy, LineNumber, ColumnNumber);

		if (NotificationPtr.IsValid())
		{
			if (bSuccess)
			{
				NotificationPtr->SetCompletionState(SNotificationItem::CS_Success);
			}
			else
			{
				NotificationPtr->SetText(LOCTEXT("OpenSourceFileFailed", "Failed to open source file"));
				NotificationPtr->SetCompletionState(SNotificationItem::CS_Fail);
			}
			NotificationPtr->ExpireAndFadeout();
		}
	});
}

void FSourceCodeNavigation::OpenSourceFilesAsync(const TArray<FString>& AbsoluteSourcePaths)
{
	if (!GAsyncSourceCodeNavigation)
	{
		OpenSourceFiles(AbsoluteSourcePaths);
		return;
	}

	if (!IsCompilerAvailable())
	{
		AccessOnCompilerNotFound().Broadcast();
		return;
	}

	FNotificationInfo Info(LOCTEXT("OpeningSourceFiles", "Opening source files..."));
	Info.Image = FAppStyle::GetBrush(TEXT("LevelEditor.RecompileGameCode"));
	Info.bFireAndForget = false;
	Info.ExpireDuration = 2.0f;
	Info.bUseThrobber = true;

	TSharedPtr<SNotificationItem> NotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
	if (NotificationPtr.IsValid())
	{
		NotificationPtr->SetCompletionState(SNotificationItem::CS_Pending);
	}

	TArray<FString> PathsCopy = AbsoluteSourcePaths;
	AsyncTask(ENamedThreads::GameThread, [PathsCopy = MoveTemp(PathsCopy), NotificationPtr]()
	{
		ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
		const bool bSuccess = SourceCodeAccessModule.GetAccessor().OpenSourceFiles(PathsCopy);

		if (NotificationPtr.IsValid())
		{
			if (bSuccess)
			{
				NotificationPtr->SetCompletionState(SNotificationItem::CS_Success);
			}
			else
			{
				NotificationPtr->SetText(LOCTEXT("OpenSourceFilesFailed", "Failed to open source files"));
				NotificationPtr->SetCompletionState(SNotificationItem::CS_Fail);
			}
			NotificationPtr->ExpireAndFadeout();
		}
	});
}

bool FSourceCodeNavigation::AddSourceFiles(const TArray<FString>& AbsoluteSourcePaths)
{
	if ( IsCompilerAvailable() )
	{
		ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
		return SourceCodeAccessModule.GetAccessor().AddSourceFiles(AbsoluteSourcePaths, GetSourceFileDatabase().GetModuleNames());
	}

	return false;
}

bool FSourceCodeNavigation::OpenModuleSolution()
{
	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
	return SourceCodeAccessModule.GetAccessor().OpenSolution();
}

bool FSourceCodeNavigation::OpenProjectSolution(const FString& InProjectFilename)
{
	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
	return SourceCodeAccessModule.GetAccessor().OpenSolutionAtPath(InProjectFilename);
}

/** Query if the current source code solution exists */
bool FSourceCodeNavigation::DoesModuleSolutionExist()
{
	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
	return SourceCodeAccessModule.GetAccessor().DoesSolutionExist();
}

/** Call this to access the multi-cast delegate that you can register a callback with */
FSourceCodeNavigation::FOnCompilerNotFound& FSourceCodeNavigation::AccessOnCompilerNotFound()
{
	return FSourceCodeNavigationImpl::Get().OnCompilerNotFound;
}

FSourceCodeNavigation::FOnNewModuleAdded& FSourceCodeNavigation::AccessOnNewModuleAdded()
{
	return FSourceCodeNavigationImpl::Get().OnNewModuleAdded;
}

bool FSourceCodeNavigation::FindModulePath( const FString& InModuleName, FString &OutModulePath )
{
	// Try to find a file matching the module name
	const TArray<FString>& ModuleNames = GetSourceFileDatabase().GetModuleNames();
	FString FindModuleSuffix = FString(TEXT("/")) + InModuleName + ".Build.cs";
	for (int32 Idx = 0; Idx < ModuleNames.Num(); Idx++)
	{
		if (ModuleNames[Idx].EndsWith(FindModuleSuffix))
		{
			OutModulePath = ModuleNames[Idx].Left(ModuleNames[Idx].Len() - FindModuleSuffix.Len());
			return true;
		}
	}
	return false;
}

bool FSourceCodeNavigation::FindModulePath( const UPackage* InModulePackage, FString& OutClassHeaderPath )
{
	if (!InModulePackage)
	{
		return false;
	}

	// Get the package name and skip past the "/Script/" portion to get the module name
	FString ModulePackageName = InModulePackage->GetName();

	int32 ModuleNameIdx;

	if (ModulePackageName.FindLastChar(TEXT('/'), ModuleNameIdx))
	{
		// Find the base path for the module
		return FindModulePath(*ModulePackageName + ModuleNameIdx + 1, OutClassHeaderPath);
	}

	return false;
}

bool FSourceCodeNavigation::FindClassHeaderPath( const UField* InField, FString &OutClassHeaderPath )
{
	if (!InField)
	{
		return false;
	}

	// Get the class package, and skip past the "/Script/" portion to get the module name
	UPackage* ModulePackage = InField->GetPackage();

	// Find the base path for the module
	FString ModuleBasePath;
	if(FSourceCodeNavigation::FindModulePath(ModulePackage, ModuleBasePath))
	{
		// Get the metadata for the class path relative to the module base
		const FString& ModuleRelativePath = ModulePackage->GetMetaData().GetValue(InField, TEXT("ModuleRelativePath"));
		if(ModuleRelativePath.Len() > 0)
		{
			OutClassHeaderPath = ModuleBasePath / ModuleRelativePath;
			return true;
		}
	}
	return false;
}

bool FSourceCodeNavigation::FindClassSourcePath( const UField* InField, FString &OutClassSourcePath )
{
	if (!InField)
	{
		return false;
	}

	// Get the class package, and skip past the "/Script/" portion to get the module name
	UPackage *ModulePackage = InField->GetPackage();

	// Find the base path for the module
	FString ModuleBasePath;
	if (FSourceCodeNavigation::FindModulePath(ModulePackage, ModuleBasePath))
	{
		// Get the metadata for the class path relative to the module base
		// Given this we can try and find the corresponding .cpp file
		const FString& ModuleRelativePath = ModulePackage->GetMetaData().GetValue(InField, TEXT("ModuleRelativePath"));
		if(ModuleRelativePath.Len() > 0)
		{
			const FString PotentialCppLeafname = FPaths::GetBaseFilename(ModuleRelativePath) + TEXT(".cpp");
			FString PotentialCppFilename = ModuleBasePath / FPaths::GetPath(ModuleRelativePath) / PotentialCppLeafname;

			// Is the .cpp file in the same folder as the header file?
			if(FPaths::FileExists(PotentialCppFilename))
			{
				OutClassSourcePath = PotentialCppFilename;
				return true;
			}

			const FString PublicPath = ModuleBasePath / "Public" / "";		// Ensure trailing /
			const FString PrivatePath = ModuleBasePath / "Private" / "";	// Ensure trailing /
			const FString ClassesPath = ModuleBasePath / "Classes" / "";	// Ensure trailing /

			// If the path starts with Public or Classes, try swapping those out with Private
			if(PotentialCppFilename.StartsWith(PublicPath))
			{
				PotentialCppFilename.ReplaceInline(*PublicPath, *PrivatePath);
			}
			else if(PotentialCppFilename.StartsWith(ClassesPath))
			{
				PotentialCppFilename.ReplaceInline(*ClassesPath, *PrivatePath);
			}
			else
			{
				PotentialCppFilename.Empty();
			}
			if(!PotentialCppFilename.IsEmpty() && FPaths::FileExists(PotentialCppFilename))
			{
				OutClassSourcePath = PotentialCppFilename;
				return true;
			}

			// Still no luck, try and search for the file on the filesystem
			TArray<FString> Filenames;
			IFileManager::Get().FindFilesRecursive(Filenames, *ModuleBasePath, *PotentialCppLeafname, true, false, false);
				
			if(Filenames.Num() > 0)
			{
				// Assume it's the first match (we should really only find a single file with a given name within a project anyway)
				OutClassSourcePath = Filenames[0];
				return true;
			}
		}
	}
	return false;
}

void FSourceCodeNavigationImpl::OnSuggestedIDEInstallerDownloadComplete(FHttpRequestPtr Request, FHttpResponsePtr Response, bool bWasSuccessful, FOnIDEInstallerDownloadComplete OnDownloadComplete)
{
	if (bWasSuccessful)
	{
		// Get the user's temp directory
		auto UserTempDir = FDesktopPlatformModule::Get()->GetUserTempPath();

		// Create the installer file in the temp dir
		auto InstallerName = GetSuggestedIDEInstallerFileName();
		FString Filepath = FString::Printf(TEXT("%s%s"), *UserTempDir, *InstallerName);
		auto InstallerFileHandle = IPlatformFile::GetPlatformPhysical().OpenWrite(*Filepath);

		// Copy the content from the response into the installer file
		auto InstallerContent = Response->GetContent();

		bool bWriteSucceeded = InstallerFileHandle ? InstallerFileHandle->Write(InstallerContent.GetData(), InstallerContent.Num()) : false;
		delete InstallerFileHandle;

		if (bWriteSucceeded)
		{
			// Launch the created executable in a separate window to begin the installation
			LaunchIDEInstaller(Filepath);
		}
		else
		{
			bWasSuccessful = false;
		}
	}

	OnDownloadComplete.ExecuteIfBound(bWasSuccessful);
}

void FSourceCodeNavigationImpl::TryToGatherFunctions( const FString& ModuleName, const FString& ClassName, TArray< FString >& OutFunctionSymbolNames, bool& OutIsCompleteList )
{
	FScopeLock ScopeLock( &SynchronizationObject );

	// Start out by gathering whatever functions we've already cached
	const bool bFoundFunctions = SourceSymbolDatabase.QueryFunctionsForClass( ModuleName, ClassName, OutFunctionSymbolNames, OutIsCompleteList );
	if( !bFoundFunctions )
	{
		OutIsCompleteList = false;
	}

	if( !bFoundFunctions || !OutIsCompleteList )
	{
		// Enqueue a task to gather symbols.  This will be kicked off the next time we have a chance (as early as next Tick() call)
		FSymbolGatherRequest GatherRequest = { ModuleName, ClassName };
		ClassesToGatherSymbolsFor.AddUnique( GatherRequest );
	}
}

void FSourceCodeNavigationImpl::SymbolQueryStarted()
{
	// Starting a new request! Notify the UI.
	if ( SymbolQueryNotificationPtr.IsValid() )
	{
		SymbolQueryNotificationPtr.Pin()->ExpireAndFadeout();
	}

	FNotificationInfo Info( NSLOCTEXT("SourceCodeNavigation", "SymbolQueryInProgress", "Loading C++ Symbols") );
	Info.bFireAndForget = false;

	SymbolQueryNotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);

	if ( SymbolQueryNotificationPtr.IsValid() )
	{
		SymbolQueryNotificationPtr.Pin()->SetCompletionState(SNotificationItem::CS_Pending);
	}
}

void FSourceCodeNavigationImpl::SymbolQueryFinished()
{
	// Finished all requests! Notify the UI.
	TSharedPtr<SNotificationItem> NotificationItem = SymbolQueryNotificationPtr.Pin();
	if ( NotificationItem.IsValid() )
	{
		NotificationItem->SetText( NSLOCTEXT("SourceCodeNavigation", "SymbolQueryComplete", "C++ Symbols Loaded!") );
		NotificationItem->SetCompletionState(SNotificationItem::CS_Success);
		NotificationItem->ExpireAndFadeout();

		SymbolQueryNotificationPtr.Reset();
	}

	// Let others know that we've gathered some new symbols
	OnSymbolQueryFinished.Broadcast();
}

FString FSourceCodeNavigationImpl::GetSuggestedIDEInstallerFileName()
{
	FString Extension;
#if PLATFORM_WINDOWS
	Extension = "exe";
#elif PLATFORM_MAC
	Extension = "app";
#endif

	return FString::Printf(TEXT("%s.%s"), *SourceCodeNavigationDefs::IDEInstallerFilename, *Extension);
}

void FSourceCodeNavigationImpl::LaunchIDEInstaller(const FString& Filepath)
{
#if PLATFORM_WINDOWS
	TStringBuilder<512> Params;	

	// Try and load suggested components from the Windows SDK configuration file
	const FString WindowsSDKFile = FPaths::RootDir() / TEXT("Engine/Config/Windows/Windows_SDK.json");
	if (FPaths::FileExists(WindowsSDKFile))
	{
		FString FileContents;
		if (FFileHelper::LoadFileToString(FileContents, *WindowsSDKFile))
		{
			TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(FileContents);
			TSharedPtr<FJsonObject> RootObject;
			if (FJsonSerializer::Deserialize(JsonReader, RootObject) && RootObject.IsValid())
			{
				const TArray<TSharedPtr<FJsonValue>> SuggestedComponentsArray = RootObject->GetArrayField(TEXT("VisualStudioSuggestedComponents"));
				for (const TSharedPtr<FJsonValue>& SuggestedComponents : SuggestedComponentsArray)
				{
					FString Value = SuggestedComponents->AsString();
					if (!Value.IsEmpty())
					{
						Params.Appendf(TEXT("--add \"%s\" "), *Value);
					}
				}
			}
		}
	}
	
	if (Params.Len() == 0)
	{
		UE_LOGF(LogSelectionDetails, Warning, "LaunchIDEInstaller: Failed to load suggested Visual Studio components from [%ls]", *WindowsSDKFile);
	}

	Params.Append(TEXT("--productId \"Microsoft.VisualStudio.Product.Community\" --campaign \"EpicGames_UE5\""));
	FPlatformProcess::ExecElevatedProcess(*Filepath, Params.GetData(), nullptr);
#endif
}

void FSourceCodeNavigationImpl::Tick( float DeltaTime )
{
	const bool bAsyncWorkAvailable = ClassesToGatherSymbolsFor.Num() > 0;

	// Do we have any work to do?
	if( bAsyncWorkAvailable )
	{
		// Are we still busy gathering functions?
		const bool bIsBusy = AsyncSymbolGatherer.IsValid() && !AsyncSymbolGatherer->IsDone();
		if( !bIsBusy )
		{
			const FSymbolGatherRequest GatherRequest = ClassesToGatherSymbolsFor[ 0 ];
			ClassesToGatherSymbolsFor.RemoveAt( 0 );

			// Init stack walking here to ensure that module manager doesn't need to be accessed on the thread inside the asynk task
			FPlatformStackWalk::InitStackWalking();

			// Start the async task
			AsyncSymbolGatherer = MakeShareable( new FAsyncTask< FAsyncSymbolGatherer >( GatherRequest.ModuleName, GatherRequest.ClassName ) );
			AsyncSymbolGatherer->StartBackgroundTask();
		}
		else
		{
			// Current task is still running, so wait until some other time
		}
	}

	// Determine if starting new work or finishing the last of the queued work
	const bool bAsyncWorkWasInProgress = bAsyncWorkIsInProgress;
	bAsyncWorkIsInProgress = AsyncSymbolGatherer.IsValid() && !AsyncSymbolGatherer->IsWorkDone();

	if (!bAsyncWorkWasInProgress && bAsyncWorkAvailable)
	{
		SymbolQueryStarted();
	}
	else if (bAsyncWorkWasInProgress && !bAsyncWorkIsInProgress && !bAsyncWorkAvailable)
	{
		SymbolQueryFinished();
	}

}

TStatId FSourceCodeNavigationImpl::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FSourceCodeNavigationImpl, STATGROUP_Tickables);
}

#undef LOCTEXT_NAMESPACE
