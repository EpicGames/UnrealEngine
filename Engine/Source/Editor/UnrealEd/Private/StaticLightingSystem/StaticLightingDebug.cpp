// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	StaticLightingDebug.cpp: Code for debugging static lighting
=============================================================================*/

#include "CoreMinimal.h"
#include "RawIndexBuffer.h"
#include "Components/StaticMeshComponent.h"
#include "CanvasTypes.h"
#include "Engine/Engine.h"
#include "EngineGlobals.h"
#include "SceneView.h"
#include "StaticMeshComponentLODInfo.h"
#include "StaticMeshResources.h"
#include "StaticLightingSystem/StaticLightingPrivate.h"
#include "LightMap.h"
#include "Engine/MapBuildDataRegistry.h"
#include "Components/ModelComponent.h"
#include "Engine/StaticMesh.h"
#include "TextureResource.h"

#include <limits>

/** Information about the texel that is selected */
FSelectedLightmapSample GCurrentSelectedLightmapSample;

/** Information about the last static lighting build */
FDebugLightingOutput GDebugStaticLightingInfo;

#if WITH_EDITOR

/** Helper function that writes a texel into the given texture. */
static void WriteTexel(UTexture2D* Texture, int32 X, int32 Y, FColor NewColor)
{
	if (X >= 0 && X < Texture->GetSizeX() && Y >= 0 && Y < Texture->GetSizeY())
	{
		check(X >= 0 && X < Texture->GetSizeX());
		check(Y >= 0 && Y < Texture->GetSizeY());

		// Only supporting uncompressed textures for now
		if (Texture->GetPlatformData() &&
			Texture->GetPlatformData()->PixelFormat == PF_B8G8R8A8)
		{
			// The runtime data needs to be fully cached in memory for this to work.
			// These changes won't (and don't need to) persist.
			if (Texture->GetPlatformData()->TryInlineMipData(0, Texture->GetPathName()))
			{
				// Release the texture's resources and block until the rendering thread is done accessing it
				Texture->ReleaseResource();
				FTexture2DMipMap& BaseMip = Texture->GetPlatformData()->Mips[0];
				FColor* Data = (FColor*)BaseMip.BulkData.Lock( LOCK_READ_WRITE );
				FColor& SelectedTexel = Data[Y * Texture->GetSizeX() + X];
				// Write the new color
				SelectedTexel = NewColor;
				BaseMip.BulkData.Unlock();
				// Re-initialize the textures render resources
				Texture->UpdateResource();
			}
		}
		else
		{
			UE_LOGF(LogStaticLightingSystem, Log, "Texel selection coloring failed because the lightmap is not PF_B8G8R8A8!");
		}
	}
}

static bool UpdateSelectedTexel(
	UPrimitiveComponent* Component, 
	int32 NodeIndex, 
	FLightMapRef Lightmap, 
	const FVector& Position, 
	FVector2D InterpolatedUV, 
	int32 LocalX, int32 LocalY,
	int32 LightmapSizeX, int32 LightmapSizeY)
{
	if (Component == GCurrentSelectedLightmapSample.Component
		&& NodeIndex == GCurrentSelectedLightmapSample.NodeIndex
		&& LocalX == GCurrentSelectedLightmapSample.LocalX
		&& LocalY == GCurrentSelectedLightmapSample.LocalY)
	{
		return false;
	}
	else
	{
		// Store information about the selected texel
		FSelectedLightmapSample NewSelectedTexel(Component, NodeIndex, Lightmap, Position, LocalX, LocalY, LightmapSizeX, LightmapSizeY);

		if (IsValidRef(Lightmap))
		{
			FLightMap2D* Lightmap2D = Lightmap->GetLightMap2D();
			check(Lightmap2D);
			const FVector2D CoordinateScale = Lightmap2D->GetCoordinateScale();
			const FVector2D CoordinateBias = Lightmap2D->GetCoordinateBias();
			// Calculate lightmap atlas UV's for the selected point
			FVector2D LightmapUV = InterpolatedUV * CoordinateScale + CoordinateBias;

			int32 LightmapIndex = Lightmap2D->AllowsHighQualityLightmaps() ? 0 : 1;
			UTexture2D* CurrentLightmap = Lightmap2D->GetTexture( LightmapIndex );
			{
				// UV's in the lightmap atlas
				int32 LightmapX = FMath::TruncToInt32(LightmapUV.X * CurrentLightmap->GetSizeX());
				int32 LightmapY = FMath::TruncToInt32(LightmapUV.Y * .5f * CurrentLightmap->GetSizeY());
				// Write the selection color to the selected lightmap texel
				WriteTexel(CurrentLightmap, LightmapX, LightmapY, GTexelSelectionColor);
			}

			{
				// UV's in the lightmap atlas
				int32 LightmapX = FMath::TruncToInt32(LightmapUV.X * CurrentLightmap->GetSizeX());
				int32 LightmapY = FMath::TruncToInt32((LightmapUV.Y * .5f + .5f) * CurrentLightmap->GetSizeY());
				// Write the selection color to the selected lightmap texel
				WriteTexel(CurrentLightmap, LightmapX, LightmapY, GTexelSelectionColor);
			}

			GCurrentSelectedLightmapSample = NewSelectedTexel;
			return true;
		}
		else
		{
			UE_LOGF(LogStaticLightingSystem, Log, "Texel selection failed because the lightmap is an invalid reference!");
			return false;
		}
	}
}

static bool GetBarycentricWeights(
	const FVector& Position0,
	const FVector& Position1,
	const FVector& Position2,
	FVector InterpolatePosition,
	float Tolerance,
	double& PlaneDistance,
	FVector& BarycentricWeights
	)
{
	BarycentricWeights = FVector::ZeroVector;
	FVector TriangleNormal = (Position0 - Position1) ^ (Position2 - Position0);
	FVector::FReal ParallelogramArea = TriangleNormal.Size();
	FVector UnitTriangleNormal = TriangleNormal / ParallelogramArea;
	PlaneDistance = UnitTriangleNormal | (InterpolatePosition - Position0);

	// Move the position to interpolate to into the plane of the triangle along the normal, 
	// Otherwise there will be error in our barycentric coordinates
	InterpolatePosition -= UnitTriangleNormal * PlaneDistance;

	FVector NormalU = (InterpolatePosition - Position1) ^ (Position2 - InterpolatePosition);
	// Signed area, if negative then InterpolatePosition is not in the triangle
	FVector::FReal ParallelogramAreaU = NormalU.Size() * FMath::FloatSelect(NormalU | TriangleNormal, (FVector::FReal)1.0f, (FVector::FReal)-1.0f);
	FVector::FReal BaryCentricU = ParallelogramAreaU / ParallelogramArea;

	FVector NormalV = (InterpolatePosition - Position2) ^ (Position0 - InterpolatePosition);
	FVector::FReal ParallelogramAreaV = NormalV.Size() * FMath::FloatSelect(NormalV | TriangleNormal, (FVector::FReal)1.0f, (FVector::FReal)-1.0f);
	FVector::FReal BaryCentricV = ParallelogramAreaV / ParallelogramArea;

	FVector::FReal BaryCentricW = 1.0f - BaryCentricU - BaryCentricV;
	if (BaryCentricU > -Tolerance && BaryCentricV > -Tolerance && BaryCentricW > -Tolerance)
	{
		BarycentricWeights = FVector(BaryCentricU, BaryCentricV, BaryCentricW);
		return true;
	}

	return false;
}

float TriangleTolerance = .1f;

/** Updates GCurrentSelectedLightmapSample given a selected actor's components and the location of the click. */
void SetDebugLightmapSample(TArray<UActorComponent*>* Components, UModel* Model, int32 iSurf, FVector ClickLocation)
{
	if (IsTexelDebuggingEnabled())
	{
		UStaticMeshComponent* SMComponent = NULL;
		if (Components)
		{
			// Find the first supported component
			for (int32 ComponentIndex = 0; ComponentIndex < Components->Num() && !SMComponent; ComponentIndex++)
			{
				SMComponent = Cast<UStaticMeshComponent>((*Components)[ComponentIndex]);
				if (SMComponent && (!SMComponent->GetStaticMesh() || SMComponent->LODData.Num() == 0))
				{
					SMComponent = NULL;
				}
			}
		}
	 
		bool bFoundLightmapSample = false;
		// Only static mesh components and BSP handled for now
		if (SMComponent)
		{
			UStaticMesh* StaticMesh = SMComponent->GetStaticMesh();
			check(StaticMesh);
			check(StaticMesh->GetRenderData());
			check(StaticMesh->GetRenderData()->LODResources.Num());
			// Only supporting LOD0
			const int32 LODIndex = 0;
			FStaticMeshLODResources& LODModel = StaticMesh->GetRenderData()->LODResources[LODIndex];
			FIndexArrayView Indices = LODModel.IndexBuffer.GetArrayView();
			const bool bHasStaticLighting = SMComponent->HasStaticLighting();

			if (bHasStaticLighting)
			{
				bool bUseTextureMap = false;
				int32 LightmapSizeX = 0;
				int32 LightmapSizeY = 0;
				SMComponent->GetLightMapResolution(LightmapSizeX, LightmapSizeY);

				if (LightmapSizeX > 0 && LightmapSizeY > 0 
					&& StaticMesh->GetLightMapCoordinateIndex() >= 0 
					&& (uint32)StaticMesh->GetLightMapCoordinateIndex() < LODModel.VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords()
					)
				{
					bUseTextureMap = true;
				}
				else
				{
					bUseTextureMap = false;
				}

				if (bUseTextureMap)
				{
					double ClosestPlaneDistance = std::numeric_limits<double>::max();
					FVector ClosestPlaneBaryCentricWeights = FVector::ZeroVector;
					int32 ClosestPlaneTriangleIndex = -1;

					// Search through the static mesh's triangles for the one that was hit (since we can't get triangle index from a line check)
					for(int32 TriangleIndex = 0; TriangleIndex < Indices.Num(); TriangleIndex += 3)
					{
						uint32 Index0 = Indices[TriangleIndex];
						uint32 Index1 = Indices[TriangleIndex + 1];
						uint32 Index2 = Indices[TriangleIndex + 2];

						// Transform positions to world space
						FVector Position0 = SMComponent->GetComponentTransform().TransformPosition((FVector)LODModel.VertexBuffers.PositionVertexBuffer.VertexPosition(Index0));
						FVector Position1 = SMComponent->GetComponentTransform().TransformPosition((FVector)LODModel.VertexBuffers.PositionVertexBuffer.VertexPosition(Index1));
						FVector Position2 = SMComponent->GetComponentTransform().TransformPosition((FVector)LODModel.VertexBuffers.PositionVertexBuffer.VertexPosition(Index2));

						double PlaneDistance;
						FVector BaryCentricWeights;
						// Continue if click location is in the triangle and get its barycentric weights
						if (GetBarycentricWeights(Position0, Position1, Position2, ClickLocation, TriangleTolerance, PlaneDistance, BaryCentricWeights))
						{
							if (FMath::Abs(PlaneDistance) < ClosestPlaneDistance)
							{
								ClosestPlaneBaryCentricWeights = BaryCentricWeights;
								ClosestPlaneDistance = FMath::Abs(PlaneDistance);
								ClosestPlaneTriangleIndex = TriangleIndex;
							}
						}
					}

					if (ClosestPlaneTriangleIndex != -1)
					{
						uint32 Index0 = Indices[ClosestPlaneTriangleIndex];
						uint32 Index1 = Indices[ClosestPlaneTriangleIndex + 1];
						uint32 Index2 = Indices[ClosestPlaneTriangleIndex + 2];

						// Fetch lightmap UV's
						FVector2D LightmapUV0 = FVector2D(LODModel.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(Index0, StaticMesh->GetLightMapCoordinateIndex()));
						FVector2D LightmapUV1 = FVector2D(LODModel.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(Index1, StaticMesh->GetLightMapCoordinateIndex()));
						FVector2D LightmapUV2 = FVector2D(LODModel.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(Index2, StaticMesh->GetLightMapCoordinateIndex()));
						// Interpolate lightmap UV's to the click location
						FVector2D InterpolatedUV = LightmapUV0 * ClosestPlaneBaryCentricWeights.X + LightmapUV1 * ClosestPlaneBaryCentricWeights.Y + LightmapUV2 * ClosestPlaneBaryCentricWeights.Z;

						int32 PaddedSizeX = LightmapSizeX;
						int32 PaddedSizeY = LightmapSizeY;
						if (GLightmassDebugOptions.bPadMappings && GAllowLightmapPadding && LightmapSizeX - 2 > 0 && LightmapSizeY - 2 > 0)
						{
							PaddedSizeX -= 2;
							PaddedSizeY -= 2;
						}

						const int32 LocalX = FMath::TruncToInt32(InterpolatedUV.X * PaddedSizeX);
						const int32 LocalY = FMath::TruncToInt32(InterpolatedUV.Y * PaddedSizeY);
						if (LocalX < 0 || LocalX >= PaddedSizeX
							|| LocalY < 0 || LocalY >= PaddedSizeY)
						{
							UE_LOGF(LogStaticLightingSystem, Log, "Texel selection failed because the lightmap UV's wrap!");
						}
						else
						{
							const FMeshMapBuildData* MeshMapBuildData = SMComponent->GetMeshMapBuildData(SMComponent->LODData[LODIndex]);

							if (MeshMapBuildData)
							{
								bFoundLightmapSample = UpdateSelectedTexel(SMComponent, -1, MeshMapBuildData->LightMap, ClickLocation, InterpolatedUV, LocalX, LocalY, LightmapSizeX, LightmapSizeY);
							}
						}
					}
				}

				if (!bFoundLightmapSample && Indices.Num() > 0)
				{
					const int32 SelectedTriangle = FMath::RandRange(0, Indices.Num() / 3 - 1);

					uint32 Index0 = Indices[SelectedTriangle];
					uint32 Index1 = Indices[SelectedTriangle + 1];
					uint32 Index2 = Indices[SelectedTriangle + 2];

					FVector2D LightmapUV0 = FVector2D(LODModel.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(Index0, StaticMesh->GetLightMapCoordinateIndex()));
					FVector2D LightmapUV1 = FVector2D(LODModel.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(Index1, StaticMesh->GetLightMapCoordinateIndex()));
					FVector2D LightmapUV2 = FVector2D(LODModel.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(Index2, StaticMesh->GetLightMapCoordinateIndex()));

					FVector BaryCentricWeights;
					BaryCentricWeights.X = FMath::FRandRange(0.f, 1.f);
					BaryCentricWeights.Y = FMath::FRandRange(0.f, 1.f);

					if (BaryCentricWeights.X + BaryCentricWeights.Y >= 1)
					{
						BaryCentricWeights.X = 1 - BaryCentricWeights.X;
						BaryCentricWeights.Y = 1 - BaryCentricWeights.Y;
					}

					BaryCentricWeights.Z = 1 - BaryCentricWeights.X - BaryCentricWeights.Y;

					FVector2D InterpolatedUV = LightmapUV0 * BaryCentricWeights.X + LightmapUV1 * BaryCentricWeights.Y + LightmapUV2 * BaryCentricWeights.Z;

					UE_LOGF(LogStaticLightingSystem, Log, "Failed to intersect any triangles, picking random texel");

					int32 PaddedSizeX = LightmapSizeX;
					int32 PaddedSizeY = LightmapSizeY;
					if (GLightmassDebugOptions.bPadMappings && GAllowLightmapPadding && LightmapSizeX - 2 > 0 && LightmapSizeY - 2 > 0)
					{
						PaddedSizeX -= 2;
						PaddedSizeY -= 2;
					}

					const int32 LocalX = FMath::TruncToInt32(InterpolatedUV.X * PaddedSizeX);
					const int32 LocalY = FMath::TruncToInt32(InterpolatedUV.Y * PaddedSizeY);
					if (LocalX < 0 || LocalX >= PaddedSizeX
						|| LocalY < 0 || LocalY >= PaddedSizeY)
					{
						UE_LOGF(LogStaticLightingSystem, Log, "Texel selection failed because the lightmap UV's wrap!");
					}
					else
					{
						const FMeshMapBuildData* MeshMapBuildData = SMComponent->GetMeshMapBuildData(SMComponent->LODData[LODIndex]);

						if (MeshMapBuildData)
						{
							bFoundLightmapSample = UpdateSelectedTexel(SMComponent, -1, MeshMapBuildData->LightMap, ClickLocation, InterpolatedUV, LocalX, LocalY, LightmapSizeX, LightmapSizeY);
						}
					}		
				}
			}
		}
		else if (Model)
		{
			UWorld* World = Model->LightingLevel->OwningWorld;
			check( World);

			UModelComponent* ClosestComponent = NULL;
			int32 ClosestElementIndex = -1;
			uint32 ClosestTriangleIndex = 0;
			FVector ClosestPlaneBaryCentricWeights = FVector(0);
			double ClosestPlaneDistance = std::numeric_limits<double>::max();

			for (int32 ModelIndex = 0; ModelIndex < World->GetCurrentLevel()->ModelComponents.Num(); ModelIndex++)
			{
				UModelComponent* CurrentComponent = World->GetCurrentLevel()->ModelComponents[ModelIndex];
				int32 LightmapSizeX = 0;
				int32 LightmapSizeY = 0;
				CurrentComponent->GetLightMapResolution(LightmapSizeX, LightmapSizeY);
				if (LightmapSizeX > 0 && LightmapSizeY > 0)
				{
					for (int32 ElementIndex = 0; ElementIndex < CurrentComponent->GetElements().Num(); ElementIndex++)
					{
						FModelElement& Element = CurrentComponent->GetElements()[ElementIndex];
						TUniquePtr<FRawIndexBuffer16or32>* IndexBufferRef = Model->MaterialIndexBuffers.Find(Element.Material);
						check(IndexBufferRef);
						for(uint32 TriangleIndex = Element.FirstIndex; TriangleIndex < Element.FirstIndex + Element.NumTriangles * 3; TriangleIndex += 3)
						{
							uint32 Index0 = (*IndexBufferRef)->Indices[TriangleIndex];
							uint32 Index1 = (*IndexBufferRef)->Indices[TriangleIndex + 1];
							uint32 Index2 = (*IndexBufferRef)->Indices[TriangleIndex + 2];

							FModelVertex* ModelVertices = (FModelVertex*)Model->VertexBuffer.Vertices.GetData();
							FVector Position0 = (FVector)ModelVertices[Index0].Position;
							FVector Position1 = (FVector)ModelVertices[Index1].Position;
							FVector Position2 = (FVector)ModelVertices[Index2].Position;

							double PlaneDistance;
							FVector BaryCentricWeights;
							// Continue if click location is in the triangle and get its barycentric weights
							if (GetBarycentricWeights(Position0, Position1, Position2, ClickLocation, .001f, PlaneDistance, BaryCentricWeights))
							{
								if (FMath::Abs(PlaneDistance) < ClosestPlaneDistance)
								{
									ClosestPlaneBaryCentricWeights = BaryCentricWeights;
									ClosestPlaneDistance = FMath::Abs(PlaneDistance);
									ClosestTriangleIndex = TriangleIndex;
									ClosestComponent = CurrentComponent;
									ClosestElementIndex = ElementIndex;
								}
							}
						}
					}
				}
			}

			if (ClosestComponent != NULL)
			{
				int32 LightmapSizeX = 0;
				int32 LightmapSizeY = 0;
				ClosestComponent->GetLightMapResolution(LightmapSizeX, LightmapSizeY);

				FModelVertex* ModelVertices = (FModelVertex*)Model->VertexBuffer.Vertices.GetData();

				FModelElement& Element = ClosestComponent->GetElements()[ClosestElementIndex];
				TUniquePtr<FRawIndexBuffer16or32>* IndexBufferRef = Model->MaterialIndexBuffers.Find(Element.Material);

				uint32 Index0 = (*IndexBufferRef)->Indices[ClosestTriangleIndex];
				uint32 Index1 = (*IndexBufferRef)->Indices[ClosestTriangleIndex + 1];
				uint32 Index2 = (*IndexBufferRef)->Indices[ClosestTriangleIndex + 2];

				// Fetch lightmap UV's
				FVector2D LightmapUV0 = FVector2D(ModelVertices[Index0].ShadowTexCoord);
				FVector2D LightmapUV1 = FVector2D(ModelVertices[Index1].ShadowTexCoord);
				FVector2D LightmapUV2 = FVector2D(ModelVertices[Index2].ShadowTexCoord);
				// Interpolate lightmap UV's to the click location
				FVector2D InterpolatedUV = LightmapUV0 * ClosestPlaneBaryCentricWeights.X + LightmapUV1 * ClosestPlaneBaryCentricWeights.Y + LightmapUV2 * ClosestPlaneBaryCentricWeights.Z;

				// Find the node index belonging to the selected triangle
				const UModel* CurrentModel = ClosestComponent->GetModel();
				int32 SelectedNodeIndex = INDEX_NONE;
				for (int32 ElementNodeIndex = 0; ElementNodeIndex < Element.Nodes.Num(); ElementNodeIndex++)
				{
					const FBspNode& CurrentNode = CurrentModel->Nodes[Element.Nodes[ElementNodeIndex]];
					if ((int32)Index0 >= CurrentNode.iVertexIndex && (int32)Index0 < CurrentNode.iVertexIndex + CurrentNode.NumVertices)
					{
						SelectedNodeIndex = Element.Nodes[ElementNodeIndex];
					}
				}
				check(SelectedNodeIndex >= 0);

				TArray<ULightComponentBase*> DummyLights;

				// fill out the model's NodeGroups (not the mapping part of it, but the nodes part)
				Model->GroupAllNodes(World->GetCurrentLevel(), DummyLights);

				// Find the FGatheredSurface that the selected node got put into during the last lighting rebuild
				TArray<int32> GatheredNodes;

				// find the NodeGroup that this node went into, and get all of its node
				for (TMap<int32, FNodeGroup*>::TIterator It(Model->NodeGroups); It && GatheredNodes.Num() == 0; ++It)
				{
					FNodeGroup* NodeGroup = It.Value();
					for (int32 NodeIndex = 0; NodeIndex < NodeGroup->Nodes.Num(); NodeIndex++)
					{
						if (NodeGroup->Nodes[NodeIndex] == SelectedNodeIndex)
						{
							GatheredNodes = NodeGroup->Nodes;
							break;
						}
					}
				}
				check(GatheredNodes.Num() > 0);

				// use the surface of the selected node, it will have to suffice for the GetSurfaceLightMapResolution() call
				int32 SelectedGatheredSurfIndex = Model->Nodes[SelectedNodeIndex].iSurf;

				// Get the lightmap resolution used by the FGatheredSurface containing the selected node
				FMatrix WorldToMap;
				ClosestComponent->GetSurfaceLightMapResolution(SelectedGatheredSurfIndex, 1, LightmapSizeX, LightmapSizeY, WorldToMap, &GatheredNodes);

				int32 PaddedSizeX = LightmapSizeX;
				int32 PaddedSizeY = LightmapSizeY;
				if (GLightmassDebugOptions.bPadMappings && GAllowLightmapPadding && LightmapSizeX - 2 > 0 && LightmapSizeY - 2 > 0)
				{
					PaddedSizeX -= 2;
					PaddedSizeY -= 2;
				}
				check(LightmapSizeX > 0 && LightmapSizeY > 0);

				// Apply the transform to the intersection position to find the local texel coordinates
				const FVector4 StaticLightingTextureCoordinate = WorldToMap.TransformPosition(ClickLocation);
				const int32 LocalX = FMath::TruncToInt32(StaticLightingTextureCoordinate.X * PaddedSizeX);
				const int32 LocalY = FMath::TruncToInt32(StaticLightingTextureCoordinate.Y * PaddedSizeY);
				check(LocalX >= 0 && LocalX < PaddedSizeX && LocalY >= 0 && LocalY < PaddedSizeY);

				const FMeshMapBuildData* MeshMapBuildData = Element.GetMeshMapBuildData();

				if (MeshMapBuildData)
				{
					bFoundLightmapSample = UpdateSelectedTexel(
						ClosestComponent,
						SelectedNodeIndex,
						MeshMapBuildData->LightMap,
						ClickLocation,
						InterpolatedUV,
						LocalX, LocalY,
						LightmapSizeX, LightmapSizeY);
				}

				if (!bFoundLightmapSample)
				{
					GCurrentSelectedLightmapSample = FSelectedLightmapSample();
				}
			}
		}

		if (!bFoundLightmapSample)
		{
			GCurrentSelectedLightmapSample = FSelectedLightmapSample();
		}
	}
}

/** Renders debug elements for visualizing static lighting info */
void DrawStaticLightingDebugInfo(const FSceneView* View,FPrimitiveDrawInterface* PDI)
{
	if (IsTexelDebuggingEnabled() && GDebugStaticLightingInfo.bValid)
	{
		for (int32 VertexIndex = 0; VertexIndex < GDebugStaticLightingInfo.Vertices.Num(); VertexIndex++)
		{
			const FDebugStaticLightingVertex& CurrentVertex = GDebugStaticLightingInfo.Vertices[VertexIndex];
			FColor NormalColor(250,250,50);
			if (GDebugStaticLightingInfo.SelectedVertexIndices.Contains(VertexIndex))
			{
				NormalColor = FColor(150, 250, 250);
				for (int32 CornerIndex = 0; CornerIndex < NumTexelCorners; CornerIndex++)
				{
					if (GDebugStaticLightingInfo.bCornerValid[CornerIndex])
					{
						PDI->DrawPoint(FVector4(GDebugStaticLightingInfo.TexelCorners[CornerIndex] + CurrentVertex.VertexNormal * .04f), FLinearColor(0, 1, 1), 4.0f, SDPG_World);
					}
				}
				PDI->DrawPoint(FVector4(CurrentVertex.VertexPosition), NormalColor, 4.0f, SDPG_World);
				DrawWireSphere(PDI, FVector4(CurrentVertex.VertexPosition), NormalColor, GDebugStaticLightingInfo.SampleRadius, 36, SDPG_World);
			}
			PDI->DrawLine(FVector4(CurrentVertex.VertexPosition), FVector4(CurrentVertex.VertexPosition + CurrentVertex.VertexNormal * 10), NormalColor, SDPG_World);
		}

		for (int32 RayIndex = 0; RayIndex < GDebugStaticLightingInfo.ShadowRays.Num(); RayIndex++)
		{
			const FDebugStaticLightingRay& CurrentRay = GDebugStaticLightingInfo.ShadowRays[RayIndex];
			PDI->DrawLine(FVector4(CurrentRay.Start), FVector4(CurrentRay.End), CurrentRay.bHit ? FColor::Red : FColor::Green, SDPG_World);
		}
		
		for (int32 RayIndex = 0; RayIndex < GDebugStaticLightingInfo.PathRays.Num(); RayIndex++)
		{
			const FDebugStaticLightingRay& CurrentRay = GDebugStaticLightingInfo.PathRays[RayIndex];
			const FColor RayColor = CurrentRay.bHit ? (CurrentRay.bPositive ? FColor(255,255,150) : FColor(150,150,150)) : FColor(50,50,255);
			PDI->DrawLine(FVector4(CurrentRay.Start), FVector4(CurrentRay.End), RayColor, SDPG_World);
		}

		for (int32 RecordIndex = 0; RecordIndex < GDebugStaticLightingInfo.CacheRecords.Num(); RecordIndex++)
		{
			const FDebugLightingCacheRecord& CurrentRecord = GDebugStaticLightingInfo.CacheRecords[RecordIndex];
			if (CurrentRecord.bNearSelectedTexel)
			{
				DrawWireSphere(PDI, FVector4(CurrentRecord.Vertex.VertexPosition + CurrentRecord.Vertex.VertexNormal * .1f), CurrentRecord.bAffectsSelectedTexel ? FColor(50, 255, 100) : FColor(100, 100, 100), CurrentRecord.Radius, 36, SDPG_World);
				PDI->DrawLine(FVector4(CurrentRecord.Vertex.VertexPosition), FVector4(CurrentRecord.Vertex.VertexPosition + CurrentRecord.Vertex.VertexNormal * 12), CurrentRecord.bAffectsSelectedTexel ? FColor(50, 255, 100) : FColor(100, 100, 100), SDPG_World);
			}
			PDI->DrawPoint(FVector4(CurrentRecord.Vertex.VertexPosition + CurrentRecord.Vertex.VertexNormal * .1f), FLinearColor(.5, 1, .5), 2.0f, SDPG_World);
		}

		for (int32 PhotonIndex = 0; PhotonIndex < GDebugStaticLightingInfo.DirectPhotons.Num(); PhotonIndex++)
		{
			const FDebugPhoton& CurrentPhoton = GDebugStaticLightingInfo.DirectPhotons[PhotonIndex];
			PDI->DrawLine(FVector4(CurrentPhoton.Position), FVector4(CurrentPhoton.Position + CurrentPhoton.Direction * 50), FColor(200, 200, 100), SDPG_World);
		}

		for (int32 PhotonIndex = 0; PhotonIndex < GDebugStaticLightingInfo.IndirectPhotons.Num(); PhotonIndex++)
		{
			const FDebugPhoton& CurrentPhoton = GDebugStaticLightingInfo.IndirectPhotons[PhotonIndex];
			PDI->DrawLine(FVector4(CurrentPhoton.Position), FVector4(CurrentPhoton.Position + CurrentPhoton.Direction), FColor(200, 100, 100), SDPG_World);
		}

		for (int32 PhotonIndex = 0; PhotonIndex < GDebugStaticLightingInfo.IrradiancePhotons.Num(); PhotonIndex++)
		{
			const FDebugPhoton& CurrentPhoton = GDebugStaticLightingInfo.IrradiancePhotons[PhotonIndex];
			PDI->DrawLine(FVector4(CurrentPhoton.Position), FVector4(CurrentPhoton.Position + CurrentPhoton.Direction * 50), FColor(150, 100, 250), SDPG_World);
		}

		for (int32 PhotonIndex = 0; PhotonIndex < GDebugStaticLightingInfo.GatheredPhotons.Num(); PhotonIndex++)
		{
			const FDebugPhoton& CurrentPhoton = GDebugStaticLightingInfo.GatheredPhotons[PhotonIndex];
			PDI->DrawLine(FVector4(CurrentPhoton.Position), FVector4(CurrentPhoton.Position + CurrentPhoton.Normal * 50), FColor(100, 100, 100), SDPG_World);
			PDI->DrawLine(FVector4(CurrentPhoton.Position), FVector4(CurrentPhoton.Position + CurrentPhoton.Direction * 50), FColor(50, 255, 100), SDPG_World);
			PDI->DrawPoint(FVector4(CurrentPhoton.Position + CurrentPhoton.Direction * .1f), FLinearColor(.5, 1, .5), 4.0f, SDPG_World);
		}

		for (int32 PhotonIndex = 0; PhotonIndex < GDebugStaticLightingInfo.GatheredImportancePhotons.Num(); PhotonIndex++)
		{
			const FDebugPhoton& CurrentPhoton = GDebugStaticLightingInfo.GatheredImportancePhotons[PhotonIndex];
			PDI->DrawLine(FVector4(CurrentPhoton.Position), FVector4(CurrentPhoton.Position + CurrentPhoton.Normal * 50), FColor(100, 100, 100), SDPG_World);
			PDI->DrawLine(FVector4(CurrentPhoton.Position), FVector4(CurrentPhoton.Position + CurrentPhoton.Direction * 50), FColor(200, 100, 100), SDPG_World);
			PDI->DrawPoint(FVector4(CurrentPhoton.Position + CurrentPhoton.Direction * .1f), FLinearColor(.5, 1, .5), 4.0f, SDPG_World);
		}
		const FColor NodeColor(150, 170, 180);
		for (int32 NodeIndex = 0; NodeIndex < GDebugStaticLightingInfo.GatheredPhotonNodes.Num(); NodeIndex++)
		{
			const FDebugOctreeNode& CurrentNode = GDebugStaticLightingInfo.GatheredPhotonNodes[NodeIndex];
			PDI->DrawLine(FVector4(CurrentNode.Center) + FVector(CurrentNode.Extent.X, CurrentNode.Extent.Y, CurrentNode.Extent.Z), FVector4(CurrentNode.Center) + FVector(-CurrentNode.Extent.X, CurrentNode.Extent.Y, CurrentNode.Extent.Z), NodeColor, SDPG_World);
			PDI->DrawLine(FVector4(CurrentNode.Center) + FVector(CurrentNode.Extent.X, CurrentNode.Extent.Y, CurrentNode.Extent.Z), FVector4(CurrentNode.Center) + FVector(CurrentNode.Extent.X, -CurrentNode.Extent.Y, CurrentNode.Extent.Z), NodeColor, SDPG_World);
			PDI->DrawLine(FVector4(CurrentNode.Center) + FVector(CurrentNode.Extent.X, CurrentNode.Extent.Y, CurrentNode.Extent.Z), FVector4(CurrentNode.Center) + FVector(CurrentNode.Extent.X, CurrentNode.Extent.Y, -CurrentNode.Extent.Z), NodeColor, SDPG_World);
			
			PDI->DrawLine(FVector4(CurrentNode.Center) - FVector(CurrentNode.Extent.X, CurrentNode.Extent.Y, CurrentNode.Extent.Z), FVector4(CurrentNode.Center) - FVector(-CurrentNode.Extent.X, CurrentNode.Extent.Y, CurrentNode.Extent.Z), NodeColor, SDPG_World);
			PDI->DrawLine(FVector4(CurrentNode.Center) - FVector(CurrentNode.Extent.X, CurrentNode.Extent.Y, CurrentNode.Extent.Z), FVector4(CurrentNode.Center) - FVector(CurrentNode.Extent.X, -CurrentNode.Extent.Y, CurrentNode.Extent.Z), NodeColor, SDPG_World);
			PDI->DrawLine(FVector4(CurrentNode.Center) - FVector(CurrentNode.Extent.X, CurrentNode.Extent.Y, CurrentNode.Extent.Z), FVector4(CurrentNode.Center) - FVector(CurrentNode.Extent.X, CurrentNode.Extent.Y, -CurrentNode.Extent.Z), NodeColor, SDPG_World);
			
			PDI->DrawLine(FVector4(CurrentNode.Center) + FVector(CurrentNode.Extent.X, -CurrentNode.Extent.Y, CurrentNode.Extent.Z), FVector4(CurrentNode.Center) + FVector(CurrentNode.Extent.X, -CurrentNode.Extent.Y, -CurrentNode.Extent.Z), NodeColor, SDPG_World);
			PDI->DrawLine(FVector4(CurrentNode.Center) + FVector(CurrentNode.Extent.X, -CurrentNode.Extent.Y, CurrentNode.Extent.Z), FVector4(CurrentNode.Center) + FVector(-CurrentNode.Extent.X, -CurrentNode.Extent.Y, CurrentNode.Extent.Z), NodeColor, SDPG_World);

			PDI->DrawLine(FVector4(CurrentNode.Center) + FVector(-CurrentNode.Extent.X, CurrentNode.Extent.Y, CurrentNode.Extent.Z), FVector4(CurrentNode.Center) + FVector(-CurrentNode.Extent.X, -CurrentNode.Extent.Y, CurrentNode.Extent.Z), NodeColor, SDPG_World);
			PDI->DrawLine(FVector4(CurrentNode.Center) + FVector(-CurrentNode.Extent.X, CurrentNode.Extent.Y, CurrentNode.Extent.Z), FVector4(CurrentNode.Center) + FVector(-CurrentNode.Extent.X, CurrentNode.Extent.Y, -CurrentNode.Extent.Z), NodeColor, SDPG_World);

			PDI->DrawLine(FVector4(CurrentNode.Center) + FVector(CurrentNode.Extent.X, CurrentNode.Extent.Y, -CurrentNode.Extent.Z), FVector4(CurrentNode.Center) + FVector(CurrentNode.Extent.X, -CurrentNode.Extent.Y, -CurrentNode.Extent.Z), NodeColor, SDPG_World);
			PDI->DrawLine(FVector4(CurrentNode.Center) + FVector(CurrentNode.Extent.X, CurrentNode.Extent.Y, -CurrentNode.Extent.Z), FVector4(CurrentNode.Center) + FVector(-CurrentNode.Extent.X, CurrentNode.Extent.Y, -CurrentNode.Extent.Z), NodeColor, SDPG_World);
		}

		if (GDebugStaticLightingInfo.bDirectPhotonValid)
		{
			const FDebugPhoton& DirectPhoton = GDebugStaticLightingInfo.GatheredDirectPhoton;
			PDI->DrawLine(FVector4(DirectPhoton.Position), FVector4(DirectPhoton.Position + DirectPhoton.Direction * 60), FColor(255, 255, 100), SDPG_World);
			PDI->DrawPoint(FVector4(DirectPhoton.Position + DirectPhoton.Direction * .1f), FLinearColor(1, 1, .5), 4.0f, SDPG_World);
		}

		for (int32 RayIndex = 0; RayIndex < GDebugStaticLightingInfo.IndirectPhotonPaths.Num(); RayIndex++)
		{
			const FDebugStaticLightingRay& CurrentRay = GDebugStaticLightingInfo.IndirectPhotonPaths[RayIndex];
			PDI->DrawLine(FVector4(CurrentRay.Start), FVector4(CurrentRay.End), FColor::White, SDPG_World);
		}

		for (int32 SampleIndex = 0; SampleIndex < GDebugStaticLightingInfo.VolumeLightingSamples.Num(); SampleIndex++)
		{
			const FDebugVolumeLightingSample& CurrentSample = GDebugStaticLightingInfo.VolumeLightingSamples[SampleIndex];
			PDI->DrawPoint(FVector4(CurrentSample.Position), CurrentSample.AverageIncidentRadiance * GEngine->LightingOnlyBrightness, 12.0f, SDPG_World);
		}

		for (int32 RayIndex = 0; RayIndex < GDebugStaticLightingInfo.PrecomputedVisibilityRays.Num(); RayIndex++)
		{
			const FDebugStaticLightingRay& CurrentRay = GDebugStaticLightingInfo.PrecomputedVisibilityRays[RayIndex];
			const FColor RayColor = CurrentRay.bHit ? (CurrentRay.bPositive ? FColor(255,255,150) : FColor(150,150,150)) : FColor(50,50,255);
			PDI->DrawLine(FVector4(CurrentRay.Start), FVector4(CurrentRay.End), RayColor, SDPG_World);
		}
	}
}

/** Renders debug elements for visualizing static lighting info */
void DrawStaticLightingDebugInfo(const FSceneView* View, FCanvas* Canvas)
{
	if (IsTexelDebuggingEnabled() && GDebugStaticLightingInfo.bValid)
	{
		for (int32 RecordIndex = 0; RecordIndex < GDebugStaticLightingInfo.CacheRecords.Num(); RecordIndex++)
		{
			const FDebugLightingCacheRecord& CurrentRecord = GDebugStaticLightingInfo.CacheRecords[RecordIndex];
			if (CurrentRecord.bNearSelectedTexel)
			{
				FVector2D PixelLocation;
				if(View->ScreenToPixel(View->WorldToScreen(FVector4(CurrentRecord.Vertex.VertexPosition)),PixelLocation))
				{
					const FColor TagColor = CurrentRecord.bAffectsSelectedTexel ? FColor(50,160,200) : FColor(120,120,120);
					Canvas->DrawShadowedString(static_cast<float>(PixelLocation.X), static_cast<float>(PixelLocation.Y), *FString::FromInt(CurrentRecord.RecordId), GEngine->GetSmallFont(), TagColor);
				}
			}
		}

		for (int32 PhotonIndex = 0; PhotonIndex < GDebugStaticLightingInfo.GatheredImportancePhotons.Num(); PhotonIndex++)
		{
			const FDebugPhoton& CurrentPhoton = GDebugStaticLightingInfo.GatheredImportancePhotons[PhotonIndex];
			FVector2D PixelLocation;
			if(View->ScreenToPixel(View->WorldToScreen(FVector4(CurrentPhoton.Position)),PixelLocation))
			{
				const FColor TagColor = FColor(120,120,120);
				Canvas->DrawShadowedString(static_cast<float>(PixelLocation.X), static_cast<float>(PixelLocation.Y), *FString::FromInt(CurrentPhoton.Id), GEngine->GetSmallFont(), TagColor);
			}
		}

		for (int32 RayIndex = 0; RayIndex < GDebugStaticLightingInfo.PathRays.Num(); RayIndex++)
		{
			const FDebugStaticLightingRay& CurrentRay = GDebugStaticLightingInfo.PathRays[RayIndex];
			if (CurrentRay.bHit && CurrentRay.bPositive)
			{
				FVector2D PixelLocation;
				if(View->ScreenToPixel(View->WorldToScreen(FVector4(CurrentRay.End)),PixelLocation))
				{
					const FColor TagColor = FColor(180,180,120);
					Canvas->DrawShadowedString(static_cast<float>(PixelLocation.X), static_cast<float>(PixelLocation.Y), *FString::FromInt(RayIndex), GEngine->GetSmallFont(), TagColor);
				}
			}
		}
	}
}

#endif	//#if WITH_EDITOR
