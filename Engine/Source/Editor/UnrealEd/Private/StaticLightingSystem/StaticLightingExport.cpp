// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	StaticLightingExport.cpp: Static lighting export implementations.
=============================================================================*/

#include "CoreMinimal.h"
#include "Lightmass/LightmappedSurfaceCollection.h"

#include "StaticMeshLight.h"
#include "Components/LightComponent.h"
#include "ModelLight.h"
#include "StaticMeshResources.h"
#include "Engine/StaticMesh.h"

#include "LandscapeLight.h"
#include "LandscapeComponent.h"

#include "Lightmass/Lightmass.h"

#include "Materials/MaterialInstanceConstant.h"


// Doxygen cannot parse these definitions correctly since the declaration is in a header from a different module
#if !UE_BUILD_DOCS
/** 
* Export static lighting mapping instance data to an exporter 
* @param Exporter - export interface to process static lighting data
**/
void FBSPSurfaceStaticLighting::ExportMapping(class FLightmassExporter* Exporter)
{
	if (!Model.IsValid()) {return;}

	Exporter->BSPSurfaceMappings.AddUnique(this);

	// remember all the models used by all the BSP mappings
	Exporter->Models.AddUnique(Model.Get());

	// get all the materials used in the node group
	for (int32 NodeIndex = 0; NodeIndex < NodeGroup->Nodes.Num(); NodeIndex++)
	{
		FBspSurf& Surf = Model->Surfs[Model->Nodes[NodeGroup->Nodes[NodeIndex]].iSurf];

		UMaterialInterface* Material = Surf.Material;
		if (Material)
		{
			Exporter->AddMaterial(Material);
		}
	}

	for( int32 LightIdx=0; LightIdx < NodeGroup->RelevantLights.Num(); LightIdx++ )
	{
		ULightComponent* Light = NodeGroup->RelevantLights[LightIdx];
		if( Light )
		{
			Exporter->AddLight(Light);
		}
	}
}

/**
 *	@return	UOject*		The object that is mapped by this mapping
 */
UObject* FBSPSurfaceStaticLighting::GetMappedObject() const
{
	//@todo. THIS WILL SCREW UP IF CALLED MORE THAN ONE TIME!!!!
	// Create a collection object to allow selection of the surfaces in this mapping
	auto MappedObject = NewObject<ULightmappedSurfaceCollection>();
	// Set the owner model
	MappedObject->SourceModel = Model.Get();
	// Fill in the surface index array
	for (int32 NodeIndex = 0; NodeIndex < NodeGroup->Nodes.Num(); NodeIndex++)
	{
		MappedObject->Surfaces.Add(Model->Nodes[NodeGroup->Nodes[NodeIndex]].iSurf);
	}
	return MappedObject;
}

/** 
* Export static lighting mesh instance data to an exporter 
* @param Exporter - export interface to process static lighting data
**/
void FStaticMeshStaticLightingMesh::ExportMeshInstance(class FLightmassExporter* Exporter) const
{
	Exporter->StaticMeshLightingMeshes.AddUnique(this);	

	for( int32 LightIdx=0; LightIdx < RelevantLights.Num(); LightIdx++ )
	{
		ULightComponent* Light = RelevantLights[LightIdx];
		if( Light )
		{
			Exporter->AddLight(Light);
		}
	}

	// Add the UStaticMesh and materials to the exporter...
	if( StaticMesh && StaticMesh->GetRenderData())
	{
		Exporter->StaticMeshes.AddUnique(StaticMesh);
		if( Primitive )
		{	
			for( int32 ResourceIndex = 0; ResourceIndex < StaticMesh->GetRenderData()->LODResources.Num(); ++ResourceIndex )
			{
				const FStaticMeshLODResources& LODResourceData = StaticMesh->GetRenderData()->LODResources[ResourceIndex];
				for( int32 SectionIndex = 0; SectionIndex < LODResourceData.Sections.Num(); ++SectionIndex )
				{
					const FStaticMeshSection& Section = LODResourceData.Sections[SectionIndex];
					UMaterialInterface* Material = Primitive->GetMaterial(Section.MaterialIndex);
					Exporter->AddMaterial(Material);
				}
			}
		}
	}
}

/** 
* Export static lighting mapping instance data to an exporter 
* @param Exporter - export interface to process static lighting data
**/
void FStaticMeshStaticLightingTextureMapping::ExportMapping(class FLightmassExporter* Exporter)
{
	Exporter->StaticMeshTextureMappings.AddUnique(this);
}

void FStaticLightingGlobalVolumeMapping::ExportMapping(class FLightmassExporter* Exporter)
{
	Exporter->VolumeMappings.AddUnique(this);
}

//
//	Landscape
//
/** 
* Export static lighting mesh instance data to an exporter 
* @param Exporter - export interface to process static lighting data
**/
void FLandscapeStaticLightingMesh::ExportMeshInstance(class FLightmassExporter* Exporter) const
{
	Exporter->LandscapeLightingMeshes.AddUnique(this);

	if (LandscapeComponent)
	{
		UMaterialInstance* MaterialInstance = LandscapeComponent->GetMaterialInstance(0, false);

		if (MaterialInstance)
		{
			Exporter->AddMaterial(MaterialInstance, this);
		}
	}

	for( int32 LightIdx=0; LightIdx < RelevantLights.Num(); LightIdx++ )
	{
		ULightComponent* Light = RelevantLights[LightIdx];
		if( Light )
		{
			Exporter->AddLight(Light);
		}
	}
}

/** 
* Export static lighting mapping instance data to an exporter 
* @param Exporter - export interface to process static lighting data
**/
void FLandscapeStaticLightingTextureMapping::ExportMapping(class FLightmassExporter* Exporter)
{
	Exporter->LandscapeTextureMappings.AddUnique(this);
}

/** 
* Export static lighting mapping instance data to an exporter 
* @param Exporter - export interface to process static lighting data
**/
void FLandscapeStaticLightingGlobalVolumeMapping::ExportMapping(class FLightmassExporter* Exporter)
{
	Exporter->LandscapeVolumeMappings.AddUnique(this);
}


#endif
