// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	StaticLightingPrivate.h: Private static lighting system definitions.
=============================================================================*/

#pragma once

// Includes.

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Engine/Level.h"
#include "StaticLighting.h"
#include "LightingBuildOptions.h"
#include "Templates/UniquePtr.h"
#include "Engine/MapBuildDataRegistry.h"
#include "StaticLightingBuildContext.h"

class FCanvas;
class FLightmassProcessor;
class FPrimitiveDrawInterface;
class FSceneView;
class FShadowMapData2D;
class UActorComponent;
class ULightComponent;
class ULightComponentBase;
class UModel;
struct FNodeGroup;
struct FQuantizedLightmapData;
struct FSelectedLightmapSample;

DECLARE_LOG_CATEGORY_EXTERN(LogStaticLightingSystem, Log, All);

/** Encapsulation of all Lightmass statistics */
struct FLightmassStatistics
{
	struct FScopedGather
	{
		FScopedGather(double& Statistic)
			: StatReference(Statistic)
			, StatStartTime(0.0)
		{
			StatStartTime = FPlatformTime::Seconds();
		}

		~FScopedGather()
		{
			StatReference += FPlatformTime::Seconds() - StatStartTime;
		}

		double& StatReference;
		double StatStartTime;
	};

	/** Constructor that clears all statistics */
	FLightmassStatistics()
	{
		ClearAll();
	}
	/** Clears all statistics */
	void ClearAll()
	{
		StartupTime						= 0.0;
		CollectTime						= 0.0;
		PrepareLightsTime				= 0.0;
		GatherLightingInfoTime			= 0.0;
		ProcessingTime					= 0.0;
		CollectLightmassSceneTime		= 0.0;
		ExportTime						= 0.0;
		LightmassTime					= 0.0;
		SwarmStartupTime				= 0.0;
		SwarmCallbackTime				= 0.0;
		SwarmJobOpenTime				= 0.0;
		SwarmJobCloseTime				= 0.0;
		ImportTime						= 0.0;
		ImportTimeInProcessing			= 0.0;
		InvalidationTime				= 0.0;
		ApplyTime						= 0.0;
		ApplyTimeInProcessing			= 0.0;
		EncodingTime					= 0.0;
		EncodingLightmapsTime			= 0.0;
		EncodingShadowMapsTime			= 0.0;
		FinishingTime					= 0.0;
		TotalTime						= 0.0;
		ExportVisibilityDataTime = 0.0;
		ExportVolumetricLightmapDataTime = 0.0f;
		ExportLightsTime = 0.0;
		ExportModelsTime = 0.0;
		ExportStaticMeshesTime = 0.0;
		ExportMaterialsTime = 0.0;
		ExportMeshInstancesTime = 0.0;
		ExportLandscapeInstancesTime = 0.0;
		ExportMappingsTime = 0.0;
		Scratch0						= 0.0;
		Scratch1						= 0.0;
		Scratch2						= 0.0;
		Scratch3						= 0.0;
	}
	/** Adds timing measurements from another FLightmassStatistics. */
	FLightmassStatistics& operator+=( const FLightmassStatistics& Other )
	{
		StartupTime						+= Other.StartupTime;
		CollectTime						+= Other.CollectTime;
		PrepareLightsTime				+= Other.PrepareLightsTime;
		GatherLightingInfoTime			+= Other.GatherLightingInfoTime;
		ProcessingTime					+= Other.ProcessingTime;
		CollectLightmassSceneTime		+= Other.CollectLightmassSceneTime;
		ExportTime						+= Other.ExportTime;
		LightmassTime					+= Other.LightmassTime;
		SwarmStartupTime				+= Other.SwarmStartupTime;
		SwarmCallbackTime				+= Other.SwarmCallbackTime;
		SwarmJobOpenTime				+= Other.SwarmJobOpenTime;
		SwarmJobCloseTime				+= Other.SwarmJobCloseTime;
		ImportTime						+= Other.ImportTime;
		ImportTimeInProcessing			+= Other.ImportTimeInProcessing;
		InvalidationTime				+= Other.InvalidationTime;
		ApplyTime						+= Other.ApplyTime;
		ApplyTimeInProcessing			+= Other.ApplyTimeInProcessing;
		EncodingTime					+= Other.EncodingTime;
		EncodingLightmapsTime			+= Other.EncodingLightmapsTime;
		EncodingShadowMapsTime			+= Other.EncodingShadowMapsTime;
		FinishingTime					+= Other.FinishingTime;
		TotalTime						+= Other.TotalTime;
		ExportVisibilityDataTime += Other.ExportVisibilityDataTime;
		ExportVolumetricLightmapDataTime += Other.ExportVolumetricLightmapDataTime;
		ExportLightsTime += Other.ExportLightsTime;
		ExportModelsTime += Other.ExportModelsTime;
		ExportStaticMeshesTime += Other.ExportStaticMeshesTime;
		ExportMaterialsTime += Other.ExportMaterialsTime;
		ExportMeshInstancesTime += Other.ExportMeshInstancesTime;
		ExportLandscapeInstancesTime += Other.ExportLandscapeInstancesTime;
		ExportMappingsTime += Other.ExportMappingsTime;
		Scratch0						+= Other.Scratch0;
		Scratch1						+= Other.Scratch1;
		Scratch2						+= Other.Scratch2;
		Scratch3						+= Other.Scratch3;
		return *this;
	}

	/** Time spent starting up, in seconds. */
	double	StartupTime;
	/** Time spent preparing and collecting the scene, in seconds. */
	double	CollectTime;
	double	PrepareLightsTime;
	double	GatherLightingInfoTime;
	/** Time spent in the actual lighting path, in seconds */
	double	ProcessingTime;
	/** Time spent collecting the scene and assets for Lightmass, in seconds. */
	double	CollectLightmassSceneTime;
	/** Time spent exporting, in seconds. */
	double	ExportTime;
	/** Time spent running Lightmass. */
	double	LightmassTime;
	/** Time spent in various Swarm APIs, in seconds. */
	double	SwarmStartupTime;
	double	SwarmCallbackTime;
	double	SwarmJobOpenTime;
	double	SwarmJobCloseTime;
	/** Time spent importing and applying results, in seconds. */
	double	ImportTime;
	double	ImportTimeInProcessing;
	/** Time spent invalidating lightmass data */
	double InvalidationTime;
	/** Time spent just applying results, in seconds. */
	double	ApplyTime;
	double	ApplyTimeInProcessing;
	/** Time spent encoding textures, in seconds. */
	double	EncodingTime;
	double	EncodingLightmapsTime;
	double	EncodingShadowMapsTime;
	/** Time spent finishing up, in seconds. */
	double	FinishingTime;
	/** Total time spent for the lighting build. */
	double	TotalTime;
	/** Time spent in various export stages */
	double ExportVisibilityDataTime;
	double ExportVolumetricLightmapDataTime;
	double ExportLightsTime;
	double ExportModelsTime;
	double ExportStaticMeshesTime;
	double ExportMaterialsTime;
	double ExportMeshInstancesTime;
	double ExportLandscapeInstancesTime;
	double ExportMappingsTime;

	/** Reusable temporary statistics */
	double	Scratch0;
	double	Scratch1;
	double	Scratch2;
	double	Scratch3;
};

/** StaticLighting sorting helper */
struct FStaticLightingMappingSortHelper
{
	int32 NumTexels;
	TRefCountPtr<FStaticLightingMapping> Mapping;
};

/** Always active singleton class which manages all static light systems and subsystems */
class FStaticLightingManager : public TSharedFromThis<FStaticLightingManager>
{
public:
	static TSharedPtr<FStaticLightingManager> Get();

	/** Processes lighting data that is now pending from a finished lightmass pass */
	static void ProcessLightingData();
	/** Stops lightmass from working, and discards the data */
	static void CancelLightingBuild();
	
	/** Puts out a notification for the progress of lightmass */
	void SendProgressNotification();
	/** Removes any lightmass notifications in flight */
	void ClearCurrentNotification();
	/** Puts out a notification for when the build finishes */
	void SendBuildDoneNotification( bool AutoApplyFailed );

	/** Updates current notification with new text */
	void SetNotificationText( FText Text );
	
	static void ImportRequested();
	static void DiscardRequested();

	/** Initializes the static lighting system to defaults and kicks it off if possible */
	void CreateStaticLightingSystem(const FLightingBuildOptions& Options);
	/** Updates the build lighting with info from Lightmass, checking for completion */
	void UpdateBuildLighting();
	/** Discard results, and sends a notification telling the user the build failed */
	void FailLightingBuild( FText ErrorText = FText());

	/** True if a lighting build is currently in the works, or is finished but not accepted yet */
	bool IsLightingBuildCurrentlyRunning() const;

	bool IsLightingBuildCurrentlyExporting() const;

private:
	FStaticLightingManager() 
		: ActiveStaticLightingSystem(nullptr) 
		, bBuildReflectionCapturesOnFinish(true)
	{}
	
	class FStaticLightingSystem* ActiveStaticLightingSystem;

	/** The system for kicking off the asynchronous lightmass */
	TArray<TUniquePtr<class FStaticLightingSystem>> StaticLightingSystems;

	/** Notification we hold on to that indicates progress. */
	TWeakPtr<SNotificationItem> LightBuildNotification;

	/** Singleton of static lighting manager */
	static TSharedPtr<FStaticLightingManager> StaticLightingManager;

	bool bBuildReflectionCapturesOnFinish;

	void FinishLightingBuild();

	/** Destroys the static lighting system if it exists */
	void DestroyStaticLightingSystems();
};

/** The state of the static lighting system. */
class FStaticLightingSystem
{
public:

	/**
	 * Initializes this static lighting system, and builds static lighting based on the provided options.
	 * @param InOptions - The static lighting build options.
	 * @param InContext - The context (world, lighting scenario, world subsection, data layers)  we wish to build the lighting for
	 */
	FStaticLightingSystem(const FLightingBuildOptions& InOptions, FStaticLightingBuildContext&& InContext);
	~FStaticLightingSystem();

	bool CheckLightmassExecutableVersion();
	
	/** Kicks off the lightmass processing, and, if successful, starts the asynchronous task */
	bool BeginLightmassProcess();

	/** Updates the lightmass processor to query if the async task has completed */
	void UpdateLightingBuild();
	
	/** Starts up swarm and kicks off lightmass exes */
	void KickoffSwarm();

	/** To be called when the lightmass process is complete */
	bool FinishLightmassProcess();

	/** Marks all lights used in the calculated lightmap as used in a lightmap, and calls Apply on the texture mapping. */
	void ApplyMapping(
		FStaticLightingTextureMapping* TextureMapping,
		FQuantizedLightmapData* QuantizedData,
		const TMap<ULightComponent*,FShadowMapData2D*>& ShadowMapData) const;

	/** Get the UWorld this light system was created with */
	UWorld*	 GetWorld() const;

	/** Get the Lighting Context this light system was created with */
	const FStaticLightingBuildContext& GetLightingContext() const;

	/** True if the current stage of building is asynchronous (lightmass in flight) */
	bool IsAsyncBuilding() const;

	bool IsAmortizedExporting() const;

	bool ShouldOperateOnLevel(ULevel* InLevel) const
	{
		return InLevel && LightingContext.ShouldIncludeLevel(InLevel) && InLevel->bIsVisible;
	}

private:
	/**
	 * Generates mappings/meshes for all BSP in the given level
	 *
	 * @param Level Level to build BSP lighting info for
	 * @param bBuildLightingForBSP If true, we need BSP mappings generated as well as the meshes
	 */
	void AddBSPStaticLightingInfo(ULevel* Level, bool bBuildLightingForBSP);

	/**
	 * Generates mappings/meshes for the given NodeGroups
	 *
	 * @param Level					Level to build BSP lighting info for
	 * @param NodeGroupsToBuild		The node groups to build the BSP lighting info for
	 */
	void AddBSPStaticLightingInfo(ULevel* Level, TArray<FNodeGroup*>& NodeGroupsToBuild);

	/** Queries a primitive for its static lighting info, and adds it to the system. */
	void AddPrimitiveStaticLightingInfo(FStaticLightingPrimitiveInfo& PrimitiveInfo, bool bBuildActorLighting, bool bDeferMapping);
	
	/** Makes the lightmass processor structure for handling import and export */
	bool CreateLightmassProcessor();

	/** Collects the scene to be sent to the exporter */
	void GatherScene();

	/** Runs initial export code of the lightmass processor */
	bool InitiateLightmassProcessor();

	/**
	 * Reports lighting build statistics to the log.
	 */
	void ReportStatistics( );

	/** Collects all static lighting info for processing */
	void GatherStaticLightingInfo(bool bRebuildDirtyGeometryForLighting, bool bForceNoPrecomputedLighting);
	
	/** After importing, textures need to be encoded to be used */
	void EncodeTextures(bool bLightingSuccessful);

	/** Pushes newly collected lightmaps on to the level */
	void ApplyNewLightingData(bool bSuccessful);
	
	void CompleteDeterministicMappings(class FLightmassProcessor* LightmassProcessor);
	
	/** Invalidates the lighting of the current levels so new lighting can be applied */
	void InvalidateStaticLighting();

	/**
	 * Don't auto-apply during interpolation editing, if there's another slow task
	 * already in progress, or while a PIE world is playing or when doing automated tests.
	 */
	bool CanAutoApplyLighting() const;

	/**
	 * Clear out all the binary dump log files, so the next run will have just the needed files
	 */
	static void ClearBinaryDumps();

	/**
	 * Conditionally adds the specified mesh bounding box to a global scene bounds used to synthesize
	 * a lightmass importance volume bounds if no volume is present in the scene.
	 */
	void UpdateAutomaticImportanceVolumeBounds( const FBox& MeshBounds );

	/** Populate BuildDataResourcesToKeep from the GUIDs referenced in the given level. */
	void GatherBuildDataResourcesToKeep(const ULevel* Level);

private:

	/** The lights in the world which the system is building. */
	TArray<ULightComponentBase*> Lights;

	/** The options the system is building lighting with. */
	const FLightingBuildOptions Options;

	/** true if the static lighting build has been canceled.  Written by the main thread, read by all static lighting threads. */
	bool bBuildCanceled;

	/** A bound of all meshes being lit - used to check the ImportanceVolume when building with Lightmass */
	FBox LightingMeshBounds;

	/** Bounding box to use for a synthesized importance volume if one is missing from the scene. */
	FBox AutomaticImportanceVolumeBounds;

	/** All meshes in the system. */

	TArray< TRefCountPtr<FStaticLightingMesh> > Meshes;

	/** All mappings in the system. */
	TArray< TRefCountPtr<FStaticLightingMapping> > Mappings;

	TArray<FStaticLightingMappingSortHelper> UnSortedMappings;

	/** Lightmass statistics */
	FLightmassStatistics LightmassStatistics;

	/** The current index for deterministic lighting */
	int32 DeterministicIndex;

	int32 NextVisibilityId;

	/** Lighting comes in various stages (amortized, async, etc.), we track them here. */
	enum class LightingStage
	{
		NotRunning,
		Startup,
		AmortizedExport,
		SwarmKickoff,
		AsynchronousBuilding,
		AutoApplyingImport,
		WaitingForImport,
		ImportRequested,
		Import,
		Finished
	};
	FStaticLightingSystem::LightingStage CurrentBuildStage;

	/** Stats we must cache off because the process is async */
	// A separate statistics structure for tracking the LightmassProcess routines times
	FLightmassStatistics LightmassProcessStatistics;
	double StartTime;
	double ProcessingStartTime;
	double WaitForUserAcceptStartTime;
	
	/** The lighting context (world, scenario, cells, datalayer) that's currently being built, if any.  */
	FStaticLightingBuildContext	LightingContext;

	/** The resource guid for all hidden/excluded levels. Used to keep those level data valid. */
	TSet<FGuid> BuildDataResourcesToKeep;

	/** A handle on the processor that actually interfaces with Lightmass */
	class FLightmassProcessor* LightmassProcessor;

	friend FStaticLightingManager;
	friend FLightmassProcessor;
};

/** 
 * Types used for debugging static lighting.  
 * NOTE: These must remain binary compatible with the ones in Lightmass.
 */

#if !PLATFORM_MAC && !PLATFORM_LINUX
	#pragma pack(push, 1)
#endif

/** Stores debug information about a static lighting ray. */
struct FDebugStaticLightingRay
{
	FVector4f Start;
	FVector4f End;
	bool bHit;
	bool bPositive;
};

struct FDebugStaticLightingVertex
{
	FDebugStaticLightingVertex() {}

	FDebugStaticLightingVertex(const FStaticLightingVertex& InVertex) :
		VertexNormal((FVector3f)InVertex.WorldTangentZ), // LWC_TODO: precision loss
		VertexPosition((FVector3f)InVertex.WorldPosition) // LWC_TODO: precision loss
	{}

	FVector4f VertexNormal;
	FVector4f VertexPosition;
};

struct FDebugLightingCacheRecord
{
	bool bNearSelectedTexel;
	bool bAffectsSelectedTexel;
	int32 RecordId;
	FDebugStaticLightingVertex Vertex;
	float Radius;
};

struct FDebugPhoton
{
	int32 Id;
	FVector4f Position;
	FVector4f Direction;
	FVector4f Normal;
};

struct FDebugOctreeNode
{
	FVector4f Center;
	FVector4f Extent;
};

struct FDebugVolumeLightingSample
{
	FVector4f Position;
	FLinearColor AverageIncidentRadiance;
};

static const int32 NumTexelCorners = 4;

/** 
 * Debug output from the static lighting build.  
 * See Lightmass::FDebugLightingOutput for documentation.
 */
struct FDebugLightingOutput
{
	bool bValid;
	TArray<FDebugStaticLightingRay> PathRays;
	TArray<FDebugStaticLightingRay> ShadowRays;
	TArray<FDebugStaticLightingRay> IndirectPhotonPaths;
	TArray<int32> SelectedVertexIndices;
	TArray<FDebugStaticLightingVertex> Vertices;
	TArray<FDebugLightingCacheRecord> CacheRecords;
	TArray<FDebugPhoton> DirectPhotons;
	TArray<FDebugPhoton> IndirectPhotons;
	TArray<FDebugPhoton> IrradiancePhotons;
	TArray<FDebugPhoton> GatheredPhotons;
	TArray<FDebugPhoton> GatheredImportancePhotons;
	TArray<FDebugOctreeNode> GatheredPhotonNodes;
	TArray<FDebugVolumeLightingSample> VolumeLightingSamples;
	TArray<FDebugStaticLightingRay> PrecomputedVisibilityRays;
	bool bDirectPhotonValid;
	FDebugPhoton GatheredDirectPhoton;
	FVector4f TexelCorners[NumTexelCorners];
	bool bCornerValid[NumTexelCorners];
	float SampleRadius;

	FDebugLightingOutput() :
		bValid(false),
		bDirectPhotonValid(false)
	{}
};

#if !PLATFORM_MAC && !PLATFORM_LINUX
	#pragma pack(pop)
#endif

/** Information about the lightmap sample that is selected */
UNREALED_API extern FSelectedLightmapSample GCurrentSelectedLightmapSample;

/** Information about the last static lighting build */
extern FDebugLightingOutput GDebugStaticLightingInfo;

/** Updates GCurrentSelectedLightmapSample given a selected actor's components and the location of the click. */
extern void SetDebugLightmapSample(TArray<UActorComponent*>* Components, UModel* Model, int32 iSurf, FVector ClickLocation);

/** Renders debug elements for visualizing static lighting info */
extern void DrawStaticLightingDebugInfo(const FSceneView* View,FPrimitiveDrawInterface* PDI);

/** Renders debug elements for visualizing static lighting info */
extern void DrawStaticLightingDebugInfo(const FSceneView* View, FCanvas* Canvas);

