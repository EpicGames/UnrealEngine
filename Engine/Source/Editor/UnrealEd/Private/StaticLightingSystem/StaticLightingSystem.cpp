// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	StaticLightingSystem.cpp: Bsp light mesh illumination builder code
=============================================================================*/

#include "CoreMinimal.h"
#include "Misc/MessageDialog.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/Guid.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigContext.h"
#include "HAL/IConsoleManager.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "UObject/ObjectMacros.h"
#include "UObject/GarbageCollection.h"
#include "Layout/Visibility.h"
#include "Framework/Application/SlateApplication.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/Actor.h"
#include "Components/PrimitiveComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Components/LightComponentBase.h"
#include "Components/ReflectionCaptureComponent.h"
#include "AI/NavigationSystemBase.h"
#include "Engine/MapBuildDataRegistry.h"
#include "Components/LightComponent.h"
#include "Model.h"
#include "Engine/Brush.h"
#include "Misc/PackageName.h"
#include "Editor/EditorEngine.h"
#include "Settings/EditorExperimentalSettings.h"
#include "Settings/LevelEditorMiscSettings.h"
#include "Engine/Texture2D.h"
#include "Misc/FeedbackContext.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "GameFramework/WorldSettings.h"
#include "Engine/GeneratedMeshAreaLight.h"
#include "Components/SkyLightComponent.h"
#include "Components/SkyAtmosphereComponent.h"
#include "Components/ModelComponent.h"
#include "StaticMeshComponentLODInfo.h"
#include "Engine/LightMapTexture2D.h"
#include "Editor.h"
#include "Engine/Selection.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "Dialogs/Dialogs.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/HLOD/HLODActor.h"
#include "WorldPartition/StaticLightingData/VolumetricLightmapGrid.h"
#include "WorldPartition/StaticLightingData/MapBuildDataActor.h"
#include "WorldPartition/HLOD/HLODLayer.h"
#include "WorldPartition/HLOD/HLODSourceActorsFromCell.h"
#include "EngineUtils.h"
#include "WorldPartition/StaticLightingData/StaticLightingDescriptors.h"

FSwarmDebugOptions GSwarmDebugOptions;

#include "Lightmass/LightmassCharacterIndirectDetailVolume.h"
#include "Lightmass/VolumetricLightmapDensityVolume.h"
#include "StaticLighting.h"
#include "StaticLightingSystem/StaticLightingPrivate.h"
#include "ModelLight.h"
#include "Engine/LevelStreaming.h"
#include "LevelUtils.h"
#include "EngineModule.h"
#include "LightMap.h"
#include "ShadowMap.h"
#include "EditorBuildUtils.h"
#include "ComponentRecreateRenderStateContext.h"
#include "Engine/LODActor.h"

DEFINE_LOG_CATEGORY(LogStaticLightingSystem);

#include "EngineGlobals.h"


#include "Lightmass/LightmassImportanceVolume.h"
#include "Components/LightmassPortalComponent.h"
#include "Lightmass/Lightmass.h"
#include "StatsViewerModule.h"
#include "Logging/TokenizedMessage.h"
#include "Logging/MessageLog.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Misc/UObjectToken.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Rendering/StaticLightingSystemInterface.h"
#include "BuildSettings.h"
#include "Misc/EngineBuildSettings.h"
#include "TargetReceipt.h"
#include "LevelInstance/LevelInstanceSubsystem.h"

#define LOCTEXT_NAMESPACE "StaticLightingSystem"

/** The number of hardware threads to not use for building static lighting. */
#define NUM_STATIC_LIGHTING_UNUSED_THREADS 0

bool GbLogAddingMappings = false;

/** Counts the number of lightmap textures generated each lighting build. */
extern ENGINE_API int32 GLightmapCounter;
/** Whether to compress lightmaps. Reloaded from ini each lighting build. */
extern ENGINE_API bool GCompressLightmaps;

/** Whether to allow lighting builds to generate streaming lightmaps. */
extern ENGINE_API bool GAllowStreamingLightmaps;

// NOTE: We're only counting the top-level mip-map for the following variables.
/** Total number of texels allocated for all lightmap textures. */
extern ENGINE_API uint64 GNumLightmapTotalTexels;
/** Total number of texels used if the texture was non-power-of-two. */
extern ENGINE_API uint64 GNumLightmapTotalTexelsNonPow2;
/** Number of lightmap textures generated. */
extern ENGINE_API int32 GNumLightmapTextures;
/** Total number of mapped texels. */
extern ENGINE_API uint64 GNumLightmapMappedTexels;
/** Total number of unmapped texels. */
extern ENGINE_API uint64 GNumLightmapUnmappedTexels;
/** Whether to allow cropping of unmapped borders in lightmaps and shadowmaps. Controlled by BaseEngine.ini setting. */
extern ENGINE_API bool GAllowLightmapCropping;
/** Total lightmap texture memory size (in bytes), including GLightmapTotalStreamingSize. */
extern ENGINE_API uint64 GLightmapTotalSize;
/** Total memory size for streaming lightmaps (in bytes). */
extern ENGINE_API uint64 GLightmapTotalStreamingSize;
/** Largest boundingsphere radius to use when packing lightmaps into a texture atlas. */
extern ENGINE_API float GMaxLightmapRadius;

/** Total number of texels allocated for all shadowmap textures. */
extern ENGINE_API uint64 GNumShadowmapTotalTexels;
/** Number of shadowmap textures generated. */
extern ENGINE_API int32 GNumShadowmapTextures;
/** Total number of mapped texels. */
extern ENGINE_API uint64 GNumShadowmapMappedTexels;
/** Total number of unmapped texels. */
extern ENGINE_API uint64 GNumShadowmapUnmappedTexels;
/** Total shadowmap texture memory size (in bytes), including GShadowmapTotalStreamingSize. */
extern ENGINE_API uint64 GShadowmapTotalSize;
/** Total texture memory size for streaming shadowmaps. */
extern ENGINE_API uint64 GShadowmapTotalStreamingSize;

/** If non-zero, purge old lightmap data when rebuilding lighting. */
int32 GPurgeOldLightmaps=1;
static FAutoConsoleVariableRef CVarPurgeOldLightmaps(
	TEXT("PurgeOldLightmaps"),
	GPurgeOldLightmaps,
	TEXT("If non-zero, purge old lightmap data when rebuilding lighting.")
	);


int32 GMultithreadedLightmapEncode = 1;
static FAutoConsoleVariableRef CVarMultithreadedLightmapEncode(TEXT("r.MultithreadedLightmapEncode"), GMultithreadedLightmapEncode, TEXT("Lightmap encoding after rebuild lightmaps is done multithreaded."));
int32 GMultithreadedShadowmapEncode = 1;
static FAutoConsoleVariableRef CVarMultithreadedShadowmapEncode(TEXT("r.MultithreadedShadowmapEncode"), GMultithreadedShadowmapEncode, TEXT("Shadowmap encoding after rebuild lightmaps is done multithreaded."));




TSharedPtr<FStaticLightingManager> FStaticLightingManager::StaticLightingManager;

TSharedPtr<FStaticLightingManager> FStaticLightingManager::Get()
{
	if (!StaticLightingManager.IsValid())
	{
		StaticLightingManager = MakeShareable(new FStaticLightingManager);
	}
	return StaticLightingManager;
}

void FStaticLightingManager::ProcessLightingData()
{
	auto StaticLightingSystem = FStaticLightingManager::Get()->ActiveStaticLightingSystem;

	check(StaticLightingSystem);

	FNavigationLockContext NavUpdateLock(StaticLightingSystem->GetWorld(), ENavigationLockReason::LightingUpdate);

	bool bSuccessful = StaticLightingSystem->FinishLightmassProcess();

	FEditorDelegates::OnLightingBuildKept.Broadcast();

	if (!bSuccessful)
	{
		FStaticLightingManager::Get()->FailLightingBuild();
	}

	FStaticLightingManager::Get()->ClearCurrentNotification();
}

void FStaticLightingManager::CancelLightingBuild()
{
	if (FStaticLightingManager::Get()->ActiveStaticLightingSystem->IsAsyncBuilding())
	{
		GEditor->SetMapBuildCancelled( true );
		FStaticLightingManager::Get()->ClearCurrentNotification();
		FEditorDelegates::OnLightingBuildFailed.Broadcast();
	}
	else
	{
		FStaticLightingManager::Get()->FailLightingBuild();
	}
}

void FStaticLightingManager::SendProgressNotification()
{
	// Start the lightmass 'progress' notification
	FNotificationInfo Info( LOCTEXT("LightBuildMessage", "Building lighting") );
	Info.bFireAndForget = false;
	Info.ButtonDetails.Add(FNotificationButtonInfo(
		LOCTEXT("LightBuildCancel","Cancel"),
		LOCTEXT("LightBuildCancelToolTip","Cancels the lighting build in progress."),
		FSimpleDelegate::CreateStatic(&FStaticLightingManager::CancelLightingBuild)));

	LightBuildNotification = FSlateNotificationManager::Get().AddNotification(Info);
	if (LightBuildNotification.IsValid())
	{
		LightBuildNotification.Pin()->SetCompletionState(SNotificationItem::CS_Pending);
	}
}

void FStaticLightingManager::ClearCurrentNotification()
{
	if ( LightBuildNotification.IsValid() )
	{
		LightBuildNotification.Pin()->SetCompletionState(SNotificationItem::CS_None);
		LightBuildNotification.Pin()->ExpireAndFadeout();
		LightBuildNotification.Reset();
	}
}

void FStaticLightingManager::SetNotificationText( FText Text )
{
	if ( LightBuildNotification.IsValid() )
	{
		LightBuildNotification.Pin()->SetText( Text );
	}
}

void FStaticLightingManager::ImportRequested()
{
	if (FStaticLightingManager::Get()->ActiveStaticLightingSystem)
	{
		FStaticLightingManager::Get()->ActiveStaticLightingSystem->CurrentBuildStage = FStaticLightingSystem::LightingStage::ImportRequested;
	}
}

void FStaticLightingManager::DiscardRequested()
{
	if (FStaticLightingManager::Get()->ActiveStaticLightingSystem)
	{
		FStaticLightingManager::Get()->ClearCurrentNotification();
		FStaticLightingManager::Get()->ActiveStaticLightingSystem->CurrentBuildStage = FStaticLightingSystem::LightingStage::Finished;
	}
}

void FStaticLightingManager::SendBuildDoneNotification( bool AutoApplyFailed )
{
	FText CompletedText = LOCTEXT("LightBuildDoneMessage", "Lighting build completed");

	if (ActiveStaticLightingSystem != StaticLightingSystems.Last().Get() && ActiveStaticLightingSystem->LightingContext.LightingScenario)
	{
		FString PackageName = FPackageName::GetShortName(ActiveStaticLightingSystem->LightingContext.LightingScenario->GetOutermost()->GetName());
		CompletedText = FText::Format(LOCTEXT("LightScenarioBuildDoneMessage", "{0} Lighting Scenario completed"), FText::FromString(PackageName));
	}

	FNotificationInfo Info(CompletedText);
	Info.bFireAndForget = false;
	Info.bUseThrobber = false;

	FNotificationButtonInfo ApplyNow = FNotificationButtonInfo(
		LOCTEXT( "LightBuildKeep", "Apply Now" ),
		LOCTEXT( "LightBuildKeepToolTip", "Keeps and applies built lighting data." ),
		FSimpleDelegate::CreateStatic( &FStaticLightingManager::ImportRequested ) );
	ApplyNow.VisibilityOnSuccess = EVisibility::Collapsed;

	FNotificationButtonInfo Discard = FNotificationButtonInfo(
		LOCTEXT( "LightBuildDiscard", "Discard" ),
		LOCTEXT( "LightBuildDiscardToolTip", "Ignores the built lighting data generated." ),
		FSimpleDelegate::CreateStatic( &FStaticLightingManager::DiscardRequested ) );
	Discard.VisibilityOnSuccess = EVisibility::Collapsed;

	Info.ButtonDetails.Add( ApplyNow );
	Info.ButtonDetails.Add( Discard );

	FEditorDelegates::OnLightingBuildSucceeded.Broadcast();

	LightBuildNotification = FSlateNotificationManager::Get().AddNotification( Info );
	if ( LightBuildNotification.IsValid() )
	{
		LightBuildNotification.Pin()->SetCompletionState( AutoApplyFailed ? SNotificationItem::CS_Pending : SNotificationItem::CS_Success );
	}
}

void FStaticLightingManager::CreateStaticLightingSystem(const FLightingBuildOptions& Options)
{
	if (StaticLightingSystems.Num() == 0)
	{
		check(!ActiveStaticLightingSystem);

		bBuildReflectionCapturesOnFinish = !Options.bOnlyBuildVisibility && !Options.bApplyDeferedActorMappingPass && !Options.bVolumetricLightmapFinalizerPass;

		UWorld* World = GWorld;
		
		for (ULevel* Level : World->GetLevels())
		{
			if (Level->bIsLightingScenario && Level->bIsVisible)
			{
				StaticLightingSystems.Emplace(new FStaticLightingSystem(Options, FStaticLightingBuildContext(World, Level)));
			}
		}

		if (StaticLightingSystems.Num() == 0)
		{
			StaticLightingSystems.Emplace(new FStaticLightingSystem(Options, FStaticLightingBuildContext(World, nullptr)));
		}

		//@todo_ow: Initialize descriptors if it's a paritioned world? (editor build)

		ActiveStaticLightingSystem = StaticLightingSystems[0].Get();

		if (ActiveStaticLightingSystem->CheckLightmassExecutableVersion())
		{
			if (ActiveStaticLightingSystem->BeginLightmassProcess())
			{
				SendProgressNotification();
			}
			else
			{
				FStaticLightingManager::Get()->FailLightingBuild();
			}
		}
		else
		{
			if (FEngineBuildSettings::IsSourceDistribution())
			{
				FStaticLightingManager::Get()->FailLightingBuild(LOCTEXT("LightmassExecutableOutdatedMessage", "Unreal Lightmass executable is outdated. Recompile UnrealLightmass project with Development configuration in Visual Studio."));
			}
			else
			{
				// Lightmass should never be outdated in a launcher binary build.
				FStaticLightingManager::Get()->FailLightingBuild(LOCTEXT("LauncherBuildNeedsVerificationMessage", "Unreal Lightmass executable is damaged. Try verifying your engine installation in Epic Games Launcher."));
			}
		}
	}
	else
	{
		// Tell the user that they must close their current build first.
		FNotificationInfo Info( LOCTEXT("LightBuildInProgressWarning", "A lighting build is already in progress! Please cancel it before triggering a new build.") );
		Info.ExpireDuration = 5.0f;
		TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
		if (Notification.IsValid())
		{
			Notification->SetCompletionState(SNotificationItem::CS_Fail);
		}

		FEditorDelegates::OnLightingBuildFailed.Broadcast();
	}
}

void FStaticLightingManager::UpdateBuildLighting()
{
	if (ActiveStaticLightingSystem != NULL)
	{
		// Note: UpdateLightingBuild can change ActiveStaticLightingSystem
		ActiveStaticLightingSystem->UpdateLightingBuild();

		if (ActiveStaticLightingSystem && ActiveStaticLightingSystem->CurrentBuildStage == FStaticLightingSystem::LightingStage::Finished)
		{
			ActiveStaticLightingSystem = nullptr;
			StaticLightingSystems.RemoveAt(0);

			if (StaticLightingSystems.Num() > 0)
			{
				ActiveStaticLightingSystem = StaticLightingSystems[0].Get();

				bool bSuccess = ActiveStaticLightingSystem->BeginLightmassProcess();

				if (bSuccess)
				{
					SendProgressNotification();
				}
				else
				{
					// BeginLightmassProcess returns false if there are errors or no precomputed lighting is allowed. Handle both cases.
					const bool bAllowStaticLighting = IsStaticLightingAllowed();
					const bool bForceNoPrecomputedLighting = GWorld->GetWorldSettings()->bForceNoPrecomputedLighting || !bAllowStaticLighting;

					if (bForceNoPrecomputedLighting)
					{
						DestroyStaticLightingSystems();
					}
					else
					{
						FStaticLightingManager::Get()->FailLightingBuild();
					}
				}
			}
		}

		if (!ActiveStaticLightingSystem)
		{
			FinishLightingBuild();
		}
	}
}

void FStaticLightingManager::FailLightingBuild( FText ErrorText)
{
	const bool bAllowStaticLighting = IsStaticLightingAllowed();
	const bool bForceNoPrecomputedLighting = GWorld->GetWorldSettings()->bForceNoPrecomputedLighting || !bAllowStaticLighting;

	FStaticLightingManager::Get()->ClearCurrentNotification();
	
	if (!bForceNoPrecomputedLighting)
	{
		if (GEditor->GetMapBuildCancelled())
		{
			ErrorText = LOCTEXT("LightBuildCanceledMessage", "Lighting build canceled.");
		}
		else
		{
			// Override failure message if one provided
			if (ErrorText.IsEmpty())
			{
				ErrorText = LOCTEXT("LightBuildFailedMessage", "Lighting build failed.");
			}
		}

		FNotificationInfo Info( ErrorText );
		Info.ExpireDuration = 4.f;
	
		FEditorDelegates::OnLightingBuildFailed.Broadcast();

		LightBuildNotification = FSlateNotificationManager::Get().AddNotification(Info);
		if (LightBuildNotification.IsValid())
		{
			LightBuildNotification.Pin()->SetCompletionState(SNotificationItem::CS_Fail);
		}

		UE_LOGF(LogStaticLightingSystem, Warning, "Failed to build lighting!!! %ls",*ErrorText.ToString());

		FMessageLog("LightingResults").Open();
	}

	DestroyStaticLightingSystems();
}

void FStaticLightingManager::FinishLightingBuild()
{
	UWorld* World = GWorld;

	GetRendererModule().UpdateMapNeedsLightingFullyRebuiltState(World);
	GEngine->DeferredCommands.AddUnique(TEXT("MAP CHECK NOTIFYRESULTS"));

	if (World->Scene)
	{
		// Everything should be built at this point, dump unbuilt interactions for debugging
		World->Scene->DumpUnbuiltLightInteractions(*GLog);
	}

	// Verify if new MapBuildDataIDs were created during the build and mark their object's package dirty
	for (UStaticMeshComponent* StaticMeshComponent : TObjectRange<UStaticMeshComponent>())
	{
		if (StaticMeshComponent->IsTemplate() || !StaticMeshComponent->HasStaticLighting())
		{
			continue;
		}

		for (FStaticMeshComponentLODInfo& LODInfo : StaticMeshComponent->LODData)
		{
			if (LODInfo.bMapBuildDataChanged)
			{
				UE_LOGF(LogEngine, Warning, "MapBuildDataID for %ls was updated during the static lighting build, marking package dirty (%ls)", *StaticMeshComponent->GetOwner()->GetActorNameOrLabel(), *StaticMeshComponent->GetFullName());
				StaticMeshComponent->MarkPackageDirty();
			}
		}
	}

	if (bBuildReflectionCapturesOnFinish)
	{
		GEditor->BuildReflectionCaptures(World);
	}
}

void FStaticLightingManager::DestroyStaticLightingSystems()
{
	ActiveStaticLightingSystem = NULL;
	StaticLightingSystems.Empty();
}

bool FStaticLightingManager::IsLightingBuildCurrentlyRunning() const
{
	return ActiveStaticLightingSystem != NULL;
}

bool FStaticLightingManager::IsLightingBuildCurrentlyExporting() const
{
	return ActiveStaticLightingSystem != NULL && ActiveStaticLightingSystem->IsAmortizedExporting();
}

FStaticLightingSystem::FStaticLightingSystem(const FLightingBuildOptions& InOptions, FStaticLightingBuildContext&& context)
	: Options(InOptions)
	, bBuildCanceled(false)
	, DeterministicIndex(0)
	, NextVisibilityId(0)
	, CurrentBuildStage(FStaticLightingSystem::LightingStage::NotRunning)
	, LightingContext(MoveTemp(context))
	, LightmassProcessor(NULL)
{
}

FStaticLightingSystem::~FStaticLightingSystem()
{
	if (LightmassProcessor)
	{
		delete LightmassProcessor;
	}
}

bool FStaticLightingSystem::CheckLightmassExecutableVersion()
{
	FTargetReceipt LightmassReceipt;
	
	if (!LightmassReceipt.Read(FTargetReceipt::GetDefaultPath(*FPaths::EngineDir(), TEXT("UnrealLightmass"), FPlatformProcess::GetBinariesSubdirectory(), EBuildConfiguration::Development, nullptr)))
	{
		return false;
	}

	return BuildSettings::GetCurrentChangelist() == LightmassReceipt.Version.Changelist;
}

bool FStaticLightingSystem::BeginLightmassProcess()
{
	StartTime = FPlatformTime::Seconds();
	
	CurrentBuildStage = FStaticLightingSystem::LightingStage::Startup;

	bool bRebuildDirtyGeometryForLighting = true;
	bool bForceNoPrecomputedLighting = false;

	GDebugStaticLightingInfo = FDebugLightingOutput();

	{
		FLightmassStatistics::FScopedGather StartupStatScope(LightmassStatistics.StartupTime);

		// Flip the results page
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("TimeStamp"), FText::AsDateTime(FDateTime::UtcNow()));
		FText LightingResultsPageName(FText::Format(LOCTEXT("LightingResultsPageName", "Lighting Build - {TimeStamp}"), Arguments));
		FMessageLog("LightingResults").NewPage(LightingResultsPageName);


		FStatsViewerModule& StatsViewerModule = FModuleManager::Get().LoadModuleChecked<FStatsViewerModule>(TEXT("StatsViewer"));
		StatsViewerModule.GetPage(EStatsPage::LightingBuildInfo)->Clear();

		GLightmapCounter = 0;
		GNumLightmapTotalTexels = 0;
		GNumLightmapTotalTexelsNonPow2 = 0;
		GNumLightmapTextures = 0;
		GNumLightmapMappedTexels = 0;
		GNumLightmapUnmappedTexels = 0;
		GLightmapTotalSize = 0;
		GLightmapTotalStreamingSize = 0;

		GNumShadowmapTotalTexels = 0;
		GNumShadowmapTextures = 0;
		GNumShadowmapMappedTexels = 0;
		GNumShadowmapUnmappedTexels = 0;
		GShadowmapTotalSize = 0;
		GShadowmapTotalStreamingSize = 0;

		TSet<UPackage*> PackagesToDirty;

		for( TObjectIterator<UPrimitiveComponent> It ; It ; ++It )
		{
			UPrimitiveComponent* Component = *It;
			if (Component->VisibilityId != INDEX_NONE)
			{
				Component->VisibilityId = INDEX_NONE;
				PackagesToDirty.Add(Component->GetPackage());
			}
		}

		{
			TMap<FGuid, ULevel*> LevelGuids;

			for (int32 LevelIndex = 0; LevelIndex < LightingContext.World->GetNumLevels(); LevelIndex++)
			{
				ULevel* Level = LightingContext.World->GetLevel(LevelIndex);

				if (ShouldOperateOnLevel(Level) && Options.ShouldBuildLightingForLevel(Level))
				{
					if (LevelGuids.Contains(Level->LevelBuildDataId))
					{
						FMessageLog("LightingResults").Warning()
							->AddToken(FUObjectToken::Create(LevelGuids[Level->LevelBuildDataId]->GetOuter()))
							->AddToken(FTextToken::Create(LOCTEXT("LightmassError_DuplicatedLevelGuids1", "has the same level built data GUID as")))
							->AddToken(FUObjectToken::Create(Level->GetOuter()))
							->AddToken(FTextToken::Create(LOCTEXT("LightmassError_DuplicatedLevelGuids2", ". A new GUID is assigned to the later one. All previously built lighting is invalidated and the level needs to be resaved.")));

						Level->LevelBuildDataId = FGuid::NewGuid();
						PackagesToDirty.Add(Level->GetPackage());
					}

					LevelGuids.Add(Level->LevelBuildDataId, Level);
				}
			}
		}

		// Mark package(s) as dirty
		for (UPackage* Package : PackagesToDirty)
		{
			Package->MarkPackageDirty();
		}

		FString SkippedLevels;
		for ( int32 LevelIndex=0; LevelIndex < LightingContext.World->GetNumLevels(); LevelIndex++ )
		{
			ULevel* Level = LightingContext.World->GetLevel(LevelIndex);

			if (ShouldOperateOnLevel(Level))
			{
				Level->LightmapTotalSize = 0.0f;
				Level->ShadowmapTotalSize = 0.0f;
				ULevelStreaming* LevelStreaming = NULL;
				if ( LightingContext.World->PersistentLevel != Level )
				{
					LevelStreaming = FLevelUtils::FindStreamingLevel( Level );
				}
				if (!Options.ShouldBuildLightingForLevel(Level))
				{
					if (SkippedLevels.Len() > 0)
					{
						SkippedLevels += FString(TEXT(", "));
					}
					SkippedLevels += Level->GetName();
					GatherBuildDataResourcesToKeep(Level);
				}
			}
			else if (Level && !Level->bIsLightingScenario && !Level->bIsVisible)
			{
				GatherBuildDataResourcesToKeep(Level);
			}
		}

		// keep the VLM data if we won't be be rebuilding it
		if (Options.bApplyDeferedActorMappingPass && !Options.bVolumetricLightmapFinalizerPass)
		{
			 BuildDataResourcesToKeep.Add(LightingContext.GetLevelBuildDataID(LightingContext.GetPersistentLevelGuid()));
		}

		for (ULevelStreaming* CurStreamingLevel : LightingContext.World->GetStreamingLevels())
		{
			if (CurStreamingLevel && CurStreamingLevel->GetLoadedLevel() && !CurStreamingLevel->GetShouldBeVisibleInEditor())
			{
				if (SkippedLevels.Len() > 0)
				{
					SkippedLevels += FString(TEXT(", ")) + CurStreamingLevel->GetWorldAssetPackageName();
				}
				else
				{
					SkippedLevels += CurStreamingLevel->GetWorldAssetPackageName();
				}
			}
		}

		if (SkippedLevels.Len() > 0 && !IsRunningCommandlet())
		{
			// Warn when some levels are not visible and therefore will not be built, because that indicates that only a partial build will be done,
			// Lighting will still be unbuilt for some areas when playing through the level.
			const FText SkippedLevelsWarning = FText::Format( LOCTEXT("SkippedLevels", "The following levels will not have the lighting rebuilt because of your selected lighting build options: {0}"), FText::FromString( SkippedLevels ) );
			FSuppressableWarningDialog::FSetupInfo Info( SkippedLevelsWarning, LOCTEXT("SkippedLevelsDialogTitle", "Rebuild Lighting - Warning" ), "WarnOnHiddenLevelsBeforeRebuild" );
			Info.ConfirmText = LOCTEXT("SkippedWarningConfirm", "Build");

			FSuppressableWarningDialog WarnAboutSkippedLevels( Info );
			WarnAboutSkippedLevels.ShowModal();
		}

		const bool bAllowStaticLighting = IsStaticLightingAllowed();
		bForceNoPrecomputedLighting = LightingContext.World->GetWorldSettings()->bForceNoPrecomputedLighting || !bAllowStaticLighting;
		GConfig->GetFloat( TEXT("TextureStreaming"), TEXT("MaxLightmapRadius"), GMaxLightmapRadius, GEngineIni );
		GConfig->GetBool( TEXT("TextureStreaming"), TEXT("AllowStreamingLightmaps"), GAllowStreamingLightmaps, GEngineIni );
		
		if (!bForceNoPrecomputedLighting)
		{
			// Begin the static lighting progress bar.
			GWarn->BeginSlowTask( LOCTEXT("BeginBuildingStaticLightingTaskStatus", "Building lighting"), false );
		}
		else
		{
			UE_LOGF(LogStaticLightingSystem, Warning, "WorldSettings.bForceNoPrecomputedLighting is true, Skipping Lighting Build!");
		}
		
		FConfigContext::ForceReloadIntoGConfig().Load(TEXT("Lightmass"), GLightmassIni);

		verify(GConfig->GetBool(TEXT("DevOptions.StaticLighting"), TEXT("bUseBilinearFilterLightmaps"), GUseBilinearLightmaps, GLightmassIni));
		verify(GConfig->GetBool(TEXT("DevOptions.StaticLighting"), TEXT("bAllowCropping"), GAllowLightmapCropping, GLightmassIni));
		verify(GConfig->GetBool(TEXT("DevOptions.StaticLighting"), TEXT("bRebuildDirtyGeometryForLighting"), bRebuildDirtyGeometryForLighting, GLightmassIni));
		verify(GConfig->GetBool(TEXT("DevOptions.StaticLighting"), TEXT("bCompressLightmaps"), GCompressLightmaps, GLightmassIni));

		GCompressLightmaps = GCompressLightmaps && LightingContext.World->GetWorldSettings()->LightmassSettings.bCompressLightmaps;

		GAllowLightmapPadding = true;
		FMemory::Memzero(&LightingMeshBounds, sizeof(FBox));
		FMemory::Memzero(&AutomaticImportanceVolumeBounds, sizeof(FBox));

		GLightingBuildQuality = Options.QualityLevel;
	
	}

	{
		FLightmassStatistics::FScopedGather CollectStatScope(LightmassStatistics.CollectTime);

		// Prepare lights for rebuild.
		{
			FLightmassStatistics::FScopedGather PrepareStatScope(LightmassStatistics.PrepareLightsTime);

			if (!Options.bOnlyBuildVisibility)
			{
				// Delete all AGeneratedMeshAreaLight's, since new ones will be created after the build with updated properties.
				USelection* EditorSelection = GEditor->GetSelectedActors();
				for(TObjectIterator<AGeneratedMeshAreaLight> LightIt;LightIt;++LightIt)
				{
					if (ShouldOperateOnLevel((*LightIt)->GetLevel()))
					{
					if (EditorSelection)
					{
						EditorSelection->Deselect(*LightIt);
					}
					(*LightIt)->GetWorld()->DestroyActor(*LightIt);
				}
				}

				for (TObjectIterator<ULightComponentBase> LightIt(RF_ClassDefaultObject, /** bIncludeDerivedClasses */ true, /** InternalExcludeFlags */ EInternalObjectFlags::Garbage); LightIt; ++LightIt)
				{
					ULightComponentBase* const Light = *LightIt;
					const bool bLightIsInWorld = IsValid(Light->GetOwner()) 
						&& LightingContext.World->ContainsActor(Light->GetOwner());

					if (bLightIsInWorld && ShouldOperateOnLevel(Light->GetOwner()->GetLevel()))
					{
						if (Light->bAffectsWorld 
							&& Light->IsRegistered()
							&& (Light->HasStaticShadowing() || Light->HasStaticLighting()))
						{
							// Make sure the light GUIDs are up-to-date.
							Light->ValidateLightGUIDs();

							// Add the light to the system's list of lights in the world.
							Lights.Add(Light);
						}
					}
				}
			}
		}

		{
			FLightmassStatistics::FScopedGather GatherStatScope(LightmassStatistics.GatherLightingInfoTime);

			if (IsTexelDebuggingEnabled())
			{
				// Clear reference to the selected lightmap
				GCurrentSelectedLightmapSample.Lightmap = NULL;
			}
			
			GatherStaticLightingInfo(bRebuildDirtyGeometryForLighting, bForceNoPrecomputedLighting);
		}

		// Sort the mappings - and tag meshes if doing deterministic mapping
		if (GLightmassDebugOptions.bSortMappings)
		{
			struct FCompareNumTexels
			{
				FORCEINLINE bool operator()( const FStaticLightingMappingSortHelper& A, const FStaticLightingMappingSortHelper& B ) const
				{
					return B.NumTexels < A.NumTexels;
				}
			};
			UnSortedMappings.Sort( FCompareNumTexels() );
		
			for (int32 SortIndex = 0; SortIndex < UnSortedMappings.Num(); SortIndex++)
			{
				FStaticLightingMapping* Mapping = UnSortedMappings[SortIndex].Mapping;
				Mappings.Add(Mapping);

				if (Mapping->bProcessMapping)
				{
					if (Mapping->Mesh)
					{
						Mapping->Mesh->Guid = FGuid(0,0,0,DeterministicIndex++);
					}
				}
			}
			UnSortedMappings.Empty();
		}

		// Verify deterministic lighting setup, if it is enabled...
		for (int32 CheckMapIdx = 0; CheckMapIdx < Mappings.Num(); CheckMapIdx++)
		{
			if (Mappings[CheckMapIdx]->bProcessMapping)
			{
				FGuid CheckGuid = Mappings[CheckMapIdx]->Mesh->Guid;
				if ((CheckGuid.A != 0) ||
					(CheckGuid.B != 0) || 
					(CheckGuid.C != 0) ||
					(CheckGuid.D >= (uint32)(Mappings.Num()))
					)
				{
					UE_LOGF(LogStaticLightingSystem, Warning, "Lightmass: Error in deterministic lighting for %ls:%ls",
						*(Mappings[CheckMapIdx]->Mesh->Guid.ToString()), *(Mappings[CheckMapIdx]->GetDescription()));
				}
			}
		}

		// if we are dumping binary results, clear up any existing ones
		if (Options.bDumpBinaryResults)
		{
			FStaticLightingSystem::ClearBinaryDumps();
		}
	}

	ProcessingStartTime = FPlatformTime::Seconds();

	bool bLightingSuccessful = false;
	if (!bForceNoPrecomputedLighting)
	{
		bool bSavedUpdateStatus_LightMap = FLightMap2D::GetStatusUpdate();
		if (GLightmassDebugOptions.bImmediateProcessMappings)
		{
			FLightMap2D::SetStatusUpdate(false);
		}

		bLightingSuccessful = CreateLightmassProcessor();
		if (bLightingSuccessful)
		{
			GatherScene();
			bLightingSuccessful = InitiateLightmassProcessor();
		}

		if (GLightmassDebugOptions.bImmediateProcessMappings)
		{
			FLightMap2D::SetStatusUpdate(bSavedUpdateStatus_LightMap);
		}
	}
	else
	{
		InvalidateStaticLighting();

		// Calling ApplyNewLightingData() results in creating an empty MapBuildData which is causing cooking issues
		// Disabled for now
		//ApplyNewLightingData(true);
	}
	
	if (!bForceNoPrecomputedLighting)
	{
		// End the static lighting progress bar.
		GWarn->EndSlowTask();
	}

	return bLightingSuccessful;
}

void FStaticLightingSystem::InvalidateStaticLighting()
{
	FLightmassStatistics::FScopedGather InvalidationScopeStat(LightmassStatistics.InvalidationTime);
	FGlobalComponentRecreateRenderStateContext Context;

	for( int32 LevelIndex=0; LevelIndex < LightingContext.World->GetNumLevels(); LevelIndex++ )
	{
		bool bMarkLevelDirty = false;
		ULevel* Level = LightingContext.World->GetLevel(LevelIndex);
		
		if (!ShouldOperateOnLevel(Level))
		{
			continue;
		}

		const bool bBuildLightingForLevel = Options.ShouldBuildLightingForLevel( Level );
		
		if (bBuildLightingForLevel)
		{
			if (!Options.bOnlyBuildVisibility)
			{
				Level->ReleaseRenderingResources();

				if (Level->MapBuildData)
				{
					Level->MapBuildData->InvalidateStaticLighting(LightingContext.World, false, &BuildDataResourcesToKeep);
				}
			}
			if (Level == LightingContext.World->PersistentLevel)
			{
				Level->PrecomputedVisibilityHandler.Invalidate(LightingContext.World->Scene);
				Level->PrecomputedVolumeDistanceField.Invalidate(LightingContext.World->Scene);
			}

			// Mark any existing cached lightmap data as transient. This allows the derived data cache to purge it more aggressively.
			// It is safe to do so even if some of these lightmaps are needed. It just means compressed data will have to be retrieved
			// from the network cache or rebuilt.
			if (GPurgeOldLightmaps != 0 && Level->MapBuildData)
			{
				UPackage* MapDataPackage = Level->MapBuildData->GetOutermost();

				for (TObjectIterator<ULightMapTexture2D> It; It; ++It)
				{
					ULightMapTexture2D* LightMapTexture = *It;
					if (LightMapTexture->GetOutermost() == MapDataPackage)
					{
						LightMapTexture->MarkPlatformDataTransient();
					}
				}
			}
		}
	}

	// Gather and add all AMapBuildDataActor so they can collect the per-actor lighting data
	if (LightingContext.Descriptors)
	{
		TArray<UMapBuildDataRegistry*> MapRegistries = LightingContext.Descriptors->GetAllMapBuildData();

		for (UMapBuildDataRegistry* MapDataRegistry : MapRegistries)
		{
			MapDataRegistry->InvalidateStaticLighting(LightingContext.World, false, &BuildDataResourcesToKeep);
		}	
	}
}

void UpdateStaticLightingWorldPartitionHLODTreeIndices(TMultiMap<AActor*, FStaticLightingMesh*>& ActorMeshMap, AWorldPartitionHLOD* LODActor, uint32 HLODTreeIndex, uint32& HLODLeafIndex, const TMap<FGuid, AActor*>& ActorInstanceGuidToActorPtr)
{	
	check(LODActor && HLODTreeIndex > 0);

	uint32 LeafStartIndex = HLODLeafIndex;
	++HLODLeafIndex;
	
	UWorldPartitionHLODSourceActorsFromCell* SourceActors = Cast<UWorldPartitionHLODSourceActorsFromCell>(LODActor->GetSourceActors());
	if (!SourceActors)
	{
		return;
	}

	UWorldPartition* WorldPartition = LODActor->GetWorld()->GetWorldPartition();

	// Iterate over all sub actors
	for (const FWorldPartitionRuntimeCellObjectMapping& LODSubActor : SourceActors->GetActors())
	{
		// get the AActor from the SubActor
		AActor* SubActor = nullptr;
				
		FSoftObjectPath SoftPath(LODSubActor.Path.ToString());
		TSoftObjectPtr<AActor> SoftPtr(SoftPath);
		SubActor = SoftPtr.Get();
				
		if (!SubActor)
		{
			if (AActor* const * FoundActor = ActorInstanceGuidToActorPtr.Find(LODSubActor.ActorInstanceGuid))
			{
				SubActor = *FoundActor;
			}
		}
		
		if (ensureMsgf(SubActor, TEXT("Error during HLOD LOD tree setup, could not resolve ptr to Actor: %s\n"), *LODSubActor.Path.ToString()))
		{
			// if this happens we need to merge the WP HLOD tree with the normal actor HLOD tree`
			check(!Cast<ALODActor>(SubActor));		

			// If sub actor is an WP HLODActor iterate over that too
			if (AWorldPartitionHLOD* HLODSubActor = Cast<AWorldPartitionHLOD>(SubActor))
			{
				UpdateStaticLightingWorldPartitionHLODTreeIndices(ActorMeshMap, HLODSubActor, HLODTreeIndex, HLODLeafIndex, ActorInstanceGuidToActorPtr);
			}
			else
			{
				// Otherwise integrate sub actors into the tree
				TArray<FStaticLightingMesh*> SubActorMeshes;
				ActorMeshMap.MultiFind(SubActor, SubActorMeshes);

				for (FStaticLightingMesh* SubActorMesh : SubActorMeshes)
				{
					if (SubActorMesh->HLODTreeIndex == 0)
					{
						SubActorMesh->HLODTreeIndex = HLODTreeIndex;
						SubActorMesh->HLODChildStartIndex = HLODLeafIndex;
						SubActorMesh->HLODChildEndIndex = HLODLeafIndex;
						++HLODLeafIndex;
					}
					else
					{
						// Output error to message log containing tokens to the problematic objects
						FMessageLog("LightingResults").Warning()
							->AddToken(FUObjectToken::Create(SubActorMesh->Component->GetOwner()))
							->AddToken(FTextToken::Create(LOCTEXT("LightmassError_InvalidHLODTreeIndex", "will not be correctly lit since it is part of another Hierarchical LOD cluster besides ")))
							->AddToken(FUObjectToken::Create(LODActor));
					}
				}
			}
		}
	}

	TArray<FStaticLightingMesh*> LODActorMeshes;
	ActorMeshMap.MultiFind(LODActor, LODActorMeshes);
	for (FStaticLightingMesh* LODActorMesh : LODActorMeshes)
	{
		LODActorMesh->HLODTreeIndex = HLODTreeIndex;
		LODActorMesh->HLODChildStartIndex = LeafStartIndex;
		LODActorMesh->HLODChildEndIndex = HLODLeafIndex - 1;
		check(LODActorMesh->HLODChildEndIndex >= LODActorMesh->HLODChildStartIndex);
	}
}

void UpdateStaticLightingHLODTreeIndices(TMultiMap<AActor*, FStaticLightingMesh*>& ActorMeshMap, ALODActor* LODActor, uint32 HLODTreeIndex, uint32& HLODLeafIndex)
{
	check(LODActor && HLODTreeIndex > 0);

	uint32 LeafStartIndex = HLODLeafIndex;
	++HLODLeafIndex;

	for (AActor* SubActor : LODActor->SubActors)
	{
		if (ALODActor* LODSubActor = Cast<ALODActor>(SubActor))
		{
			UpdateStaticLightingHLODTreeIndices(ActorMeshMap, LODSubActor, HLODTreeIndex, HLODLeafIndex);
		}
		else
		{
			TArray<FStaticLightingMesh*> SubActorMeshes;
			ActorMeshMap.MultiFind(SubActor,SubActorMeshes);

			for (FStaticLightingMesh* SubActorMesh : SubActorMeshes)
			{
				if (SubActorMesh->HLODTreeIndex == 0)
				{
					SubActorMesh->HLODTreeIndex = HLODTreeIndex;
					SubActorMesh->HLODChildStartIndex = HLODLeafIndex;
					SubActorMesh->HLODChildEndIndex = HLODLeafIndex;
					++HLODLeafIndex;
				}
				else
				{
					// Output error to message log containing tokens to the problematic objects
					FMessageLog("LightingResults").Warning()
						->AddToken(FUObjectToken::Create(SubActorMesh->Component->GetOwner()))
						->AddToken(FTextToken::Create(LOCTEXT("LightmassError_InvalidHLODTreeIndex", "will not be correctly lit since it is part of another Hierarchical LOD cluster besides ")))
						->AddToken(FUObjectToken::Create(LODActor));
				}
			}
		}
	}

	TArray<FStaticLightingMesh*> LODActorMeshes;
	ActorMeshMap.MultiFind(LODActor, LODActorMeshes);
	for (FStaticLightingMesh* LODActorMesh : LODActorMeshes)
	{
		LODActorMesh->HLODTreeIndex = HLODTreeIndex;
		LODActorMesh->HLODChildStartIndex = LeafStartIndex;
		LODActorMesh->HLODChildEndIndex = HLODLeafIndex - 1;
		check(LODActorMesh->HLODChildEndIndex >= LODActorMesh->HLODChildStartIndex);
	}
}

void FStaticLightingSystem::GatherStaticLightingInfo(bool bRebuildDirtyGeometryForLighting, bool bForceNoPrecomputedLighting)
{
	uint32 ActorsInvalidated = 0;
	uint32 ActorsToInvalidate = 0;
	for( int32 LevelIndex=0; LevelIndex< LightingContext.World->GetNumLevels(); LevelIndex++ )
	{
		ActorsToInvalidate += LightingContext.World->GetLevel(LevelIndex)->Actors.Num();
	}
	const int32 ProgressUpdateFrequency = FMath::Max<int32>(ActorsToInvalidate / 20, 1);

	GWarn->StatusUpdate( ActorsInvalidated, ActorsToInvalidate, LOCTEXT("GatheringSceneGeometryStatus", "Gathering scene geometry...") );

	TMultiMap<AActor*, FStaticLightingMesh*> ActorMeshMap;	
	TArray<AWorldPartitionHLOD*> WorldPartitionHLODActors;

	bool bObjectsToBuildLightingForFound = false;
	// Gather static lighting info from actor components.
	for (int32 LevelIndex = 0; LevelIndex < LightingContext.World->GetNumLevels(); LevelIndex++)
	{
		TSet<UPackage*> PackagesToDirty;
		ULevel* Level = LightingContext.World->GetLevel(LevelIndex);

		if (!ShouldOperateOnLevel(Level))
		{
			continue;
		}

		// If the geometry is dirty and we're allowed to automatically clean it up, do so
		if (Level->bGeometryDirtyForLighting)
		{
			UE_LOGF(LogStaticLightingSystem, Warning, "WARNING: Lighting build detected that geometry needs to be rebuilt to avoid incorrect lighting (due to modifying a lighting property).");
			if (bRebuildDirtyGeometryForLighting)
			{
				// This will go ahead and clean up lighting on all dirty levels (not just this one)
				UE_LOGF(LogStaticLightingSystem, Warning, "WARNING: Lighting build automatically rebuilding geometry." );
				GEditor->Exec(LightingContext.World, TEXT("MAP REBUILD ALLDIRTYFORLIGHTING"));
			}
		}

		const bool bBuildLightingForLevel = Options.ShouldBuildLightingForLevel(Level);

		// Gather static lighting info from BSP.
		bool bBuildBSPLighting = bBuildLightingForLevel;

		TArray<FNodeGroup*> NodeGroupsToBuild;
		TArray<UModelComponent*> SelectedModelComponents;
		if (bBuildBSPLighting && !Options.bOnlyBuildVisibility)
		{
			if (Options.bOnlyBuildSelected)
			{
				UModel* Model = Level->Model;
				GLightmassDebugOptions.bGatherBSPSurfacesAcrossComponents = false;
				Model->GroupAllNodes(Level, Lights);
				bBuildBSPLighting = false;
				// Build only selected brushes/surfaces
				TArray<ABrush*> SelectedBrushes;
				for (int32 ActorIndex = 0; ActorIndex < Level->Actors.Num(); ActorIndex++)
				{
					AActor* Actor = Level->Actors[ActorIndex];
					if (Actor)
					{
						ABrush* Brush = Cast<ABrush>(Actor);
						if (Brush && Brush->IsSelected())
						{
							SelectedBrushes.Add(Brush);
						}
					}
				}

				TArray<int32> SelectedSurfaceIndices;
				// Find selected surfaces...
				for (int32 SurfIdx = 0; SurfIdx < Model->Surfs.Num(); SurfIdx++)
				{
					bool bSurfaceSelected = false;
					FBspSurf& Surf = Model->Surfs[SurfIdx];
					if ((Surf.PolyFlags & PF_Selected) != 0)
					{
						SelectedSurfaceIndices.Add(SurfIdx);
						bSurfaceSelected = true;
					}
					else
					{
						int32 DummyIdx;
						if (SelectedBrushes.Find(Surf.Actor, DummyIdx) == true)
						{
							SelectedSurfaceIndices.Add(SurfIdx);
							bSurfaceSelected = true;
						}
					}

					if (bSurfaceSelected == true)
					{
						// Find it's model component...
						for (int32 NodeIdx = 0; NodeIdx < Model->Nodes.Num(); NodeIdx++)
						{
							const FBspNode& Node = Model->Nodes[NodeIdx];
							if (Node.iSurf == SurfIdx)
							{
								UModelComponent* SomeModelComponent = Level->ModelComponents[Node.ComponentIndex];
								if (SomeModelComponent)
								{
									SelectedModelComponents.AddUnique(SomeModelComponent);
									for (int32 InnerNodeIndex = 0; InnerNodeIndex < SomeModelComponent->Nodes.Num(); InnerNodeIndex++)
									{
										FBspNode& InnerNode = Model->Nodes[SomeModelComponent->Nodes[InnerNodeIndex]];
										SelectedSurfaceIndices.AddUnique(InnerNode.iSurf);
									}
								}
							}
						}
					}
				}

				// Pass 2...
				if (SelectedSurfaceIndices.Num() > 0)
				{
					for (int32 SSIdx = 0; SSIdx < SelectedSurfaceIndices.Num(); SSIdx++)
					{
						int32 SurfIdx = SelectedSurfaceIndices[SSIdx];
						// Find it's model component...
						for (int32 NodeIdx = 0; NodeIdx < Model->Nodes.Num(); NodeIdx++)
						{
							const FBspNode& Node = Model->Nodes[NodeIdx];
							if (Node.iSurf == SurfIdx)
							{
								UModelComponent* SomeModelComponent = Level->ModelComponents[Node.ComponentIndex];
								if (SomeModelComponent)
								{
									SelectedModelComponents.AddUnique(SomeModelComponent);
									for (int32 InnerNodeIndex = 0; InnerNodeIndex < SomeModelComponent->Nodes.Num(); InnerNodeIndex++)
									{
										FBspNode& InnerNode = Model->Nodes[SomeModelComponent->Nodes[InnerNodeIndex]];
										SelectedSurfaceIndices.AddUnique(InnerNode.iSurf);
									}
								}
							}
						}
					}
				}

				if (SelectedSurfaceIndices.Num() > 0)
				{
					// Fill in a list of all the node group to rebuild...
					bBuildBSPLighting = false;
					for (TMap<int32, FNodeGroup*>::TIterator It(Model->NodeGroups); It; ++It)
					{
						FNodeGroup* NodeGroup = It.Value();
						if (NodeGroup && (NodeGroup->Nodes.Num() > 0))
						{
							for (int32 GroupNodeIdx = 0; GroupNodeIdx < NodeGroup->Nodes.Num(); GroupNodeIdx++)
							{
								int32 CheckIdx;
								if (SelectedSurfaceIndices.Find(Model->Nodes[NodeGroup->Nodes[GroupNodeIdx]].iSurf, CheckIdx) == true)
								{
									NodeGroupsToBuild.AddUnique(NodeGroup);
									bBuildBSPLighting = true;
								}
							}
						}
					}
				}
			}
		}

		if (bBuildBSPLighting && !bForceNoPrecomputedLighting)
		{
			if (!Options.bOnlyBuildSelected || Options.bOnlyBuildVisibility)
			{
				// generate BSP mappings across the whole level
				AddBSPStaticLightingInfo(Level, bBuildBSPLighting);
			}
			else
			{
				if (NodeGroupsToBuild.Num() > 0)
				{
					bObjectsToBuildLightingForFound = true;
					AddBSPStaticLightingInfo(Level, NodeGroupsToBuild);
				}
			}
		}

		// Gather HLOD primitives
		TMultiMap<AActor*, UPrimitiveComponent*> PrimitiveActorMap;
		TMultiMap<UPrimitiveComponent*, UStaticMeshComponent*> PrimitiveSubStaticMeshMap;

		for (int32 ActorIndex = 0; ActorIndex < Level->Actors.Num(); ActorIndex++)
		{
			AActor* Actor = Level->Actors[ActorIndex];
			if (Actor)
			{
				ALODActor* LODActor = Cast<ALODActor>(Actor);
				if (LODActor && LODActor->GetStaticMeshComponent())
				{

					UPrimitiveComponent* PrimitiveParent = LODActor->GetStaticMeshComponent()->GetLODParentPrimitive();

					for (auto SubActor : LODActor->SubActors)
					{
						PrimitiveActorMap.Add(SubActor, LODActor->GetStaticMeshComponent());

						if (PrimitiveParent)
						{
							PrimitiveActorMap.Add(SubActor, PrimitiveParent);
						}

						TArray<UStaticMeshComponent*> SubStaticMeshComponents;
						SubActor->GetComponents(SubStaticMeshComponents);
						for (auto SMC : SubStaticMeshComponents)
						{
							PrimitiveSubStaticMeshMap.Add(LODActor->GetStaticMeshComponent(), SMC);
						}
					}
				}
			}
		}

		TArray<ALODActor*> LODActors;

		// Gather static lighting info from actors.
		for (int32 ActorIndex = 0; ActorIndex < Level->Actors.Num(); ActorIndex++)
		{
			AActor* Actor = Level->Actors[ActorIndex];
			if (Actor)
			{
				bool bBuildLightingForActor  = true;
				bool bIncludeActorInLighting = true;
				bool bDeferActorMappping = false;

				if (Options.ShouldBuildLighting)
				{
					Options.ShouldBuildLighting(Actor, bBuildLightingForActor, bIncludeActorInLighting, bDeferActorMappping);
				}


				const bool bBuildActorLighting =
					bBuildLightingForLevel &&
					bBuildLightingForActor &&
					(!Options.bOnlyBuildSelected || Actor->IsSelected());

				TInlineComponentArray<UPrimitiveComponent*> Components;
				Actor->GetComponents(Components);

				if (bBuildActorLighting)
				{
					bObjectsToBuildLightingForFound = true;
				}

				TArray<UPrimitiveComponent*> HLODPrimitiveParents;
				PrimitiveActorMap.MultiFind(Actor, HLODPrimitiveParents);
				
				ALODActor* LODActor = Cast<ALODActor>(Actor);
				if (LODActor)
				{
					LODActors.Add(LODActor);
				}

				AWorldPartitionHLOD* WorldPartitionLODActor = Cast<AWorldPartitionHLOD>(Actor);
				if (WorldPartitionLODActor)
				{
					WorldPartitionHLODActors.Add(WorldPartitionLODActor);
				}
				
				// Gather static lighting info from each of the actor's components.
				for (int32 ComponentIndex = 0; ComponentIndex < Components.Num(); ComponentIndex++)
				{
					UPrimitiveComponent* Primitive = Components[ComponentIndex];
					if (Primitive->IsRegistered() && !bForceNoPrecomputedLighting && bIncludeActorInLighting)
					{
						// Find the lights relevant to the primitive.
						TArray<ULightComponent*> PrimitiveRelevantLights;
						for (int32 LightIndex = 0; LightIndex < Lights.Num(); LightIndex++)
						{
							ULightComponentBase* LightBase = Lights[LightIndex];
							ULightComponent* Light = Cast<ULightComponent>(LightBase);

							// Only add enabled lights
							if (Light && Light->AffectsPrimitive(Primitive))
							{
								PrimitiveRelevantLights.Add(Light);
							}
						}

						// Query the component for its static lighting info.
						FStaticLightingPrimitiveInfo PrimitiveInfo;
						Primitive->GetStaticLightingInfo(PrimitiveInfo, PrimitiveRelevantLights, Options);
						if (PrimitiveInfo.Meshes.Num() > 0 && (Primitive->Mobility == EComponentMobility::Static))
						{
							if (LightingContext.World->GetWorldSettings()->bPrecomputeVisibility)
							{
								// Make sure packages gets dirtied since we are changing the visibility Id of a component in them
								PackagesToDirty.Add(Primitive->GetPackage());
							}

							PrimitiveInfo.VisibilityId = Primitive->VisibilityId = NextVisibilityId;
							NextVisibilityId++;
						}

						TArray<UStaticMeshComponent*> LODSubActorSMComponents;

						if (LODActor)
						{
							PrimitiveSubStaticMeshMap.MultiFind(Primitive, LODSubActorSMComponents);
						}

						for (auto Mesh : PrimitiveInfo.Meshes)
						{
							ActorMeshMap.Add(Actor, Mesh);
						}

						AddPrimitiveStaticLightingInfo(PrimitiveInfo, bBuildActorLighting, bDeferActorMappping);
					}
				}
			}

			ActorsInvalidated++;

			if (ActorsInvalidated % ProgressUpdateFrequency == 0)
			{
				GWarn->UpdateProgress(ActorsInvalidated, ActorsToInvalidate);
			}
		}

		// Recurse through HLOD trees, group actors and calculate child ranges
		uint32 HLODTreeIndex = 1;
		uint32 HLODLeafIndex;

		for (ALODActor* LODActor : LODActors)		
		{
			// Only process fully merged (root) HLOD nodes
			if (LODActor->GetStaticMeshComponent() && !LODActor->GetStaticMeshComponent()->GetLODParentPrimitive())
			{
				HLODLeafIndex = 0;

				UpdateStaticLightingHLODTreeIndices(ActorMeshMap, LODActor, HLODTreeIndex, HLODLeafIndex);

				++HLODTreeIndex;
			}
		}

		// Mark package(s) as dirty
		for (UPackage* Package : PackagesToDirty)
		{
			Package->MarkPackageDirty();
		}
	}

	// WorldPartition HLOD trees must be setup after we've iterated over all Actors & Levels Instances
	uint32 WPHLODTreeIndex = 1;
	uint32 WPHLODLeafIndex;
	TMap<FGuid, AActor*> ActorInstanceGuidToActorPtr;

	if (WorldPartitionHLODActors.Num())
	{
		for(TActorIterator<AActor> It(LightingContext.World); It; ++It)
		{
			ActorInstanceGuidToActorPtr.Add(It->GetActorInstanceGuid(), *It);
		}
	}

	for (AWorldPartitionHLOD* LODActor : WorldPartitionHLODActors)
	{
		// Process from the highest layer downwards
		if (!LODActor->GetSourceActors()->GetHLODLayer()->GetParentLayer())
		{
			WPHLODLeafIndex = 0;

			UpdateStaticLightingWorldPartitionHLODTreeIndices(ActorMeshMap, LODActor, WPHLODTreeIndex, WPHLODLeafIndex, ActorInstanceGuidToActorPtr);

			++WPHLODTreeIndex;
		}
	}

	if (Options.bOnlyBuildSelected)
	{
		FMessageLog("LightingResults").Warning(LOCTEXT("LightmassError_BuildSelected", "Building selected actors only, lightmap memory and quality will be sub-optimal until the next full rebuild."));

		if (!bObjectsToBuildLightingForFound)
		{
			FMessageLog("LightingResults").Error(LOCTEXT("LightmassError_BuildSelectedNothingSelected", "Building selected actors and BSP only, but no actors or BSP selected!"));
		}
	}
}

void FStaticLightingSystem::EncodeTextures(bool bLightingSuccessful)
{
	FLightmassStatistics::FScopedGather EncodeStatScope(LightmassStatistics.EncodingTime);

	FScopedSlowTask SlowTask(2);
	{
		FLightmassStatistics::FScopedGather EncodeStatScope2(LightmassStatistics.EncodingLightmapsTime);
		// Flush pending shadow-map and light-map encoding.
		SlowTask.EnterProgressFrame(1, LOCTEXT("EncodingImportedStaticLightMapsStatusMessage", "Encoding imported static light maps."));
		FLightMap2D::EncodeTextures(&LightingContext, bLightingSuccessful, GMultithreadedLightmapEncode ? true : false);
	}

	{
		FLightmassStatistics::FScopedGather EncodeStatScope2(LightmassStatistics.EncodingShadowMapsTime);
		SlowTask.EnterProgressFrame(1, LOCTEXT("EncodingImportedStaticShadowMapsStatusMessage", "Encoding imported static shadow maps."));
		FShadowMap2D::EncodeTextures(&LightingContext, bLightingSuccessful, GMultithreadedShadowmapEncode ? true : false);
	}
}

void FStaticLightingSystem::ApplyNewLightingData(bool bLightingSuccessful)
{
	{
		FLightmassStatistics::FScopedGather ApplyStatScope(LightmassStatistics.ApplyTime);
		// Now that the lighting is done, we can tell the model components to use their new elements,
		// instead of the pre-lighting ones
		UModelComponent::ApplyTempElements(bLightingSuccessful);
	}

	{
		FLightmassStatistics::FScopedGather FinishStatScope(LightmassStatistics.FinishingTime);

		// Mark lights of the computed level to have valid precomputed lighting.
		for (int32 LevelIndex = 0; LevelIndex < LightingContext.World->GetNumLevels(); LevelIndex++)
		{
			ULevel* Level = LightingContext.World->GetLevel(LevelIndex);

			if (!ShouldOperateOnLevel(Level))
			{
				continue;
			}

			// Notify level about new lighting data
			Level->OnApplyNewLightingData(bLightingSuccessful);

			if (LightingContext.World->PersistentLevel == Level)
			{
				Level->PrecomputedVisibilityHandler.UpdateScene(LightingContext.World->Scene);
				Level->PrecomputedVolumeDistanceField.UpdateScene(LightingContext.World->Scene);
			}

			uint32 ActorCount = Level->Actors.Num();

			for (uint32 ActorIndex = 0; ActorIndex < ActorCount; ++ActorIndex)
			{
				AActor* Actor = Level->Actors[ActorIndex];

				if (Actor && bLightingSuccessful && !Options.bOnlyBuildSelected)
				{
					UMapBuildDataRegistry* Registry = LightingContext.GetOrCreateRegistryForActor(Actor);

					TInlineComponentArray<ULightComponent*> LightComponents;
					Actor->GetComponents(LightComponents);

					for (int32 ComponentIndex = 0; ComponentIndex < LightComponents.Num(); ComponentIndex++)
					{
						ULightComponent* LightComponent = LightComponents[ComponentIndex];
						if (LightComponent && (LightComponent->HasStaticShadowing() || LightComponent->HasStaticLighting()))
						{
							if (!Registry->GetLightBuildData(LightComponent->LightGuid))
							{
								// Add a dummy entry for ULightComponent::IsPrecomputedLightingValid()
								Registry->FindOrAllocateLightBuildData(LightComponent->LightGuid, true);
							}
						}
					}

					// For each SkyAtmosphere which is a dependency of the build, add its guid to MapBuildData to track that it now has been built.
					TInlineComponentArray<USkyAtmosphereComponent*> SkyAtmosphereComponents;
					Actor->GetComponents(SkyAtmosphereComponents);
					for (int32 ComponentIndex = 0; ComponentIndex < SkyAtmosphereComponents.Num(); ComponentIndex++)
					{
						USkyAtmosphereComponent* SkyAtmosphereComponent = SkyAtmosphereComponents[ComponentIndex];
						if (SkyAtmosphereComponent)
						{
							if (!Registry->GetSkyAtmosphereBuildData(SkyAtmosphereComponent->GetStaticLightingBuiltGuid()))
							{
								Registry->FindOrAllocateSkyAtmosphereBuildData(SkyAtmosphereComponent->GetStaticLightingBuiltGuid());
							}
						}
					}
				}
			}
		}

		// Mark lights of the computed level to have valid precomputed lighting.
		for (int32 LevelIndex = 0; LevelIndex < LightingContext.World->GetNumLevels(); LevelIndex++)
		{
			ULevel* Level = LightingContext.World->GetLevel(LevelIndex);

			// in this specific case the Level MapBuildData is shared with another level and 
			// this specific level doesn't own it's data so we need to skip it
			if (!ShouldOperateOnLevel(Level))
			{
				continue;
			}

			const bool bBuildLightingForLevel = Options.ShouldBuildLightingForLevel( Level );

			// in this specific case the Level MapBuildData is shared with another level and 
			// this specific level doesn't own it's data so we need to skip it
			if (!Level->IsMapBuildDataOwner())
			{
				continue;
			}

			UMapBuildDataRegistry* Registry = LightingContext.GetRegistryForLevel(Level);
		
			if (Level->IsPersistentLevel() && LightingContext.World->IsPartitionedWorld() && (!Options.bApplyDeferedActorMappingPass))
			{
				// Transfer the new VolumetricLightMapGrid to the PersistentLevel
				Registry->SetVolumetricLightMapGridDesc(LightingContext.GetVolumetricLightMapGridDesc());
				LightingContext.ReleaseVolumetricLightMapGridDesc();
			}
			

			// Store off the quality of the lighting for the level if lighting was successful and we build lighting for this level.
			if( bLightingSuccessful && bBuildLightingForLevel )
			{
				Registry->LevelLightingQuality = Options.QualityLevel;
				Registry->MarkPackageDirty();
			}

			Registry->SetupLightmapResourceClusters();

			{
				int32 NumMeshes = 0;
				int32 NumClusters = 0;
				Registry->GetLightmapResourceClusterStats(NumMeshes, NumClusters);

				if (NumMeshes > 1)
				{
					const float Ratio = (float)NumMeshes / (float)NumClusters;
					const FString StatsString = FString::Printf(TEXT("%s storing lightmap data for %u meshes in %u LightmapResourceClusters (%.1f Meshes per cluster)."),
						*Registry->GetName(), NumMeshes, NumClusters, Ratio);

					UE_LOGF(LogStaticLightingSystem, Log, "%ls", *StatsString);
				}
			}

			Level->InitializeRenderingResources();
		}

		if (bLightingSuccessful && LightingContext.Descriptors)
		{
			TArray<UMapBuildDataRegistry*> MapRegistries = LightingContext.Descriptors->GetAllMapBuildData();

			for (UMapBuildDataRegistry* MapDataRegistry : MapRegistries)
			{
				MapDataRegistry->LevelLightingQuality = Options.QualityLevel;
				MapDataRegistry->MarkPackageDirty();
				
				MapDataRegistry->SetupLightmapResourceClusters();
				MapDataRegistry->InitializeClusterRenderingResources(LightingContext.World->Scene->GetFeatureLevel());
			}
		}

		// Ensure all primitives which were marked dirty by the lighting build are updated.
		// First clear all components so that any references to static lighting assets held 
		// by scene proxies will be fully released before any components are reregistered.
		// We do not rerun construction scripts - nothing should have changed that requires that, and 
		// want to know which components were not moved during lighting rebuild
		{
			FGlobalComponentRecreateRenderStateContext RecreateRenderState;
		}

		// Clean up old shadow-map and light-map data.
		CollectGarbage( GARBAGE_COLLECTION_KEEPFLAGS );

		// Commit the changes to the world's BSP surfaces.
		LightingContext.World->CommitModelSurfaces();
	}

	// Report failed lighting build (don't count cancelled builds as failure).
	if ( !bLightingSuccessful && !bBuildCanceled )
	{
		FMessageDialog::Open( EAppMsgType::Ok, LOCTEXT("LightingBuildFailedDialogMessage", "The lighting build failed! See the log for more information!") );
	}
}

/**
 * Reports lighting build statistics to the log.
 */
void FStaticLightingSystem::ReportStatistics()
{
	extern UNREALED_API bool GLightmassStatsMode;
	if ( GLightmassStatsMode )
	{
		double TrackedTime =
			LightmassStatistics.StartupTime
			+ LightmassStatistics.CollectTime
			+ LightmassStatistics.ProcessingTime
			+ LightmassStatistics.ImportTime
			+ LightmassStatistics.ApplyTime
			+ LightmassStatistics.EncodingTime
			+ LightmassStatistics.InvalidationTime
			+ LightmassStatistics.FinishingTime;
		double UntrackedTime = LightmassStatistics.TotalTime - TrackedTime;
		UE_LOGF(LogStaticLightingSystem, Log,
			"Illumination: %ls total\n" "   %3.1f%%\t%8.1fs    Untracked time\n"
			, *FPlatformTime::PrettyTime(LightmassStatistics.TotalTime)
			, UntrackedTime / LightmassStatistics.TotalTime * 100.0
			, UntrackedTime
		);
		UE_LOGF(LogStaticLightingSystem, Log,
			"Breakdown of Illumination time\n" "   %3.1f%%\t%8.1fs \tStarting up\n" "   %3.1f%%\t%8.1fs \tCollecting\n" "   %3.1f%%\t%8.1fs \t--> Preparing lights\n" "   %3.1f%%\t%8.1fs \t--> Gathering lighting info\n" "   %3.1f%%\t%8.1fs \tProcessing\n" "   %3.1f%%\t%8.1fs \tImporting\n" "   %3.1f%%\t%8.1fs \tApplying\n" "   %3.1f%%\t%8.1fs \tEncoding\n" "   %3.1f%%\t%8.1fs \tInvalidating\n" "   %3.1f%%\t%8.1fs \tFinishing\n"
			, LightmassStatistics.StartupTime / LightmassStatistics.TotalTime * 100.0
			, LightmassStatistics.StartupTime
			, LightmassStatistics.CollectTime / LightmassStatistics.TotalTime * 100.0
			, LightmassStatistics.CollectTime
			, LightmassStatistics.PrepareLightsTime / LightmassStatistics.TotalTime * 100.0
			, LightmassStatistics.PrepareLightsTime
			, LightmassStatistics.GatherLightingInfoTime / LightmassStatistics.TotalTime * 100.0
			, LightmassStatistics.GatherLightingInfoTime
			, LightmassStatistics.ProcessingTime / LightmassStatistics.TotalTime * 100.0
			, LightmassStatistics.ProcessingTime
			, LightmassStatistics.ImportTime / LightmassStatistics.TotalTime * 100.0
			, LightmassStatistics.ImportTime
			, LightmassStatistics.ApplyTime / LightmassStatistics.TotalTime * 100.0
			, LightmassStatistics.ApplyTime
			, LightmassStatistics.EncodingTime / LightmassStatistics.TotalTime * 100.0
			, LightmassStatistics.EncodingTime
			, LightmassStatistics.InvalidationTime / LightmassStatistics.TotalTime * 100.0
			, LightmassStatistics.InvalidationTime
			, LightmassStatistics.FinishingTime / LightmassStatistics.TotalTime * 100.0
			, LightmassStatistics.FinishingTime
			);
		UE_LOGF(LogStaticLightingSystem, Log,
			"Breakdown of Processing time\n" "   %3.1f%%\t%8.1fs \tCollecting Lightmass scene\n" "   %3.1f%%\t%8.1fs \tExporting\n" "   %3.1f%%\t%8.1fs \tLightmass\n" "   %3.1f%%\t%8.1fs \tSwarm startup\n" "   %3.1f%%\t%8.1fs \tSwarm callback\n" "   %3.1f%%\t%8.1fs \tSwarm job open\n" "   %3.1f%%\t%8.1fs \tSwarm job close\n" "   %3.1f%%\t%8.1fs \tImporting\n" "   %3.1f%%\t%8.1fs \tApplying\n"
			, LightmassStatistics.CollectLightmassSceneTime / LightmassStatistics.TotalTime * 100.0
			, LightmassStatistics.CollectLightmassSceneTime
			, LightmassStatistics.ExportTime / LightmassStatistics.TotalTime * 100.0
			, LightmassStatistics.ExportTime
			, LightmassStatistics.LightmassTime / LightmassStatistics.TotalTime * 100.0
			, LightmassStatistics.LightmassTime
			, LightmassStatistics.SwarmStartupTime / LightmassStatistics.TotalTime * 100.0
			, LightmassStatistics.SwarmStartupTime
			, LightmassStatistics.SwarmCallbackTime / LightmassStatistics.TotalTime * 100.0
			, LightmassStatistics.SwarmCallbackTime
			, LightmassStatistics.SwarmJobOpenTime / LightmassStatistics.TotalTime * 100.0
			, LightmassStatistics.SwarmJobOpenTime
			, LightmassStatistics.SwarmJobCloseTime / LightmassStatistics.TotalTime * 100.0
			, LightmassStatistics.SwarmJobCloseTime
			, LightmassStatistics.ImportTimeInProcessing / LightmassStatistics.TotalTime * 100.0
			, LightmassStatistics.ImportTimeInProcessing
			, LightmassStatistics.ApplyTimeInProcessing / LightmassStatistics.TotalTime * 100.0
			, LightmassStatistics.ApplyTimeInProcessing
		);

		UE_LOGF(LogStaticLightingSystem, Log,
			"Breakdown of Export Times\n" "   %8.1fs\tVisibility Data\n" "   %8.1fs\tVolumetricLightmap Data\n" "   %8.1fs\tLights\n" "   %8.1fs\tModels\n" "   %8.1fs\tStatic Meshes\n" "   %8.1fs\tMaterials\n" "   %8.1fs\tMesh Instances\n" "   %8.1fs\tLandscape Instances\n" "   %8.1fs\tMappings\n"
			, LightmassStatistics.ExportVisibilityDataTime
			, LightmassStatistics.ExportVolumetricLightmapDataTime
			, LightmassStatistics.ExportLightsTime
			, LightmassStatistics.ExportModelsTime
			, LightmassStatistics.ExportStaticMeshesTime
			, LightmassStatistics.ExportMaterialsTime
			, LightmassStatistics.ExportMeshInstancesTime
			, LightmassStatistics.ExportLandscapeInstancesTime
			, LightmassStatistics.ExportMappingsTime
		);

		UE_LOGF(LogStaticLightingSystem, Log,
			"Scratch counters\n" "   %3.1f%%\tScratch0\n" "   %3.1f%%\tScratch1\n" "   %3.1f%%\tScratch2\n" "   %3.1f%%\tScratch3\n"
			, LightmassStatistics.Scratch0
			, LightmassStatistics.Scratch1
			, LightmassStatistics.Scratch2
			, LightmassStatistics.Scratch3
		);

		float NumLightmapTotalTexels = float(FMath::Max<uint64>(GNumLightmapTotalTexels,1));
		float NumShadowmapTotalTexels = float(FMath::Max<uint64>(GNumShadowmapTotalTexels,1));
		float LightmapTexelsToMT = float(NUM_HQ_LIGHTMAP_COEF)/float(NUM_STORED_LIGHTMAP_COEF)/1024.0f/1024.0f;	// Strip out the SimpleLightMap
		float ShadowmapTexelsToMT = 1.0f/1024.0f/1024.0f;
		UE_LOGF(LogStaticLightingSystem, Log, "Lightmap textures: %.1f M texels (%.1f%% mapped, %.1f%% unmapped, %.1f%% wasted by packing, %.1f M non-pow2 texels)"
			, NumLightmapTotalTexels * LightmapTexelsToMT
			, 100.0f * float(GNumLightmapMappedTexels) / NumLightmapTotalTexels
			, 100.0f * float(GNumLightmapUnmappedTexels) / NumLightmapTotalTexels
			, 100.0f * float(GNumLightmapTotalTexels - GNumLightmapMappedTexels - GNumLightmapUnmappedTexels) / NumLightmapTotalTexels
			, GNumLightmapTotalTexelsNonPow2 * LightmapTexelsToMT
			);

		UE_LOGF(LogStaticLightingSystem, Log, "Shadowmap textures: %.1f M texels (%.1f%% mapped, %.1f%% unmapped, %.1f%% wasted by packing)"
			, NumShadowmapTotalTexels * ShadowmapTexelsToMT
			, 100.0f * float(GNumShadowmapMappedTexels) / NumShadowmapTotalTexels
			, 100.0f * float(GNumShadowmapUnmappedTexels) / NumShadowmapTotalTexels
			, 100.0f * float(GNumShadowmapTotalTexels - GNumShadowmapMappedTexels - GNumShadowmapUnmappedTexels) / NumShadowmapTotalTexels
			);

		for ( int32 LevelIndex=0; LevelIndex < LightingContext.World->GetNumLevels(); LevelIndex++ )
		{
			ULevel* Level = LightingContext.World->GetLevel(LevelIndex);
			UE_LOGF(LogStaticLightingSystem, Log,  "Level %2d - Lightmaps: %.1f MB. Shadowmaps: %.1f MB.", LevelIndex, Level->LightmapTotalSize/1024.0f, Level->ShadowmapTotalSize/1024.0f );
		}
	}
	else	//if ( GLightmassStatsMode)
	{
		UE_LOGF(LogStaticLightingSystem, Log, "Illumination: %ls (%ls encoding lightmaps, %ls encoding shadowmaps)", *FPlatformTime::PrettyTime(LightmassStatistics.TotalTime), *FPlatformTime::PrettyTime(LightmassStatistics.EncodingLightmapsTime), *FPlatformTime::PrettyTime(LightmassStatistics.EncodingShadowMapsTime));
	}
	UE_LOGF(LogStaticLightingSystem, Log, "Lightmap texture memory:  %.1f MB (%.1f MB streaming, %.1f MB non-streaming), %d textures",
		GLightmapTotalSize/1024.0f/1024.0f,
		GLightmapTotalStreamingSize/1024.0f/1024.0f,
		(GLightmapTotalSize - GLightmapTotalStreamingSize)/1024.0f/1024.0f,
		GNumLightmapTextures);

	UE_LOGF(LogStaticLightingSystem, Log, "Shadowmap texture memory: %.1f MB (%.1f MB streaming, %.1f MB non-streaming), %d textures",
		GShadowmapTotalSize/1024.0f/1024.0f,
		GShadowmapTotalStreamingSize/1024.0f/1024.0f,
		(GShadowmapTotalSize - GShadowmapTotalStreamingSize)/1024.0f/1024.0f,
		GNumShadowmapTextures);
}

void FStaticLightingSystem::CompleteDeterministicMappings(class FLightmassProcessor* InLightmassProcessor)
{
	check(InLightmassProcessor != NULL);
	if (InLightmassProcessor && GLightmassDebugOptions.bUseImmediateImport && GLightmassDebugOptions.bImmediateProcessMappings)
	{
		// Already completed in the Lightmass Run function...
		return;
	}

	double ImportAndApplyStartTime = FPlatformTime::Seconds();
	double ApplyTime = 0.0;

	int32 CurrentStep = Mappings.Num();
	int32 TotalSteps = Mappings.Num() * 2;
	const int32 ProgressUpdateFrequency = FMath::Max<int32>(TotalSteps / 20, 1);
	GWarn->StatusUpdate( CurrentStep, TotalSteps, LOCTEXT("CompleteDeterministicMappingsStatusMessage", "Importing and applying deterministic mappings...") );

	// Process all the texture mappings first...
	for (int32 MappingIndex = 0; MappingIndex < Mappings.Num(); MappingIndex++)
	{
		FStaticLightingTextureMapping* TextureMapping = Mappings[MappingIndex]->GetTextureMapping();
		if (TextureMapping)
		{
			//UE_LOGF(LogStaticLightingSystem, Log, "%32ls Completed - %ls", *(TextureMapping->GetDescription()), *(TextureMapping->GetLightingGuid().ToString()));

			if (!GLightmassDebugOptions.bUseImmediateImport)
			{
				InLightmassProcessor->ImportMapping(TextureMapping->GetLightingGuid(), true);
			}
			else
			{
				double ApplyStartTime = FPlatformTime::Seconds();
				InLightmassProcessor->ProcessMapping(TextureMapping->GetLightingGuid());
				ApplyTime += FPlatformTime::Seconds() - ApplyStartTime;
			}
		}

		CurrentStep++;

		if (CurrentStep % ProgressUpdateFrequency == 0)
		{
			GWarn->UpdateProgress(CurrentStep , TotalSteps);
		}
	}

	LightmassStatistics.ImportTimeInProcessing += FPlatformTime::Seconds() - ImportAndApplyStartTime - ApplyTime;
	LightmassStatistics.ApplyTimeInProcessing += ApplyTime;
}

struct FCompareByArrayCount
{
	FORCEINLINE bool operator()( const TArray<ULightComponent*>& A, const TArray<ULightComponent*>& B ) const 
	{ 
		// Sort by descending array count
		return B.Num() < A.Num(); 
	}
};

/**
 * Generates mappings/meshes for all BSP in the given level
 *
 * @param Level Level to build BSP lighting info for
 * @param bBuildLightingForBSP If true, we need BSP mappings generated as well as the meshes
 */
void FStaticLightingSystem::AddBSPStaticLightingInfo(ULevel* Level, bool bBuildLightingForBSP)
{
	// For BSP, we aren't Component-centric, so we can't use the GetStaticLightingInfo 
	// function effectively. Instead, we look across all nodes in the Level's model and
	// generate NodeGroups - which are groups of nodes that are coplanar, adjacent, and 
	// have the same lightmap resolution (henceforth known as being "conodes"). Each 
	// NodeGroup will get a mapping created for it

	// cache the model
	UModel* Model = Level->Model;

	// reset the number of incomplete groups
	Model->NumIncompleteNodeGroups = 0;
	Model->CachedMappings.Empty();
	Model->bInvalidForStaticLighting = false;

	// create all NodeGroups
	Model->GroupAllNodes(Level, Lights);

	TSet<UPackage*> PackagesToDirty;

	// now we need to make the mappings/meshes
	for (TMap<int32, FNodeGroup*>::TIterator It(Model->NodeGroups); It; ++It)
	{
		FNodeGroup* NodeGroup = It.Value();

		if (NodeGroup->Nodes.Num() && Level->ModelComponents.Num())
		{
			// get one of the surfaces/components from the NodeGroup
			// @todo: Remove need for GetSurfaceLightMapResolution to take a surfaceindex, or a ModelComponent :)
			UModelComponent* SomeModelComponent = Level->ModelComponents[Model->Nodes[NodeGroup->Nodes[0]].ComponentIndex];
			int32 SurfaceIndex = Model->Nodes[NodeGroup->Nodes[0]].iSurf;

			// fill out the NodeGroup/mapping, as UModelComponent::GetStaticLightingInfo did
			SomeModelComponent->GetSurfaceLightMapResolution(SurfaceIndex, true, NodeGroup->SizeX, NodeGroup->SizeY, NodeGroup->WorldToMap, &NodeGroup->Nodes);

			// Make sure mapping will have valid size
			NodeGroup->SizeX = FMath::Max(NodeGroup->SizeX, 1);
			NodeGroup->SizeY = FMath::Max(NodeGroup->SizeY, 1);

			NodeGroup->MapToWorld = NodeGroup->WorldToMap.InverseFast();

			// Cache the surface's vertices and triangles.
			NodeGroup->BoundingBox.Init();

			TArray<int32> ComponentVisibilityIds;
			for(int32 NodeIndex = 0;NodeIndex < NodeGroup->Nodes.Num();NodeIndex++)
			{
				const FBspNode& Node = Model->Nodes[NodeGroup->Nodes[NodeIndex]];
				const FBspSurf& NodeSurf = Model->Surfs[Node.iSurf];
				const FVector& TextureBase = (FVector)Model->Points[NodeSurf.pBase];
				const FVector& TextureX = (FVector)Model->Vectors[NodeSurf.vTextureU];
				const FVector& TextureY = (FVector)Model->Vectors[NodeSurf.vTextureV];
				const int32 BaseVertexIndex = NodeGroup->Vertices.Num();
				// Compute the surface's tangent basis.
				FVector NodeTangentX = (FVector)Model->Vectors[NodeSurf.vTextureU].GetSafeNormal();
				FVector NodeTangentY = (FVector)Model->Vectors[NodeSurf.vTextureV].GetSafeNormal();
				FVector NodeTangentZ = (FVector)Model->Vectors[NodeSurf.vNormal].GetSafeNormal();

				// Generate the node's vertices.
				for(uint32 VertexIndex = 0;VertexIndex < Node.NumVertices;VertexIndex++)
				{
					const FVert& Vert = Model->Verts[Node.iVertPool + VertexIndex];
					const FVector& VertexWorldPosition = (FVector)Model->Points[Vert.pVertex];

					FStaticLightingVertex* DestVertex = new(NodeGroup->Vertices) FStaticLightingVertex;
					DestVertex->WorldPosition = VertexWorldPosition;
					DestVertex->TextureCoordinates[0].X = ((VertexWorldPosition - TextureBase) | TextureX) / UModel::GetGlobalBSPTexelScale();
					DestVertex->TextureCoordinates[0].Y = ((VertexWorldPosition - TextureBase) | TextureY) / UModel::GetGlobalBSPTexelScale();
					DestVertex->TextureCoordinates[1].X = NodeGroup->WorldToMap.TransformPosition(VertexWorldPosition).X;
					DestVertex->TextureCoordinates[1].Y = NodeGroup->WorldToMap.TransformPosition(VertexWorldPosition).Y;
					DestVertex->WorldTangentX = NodeTangentX;
					DestVertex->WorldTangentY = NodeTangentY;
					DestVertex->WorldTangentZ = NodeTangentZ;

					// Include the vertex in the surface's bounding box.
					NodeGroup->BoundingBox += VertexWorldPosition;
				}

				// Generate the node's vertex indices.
				for(uint32 VertexIndex = 2;VertexIndex < Node.NumVertices;VertexIndex++)
				{
					NodeGroup->TriangleVertexIndices.Add(BaseVertexIndex + 0);
					NodeGroup->TriangleVertexIndices.Add(BaseVertexIndex + VertexIndex);
					NodeGroup->TriangleVertexIndices.Add(BaseVertexIndex + VertexIndex - 1);

					// track the source surface for each triangle
					NodeGroup->TriangleSurfaceMap.Add(Node.iSurf);
				}

				UModelComponent* Component = Level->ModelComponents[Node.ComponentIndex];
				if (Component->VisibilityId == INDEX_NONE)
				{
					if (LightingContext.World->GetWorldSettings()->bPrecomputeVisibility)
					{
						// Make sure packages gets dirtied since we are changing the visibility Id of a component in them
						PackagesToDirty.Add(Component->GetPackage());
					}
					Component->VisibilityId = NextVisibilityId;
					NextVisibilityId++;
				}
				ComponentVisibilityIds.AddUnique(Component->VisibilityId);
			}

			// Continue only if the component accepts lights (all components in a node group have the same value)
			// TODO: If we expose CastShadow for BSP in the future, reenable this condition and make sure
			// node grouping logic is updated to account for CastShadow as well
			//if (SomeModelComponent->bAcceptsLights || SomeModelComponent->CastShadow)
			{
				// Create the object to represent the surface's mapping/mesh to the static lighting system,
				// the model is now the owner, and all nodes have the same 
				FBSPSurfaceStaticLighting* SurfaceStaticLighting = new FBSPSurfaceStaticLighting(NodeGroup, Model, SomeModelComponent);
				// Give the surface mapping the visibility Id's of all components that have nodes in it
				// This results in fairly ineffective precomputed visibility with BSP but is necessary since BSP mappings contain geometry from multiple components
				SurfaceStaticLighting->VisibilityIds = ComponentVisibilityIds;

				Meshes.Add(SurfaceStaticLighting);
				LightingMeshBounds += SurfaceStaticLighting->BoundingBox;

				if (SomeModelComponent->CastShadow)
				{
					UpdateAutomaticImportanceVolumeBounds( SurfaceStaticLighting->BoundingBox );
				}

				FStaticLightingMapping* CurrentMapping = SurfaceStaticLighting;
				if (GLightmassDebugOptions.bSortMappings)
				{
					int32 InsertIndex = UnSortedMappings.AddZeroed();
					FStaticLightingMappingSortHelper& Helper = UnSortedMappings[InsertIndex];
					Helper.Mapping = CurrentMapping;
					Helper.NumTexels = CurrentMapping->GetTexelCount();
				}
				else
				{
					Mappings.Add(CurrentMapping);
					if (bBuildLightingForBSP)
					{
						CurrentMapping->Mesh->Guid = FGuid(0,0,0,DeterministicIndex++);
					}
				}

				if (bBuildLightingForBSP)
				{
					CurrentMapping->bProcessMapping = true;
				}

				// count how many node groups have yet to come back as complete
				Model->NumIncompleteNodeGroups++;

				// add this mapping to the list of mappings to be applied later
				Model->CachedMappings.Add(SurfaceStaticLighting);
			}
		}
	}

	// Mark package(s) as dirty
	for (UPackage* Package : PackagesToDirty)
	{
		Package->MarkPackageDirty();
	}
}

/**
 * Generates mappings/meshes for the given NodeGroups
 *
 * @param Level					Level to build BSP lighting info for
 * @param NodeGroupsToBuild		The node groups to build the BSP lighting info for
 */
void FStaticLightingSystem::AddBSPStaticLightingInfo(ULevel* Level, TArray<FNodeGroup*>& NodeGroupsToBuild)
{
	// For BSP, we aren't Component-centric, so we can't use the GetStaticLightingInfo 
	// function effectively. Instead, we look across all nodes in the Level's model and
	// generate NodeGroups - which are groups of nodes that are coplanar, adjacent, and 
	// have the same lightmap resolution (henceforth known as being "conodes"). Each 
	// NodeGroup will get a mapping created for it

	// cache the model
	UModel* Model = Level->Model;

	// reset the number of incomplete groups
	Model->NumIncompleteNodeGroups = 0;
	Model->CachedMappings.Empty();
	Model->bInvalidForStaticLighting = false;

	// now we need to make the mappings/meshes
	for (int32 NodeGroupIdx = 0; NodeGroupIdx < NodeGroupsToBuild.Num(); NodeGroupIdx++)
	{
		FNodeGroup* NodeGroup = NodeGroupsToBuild[NodeGroupIdx];
		if (NodeGroup && NodeGroup->Nodes.Num())
		{
			// get one of the surfaces/components from the NodeGroup
			// @todo: Remove need for GetSurfaceLightMapResolution to take a surfaceindex, or a ModelComponent :)
			UModelComponent* SomeModelComponent = Level->ModelComponents[Model->Nodes[NodeGroup->Nodes[0]].ComponentIndex];
			int32 SurfaceIndex = Model->Nodes[NodeGroup->Nodes[0]].iSurf;

			// fill out the NodeGroup/mapping, as UModelComponent::GetStaticLightingInfo did
			SomeModelComponent->GetSurfaceLightMapResolution(SurfaceIndex, true, NodeGroup->SizeX, NodeGroup->SizeY, NodeGroup->WorldToMap, &NodeGroup->Nodes);
			NodeGroup->MapToWorld = NodeGroup->WorldToMap.InverseFast();

			// Cache the surface's vertices and triangles.
			NodeGroup->BoundingBox.Init();

			for(int32 NodeIndex = 0;NodeIndex < NodeGroup->Nodes.Num();NodeIndex++)
			{
				const FBspNode& Node = Model->Nodes[NodeGroup->Nodes[NodeIndex]];
				const FBspSurf& NodeSurf = Model->Surfs[Node.iSurf];
				const FVector& TextureBase = (FVector)Model->Points[NodeSurf.pBase];
				const FVector& TextureX = (FVector)Model->Vectors[NodeSurf.vTextureU];
				const FVector& TextureY = (FVector)Model->Vectors[NodeSurf.vTextureV];
				const int32 BaseVertexIndex = NodeGroup->Vertices.Num();
				// Compute the surface's tangent basis.
				FVector NodeTangentX = (FVector)Model->Vectors[NodeSurf.vTextureU].GetSafeNormal();
				FVector NodeTangentY = (FVector)Model->Vectors[NodeSurf.vTextureV].GetSafeNormal();
				FVector NodeTangentZ = (FVector)Model->Vectors[NodeSurf.vNormal].GetSafeNormal();

				// Generate the node's vertices.
				for(uint32 VertexIndex = 0;VertexIndex < Node.NumVertices;VertexIndex++)
				{
					const FVert& Vert = Model->Verts[Node.iVertPool + VertexIndex];
					const FVector& VertexWorldPosition = (FVector)Model->Points[Vert.pVertex];

					FStaticLightingVertex* DestVertex = new(NodeGroup->Vertices) FStaticLightingVertex;
					DestVertex->WorldPosition = VertexWorldPosition;
					DestVertex->TextureCoordinates[0].X = ((VertexWorldPosition - TextureBase) | TextureX) / UModel::GetGlobalBSPTexelScale();
					DestVertex->TextureCoordinates[0].Y = ((VertexWorldPosition - TextureBase) | TextureY) / UModel::GetGlobalBSPTexelScale();
					DestVertex->TextureCoordinates[1].X = NodeGroup->WorldToMap.TransformPosition(VertexWorldPosition).X;
					DestVertex->TextureCoordinates[1].Y = NodeGroup->WorldToMap.TransformPosition(VertexWorldPosition).Y;
					DestVertex->WorldTangentX = NodeTangentX;
					DestVertex->WorldTangentY = NodeTangentY;
					DestVertex->WorldTangentZ = NodeTangentZ;

					// Include the vertex in the surface's bounding box.
					NodeGroup->BoundingBox += VertexWorldPosition;
				}

				// Generate the node's vertex indices.
				for(uint32 VertexIndex = 2;VertexIndex < Node.NumVertices;VertexIndex++)
				{
					NodeGroup->TriangleVertexIndices.Add(BaseVertexIndex + 0);
					NodeGroup->TriangleVertexIndices.Add(BaseVertexIndex + VertexIndex);
					NodeGroup->TriangleVertexIndices.Add(BaseVertexIndex + VertexIndex - 1);

					// track the source surface for each triangle
					NodeGroup->TriangleSurfaceMap.Add(Node.iSurf);
				}
			}

			// Continue only if the component accepts lights (all components in a node group have the same value)
			// TODO: If we expose CastShadow for BSP in the future, reenable this condition and make sure
			// node grouping logic is updated to account for CastShadow as well
			//if (SomeModelComponent->bAcceptsLights || SomeModelComponent->CastShadow)
			{
				// Create the object to represent the surface's mapping/mesh to the static lighting system,
				// the model is now the owner, and all nodes have the same 
				FBSPSurfaceStaticLighting* SurfaceStaticLighting = new FBSPSurfaceStaticLighting(NodeGroup, Model, SomeModelComponent);
				Meshes.Add(SurfaceStaticLighting);
				LightingMeshBounds += SurfaceStaticLighting->BoundingBox;

				if (SomeModelComponent->CastShadow)
				{
					UpdateAutomaticImportanceVolumeBounds( SurfaceStaticLighting->BoundingBox );
				}
				
				FStaticLightingMapping* CurrentMapping = SurfaceStaticLighting;
				if (GLightmassDebugOptions.bSortMappings)
				{
					int32 InsertIndex = UnSortedMappings.AddZeroed();
					FStaticLightingMappingSortHelper& Helper = UnSortedMappings[InsertIndex];
					Helper.Mapping = CurrentMapping;
					Helper.NumTexels = CurrentMapping->GetTexelCount();
				}
				else
				{
					Mappings.Add(CurrentMapping);
					CurrentMapping->Mesh->Guid = FGuid(0,0,0,DeterministicIndex++);
				}

				CurrentMapping->bProcessMapping = true;

				// count how many node groups have yet to come back as complete
				Model->NumIncompleteNodeGroups++;

				// add this mapping to the list of mappings to be applied later
				Model->CachedMappings.Add(SurfaceStaticLighting);
			}
		}
	}
}

void FStaticLightingSystem::AddPrimitiveStaticLightingInfo(FStaticLightingPrimitiveInfo& PrimitiveInfo, bool bBuildActorLighting, bool bDeferMapping)
{
	// Verify a one to one relationship between mappings and meshes
	//@todo - merge FStaticLightingMesh and FStaticLightingMapping
	check(PrimitiveInfo.Meshes.Num() == PrimitiveInfo.Mappings.Num());

	// Add the component's shadow casting meshes to the system.
	for(int32 MeshIndex = 0;MeshIndex < PrimitiveInfo.Meshes.Num();MeshIndex++)
	{
		FStaticLightingMesh* Mesh = PrimitiveInfo.Meshes[MeshIndex];
		if (Mesh)
		{
			Mesh->VisibilityIds.Add(PrimitiveInfo.VisibilityId);
			if (!GLightmassDebugOptions.bSortMappings && bBuildActorLighting)
			{
				Mesh->Guid = FGuid(0, 0, 0, DeterministicIndex++);
			}
			Meshes.Add(Mesh);
			LightingMeshBounds += Mesh->BoundingBox;

			if (Mesh->bCastShadow)
			{
				UpdateAutomaticImportanceVolumeBounds(Mesh->BoundingBox);
			}
		}
	}

	// If lighting is being built for this component, add its mappings to the system.
	for(int32 MappingIndex = 0;MappingIndex < PrimitiveInfo.Mappings.Num();MappingIndex++)
	{
		FStaticLightingMapping* CurrentMapping = PrimitiveInfo.Mappings[MappingIndex];
		if (GbLogAddingMappings)
		{
			FStaticLightingMesh* SLMesh = CurrentMapping->Mesh;
			if (SLMesh)
			{
				//UE_LOGF(LogStaticLightingSystem, Log, "Adding %32ls: 0x%08p - %ls", *(CurrentMapping->GetDescription()), (PTRINT)(SLMesh->Component), *(SLMesh->Guid.ToString()));
			}
			else
			{
				//UE_LOGF(LogStaticLightingSystem, Log, "Adding %32ls: 0x%08x - %ls", *(CurrentMapping->GetDescription()), 0, TEXT("NO MESH????"));
			}
		}

		if (bBuildActorLighting)
		{
			CurrentMapping->bProcessMapping = true;
		}

		CurrentMapping->bIsDeferred = bDeferMapping;

		if (GLightmassDebugOptions.bSortMappings)
		{
			int32 InsertIndex = UnSortedMappings.AddZeroed();
			FStaticLightingMappingSortHelper& Helper = UnSortedMappings[InsertIndex];
			Helper.Mapping = CurrentMapping;
			Helper.NumTexels = Helper.Mapping->GetTexelCount();
		}
		else
		{
			Mappings.Add(CurrentMapping);
		}
	}
}

bool FStaticLightingSystem::CreateLightmassProcessor()
{
	FLightmassStatistics::FScopedGather SwarmStartStatScope(LightmassProcessStatistics.SwarmStartupTime);
	
	GWarn->StatusForceUpdate( -1, -1, LOCTEXT("StartingSwarmConnectionStatus", "Starting up Swarm Connection...") );
	
	if (Options.bOnlyBuildVisibility && !LightingContext.World->GetWorldSettings()->bPrecomputeVisibility)
	{
		FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "BuildFailed_VisibilityOnlyButVisibilityDisabled", "'Build Only Visibility' option was enabled but precomputed visibility is disabled!  Aborting build."));
		return false;
	}

	if (NSwarm::FSwarmInterface::Initialize(*(FPaths::EngineDir() / TEXT("Binaries/DotNET/SwarmInterface.dll"))) == false)
	{
		UE_LOGF(LogStaticLightingSystem, Warning, "Failed to initialize Swarm.");
		FMessageDialog::Open(EAppMsgType::Ok,
#if USE_LOCAL_SWARM_INTERFACE
			LOCTEXT("FailedToInitializeSwarmDialogMessage_CheckNetwork", "Failed to initialize Swarm. Check to make sure you have the right version of Swarm installed.")
#else
			LOCTEXT("FailedToInitializeSwarmDialogMessage", "Failed to initialize Swarm. Check to make sure you have the right version of Swarm installed.")
#endif	
		);

		return false;
	}

	// Create the processor
	check(LightmassProcessor == NULL);
	LightmassProcessor = new FLightmassProcessor(*this, Options.bDumpBinaryResults, Options.bOnlyBuildVisibility);
	check(LightmassProcessor);

	if (LightingContext.World->IsPartitionedWorld())
	{
		LightmassProcessor->SetVolumetricLightMapImportMode(true);
	}	

	if (LightmassProcessor->IsSwarmConnectionIsValid() == false)
	{
		UE_LOGF(LogStaticLightingSystem, Warning, "Failed to connect to Swarm.");
		FMessageDialog::Open( EAppMsgType::Ok, 
#if USE_LOCAL_SWARM_INTERFACE
			LOCTEXT("FailedToConnectToSwarmDialogMessage_CheckNetwork", "Failed to connect to Swarm. Check that your network interface supports multicast.")
#else
			LOCTEXT("FailedToConnectToSwarmDialogMessage", "Failed to connect to Swarm.")
#endif	
		);
		delete LightmassProcessor;
		LightmassProcessor = NULL;
		return false;
	}

	return true;
}

void FStaticLightingSystem::GatherScene()
{
	LightmassProcessStatistics = FLightmassStatistics();

	GWarn->StatusUpdate( 0, Meshes.Num() + Mappings.Num(), LOCTEXT("GatherSceneStatusMessage", "Collecting the scene...") );
	
	FLightmassStatistics::FScopedGather SceneStatScope(LightmassProcessStatistics.CollectLightmassSceneTime);

	// Grab the exporter and fill in the meshes
	//@todo. This should be exported to the 'processor' as it will be used on the input side as well...
	FLightmassExporter* LightmassExporter = LightmassProcessor->GetLightmassExporter();
	check(LightmassExporter);

	// The Level settings...
	AWorldSettings* WorldSettings = LightingContext.World->GetWorldSettings();
	if (WorldSettings)
	{
		LightmassExporter->SetLevelSettings(WorldSettings->LightmassSettings);
	}
	else
	{
		FLightmassWorldInfoSettings TempSettings;
		LightmassExporter->SetLevelSettings(TempSettings);
	}
	LightmassExporter->SetNumUnusedLocalCores(Options.NumUnusedLocalCores);
	LightmassExporter->SetQualityLevel(Options.QualityLevel);

	if (LightingContext.World->PersistentLevel && Options.ShouldBuildLightingForLevel(LightingContext.World->PersistentLevel ))
	{
		LightmassExporter->SetLevelName(LightingContext.World->PersistentLevel->GetPathName());
	}
	
	LightmassExporter->ClearImportanceVolumes();

	if (!LightingContext.World->IsPartitionedWorld())
	{
		for( TObjectIterator<ALightmassImportanceVolume> It ; It ; ++It )
		{
			ALightmassImportanceVolume* LMIVolume = *It;
			if (LightingContext.World->ContainsActor(LMIVolume) && IsValid(LMIVolume) && ShouldOperateOnLevel(LMIVolume->GetLevel()))
			{
				LightmassExporter->AddImportanceVolume(LMIVolume);
			}			
		}
	}
	else
	{
		// Ignore all user set ImportanceVolumes and add a LightMassImportanceVolume per VLM cell 
		//@todo_ow: Possibly interesect user specified ALightmassImportanceVolume with world bounds to restrict area in which we compute static lighting
		FBox ImportanceBounds(ForceInit);
				
		LightmassExporter->AddImportanceVolumeBoundingBox(LightingContext.GetVolumetricLightMapGridDesc()->GridBounds);
		ImportanceBounds += LightingContext.GetVolumetricLightMapGridDesc()->GridBounds;

		LightingContext.SetImportanceBounds(ImportanceBounds);
	}

	for( TObjectIterator<ALightmassCharacterIndirectDetailVolume> It ; It ; ++It )
	{
		ALightmassCharacterIndirectDetailVolume* LMDetailVolume = *It;
		if (LightingContext.World->ContainsActor(LMDetailVolume) && IsValid(LMDetailVolume) && ShouldOperateOnLevel(LMDetailVolume->GetLevel()))
		{
			LightmassExporter->AddCharacterIndirectDetailVolume(LMDetailVolume);
		}
	}

	for (TObjectIterator<AVolumetricLightmapDensityVolume> It; It; ++It)
	{
		AVolumetricLightmapDensityVolume* DetailVolume = *It;
		if (LightingContext.World->ContainsActor(DetailVolume) && IsValid(DetailVolume) && ShouldOperateOnLevel(DetailVolume->GetLevel()))
		{
			LightmassExporter->VolumetricLightmapDensityVolumes.Add(DetailVolume);
		}
	}

	for( TObjectIterator<ULightmassPortalComponent> It ; It ; ++It )
	{
		ULightmassPortalComponent* LMPortal = *It;
		if (LMPortal->GetOwner() && LightingContext.World->ContainsActor(LMPortal->GetOwner()) && IsValid(LMPortal) && ShouldOperateOnLevel(LMPortal->GetOwner()->GetLevel()))
		{
			LightmassExporter->AddPortal(LMPortal);
		}
	}

	for (TObjectIterator<USkyAtmosphereComponent> It; It; ++It)
	{
		USkyAtmosphereComponent* SkyAtmosphere = *It;
		if (SkyAtmosphere->GetOwner() && LightingContext.World->ContainsActor(SkyAtmosphere->GetOwner()) && IsValid(SkyAtmosphere) && ShouldOperateOnLevel(SkyAtmosphere->GetOwner()->GetLevel()))
		{
			LightmassExporter->SetSkyAtmosphereComponent(SkyAtmosphere);
			break;	// We only register the first we find
		}
	}

	float MinimumImportanceVolumeExtentWithoutWarning = 0.0f;
	verify(GConfig->GetFloat(TEXT("DevOptions.StaticLightingSceneConstants"), TEXT("MinimumImportanceVolumeExtentWithoutWarning"), MinimumImportanceVolumeExtentWithoutWarning, GLightmassIni));

	// If we have no importance volumes, then we'll synthesize one now.  A scene without any importance volumes will not yield
	// expected lighting results, so it's important to have a volume to pass to Lightmass.
	if (LightmassExporter->GetImportanceVolumes().Num() == 0)
	{
		FBox ReasonableSceneBounds = AutomaticImportanceVolumeBounds;
		if (ReasonableSceneBounds.GetExtent().SizeSquared() > (MinimumImportanceVolumeExtentWithoutWarning * MinimumImportanceVolumeExtentWithoutWarning))
		{
			// Emit a serious warning to the user about performance.
			FMessageLog("LightingResults").PerformanceWarning(LOCTEXT("LightmassError_MissingImportanceVolume", "No importance volume found and the scene is so large that the automatically synthesized volume will not yield good results.  Please add a tightly bounding lightmass importance volume to optimize your scene's quality and lighting build times."));

			// Clamp the size of the importance volume we create to a reasonable size
			ReasonableSceneBounds = FBox(ReasonableSceneBounds.GetCenter() - MinimumImportanceVolumeExtentWithoutWarning, ReasonableSceneBounds.GetCenter() + MinimumImportanceVolumeExtentWithoutWarning);
		}
		else
		{
			// The scene isn't too big, so we'll use the scene's bounds as a synthetic importance volume
			// NOTE: We don't want to pop up a message log for this common case when creating a new level, so we just spray a log message.  It's not very important to a user.
			UE_LOGF(LogStaticLightingSystem, Warning, "No importance volume found, so the scene bounding box was used.  You can optimize your scene's quality and lighting build times by adding importance volumes.");

			float AutomaticImportanceVolumeExpandBy = 0.0f;
			verify(GConfig->GetFloat(TEXT("DevOptions.StaticLightingSceneConstants"), TEXT("AutomaticImportanceVolumeExpandBy"), AutomaticImportanceVolumeExpandBy, GLightmassIni));

			// Expand the scene's bounds a bit to make sure volume lighting samples placed on surfaces are inside
			ReasonableSceneBounds = ReasonableSceneBounds.ExpandBy(AutomaticImportanceVolumeExpandBy);
		}

		LightmassExporter->AddImportanceVolumeBoundingBox(ReasonableSceneBounds);
	}

	const int32 NumMeshesAndMappings = Meshes.Num() + Mappings.Num();
	const int32 ProgressUpdateFrequency = FMath::Max<int32>(NumMeshesAndMappings / 20, 1);

	// Meshes
	for( int32 MeshIdx=0; !GEditor->GetMapBuildCancelled() && MeshIdx < Meshes.Num(); MeshIdx++ )
	{
		Meshes[MeshIdx]->ExportMeshInstance(LightmassExporter);

		if (MeshIdx % ProgressUpdateFrequency == 0)
		{
			GWarn->UpdateProgress( MeshIdx, NumMeshesAndMappings );
		}
	}

	// Mappings
	for( int32 MappingIdx=0; !GEditor->GetMapBuildCancelled() && MappingIdx < Mappings.Num(); MappingIdx++ )
	{
		Mappings[MappingIdx]->ExportMapping(LightmassExporter);

		if (MappingIdx % ProgressUpdateFrequency == 0)
		{
			GWarn->UpdateProgress( Meshes.Num() + MappingIdx, NumMeshesAndMappings );
		}
	}

	for (int32 LightIndex = 0; LightIndex < Lights.Num(); LightIndex++)
	{
		ULightComponentBase* LightBase = Lights[LightIndex];
		USkyLightComponent* SkyLight = Cast<USkyLightComponent>(LightBase);

		if (SkyLight && (SkyLight->Mobility == EComponentMobility::Static || SkyLight->Mobility == EComponentMobility::Stationary))
		{
			LightmassExporter->AddLight(SkyLight);
		}
	}
}
	
bool FStaticLightingSystem::InitiateLightmassProcessor()
{
	// Run!
	bool bSuccessful = false;
	bool bOpenJobSuccessful = false;
	if ( !GEditor->GetMapBuildCancelled() )
	{
		UE_LOGF(LogStaticLightingSystem, Log, "Running Lightmass w/ ImmediateImport mode %ls", GLightmassDebugOptions.bUseImmediateImport ? TEXT("ENABLED") : TEXT("DISABLED"));
		LightmassProcessor->SetImportCompletedMappingsImmediately(GLightmassDebugOptions.bUseImmediateImport);
		UE_LOGF(LogStaticLightingSystem, Log, "Running Lightmass w/ ImmediateProcess mode %ls", GLightmassDebugOptions.bImmediateProcessMappings ? TEXT("ENABLED") : TEXT("DISABLED"));
		UE_LOGF(LogStaticLightingSystem, Log, "Running Lightmass w/ Sorting mode %ls", GLightmassDebugOptions.bSortMappings ? TEXT("ENABLED") : TEXT("DISABLED"));
		UE_LOGF(LogStaticLightingSystem, Log, "Running Lightmass w/ Mapping paddings %ls", GLightmassDebugOptions.bPadMappings ? TEXT("ENABLED") : TEXT("DISABLED"));
		UE_LOGF(LogStaticLightingSystem, Log, "Running Lightmass w/ Mapping debug paddings %ls", GLightmassDebugOptions.bDebugPaddings ? TEXT("ENABLED") : TEXT("DISABLED"));

		{
			FLightmassStatistics::FScopedGather OpenJobStatScope(LightmassProcessStatistics.SwarmJobOpenTime);
			bOpenJobSuccessful = LightmassProcessor->OpenJob();
		}

		if (bOpenJobSuccessful)
		{
			LightmassProcessor->InitiateExport();
			bSuccessful = true;
			CurrentBuildStage = FStaticLightingSystem::LightingStage::AmortizedExport;
		}
	}
	
	return bSuccessful;
}

void FStaticLightingSystem::KickoffSwarm()
{
	bool bSuccessful = LightmassProcessor->BeginRun();
	
	if (bSuccessful)
	{
		CurrentBuildStage = FStaticLightingSystem::LightingStage::AsynchronousBuilding;
	}
	else
	{
		FStaticLightingManager::Get()->FailLightingBuild(LOCTEXT("SwarmKickoffFailedMessage", "Lighting build failed. Swarm failed to kick off.  Compile Unreal Lightmass."));
	}
}

bool FStaticLightingSystem::FinishLightmassProcess()
{
	bool bSuccessful = false;

	GEditor->ResetTransaction( LOCTEXT("KeepLightingTransReset", "Applying Lighting") );

	CurrentBuildStage = FStaticLightingSystem::LightingStage::Import;

	double TimeWaitingOnUserToAccept = FPlatformTime::Seconds() - WaitForUserAcceptStartTime;
	
	{
		FScopedSlowTask SlowTask(7);
		SlowTask.MakeDialog();

		SlowTask.EnterProgressFrame(1, LOCTEXT("InvalidatingPreviousLightingStatus", "Invalidating previous lighting"));
		InvalidateStaticLighting();
	
		SlowTask.EnterProgressFrame(1, LOCTEXT("ImportingBuiltStaticLightingStatus", "Importing built static lighting"));
		bSuccessful = LightmassProcessor->CompleteRun();

		SlowTask.EnterProgressFrame();
		if (bSuccessful)
		{
			CompleteDeterministicMappings(LightmassProcessor);
		
			if (!Options.bOnlyBuildVisibility)
			{
				FLightmassStatistics::FScopedGather FinishStatScope(LightmassStatistics.FinishingTime);
				ULightComponent::ReassignStationaryLightChannels(GWorld, true, &LightingContext);
			}
		}
	

		SlowTask.EnterProgressFrame(1, LOCTEXT("EncodingTexturesStaticLightingStatis", "Encoding textures"));
		EncodeTextures(bSuccessful);


		SlowTask.EnterProgressFrame();
		{
			FLightmassStatistics::FScopedGather CloseJobStatScope(LightmassProcessStatistics.SwarmJobCloseTime);
			bSuccessful = LightmassProcessor->CloseJob() && bSuccessful;
		}

		{
			FLightmassStatistics::FScopedGather FinishStatScope(LightmassStatistics.FinishingTime);
			// Add in the time measurements from the LightmassProcessor
			LightmassStatistics += LightmassProcessor->GetStatistics();

			// A final update on the lighting build warnings and errors dialog, now that everything is finished
			FMessageLog("LightingResults").Open();

			// Check the for build cancellation.
			bBuildCanceled = bBuildCanceled || GEditor->GetMapBuildCancelled();
			bSuccessful = bSuccessful && !bBuildCanceled;

			FStatsViewerModule& StatsViewerModule = FModuleManager::Get().LoadModuleChecked<FStatsViewerModule>(TEXT("StatsViewer"));
			if (bSuccessful)
			{
				StatsViewerModule.GetPage(EStatsPage::LightingBuildInfo)->Refresh();
			}
		
			bool bShowLightingBuildInfo = false;
			GConfig->GetBool( TEXT("LightingBuildOptions"), TEXT("ShowLightingBuildInfo"), bShowLightingBuildInfo, GEditorPerProjectIni );
			if( bShowLightingBuildInfo )
			{
				StatsViewerModule.GetPage(EStatsPage::LightingBuildInfo)->Show();
			}
		}

		SlowTask.EnterProgressFrame();
		ApplyNewLightingData(bSuccessful);

		SlowTask.EnterProgressFrame();

		// Finish up timing statistics
		LightmassStatistics += LightmassProcessStatistics;
		LightmassStatistics.TotalTime += FPlatformTime::Seconds() - StartTime - TimeWaitingOnUserToAccept;
	}

	ReportStatistics();
	
	return bSuccessful;
}

void FStaticLightingSystem::UpdateLightingBuild()
{
	if (CurrentBuildStage == FStaticLightingSystem::LightingStage::AmortizedExport)
	{
		bool bCompleted = LightmassProcessor->ExecuteAmortizedMaterialExport();

		FFormatNamedArguments Args;
		Args.Add( TEXT("PercentDone"), FText::AsPercent( LightmassProcessor->GetAmortizedExportPercentDone() ) );
		FText Text = FText::Format( LOCTEXT("LightExportProgressMessage", "Exporting lighting data: {PercentDone} Done"), Args );

		FStaticLightingManager::Get()->SetNotificationText( Text );

		if (bCompleted)
		{
			CurrentBuildStage = FStaticLightingSystem::LightingStage::SwarmKickoff;
		}
	}
	else if (CurrentBuildStage == FStaticLightingSystem::LightingStage::SwarmKickoff)
	{
		FText Text = LOCTEXT("LightKickoffSwarmMessage", "Kicking off Swarm");
		FStaticLightingManager::Get()->SetNotificationText( Text );
		KickoffSwarm();
	}
	else if (CurrentBuildStage == FStaticLightingSystem::LightingStage::AsynchronousBuilding)
	{
		bool bFinished = LightmassProcessor->Update();
		
		FString ScenarioString;

		if (LightingContext.LightingScenario)
		{
			FString PackageName = FPackageName::GetShortName(LightingContext.LightingScenario->GetOutermost()->GetName());
			ScenarioString = FString(TEXT(" for ")) + PackageName;
		}

		FText Text = FText::Format(LOCTEXT("LightBuildProgressMessage", "Building lighting{0}:  {1}%"), FText::FromString(ScenarioString), FText::AsNumber(LightmassProcessor->GetAsyncPercentDone()));
		FStaticLightingManager::Get()->SetNotificationText( Text );

		if (bFinished)
		{
			LightmassStatistics.ProcessingTime += FPlatformTime::Seconds() - ProcessingStartTime;
			WaitForUserAcceptStartTime = FPlatformTime::Seconds();

			FStaticLightingManager::Get()->ClearCurrentNotification();

			if (LightmassProcessor->IsProcessingCompletedSuccessfully())
			{
				CurrentBuildStage = FStaticLightingSystem::LightingStage::AutoApplyingImport;
			}
			else
			{
				// automatically fail lighting build (discard)
				FStaticLightingManager::Get()->FailLightingBuild();
				CurrentBuildStage = FStaticLightingSystem::LightingStage::Finished;
			}
		}
	}
	else if ( CurrentBuildStage == FStaticLightingSystem::LightingStage::AutoApplyingImport )
	{
		if (IsRunningCommandlet() || CanAutoApplyLighting())
		{
			bool bAutoApplyFailed = false;
			FStaticLightingManager::Get()->SendBuildDoneNotification(bAutoApplyFailed);

			FStaticLightingManager::ProcessLightingData();
			CurrentBuildStage = FStaticLightingSystem::LightingStage::Finished;
		}
		else
		{
			bool bAutoApplyFailed = true;
			FStaticLightingManager::Get()->SendBuildDoneNotification(bAutoApplyFailed);

			CurrentBuildStage = FStaticLightingSystem::LightingStage::WaitingForImport;
		}
	}
	else if (CurrentBuildStage == FStaticLightingSystem::LightingStage::ImportRequested)
	{
		FStaticLightingManager::ProcessLightingData();
		CurrentBuildStage = FStaticLightingSystem::LightingStage::Finished;
	}
}

void FStaticLightingSystem::UpdateAutomaticImportanceVolumeBounds( const FBox& MeshBounds )
{
	// Note: skyboxes will be excluded if they are properly setup to not cast shadows
	AutomaticImportanceVolumeBounds += MeshBounds;
}

void FStaticLightingSystem::GatherBuildDataResourcesToKeep(const ULevel* InLevel)
{
	// This is only required is using a lighting scenario, otherwise the build data is saved within the level itself and follows it's inclusion in the lighting build.	
	if (InLevel && LightingContext.LightingScenario)
	{
		BuildDataResourcesToKeep.Add(InLevel->LevelBuildDataId);

		for (const UModelComponent * ModelComponent : InLevel->ModelComponents)
		{
			if (!ModelComponent) // Skip null models
			{
				continue;
			}
			ModelComponent->AddMapBuildDataGUIDs(BuildDataResourcesToKeep);
		}

		for (const AActor* Actor : InLevel->Actors)
		{
			if (!Actor) // Skip null actors
			{
				continue;
			}

			for (const UActorComponent* Component : Actor->GetComponents())
			{
				if (!Component) // Skip null components
				{
					continue;
				}

				const UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Component);
				if (PrimitiveComponent)
				{
					PrimitiveComponent->AddMapBuildDataGUIDs(BuildDataResourcesToKeep);
					continue;
				}

				const ULightComponent* LightComponent = Cast<ULightComponent>(Component);
				if (LightComponent)
				{
					BuildDataResourcesToKeep.Add(LightComponent->LightGuid);
					continue;
				}

				const UReflectionCaptureComponent* ReflectionCaptureComponent = Cast<UReflectionCaptureComponent>(Component);
				if (ReflectionCaptureComponent)
				{
					BuildDataResourcesToKeep.Add(ReflectionCaptureComponent->MapBuildDataId);
					continue;
				}
			}
		}
	}
}


bool FStaticLightingSystem::CanAutoApplyLighting() const
{
	const bool bAutoApplyEnabled = GetDefault<ULevelEditorMiscSettings>()->bAutoApplyLightingEnable;
	const bool bSlowTask = GIsSlowTask;
	const bool bPlayWorldValid = GEditor->PlayWorld != nullptr;
	const bool bAnyMenusVisible = (FSlateApplication::IsInitialized() && FSlateApplication::Get().AnyMenusVisible());
	//const bool bIsInteratcting = false;// FSlateApplication::Get().GetMouseCaptor().IsValid() || GEditor->IsUserInteracting();
	const bool bHasGameOrProjectLoaded = FApp::HasProjectName();

	return ( bAutoApplyEnabled && !bSlowTask && !bPlayWorldValid && !bAnyMenusVisible/* && !bIsInteratcting */&& !GIsDemoMode && bHasGameOrProjectLoaded );
}

/**
 * Clear out all the binary dump log files, so the next run will have just the needed files for rendering
 */
void FStaticLightingSystem::ClearBinaryDumps()
{
	IFileManager::Get().DeleteDirectory(*FString::Printf(TEXT("%sLogs/Lighting_%s"), *FPaths::ProjectDir(), TEXT("Lightmass")), false, true);
}

/** Marks all lights used in the calculated lightmap as used in a lightmap, and calls Apply on the texture mapping. */
void FStaticLightingSystem::ApplyMapping(
	FStaticLightingTextureMapping* TextureMapping,
	FQuantizedLightmapData* QuantizedData,
	const TMap<ULightComponent*,FShadowMapData2D*>& ShadowMapData) const
{
	TextureMapping->Apply(QuantizedData, ShadowMapData, &LightingContext);
}

UWorld* FStaticLightingSystem::GetWorld() const
{
	return LightingContext.World;
}

const FStaticLightingBuildContext& FStaticLightingSystem::GetLightingContext() const
{
	return LightingContext;
}

bool FStaticLightingSystem::IsAsyncBuilding() const
{
	return CurrentBuildStage == FStaticLightingSystem::LightingStage::AsynchronousBuilding;
}

bool FStaticLightingSystem::IsAmortizedExporting() const
{
	return CurrentBuildStage == FStaticLightingSystem::LightingStage::AmortizedExport;
}

void UEditorEngine::BuildLighting(const FLightingBuildOptions& Options)
{
	// Forcibly shut down all texture property windows as they become invalid during a light build
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	TArray<UObject*> EditedAssets = AssetEditorSubsystem->GetAllEditedAssets();

	for (int32 AssetIdx = 0; AssetIdx < EditedAssets.Num(); AssetIdx++)
	{
		UObject* EditedAsset = EditedAssets[AssetIdx];

		if (EditedAsset->IsA(UTexture2D::StaticClass()))
		{
			IAssetEditorInstance* Editor = AssetEditorSubsystem->FindEditorForAsset(EditedAsset, false);
			if (Editor)
			{
				Editor->CloseWindow(EAssetEditorCloseReason::AssetUnloadingOrInvalid);
			}
		}
	}
	
	FEditorDelegates::OnLightingBuildStarted.Broadcast();

	FStaticLightingManager::Get()->CreateStaticLightingSystem(Options);
}

void UEditorEngine::UpdateBuildLighting()
{
	FStaticLightingManager::Get()->UpdateBuildLighting();

	FStaticLightingSystemInterface::EditorTick();
}

bool UEditorEngine::IsLightingBuildCurrentlyRunning() const
{
	return FStaticLightingManager::Get()->IsLightingBuildCurrentlyRunning();
}

bool UEditorEngine::IsLightingBuildCurrentlyExporting() const
{
	return FStaticLightingManager::Get()->IsLightingBuildCurrentlyExporting(); 
}

bool UEditorEngine::WarnIfLightingBuildIsCurrentlyRunning()
{
	bool bFailure = IsLightingBuildCurrentlyRunning();
	if (bFailure)
	{
		FNotificationInfo Info( LOCTEXT("LightBuildUnderwayWarning", "Static light is currently building! Please cancel it to proceed!") );
		Info.ExpireDuration = 5.0f;
		TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
		if (Notification.IsValid())
		{
			Notification->SetCompletionState(SNotificationItem::CS_Fail);
		}
	}
	else if (FEditorBuildUtils::IsBuildCurrentlyRunning())
	{
		// Another, non-lighting editor build is running.
		FNotificationInfo Info( LOCTEXT("EditorBuildUnderwayWarning", "A build process is currently underway! Please cancel it to proceed!") );
		Info.ExpireDuration = 5.0f;
		TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
		if (Notification.IsValid())
		{
			Notification->SetCompletionState(SNotificationItem::CS_Fail);
		}

		bFailure = true;
	}
	return bFailure;
}

#undef LOCTEXT_NAMESPACE
