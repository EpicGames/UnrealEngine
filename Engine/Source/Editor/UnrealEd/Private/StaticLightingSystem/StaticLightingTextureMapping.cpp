// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	StaticLightingTextureMapping.cpp: Static lighting texture mapping implementation.
=============================================================================*/

#include "CoreMinimal.h"
#include "StaticLighting.h"
#include "StaticLightingSystem/StaticLightingPrivate.h"
#include "LightMap.h"

// Doxygen cannot not parse this definition since the declaration is in Engine, not UnrealEd
#if !UE_BUILD_DOCS
bool FStaticLightingTextureMapping::DebugThisMapping() const
{
	// This only works for mappings that have a one to one relationship with the associated component.
	// Other mapping types need to override this function.
	const bool bDebug = GCurrentSelectedLightmapSample.Component 
		&& GCurrentSelectedLightmapSample.Component == Mesh->Component
		// Only allow debugging if the lightmap resolution hasn't changed
		&& GCurrentSelectedLightmapSample.MappingSizeX == SizeX 
		&& GCurrentSelectedLightmapSample.MappingSizeY == SizeY;
	return bDebug;
}
#endif
