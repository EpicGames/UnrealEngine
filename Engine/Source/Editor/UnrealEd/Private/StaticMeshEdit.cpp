// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	StaticMeshEdit.cpp: Static mesh edit functions.
=============================================================================*/

#include "CoreMinimal.h"
#include "Misc/CoreMisc.h"
#include "Misc/FeedbackContext.h"
#include "CoreGlobals.h" // GUndo
#include "Engine/EngineTypes.h"
#include "Model.h"
#include "EditorFramework/AssetImportData.h"
#include "EditorFramework/ThumbnailInfo.h"
#include "Engine/StaticMesh.h"
#include "StaticMeshAttributes.h"
#include "Engine/StaticMeshSocket.h"
#include "Engine/Polys.h"
#include "Editor.h"
#include "StaticMeshResources.h"
#include "BSPOps.h"
#include "PhysicsEngine/ConvexElem.h"
#include "PhysicsEngine/BoxElem.h"
#include "PhysicsEngine/SphereElem.h"
#include "PhysicsEngine/BodySetup.h"
#include "FbxImporter.h"
#include "Misc/ScopedSlowTask.h"

#include "MaterialDomain.h"
#include "Materials/MaterialInterface.h"
#include "Materials/Material.h"
#include "UObject/PerPlatformProperties.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Interfaces/ITargetPlatform.h"
#include "Settings/EditorExperimentalSettings.h"

#include "Modules/ModuleManager.h"
#include "IMeshReductionManagerModule.h"
#include "IMeshReductionInterfaces.h"

#include "UObject/MetaData.h"

bool GBuildStaticMeshCollision = 1;

DEFINE_LOG_CATEGORY_STATIC(LogStaticMeshEdit, Log, All);

#define LOCTEXT_NAMESPACE "StaticMeshEdit"

/**
 * Creates a static mesh object from raw triangle data.
 */
UStaticMesh* CreateStaticMesh(FMeshDescription& RawMesh,TArray<FStaticMaterial>& Materials,UObject* InOuter,FName InName)
{
	// Create the UStaticMesh object.
	FStaticMeshComponentRecreateRenderStateContext RecreateRenderStateContext(FindObject<UStaticMesh>(InOuter,*InName.ToString()));
	auto StaticMesh = NewObject<UStaticMesh>(InOuter, InName, RF_Public | RF_Standalone);

	// Add one LOD for the base mesh
	StaticMesh->AddSourceModel();
	FMeshDescription* MeshDescription = StaticMesh->CreateMeshDescription(0, RawMesh);
	StaticMesh->CommitMeshDescription(0);
	StaticMesh->SetStaticMaterials(Materials);

	int32 NumSections = StaticMesh->GetStaticMaterials().Num();

	// Set up the SectionInfoMap to enable collision
	for (int32 SectionIdx = 0; SectionIdx < NumSections; ++SectionIdx)
	{
		FMeshSectionInfo Info = StaticMesh->GetSectionInfoMap().Get(0, SectionIdx);
		Info.MaterialIndex = SectionIdx;
		Info.bEnableCollision = true;
		StaticMesh->GetSectionInfoMap().Set(0, SectionIdx, Info);
		StaticMesh->GetOriginalSectionInfoMap().Set(0, SectionIdx, Info);
	}

	//Set the Imported version before calling the build
	StaticMesh->SetImportVersion(EImportStaticMeshVersion::LastVersion);

	StaticMesh->Build();
	StaticMesh->MarkPackageDirty();
	return StaticMesh;
}


/**
 *Constructor, setting all values to usable defaults 
 */
FMergeStaticMeshParams::FMergeStaticMeshParams()
	: Offset(0.0f, 0.0f, 0.0f)
	, Rotation(0, 0, 0)
	, ScaleFactor(1.0f)
	, ScaleFactor3D(1.0f, 1.0f, 1.0f)
	, bDeferBuild(false)
	, OverrideElement(INDEX_NONE)
	, bUseUVChannelRemapping(false)
	, bUseUVScaleBias(false)
{
	// initialize some UV channel arrays
	for (int32 Channel = 0; Channel < UE_ARRAY_COUNT(UVChannelRemap); Channel++)
	{
		// we can't just map channel to channel by default, because we need to know when a UV channel is
		// actually being redirected in to, so that we can update Triangle.NumUVs
		UVChannelRemap[Channel] = INDEX_NONE;

		// default to a noop scale/bias
		UVScaleBias[Channel] = FVector4(1.0f, 1.0f, 0.0f, 0.0f);
	}
}

/**
 * Merges SourceMesh into DestMesh, applying transforms along the way
 *
 * @param DestMesh The static mesh that will have SourceMesh merged into
 * @param SourceMesh The static mesh to merge in to DestMesh
 * @param Params Settings for the merge
 */
void MergeStaticMesh(UStaticMesh* DestMesh, UStaticMesh* SourceMesh, const FMergeStaticMeshParams& Params)
{
	// TODO_STATICMESH: Remove me.
}

//
//	FVerticesEqual
//

inline bool FVerticesEqual(FVector3f& V1,FVector3f& V2)
{
	if(FMath::Abs(V1.X - V2.X) > THRESH_POINTS_ARE_SAME * 4.0f)
	{
		return 0;
	}

	if(FMath::Abs(V1.Y - V2.Y) > THRESH_POINTS_ARE_SAME * 4.0f)
	{
		return 0;
	}

	if(FMath::Abs(V1.Z - V2.Z) > THRESH_POINTS_ARE_SAME * 4.0f)
	{
		return 0;
	}

	return 1;
}

void GetBrushMesh(ABrush* Brush, UModel* Model, FMeshDescription& MeshDescription, TArray<FStaticMaterial>& OutMaterials)
{
	FStaticMeshAttributes Attributes(MeshDescription);
	TVertexAttributesRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();
	TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors();
	TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();
	TEdgeAttributesRef<bool> EdgeHardnesses = Attributes.GetEdgeHardnesses();
	TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();
	
	//Make sure we have one UVChannel
	VertexInstanceUVs.SetNumChannels(1);
	
	// Calculate the local to world transform for the source brush.
	FMatrix	ActorToWorld = Brush ? Brush->ActorToWorld().ToMatrixWithScale() : FMatrix::Identity;
	bool	ReverseVertices = 0;
	FVector4	PostSub = Brush ? FVector4(Brush->GetActorLocation()) : FVector4(0, 0, 0, 0);

	TMap<uint32, FEdgeID> RemapEdgeID;
	int32 NumPolys = Model->Polys->Element.Num();
	//Create Fill the vertex position
	for (int32 PolygonIndex = 0; PolygonIndex < NumPolys; ++PolygonIndex)
	{
		FPoly& Polygon = Model->Polys->Element[PolygonIndex];

		// Find a material index for this polygon.
		UMaterialInterface*	Material = Polygon.Material;
		if (Material == nullptr)
		{
			Material = UMaterial::GetDefaultMaterial(MD_Surface);
		}

		int32 MaterialIndex = OutMaterials.AddUnique(FStaticMaterial(Material, Material->GetFName(), Material->GetFName()));
		FPolygonGroupID CurrentPolygonGroupID = INDEX_NONE;
		for (const FPolygonGroupID PolygonGroupID : MeshDescription.PolygonGroups().GetElementIDs())
		{
			if (Material->GetFName() == PolygonGroupImportedMaterialSlotNames[PolygonGroupID])
			{
				CurrentPolygonGroupID = PolygonGroupID;
				break;
			}
		}
		if (CurrentPolygonGroupID == INDEX_NONE)
		{
			CurrentPolygonGroupID = MeshDescription.CreatePolygonGroup();
			PolygonGroupImportedMaterialSlotNames[CurrentPolygonGroupID] = Material->GetFName();
		}

		// Cache the texture coordinate system for this polygon.
		FVector3f	TextureBase = Polygon.Base - (Brush ? (FVector3f)Brush->GetPivotOffset() : FVector3f::ZeroVector),
			TextureX = Polygon.TextureU / UModel::GetGlobalBSPTexelScale(),
			TextureY = Polygon.TextureV / UModel::GetGlobalBSPTexelScale();
		// For each vertex after the first two vertices...
		for (int32 VertexIndex = 2; VertexIndex < Polygon.Vertices.Num(); VertexIndex++)
		{
			FVector3f Positions[3];
			Positions[ReverseVertices ? 0 : 2] = FVector4f(ActorToWorld.TransformPosition((FVector)Polygon.Vertices[0]) - PostSub);
			Positions[1] = FVector4f(ActorToWorld.TransformPosition((FVector)Polygon.Vertices[VertexIndex - 1]) - PostSub);
			Positions[ReverseVertices ? 2 : 0] = FVector4f(ActorToWorld.TransformPosition((FVector)Polygon.Vertices[VertexIndex]) - PostSub);
			FVertexID VertexID[3] = { INDEX_NONE, INDEX_NONE, INDEX_NONE };
			for (FVertexID IterVertexID : MeshDescription.Vertices().GetElementIDs())
			{
				if (FVerticesEqual(Positions[0], VertexPositions[IterVertexID]))
				{
					VertexID[0] = IterVertexID;
				}
				if (FVerticesEqual(Positions[1], VertexPositions[IterVertexID]))
				{
					VertexID[1] = IterVertexID;
				}
				if (FVerticesEqual(Positions[2], VertexPositions[IterVertexID]))
				{
					VertexID[2] = IterVertexID;
				}
			}

			//Create the vertex instances
			TArray<FVertexInstanceID> VertexInstanceIDs;
			VertexInstanceIDs.SetNum(3);

			for (int32 CornerIndex = 0; CornerIndex < 3; ++CornerIndex)
			{
				if (VertexID[CornerIndex] == INDEX_NONE)
				{
					VertexID[CornerIndex] = MeshDescription.CreateVertex();
					VertexPositions[VertexID[CornerIndex]] = Positions[CornerIndex];
				}
				VertexInstanceIDs[CornerIndex] = MeshDescription.CreateVertexInstance(VertexID[CornerIndex]);
				VertexInstanceUVs.Set(VertexInstanceIDs[CornerIndex], 0, FVector2f(
					(Positions[CornerIndex] - TextureBase) | TextureX,
					(Positions[CornerIndex] - TextureBase) | TextureY));
			}

			// Create a polygon with the 3 vertex instances
			
			TArray<FEdgeID> NewEdgeIDs;
			const FPolygonID NewPolygonID = MeshDescription.CreatePolygon(CurrentPolygonGroupID, VertexInstanceIDs, &NewEdgeIDs);
			for (const FEdgeID& NewEdgeID : NewEdgeIDs)
			{
				//All edge are hard for BSP
				EdgeHardnesses[NewEdgeID] = true;
			}
		}
	}
}

//
//	CreateStaticMeshFromBrush - Creates a static mesh from the triangles in a model.
//

UStaticMesh* CreateStaticMeshFromBrush(UObject* Outer, FName Name, ABrush* Brush, UModel* Model)
{
	// MINOR HACK: Make sure we don't transact static mesh object creation because we are currently
	// unable to undo it properly, where undoing it places the asset in a broken state that can cause
	// crashes. Instead, temporarily disconnect the current transaction object, if any.
	ITransaction* UndoState = GUndo;
	GUndo = nullptr; // Pretend we're not in a transaction
	ON_SCOPE_EXIT{ GUndo = UndoState; }; // Revert

	FScopedSlowTask SlowTask(0.0f, NSLOCTEXT("UnrealEd", "CreatingStaticMeshE", "Creating static mesh..."));
	SlowTask.MakeDialog();

	// Create the UStaticMesh object.
	FStaticMeshComponentRecreateRenderStateContext RecreateRenderStateContext(FindObject<UStaticMesh>(Outer, *Name.ToString()));
	UStaticMesh* StaticMesh = NewObject<UStaticMesh>(Outer, Name, RF_Public | RF_Standalone);

	// Add one LOD for the base mesh
	StaticMesh->AddSourceModel();
	const int32 LodIndex = StaticMesh->GetNumSourceModels() - 1;
	FMeshDescription* MeshDescription = StaticMesh->CreateMeshDescription(LodIndex);

	// Fill out the mesh description and materials from the brush geometry
	TArray<FStaticMaterial> Materials;
	GetBrushMesh(Brush, Model, *MeshDescription, Materials);

	// Commit mesh description and materials list to static mesh
	StaticMesh->CommitMeshDescription(LodIndex);
	StaticMesh->SetStaticMaterials(Materials);

	// Set up the SectionInfoMap to enable collision
	const int32 NumSections = StaticMesh->GetStaticMaterials().Num();
	for (int32 SectionIdx = 0; SectionIdx < NumSections; ++SectionIdx)
	{
		FMeshSectionInfo Info = StaticMesh->GetSectionInfoMap().Get(0, SectionIdx);
		Info.MaterialIndex = SectionIdx;
		Info.bEnableCollision = true;
		StaticMesh->GetSectionInfoMap().Set(0, SectionIdx, Info);
		StaticMesh->GetOriginalSectionInfoMap().Set(0, SectionIdx, Info);
	}
	// The default settings auto generate lightmaps at channel 1
	StaticMesh->SetLightMapCoordinateIndex(1);
	// 64 is a common default, used in import
	StaticMesh->SetLightMapResolution(64);

	//Set the Imported version before calling the build
	StaticMesh->SetImportVersion(EImportStaticMeshVersion::LastVersion);

	StaticMesh->Build();
	StaticMesh->MarkPackageDirty();

	return StaticMesh;

}

// Accepts a triangle (XYZ and UV values for each point) and returns a poly base and UV vectors
// NOTE : the UV coords should be scaled by the texture size
static inline void FTexCoordsToVectors(const FVector3f& V0, const FVector3f& UV0,
									   const FVector3f& V1, const FVector3f& InUV1,
									   const FVector3f& V2, const FVector3f& InUV2,
									   FVector3f* InBaseResult, FVector3f* InUResult, FVector3f* InVResult )
{
	// Create polygon normal.
	FVector3f PN = FVector3f((V0-V1) ^ (V2-V0));
	PN = PN.GetSafeNormal();

	FVector3f UV1( InUV1 );
	FVector3f UV2( InUV2 );

	// Fudge UV's to make sure no infinities creep into UV vector math, whenever we detect identical U or V's.
	if( ( UV0.X == UV1.X ) || ( UV2.X == UV1.X ) || ( UV2.X == UV0.X ) ||
		( UV0.Y == UV1.Y ) || ( UV2.Y == UV1.Y ) || ( UV2.Y == UV0.Y ) )
	{
		UV1 += FVector3f(0.004173f,0.004123f,0.0f);
		UV2 += FVector3f(0.003173f,0.003123f,0.0f);
	}

	//
	// Solve the equations to find our texture U/V vectors 'TU' and 'TV' by stacking them 
	// into a 3x3 matrix , one for  u(t) = TU dot (x(t)-x(o) + u(o) and one for v(t)=  TV dot (.... , 
	// then the third assumes we're perpendicular to the normal. 
	//
	FMatrix44f TexEqu = FMatrix44f::Identity;
	TexEqu.SetAxis( 0, FVector3f(	V1.X - V0.X, V1.Y - V0.Y, V1.Z - V0.Z ) );
	TexEqu.SetAxis( 1, FVector3f( V2.X - V0.X, V2.Y - V0.Y, V2.Z - V0.Z ) );
	TexEqu.SetAxis( 2, FVector3f( PN.X,        PN.Y,        PN.Z        ) );
	TexEqu = TexEqu.InverseFast();

	const FVector3f UResult( UV1.X-UV0.X, UV2.X-UV0.X, 0.0f );
	const FVector3f TUResult = TexEqu.TransformVector( UResult );

	const FVector3f VResult( UV1.Y-UV0.Y, UV2.Y-UV0.Y, 0.0f );
	const FVector3f TVResult = TexEqu.TransformVector( VResult );

	//
	// Adjust the BASE to account for U0 and V0 automatically, and force it into the same plane.
	//				
	FMatrix44f BaseEqu = FMatrix44f::Identity;
	BaseEqu.SetAxis( 0, TUResult );
	BaseEqu.SetAxis( 1, TVResult ); 
	BaseEqu.SetAxis( 2, FVector3f( PN.X, PN.Y, PN.Z ) );
	BaseEqu = BaseEqu.InverseFast();

	const FVector3f BResult = FVector3f( UV0.X - ( TUResult|V0 ), UV0.Y - ( TVResult|V0 ),  0.0f );

	*InBaseResult = - 1.0f *  BaseEqu.TransformVector( BResult );
	*InUResult = TUResult;
	*InVResult = TVResult;

}


/**
 * Creates a model from the triangles in a static mesh.
 */
void CreateModelFromStaticMesh(UModel* Model,AStaticMeshActor* StaticMeshActor)
{
#ifdef TODO_STATICMESH
	UStaticMesh*	StaticMesh = StaticMeshActor->StaticMeshComponent->StaticMesh;
	FMatrix			ActorToWorld = StaticMeshActor->ActorToWorld().ToMatrixWithScale();

	Model->Polys->Element.Empty();

	const FStaticMeshTriangle* RawTriangleData = (FStaticMeshTriangle*) StaticMesh->LODModels[0].RawTriangles.Lock(LOCK_READ_ONLY);
	if(StaticMesh->LODModels[0].RawTriangles.GetElementCount())
	{
		for(int32 TriangleIndex = 0;TriangleIndex < StaticMesh->LODModels[0].RawTriangles.GetElementCount();TriangleIndex++)
		{
			const FStaticMeshTriangle&	Triangle	= RawTriangleData[TriangleIndex];
			FPoly*						Polygon		= new(Model->Polys->Element) FPoly;

			Polygon->Init();
			Polygon->iLink = Polygon - Model->Polys->Element.GetData();
			Polygon->Material = StaticMesh->LODModels[0].Elements[Triangle.MaterialIndex].Material;
			Polygon->PolyFlags = PF_DefaultFlags;
			Polygon->SmoothingMask = Triangle.SmoothingMask;

			new(Polygon->Vertices) FVector3f(ActorToWorld.TransformPosition(Triangle.Vertices[2]));
			new(Polygon->Vertices) FVector3f(ActorToWorld.TransformPosition(Triangle.Vertices[1]));
			new(Polygon->Vertices) FVector3f(ActorToWorld.TransformPosition(Triangle.Vertices[0]));

			Polygon->CalcNormal(1);
			Polygon->Finalize(NULL,0);
			FTexCoordsToVectors(Polygon->Vertices[2],FVector3f(Triangle.UVs[0][0].X * UModel::GetGlobalBSPTexelScale(),Triangle.UVs[0][0].Y * UModel::GetGlobalBSPTexelScale(),1),
								Polygon->Vertices[1],FVector3f(Triangle.UVs[1][0].X * UModel::GetGlobalBSPTexelScale(),Triangle.UVs[1][0].Y * UModel::GetGlobalBSPTexelScale(),1),
								Polygon->Vertices[0],FVector3f(Triangle.UVs[2][0].X * UModel::GetGlobalBSPTexelScale(),Triangle.UVs[2][0].Y * UModel::GetGlobalBSPTexelScale(),1),
								&Polygon->Base,&Polygon->TextureU,&Polygon->TextureV);
		}
	}
	StaticMesh->LODModels[0].RawTriangles.Unlock();

	Model->Linked = 1;
	FBSPOps::bspValidateBrush(Model,0,1);
	Model->BuildBound();
#endif // #if TODO_STATICMESH
}

#undef LOCTEXT_NAMESPACE
