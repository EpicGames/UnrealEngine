// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/ActorEditorContextSubsystem.h"
#include "ActorEditorContextState.h"
#include "Editor/UnrealEdEngine.h"
#include "GameFramework/Actor.h"
#include "IActorEditorContextClient.h"
#include "ScopedTransaction.h"
#include "UnrealEdGlobals.h"
#include "Editor.h"
#include "EditorState/EditorStateSubsystem.h"
#include "EditorState/ActorEditorContextEditorState.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorEditorContextSubsystem)

#define LOCTEXT_NAMESPACE "ActorEditorContext"

UActorEditorContextSubsystem* UActorEditorContextSubsystem::Get()
{ 
	check(GEditor);
	return GEditor->GetEditorSubsystem<UActorEditorContextSubsystem>();
}

void UActorEditorContextSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	GEditor->OnLevelActorAdded().AddUObject(this, &UActorEditorContextSubsystem::ApplyContext);
	if (GUnrealEd)
	{
		GUnrealEd->OnPasteActorsBegin().AddUObject(this, &UActorEditorContextSubsystem::OnPasteActorsBegin);
		GUnrealEd->OnPasteActorsEnd().AddUObject(this, &UActorEditorContextSubsystem::OnPasteActorsEnd);
	}

	Collection.InitializeDependency<UEditorStateSubsystem>();
	UEditorStateSubsystem::RegisterEditorStateType<UActorEditorContextEditorState>();
}

void UActorEditorContextSubsystem::Deinitialize()
{
	GEditor->OnLevelActorAdded().RemoveAll(this);

	if (GUnrealEd)
	{
		GUnrealEd->OnPasteActorsBegin().RemoveAll(this);
		GUnrealEd->OnPasteActorsEnd().RemoveAll(this);
	}

	UEditorStateSubsystem::UnregisterEditorStateType<UActorEditorContextEditorState>();

	Super::Deinitialize();
}

void UActorEditorContextSubsystem::OnPasteActorsBegin()
{
	// Disable ApplyContext while UUnrealEdEngine::PasteActors is executing as ImportObjectProperties is called after OnLevelActorAdded anyways
	bIsApplyEnabled = false;
}

void UActorEditorContextSubsystem::OnPasteActorsEnd(const TArray<AActor*>& InActors)
{
	// Enable and run ApplyContext now that UUnrealEdEngine::PasteActors is done executing
	bIsApplyEnabled = true;
	for (AActor* Actor : InActors)
	{
		ApplyContext(Actor);
	}
}

void UActorEditorContextSubsystem::RegisterClient(IActorEditorContextClient* Client)
{
	if (Client && !Clients.Contains(Client))
	{
		Client->GetOnActorEditorContextClientChanged().AddUObject(this, &UActorEditorContextSubsystem::OnActorEditorContextClientChanged);
		Clients.Add(Client);
	}
}

void UActorEditorContextSubsystem::UnregisterClient(IActorEditorContextClient* Client)
{
	if (Client)
	{
		if (Clients.Remove(Client))
		{
			Client->GetOnActorEditorContextClientChanged().RemoveAll(this);

			for (TArray<IActorEditorContextClient*>& PushedClients : PushedContextsStack)
			{
				PushedClients.Remove(Client);
			}
		}
	}
}

void UActorEditorContextSubsystem::ApplyContext(AActor* InActor)
{
	if (GIsReinstancing || !bIsApplyEnabled)
	{
		return;
	}

	UWorld* World = GetWorld();
	if (Clients.IsEmpty() || !World)
	{
		return;
	}

	if (!InActor || (InActor->GetWorld() != World) || InActor->HasAnyFlags(RF_Transient) || InActor->IsChildActor())
	{
		return;
	}

	for (IActorEditorContextClient* Client : Clients)
	{
		Client->OnExecuteActorEditorContextAction(World, EActorEditorContextAction::ApplyContext, InActor);
	}
}

void UActorEditorContextSubsystem::ResetContext()
{
	UWorld* World = GetWorld();
	if (Clients.IsEmpty() || !World)
	{
		return;
	}
		
	const FScopedTransaction Transaction(LOCTEXT("Reset Actor Editor Context", "Reset Actor Editor Context"));
	for (IActorEditorContextClient* Client : Clients)
	{
		Client->OnExecuteActorEditorContextAction(World, EActorEditorContextAction::ResetContext);
	}
	ActorEditorContextSubsystemChanged.Broadcast();
}

void UActorEditorContextSubsystem::ResetContext(IActorEditorContextClient* Client)
{
	UWorld* World = GetWorld();
	if (Clients.IsEmpty() || !World)
	{
		return;
	}

	if (Clients.Contains(Client))
	{
		const FScopedTransaction Transaction(LOCTEXT("Reset Actor Editor Context", "Reset Actor Editor Context"));
		Client->OnExecuteActorEditorContextAction(World, EActorEditorContextAction::ResetContext);
		ActorEditorContextSubsystemChanged.Broadcast();
	}
}

void UActorEditorContextSubsystem::PushContext(bool bDuplicateContext)
{
	UWorld* World = GetWorld();
	if (Clients.IsEmpty() || !World)
	{
		return;
	}

	for (IActorEditorContextClient* Client : Clients)
	{
		Client->OnExecuteActorEditorContextAction(World, bDuplicateContext ? EActorEditorContextAction::PushDuplicateContext : EActorEditorContextAction::PushContext);
	}

	PushedContextsStack.Push(Clients);

	ActorEditorContextSubsystemChanged.Broadcast();
}

void UActorEditorContextSubsystem::PopContext()
{
	UWorld* World = GetWorld();
	if (Clients.IsEmpty() || !World)
	{
		return;
	}

	for (IActorEditorContextClient* Client : PushedContextsStack.Pop(EAllowShrinking::No))
	{
		Client->OnExecuteActorEditorContextAction(World, EActorEditorContextAction::PopContext);
	}
	ActorEditorContextSubsystemChanged.Broadcast();
}

void UActorEditorContextSubsystem::InitializeContextFromActor(AActor* Actor)
{
	UWorld* World = GetWorld();
	if (Clients.IsEmpty() || !World || !Actor)
	{
		return;
	}

	for (IActorEditorContextClient* Client : Clients)
	{
		Client->OnExecuteActorEditorContextAction(World, EActorEditorContextAction::InitializeContextFromActor, Actor);
	}
	ActorEditorContextSubsystemChanged.Broadcast();
}

void UActorEditorContextSubsystem::CaptureContext(UActorEditorContextStateCollection* InStateCollection)
{
	UWorld* World = GetWorld();
	if (Clients.IsEmpty() || !World || !InStateCollection)
	{
		return;
	}

	InStateCollection->Reset();

	for (IActorEditorContextClient* Client : Clients)
	{
		Client->CaptureActorEditorContextState(World, InStateCollection);
	}
}

void UActorEditorContextSubsystem::RestoreContext(const UActorEditorContextStateCollection* InStateCollection) const
{
	UWorld* World = GetWorld();
	if (Clients.IsEmpty() || !World || !InStateCollection)
	{
		return;
	}

	for (IActorEditorContextClient* Client : Clients)
	{
		Client->RestoreActorEditorContextState(World, InStateCollection);
	}
}

TArray<IActorEditorContextClient*> UActorEditorContextSubsystem::GetDisplayableClients() const
{
	TArray<IActorEditorContextClient*> DisplayableClients;
	UWorld* World = GetWorld();
	if (!Clients.IsEmpty() && World)
	{
		for (IActorEditorContextClient* Client : Clients)
		{
			FActorEditorContextClientDisplayInfo Info;
			if (Client->GetActorEditorContextDisplayInfo(World, Info))
			{
				DisplayableClients.Add(Client);
			}
		}
	}
	return DisplayableClients;
}

UWorld* UActorEditorContextSubsystem::GetWorld() const
{
	return GEditor->GetEditorWorldContext().World();
}

void UActorEditorContextSubsystem::OnActorEditorContextClientChanged(IActorEditorContextClient* Client)
{
	if (IsRunningGame() || (GIsEditor && GEditor->GetPIEWorldContext()))
	{
		return;
	}

	ActorEditorContextSubsystemChanged.Broadcast();
}

#undef LOCTEXT_NAMESPACE
