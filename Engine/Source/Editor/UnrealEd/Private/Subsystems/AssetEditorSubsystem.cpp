// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/AssetEditorSubsystem.h"
#include "AssetEditorMessages.h"
#include "Containers/VersePath.h"
#include "MessageEndpoint.h"
#include "EngineAnalytics.h"
#include "AnalyticsEventAttribute.h"
#include "UObject/Package.h"
#include "CoreGlobals.h"
#include "AssetToolsModule.h"
#include "IAssetTools.h"
#include "IMessageContext.h"
#include "LevelEditor.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Toolkits/SimpleAssetEditor.h"
#include "Engine/MapBuildDataRegistry.h"
#include "MRUFavoritesList.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "PackageTools.h"
#include "UObject/PackageReload.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "Misc/FeedbackContext.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/NamePermissionList.h"
#include "EditorModeRegistry.h"
#include "Tools/UEdMode.h"
#include "AssetEditorMessages.h"
#include "EditorModeManager.h"
#include "Tools/LegacyEdMode.h"
#include "ProfilingDebugging/StallDetector.h"
#include "ToolMenus.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/AssetRegistryModule.h"

#include "Elements/SMInstance/SMInstanceElementData.h" // For SMInstanceElementDataUtil::SMInstanceElementsEnabled

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetEditorSubsystem)


#define LOCTEXT_NAMESPACE "AssetEditorSubsystem"

DEFINE_LOG_CATEGORY_STATIC(LogAssetEditorSubsystem, Log, All);

UAssetEditorSubsystem::UAssetEditorSubsystem()
	: Super()
	, bSavingOnShutdown(false)
	, bRequestRestorePreviouslyOpenAssets(false)
{
}

void UAssetEditorSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	TickDelegate = FTickerDelegate::CreateUObject(this, &UAssetEditorSubsystem::HandleTicker);
	FTSTicker::GetCoreTicker().AddTicker(TickDelegate, 1.f);

	FCoreUObjectDelegates::OnPackageReloaded.AddUObject(this, &UAssetEditorSubsystem::HandlePackageReloaded);

	GEditor->OnEditorClose().AddUObject(this, &UAssetEditorSubsystem::OnEditorClose);
	FCoreDelegates::OnEnginePreExit.AddUObject(this, &UAssetEditorSubsystem::UnregisterEditorModes);
	FCoreDelegates::OnAllModuleLoadingPhasesComplete.AddUObject(this, &UAssetEditorSubsystem::RegisterEditorModes);

	if (FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>(TEXT("AssetRegistry")))
	{
		AssetRegistryModule->Get().OnAssetRemoved().AddUObject(this, &UAssetEditorSubsystem::OnAssetRemoved);
		AssetRegistryModule->Get().OnAssetRenamed().AddUObject(this, &UAssetEditorSubsystem::OnAssetRenamed);
	}

	SMInstanceElementDataUtil::OnSMInstanceElementsEnabledChanged().AddUObject(this, &UAssetEditorSubsystem::OnSMInstanceElementsEnabled);

	RegisterLevelEditorMenuExtensions();

	InitializeRecentAssets();
}

void UAssetEditorSubsystem::Deinitialize()
{
	FCoreUObjectDelegates::OnPackageReloaded.RemoveAll(this);
	GEditor->OnEditorClose().RemoveAll(this);
	FCoreDelegates::OnEnginePreExit.RemoveAll(this);
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
	SMInstanceElementDataUtil::OnSMInstanceElementsEnabledChanged().RemoveAll(this);

	if (FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>(TEXT("AssetRegistry")))
	{
		AssetRegistryModule->Get().OnAssetRemoved().RemoveAll(this);
		AssetRegistryModule->Get().OnAssetRenamed().RemoveAll(this);
	}

	// Don't attempt to report usage stats if analytics isn't available
	if (FEngineAnalytics::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> EditorUsageAttribs;
		EditorUsageAttribs.Empty(3);
		for (auto Iter = EditorUsageAnalytics.CreateConstIterator(); Iter; ++Iter)
		{
			const FAssetEditorAnalyticInfo& Data = Iter.Value();
			EditorUsageAttribs.Reset();
			EditorUsageAttribs.Emplace(TEXT("TotalDurationSeconds"), Data.SumDuration.GetTotalSeconds());
			EditorUsageAttribs.Emplace(TEXT("OpenedInstancesCount"), Data.NumTimesOpened);
			EditorUsageAttribs.Emplace(TEXT("AssetEditor"), *Iter.Key().ToString());
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.AssetEditorClosed"), EditorUsageAttribs);
		}
	}

	SaveRecentAssets();
	RecentAssetsList.Reset();
}

void UAssetEditorSubsystem::InitializeRecentAssets()
{
	// The current max allowed is 30 assets
	RecentAssetsList = MakeUnique<FMainMRUFavoritesList>(TEXT("AssetEditorSubsystemRecents"), TEXT("AssetEditorSubsystemFavorites"), 30);
	RecentAssetsList->ReadFromINI();
	
	TArray<FString> RecentAssetEditors;

	GConfig->GetArray(TEXT("AssetEditorSubsystem"), TEXT("RecentAssetEditors"), RecentAssetEditors, GEditorPerProjectIni);

	bool bFoundRecentAssetEditorsCleanly = true;

	// If the number of recent assets and recent asset editors don't match, something went wrong during saving and we have potentially corrupt data
	if(RecentAssetsList->GetNumItems() != RecentAssetEditors.Num())
	{
		bFoundRecentAssetEditorsCleanly = false;
		UE_LOGF(LogAssetEditorSubsystem, Display, "Something went wrong while loading recent assets! Num Recent Assets  = %d, Num Recent Asset Editors = %d", RecentAssetsList->GetNumItems(), RecentAssetEditors.Num());
	}

	// If we have corrupt data, simply ignore the asset editor list instead of potentially showing an asset in the wrong asset editor's recents
	if(bFoundRecentAssetEditorsCleanly)
	{
		// Go in reverse since the first item should be the most recent
		for(int CurRecentIndex = 0; CurRecentIndex < RecentAssetsList->GetNumItems(); ++CurRecentIndex)
		{
			if(!RecentAssetEditors[CurRecentIndex].IsEmpty())
			{
				RecentAssetToAssetEditorMap.Add(RecentAssetsList->GetMRUItem(CurRecentIndex), RecentAssetEditors[CurRecentIndex]);
			}
		}
	}
}

void UAssetEditorSubsystem::SaveRecentAssets(const bool bOnShutdown)
{
	TArray<FString> RecentAssetEditorsToSave;

	// If we are closing the editor, remove all assets that weren't actually saved to disk (transient)
	if(bOnShutdown)
	{
		for(int CurRecentIndex = 0; CurRecentIndex < RecentAssetsList->GetNumItems(); ++CurRecentIndex)
		{
			const FString& RecentAsset = RecentAssetsList->GetMRUItem(CurRecentIndex);
			if(!FPackageName::DoesPackageExist(RecentAsset))
			{
				RecentAssetsList->RemoveMRUItem(CurRecentIndex);
				--CurRecentIndex;
			}
		}
	}
	
	for(int CurRecentIndex = 0; CurRecentIndex < RecentAssetsList->GetNumItems(); ++CurRecentIndex)
	{
		FString CurRecentAsset = RecentAssetsList->GetMRUItem(CurRecentIndex);

		// If we have a valid asset editor for the current asset, save it
		if(FString* CurrentAssetEditorName = RecentAssetToAssetEditorMap.Find(CurRecentAsset))
		{
			RecentAssetEditorsToSave.Add(*CurrentAssetEditorName);
		}
		// Otherwise add an empty entry (e.g levels) so the two arrays are always the same size
		else
		{
			RecentAssetEditorsToSave.Add(FString());
		}
		
	}

	RecentAssetsList->WriteToINI();
	GConfig->SetArray(TEXT("AssetEditorSubsystem"), TEXT("RecentAssetEditors"), RecentAssetEditorsToSave, GEditorPerProjectIni);
}

void UAssetEditorSubsystem::CullRecentAssetEditorsMap()
{
	/* Since the Recent Asset -> Asset Editor Map is not an MRU list, it can keep infinitely growing as the user opens assets
	 * To keep it a reasonable size while also not culling it too often, we cull it when it gets twice as big as the MRU list
	 */
	if(RecentAssetToAssetEditorMap.Num() > 2 * RecentAssetsList->GetMaxItems())
	{
		for (TMap<FString, FString>::TIterator It(RecentAssetToAssetEditorMap); It; ++It)
		{
			// Remove any entries that are not in the mru list or if the package isn't valid anymore
			if(!FPackageName::IsValidLongPackageName(It->Key) || RecentAssetsList->FindMRUItemIdx(It->Key) == INDEX_NONE)
			{
				It.RemoveCurrent();
			}
		}

	}
}

void UAssetEditorSubsystem::OnAssetRemoved(const FAssetData& AssetData)
{
	FString PathName = AssetData.PackageName.ToString();

	// We need this early exit because FindMRUItemIdx has a check() for non valid long package names
	if (!FPackageName::IsValidLongPackageName(PathName))
	{
		return;
	}

	// If the asset that was deleted was not found in the recent assets list, we have nothing to do
	if(RecentAssetsList->FindMRUItemIdx(PathName) == INDEX_NONE)
	{
		return;
	}

	// Remove the asset from our list and map
	RecentAssetsList->RemoveMRUItem(PathName);
	RecentAssetToAssetEditorMap.Remove(PathName);

	SaveRecentAssets();
}

void UAssetEditorSubsystem::OnAssetRenamed(const FAssetData& AssetData, const FString& AssetOldName)
{
	FString OldPathName = FSoftObjectPath(AssetOldName).GetLongPackageName();
	FString NewPathName = AssetData.PackageName.ToString();

	// We need this early exit because FindMRUItemIdx has a check() for non valid long package names
	if (!FPackageName::IsValidLongPackageName(OldPathName) || !FPackageName::IsValidLongPackageName(NewPathName))
	{
		return;
	}

	// If the asset did not previously exist in the recents list, we have nothing to do
	if(RecentAssetsList->FindMRUItemIdx(OldPathName) == INDEX_NONE)
	{
		return;
	}

	// Otherwise remove the old name of the asset, and re-add it with the new name
	// NOTE: This has an unintentional side effect of bringing it to the top of the MRU list that can't be avoided
	RecentAssetsList->RemoveMRUItem(OldPathName);
	RecentAssetsList->AddMRUItem(NewPathName);

	if(FString* AssetEditorName = RecentAssetToAssetEditorMap.Find(OldPathName))
	{
		RecentAssetToAssetEditorMap.Add(NewPathName, *AssetEditorName);
		RecentAssetToAssetEditorMap.Remove(OldPathName);
	}

	SaveRecentAssets();
}

void UAssetEditorSubsystem::OnEditorClose()
{
	SaveOpenAssetEditors(true);
	TGuardValue<bool> GuardOnShutdown(bSavingOnShutdown, true);
	CloseAllAssetEditors();
}

IAssetEditorInstance* UAssetEditorSubsystem::FindEditorForAsset(UObject* Asset, bool bFocusIfOpen)
{
	const TArray<IAssetEditorInstance*> AssetEditors = FindEditorsForAsset(Asset);

	IAssetEditorInstance* const * PrimaryEditor = AssetEditors.FindByPredicate([](IAssetEditorInstance* Editor) { return Editor->IsPrimaryEditor(); });

	const bool bEditorOpen = PrimaryEditor != NULL;
	if (bEditorOpen && bFocusIfOpen)
	{
		// @todo toolkit minor: We may need to handle this differently for world-centric vs standalone.  (multiple level editors, etc)
		(*PrimaryEditor)->FocusWindow(Asset);
	}

	return bEditorOpen ? *PrimaryEditor : NULL;
}


TArray<IAssetEditorInstance*> UAssetEditorSubsystem::FindEditorsForAsset(UObject* Asset)
{
	TArray<IAssetEditorInstance*> AssetEditors;
	OpenedAssets.MultiFind(Asset, AssetEditors);
	return AssetEditors;
}

TArray<IAssetEditorInstance*> UAssetEditorSubsystem::FindEditorsForAssetAndSubObjects(UObject* Asset)
{
	TArray<IAssetEditorInstance*> EditorInstances;

	if (Asset)
	{
		for (const TPair<FAssetEntry, IAssetEditorInstance*>& Pair : OpenedAssets)
		{
			if (Pair.Key.RawPtr == Asset || (Pair.Key.ObjectPtr.IsValid() && Pair.Key.ObjectPtr.Get()->IsIn(Asset)))
			{		
				EditorInstances.Add(Pair.Value);
			}
		}
	}
	
	return EditorInstances;
}

int32 UAssetEditorSubsystem::CloseAllEditorsForAsset(UObject* Asset)
{
	TArray<IAssetEditorInstance*> EditorInstances = FindEditorsForAssetAndSubObjects(Asset);

	for (IAssetEditorInstance* EditorInstance : EditorInstances)
	{
		if (EditorInstance)
		{
			EditorInstance->CloseWindow(EAssetEditorCloseReason::CloseAllEditorsForAsset);
		}
	}

	AssetEditorRequestCloseEvent.Broadcast(Asset, EAssetEditorCloseReason::CloseAllEditorsForAsset);

	return EditorInstances.Num();
}


void UAssetEditorSubsystem::RemoveAssetFromAllEditors(UObject* Asset)
{
	TArray<IAssetEditorInstance*> EditorInstances = FindEditorsForAsset(Asset);

	for (IAssetEditorInstance* EditorIter : EditorInstances)
	{
		if (EditorIter)
		{
			EditorIter->RemoveEditingAsset(Asset);
		}
	}

	AssetEditorRequestCloseEvent.Broadcast(Asset, EAssetEditorCloseReason::RemoveAssetFromAllEditors);
}


void UAssetEditorSubsystem::CloseOtherEditors(UObject* Asset, IAssetEditorInstance* OnlyEditor)
{
	TArray<UObject*> AllAssets;
	for (TMultiMap<FAssetEntry, IAssetEditorInstance*>::TIterator It(OpenedAssets); It; ++It)
	{
		IAssetEditorInstance* Editor = It.Value();
		if (Asset == It.Key().RawPtr && Editor != OnlyEditor)
		{
			Editor->CloseWindow(EAssetEditorCloseReason::CloseOtherEditors);
		}
	}

	AssetEditorRequestCloseEvent.Broadcast(Asset, EAssetEditorCloseReason::CloseOtherEditors);
}


TArray<UObject*> UAssetEditorSubsystem::GetAllEditedAssets()
{
	TArray<UObject*> AllAssets;
	for (TMultiMap<FAssetEntry, IAssetEditorInstance*>::TIterator It(OpenedAssets); It; ++It)
	{
		UObject* Asset = It.Key().ObjectPtr.Get();
		if (Asset != nullptr)
		{
			AllAssets.AddUnique(Asset);
		}
	}
	return AllAssets;
}


TArray<IAssetEditorInstance*> UAssetEditorSubsystem::GetAllOpenEditors() const
{
	TArray<IAssetEditorInstance*> AllEditors;
	OpenedEditors.GenerateKeyArray(AllEditors);
	return AllEditors;
}


void UAssetEditorSubsystem::NotifyEditorOpeningPreWidgets(const TArray< UObject* >& Assets, IAssetEditorInstance* InInstance)
{
	EditorOpeningPreWidgetsEvent.Broadcast(Assets, InInstance);
}


void UAssetEditorSubsystem::NotifyAssetOpened(UObject* Asset, IAssetEditorInstance* InInstance)
{
	if (!OpenedEditors.Contains(InInstance))
	{
		FOpenedEditorTime EditorTime;
		EditorTime.EditorName = InInstance->GetEditorName();
		EditorTime.OpenedTime = FDateTime::UtcNow();

		OpenedEditorTimes.Add(InInstance, EditorTime);
	}

	FString AssetPath = Asset->GetOuter()->GetPathName();
	
	OpenedAssets.Add(Asset, InInstance);
	OpenedEditors.Add(InInstance, Asset);
	RecentAssetToAssetEditorMap.Add(AssetPath, InInstance->GetEditorName().ToString());

	AssetOpenedInEditorEvent.Broadcast(Asset, InInstance);

	if(InInstance->IncludeAssetInRestoreOpenAssetsPrompt(Asset))
	{
		SaveOpenAssetEditors(false);
	}
}


void UAssetEditorSubsystem::NotifyAssetsOpened(const TArray< UObject* >& Assets, IAssetEditorInstance* InInstance)
{
	for (auto AssetIter = Assets.CreateConstIterator(); AssetIter; ++AssetIter)
	{
		NotifyAssetOpened(*AssetIter, InInstance);
	}
}


void UAssetEditorSubsystem::NotifyAssetClosed(UObject* Asset, IAssetEditorInstance* InInstance)
{
	AssetClosedInEditorEvent.Broadcast(Asset, InInstance);

	OpenedEditors.RemoveSingle(InInstance, Asset);
	OpenedAssets.RemoveSingle(Asset, InInstance);

	SaveOpenAssetEditors(false);
}


void UAssetEditorSubsystem::NotifyEditorClosed(IAssetEditorInstance* InInstance)
{
	// Remove all assets associated with the editor
	TArray<FAssetEntry> Assets;
	OpenedEditors.MultiFind(InInstance, /*out*/ Assets);
	for (int32 AssetIndex = 0; AssetIndex < Assets.Num(); ++AssetIndex)
	{
		if(UObject* Asset = Assets[AssetIndex].ObjectPtr.Get())
		{
			AssetClosedInEditorEvent.Broadcast(Asset, InInstance);
		}
		OpenedAssets.Remove(Assets[AssetIndex], InInstance);
	}

	// Remove the editor itself
	OpenedEditors.Remove(InInstance);
	FOpenedEditorTime EditorTime = OpenedEditorTimes.FindAndRemoveChecked(InInstance);

	// Record the editor open-close duration
	FAssetEditorAnalyticInfo& AnalyticsForThisAsset = EditorUsageAnalytics.FindOrAdd(EditorTime.EditorName);
	AnalyticsForThisAsset.SumDuration += FDateTime::UtcNow() - EditorTime.OpenedTime;
	AnalyticsForThisAsset.NumTimesOpened++;

	SaveOpenAssetEditors(false);
}


bool UAssetEditorSubsystem::CloseAllAssetEditors()
{
	bool bAllEditorsClosed = true;
	TArray<IAssetEditorInstance*> OpenedEditorInstances;
	OpenedEditors.GetKeys(OpenedEditorInstances);
	for (IAssetEditorInstance* Editor : OpenedEditorInstances) // operate on copy since closing will remove from map
	{
		if (Editor != nullptr)
		{
			if (!Editor->CloseWindow(EAssetEditorCloseReason::CloseAllAssetEditors))
			{
				bAllEditorsClosed = false;
			}
		}
	}

	AssetEditorRequestCloseEvent.Broadcast(nullptr, EAssetEditorCloseReason::CloseAllAssetEditors);

	return bAllEditorsClosed;
}

bool UAssetEditorSubsystem::IsAssetEditable(const UObject* Asset)
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));

	if (!Asset)
	{
		return false;
	}

	if (UPackage* Package = Asset->GetPackage())
	{
		if (Package->bIsCookedForEditor)
		{
			return false;
		}

		if (!AssetToolsModule.Get().GetWritableFolderPermissionList()->PassesStartsWithFilter(Package->GetName()))
		{
			return false;
		}
	}

	return true;
}

bool UAssetEditorSubsystem::OpenEditorForAsset(UObject* Asset, const EToolkitMode::Type ToolkitMode, TSharedPtr< IToolkitHost > OpenedFromLevelEditor, const bool bShowProgressWindow, const EAssetTypeActivationOpenedMethod OpenedMethod)
{
	FText ErrorMessage;
	if(!CanOpenEditorForAsset(Asset, OpenedMethod, &ErrorMessage))
	{
		// We also log the error if the asset was null
		if(!Asset)
		{
			UE_LOGF(LogAssetEditorSubsystem, Error, "%ls", *ErrorMessage.ToString());
		}
		
		if (TSharedPtr<SNotificationItem> InfoItem = FSlateNotificationManager::Get().AddNotification(FNotificationInfo(ErrorMessage)))
        {
        	InfoItem->SetCompletionState(SNotificationItem::CS_Fail);
        }

		return false;
	}

	// @todo toolkit minor: When "Edit Here" happens in a different level editor from the one that an asset is already
	//    being edited within, we should decide whether to disallow "Edit Here" in that case, or to close the old asset
	//    editor and summon it in the new level editor, or to just foreground the old level editor (current behavior)

	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
	TWeakPtr<IAssetTypeActions> AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(Asset->GetClass());

	AssetEditorRequestOpenEvent.Broadcast(Asset);

	const bool bBringToFrontIfOpen = true;
	if ((AssetTypeActions.IsValid() && AssetTypeActions.Pin()->ShouldFindEditorForAsset()) && FindEditorForAsset(Asset, bBringToFrontIfOpen) != nullptr)
	{
		// This asset is already open in an editor! (the call to FindEditorForAsset above will bring it to the front)
		return true;
	}
	else
	{
		if (bShowProgressWindow)
		{
			GWarn->BeginSlowTask(LOCTEXT("OpenEditor", "Opening Editor..."), true);
		}
	}

	UE_LOGF(LogAssetEditorSubsystem, Log, "Opening Asset editor for %ls", *Asset->GetFullName());

	const FString AssetPath = Asset->GetPathName();
	EAssetOpenMethod AssetOpenMethod = OpenedMethod == EAssetTypeActivationOpenedMethod::View ? EAssetOpenMethod::View : EAssetOpenMethod::Edit;

	// We store the open method for this asset, so that FAssetEditorToolkit can query us for this information during init
	AssetOpenMethodCache.Add(AssetPath, AssetOpenMethod);

	EToolkitMode::Type ActualToolkitMode = ToolkitMode;
	if (AssetTypeActions.IsValid())
	{
		if (AssetTypeActions.Pin()->ShouldForceWorldCentric())
		{
			// This asset type prefers a specific toolkit mode
			ActualToolkitMode = EToolkitMode::WorldCentric;

			if (!OpenedFromLevelEditor.IsValid())
			{
				// We don't have a level editor to spawn in world-centric mode, so we'll find one now
				// @todo sequencer: We should eventually eliminate this code (incl include dependencies) or change it to not make assumptions about a single level editor
				OpenedFromLevelEditor = FModuleManager::LoadModuleChecked< FLevelEditorModule >("LevelEditor").GetFirstLevelEditor();
			}
		}
	}

	if (ActualToolkitMode != EToolkitMode::WorldCentric && OpenedFromLevelEditor.IsValid())
	{
		// @todo toolkit minor: Kind of lame use of a static variable here to prime the new asset editor.  This was done to avoid refactoring a few dozen files for a very minor change.
		FAssetEditorToolkit::SetPreviousWorldCentricToolkitHostForNewAssetEditor(OpenedFromLevelEditor.ToSharedRef());
	}

	if (AssetTypeActions.IsValid())
	{
		TArray<UObject*> AssetsToEdit;
		AssetsToEdit.Add(Asset);

		// Some assets (like UWorlds) may be destroyed and recreated as part of opening. To protect against this, keep the path to the asset and try to re-find it if it disappeared.
		TWeakObjectPtr<UObject> WeakAsset = Asset;

		AssetTypeActions.Pin()->OpenAssetEditor(AssetsToEdit, OpenedMethod, ActualToolkitMode == EToolkitMode::WorldCentric ? OpenedFromLevelEditor : TSharedPtr<IToolkitHost>());

		// If the Asset was destroyed, attempt to find it if it was recreated
		if (!WeakAsset.IsValid() && !AssetPath.IsEmpty())
		{
			Asset = FindObject<UObject>(nullptr, *AssetPath);
		}

		AssetEditorOpenedEvent.Broadcast(Asset);
	}
	else
	{
		// No asset type actions for this asset. Just use a properties editor.
		FSimpleAssetEditor::CreateEditor(ActualToolkitMode, ActualToolkitMode == EToolkitMode::WorldCentric ? OpenedFromLevelEditor : TSharedPtr<IToolkitHost>(), Asset);
	}

	if (bShowProgressWindow)
	{
		GWarn->EndSlowTask();
	}
	// Must check Asset here in addition to at the beginning of the function, because if the asset was destroyed and recreated it might not be found correctly
	// Do not add to recently opened asset list if this is a level-associated asset like Level Blueprint or Built Data. Their naming is not compatible
	if (Asset)
	{
		if (Asset->IsAsset() && !Asset->IsA(UMapBuildDataRegistry::StaticClass()))
		{
			FString AssetOuterPath = Asset->GetOuter()->GetPathName();
			if (FPackageName::IsValidLongPackageName(AssetOuterPath))
			{
				RecentAssetsList->AddMRUItem(AssetOuterPath);
				CullRecentAssetEditorsMap();
			}
		}
	}

	// Since the Asset Editor has finished init once we are here, we can remove the open method from the cache
	AssetOpenMethodCache.Remove(AssetPath);
	
	return true;
}

bool UAssetEditorSubsystem::OpenEditorForAssets_Advanced(const TArray <UObject*>& InAssets, const EToolkitMode::Type ToolkitMode, TSharedPtr< IToolkitHost > OpenedFromLevelEditor, const EAssetTypeActivationOpenedMethod OpenedMethod)
{
	TArray<UObject*> Assets;
	Assets.Reserve(InAssets.Num());
	int32 NumNullAssets = 0;
	for (UObject* Asset : InAssets)
	{
		if (Asset)
		{
			Assets.AddUnique(Asset);
		}
		else
		{
			++NumNullAssets;
		}
	}

	if (NumNullAssets > 1)
	{
		UE_LOGF(LogAssetEditorSubsystem, Error, "Opening Asset editors failed because of null assets");
	}
	else if (NumNullAssets > 0)
	{
		UE_LOGF(LogAssetEditorSubsystem, Error, "Opening Asset editor failed because of null asset");
	}

	if (Assets.Num() == 1)
	{
		return OpenEditorForAsset(Assets[0], ToolkitMode, OpenedFromLevelEditor, true, OpenedMethod);
	}
	else if (Assets.Num() > 0)
	{
		TArray<UObject*> SkipOpenAssets;
		for (UObject* Asset : Assets)
		{
			// If any of the assets are already open or they cannot be opened in this open method
			// remove them from the list of assets to open an editor for
			UPackage* Package = Asset->GetOutermost();
			FText ErrorMessage;
			if (FindEditorForAsset(Asset, true) != nullptr || !CanOpenEditorForAsset(Asset, OpenedMethod, &ErrorMessage))
			{
				SkipOpenAssets.Add(Asset);
			}
		}

		// Verify that all the assets are of the same class
		bool bAssetClassesMatch = true;
		UClass* AssetClass = Assets[0]->GetClass();
		for (int32 i = 1; i < Assets.Num(); i++)
		{
			if (Assets[i]->GetClass() != AssetClass)
			{
				bAssetClassesMatch = false;
				break;
			}
		}

		// If the classes don't match or any of the selected assets are already open, just open each asset in its own editor.
		if (bAssetClassesMatch && SkipOpenAssets.Num() == 0)
		{
			FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
			TWeakPtr<IAssetTypeActions> AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(AssetClass);

			if (AssetTypeActions.IsValid() && AssetTypeActions.Pin()->SupportsOpenedMethod(OpenedMethod))
			{
				GWarn->BeginSlowTask(LOCTEXT("OpenEditors", "Opening Editor(s)..."), true);

				// Determine the appropriate toolkit mode for the asset type
				EToolkitMode::Type ActualToolkitMode = ToolkitMode;
				if (AssetTypeActions.Pin()->ShouldForceWorldCentric())
				{
					// This asset type prefers a specific toolkit mode
					ActualToolkitMode = EToolkitMode::WorldCentric;

					if (!OpenedFromLevelEditor.IsValid())
					{
						// We don't have a level editor to spawn in world-centric mode, so we'll find one now
						// @todo sequencer: We should eventually eliminate this code (incl include dependencies) or change it to not make assumptions about a single level editor
						OpenedFromLevelEditor = FModuleManager::LoadModuleChecked< FLevelEditorModule >("LevelEditor").GetFirstLevelEditor();
					}
				}

				if (ActualToolkitMode != EToolkitMode::WorldCentric && OpenedFromLevelEditor.IsValid())
				{
					// @todo toolkit minor: Kind of lame use of a static variable here to prime the new asset editor.  This was done to avoid refactoring a few dozen files for a very minor change.
					FAssetEditorToolkit::SetPreviousWorldCentricToolkitHostForNewAssetEditor(OpenedFromLevelEditor.ToSharedRef());
				}

				// Some assets (like UWorlds) may be destroyed and recreated as part of opening. To protect against this, keep the path to each asset and try to re-find any if they disappear.
				struct FLocalAssetInfo
				{
					TWeakObjectPtr<UObject> WeakAsset;
					FString AssetPath;

					FLocalAssetInfo(const TWeakObjectPtr<UObject>& InWeakAsset, const FString& InAssetPath)
						: WeakAsset(InWeakAsset), AssetPath(InAssetPath) {}
				};

				EAssetOpenMethod AssetOpenMethod = OpenedMethod == EAssetTypeActivationOpenedMethod::View ? EAssetOpenMethod::View : EAssetOpenMethod::Edit;

				TArray<FLocalAssetInfo> AssetInfoList;
				AssetInfoList.Reserve(Assets.Num());
				for (UObject* Asset : Assets)
				{
					AssetInfoList.Add(FLocalAssetInfo(Asset, Asset->GetPathName()));

					// We store the open method for this asset, so that FAssetEditorToolkit can query us for this information during init
					AssetOpenMethodCache.Add(Asset->GetPathName(), AssetOpenMethod);
				}

				// How to handle multiple assets is left up to the type actions (i.e. open a single shared editor or an editor for each)
				AssetTypeActions.Pin()->OpenAssetEditor(Assets, OpenedMethod, ActualToolkitMode == EToolkitMode::WorldCentric ? OpenedFromLevelEditor : TSharedPtr<IToolkitHost>());

				// If any assets were destroyed, attempt to find them if they were recreated
				for (int32 i = 0; i < Assets.Num(); i++)
				{
					const FLocalAssetInfo& AssetInfo = AssetInfoList[i];
					UObject* Asset = Assets[i];

					if (!AssetInfo.WeakAsset.IsValid() && !AssetInfo.AssetPath.IsEmpty())
					{
						Asset = FindObject<UObject>(nullptr, *AssetInfo.AssetPath);
					}

					// Since the Asset Editor has finished init once we are here, we can remove the open method from the cache
					AssetOpenMethodCache.Remove(AssetInfo.AssetPath);
				}

				//@todo if needed, broadcast the event for every asset. It is possible, however, that a single shared editor was opened by the AssetTypeActions, not an editor for each asset.
				/*AssetEditorOpenedEvent.Broadcast(Asset);*/

				GWarn->EndSlowTask();
			}
		}
		else
		{
			// Asset types don't match or some are already open, so just open individual editors for the unopened ones
			for (UObject* Asset : Assets)
			{
				if (!SkipOpenAssets.Contains(Asset))
				{
					OpenEditorForAsset(Asset, ToolkitMode, OpenedFromLevelEditor, true, OpenedMethod);
				}
			}
		}
	}

	return NumNullAssets == 0;
}

bool UAssetEditorSubsystem::OpenDebugObjectInEditor(UObject* DebugObject)
{
	if (DebugObject)
	{
		DebugObjectDelegate.Broadcast(DebugObject);

		for (UClass* DebugClass = DebugObject->GetClass(); !!DebugClass; DebugClass = DebugClass->GetSuperClass())
		{
			if (const FAssetEditorOpenDebugObject* Delegate = DebugObjectDelegates.Find(DebugClass->GetFName()))
			{
				Delegate->Broadcast(DebugObject);
			}
		}

		return true;
	}

	return false;
}

FAssetEditorOpenDebugObject* UAssetEditorSubsystem::GetDebugObjectDelegateForClass(UClass* InClass)
{
	if (InClass)
	{
		return &DebugObjectDelegates.FindOrAdd(InClass->GetFName());
	}

	return nullptr;
}

bool UAssetEditorSubsystem::OpenEditorForAssets(const TArray<UObject*>& Assets, const EAssetTypeActivationOpenedMethod OpenedMethod)
{
	return OpenEditorForAssets_Advanced(Assets, EToolkitMode::Standalone, TSharedPtr<IToolkitHost>(), OpenedMethod);
}

TOptional<EAssetOpenMethod> UAssetEditorSubsystem::GetAssetBeingOpenedMethod(TObjectPtr<UObject> Asset)
{
	if(!Asset)
	{
		return TOptional<EAssetOpenMethod>();
	}
	
	EAssetOpenMethod* OpenMethod = AssetOpenMethodCache.Find(Asset->GetPathName());

	return OpenMethod ? *OpenMethod : TOptional<EAssetOpenMethod>();
}

TOptional<EAssetOpenMethod> UAssetEditorSubsystem::GetAssetsBeingOpenedMethod(TArray<TObjectPtr<UObject>> Assets)
{
	TOptional<EAssetOpenMethod> FoundOpenMethod;

	// If an asset editor supports opening multiple assets, if any of them are being opened in read only mode we ask the asset editor to open in read only mode
	for(const TObjectPtr<UObject>& Asset : Assets)
	{
		TOptional<EAssetOpenMethod> AssetOpenMethod = GetAssetBeingOpenedMethod(Asset);

		if(AssetOpenMethod.IsSet())
		{
			FoundOpenMethod = AssetOpenMethod.GetValue();

			if(FoundOpenMethod == EAssetOpenMethod::View)
			{
				return FoundOpenMethod;
			}
		}
	}

	return FoundOpenMethod;
}

void UAssetEditorSubsystem::AddReadOnlyAssetFilter(const FName& Owner, const FReadOnlyAssetFilter& ReadOnlyAssetFilter)
{
	ReadOnlyAssetFilters.Add(Owner, ReadOnlyAssetFilter);
}

void UAssetEditorSubsystem::RemoveReadOnlyAssetFilter(const FName& Owner)
{
	ReadOnlyAssetFilters.Remove(Owner);
}


void UAssetEditorSubsystem::HandleRequestOpenAssetMessage(const FAssetEditorRequestOpenAsset& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	OpenEditorForAsset(Message.AssetName);
}

UObject* UAssetEditorSubsystem::FindOrLoadAssetForOpening(const FSoftObjectPath& AssetPath)
{
	UObject* Object = FindObject<UObject>(AssetPath.GetAssetPath());
	if (!Object)
	{
		Object = LoadObject<UObject>(nullptr, *AssetPath.GetAssetPathString(), nullptr, LOAD_NoRedirects);
	}

	return Object;
	
}

void UAssetEditorSubsystem::OpenEditorForAsset(const FSoftObjectPath& AssetPath, const EAssetTypeActivationOpenedMethod OpenedMethod)
{
	if (UObject* Object = FindOrLoadAssetForOpening(AssetPath))
	{
		OpenEditorForAsset(Object, EToolkitMode::Standalone, TSharedPtr<IToolkitHost>(), true, OpenedMethod);
	}
}

void UAssetEditorSubsystem::OpenEditorForAsset(const FString& AssetPathName, const EAssetTypeActivationOpenedMethod OpenedMethod)
{
	OpenEditorForAsset(FSoftObjectPath(AssetPathName), OpenedMethod);
}

bool UAssetEditorSubsystem::HandleTicker(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UAssetEditorSubsystem_HandleTicker);

	if (bRequestRestorePreviouslyOpenAssets)
	{
		RestorePreviouslyOpenAssets();
		bRequestRestorePreviouslyOpenAssets = false;
	}

	return true;
}

void UAssetEditorSubsystem::RequestRestorePreviouslyOpenAssets()
{
	// We defer the restore so that we guarantee that it happens once initialization is complete
	bRequestRestorePreviouslyOpenAssets = true;
}

UEdMode* UAssetEditorSubsystem::CreateEditorModeWithToolsOwner(FEditorModeID ModeID, FEditorModeTools& Owner)
{
	FRegisteredModeInfo* ScriptableMode = EditorModes.Find(ModeID);
	if (ScriptableMode && ScriptableMode->ModeClass.IsValid())
	{
		UEdMode* Instance = NewObject<UEdMode>(GetTransientPackage(), ScriptableMode->ModeClass.Get());
		Instance->Owner = &Owner;
		Instance->Initialize();

		return Instance;
	}

	// If we couldn't find a valid UEdMode based class, attempt to make a UEdMode wrapped FEdMode
	FEditorModeInfo LegacyModeInfo;
	if (FindEditorModeInfo(ModeID, LegacyModeInfo))
	{
		ULegacyEdModeWrapper* LegacyEditorMode = NewObject<ULegacyEdModeWrapper>(GetTransientPackage());
		if (LegacyEditorMode->CreateLegacyMode(ModeID, Owner))
		{
			LegacyEditorMode->Initialize();
			return LegacyEditorMode;
		}
	}

	return nullptr;
}

bool UAssetEditorSubsystem::FindEditorModeInfo(const FEditorModeID& InModeID, FEditorModeInfo& OutModeInfo) const
{
	if (!IsEditorModeAllowed(InModeID))
	{
		return false;
	}
	
	const TSharedRef<IEditorModeFactory>* ModeFactory = FEditorModeRegistry::Get().GetFactoryMap().Find(InModeID);
	if (ModeFactory)
	{
		OutModeInfo = (*ModeFactory)->GetModeInfo();
		return true;
	}

	if (!EditorModes.Contains(InModeID))
	{
		return false;
	}

	OutModeInfo = EditorModes[InModeID].ModeInfo;
	return true;
}

TArray<FEditorModeInfo> UAssetEditorSubsystem::GetEditorModeInfoOrderedByPriority() const
{
	TArray<FEditorModeInfo> ModeInfoArray;

	for (const auto& Pair : FEditorModeRegistry::Get().GetFactoryMap())
	{
		FEditorModeInfo ModeInfo = Pair.Value->GetModeInfo();
		if (IsEditorModeAllowed(ModeInfo.ID))
		{
			ModeInfoArray.Add(MoveTemp(ModeInfo));
		}
	}
	for (const auto& EditorMode : EditorModes)
	{
		const FEditorModeInfo& ModeInfo = EditorMode.Value.ModeInfo;
		if (IsEditorModeAllowed(ModeInfo.ID))
		{
			ModeInfoArray.Add(ModeInfo);
		}
	}

	ModeInfoArray.Sort([](const FEditorModeInfo& A, const FEditorModeInfo& B) {
		return A.PriorityOrder < B.PriorityOrder;
	});

	return ModeInfoArray;
}




void UAssetEditorSubsystem::RegisterUAssetEditor(UAssetEditor* NewAssetEditor)
{
	OwnedAssetEditors.Add(NewAssetEditor);
}

void UAssetEditorSubsystem::UnregisterUAssetEditor(UAssetEditor* RemovedAssetEditor)
{
	OwnedAssetEditors.Remove(RemovedAssetEditor);
}

FRegisteredModesChangedEvent& UAssetEditorSubsystem::OnEditorModesChanged()
{
	return OnEditorModesChangedEvent;
}

FOnModeRegistered& UAssetEditorSubsystem::OnEditorModeRegistered()
{
	return OnEditorModeRegisteredEvent;
}

FOnModeUnregistered& UAssetEditorSubsystem::OnEditorModeUnregistered()
{
	return OnEditorModeUnregisteredEvent;
}

void UAssetEditorSubsystem::RestorePreviouslyOpenAssets()
{
	TArray<FString> AllOpenAssets;
	TArray<FString> FilteredOpenAssets;
	
	GConfig->GetArray(TEXT("AssetEditorSubsystem"), TEXT("OpenAssetsAtExit"), AllOpenAssets, GEditorPerProjectIni);

	if(!RecentAssetsFilter.IsBound())
	{
		FilteredOpenAssets = AllOpenAssets;
	}
	else
	{
		for(const FString& Asset : AllOpenAssets)
		{
			if(RecentAssetsFilter.Execute(Asset))
			{
				FilteredOpenAssets.Add(Asset);
			}
		}
	}

	bool bCleanShutdown = true;
	GConfig->GetBool(TEXT("AssetEditorSubsystem"), TEXT("CleanShutdown"), bCleanShutdown, GEditorPerProjectIni);

	bool bDebuggerAttachedLastSession = false;
	GConfig->GetBool(TEXT("AssetEditorSubsystem"), TEXT("DebuggerAttached"), bDebuggerAttachedLastSession, GEditorPerProjectIni);
	
	SaveOpenAssetEditors(false);

	/** True if the last editor run crashed and did not have a debugger attached
	 * A "clean" shutdown for our purposes is the logical NOT of this, which includes clean shutdowns without a debugger
	 * along with any shutdowns when a debugger is attached
	 */
	bool bCrashedWithoutDebugger = !bDebuggerAttachedLastSession && !bCleanShutdown;

	if (FilteredOpenAssets.Num() > 0)
	{
		// This option overrides the saved setting
		if(bAutoRestoreAndDisableSaving.IsSet())
		{
			// If bAutoRestoreAndDisableSaving is true, we automatically restore the opened assets
			if(bAutoRestoreAndDisableSaving.GetValue())
			{
				OpenEditorsForAssets(FilteredOpenAssets);
			}
			return;
		}
		
		/* If we crashed without a debugger attached, always prompt regardless of what the user previously said
		 * to make sure the user can never get stuck in a crash loop due to corrupted assets etc
		 */
		if(bCrashedWithoutDebugger)
		{
			SpawnRestorePreviouslyOpenAssetsNotification(!bCrashedWithoutDebugger, FilteredOpenAssets);
			return;
		}

		const ERestoreOpenAssetTabsMethod AutoRestoreMethod = GetDefault<UEditorLoadingSavingSettings>()->RestoreOpenAssetTabsOnRestart;

		switch(AutoRestoreMethod)
		{
		case ERestoreOpenAssetTabsMethod::AlwaysPrompt:
			{
				SpawnRestorePreviouslyOpenAssetsNotification(!bCrashedWithoutDebugger, FilteredOpenAssets);
				break;
			}
			
		case ERestoreOpenAssetTabsMethod::NeverRestore:
			// Do nothing here since the user does not want to restore anything
			break;
		case ERestoreOpenAssetTabsMethod::AlwaysRestore:
			{
				// Pretend that we showed the notification and that the user clicked "Restore Now"
				OpenEditorsForAssets(FilteredOpenAssets);
				break;
			}
		}
	}
}

void UAssetEditorSubsystem::SetAutoRestoreAndDisableSaving(const bool bInAutoRestoreAndDisableSaving)
{
	// We preserve legacy behavior, where true is the same but false translates to not having the override set in the new logic
	SetAutoRestoreAndDisableSavingOverride(bInAutoRestoreAndDisableSaving ? bInAutoRestoreAndDisableSaving : TOptional<bool>());
}

void UAssetEditorSubsystem::SetAutoRestoreAndDisableSavingOverride(TOptional<bool> bInAutoRestoreAndDisableSaving)
{
	bAutoRestoreAndDisableSaving = bInAutoRestoreAndDisableSaving;
	
	// Disable any pending request to avoid trying to restore previously opened assets twice
	bRequestRestorePreviouslyOpenAssets = false;

}

TOptional<bool> UAssetEditorSubsystem::GetAutoRestoreAndDisableSavingOverride() const
{
	return bAutoRestoreAndDisableSaving;
}

void UAssetEditorSubsystem::SetRecentAssetsFilter(const FMainMRUFavoritesList::FDoesMRUFavoritesItemPassFilter& InFilter)
{
	RecentAssetsFilter = InFilter;

	if(RecentAssetsList)
	{
		RecentAssetsList->RegisterDoesMRUFavoritesItemPassFilterDelegate(InFilter);
	}
}

void UAssetEditorSubsystem::SpawnRestorePreviouslyOpenAssetsNotification(const bool bCleanShutdown, const TArray<FString>& AssetsToOpen)
{
	FText NotificationMessage = bCleanShutdown
		? LOCTEXT("ReopenAssetEditorsAfterClose", "{0} asset {0}|plural(one=editor was,other=editors were) open when the editor was last closed. Would you like to re-open {0}|plural(one=it,other=them)?")
		: LOCTEXT("ReopenAssetEditorsAfterCrash", "{0} asset {0}|plural(one=editor was,other=editors were) open when the editor quit unexpectedly. Would you like to re-open {0}|plural(one=it,other=them)?");
	NotificationMessage = FText::Format(NotificationMessage, AssetsToOpen.Num());

	FNotificationInfo Info = FNotificationInfo(NotificationMessage);

	// Add the buttons
	Info.ButtonDetails.Add(FNotificationButtonInfo(
		LOCTEXT("ReopenAssetEditors_Confirm", "Yes"),
		FText(),
		FSimpleDelegate::CreateUObject(this, &UAssetEditorSubsystem::OnConfirmRestorePreviouslyOpenAssets, AssetsToOpen),
		SNotificationItem::CS_None
	));
	Info.ButtonDetails.Add(FNotificationButtonInfo(
		LOCTEXT("ReopenAssetEditors_Cancel", "No"),
		FText(),
		FSimpleDelegate::CreateUObject(this, &UAssetEditorSubsystem::OnCancelRestorePreviouslyOpenAssets),
		SNotificationItem::CS_None
	));

	Info.bFireAndForget = false;

	// We want the auto-save to be subtle
	Info.bUseLargeFont = false;
	Info.bUseThrobber = false;
	Info.bUseSuccessFailIcons = false;

	// Only let the user suppress the non-crash version
	if (bCleanShutdown)
	{
		bRememberMyChoiceChecked = false;
		
		Info.CheckBoxState = TAttribute<ECheckBoxState>::CreateLambda([this]()
		{
			return bRememberMyChoiceChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		});
		Info.CheckBoxStateChanged = FOnCheckStateChanged::CreateLambda([this](ECheckBoxState NewState)
		{
			bRememberMyChoiceChecked = (NewState == ECheckBoxState::Checked) ? true : false;
		});
		Info.CheckBoxText = LOCTEXT("RememberCheckBoxMessage", "Remember my choice");
	}

	// Close any existing notification
	TSharedPtr<SNotificationItem> RestorePreviouslyOpenAssetsNotification = RestorePreviouslyOpenAssetsNotificationPtr.Pin();
	if (RestorePreviouslyOpenAssetsNotification.IsValid())
	{
		RestorePreviouslyOpenAssetsNotification->ExpireAndFadeout();
	}

	RestorePreviouslyOpenAssetsNotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
}

void UAssetEditorSubsystem::OnConfirmRestorePreviouslyOpenAssets(TArray<FString> AssetsToOpen)
{
	// Close any existing notification
	TSharedPtr<SNotificationItem> RestorePreviouslyOpenAssetsNotification = RestorePreviouslyOpenAssetsNotificationPtr.Pin();
	if (RestorePreviouslyOpenAssetsNotification.IsValid())
	{
		RestorePreviouslyOpenAssetsNotification->SetExpireDuration(0.0f);
		RestorePreviouslyOpenAssetsNotification->SetFadeOutDuration(0.5f);
		RestorePreviouslyOpenAssetsNotification->ExpireAndFadeout();

		// Change the saved setting to AlwaysRestore if the user checked "Remember my choice"
		if(bRememberMyChoiceChecked)
		{
			UEditorLoadingSavingSettings& Settings = *GetMutableDefault<UEditorLoadingSavingSettings>();
			Settings.RestoreOpenAssetTabsOnRestart = ERestoreOpenAssetTabsMethod::AlwaysRestore;
			Settings.PostEditChange();
		}
		
		// we do this inside the condition so that it can only be done once. 
		OpenEditorsForAssets(AssetsToOpen);

	}
}

void UAssetEditorSubsystem::OnCancelRestorePreviouslyOpenAssets()
{
	// Close any existing notification
	TSharedPtr<SNotificationItem> RestorePreviouslyOpenAssetsNotification = RestorePreviouslyOpenAssetsNotificationPtr.Pin();
	if (RestorePreviouslyOpenAssetsNotification.IsValid())
	{
		// Change the saved setting to NeverRestore if the user checked "Remember my choice"
		if(bRememberMyChoiceChecked)
		{
			UEditorLoadingSavingSettings& Settings = *GetMutableDefault<UEditorLoadingSavingSettings>();
			Settings.RestoreOpenAssetTabsOnRestart = ERestoreOpenAssetTabsMethod::NeverRestore;
			Settings.PostEditChange();
		}
		
		RestorePreviouslyOpenAssetsNotification->SetExpireDuration(0.0f);
		RestorePreviouslyOpenAssetsNotification->SetFadeOutDuration(0.5f);
		RestorePreviouslyOpenAssetsNotification->ExpireAndFadeout();
	}
}

bool UAssetEditorSubsystem::ShouldShowRecentAsset(const FString& AssetName, int32 RecentAssetIndex, const FName& InAssetEditorName) const
{
	const FString* AssetEditorForCurrRecent = RecentAssetToAssetEditorMap.Find(AssetName);

	// If this asset wasn't opened in any valid asset editor (e.g Levels)
	if(!AssetEditorForCurrRecent)
	{
		return false;
	}

	// If we have a valid asset editor we are adding assets for
	if(!InAssetEditorName.IsNone())
	{
		// If this asset was not opened in InAssetEditorName, ignore it
		if(*AssetEditorForCurrRecent != InAssetEditorName.ToString())
		{
			return false;
		}
	}
		
	// If this asset does not pass the set filter, ignore it
	if (!RecentAssetsList->MRUItemPassesCurrentFilter(RecentAssetIndex))
	{
		return false;
	}

	return true;
}

bool UAssetEditorSubsystem::ShouldShowRecentAssetsMenu(const FName& InAssetEditorName) const
{
	// If we have no recent assets at all
	if(RecentAssetsList->GetNumItems() == 0)
	{
		return false;
	}

	for ( int32 CurRecentIndex = 0; CurRecentIndex < RecentAssetsList->GetNumItems() && CurRecentIndex < MaxRecentAssetsToShowInMenu; ++CurRecentIndex )
	{
		const FString& CurRecent = RecentAssetsList->GetMRUItem(CurRecentIndex);

		// If any of the assets in the recents wil be shown, we show the menu
		if(ShouldShowRecentAsset(CurRecent, CurRecentIndex, InAssetEditorName))
		{
			return true;
		}
	}

	return false;
}

void UAssetEditorSubsystem::CreateRecentAssetsMenu(UToolMenu* InMenu, const FName InAssetEditorName)
{
	FToolMenuSection& Section = InMenu->FindOrAddSection("Recents");
	
	const bool bShowingContentVersePath = IAssetTools::Get().ShowingContentVersePath();

	// Keep adding assets until we reach the end of the MRU list, or we reach the max allowed assets
	for ( int32 CurRecentIndex = 0; CurRecentIndex < RecentAssetsList->GetNumItems() && CurRecentIndex < MaxRecentAssetsToShowInMenu; ++CurRecentIndex )
	{
		const FString& CurRecent = RecentAssetsList->GetMRUItem(CurRecentIndex);

		if(!ShouldShowRecentAsset(CurRecent, CurRecentIndex, InAssetEditorName))
		{
			continue;
		}

		FSoftObjectPath AssetPath(CurRecent);

		if (AssetPath.GetAssetFName().IsNone())
		{
			// Mimic LoadObject in FindOrLoadAssetForOpening falling back to the asset with the same name as the package if the asset name is not provided.
			AssetPath = FSoftObjectPath::ConstructFromAssetPath(FTopLevelAssetPath(AssetPath.GetLongPackageFName(), FPackageName::GetShortFName(AssetPath.GetLongPackageFName())));
		}

		FText CurPath;
		if (bShowingContentVersePath)
		{
			if (const IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
			{
				FAssetData AssetData;
				if (AssetRegistry->TryGetAssetByObjectPath(AssetPath, AssetData) == UE::AssetRegistry::EExists::Exists)
				{
					UE::Core::FVersePath VersePath = AssetData.GetVersePath();
					if (VersePath.IsValid())
					{
						CurPath = FText::FromString(MoveTemp(VersePath).ToString());
					}
				}
			}
		}

		if (CurPath.IsEmpty())
		{
			CurPath = FText::FromString(CurRecent);
		}

		const FText ToolTip = FText::Format( LOCTEXT( "RecentAssetsToolTip", "Open {0}" ), CurPath );
		const FText Label = FText::FromString( AssetPath.GetAssetName() );

		Section.AddMenuEntry(
		NAME_None,
		Label,
		ToolTip,
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this, AssetPath]()
				{
					if (UObject* Object = FindOrLoadAssetForOpening(AssetPath))
					{
						FText ErrorMessage;

						// Try to open the asset in edit mode. If that is not allowed, try to open it in read only mode
						if(CanOpenEditorForAsset(Object, EAssetTypeActivationOpenedMethod::Edit, &ErrorMessage))
						{
							OpenEditorForAsset(Object, EToolkitMode::Standalone, TSharedPtr<IToolkitHost>(), true, EAssetTypeActivationOpenedMethod::Edit);
						}
						else if(CanOpenEditorForAsset(Object, EAssetTypeActivationOpenedMethod::View, &ErrorMessage))
						{
							OpenEditorForAsset(Object, EToolkitMode::Standalone, TSharedPtr<IToolkitHost>(), true, EAssetTypeActivationOpenedMethod::View);
						}
					}
				})
			)
		);
	}
}

void UAssetEditorSubsystem::RegisterLevelEditorMenuExtensions()
{
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.MainMenu.File");
	
	FToolMenuSection& Section = Menu->FindOrAddSection("FileAsset");

	Section.AddDynamicEntry("FileRecentAssets", FNewToolMenuSectionDelegate::CreateLambda([this](FToolMenuSection& InSection)
	{
		// Since we want to show all asset types for the Level Editor, we use an empty asset editor name
		const FName AssetEditorName = FName();

		if (!ShouldShowRecentAssetsMenu(AssetEditorName))
		{
			return;
		}
		
		InSection.AddSubMenu(
			"RecentAssetsSubmenu",
			LOCTEXT("RecentAssetsSubmenu_Label", "Recent Assets"),
			FText::Format(LOCTEXT("RecentAssetsSubMenu_ToolTip", "Access your last {0} recently opened assets"), MaxRecentAssetsToShowInMenu),
			FNewToolMenuDelegate::CreateUObject(this, &UAssetEditorSubsystem::CreateRecentAssetsMenu, AssetEditorName),
			false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.RecentAssets"));
	}));
}

void UAssetEditorSubsystem::CreateRecentAssetsMenuForEditor(const IAssetEditorInstance* InAssetEditorInstance, FToolMenuSection& InSection)
{
	if(!InAssetEditorInstance)
	{
		return;
	}

	InSection.AddDynamicEntry("FileRecentAssetEditorAssets", FNewToolMenuSectionDelegate::CreateLambda([this, InAssetEditorInstance](FToolMenuSection& InSection)
	{
		const FName EditingAssetTypeName = InAssetEditorInstance->GetEditingAssetTypeName();

		// For generic asset editors (or any other special cases) that don't have one singular type of asset they are editing, show all recent assets
		const FName AssetEditorName = EditingAssetTypeName.IsNone() ? FName() : InAssetEditorInstance->GetEditorName();

		if(!ShouldShowRecentAssetsMenu(AssetEditorName))
		{
			return;
		}

		// Show all Recent Assets
		if(AssetEditorName.IsNone())
		{
			InSection.AddSubMenu(
			"RecentAssetsSubmenu",
			LOCTEXT("RecentAssetsSubmenu_Label", "Recent Assets"),
			FText::Format(LOCTEXT("RecentAssetsSubMenu_ToolTip", "Access your last {0} recently opened assets"), MaxRecentAssetsToShowInMenu),
			FNewToolMenuDelegate::CreateUObject(this, &UAssetEditorSubsystem::CreateRecentAssetsMenu, AssetEditorName),
			false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.RecentAssets")
			);
		}
		// Only show recent assets opened in this Asset Editor
		else
		{
			// Example submenu name: "Recent Material Assets" for the Material Editor
			InSection.AddSubMenu(
				"RecentAssetEditorAssetsSubmenu",
				FText::Format(LOCTEXT("RecentAssetEditorAssetsSubmenu_Label", "Recent {0} Assets"), FText::FromName(EditingAssetTypeName)),
				FText::Format(LOCTEXT("RecentAssetEditorAssetsSubmenu_Tooltip", "Access your recently opened {0} assets"), FText::FromName(EditingAssetTypeName)),
				FNewToolMenuDelegate::CreateUObject(this, &UAssetEditorSubsystem::CreateRecentAssetsMenu, AssetEditorName),
				false,
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.RecentAssets")
			);
		}
		
	}));
;
}

void UAssetEditorSubsystem::SaveOpenAssetEditors(const bool bOnShutdown)
{
	SaveRecentAssets(bOnShutdown);

	// We are already saving the open asset editors manually before bSavingOnShutdown is true. This is to avoid saving that there are no open asset editors b/c the editor is shutting down
	// If we are restoring the layout, bAutoRestoreAndDisable saving is true
	if (!bSavingOnShutdown && !bAutoRestoreAndDisableSaving.IsSet())
	{
		TArray<FString> OpenAssets;
		
		for (const TPair<IAssetEditorInstance*, FAssetEntry>& EditorPair : OpenedEditors)
		{
			IAssetEditorInstance* Editor = EditorPair.Key;
			if (Editor != nullptr)
			{
				UObject* EditedObject = EditorPair.Value.ObjectPtr.Get();
				if (EditedObject != nullptr && Editor->IncludeAssetInRestoreOpenAssetsPrompt(EditedObject))
				{
					// only record assets that have a valid saved package
					UPackage* Package = EditedObject->GetOutermost();
					if (Package != nullptr && Package->GetFileSize() != 0 )
					{
						OpenAssets.Add(EditedObject->GetPathName());
					}
				}
			}
		}

		GConfig->SetArray(TEXT("AssetEditorSubsystem"), TEXT("OpenAssetsAtExit"), OpenAssets, GEditorPerProjectIni);
		GConfig->SetBool(TEXT("AssetEditorSubsystem"), TEXT("CleanShutdown"), bOnShutdown, GEditorPerProjectIni);
		GConfig->SetBool(TEXT("AssetEditorSubsystem"), TEXT("DebuggerAttached"), FPlatformMisc::IsDebuggerPresent(), GEditorPerProjectIni);

		GConfig->Flush(false, GEditorPerProjectIni);
	}
}

void UAssetEditorSubsystem::SaveOpenAssetEditors(const bool bOnShutdown, const bool bCancelIfDebugger)
{
	SaveOpenAssetEditors(bOnShutdown);
}

void UAssetEditorSubsystem::HandlePackageReloaded(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent)
{
	static TArray<TWeakObjectPtr<UObject>> PendingAssetsToOpen;

	if (InPackageReloadPhase == EPackageReloadPhase::PrePackageFixup)
	{
		/** Call close for all old assets even if not open, so global callback will go off */
		TArray<UObject*> ObjectsToClose;
		const TMap<UObject*, UObject*>& RepointedMap = InPackageReloadedEvent->GetRepointedObjects();

		for (const TPair<UObject*, UObject*>& RepointPair : RepointedMap)
		{
			if (RepointPair.Key->IsAsset())
			{
				ObjectsToClose.Add(RepointPair.Key);
			}
		}

		/** Look for replacement for assets that are open now so we can reopen */
		for (TPair<FAssetEntry, IAssetEditorInstance*>& AssetEditorPair : OpenedAssets)
		{
			UObject* NewAsset = nullptr;
			if (AssetEditorPair.Key.RawPtr && InPackageReloadedEvent->GetRepointedObject(AssetEditorPair.Key.RawPtr, NewAsset))
			{
				if (NewAsset)
				{
					PendingAssetsToOpen.AddUnique(NewAsset);
				}

				// Not validating the asset here since we'd want to close editors for garbage collected assets
				UObject* OldAsset = AssetEditorPair.Key.RawPtr;
				ObjectsToClose.AddUnique(OldAsset);

				// Gather other assets referencing reloaded asset and mark their editors to be closed too.
				TArray<FReferencerInformation> AssetInternalReferencers, AssetExternalReferencers;
				AssetEditorPair.Key.RawPtr->RetrieveReferencers(&AssetInternalReferencers, &AssetExternalReferencers);
				for (const FReferencerInformation& Ref : AssetExternalReferencers)
				{
					ObjectsToClose.AddUnique(Ref.Referencer);

					if (!FindEditorsForAssetAndSubObjects(Ref.Referencer).IsEmpty())
					{
						PendingAssetsToOpen.AddUnique(Ref.Referencer);
					}
				}
			}
		}

		int32 NumAssetEditorsClosed = 0;
		for (UObject* OldAsset : ObjectsToClose)
		{
			NumAssetEditorsClosed += CloseAllEditorsForAsset(OldAsset);
		}

		if (NumAssetEditorsClosed > 0)
		{
			// Closing asset editors might have have left objects pending GC that still reference the asset we're about to reload
			// Run a GC now to ensure those are cleaned up before the fix-up phase happens
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
		}

	}

	if (InPackageReloadPhase == EPackageReloadPhase::PostBatchPostGC)
	{
		for (TWeakObjectPtr<UObject>& NewAsset : PendingAssetsToOpen)
		{
			if (NewAsset.IsValid())
			{
				OpenEditorForAsset(NewAsset.Get());
			}
		}
		PendingAssetsToOpen.Reset();
	}
}

void UAssetEditorSubsystem::OpenEditorsForAssets(const TArray<FSoftObjectPath>& AssetsToOpen)
{
	for (const FSoftObjectPath& AssetName : AssetsToOpen)
	{
		OpenEditorForAsset(AssetName);
	}
}

void UAssetEditorSubsystem::OpenEditorsForAssets(const TArray<FString>& AssetsToOpen, const EAssetTypeActivationOpenedMethod OpenedMethod)
{
	for (const FString& AssetName : AssetsToOpen)
	{
		OpenEditorForAsset(AssetName, OpenedMethod);
	}
}

void UAssetEditorSubsystem::OpenEditorsForAssets(const TArray<FName>& AssetsToOpen, const EAssetTypeActivationOpenedMethod OpenedMethod)
{
	for (const FName& AssetName : AssetsToOpen)
	{
		OpenEditorForAsset(AssetName.ToString(), OpenedMethod);
	}
}

bool UAssetEditorSubsystem::CanOpenEditorForAsset(UObject* Asset, const EAssetTypeActivationOpenedMethod OpenedMethod, FText* OutErrorMsg)
{
	if (!Asset)
	{
		if(OutErrorMsg)
		{
			*OutErrorMsg = LOCTEXT("AssetNull", "Opening Asset editor failed because asset is null");
		}
		
		return false;
	}
	
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
	TWeakPtr<IAssetTypeActions> AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(Asset->GetClass());

	// First we check with the asset type action to see if it supports the open method
	if(AssetTypeActions.IsValid())
	{
		if(!AssetTypeActions.Pin()->SupportsOpenedMethod(OpenedMethod))
		{
			if(OutErrorMsg)
			{
				if(OpenedMethod == EAssetTypeActivationOpenedMethod::Edit)
				{
					*OutErrorMsg = FText::Format(LOCTEXT("AssetTypeDoesntSupportEdit", "A {0} does not support being edited!"), AssetTypeActions.Pin()->GetName());
				}
				else
				{
					*OutErrorMsg = FText::Format(LOCTEXT("AssetTypeDoesntSupportReadOnly", "A {0} does not support being opened in read only mode!"), AssetTypeActions.Pin()->GetName());
				}

			}
			return false;
		}
	}
	else
	{
		// Disallow opening an asset editor for classes
		if(Asset->IsA<UClass>())
		{
			if(OutErrorMsg)
			{
				*OutErrorMsg = LOCTEXT("UClassesCantBeOpened", "UClasses cannot be opened in Asset Editors!");
			}
			return false;
		}
	}

	// If the asset needs to be edited, make sure that is possible
	if(OpenedMethod == EAssetTypeActivationOpenedMethod::Edit)
	{
		if(!IsAssetEditable(Asset))
		{
			if(OutErrorMsg)
			{
				*OutErrorMsg = LOCTEXT("AssetCantBeEdited", "Unable to Edit Cooked asset");
			}
			return false;
		}
	}

	if(OpenedMethod == EAssetTypeActivationOpenedMethod::View)
	{
		if (UPackage* Package = Asset->GetPackage())
		{
			for(const TPair<FName, FReadOnlyAssetFilter>& Filter : ReadOnlyAssetFilters)
			{
				if(!Filter.Value.Execute(Package->GetPathName()))
				{
					if(OutErrorMsg)
					{
						*OutErrorMsg = LOCTEXT("AssetDoesntSupportOpenMethod", "This asset does not support being opened in read only mode.");
					}
					return false;
				}
			}
		}
		else
		{
			return false; // failsafe, but the package should exist at this point
		}

		
	}

	return true; 
}

void UAssetEditorSubsystem::RegisterEditorModes()
{
	for (FThreadSafeObjectIterator EditorModeIter(UEdMode::StaticClass()); EditorModeIter; ++EditorModeIter)
	{
		UEdMode* EditorMode = Cast<UEdMode>(*EditorModeIter);
		UClass* ModeClass = EditorMode->GetClass();
		if (ModeClass->HasAnyClassFlags(CLASS_Abstract | CLASS_Interface))
		{
			continue;
		}

		FEditorModeInfo EditorModeInfo = EditorMode->GetModeInfo();

		if (EditorModes.Contains(EditorModeInfo.ID))
		{
			TWeakObjectPtr<UClass> RegisteredClass = EditorModes[EditorModeInfo.ID].ModeClass;
			UE_LOGF(
				LogAssetEditorSubsystem,
				Warning,
				"UAssetEditorSubsystem::RegisterEditorModes : Attempting to initialize duplicate mode with name '%ls'. Conflicting classes: '%ls' and '%ls'.",
				*EditorModeInfo.ID.ToString(),
				*ModeClass->GetName(),
				*RegisteredClass.Get()->GetName()
			);
			continue;
		}

		EditorModes.Add(
			EditorModeInfo.ID,
			FRegisteredModeInfo{ ModeClass, EditorModeInfo }
		);

		OnEditorModeRegisteredEvent.Broadcast(EditorModeInfo.ID);
	}

	// Initialize Legacy FEditorModes
	FEditorModeRegistry::Get().Initialize();

	OnEditorModesChangedEvent.Broadcast();
}

void UAssetEditorSubsystem::UnregisterEditorModes()
{
	FEditorModeRegistry::Get().Shutdown();

	for (const auto& RegisteredMode : EditorModes)
	{
		OnEditorModeUnregisteredEvent.Broadcast(RegisteredMode.Value.ModeInfo.ID);
	}
	OnEditorModesChangedEvent.Broadcast();
	EditorModes.Empty();
}

void UAssetEditorSubsystem::OnSMInstanceElementsEnabled()
{
	// Let the modes know that SM instance elements may have been enabled or disabled and update state accordingly
	OnEditorModesChanged().Broadcast();
}

FNamePermissionList& UAssetEditorSubsystem::GetAllowedEditorModes()
{
	return AllowedEditorModes;
}

bool UAssetEditorSubsystem::IsEditorModeAllowed(const FName ModeId) const
{
	return AllowedEditorModes.PassesFilter(ModeId);
}

#undef LOCTEXT_NAMESPACE
