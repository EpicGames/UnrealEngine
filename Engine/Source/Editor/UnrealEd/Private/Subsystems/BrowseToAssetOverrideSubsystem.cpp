// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/BrowseToAssetOverrideSubsystem.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "Editor.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BrowseToAssetOverrideSubsystem)

UBrowseToAssetOverrideSubsystem* UBrowseToAssetOverrideSubsystem::Get()
{
	if (GEditor)
	{
		return GEditor->GetEditorSubsystem<UBrowseToAssetOverrideSubsystem>();
	}
	return nullptr;
}

bool UBrowseToAssetOverrideSubsystem::HasBrowseToAssetOverride(const UObject* Object) const
{
	FTopLevelAssetPath AssetPath;
	return TryGetBrowseToAssetOverride(Object, AssetPath);
}

bool UBrowseToAssetOverrideSubsystem::TryGetBrowseToAssetOverride(const UObject* Object, FTopLevelAssetPath& OutAssetPath) const
{
	// Actors also allow this to be overridden per-instance via meta-data
	// If set, that takes priority over any per-class overrides
	if (const AActor* Actor = Cast<AActor>(Object))
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const FString& ActorBrowseToAssetOverride = Actor->GetBrowseToAssetOverride();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		if (!ActorBrowseToAssetOverride.IsEmpty())
		{
			OutAssetPath = FTopLevelAssetPath(ActorBrowseToAssetOverride);
			return OutAssetPath.IsValid();
		}
	}

	// Walk the class hierarchy to see if there's a valid per-class override for this instance
	if (PerClassOverrides.Num() > 0)
	{
		for (const UClass* Class = Object->GetClass(); Class; Class = Class->GetSuperClass())
		{
			if (const FGetBrowseToAssetOverrideDelegate* CallbackPtr = PerClassOverrides.Find(Class->GetClassPathName()))
			{
				if (CallbackPtr->IsBound())
				{
					OutAssetPath = CallbackPtr->Execute(Object);
					if (OutAssetPath.IsValid())
					{
						return true;
					}
				}
			}
		}
	}

	// Query all the class interfaces to see if there's a valid per-interface override for this instance
	if (PerInterfaceOverrides.Num() > 0)
	{
		for (const UClass* ObjectClass = Object->GetClass(); ObjectClass; ObjectClass = ObjectClass->GetSuperClass())
		{
			for (const FImplementedInterface& Interface : ObjectClass->Interfaces)
			{
				if (const FGetBrowseToAssetOverrideDelegate* CallbackPtr = PerInterfaceOverrides.Find(Interface.Class->GetClassPathName()))
				{
					if (CallbackPtr->IsBound())
					{
						OutAssetPath = CallbackPtr->Execute(Object);
						if (OutAssetPath.IsValid())
						{
							return true;
						}
					}
				}
			}
		}

	}

	return false;
}

bool UBrowseToAssetOverrideSubsystem::TryGetBrowseToAssetOverride(const UObject* Object, FAssetData& OutAssetData) const
{
	FTopLevelAssetPath AssetPath;
	if (TryGetBrowseToAssetOverride(Object, AssetPath))
	{
		if (AssetPath.GetAssetName().IsNone())
		{
			TArray<FAssetData> AssetDatas;
			if (IAssetRegistry::GetChecked().GetAssetsByPackageName(AssetPath.GetPackageName(), AssetDatas) && !AssetDatas.IsEmpty())
			{
				OutAssetData = MoveTemp(AssetDatas[0]);
				return true;
			}
		}
		else
		{
			OutAssetData = IAssetRegistry::GetChecked().GetAssetByObjectPath(FSoftObjectPath::ConstructFromAssetPath(AssetPath));
			return OutAssetData.IsValid();
		}
	}

	return false;
}

FName UBrowseToAssetOverrideSubsystem::GetBrowseToAssetOverride(const UObject* Object)
{
	FTopLevelAssetPath AssetPath;
	if (TryGetBrowseToAssetOverride(Object, AssetPath))
	{
		return AssetPath.GetPackageName();
	}

	return FName();
}

void UBrowseToAssetOverrideSubsystem::RegisterBrowseToAssetOverrideForClass(const FTopLevelAssetPath& Class, FGetBrowseToAssetOverrideDelegate&& Callback)
{
	PerClassOverrides.Add(Class, MoveTemp(Callback));
}

void UBrowseToAssetOverrideSubsystem::RegisterBrowseToAssetOverrideForClass(const FTopLevelAssetPath& Class, FBrowseToAssetOverrideDelegate&& Callback)
{
	PerClassOverrides.Add(Class, FGetBrowseToAssetOverrideDelegate::CreateLambda([Callback = MoveTemp(Callback)](const UObject* Object)
	{
		FName PackageName;

		if (Callback.IsBound())
		{
			PackageName = Callback.Execute(Object);
		}

		return FTopLevelAssetPath(PackageName, FName());
	}));
}
	
void UBrowseToAssetOverrideSubsystem::UnregisterBrowseToAssetOverrideForClass(const FTopLevelAssetPath& Class)
{
	PerClassOverrides.Remove(Class);
}

void UBrowseToAssetOverrideSubsystem::RegisterBrowseToAssetOverrideForInterface(const FTopLevelAssetPath& Interface, FGetBrowseToAssetOverrideDelegate&& Callback)
{
	PerInterfaceOverrides.Add(Interface, MoveTemp(Callback));
}

void UBrowseToAssetOverrideSubsystem::RegisterBrowseToAssetOverrideForInterface(const FTopLevelAssetPath& Interface, FBrowseToAssetOverrideDelegate&& Callback)
{
	PerInterfaceOverrides.Add(Interface, FGetBrowseToAssetOverrideDelegate::CreateLambda([Callback = MoveTemp(Callback)](const UObject* Object)
	{
		FName PackageName;

		if (Callback.IsBound())
		{
			PackageName = Callback.Execute(Object);
		}

		return FTopLevelAssetPath(PackageName, FName());
	}));
}

void UBrowseToAssetOverrideSubsystem::UnregisterBrowseToAssetOverrideForInterface(const FTopLevelAssetPath& Interface)
{
	PerInterfaceOverrides.Remove(Interface);
}
