// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/BrushEditingSubsystem.h"
#include "Editor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BrushEditingSubsystem)


UBrushEditingSubsystem::UBrushEditingSubsystem()
	: UEditorSubsystem()
{

}
