// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/CollectionManagerScriptingSubsystem.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "CollectionManagerModule.h"
#include "ContentBrowserModule.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "ICollectionSource.h"
#include "IContentBrowserSingleton.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CollectionManagerScriptingSubsystem)

DEFINE_LOG_CATEGORY_STATIC(LogCollectionManagerScripting, Log, All);

#define LOCTEXT_NAMESPACE "CollectionManagerScriptingSubsystem"

namespace CollectionManagerScriptingSubsystemUtil
{

void LogLastCollectionManagerWarning(const FText& Warning)
{
	UE_LOGF(LogCollectionManagerScripting, Warning, "%ls", *Warning.ToString());
}

FString ECollectionScriptingShareType_To_String(const ECollectionScriptingShareType ShareType)
{
	static const UEnum* Enum = FindObject<UEnum>(nullptr, TEXT("/Script/CollectionManager.ECollectionScriptingShareType"), EFindObjectFlags::ExactClass);
	if (Enum)
	{
		return Enum->GetDisplayNameTextByValue((uint64)ShareType).ToString();
	}
	else
	{
		return {};
	}
}

ECollectionShareType::Type ECollectionScriptingShareType_To_ECollectionShareType(const ECollectionScriptingShareType ShareType)
{
	static_assert((int32)ECollectionShareType::CST_Local == (int32)ECollectionScriptingShareType::Local + 1, "ECollectionShareType::CST_Local is expected to be ECollectionScriptingShareType::Local + 1");
	static_assert((int32)ECollectionShareType::CST_Private == (int32)ECollectionScriptingShareType::Private + 1, "ECollectionShareType::CST_Private is expected to be ECollectionScriptingShareType::Private + 1");
	static_assert((int32)ECollectionShareType::CST_Shared == (int32)ECollectionScriptingShareType::Shared + 1, "ECollectionShareType::CST_Shared is expected to be ECollectionScriptingShareType::Shared + 1");

	return (ECollectionShareType::Type)((int32)ShareType + 1);
}

ECollectionScriptingShareType ECollectionShareType_To_ECollectionScriptingShareType(const ECollectionShareType::Type ShareType)
{
	static_assert((int32)ECollectionShareType::CST_Local == (int32)ECollectionScriptingShareType::Local + 1, "ECollectionShareType::CST_Local is expected to be ECollectionScriptingShareType::Local + 1");
	static_assert((int32)ECollectionShareType::CST_Private == (int32)ECollectionScriptingShareType::Private + 1, "ECollectionShareType::CST_Private is expected to be ECollectionScriptingShareType::Private + 1");
	static_assert((int32)ECollectionShareType::CST_Shared == (int32)ECollectionScriptingShareType::Shared + 1, "ECollectionShareType::CST_Shared is expected to be ECollectionScriptingShareType::Shared + 1");

	return (ECollectionScriptingShareType)((int32)ShareType - 1);
}

struct FConstants
{
	static const FConstants& Get()
	{
		static FConstants Instance;
		return Instance;
	}

	const FText ContainerNotFound;
	const FText CollectionNotFound;
	const FText CollectionExistsInContainer;
	const FText ContainerMismatch;

private:
	FConstants() :
		ContainerNotFound(LOCTEXT("ContainerNotFound", "No container with the name '{0}' could be found.")),
		CollectionNotFound(LOCTEXT("CollectionNotFound", "No collection with the name '{0}' and share type '{1}' was found in container with name '{2}'")),
		CollectionExistsInContainer(LOCTEXT("CollectionExistsInContainer", "Container '{0}' already has a collection named '{1}' with share type '{2}'.")),
		ContainerMismatch(LOCTEXT("ContainerMismatch", "Collections must be in the same container to reparent. Found target collection in container '{0}' and new parent collection in container '{1}'."))
	{}
};

} // namespace CollectionManagerScriptingSubsystemUtil

TArray<FCollectionScriptingContainerSource> UCollectionManagerScriptingSubsystem::GetCollectionContainers()
{
	TArray<TSharedPtr<ICollectionContainer>> CollectionContainers;
	FCollectionManagerModule::GetModule().Get().GetCollectionContainers(CollectionContainers);

	TArray<FCollectionScriptingContainerSource> ContainerNames;
	ContainerNames.Reserve(CollectionContainers.Num());
	Algo::Transform(CollectionContainers, ContainerNames, [](const TSharedPtr<ICollectionContainer>& CollectionContainer)
		{
			return FCollectionScriptingContainerSource {
				.Name = CollectionContainer->GetCollectionSource()->GetName(),
				.Title = CollectionContainer->GetCollectionSource()->GetTitle()
			};
		}
	);

	return ContainerNames;
}

bool UCollectionManagerScriptingSubsystem::CreateCollection(const FCollectionScriptingContainerSource Container, const FName Collection, const ECollectionScriptingShareType ShareType, FCollectionScriptingRef& OutNewCollection)
{
	const TSharedPtr<ICollectionContainer> CollectionContainer = FCollectionManagerModule::GetModule().Get().FindCollectionContainer(Container.Name);
	if (!CollectionContainer.IsValid())
	{
		CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(FText::Format(CollectionManagerScriptingSubsystemUtil::FConstants::Get().ContainerNotFound, FText::FromName(Container.Name)));
		return false;
	}

	const ECollectionShareType::Type CastShareType = CollectionManagerScriptingSubsystemUtil::ECollectionScriptingShareType_To_ECollectionShareType(ShareType);
	if (CollectionContainer->CollectionExists(Collection, CastShareType))
	{
		CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(FText::Format(CollectionManagerScriptingSubsystemUtil::FConstants::Get().CollectionExistsInContainer, { CollectionContainer->GetCollectionSource()->GetTitle(), FText::FromName(Collection), CastShareType }));
		return false;
	}

	FText Error;
	if (!CollectionContainer->IsValidCollectionName(Collection.ToString(), ECollectionShareType::CST_All, &Error))
	{
		CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(Error);
		return false;
	}

	if (!CollectionContainer->CreateCollection(Collection, CastShareType, ECollectionStorageMode::Static, &Error))
	{
		CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(Error);
		return false;
	}

	OutNewCollection.Name = Collection;
	OutNewCollection.ShareType = ShareType;
	OutNewCollection.Container = CollectionContainer->GetCollectionSource()->GetName();

	return true;
}

bool UCollectionManagerScriptingSubsystem::CreateOrEmptyCollection(const FCollectionScriptingContainerSource Container, const FName Collection, const ECollectionScriptingShareType ShareType, FCollectionScriptingRef& OutNewOrEmptyCollection)
{
	const TSharedPtr<ICollectionContainer> CollectionContainer = FCollectionManagerModule::GetModule().Get().FindCollectionContainer(Container.Name);
	if (!CollectionContainer.IsValid())
	{
		CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(FText::Format(CollectionManagerScriptingSubsystemUtil::FConstants::Get().ContainerNotFound, FText::FromName(Container.Name)));
		return false;
	}

	const ECollectionShareType::Type CastShareType = CollectionManagerScriptingSubsystemUtil::ECollectionScriptingShareType_To_ECollectionShareType(ShareType);
	if (CollectionContainer->CollectionExists(Collection, CastShareType))
	{
		FText Error;
		if (!CollectionContainer->EmptyCollection(Collection, CastShareType, &Error))
		{
			CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(Error);
			return false;
		}
	}
	else
	{
		FText Error;
		if (!CollectionContainer->IsValidCollectionName(Collection.ToString(), ECollectionShareType::CST_All, &Error))
		{
			CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(Error);
			return false;
		}

		if (!CollectionContainer->CreateCollection(Collection, CastShareType, ECollectionStorageMode::Static, &Error))
		{
			CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(Error);
			return false;
		}
	}

	OutNewOrEmptyCollection.Name = Collection;
	OutNewOrEmptyCollection.ShareType = ShareType;
	OutNewOrEmptyCollection.Container = CollectionContainer->GetCollectionSource()->GetName();

	return true;
}

bool UCollectionManagerScriptingSubsystem::GetCollections(const FCollectionScriptingContainerSource Container, TArray<FCollectionScriptingRef>& OutCollections)
{
	const TSharedPtr<ICollectionContainer> CollectionContainer = FCollectionManagerModule::GetModule().Get().FindCollectionContainer(Container.Name);
	if (!CollectionContainer.IsValid())
	{
		CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(FText::Format(CollectionManagerScriptingSubsystemUtil::FConstants::Get().ContainerNotFound, { FText::FromName(Container.Name) }));
		return false;
	}

	TArray<FCollectionNameType> Collections;
	CollectionContainer->GetCollections(Collections);

	OutCollections.Reserve(Collections.Num());
	Algo::Transform(Collections, OutCollections, [ContainerName = CollectionContainer->GetCollectionSource()->GetName()](const FCollectionNameType& CollectionName)
		{
			return FCollectionScriptingRef {
				.Container = ContainerName,
				.Name = CollectionName.Name,
				.ShareType = CollectionManagerScriptingSubsystemUtil::ECollectionShareType_To_ECollectionScriptingShareType(CollectionName.Type)
			};
		});

	return OutCollections.Num() > 0;
}

bool UCollectionManagerScriptingSubsystem::DestroyCollection(const FCollectionScriptingRef& Collection)
{
	const TSharedPtr<ICollectionContainer> CollectionContainer = FCollectionManagerModule::GetModule().Get().FindCollectionContainer(Collection.Container);
	if (!CollectionContainer.IsValid())
	{
		CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(FText::Format(CollectionManagerScriptingSubsystemUtil::FConstants::Get().ContainerNotFound, { FText::FromName(Collection.Container) }));
		return false;
	}

	FText Error;
	if (!CollectionContainer->DestroyCollection(Collection.Name, CollectionManagerScriptingSubsystemUtil::ECollectionScriptingShareType_To_ECollectionShareType(Collection.ShareType), &Error))
	{
		CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(Error);
		return false;
	}

	return true;
}

bool UCollectionManagerScriptingSubsystem::RenameCollection(const FCollectionScriptingRef& Collection, const FName NewName, const ECollectionScriptingShareType NewShareType)
{
	const TSharedPtr<ICollectionContainer> CollectionContainer = FCollectionManagerModule::GetModule().Get().FindCollectionContainer(Collection.Container);
	if (!CollectionContainer.IsValid())
	{
		CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(FText::Format(CollectionManagerScriptingSubsystemUtil::FConstants::Get().ContainerNotFound, { FText::FromName(Collection.Container) }));
		return false;
	}

	FText Error;
	if (!CollectionContainer->IsValidCollectionName(NewName.ToString(), ECollectionShareType::CST_All, &Error))
	{
		CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(Error);
		return false;
	}

	if (!CollectionContainer->RenameCollection(Collection.Name, CollectionManagerScriptingSubsystemUtil::ECollectionScriptingShareType_To_ECollectionShareType(Collection.ShareType),
		NewName, CollectionManagerScriptingSubsystemUtil::ECollectionScriptingShareType_To_ECollectionShareType(NewShareType), &Error))
	{
		CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(Error);
		return false;
	}

	return true;
}

bool UCollectionManagerScriptingSubsystem::ReparentCollection(const FCollectionScriptingRef& Collection, const FCollectionScriptingRef NewParentCollection)
{
	const TSharedPtr<ICollectionContainer> CollectionContainer = FCollectionManagerModule::GetModule().Get().FindCollectionContainer(Collection.Container);
	if (!CollectionContainer.IsValid())
	{
		CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(FText::Format(CollectionManagerScriptingSubsystemUtil::FConstants::Get().ContainerNotFound, { FText::FromName(Collection.Container) }));
		return false;
	}

	// If the intention is to reparent to a different collection, make sure its in the same container.
	if (!NewParentCollection.Name.IsNone() && Collection.Container != NewParentCollection.Container)
	{
		CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(FText::Format(CollectionManagerScriptingSubsystemUtil::FConstants::Get().ContainerMismatch, { FText::FromName(Collection.Container), FText::FromName(NewParentCollection.Container) }));
		return false;
	}

	FText Error;
	if (!CollectionContainer->ReparentCollection(Collection.Name, CollectionManagerScriptingSubsystemUtil::ECollectionScriptingShareType_To_ECollectionShareType(Collection.ShareType),
		NewParentCollection.Name, CollectionManagerScriptingSubsystemUtil::ECollectionScriptingShareType_To_ECollectionShareType(NewParentCollection.ShareType), &Error))
	{
		CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(Error);
		return false;
	}

	return true;
}

bool UCollectionManagerScriptingSubsystem::EmptyCollection(const FCollectionScriptingRef& Collection)
{
	const TSharedPtr<ICollectionContainer> CollectionContainer = FCollectionManagerModule::GetModule().Get().FindCollectionContainer(Collection.Container);
	if (!CollectionContainer.IsValid())
	{
		CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(FText::Format(CollectionManagerScriptingSubsystemUtil::FConstants::Get().ContainerNotFound, { FText::FromName(Collection.Container) }));
		return false;
	}

	FText Error;
	if (!CollectionContainer->EmptyCollection(Collection.Name, CollectionManagerScriptingSubsystemUtil::ECollectionScriptingShareType_To_ECollectionShareType(Collection.ShareType), &Error))
	{
		CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(Error);
		return false;
	}

	return true;
}

bool UCollectionManagerScriptingSubsystem::AddAssetToCollection(const FCollectionScriptingRef& Collection, const FSoftObjectPath& AssetPath)
{
	return AddAssetsToCollection(Collection, TArray<FSoftObjectPath>({ AssetPath }));
}

bool UCollectionManagerScriptingSubsystem::AddAssetDataToCollection(const FCollectionScriptingRef& Collection, const FAssetData& AssetData)
{
	return AddAssetsToCollection(Collection, TArray<FSoftObjectPath>({ AssetData.GetSoftObjectPath() }));
}

bool UCollectionManagerScriptingSubsystem::AddAssetPtrToCollection(const FCollectionScriptingRef& Collection, const UObject* AssetPtr)
{
	return AddAssetsToCollection(Collection, TArray<FSoftObjectPath>({ FSoftObjectPath(AssetPtr) }));
}

bool UCollectionManagerScriptingSubsystem::AddAssetsToCollection(const FCollectionScriptingRef& Collection, const TArray<FSoftObjectPath>& AssetPaths)
{
	const TSharedPtr<ICollectionContainer> CollectionContainer = FCollectionManagerModule::GetModule().Get().FindCollectionContainer(Collection.Container);
	if (!CollectionContainer.IsValid())
	{
		CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(FText::Format(CollectionManagerScriptingSubsystemUtil::FConstants::Get().ContainerNotFound, { FText::FromName(Collection.Container) }));
		return false;
	}

	FText Error;
	const ECollectionShareType::Type ShareType = CollectionManagerScriptingSubsystemUtil::ECollectionScriptingShareType_To_ECollectionShareType(Collection.ShareType);
	if (!CollectionContainer->CollectionExists(Collection.Name, ShareType))
	{
		if (!CollectionContainer->IsValidCollectionName(Collection.Name.ToString(), ECollectionShareType::CST_All, &Error))
		{
			CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(Error);
			return false;
		}

		if (!CollectionContainer->CreateCollection(Collection.Name, ShareType, ECollectionStorageMode::Static, &Error))
		{
			CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(Error);
			return false;
		}
	}

	if (!CollectionContainer->AddToCollection(Collection.Name, ShareType, AssetPaths, nullptr, &Error))
	{
		CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(Error);
		return false;
	}

	return true;
}

bool UCollectionManagerScriptingSubsystem::AddAssetDatasToCollection(const FCollectionScriptingRef& Collection, const TArray<FAssetData>& AssetDatas)
{
	TArray<FSoftObjectPath> ObjectPaths;
	ObjectPaths.Reserve(AssetDatas.Num());
	Algo::Transform(AssetDatas, ObjectPaths, [](const FAssetData& AssetData) { return AssetData.GetSoftObjectPath(); });

	return AddAssetsToCollection(Collection, ObjectPaths);
}

bool UCollectionManagerScriptingSubsystem::AddAssetPtrsToCollection(const FCollectionScriptingRef& Collection, const TArray<UObject*>& AssetPtrs)
{
	TArray<FSoftObjectPath> ObjectPaths;
	ObjectPaths.Reserve(AssetPtrs.Num());
	Algo::Transform(AssetPtrs, ObjectPaths, [](const UObject* AssetPtr) { return FSoftObjectPath(AssetPtr); });

	return AddAssetsToCollection(Collection, ObjectPaths);
}

bool UCollectionManagerScriptingSubsystem::RemoveAssetFromCollection(const FCollectionScriptingRef& Collection, const FSoftObjectPath& AssetPath)
{
	return RemoveAssetsFromCollection(Collection, TArray<FSoftObjectPath>({ AssetPath }));
}

bool UCollectionManagerScriptingSubsystem::RemoveAssetDataFromCollection(const FCollectionScriptingRef& Collection, const FAssetData& AssetData)
{
	return RemoveAssetsFromCollection(Collection, TArray<FSoftObjectPath>({ AssetData.GetSoftObjectPath() }));
}

bool UCollectionManagerScriptingSubsystem::RemoveAssetPtrFromCollection(const FCollectionScriptingRef& Collection, const UObject* AssetPtr)
{
	return RemoveAssetsFromCollection(Collection, TArray<FSoftObjectPath>({ FSoftObjectPath(AssetPtr) }));
}

bool UCollectionManagerScriptingSubsystem::RemoveAssetsFromCollection(const FCollectionScriptingRef& Collection, const TArray<FSoftObjectPath>& AssetPaths)
{
	const TSharedPtr<ICollectionContainer> CollectionContainer = FCollectionManagerModule::GetModule().Get().FindCollectionContainer(Collection.Container);
	if (!CollectionContainer.IsValid())
	{
		CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(FText::Format(CollectionManagerScriptingSubsystemUtil::FConstants::Get().ContainerNotFound, { FText::FromName(Collection.Container) }));
		return false;
	}

	FText Error;
	if (!CollectionContainer->RemoveFromCollection(Collection.Name, CollectionManagerScriptingSubsystemUtil::ECollectionScriptingShareType_To_ECollectionShareType(Collection.ShareType), AssetPaths, nullptr, &Error))
	{
		CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(Error);
		return false;
	}

	return true;
}

bool UCollectionManagerScriptingSubsystem::RemoveAssetDatasFromCollection(const FCollectionScriptingRef& Collection, const TArray<FAssetData>& AssetDatas)
{
	TArray<FSoftObjectPath> ObjectPaths;
	ObjectPaths.Reserve(AssetDatas.Num());
	Algo::Transform(AssetDatas, ObjectPaths, [](const FAssetData& AssetData) { return AssetData.GetSoftObjectPath(); });

	return RemoveAssetsFromCollection(Collection, ObjectPaths);
}

bool UCollectionManagerScriptingSubsystem::RemoveAssetPtrsFromCollection(const FCollectionScriptingRef& Collection, const TArray<UObject*>& AssetPtrs)
{
	TArray<FSoftObjectPath> ObjectPaths;
	ObjectPaths.Reserve(AssetPtrs.Num());
	Algo::Transform(AssetPtrs, ObjectPaths, [](const UObject* AssetPtr) { return FSoftObjectPath(AssetPtr); });

	return RemoveAssetsFromCollection(Collection, ObjectPaths);
}

bool UCollectionManagerScriptingSubsystem::CollectionExists(const FCollectionScriptingContainerSource Container, const FName Collection, const ECollectionScriptingShareType ShareType)
{
	const TSharedPtr<ICollectionContainer> CollectionContainer = FCollectionManagerModule::GetModule().Get().FindCollectionContainer(Container.Name);
	if (!CollectionContainer.IsValid())
	{
		CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(FText::Format(CollectionManagerScriptingSubsystemUtil::FConstants::Get().ContainerNotFound, { FText::FromName(Container.Name) }));
		return false;
	}

	return CollectionContainer->CollectionExists(Collection, CollectionManagerScriptingSubsystemUtil::ECollectionScriptingShareType_To_ECollectionShareType(ShareType));
}

bool UCollectionManagerScriptingSubsystem::GetCollectionsByName(const FCollectionScriptingContainerSource Container, const FName Collection, TArray<FCollectionScriptingRef>& OutCollections)
{
	const TSharedPtr<ICollectionContainer> CollectionContainer = FCollectionManagerModule::GetModule().Get().FindCollectionContainer(Container.Name);
	if (!CollectionContainer.IsValid())
	{
		CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(FText::Format(CollectionManagerScriptingSubsystemUtil::FConstants::Get().ContainerNotFound, { FText::FromName(Container.Name) }));
		return false;
	}

	TArray<FCollectionNameType> FoundCollections;
	CollectionContainer->GetCollections(Collection, FoundCollections);

	OutCollections.Reserve(FoundCollections.Num());
	Algo::Transform(FoundCollections, OutCollections, [ContainerName = Container.Name](const FCollectionNameType& Collection)
	{
		return FCollectionScriptingRef { 
			.Container = ContainerName,
			.Name = Collection.Name,
			.ShareType = CollectionManagerScriptingSubsystemUtil::ECollectionShareType_To_ECollectionScriptingShareType(Collection.Type)
		};
	});
	
	return OutCollections.Num() > 0;
}

bool UCollectionManagerScriptingSubsystem::GetAssetsInCollection(const FCollectionScriptingRef& Collection, TArray<FAssetData>& OutAssets)
{
	const TSharedPtr<ICollectionContainer> CollectionContainer = FCollectionManagerModule::GetModule().Get().FindCollectionContainer(Collection.Container);
	if (!CollectionContainer.IsValid())
	{
		CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(FText::Format(CollectionManagerScriptingSubsystemUtil::FConstants::Get().ContainerNotFound, { FText::FromName(Collection.Container) }));
		return false;
	}

	if (!CollectionContainer->CollectionExists(Collection.Name, CollectionManagerScriptingSubsystemUtil::ECollectionScriptingShareType_To_ECollectionShareType(Collection.ShareType)))
	{
		CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(FText::Format(CollectionManagerScriptingSubsystemUtil::FConstants::Get().CollectionNotFound,
			{ FText::FromName(Collection.Name), FText::FromString(CollectionManagerScriptingSubsystemUtil::ECollectionScriptingShareType_To_String(Collection.ShareType)), FText::FromName(Collection.Container) }));
		return false;
	}

	TArray<FSoftObjectPath> Assets;
	CollectionContainer->GetAssetsInCollection(Collection.Name, CollectionManagerScriptingSubsystemUtil::ECollectionScriptingShareType_To_ECollectionShareType(Collection.ShareType), Assets);

	const IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
	OutAssets.Reserve(Assets.Num());
	for (const FSoftObjectPath& Asset : Assets)
	{
		FAssetData AssetData = AssetRegistry.GetAssetByObjectPath(Asset);
		if (AssetData.IsValid())
		{
			OutAssets.Emplace(MoveTemp(AssetData));
		}
	}

	return true;
}

bool UCollectionManagerScriptingSubsystem::GetCollectionColor(const FCollectionScriptingRef& Collection, FLinearColor& OutColor)
{
	OutColor = FLinearColor::White;

	const TSharedPtr<ICollectionContainer> CollectionContainer = FCollectionManagerModule::GetModule().Get().FindCollectionContainer(Collection.Container);
	if (!CollectionContainer.IsValid())
	{
		CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(FText::Format(CollectionManagerScriptingSubsystemUtil::FConstants::Get().ContainerNotFound, { FText::FromName(Collection.Container) }));
		return false;
	}

	if (!CollectionContainer->CollectionExists(Collection.Name, CollectionManagerScriptingSubsystemUtil::ECollectionScriptingShareType_To_ECollectionShareType(Collection.ShareType)))
	{
		CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(FText::Format(CollectionManagerScriptingSubsystemUtil::FConstants::Get().CollectionNotFound,
			{ FText::FromName(Collection.Name), FText::FromString(CollectionManagerScriptingSubsystemUtil::ECollectionScriptingShareType_To_String(Collection.ShareType)), FText::FromName(Collection.Container) }));
		return false;
	}

	TOptional<FLinearColor> Color;
	FText Error;
	if (!CollectionContainer->GetCollectionColor(Collection.Name, CollectionManagerScriptingSubsystemUtil::ECollectionScriptingShareType_To_ECollectionShareType(Collection.ShareType), Color, &Error))
	{
		CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(Error);
		return false;
	}

	if (!Color.IsSet())
	{
		return false;
	}

	OutColor = Color.GetValue();
	return true;
}

bool UCollectionManagerScriptingSubsystem::SetCollectionColor(const FCollectionScriptingRef& Collection, const FLinearColor& Color)
{
	return SetCollectionColor(Collection, TOptional<FLinearColor>(Color));
}

bool UCollectionManagerScriptingSubsystem::ClearCollectionColor(const FCollectionScriptingRef& Collection)
{
	return SetCollectionColor(Collection, TOptional<FLinearColor>());
}

bool UCollectionManagerScriptingSubsystem::SetCollectionColor(const FCollectionScriptingRef& Collection, const TOptional<FLinearColor>& Color)
{
	const TSharedPtr<ICollectionContainer> CollectionContainer = FCollectionManagerModule::GetModule().Get().FindCollectionContainer(Collection.Container);
	if (!CollectionContainer.IsValid())
	{
		CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(FText::Format(CollectionManagerScriptingSubsystemUtil::FConstants::Get().ContainerNotFound, { FText::FromName(Collection.Container) }));
		return false;
	}

	if (!CollectionContainer->CollectionExists(Collection.Name, CollectionManagerScriptingSubsystemUtil::ECollectionScriptingShareType_To_ECollectionShareType(Collection.ShareType)))
	{
		CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(FText::Format(CollectionManagerScriptingSubsystemUtil::FConstants::Get().CollectionNotFound,
			{ FText::FromName(Collection.Name), FText::FromString(CollectionManagerScriptingSubsystemUtil::ECollectionScriptingShareType_To_String(Collection.ShareType)), FText::FromName(Collection.Container) }));
		return false;
	}

	FText Error;
	if (!CollectionContainer->SetCollectionColor(Collection.Name, CollectionManagerScriptingSubsystemUtil::ECollectionScriptingShareType_To_ECollectionShareType(Collection.ShareType), Color, &Error))
	{
		CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(Error);
		return false;
	}

	return true;
}

bool UCollectionManagerScriptingSubsystem::GetCollectionsContainingAsset(const FCollectionScriptingContainerSource Container, const FSoftObjectPath& AssetPath, TArray<FCollectionScriptingRef>& OutCollections)
{
	const TSharedPtr<ICollectionContainer> CollectionContainer = FCollectionManagerModule::GetModule().Get().FindCollectionContainer(Container.Name);
	if (!CollectionContainer.IsValid())
	{
		CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(FText::Format(CollectionManagerScriptingSubsystemUtil::FConstants::Get().ContainerNotFound, { FText::FromName(Container.Name) }));
		return false;
	}

	TArray<FCollectionNameType> CollectionNamesAndTypes;
	CollectionContainer->GetCollectionsContainingObject(AssetPath, CollectionNamesAndTypes);

	OutCollections.Reserve(CollectionNamesAndTypes.Num());
	for (const FCollectionNameType& CollectionNameAndType : CollectionNamesAndTypes)
	{
		OutCollections.Emplace(FCollectionScriptingRef{ .Container = Container.Name, .Name = CollectionNameAndType.Name, .ShareType = CollectionManagerScriptingSubsystemUtil::ECollectionShareType_To_ECollectionScriptingShareType(CollectionNameAndType.Type) });
	}

	return true;
}

bool UCollectionManagerScriptingSubsystem::GetCollectionsContainingAssetData(const FCollectionScriptingContainerSource Container, const FAssetData& AssetData, TArray<FCollectionScriptingRef>& OutCollections)
{
	return GetCollectionsContainingAsset(Container, AssetData.GetSoftObjectPath(), OutCollections);
}

bool UCollectionManagerScriptingSubsystem::GetCollectionsContainingAssetPtr(const FCollectionScriptingContainerSource Container, const UObject* AssetPtr, TArray<FCollectionScriptingRef>& OutCollections)
{
	return GetCollectionsContainingAsset(Container, FSoftObjectPath(AssetPtr), OutCollections);
}

FCollectionScriptingContainerSource UCollectionManagerScriptingSubsystem::GetBaseGameCollectionContainer() const
{
	FCollectionScriptingContainerSource Container;
	const TSharedRef<ICollectionContainer> CollectionContainer = FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer();
	Container.Name = CollectionContainer->GetCollectionSource()->GetName();
	Container.Title = CollectionContainer->GetCollectionSource()->GetTitle();

	return Container;
}

void UCollectionManagerScriptingSubsystem::SyncBrowserToCollections(const TArray<FCollectionScriptingRef>& Collections)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	TArray<FCollectionRef> CollectionsToSync;
	CollectionsToSync.Reserve(Collections.Num());
	for (const FCollectionScriptingRef& Collection : Collections)
	{
		const TSharedPtr<ICollectionContainer> CollectionContainer = FCollectionManagerModule::GetModule().Get().FindCollectionContainer(Collection.Container);
		if (!CollectionContainer.IsValid())
		{
			CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(FText::Format(CollectionManagerScriptingSubsystemUtil::FConstants::Get().ContainerNotFound, { FText::FromName(Collection.Container) }));
			continue;
		}

		const ECollectionShareType::Type ShareType = CollectionManagerScriptingSubsystemUtil::ECollectionScriptingShareType_To_ECollectionShareType(Collection.ShareType);
		if (!CollectionContainer->CollectionExists(Collection.Name, ShareType))
		{
			CollectionManagerScriptingSubsystemUtil::LogLastCollectionManagerWarning(FText::Format(CollectionManagerScriptingSubsystemUtil::FConstants::Get().CollectionNotFound,
				{ FText::FromName(Collection.Name), FText::FromString(CollectionManagerScriptingSubsystemUtil::ECollectionScriptingShareType_To_String(Collection.ShareType)), FText::FromName(Collection.Container) }));
			continue;
		}

		CollectionsToSync.Emplace(CollectionContainer, Collection.Name, ShareType);
	}

	if (!CollectionsToSync.IsEmpty())
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		ContentBrowserModule.Get().SyncBrowserToCollections(CollectionsToSync);
	}
}

#undef LOCTEXT_NAMESPACE
