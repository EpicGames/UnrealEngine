// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/EditorAssetSubsystem.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "AssetViewUtils.h"
#include "Editor.h"
#include "Editor/Transactor.h"
#include "EditorScriptingHelpers.h"
#include "FileHelpers.h"
#include "HAL/FileManager.h"
#include "Misc/EngineBuildSettings.h"
#include "Misc/PackageName.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "ObjectTools.h"
#include "PackageTools.h"
#include "Templates/ValueOrError.h"
#include "UObject/MetaData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorAssetSubsystem)

DEFINE_LOG_CATEGORY_STATIC(LogEditorAssetSubsystem, Log, All);

namespace UE::Editor::Private
{
	int32 AllowSavingAssetsDuringPIE = 0;
}

static FAutoConsoleVariableRef CVarAllowSavingAssetsDuringPIE(
	TEXT("Editor.AllowSavingAssetsDuringPIE"),
	UE::Editor::Private::AllowSavingAssetsDuringPIE,
	TEXT("Allow assets to get saved during PIE through the EditorAssetSubsystem. Set to 0 (default) to disable and 1 to enable this option."),
	ECVF_Default
);

namespace UE::EditorAssetUtils
{
	struct ErrorTag{};
}

template<typename T>
using TError = TValueOrError<UE::EditorAssetUtils::ErrorTag, T>;

namespace UE::EditorAssetUtils
{
static bool EnsureAssetsLoaded()
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	if (IAssetRegistry& AssetRegistry = AssetRegistryModule.Get(); AssetRegistry.IsLoadingAssets())
	{
		if (AssetRegistry.IsSearchAsync() && AssetRegistry.IsSearchAllAssets())
		{
			AssetRegistry.WaitForCompletion();
		}
		else
		{
			AssetRegistry.SearchAllAssets(true /* bSynchronousSearch */);
		}
	}
	return true;
}

static TValueOrError<FAssetData, FString> FindAssetDataFromAnyPath(const FString& AnyAssetPath)
{
	FString FailureReason;
	FString ObjectPath = EditorScriptingHelpers::ConvertAnyPathToSubObjectPath(AnyAssetPath, FailureReason);
	if (ObjectPath.IsEmpty())
	{
		return MakeError(FailureReason);
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(ObjectPath));
	if (!AssetData.IsValid())
	{
		ObjectPath = EditorScriptingHelpers::ConvertAnyPathToObjectPath(AnyAssetPath, FailureReason);
		if (ObjectPath.IsEmpty())
		{
			return MakeError(FailureReason);
		}

		AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(ObjectPath));
		if (!AssetData.IsValid())
		{
			return MakeError(FString::Printf(TEXT("The AssetData '%s' could not be found in the Asset Registry."), *ObjectPath));
		}
	}
	
	return MakeValue(AssetData);
}

static TValueOrError<UObject*, FString> LoadAssetFromData(const FAssetData& AssetData)
{
	if (!AssetData.IsValid())
	{
		return MakeError("Asset Data is not valid.");
	}

	UObject* FoundObject = AssetData.GetAsset();
	if (!IsValid(FoundObject))
	{
		return MakeError(FString::Printf(TEXT("The asset '%s' exists but was not able to be loaded."), *AssetData.GetObjectPathString()));
	}
	else if (!FoundObject->IsAsset())
	{
		return MakeError(FString::Printf(TEXT("'%s' is not a valid asset."), *AssetData.GetObjectPathString()));
	}
	return MakeValue(FoundObject);
}

static TValueOrError<UObject*, FString> LoadAssetFromPath(const FString& AssetPath)
{
	TValueOrError<FAssetData, FString> AssetDataResult = FindAssetDataFromAnyPath(AssetPath);
	if (AssetDataResult.HasError())
	{
		return MakeError(AssetDataResult.StealError());
	}
	return LoadAssetFromData(AssetDataResult.GetValue());
}

static TError<FString> IsARegisteredAsset(UObject* Object, bool bAllowSkipBrowsableTestForExternalObject = false)
{
	if (!IsValid(Object))
	{
		return MakeError(TEXT("The Asset is not valid."));
	}

	const bool bCanSkipIsBrowsable = bAllowSkipBrowsableTestForExternalObject && Object->IsPackageExternal();
	if (!bCanSkipIsBrowsable && !ObjectTools::IsObjectBrowsable(Object))
	{
		return MakeError(FString::Printf(TEXT("The object '%s' is not an asset."), *Object->GetName()));
	}

	FSoftObjectPath ObjectPath = FSoftObjectPath(Object);
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(ObjectPath);
	if (!AssetData.IsValid())
	{
		return MakeError(FString::Printf(TEXT("The AssetData '%s' could not be found in the Asset Registry."), *Object->GetPathName()));
	}

	return MakeValue();
}

static TError<FString> EnumerateAssetsInDirectory(const FString& AnyPathDirectoryPath, bool bRecursive, TArray<FAssetData>& OutResult, FString& OutDirectoryPath)
{
	OutResult.Reset();
	OutDirectoryPath.Reset();

	FString FailureReason;
	OutDirectoryPath = EditorScriptingHelpers::ConvertAnyPathToLongPackagePath(AnyPathDirectoryPath, FailureReason);
	if (OutDirectoryPath.IsEmpty())
	{
		return MakeError(FailureReason);
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	if (!AssetRegistryModule.Get().GetAssetsByPath(*OutDirectoryPath, OutResult, bRecursive))
	{
		return MakeError(FString::Printf(TEXT("Could not get assets from path '%s'"), *OutDirectoryPath));
	}
	
	return MakeValue();
}

enum class EPackageEnumerationFilter
{
	NoFilter,
	OnlyDirty
};

static TError<FString> EnumeratePackagesInDirectory(const FString& AnyDirectoryPath, EPackageEnumerationFilter EnumerationFilter, bool bRecursive, TArray<UPackage*>& OutResult)
{
	FString ValidDirectoryPath;
	TArray<FAssetData> Assets;
	if (TError<FString> Result = EnumerateAssetsInDirectory(AnyDirectoryPath, bRecursive, Assets, ValidDirectoryPath); Result.HasError())
	{
		return Result;
	}

	if (EnumerationFilter == EPackageEnumerationFilter::OnlyDirty)
	{
		for (const FAssetData& AssetData : Assets)
		{
			// Can't be dirty if not loaded
			if (AssetData.IsAssetLoaded())
			{
				if (UPackage* Package = AssetData.GetPackage(); Package && Package->IsDirty())
				{
					Package->FullyLoad();
					OutResult.AddUnique(Package);
				}
			}
		}
	}
	else
	{
		// load all assets
		for (const FAssetData& AssetData : Assets)
		{
			if (UPackage* Package = AssetData.GetPackage())
			{
				Package->FullyLoad();
				OutResult.AddUnique(Package);
			}
		}
	}
	return MakeValue();
}

static bool DeleteEmptyDirectoryFromDisk(const FString& LongPackagePath)
{
	struct FEmptyDirectoryVisitor : public IPlatformFile::FDirectoryVisitor
	{
		bool bIsEmpty = true;

		virtual bool Visit(const TCHAR*, bool bIsDirectory) override
		{
			if (!bIsDirectory)
			{
				bIsEmpty = false;
				return false; // abort searching
			}
			return true; // continue searching
		}
	};
	
	if (FString PathToDeleteOnDisk = UPackageTools::PackageNameToFilename(LongPackagePath); !PathToDeleteOnDisk.IsEmpty())
	{
		// Look for files on disk in case the directory contains things not tracked by the asset registry
		FEmptyDirectoryVisitor EmptyDirectoryVisitor;
		IFileManager::Get().IterateDirectoryRecursively(*PathToDeleteOnDisk, EmptyDirectoryVisitor);

		if (EmptyDirectoryVisitor.bIsEmpty)
		{
			return IFileManager::Get().DeleteDirectory(*PathToDeleteOnDisk, false, true);
		}
	}
	return false;
}

struct FDirectoryRenamePaths
{
	FString SourceDirectoryPath;
	FString SourceFilePath;
	FString DestinationDirectoryPath;
	FString DestinationFilePath;
};

static TValueOrError<FDirectoryRenamePaths, FString> GetDirectoryRenamePaths(const FString& SourceDirectoryPath, const FString& DestinationDirectoryPath)
{
	FDirectoryRenamePaths Paths;
	FString FailureReason;
	Paths.SourceDirectoryPath = EditorScriptingHelpers::ConvertAnyPathToLongPackagePath(SourceDirectoryPath, FailureReason);
	if (Paths.SourceDirectoryPath.IsEmpty())
	{
		return MakeError(FString::Printf(TEXT("Failed to convert the source path. %s"), *FailureReason));
	}

	Paths.SourceFilePath = FPaths::ConvertRelativePathToFull(UPackageTools::PackageNameToFilename(Paths.SourceDirectoryPath));
	if (Paths.SourceFilePath.IsEmpty())
	{
		return MakeError(FString::Printf(TEXT("Failed to convert the source path '%s' to a full path."), *Paths.SourceDirectoryPath));
	}

	Paths.DestinationDirectoryPath = EditorScriptingHelpers::ConvertAnyPathToLongPackagePath(DestinationDirectoryPath, FailureReason);
	if (Paths.DestinationDirectoryPath.IsEmpty())
	{
		return MakeError(FString::Printf(TEXT("Failed to convert the destination path. %s"), *FailureReason));
	}

	Paths.DestinationFilePath = FPaths::ConvertRelativePathToFull(UPackageTools::PackageNameToFilename(Paths.DestinationDirectoryPath));
	if (Paths.DestinationFilePath.IsEmpty())
	{
		return MakeError(FString::Printf(TEXT("Failed to convert the destination path '%s' to a full path."), *Paths.DestinationDirectoryPath));
	}
	
	return MakeValue(Paths);
}

static TError<FString> SetupDirectoryRename(const FDirectoryRenamePaths& Paths)
{
	// If the source directory doesn't exist on disk then it can't be operated on
	if (!IFileManager::Get().DirectoryExists(*Paths.SourceFilePath))
	{
		return MakeError(FString::Printf(TEXT("The source directory '%s' does not exist on disk."), *Paths.SourceFilePath));
	}

	// Create the destination directory if it doesn't already exist
	if (!IFileManager::Get().DirectoryExists(*Paths.DestinationFilePath))
	{
		const bool bTree = true;
		if (!IFileManager::Get().MakeDirectory(*Paths.DestinationFilePath, bTree))
		{
			return MakeError(FString::Printf(TEXT("The destination directory '%s' could not be created."), *Paths.DestinationFilePath));
		}
	}

	return MakeValue();
}

struct FDirectoryRenameAssetPaths
{
	TArray<FAssetData> SourceDirectoryAssetDatas;
	TArray<FString> DestinationDirectoryAssetPaths;
};

static TValueOrError<FDirectoryRenameAssetPaths, FString> GetDirectoryRenameAssetPaths(const FDirectoryRenamePaths& Paths)
{
	FDirectoryRenameAssetPaths AssetPaths;
	
	// Load all assets the directory contains
	// Because we want to rename a directory, we can't rename any files that can't be deleted
	FString OutPath;
	if (TError<FString> Result = EnumerateAssetsInDirectory(Paths.SourceDirectoryPath, true, AssetPaths.SourceDirectoryAssetDatas, OutPath); Result.HasError())
	{
		return MakeError(Result.StealError());
	}
	
	for (const FAssetData& AssetData : AssetPaths.SourceDirectoryAssetDatas)
	{
		FString PackageName = AssetData.PackageName.ToString();
		FString LongPackagePath = FPackageName::GetLongPackagePath(PackageName);

		// Remove source from the object name
		LongPackagePath.MidInline(Paths.SourceDirectoryPath.Len(), MAX_int32, EAllowShrinking::No);

		// Create AssetPath /Game/MyFolder/MyAsset.MyAsset
		FString NewAssetPackageName;
		if (LongPackagePath.IsEmpty())
		{
			NewAssetPackageName = FString::Printf(TEXT("%s/%s.%s"), *Paths.DestinationDirectoryPath, *AssetData.AssetName.ToString(), *AssetData.AssetName.ToString());
		}
		else
		{
			NewAssetPackageName = FString::Printf(TEXT("%s%s/%s.%s"), *Paths.DestinationDirectoryPath, *LongPackagePath, *AssetData.AssetName.ToString(), *AssetData.AssetName.ToString());
		}

		FString FailureReason;
		if (!EditorScriptingHelpers::IsAValidPathForCreateNewAsset(NewAssetPackageName, FailureReason))
		{
			return MakeError(FString::Printf(TEXT("Failed to validate the destination for asset '%s'. %s"), *AssetData.AssetName.ToString(), *FailureReason));
		}
		
		if (FPackageName::DoesPackageExist(NewAssetPackageName))
		{
			return MakeError(FString::Printf(TEXT("Failed to validate the destination for asset '%s'. There's already an asset at the destination."), *NewAssetPackageName));
		}

		// Keep AssetPath /Game/MyFolder
		AssetPaths.DestinationDirectoryAssetPaths.Add(NewAssetPackageName);
	}
	return MakeValue(AssetPaths);
}

template<typename TLessThan>
static void SortAssets(
	TArray<FAssetData>& Assets,
	TLessThan&& Predicate,
	EEditorAssetSortOrder SortOrder
)
{
	switch (SortOrder)
	{
	case EEditorAssetSortOrder::Ascending: Assets.Sort(Predicate); break;
	case EEditorAssetSortOrder::Descending:
		Assets.Sort([&Predicate](const FAssetData& Left, const FAssetData& Right)
		{
			return Predicate(Right, Left);
		});
		break;
	default: checkNoEntry(); break;
	}
}

template<typename TType>
static bool Sort(
	TArray<FAssetData>& Assets,
	FName MetaDataTag,
	TFunctionRef<bool(const FString& TagType, TType& Converted)> Converter,
	EEditorAssetSortOrder SortOrder
	)
{
	TMap<FSoftObjectPath, TType> MetaData;
	MetaData.Reserve(Assets.Num());

	for (const FAssetData& AssetData : Assets)
	{
		const FAssetTagValueRef Value = AssetData.TagsAndValues.FindTag(MetaDataTag);
		TType AssetTagValue;
		if (Value.IsSet() && Converter(Value.GetValue(), AssetTagValue))
		{
			MetaData.Add(AssetData.GetSoftObjectPath(), AssetTagValue);
		}
		else
		{
			UE_LOGF(LogEditorAssetSubsystem, Warning, "Not all assets have the tag '%ls'", *MetaDataTag.ToString());
			return false;
		}
	}

	SortAssets(Assets, [&MetaData](const FAssetData& Left, const FAssetData& Right)
	{
		return MetaData[Left.GetSoftObjectPath()] <= MetaData[Right.GetSoftObjectPath()]; 
	}, SortOrder);
	return true;
}
}

UEditorAssetSubsystem::UEditorAssetSubsystem()
{
}

void UEditorAssetSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	OnExtractAssetFromFile.AddUObject(this, &UEditorAssetSubsystem::CallOnExtractAssetFromFileDynamicArray);
}

void UEditorAssetSubsystem::Deinitialize()
{
	OnExtractAssetFromFile.RemoveAll(this);
}

// Load operations

// A wrapper around
//unreal.AssetRegistryHelpers.get_asset(unreal.AssetRegistryHelpers.get_asset_registry().get_asset_by_object_path("/Game/NewDataTable.NewDataTable"))
UObject* UEditorAssetSubsystem::LoadAsset(const FString& AssetPath)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE() || !UE::EditorAssetUtils::EnsureAssetsLoaded())
	{
		return nullptr;
	}
	
	TValueOrError<UObject*, FString> LoadedAssetResult = UE::EditorAssetUtils::LoadAssetFromPath(AssetPath);
	if (LoadedAssetResult.HasError())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "LoadAsset failed: %ls", *LoadedAssetResult.GetError());
		return nullptr;
	}
	return LoadedAssetResult.GetValue();
}

UClass* UEditorAssetSubsystem::LoadBlueprintClass(const FString& AssetPath)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE() || !UE::EditorAssetUtils::EnsureAssetsLoaded())
	{
		return nullptr;
	}

	TValueOrError<UObject*, FString> LoadedAssetResult = UE::EditorAssetUtils::LoadAssetFromPath(AssetPath);
	if (LoadedAssetResult.HasError())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "LoadBlueprintClass failed: %ls", *LoadedAssetResult.GetError());
		return nullptr;
	}

	UBlueprint* Blueprint = Cast<UBlueprint>(LoadedAssetResult.GetValue());
	if (!Blueprint)
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "LoadBlueprintClass failed: The asset is not a Blueprint.");
		return nullptr;
	}
	return Blueprint->GeneratedClass.Get();
}

FString UEditorAssetSubsystem::GetPathNameForLoadedAsset(UObject* LoadedAsset)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE() || !UE::EditorAssetUtils::EnsureAssetsLoaded())
	{
		return FString();
	}
	
	if (TError<FString> Result = UE::EditorAssetUtils::IsARegisteredAsset(LoadedAsset, true/*bAllowSkipBrowsableTestForExternalObject*/); Result.HasError())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "GetPathNameForLoadedAsset failed: %ls", *Result.GetError());
	}
	return LoadedAsset->GetPathName();
}

FAssetData UEditorAssetSubsystem::FindAssetData(const FString& AssetPath)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);
	
	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE() || !UE::EditorAssetUtils::EnsureAssetsLoaded())
	{
		return FAssetData();
	}

	auto AssetDataResult = UE::EditorAssetUtils::FindAssetDataFromAnyPath(AssetPath);
	if (AssetDataResult.HasError())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "FindAssetData failed: %ls", *AssetDataResult.GetError());
		return FAssetData();
	}
	return AssetDataResult.GetValue();
}

bool UEditorAssetSubsystem::DoesAssetExist(const FString& AssetPath)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE() || !UE::EditorAssetUtils::EnsureAssetsLoaded())
	{
		return false;
	}

	FString FailureReason;
	FString ObjectPath = EditorScriptingHelpers::ConvertAnyPathToSubObjectPath(AssetPath, FailureReason);
	if (ObjectPath.IsEmpty())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "DoesAssetExist failed: %ls", *FailureReason);
		return false;
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(ObjectPath));
	if (!AssetData.IsValid())
	{
		ObjectPath = EditorScriptingHelpers::ConvertAnyPathToObjectPath(AssetPath, FailureReason);
		if (ObjectPath.IsEmpty())
		{
			UE_LOGF(LogEditorAssetSubsystem, Error, "DoesAssetExist failed: %ls", *FailureReason);
			return false;
		}

		AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(ObjectPath));
		if (!AssetData.IsValid())
		{
			return false;
		}
	}

	return true;
}

bool UEditorAssetSubsystem::DoAssetsExist(const TArray<FString>& AssetPaths)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE() || !UE::EditorAssetUtils::EnsureAssetsLoaded())
	{
		return false;
	}

	FString FailureReason;
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

	for (const FString& Path : AssetPaths)
	{
		FString ObjectPath = EditorScriptingHelpers::ConvertAnyPathToSubObjectPath(Path, FailureReason);
		if (ObjectPath.IsEmpty())
		{
			UE_LOGF(LogEditorAssetSubsystem, Error, "DoAssetsExist failed: %ls", *FailureReason);
			return false;
		}

		FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(ObjectPath));
		if (!AssetData.IsValid())
		{
			ObjectPath = EditorScriptingHelpers::ConvertAnyPathToObjectPath(Path, FailureReason);
			if (ObjectPath.IsEmpty())
			{
				UE_LOGF(LogEditorAssetSubsystem, Error, "DoAssetsExist failed: %ls", *FailureReason);
				return false;
			}

			AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(ObjectPath));
			if (!AssetData.IsValid())
			{
				return false;
			}
		}
	}
	return true;
}

TArray<FString> UEditorAssetSubsystem::FindPackageReferencersForAsset(const FString& AnyAssetPath, bool bLoadAssetsToConfirm)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	TArray<FString> Result;
	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE() || !UE::EditorAssetUtils::EnsureAssetsLoaded())
	{
		return Result;
	}

	FString FailureReason;
	FString AssetPath = EditorScriptingHelpers::ConvertAnyPathToSubObjectPath(AnyAssetPath, FailureReason);
	if (AssetPath.IsEmpty())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "FindAssetPackageReferencers failed: %ls", *FailureReason);
		return Result;
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(AnyAssetPath));
	if (!AssetData.IsValid())
	{
		FString ObjectPath = EditorScriptingHelpers::ConvertAnyPathToObjectPath(AnyAssetPath, FailureReason);
		if (ObjectPath.IsEmpty())
		{
			UE_LOGF(LogEditorAssetSubsystem, Error, "FindAssetPackageReferencers failed: %ls", *FailureReason);
			return Result;
		}
	}

	// Find the reference in packages. Load them to confirm the reference.
	TArray<FName> PackageReferencers;
	AssetRegistryModule.Get().GetReferencers(*FPackageName::ObjectPathToPackageName(AssetPath), PackageReferencers, UE::AssetRegistry::EDependencyCategory::Package);

	if (bLoadAssetsToConfirm)
	{
		// Load the asset to confirm 
		TValueOrError<UObject*, FString> LoadedAssetResult = UE::EditorAssetUtils::LoadAssetFromPath(AssetPath);
		if (LoadedAssetResult.HasError())
		{
			UE_LOGF(LogEditorAssetSubsystem, Error, "FindAssetPackageReferencers failed: Could not load asset. %ls", *LoadedAssetResult.GetError());
			return Result;
		}

		// Load the asset referencers to confirm 
		for (FName Referencer : PackageReferencers)
		{
			TValueOrError<UObject*, FString> ReferencerAssetResult = UE::EditorAssetUtils::LoadAssetFromPath(Referencer.ToString());
			if (ReferencerAssetResult.HasError())
			{
				UE_LOGF(LogEditorAssetSubsystem, Warning, "FindAssetPackageReferencers: Not able to confirm reference: %ls", *ReferencerAssetResult.GetError());
				// Add it to the list anyway
				Result.AddUnique(Referencer.ToString());
			}
		}

		// Check what we have in memory (but not in undo memory)
		FReferencerInformationList MemoryReferences;
		{
			if (GEditor && GEditor->Trans)
			{
				GEditor->Trans->DisableObjectSerialization();
			}
			IsReferenced(LoadedAssetResult.GetValue(), GARBAGE_COLLECTION_KEEPFLAGS, EInternalObjectFlags_GarbageCollectionKeepFlags, true, &MemoryReferences);
			if (GEditor && GEditor->Trans)
			{
				GEditor->Trans->EnableObjectSerialization();
			}
		}

		// Check if any references are in the list. Only confirm if the package is referencing it. An inner object of the asset could reference it.
		TArray<FName> ConfirmedReferencers;
		ConfirmedReferencers.Reserve(PackageReferencers.Num());

		for (const FReferencerInformation& RefInfo : MemoryReferences.InternalReferences)
		{
			FName PackageName = RefInfo.Referencer->GetPackage()->GetFName();
			if (PackageReferencers.Contains(PackageName))
			{
				ConfirmedReferencers.AddUnique(PackageName);
			}
		}
		for (const FReferencerInformation& RefInfo : MemoryReferences.ExternalReferences)
		{
			FName PackageName = RefInfo.Referencer->GetPackage()->GetFName();
			if (PackageReferencers.Contains(PackageName))
			{
				ConfirmedReferencers.AddUnique(PackageName);
			}
		}
		
		PackageReferencers.Empty();
		PackageReferencers = MoveTemp(ConfirmedReferencers);
	}

	// Copy the list. Result may already have Map packages.
	Result.Reserve(PackageReferencers.Num());
	for (FName PackagePath : PackageReferencers)
	{
		Result.Add(PackagePath.ToString());
	}

	return Result;
}

bool UEditorAssetSubsystem::ConsolidateAssets(UObject* AssetToConsolidateTo, const TArray<UObject*>& AssetsToConsolidate)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE() || !UE::EditorAssetUtils::EnsureAssetsLoaded())
	{
		return false;
	}
	
	if (AssetsToConsolidate.Num() == 0)
	{
		UE_LOGF(LogEditorAssetSubsystem, Warning, "ConsolidateAssets: No assets to consolidate.");
		return false;
	}
	if (TError<FString> Result = UE::EditorAssetUtils::IsARegisteredAsset(AssetToConsolidateTo); Result.HasError())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "ConsolidateAssets failed: Asset to consolidate to is not registered. %ls", *Result.GetError());
		return false;
	}

	TArray<UObject*> ToConsolidationObjects;
	ToConsolidationObjects.Reserve(AssetsToConsolidate.Num());
	for (UObject* Object : AssetsToConsolidate)
	{
		if (TError<FString> Result = UE::EditorAssetUtils::IsARegisteredAsset(Object); Result.HasError())
		{
			UE_LOGF(LogEditorAssetSubsystem, Error, "ConsolidateAssets failed: Asset '%ls' to consolidate is not registered. %ls", *Object->GetName(), *Result.GetError());
			return false;
		}
		if (AssetToConsolidateTo->GetClass() != Object->GetClass())
		{
			UE_LOGF(LogEditorAssetSubsystem, Error, "ConsolidateAssets failed: The asset '%ls' doesn't have the same class as the AssetToConsolidateTo.", *Object->GetName());
			return false;
		}
		ToConsolidationObjects.Add(Object);
	}

	// Perform the object consolidation
	const bool bShowDeleteConfirmation = false;
	ObjectTools::FConsolidationResults ConsResults = ObjectTools::ConsolidateObjects(AssetToConsolidateTo, ToConsolidationObjects, bShowDeleteConfirmation);

	// If the consolidation went off successfully with no failed objects
	if (ConsResults.DirtiedPackages.Num() > 0 && ConsResults.FailedConsolidationObjs.Num() == 0)
	{
		const bool bOnlyIfIsDirty = false;
		UEditorLoadingAndSavingUtils::SavePackages(ObjectPtrDecay(ConsResults.DirtiedPackages), bOnlyIfIsDirty);
	}
	// If the consolidation resulted in failed (partially consolidated) objects, do not save, and inform the user no save attempt was made
	else if (ConsResults.FailedConsolidationObjs.Num() > 0)
	{
		UE_LOGF(LogEditorAssetSubsystem, Warning, "ConsolidateAssets: Not all objects could be consolidated, no save has occurred");
		return false;
	}

	return true;
}

// Delete operations

bool UEditorAssetSubsystem::DeleteLoadedAsset(UObject* AssetToDelete)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE() || !UE::EditorAssetUtils::EnsureAssetsLoaded())
	{
		return false;
	}
	
	if (TError<FString> Result = UE::EditorAssetUtils::IsARegisteredAsset(AssetToDelete, true/*bAllowSkipBrowsableTestForExternalObject*/); Result.HasError())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "DeleteLoadedAsset failed: Asset is not registered. %ls", *Result.GetError());
		return false;
	}

	TArray<UObject*> AssetsToDelete;
	AssetsToDelete.Add(AssetToDelete);
	const bool bShowConfirmation = false;
	return ObjectTools::ForceDeleteObjects(AssetsToDelete, bShowConfirmation) == AssetsToDelete.Num();
}

bool UEditorAssetSubsystem::DeleteLoadedAssets(const TArray<UObject*>& AssetsToDelete)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE() || !UE::EditorAssetUtils::EnsureAssetsLoaded())
	{
		return false;
	}

	FString FailureReason;
	for (UObject* Asset : AssetsToDelete)
	{
		if (TError<FString> Result = UE::EditorAssetUtils::IsARegisteredAsset(Asset, true/*bAllowSkipBrowsableTestForExternalObject*/); Result.HasError())
		{
			UE_LOGF(LogEditorAssetSubsystem, Warning, "DeleteLoadedAssets failed: An asset is not registered. %ls", *Result.GetError());
			return false;
		}
	}
	
	const bool bShowConfirmation = false;
	return ObjectTools::ForceDeleteObjects(AssetsToDelete, bShowConfirmation) == AssetsToDelete.Num();
}

bool UEditorAssetSubsystem::DeleteAsset(const FString& AssetPathToDelete)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE() || !UE::EditorAssetUtils::EnsureAssetsLoaded())
	{
		return false;
	}
	
	TValueOrError<UObject*, FString> AssetToDeleteResult = UE::EditorAssetUtils::LoadAssetFromPath(AssetPathToDelete);
	if (AssetToDeleteResult.HasError())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "DeleteAsset failed: Could not find the source asset. %ls", *AssetToDeleteResult.GetError());
		return false;
	}

	TArray<UObject*> AssetsToDelete;
	AssetsToDelete.Add(AssetToDeleteResult.GetValue());
	const bool bShowConfirmation = false;
	return ObjectTools::ForceDeleteObjects(AssetsToDelete, bShowConfirmation) == AssetsToDelete.Num();
}

bool UEditorAssetSubsystem::DeleteDirectory(const FString& DirectoryPath)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE() || !UE::EditorAssetUtils::EnsureAssetsLoaded())
	{
		return false;
	}

	FString FailureReason;
	FString ValidDirectoryPath = EditorScriptingHelpers::ConvertAnyPathToLongPackagePath(DirectoryPath, FailureReason);
	if (ValidDirectoryPath.IsEmpty())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "DeleteDirectory failed: Could not convert the path. %ls", *FailureReason);
		return false;
	}

	FString FilePath = FPaths::ConvertRelativePathToFull(UPackageTools::PackageNameToFilename(ValidDirectoryPath + TEXT("/")));
	if (FilePath.IsEmpty())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "DeleteDirectory failed: Could not convert the path '%ls' to a full path.", *ValidDirectoryPath);
		return false;
	}

	TArray<FAssetData> AssetDatas;
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	if (!AssetRegistryModule.Get().GetAssetsByPath(*ValidDirectoryPath, AssetDatas, true))
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "DeleteDirectory failed: Could not get asset data for the directory '%ls'", *ValidDirectoryPath);
		return false;
	}

	// Load all assets
	TArray<UObject*> LoadedAssets;
	LoadedAssets.Reserve(AssetDatas.Num());
	for (const FAssetData& AssetData : AssetDatas)
	{
		TValueOrError<UObject*, FString> LoadedObjectResult = UE::EditorAssetUtils::LoadAssetFromData(AssetData);
		if (LoadedObjectResult.HasError())
		{
			UE_LOGF(LogEditorAssetSubsystem, Error, "DeleteDirectory failed: Some assets couldn't be loaded. %ls", *LoadedObjectResult.GetError());
			return false;
		}
		LoadedAssets.Add(LoadedObjectResult.GetValue());
	}

	const bool bShowConfirmation = false;
	if (ObjectTools::ForceDeleteObjects(LoadedAssets, bShowConfirmation) != LoadedAssets.Num())
	{
		UE_LOGF(LogEditorAssetSubsystem, Warning, "DeleteDirectory: Not all assets were deleted.");
		return false;
	}

	// Remove the path from the Asset Registry
	if (!AssetRegistryModule.Get().RemovePath(ValidDirectoryPath))
	{
		UE_LOGF(LogEditorAssetSubsystem, Warning, "DeleteDirectory: Could not remove the directory from the Asset Registry.");
	}

	// Delete the directory from the drive
	if (!UE::EditorAssetUtils::DeleteEmptyDirectoryFromDisk(ValidDirectoryPath))
	{
		UE_LOGF(LogEditorAssetSubsystem, Warning, "DeleteDirectory: Could not remove the directory but its contained assets have been removed.");
		return false;
	}

	return true;
}

// Duplicate operations

namespace UE::EditorAssetUtils
{
	UObject* DuplicateAsset(UObject* SourceObject, const FString& DestinationAssetPath)
	{
		check(SourceObject);
		if (!EditorScriptingHelpers::CheckIfInEditorAndPIE() || !UE::EditorAssetUtils::EnsureAssetsLoaded())
		{
			return nullptr;
		}

		FString FailureReason;
		FString DestinationObjectPath = EditorScriptingHelpers::ConvertAnyPathToObjectPath(DestinationAssetPath, FailureReason);
		if (DestinationObjectPath.IsEmpty())
		{
			UE_LOGF(LogEditorAssetSubsystem, Error, "DuplicateAsset failed: The destination asset path '%ls' is not valid. %ls", *DestinationAssetPath, *FailureReason);
			return nullptr;
		}

		if (!EditorScriptingHelpers::IsAValidPathForCreateNewAsset(DestinationObjectPath, FailureReason))
		{
			UE_LOGF(LogEditorAssetSubsystem, Error, "DuplicateAsset failed: The destination path '%ls' is not valid. %ls", *DestinationObjectPath, *FailureReason);
			return nullptr;
		}

		// when IAssetTools::DuplicateAsset fails, it opens a modal, so check beforehand
		if (FPackageName::DoesPackageExist(DestinationObjectPath))
		{
			UE_LOGF(LogEditorAssetSubsystem, Error, "DuplicateAsset failed: The asset at the path '%ls' already exists.", *DestinationObjectPath);
			return nullptr;
		}

		FString DestinationLongPackagePath = FPackageName::GetLongPackagePath(DestinationObjectPath);
		FString DestinationObjectName = FPackageName::ObjectPathToObjectName(DestinationObjectPath);

		// duplicate asset
		FAssetToolsModule& Module = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
		UObject* DuplicatedAsset = Module.Get().DuplicateAsset(DestinationObjectName, DestinationLongPackagePath, SourceObject);

		return DuplicatedAsset;
	}
}

UObject* UEditorAssetSubsystem::DuplicateLoadedAsset(UObject* SourceAsset, const FString& DestinationAssetPath)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	FString FailureReason;

	if (TError<FString> Result = UE::EditorAssetUtils::IsARegisteredAsset(SourceAsset); Result.HasError())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "DuplicateAsset failed: Asset is not registered. %ls", *Result.GetError());
		return nullptr;
	}

	return UE::EditorAssetUtils::DuplicateAsset(SourceAsset, DestinationAssetPath);
}

UObject* UEditorAssetSubsystem::DuplicateAsset(const FString& SourceAssetPath, const FString& DestinationAssetPath)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	FString FailureReason;
	
	TValueOrError<UObject*, FString> SourceObjectResult = UE::EditorAssetUtils::LoadAssetFromPath(SourceAssetPath);
	if (SourceObjectResult.HasError())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "DuplicateAsset failed: Could not find the source asset. %ls", *SourceObjectResult.GetError());
		return nullptr;
	}

	return UE::EditorAssetUtils::DuplicateAsset(SourceObjectResult.GetValue(), DestinationAssetPath);
}

bool UEditorAssetSubsystem::DuplicateDirectory(const FString& SourceDirectoryPath, const FString& DestinationDirectoryPath)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE() || !UE::EditorAssetUtils::EnsureAssetsLoaded())
	{
		return false;
	}

	TValueOrError<UE::EditorAssetUtils::FDirectoryRenamePaths, FString> PathsResult = UE::EditorAssetUtils::GetDirectoryRenamePaths(SourceDirectoryPath, DestinationDirectoryPath);
	if (PathsResult.HasError())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "DuplicateDirectory failed: Could not duplicate directory. %ls", *PathsResult.GetError());
		return false;
	}
	if (TError<FString> Result = UE::EditorAssetUtils::SetupDirectoryRename(PathsResult.GetValue()); Result.HasError())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "DuplicateDirectory failed: Could not duplicate directory. %ls", *Result.GetError());
		return false;
	}
	TValueOrError<UE::EditorAssetUtils::FDirectoryRenameAssetPaths, FString> AssetPathsResult = UE::EditorAssetUtils::GetDirectoryRenameAssetPaths(PathsResult.GetValue());
	if (AssetPathsResult.HasError())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "DuplicateDirectory failed: Could not duplicate directory. %ls", *AssetPathsResult.GetError());
        return false;
	}

	// Prepare data
	bool bHaveAFailedAsset = false;
	UE::EditorAssetUtils::FDirectoryRenameAssetPaths& AssetPaths = AssetPathsResult.GetValue();
	if (AssetPaths.SourceDirectoryAssetDatas.IsEmpty())
	{
		UE_LOGF(LogEditorAssetSubsystem, Log, "DuplicateDirectory: No assets to duplicate.");
	}
	
	FAssetToolsModule& Module = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	for (int32 Index = 0; Index < AssetPaths.SourceDirectoryAssetDatas.Num(); ++Index)
	{
		UObject* DuplicatedAsset = Module.Get().DuplicateAsset(AssetPaths.SourceDirectoryAssetDatas[Index].AssetName.ToString(), FPackageName::GetLongPackagePath(AssetPaths.DestinationDirectoryAssetPaths[Index]), AssetPaths.SourceDirectoryAssetDatas[Index].GetAsset());
		if (DuplicatedAsset == nullptr)
		{
			UE_LOGF(LogEditorAssetSubsystem, Warning, "DuplicateDirectory failed: Could not duplicate object '%ls'", *AssetPaths.SourceDirectoryAssetDatas[Index].GetObjectPathString());
			bHaveAFailedAsset = true;
		}
	}

	// Make sure the Asset Registry knows about the new directory
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	AssetRegistryModule.Get().AddPath(PathsResult.GetValue().DestinationDirectoryPath);

	return !bHaveAFailedAsset;
}

// Rename operations

namespace UE::EditorAssetUtils
{
	bool RenameLoadedAsset(UObject* SourceObject, const FString& DestinationAssetPath)
	{
		check(SourceObject);
		if (!EditorScriptingHelpers::CheckIfInEditorAndPIE() || !UE::EditorAssetUtils::EnsureAssetsLoaded())
		{
			return false;
		}

		FString FailureReason;
		FString DestinationObjectPath = EditorScriptingHelpers::ConvertAnyPathToObjectPath(DestinationAssetPath, FailureReason);
		if (DestinationObjectPath.IsEmpty())
		{
			UE_LOGF(LogEditorAssetSubsystem, Error, "RenameAsset failed: The destination asset path '%ls' is not valid. %ls", *DestinationAssetPath, *FailureReason);
			return false;
		}

		if (!EditorScriptingHelpers::IsAValidPathForCreateNewAsset(DestinationObjectPath, FailureReason))
		{
			UE_LOGF(LogEditorAssetSubsystem, Error, "RenameAsset failed: The destination path '%ls' is not valid. %ls", *DestinationObjectPath, *FailureReason);
			return false;
		}

		// IAssetTools::Rename will do this check, but will suggest another location via a modal
		if (FPackageName::DoesPackageExist(DestinationObjectPath))
		{
			UE_LOGF(LogEditorAssetSubsystem, Error, "RenameAsset failed: The destination path '%ls' already exists.", *DestinationAssetPath);
			return false;
		}

		FString DestinationLongPackagePath = FPackageName::GetLongPackagePath(DestinationObjectPath);
		FString DestinationObjectName = FPackageName::ObjectPathToObjectName(DestinationObjectPath);

		// rename asset
		TArray<FAssetRenameData> AssetToRename;
		AssetToRename.Add(FAssetRenameData(SourceObject, DestinationLongPackagePath, DestinationObjectName));

		FAssetToolsModule& Module = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
		return Module.Get().RenameAssets(AssetToRename);
	}
}

bool UEditorAssetSubsystem::RenameLoadedAsset(UObject* SourceAsset, const FString& DestinationAssetPath)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	FString FailureReason;
	if (TError<FString> Result = UE::EditorAssetUtils::IsARegisteredAsset(SourceAsset); Result.HasError())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "RenameAsset failed: Asset is not registered. %ls", *Result.GetError());
		return false;
	}

	return UE::EditorAssetUtils::RenameLoadedAsset(SourceAsset, DestinationAssetPath);
}

bool UEditorAssetSubsystem::RenameAsset(const FString& SourceAssetPath, const FString& DestinationAssetPath)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);
	
	TValueOrError<UObject*, FString> SourceObjectResult = UE::EditorAssetUtils::LoadAssetFromPath(SourceAssetPath);
	if (SourceObjectResult.HasError())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "RenameAsset failed: Could not find the source asset. %ls", *SourceObjectResult.GetError());
		return false;
	}

	return UE::EditorAssetUtils::RenameLoadedAsset(SourceObjectResult.GetValue(), DestinationAssetPath);
}

bool UEditorAssetSubsystem::RenameDirectory(const FString& SourceDirectoryPath, const FString& DestinationDirectoryPath)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE() || !UE::EditorAssetUtils::EnsureAssetsLoaded())
	{
		return false;
	}

	TValueOrError<UE::EditorAssetUtils::FDirectoryRenamePaths, FString> PathsResult = UE::EditorAssetUtils::GetDirectoryRenamePaths(SourceDirectoryPath, DestinationDirectoryPath);
	if (PathsResult.HasError())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "RenameDirectory failed: Could not rename directory. %ls", *PathsResult.GetError());
		return false;
	}
	if (TError<FString> Result = UE::EditorAssetUtils::SetupDirectoryRename(PathsResult.GetValue()); Result.HasError())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "RenameDirectory failed: Could not rename directory. %ls", *Result.GetError());
		return false;
	}
	TValueOrError<UE::EditorAssetUtils::FDirectoryRenameAssetPaths, FString> AssetPathsResult = UE::EditorAssetUtils::GetDirectoryRenameAssetPaths(PathsResult.GetValue());
	if (AssetPathsResult.HasError())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "RenameDirectory failed: Could not rename directory. %ls", *AssetPathsResult.GetError());
		return false;
	}

	// Prepare data
	if (AssetPathsResult.GetValue().SourceDirectoryAssetDatas.IsEmpty())
	{
		UE_LOGF(LogEditorAssetSubsystem, Log, "RenameDirectory. No assets to rename.");
	}
	
	const int32 NumAssetsToRename = AssetPathsResult.GetValue().SourceDirectoryAssetDatas.Num();
	if (NumAssetsToRename > 0)
	{
		TArray<FAssetRenameData> AssetsToRename;
		AssetsToRename.Reserve(NumAssetsToRename);

		FScopedSlowTask SlowTask((float)NumAssetsToRename, NSLOCTEXT("EditorUtilities", "RenameDirectorySlowTaskLabel", "Renaming directory and assets."));
		for (int32 Index = 0; Index < AssetPathsResult.GetValue().SourceDirectoryAssetDatas.Num(); ++Index)
		{
			// Try to load the asset, since if it has not previously been loaded, the rename process will fail to resolve the object in AssetRenameManager
			TValueOrError<UObject*, FString> LoadResult = UE::EditorAssetUtils::LoadAssetFromData(AssetPathsResult.GetValue().SourceDirectoryAssetDatas[Index]);
			if (LoadResult.HasError())
			{
				UE_LOGF(LogEditorAssetSubsystem, Error, "RenameDirectory failed: Some assets couldn't be renamed. %ls", *LoadResult.GetError());
				return false;
			}

			AssetsToRename.Add(FAssetRenameData(AssetPathsResult.GetValue().SourceDirectoryAssetDatas[Index].GetSoftObjectPath(), FSoftObjectPath(AssetPathsResult.GetValue().DestinationDirectoryAssetPaths[Index])));
			SlowTask.EnterProgressFrame(1.f);
		}

		// Rename the assets
		FAssetToolsModule& Module = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
		if (!Module.Get().RenameAssets(AssetsToRename))
		{
			UE_LOGF(LogEditorAssetSubsystem, Error, "RenameDirectory failed: Could not rename the assets.");
			return false;
		}
	}

	// Delete the old directory
	if (!UE::EditorAssetUtils::DeleteEmptyDirectoryFromDisk(PathsResult.GetValue().SourceDirectoryPath))
	{
		UE_LOGF(LogEditorAssetSubsystem, Warning, "RenameDirectory: Could not delete the original directory but the assets have been renamed.");
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	AssetRegistryModule.Get().RemovePath(PathsResult.GetValue().SourceDirectoryPath);
	AssetRegistryModule.Get().AddPath(PathsResult.GetValue().DestinationDirectoryPath);
	return true;
}

bool UEditorAssetSubsystem::SetDirtyFlag(UObject* Object, const bool bDirtyState)
{
	if (!IsValid(Object) || !Object->IsAsset())
	{
		return false;
	}
	UPackage* Pkg = Object->GetOutermost();
	if (!Pkg || Pkg == GetTransientPackage() || Pkg->ContainsMap())
	{
		return false;
	}
	bool bOldDirtyValue = Pkg->IsDirty();
	if (bOldDirtyValue != bDirtyState)
	{
		//If we need to change the state, return true if the state has change to the one specified
		Pkg->SetDirtyFlag(bDirtyState);
		return bDirtyState == Pkg->IsDirty();
	}
	return true;
}

// Checkout operations
 
 namespace UE::EditorAssetUtils
 {
 	bool Checkout(const TArray<UPackage*>& Packages)
 	{
 		if (Packages.Num() > 0)
 		{
 			// Checkout without a prompt
 			TArray<UPackage*>* PackagesCheckedOut = nullptr;
 			const bool bErrorIfAlreadyCheckedOut = false;
 			ECommandResult::Type Result = FEditorFileUtils::CheckoutPackages(Packages, PackagesCheckedOut, bErrorIfAlreadyCheckedOut);
 			return Result == ECommandResult::Succeeded;
 		}
 		return true;
 	}
 }

bool UEditorAssetSubsystem::CheckoutLoadedAsset(UObject* AssetToCheckout)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE() || !UE::EditorAssetUtils::EnsureAssetsLoaded())
	{
		return false;
	}

	if (TError<FString> Result = UE::EditorAssetUtils::IsARegisteredAsset(AssetToCheckout, true/*bAllowSkipBrowsableTestForExternalObject*/); Result.HasError())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "CheckoutLoadedAsset failed: Asset is not registered. %ls", *Result.GetError());
		return false;
	}

	TArray<UPackage*> Packages;
	Packages.Add(AssetToCheckout->GetPackage()); // Fully load and check out is done in FEditorFileUtils::CheckoutPackages

	return UE::EditorAssetUtils::Checkout(Packages);
}

bool UEditorAssetSubsystem::CheckoutLoadedAssets(const TArray<UObject*>& AssetsToCheckout)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE() || !UE::EditorAssetUtils::EnsureAssetsLoaded())
	{
		return false;
	}
	
	TArray<UPackage*> Packages;
	Packages.Reserve(AssetsToCheckout.Num());
	for (UObject* AssetToCheckout: AssetsToCheckout)
	{
		if (TError<FString> Result = UE::EditorAssetUtils::IsARegisteredAsset(AssetToCheckout, true/*bAllowSkipBrowsableTestForExternalObject*/); Result.HasError())
		{
			UE_LOGF(LogEditorAssetSubsystem, Warning, "CheckoutLoadedAssets: An asset is not registered. %ls", *Result.GetError());
			continue;
		}
		Packages.Add(AssetToCheckout->GetPackage());
	}

	return UE::EditorAssetUtils::Checkout(Packages);
}

bool UEditorAssetSubsystem::CheckoutAsset(const FString& AssetsToCheckout)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE() || !UE::EditorAssetUtils::EnsureAssetsLoaded())
	{
		return false;
	}
	
	TValueOrError<UObject*, FString> LoadedAssetResult = UE::EditorAssetUtils::LoadAssetFromPath(AssetsToCheckout);
	if (LoadedAssetResult.HasError())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "CheckoutAsset failed: Could not load asset: %ls", *LoadedAssetResult.GetError());
		return false;
	}

	TArray<UPackage*> Packages;
	Packages.Add(LoadedAssetResult.GetValue()->GetPackage()); // Fully loading and checking out is done in UEditorLoadingAndSavingUtils::SavePackages

	return UE::EditorAssetUtils::Checkout(Packages);
}

bool UEditorAssetSubsystem::CheckoutDirectory(const FString& DirectoryPath, bool bRecursive)
{
	using UE::EditorAssetUtils::EPackageEnumerationFilter;
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}
	
	TArray<UPackage*> Packages;
	if (TError<FString> Result = UE::EditorAssetUtils::EnumeratePackagesInDirectory(DirectoryPath, EPackageEnumerationFilter::NoFilter, bRecursive, Packages); Result.HasError())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "CheckoutAssets failed: Could not get packages in directory. %ls", *Result.GetError());
		return false;
	}

	return UE::EditorAssetUtils::Checkout(Packages);
}

// Save operations

bool UEditorAssetSubsystem::SaveLoadedAsset(UObject* AssetToSave, bool bOnlyIfIsDirty)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if ((UE::Editor::Private::AllowSavingAssetsDuringPIE == 0 && !EditorScriptingHelpers::CheckIfInEditorAndPIE()) || !UE::EditorAssetUtils::EnsureAssetsLoaded())
	{
		return false;
	}
	
	if (TError<FString> Result = UE::EditorAssetUtils::IsARegisteredAsset(AssetToSave, true/*bAllowSkipBrowsableTestForExternalObject*/); Result.HasError())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "SaveLoadedAsset failed: Asset is not registered. %ls", *Result.GetError());
		return false;
	}

	TArray<UPackage*> Packages;
	Packages.Add(AssetToSave->GetPackage()); // Fully loading and checking out is done in UEditorLoadingAndSavingUtils::SavePackages

	// Save without a prompt
	return UEditorLoadingAndSavingUtils::SavePackages(Packages, bOnlyIfIsDirty);
}

bool UEditorAssetSubsystem::SaveLoadedAssets(const TArray<UObject*>& AssetsToSave, bool bOnlyIfIsDirty)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if ((UE::Editor::Private::AllowSavingAssetsDuringPIE == 0 && !EditorScriptingHelpers::CheckIfInEditorAndPIE()) || !UE::EditorAssetUtils::EnsureAssetsLoaded())
	{
		return false;
	}

	FString FailureReason;
	TArray<UPackage*> Packages;
	Packages.Reserve(AssetsToSave.Num());
	for (UObject* AssetToSave : AssetsToSave)
	{
		if (TError<FString> Result = UE::EditorAssetUtils::IsARegisteredAsset(AssetToSave, true/*bAllowSkipBrowsableTestForExternalObject*/); Result.HasError())
		{
			UE_LOGF(LogEditorAssetSubsystem, Warning, "SaveLoadedAsset: An asset is not registered. %ls", *Result.GetError());
		}
		else
		{
			Packages.Add(AssetToSave->GetPackage());
		}
	}

	// Save without a prompt
	return UEditorLoadingAndSavingUtils::SavePackages(Packages, bOnlyIfIsDirty);
}

bool UEditorAssetSubsystem::SaveAsset(const FString& AssetsToSave, bool bOnlyIfIsDirty)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if ((UE::Editor::Private::AllowSavingAssetsDuringPIE == 0 && !EditorScriptingHelpers::CheckIfInEditorAndPIE()) || !UE::EditorAssetUtils::EnsureAssetsLoaded())
	{
		return false;
	}
	
	TValueOrError<UObject*, FString> LoadedAssetResult = UE::EditorAssetUtils::LoadAssetFromPath(AssetsToSave);
	if (LoadedAssetResult.HasError())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "SaveAsset failed: Could not load asset: %ls", *LoadedAssetResult.GetError());
		return false;
	}

	TArray<UPackage*> Packages;
	Packages.Add(LoadedAssetResult.GetValue()->GetPackage()); // Fully load and check out is done in UEditorLoadingAndSavingUtils::SavePackages

	// Save without a prompt
	return UEditorLoadingAndSavingUtils::SavePackages(Packages, bOnlyIfIsDirty);
}

int32 UEditorAssetSubsystem::GetAssetFilenameLengthForCooking(const FString& AssetPath)
{
	const FString PackagePath = FPackageName::ObjectPathToPackageName(AssetPath);
	return AssetViewUtils::GetPackageLengthForCooking(PackagePath, FEngineBuildSettings::IsInternalBuild());
}

int32 UEditorAssetSubsystem::GetLoadedAssetFilenameLengthForCooking(const UObject* Asset)
{
	if (Asset)
	{
		return AssetViewUtils::GetPackageLengthForCooking(Asset->GetPackage()->GetName(), FEngineBuildSettings::IsInternalBuild());
	}
	return 0;
}

bool UEditorAssetSubsystem::SaveDirectory(const FString& DirectoryPath, bool bOnlyIfIsDirty, bool bRecursive)
{
	using UE::EditorAssetUtils::EPackageEnumerationFilter;
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if ((UE::Editor::Private::AllowSavingAssetsDuringPIE == 0 && !EditorScriptingHelpers::CheckIfInEditorAndPIE()) || !UE::EditorAssetUtils::EnsureAssetsLoaded())
	{
		return false;
	}
	
	TArray<UPackage*> Packages;
	const EPackageEnumerationFilter PackageFilter = bOnlyIfIsDirty ? EPackageEnumerationFilter::OnlyDirty : EPackageEnumerationFilter::NoFilter;
	if (TError<FString> Result = UE::EditorAssetUtils::EnumeratePackagesInDirectory(DirectoryPath, PackageFilter, bRecursive, Packages); Result.HasError())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "SaveDirectory failed: Could not save. %ls", *Result.GetError());
		return false;
	}

	// Save without a prompt
	return UEditorLoadingAndSavingUtils::SavePackages(Packages, bOnlyIfIsDirty);
}

// Directory operations

bool UEditorAssetSubsystem::DoesDirectoryExist(const FString& DirectoryPath)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE() || !UE::EditorAssetUtils::EnsureAssetsLoaded())
	{
		return false;
	}

	FString FailureReason;
	FString DirectoryPackageName = EditorScriptingHelpers::ConvertAnyPathToLongPackagePath(DirectoryPath, FailureReason);
	if (DirectoryPackageName.IsEmpty())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "DoesDirectoryExist failed: Could not convert path: %ls", *FailureReason);
		return false;
	}
	
	// First check the Asset Registry's cache
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	if (AssetRegistryModule.Get().PathExists(DirectoryPackageName))
	{
		return true;
	}

	// Get the on-disk path
	FString FilePath = FPaths::ConvertRelativePathToFull(UPackageTools::PackageNameToFilename(DirectoryPackageName + TEXT("/")));
	if (FilePath.IsEmpty())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "DoesDirectoryExist failed: Could not convert path '%ls' to a full path.", *DirectoryPackageName);
		return false;
	}
	// If it's not cached, check if it's on disk
	if (IFileManager::Get().DirectoryExists(*FilePath))
	{
		// Cache it in the Asset Registry if it exists
		AssetRegistryModule.Get().AddPath(DirectoryPackageName);
		return true;
	}
	return false;
}

bool UEditorAssetSubsystem::DoesDirectoryContainAssets(const FString& DirectoryPath, bool bRecursive)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE() || !UE::EditorAssetUtils::EnsureAssetsLoaded())
	{
		return false;
	}

	FString FailureReason;
	FString DirectoryPackageName = EditorScriptingHelpers::ConvertAnyPathToLongPackagePath(DirectoryPath, FailureReason);
	if (DirectoryPackageName.IsEmpty())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "DoesDirectoryContainAssets failed: Could not convert path: %ls", *FailureReason);
		return false;
	}

	FString FilePath = FPaths::ConvertRelativePathToFull(UPackageTools::PackageNameToFilename(DirectoryPackageName + TEXT("/")));
	if (FilePath.IsEmpty())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "DoesDirectoryContainAssets failed: Could not convert path '%ls' to a full path.", *DirectoryPackageName);
		return false;
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	return AssetRegistryModule.Get().HasAssets(*DirectoryPackageName, bRecursive);
}

bool UEditorAssetSubsystem::MakeDirectory(const FString& DirectoryPath)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE() || !UE::EditorAssetUtils::EnsureAssetsLoaded())
	{
		return false;
	}

	FString FailureReason;
	FString DirectoryPackageName = EditorScriptingHelpers::ConvertAnyPathToLongPackagePath(DirectoryPath, FailureReason);
	if (DirectoryPackageName.IsEmpty())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "MakeDirectory failed: Could not convert the path '%ls'. %ls", *DirectoryPath, *FailureReason);
		return false;
	}

	FString FilePath = FPaths::ConvertRelativePathToFull(FPackageName::LongPackageNameToFilename(DirectoryPackageName + TEXT("/")));
	if (FilePath.IsEmpty())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "MakeDirectory failed: Could not convert the path '%ls' to a full path.", *DirectoryPackageName);
		return false;
	}

	// If the folder has not yet been created, create it before we try to add it to the AssetRegistry.
	if (!IFileManager::Get().DirectoryExists(*FilePath))
	{
		const bool bTree = true;
		if (IFileManager::Get().MakeDirectory(*FilePath, bTree))
		{
			// Add to the AssetRegistry cache
			FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
			AssetRegistryModule.Get().AddPath(DirectoryPackageName);
			return true;
		}
	}
	
	return false;
}

// List operations

TArray<FString> UEditorAssetSubsystem::ListAssets(const FString& DirectoryPath, bool bRecursive, bool bIncludeFolder)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	TArray<FString> AssetPaths;
	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE() || !UE::EditorAssetUtils::EnsureAssetsLoaded())
	{
		return AssetPaths;
	}

	TArray<FAssetData> AssetDatas;
	FString DirectoryPackageName;
	
	// if there is a valid asset data (i.e. path belongs to a file)
	if (TValueOrError<FAssetData, FString> AssetDataResult = UE::EditorAssetUtils::FindAssetDataFromAnyPath(DirectoryPath); AssetDataResult.HasValue() && AssetDataResult.GetValue().IsValid())
	{
		AssetDatas.Add(AssetDataResult.GetValue());
	}
	// path may belong to a directory
	else if (TError<FString> Result = UE::EditorAssetUtils::EnumerateAssetsInDirectory(DirectoryPath, bRecursive, AssetDatas, DirectoryPackageName); Result.HasError())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "ListAssets failed: Could not enumerate assets in directory '%ls'. %ls", *DirectoryPath, *Result.GetError());
		return AssetPaths;
	}

	if (AssetDatas.Num() > 0)
	{
		AssetPaths.Reserve(AssetDatas.Num());
		for (const FAssetData& AssetData : AssetDatas)
		{
			AssetPaths.Add(AssetData.GetObjectPathString());
		}
	}

	if (bIncludeFolder && !DirectoryPackageName.IsEmpty())
	{
		TArray<FString> SubPaths;
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(FName("AssetRegistry"));
		AssetRegistryModule.Get().GetSubPaths(DirectoryPackageName, SubPaths, bRecursive);

		for (const FString& SubPath : SubPaths)
		{
			if (SubPath.Contains(DirectoryPath) && SubPath != DirectoryPath)
			{
				AssetPaths.Add(SubPath + TEXT('/'));
			}
		}
	}

	AssetPaths.Sort();
	return AssetPaths;
}

TArray<FString> UEditorAssetSubsystem::ListAssetsByTagValue(FName TagName, const FString& TagValue)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	TArray<FString> Result;
	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE() || !UE::EditorAssetUtils::EnsureAssetsLoaded())
	{
		return Result;
	}

	if (TagName == NAME_None)
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "ListAssetsByTagValue failed: The empty tag '' is not valid.");
		return Result;
	}

	TMultiMap<FName, FString> TagValues;
	TagValues.Add(TagName, TagValue);

	TArray<FAssetData> FoundAssets;
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	if (!AssetRegistryModule.Get().GetAssetsByTagValues(TagValues, FoundAssets))
	{
		UE_LOGF(LogEditorAssetSubsystem, Warning, "ListAssetsByTagValue failed: Could not get assets with given tag values.");
		return Result;
	}

	for (const FAssetData& AssetData : FoundAssets)
	{
		FString ObjectPathStr = AssetData.PackageName.ToString();
		Result.Add(ObjectPathStr);
	}

	return Result;
}

TMap<FName, FString> UEditorAssetSubsystem::GetTagValues(const FString& AssetPath)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	TMap<FName, FString> Result;
	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE() || !UE::EditorAssetUtils::EnsureAssetsLoaded())
	{
		return Result;
	}
	
	TValueOrError<FAssetData, FString> AssetDataResult = UE::EditorAssetUtils::FindAssetDataFromAnyPath(AssetPath);
	if (AssetDataResult.HasError())
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "GetTagValues failed: Could not get Asset Data. %ls", *AssetDataResult.GetError());
		return Result;
	}

	for (TPair<FName, FAssetTagValueRef> Itt : AssetDataResult.GetValue().TagsAndValues)
	{
		Result.Add(Itt.Key, Itt.Value.AsString());
	}
	return Result;
}

TMap<FName, FString> UEditorAssetSubsystem::GetMetadataTagValues(UObject* Object)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	TMap<FName, FString> Result;
	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return Result;
	}

#if WITH_EDITORONLY_DATA
	if (Object)
	{
		TMap<FName, FString>* TagValues = FMetaData::GetMapForObject(Object);
		if (TagValues != nullptr)
		{
			return *TagValues;
		}
	}
	else
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "GetMetadataTagValues failed: Object is null.");
	}
#endif // WITH_EDITORONLY_DATA
	return Result;
}

FString UEditorAssetSubsystem::GetMetadataTag(UObject* Object, FName Tag)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return FString();
	}

#if WITH_EDITORONLY_DATA
	if (!Object)
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "GetMetadataTag failed: Object is null.");
		return FString();
	}
	return Object->GetPackage()->GetMetaData().GetValue(Object, Tag);
#else
	return FString();
#endif // WITH_EDITORONLY_DATA
}

void UEditorAssetSubsystem::SetMetadataTag(UObject* Object, FName Tag, const FString& Value)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return;
	}

#if WITH_EDITORONLY_DATA
	if (!Object)
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "SetMetadataTag failed: Object is null.");
		return;
	}
	Object->Modify();
	Object->GetPackage()->GetMetaData().SetValue(Object, Tag, *Value);
#endif // WITH_EDITORONLY_DATA
}

void UEditorAssetSubsystem::RemoveMetadataTag(UObject* Object, FName Tag)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return;
	}

#if WITH_EDITORONLY_DATA
	if (!Object)
	{
		UE_LOGF(LogEditorAssetSubsystem, Error, "RemoveMetadataTag failed: Object is null.");
		return;
	}
	Object->Modify();
	Object->GetPackage()->GetMetaData().RemoveValue(Object, Tag);
#endif // WITH_EDITORONLY_DATA
}


void UEditorAssetSubsystem::AddOnExtractAssetFromFile(FOnExtractAssetFromFileDynamic Delegate)
{
	OnExtractAssetFromFileDynamicArray.Add(Delegate);
}

void UEditorAssetSubsystem::RemoveOnExtractAssetFromFile(FOnExtractAssetFromFileDynamic Delegate)
{
	OnExtractAssetFromFileDynamicArray.Remove(Delegate);
}

void UEditorAssetSubsystem::CallOnExtractAssetFromFileDynamicArray(
	const TArray<FString>& Files,
	TArray<FAssetData>& OutAssetDataArray)
{
	TArray<FAssetData> LocalArray;
	for (FOnExtractAssetFromFileDynamic& Delegate : OnExtractAssetFromFileDynamicArray)
	{
		Delegate.ExecuteIfBound(Files, LocalArray);
		OutAssetDataArray += LocalArray;
		LocalArray.Reset();
	}
}

TArray<FAssetData> UEditorAssetSubsystem::GetAllAssetsByMetaDataTags(
	const TSet<FName>& RequiredTags,
	const TSet<UClass*>& AllowedClasses
	)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	TArray<FAssetData> Result;
	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE() || !UE::EditorAssetUtils::EnsureAssetsLoaded())
	{
		return Result;
	}
	
	FARFilter Filter;
	Filter.TagsAndValues.Reserve(RequiredTags.Num());
	for (FName RequiredTag : RequiredTags)
	{
		Filter.TagsAndValues.Add(RequiredTag);
	}
	Filter.ClassPaths.Reserve(RequiredTags.Num());
	for (UClass* AllowedClass : AllowedClasses)
	{
		Filter.ClassPaths.Add(FTopLevelAssetPath(AllowedClass));
	}

	IAssetRegistry::Get()->GetAssets(Filter, Result);
	return Result;
}

bool UEditorAssetSubsystem::SortByMetaData(
	TArray<FAssetData>& Assets,
	FName MetaDataTag,
	EEditorAssetMetaDataSortType MetaDataType,
	EEditorAssetSortOrder SortOrder
	)
{
	using namespace UE::EditorAssetUtils;
	switch (MetaDataType)
	{
	case EEditorAssetMetaDataSortType::String:
		return Sort<FString>(Assets, MetaDataTag, [](const FString& String, FString& Result){ Result = String; return true; }, SortOrder);
	case EEditorAssetMetaDataSortType::Numeric:
		return Sort<double>(Assets, MetaDataTag, [](const FString& String, double& Result)
		{
			if (String.IsNumeric())
			{
				Result = FCString::Atod(*String);
				return true;
			}
			return false;
		}, SortOrder);
	case EEditorAssetMetaDataSortType::DateTime:
		return Sort<FDateTime>(Assets, MetaDataTag, [](const FString& String, FDateTime& Result){ return FDateTime::Parse(String, Result); }, SortOrder);
	default:
		checkNoEntry();
		return false;
	}
}
