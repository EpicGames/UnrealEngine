// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/EditorSubsystemBlueprintLibrary.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Editor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorSubsystemBlueprintLibrary)

/*static*/ UEditorSubsystem* UEditorSubsystemBlueprintLibrary::GetEditorSubsystem(TSubclassOf<UEditorSubsystem> Class)
{
	return GEditor->GetEditorSubsystemBase(Class);
}

FName GetPreviewPlatformDropdownName(const FName& PreviewShaderPlatformName, const FName& DeviceProfileName)
{
	// We cannot use the localized name, because it will get cached and not work for other Languages.
	// Instead we take the PreviewShaderPlatformName and remove the postfix and suffix if they are present
	// and concatenate it with the DeviceProfileName if present.

	FString DropdownName = PreviewShaderPlatformName.ToString();
	DropdownName.RemoveFromEnd(TEXT("_Preview"));
	DropdownName.RemoveFromStart(TEXT("SP_"));

	if (DeviceProfileName != NAME_None)
	{
		DropdownName += TEXT(" ") + DeviceProfileName.ToString();
	}

	return FName(DropdownName);
}

TArray<FName> UEditorSubsystemBlueprintLibrary::GetPreviewPlatformOptions() const
{
	// We gather all the available PreviewPlatform options from the PreviewPlatformMenu.

	TArray<FName> Outputs;
	for (const FPreviewPlatformMenuItem& Item : FDataDrivenPlatformInfoRegistry::GetAllPreviewPlatformMenuItems())
	{
		FName PreviewNameAndDeviceProfileName = GetPreviewPlatformDropdownName(Item.PreviewShaderPlatformName, Item.DeviceProfileName);
		Outputs.Add(PreviewNameAndDeviceProfileName);
	}

	return Outputs;
}

void UEditorSubsystemBlueprintLibrary::TogglePreviewPlatform()
{
	GEditor->TogglePreviewPlatform();
}

void UEditorSubsystemBlueprintLibrary::SetPreviewPlatform(FName PreviewShaderPlatformName)
{
	if (PreviewShaderPlatformName == NAME_None)
	{
		return;
	}

	bool bFoundMatch = false;
	for (const FPreviewPlatformMenuItem& Item : FDataDrivenPlatformInfoRegistry::GetAllPreviewPlatformMenuItems())
	{
		// We check to see if we can find the matching PreviewShaderPlatform DeviceProfile combination in the Preview Platform Menu.

		FName NameToCheck = GetPreviewPlatformDropdownName(Item.PreviewShaderPlatformName, Item.DeviceProfileName);
		if (NameToCheck == PreviewShaderPlatformName)
		{
			bFoundMatch = true;
			EShaderPlatform ShaderPlatform = FDataDrivenShaderPlatformInfo::GetShaderPlatformFromName(Item.PreviewShaderPlatformName);

			if (ShaderPlatform < SP_NumPlatforms)
			{
				const bool bIsDefaultShaderPlatform = FDataDrivenShaderPlatformInfo::GetPreviewShaderPlatformParent(ShaderPlatform) == GMaxRHIShaderPlatform;

				auto GetPreviewFeatureLevelInfo = [&]()
					{
						if (bIsDefaultShaderPlatform)
						{
							return FPreviewPlatformInfo(GMaxRHIFeatureLevel, GMaxRHIShaderPlatform, NAME_None, NAME_None, NAME_None, false, NAME_None);
						}

						const ERHIFeatureLevel::Type FeatureLevel = GetMaxSupportedFeatureLevel(ShaderPlatform);
						return FPreviewPlatformInfo(FeatureLevel, ShaderPlatform, Item.PlatformName, Item.ShaderFormat, Item.DeviceProfileName, true, Item.PreviewShaderPlatformName);
					};

				FPreviewPlatformInfo PreviewFeatureLevelInfo = GetPreviewFeatureLevelInfo();
				GEditor->SetPreviewPlatform(PreviewFeatureLevelInfo, false);
				break;
			}
		}
	}

	// If we cannot find a match we emit a warning.
	if (!bFoundMatch)
	{
		UE_LOGF(LogBlueprint, Warning, "SetPreviewPlatform Blueprint Function could not find the PreviewShaderPlatform DeviceProfile combination: %ls", *PreviewShaderPlatformName.ToString());
	}
}

FName UEditorSubsystemBlueprintLibrary::GetPreviewPlatformName()
{
	FName PreviewPlatformName = NAME_None;
	if (GEditor)
	{
		FPreviewPlatformInfo PreviewPlatform = GEditor->PreviewPlatform;
		if (PreviewPlatform.bPreviewShaderPlatformActive)
		{
			PreviewPlatformName = GetPreviewPlatformDropdownName(PreviewPlatform.PreviewShaderPlatformName, PreviewPlatform.DeviceProfileName);
		}
	}
	return PreviewPlatformName;
}

void UEditorSubsystemBlueprintLibrary::DisablePreviewPlatform()
{
	FPreviewPlatformInfo DisablePreviewPlatformInfo(GMaxRHIFeatureLevel, GMaxRHIShaderPlatform, NAME_None, NAME_None, NAME_None, false, NAME_None);
	GEditor->SetPreviewPlatform(DisablePreviewPlatformInfo, false);
}
