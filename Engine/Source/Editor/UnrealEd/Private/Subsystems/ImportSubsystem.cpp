// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/ImportSubsystem.h"

#include "Editor.h"
#include "AssetToolsModule.h"
#include "ObjectTools.h"
#include "FileHelpers.h"
#include "EditorReimportHandler.h"
#include "Misc/NamePermissionList.h"
#include "InterchangeManager.h"
#include "InterchangeFactoryBase.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ImportSubsystem)

class FImportFilesByPath : public IImportSubsystemTask
{
public:
	FImportFilesByPath(const TArray<FString>& InFiles, const FString& InRootDestinationPath) :
		Files(InFiles),
		RootDestinationPath(InRootDestinationPath)
	{
	}

	virtual void Run() override
	{
		TArray<FString> ImportFiles;
		TMap<FString, UObject*> ReimportFiles;
		FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");

		TArray<TPair<FString, FString>> FilesAndDestinations;
		AssetToolsModule.Get().ExpandDirectories(Files, RootDestinationPath, FilesAndDestinations);

		TArray<int32> ReImportIndexes;
		for (int32 FileIdx = 0; FileIdx < FilesAndDestinations.Num(); ++FileIdx)
		{
			const FString& Filename = FilesAndDestinations[FileIdx].Key;
			const FString& DestinationPath = FilesAndDestinations[FileIdx].Value;
			FString Name = ObjectTools::SanitizeObjectName(FPaths::GetBaseFilename(Filename));
			FString PackageName = ObjectTools::SanitizeInvalidChars(DestinationPath + TEXT("/") + Name, INVALID_LONGPACKAGE_CHARACTERS);

			// We can not create assets that share the name of a map file in the same location
			if (FEditorFileUtils::IsMapPackageAsset(PackageName))
			{
				//The error message will be log in the import process
				ImportFiles.Add(Filename);
				continue;
			}
			//Check if package exist in memory
			UPackage* Pkg = FindPackage(nullptr, *PackageName);
			bool IsPkgExist = Pkg != nullptr;
			//check if package exist on file
			if (!IsPkgExist && !FPackageName::DoesPackageExist(PackageName))
			{
				ImportFiles.Add(Filename);
				continue;
			}
			if (Pkg == nullptr)
			{
				Pkg = CreatePackage( *PackageName);
				if (Pkg == nullptr)
				{
					//Cannot create a package that don't exist on disk or in memory!!!
					//The error message will be log in the import process
					ImportFiles.Add(Filename);
					continue;
				}
			}
			// Make sure the destination package is loaded
			Pkg->FullyLoad();

			// Check for an existing object
			UObject* ExistingObject = StaticFindObject(UObject::StaticClass(), Pkg, *Name);
			if (ExistingObject != nullptr)
			{
				ReimportFiles.Add(Filename, ExistingObject);
				ReImportIndexes.Add(FileIdx);
			}
			else
			{
				ImportFiles.Add(Filename);
			}
		}

		//Reimport
		for (auto kvp : ReimportFiles)
		{
			FReimportManager::Instance()->Reimport(kvp.Value, false, true, kvp.Key);
		}
		
		//Import
		if (ImportFiles.Num() > 0)
		{
			//Remove it in reverse so the smaller index are still valid
			for (int32 IndexToRemove = ReImportIndexes.Num() - 1; IndexToRemove >= 0; --IndexToRemove)
			{
				FilesAndDestinations.RemoveAt(ReImportIndexes[IndexToRemove]);
			}
			AssetToolsModule.Get().ImportAssets(ImportFiles, RootDestinationPath, nullptr, true, &FilesAndDestinations, true);
		}
	}

private:
	TArray<FString> Files;
	FString RootDestinationPath;
};


UImportSubsystem::UImportSubsystem()
	: UEditorSubsystem()
{

}

void UImportSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	auto AttachDelegates = [this]()
	{
		//Hook on Interchange manager to know when to broadcast import/reimport
		UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
		InterchangeManager.OnAssetPostImport.AddUObject(this, &UImportSubsystem::InterchangeBroadcastAssetPostImport);
		InterchangeManager.OnAssetPostReimport.AddUObject(this, &UImportSubsystem::InterchangeBroadcastAssetPostReimport);
	};

	if (GEngine)
	{
		AttachDelegates();
	}
	else
	{
		FCoreDelegates::GetOnPostEngineInit().AddLambda(AttachDelegates);
	}

	//Unregister before the Interchange manager get destroy
	UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
	InterchangeManager.OnPreDestroyInterchangeManager.AddLambda([this]()
	{
		//Unhook interchange manager import/reimport delegates
		UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
		InterchangeManager.OnAssetPostImport.RemoveAll(this);
		InterchangeManager.OnAssetPostReimport.RemoveAll(this);
	});
}

void UImportSubsystem::Deinitialize()
{
}

void UImportSubsystem::ImportNextTick(const TArray<FString>& Files, const FString& DestinationPath)
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	if (!AssetToolsModule.Get().GetWritableFolderPermissionList()->PassesStartsWithFilter(DestinationPath))
	{
		AssetToolsModule.Get().NotifyBlockedByWritableFolderFilter();
		return;
	}
	
	PendingTasks.Enqueue(MakeShared<FImportFilesByPath>(Files, DestinationPath));

	GEditor->GetTimerManager()->SetTimerForNextTick(this, &UImportSubsystem::HandleNextTick);
}

void UImportSubsystem::HandleNextTick()
{
	if (!PendingTasks.IsEmpty())
	{
		TSharedPtr<IImportSubsystemTask> Task;
		while (PendingTasks.Dequeue(Task))
		{
			if (Task.IsValid())
			{
				Task->Run();
			}
		}
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS

void UImportSubsystem::BroadcastAssetPreImport(UFactory* InFactory, UClass* InClass, UObject* InParent, const FName& Name, const TCHAR* Type)
{
	FEditorDelegates::OnAssetPreImport.Broadcast(InFactory, InClass, InParent, Name, Type);
	OnAssetPreImport.Broadcast(InFactory, InClass, InParent, Name, Type);
	OnAssetPreImport_BP.Broadcast(InFactory, InClass, InParent, Name, FString(Type));
}

void UImportSubsystem::BroadcastAssetPostImport(UFactory* InFactory, UObject* InCreatedObject)
{
	FEditorDelegates::OnAssetPostImport.Broadcast(InFactory, InCreatedObject);
	OnAssetPostImport.Broadcast(InFactory, InCreatedObject);
	OnAssetPostImport_BP.Broadcast(InFactory, InCreatedObject);
}

void UImportSubsystem::BroadcastAssetReimport(UObject* InCreatedObject)
{
	FEditorDelegates::OnAssetReimport.Broadcast(InCreatedObject);
	OnAssetReimport.Broadcast(InCreatedObject);
	OnAssetReimport_BP.Broadcast(InCreatedObject);
}

void UImportSubsystem::BroadcastAssetPostLODImport(UObject* InObject, int32 inLODIndex)
{
	OnAssetPostLODImport.Broadcast(InObject, inLODIndex);
	OnAssetPostLODImport_BP.Broadcast(InObject, inLODIndex);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UImportSubsystem::InterchangeBroadcastAssetPostImport(UObject* InCreatedObject)
{
	UFactory* NullFactory = nullptr;
	BroadcastAssetPostImport(NullFactory, InCreatedObject);
}

void UImportSubsystem::InterchangeBroadcastAssetPostReimport(UObject* InCreatedObject)
{
	BroadcastAssetReimport(InCreatedObject);
}
