// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/PanelExtensionSubsystem.h"

#include "Editor.h"
#include "AssetToolsModule.h"
#include "ObjectTools.h"
#include "FileHelpers.h"
#include "EditorReimportHandler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PanelExtensionSubsystem)

DECLARE_LOG_CATEGORY_CLASS(LogPanelExtensions, Log, All);

SExtensionPanel::~SExtensionPanel()
{
	if (GEditor && ExtensionPanelID != NAME_None)
	{
		if (UPanelExtensionSubsystem* PanelExtensionSubsystem = GEditor->GetEditorSubsystem<UPanelExtensionSubsystem>())
		{
			PanelExtensionSubsystem->OnPanelFactoryRegistryChanged(ExtensionPanelID).RemoveAll(this);
		}
	}
}

void SExtensionPanel::Construct(const FArguments& InArgs)
{
	ExtensionPanelID = InArgs._ExtensionPanelID.Get(NAME_None);
	DefaultWidget = InArgs._DefaultWidget.Get(nullptr);
	ExtensionContext = InArgs._ExtensionContext.Get(nullptr);
	WindowZoneOverride = InArgs._WindowZoneOverride;

	if (GEditor && ExtensionPanelID != NAME_None)
	{
		if (UPanelExtensionSubsystem* PanelExtensionSubsystem = GEditor->GetEditorSubsystem<UPanelExtensionSubsystem>())
		{
			PanelExtensionSubsystem->OnPanelFactoryRegistryChanged(ExtensionPanelID).AddRaw(this, &SExtensionPanel::RebuildWidget);
			RebuildWidget();
		}
	}
}

void SExtensionPanel::RebuildWidget()
{
	if (GEditor && ExtensionPanelID != NAME_None)
	{
		if (UPanelExtensionSubsystem* PanelExtensionSubsystem = GEditor->GetEditorSubsystem<UPanelExtensionSubsystem>())
		{
			TSharedRef<SWidget> Widget = PanelExtensionSubsystem->CreateWidget(ExtensionPanelID, ExtensionContext);
			if (Widget == SNullWidget::NullWidget && DefaultWidget.IsValid())
			{
				Widget = DefaultWidget.ToSharedRef();
			}

			ChildSlot
			[
				Widget
			];
		}
	}
}


UPanelExtensionSubsystem::UPanelExtensionSubsystem()
	: UEditorSubsystem()
{

}

void UPanelExtensionSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{

}

void UPanelExtensionSubsystem::Deinitialize()
{

}

void UPanelExtensionSubsystem::RegisterPanelFactory(FName ExtensionPanelID, const FPanelExtensionFactory& InPanelExtensionFactory)
{
	if (const FText* DeprecationMessage = DeprecatedExtensions.Find(ExtensionPanelID))
	{
		UE_LOGF(LogPanelExtensions, Warning, "Extension panel id \"%ls\" has been deprecated. %ls", *ExtensionPanelID.ToString(), *DeprecationMessage->ToString());
	}

	TArray<FPanelExtensionFactory>& Factories = ExtensionPointMap.FindOrAdd(ExtensionPanelID);
	Factories.Add(InPanelExtensionFactory);
	Factories.StableSort([](const FPanelExtensionFactory& One, const FPanelExtensionFactory& Two)
	{
		return One.SortWeight > Two.SortWeight;
	});

	UPanelExtensionSubsystem::FPanelFactoryRegistryChanged& PanelFactoryRegsitered = OnPanelFactoryRegistryChanged(ExtensionPanelID);
	PanelFactoryRegsitered.Broadcast();
}

void UPanelExtensionSubsystem::UnregisterPanelFactory(FName Identifier, FName ExtensionPanelID)
{
	for (auto& AssetPair : ExtensionPointMap)
	{
		if (ExtensionPanelID == NAME_None || AssetPair.Key == ExtensionPanelID)
		{
			int32 numRemoved = AssetPair.Value.RemoveAll([Identifier](const FPanelExtensionFactory& Factory) { return Factory.Identifier == Identifier; });
			if (numRemoved > 0)
			{
				UPanelExtensionSubsystem::FPanelFactoryRegistryChanged& PanelFactoryRegsitered = OnPanelFactoryRegistryChanged(AssetPair.Key);
				PanelFactoryRegsitered.Broadcast();
			}
		}
	}
}

bool UPanelExtensionSubsystem::IsPanelFactoryRegistered(FName Identifier, FName ExtensionPanelID) const
{
	for (auto& AssetPair : ExtensionPointMap)
	{
		if (ExtensionPanelID == NAME_None || AssetPair.Key == ExtensionPanelID)
		{
			if (AssetPair.Value.FindByPredicate([Identifier](const FPanelExtensionFactory& Factory) { return Factory.Identifier == Identifier; }))
			{
				return true;
			}
		}
	}
	return false;
}

void UPanelExtensionSubsystem::MarkDeprecatedExtension(FName ExtensionID, const FText& Message)
{
	DeprecatedExtensions.Add(ExtensionID, Message);
	
	if (const TArray<FPanelExtensionFactory>* Factories = ExtensionPointMap.Find(ExtensionID))
	{
		if (Factories->Num())
		{
			UE_LOGF(LogPanelExtensions, Warning, "Extension panel id \"%ls\" has been deprecated. %ls", *ExtensionID.ToString(), *Message.ToString());
		}
	}
}

TSharedRef<SWidget> UPanelExtensionSubsystem::CreateWidget(FName ExtensionPanelID, FWeakObjectPtr ExtensionContext)
{
	const TArray<FPanelExtensionFactory>* ExtensionArray = ExtensionPointMap.Find(ExtensionPanelID);
	if (ExtensionArray && ExtensionArray->Num() > 0)
	{
		TSharedRef<SHorizontalBox> ExtensionWidgets = SNew(SHorizontalBox);

		for (const FPanelExtensionFactory& Extension : *ExtensionArray)
		{
			TSharedPtr<SWidget> ExtensionWidget;
			if (Extension.CreateExtensionWidget.IsBound())
			{
				ExtensionWidget = Extension.CreateExtensionWidget.Execute(ExtensionContext);
			}
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			else if (Extension.CreateWidget.IsBound())
			{
				TArray<UObject*> Temp;
				ExtensionWidget = Extension.CreateWidget.Execute(Temp);
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			if (ExtensionWidget && ExtensionWidget != SNullWidget::NullWidget)
			{
				ExtensionWidgets->AddSlot()
				.AutoWidth()
				[
					ExtensionWidget.ToSharedRef()
				];
			}
		}

		return ExtensionWidgets;
	}
	return SNullWidget::NullWidget;
}

UPanelExtensionSubsystem::FPanelFactoryRegistryChanged& UPanelExtensionSubsystem::OnPanelFactoryRegistryChanged(FName ExtensionPanelID)
{
	return PanelFactoryRegistryChangedCallbackMap.FindOrAdd(ExtensionPanelID);
}
