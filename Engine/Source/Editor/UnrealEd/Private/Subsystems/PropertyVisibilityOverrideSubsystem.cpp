// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/PropertyVisibilityOverrideSubsystem.h"

#include "Editor.h"
#include "UObject/PropertyNames.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PropertyVisibilityOverrideSubsystem)

bool UE::PropertyVisibility::ConsiderPropertyForOverriddenState(TNotNull<const FProperty*> Property)
{
	// @Todo: find a better way to do this.
	// Hack for object array that are not CP_Edit but are editable via the card layout using a custom details panel like entities and components
	constexpr EPropertyFlags MustHaveFlagsToBeConsidered = CPF_Edit | CPF_ExperimentalOverridableLogic;

	// Check if the property or the owner property(containers) has any of the must-have flags
	const FProperty* OwnerProperty = Property->GetOwnerProperty();
	if (!Property->HasAnyPropertyFlags(MustHaveFlagsToBeConsidered) &&
		(!OwnerProperty || !OwnerProperty->HasAnyPropertyFlags(MustHaveFlagsToBeConsidered)))
	{
		return false;
	}

	// Is the property filtered by user permission?
	if (Property->GetBoolMetaData(PropertyNames::PropertyVisibilityOverrideName))
	{
		if (UPropertyVisibilityOverrideSubsystem::Get() && UPropertyVisibilityOverrideSubsystem::Get()->ShouldHideProperty(Property))
		{
			return false;
		}
	}

	return true;
}


UPropertyVisibilityOverrideSubsystem* UPropertyVisibilityOverrideSubsystem::Get()
{
	if (GEditor)
	{
		return GEditor->GetEditorSubsystem<UPropertyVisibilityOverrideSubsystem>();
	}

	return nullptr;
}

void UPropertyVisibilityOverrideSubsystem::RegisterShouldHidePropertyDelegate(const FName& DelegateName, const FShouldHidePropertyDelegate& Delegate)
{
	ShouldHidePropertyDelegates.Add(DelegateName, Delegate);
}

void UPropertyVisibilityOverrideSubsystem::UnregisterShouldHidePropertyDelegate(const FName& DelegateName)
{
	ShouldHidePropertyDelegates.Remove(DelegateName);
}

bool UPropertyVisibilityOverrideSubsystem::ShouldHideProperty(const FProperty* Property) const
{
	for (auto&& DelegatePair : ShouldHidePropertyDelegates)
	{
		const FShouldHidePropertyDelegate& Delegate = DelegatePair.Value;
		if (Delegate.IsBound() && Delegate.Execute(Property))
		{
			return true;
		}
	}

	return false;
}

void UPropertyVisibilityOverrideSubsystem::RegisterShouldHideDetailsPanelPropertyDelegate(const FName& DelegateName, const FShouldHideDetailsPanelPropertyDelegate& Delegate)
{
	DetailsPanelShouldHidePropertyDelegates.Add(DelegateName, Delegate);
}

void UPropertyVisibilityOverrideSubsystem::UnregisterShouldHideDetailsPanelPropertyDelegate(const FName& DelegateName)
{
	DetailsPanelShouldHidePropertyDelegates.Remove(DelegateName);
}

bool UPropertyVisibilityOverrideSubsystem::ShouldHidePropertyForDetailsPanel(const IPropertyHandle& PropertyHandle) const
{
	for (auto&& DelegatePair : DetailsPanelShouldHidePropertyDelegates)
	{
		const FShouldHideDetailsPanelPropertyDelegate& Delegate = DelegatePair.Value;
		if (Delegate.IsBound() && Delegate.Execute(PropertyHandle))
		{
			return true;
		}
	}

	return false;
}
