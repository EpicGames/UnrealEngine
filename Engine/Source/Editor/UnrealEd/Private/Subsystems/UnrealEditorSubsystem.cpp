// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/UnrealEditorSubsystem.h"
#include "Editor.h"
#include "SLevelViewport.h"
#include "SceneView.h"
#include "LevelEditor.h"
#include "Utils.h"
#include "GameFramework/Actor.h"
#include "EditorScriptingHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UnrealEditorSubsystem)

namespace InternalUnrealEditorSubsystemLibrary
{
	UWorld* GetEditorWorld()
	{
		return GEditor ? GEditor->GetEditorWorldContext(false).World() : nullptr;
	}

	UWorld* GetGameWorld()
	{
		if (GEditor)
		{
			if (FWorldContext* WorldContext = GEditor->GetPIEWorldContext())
			{
				return WorldContext->World();
			}

			return nullptr;
		}

		return GWorld;
	}
}

bool UUnrealEditorSubsystem::GetLevelViewportCameraInfo(FVector& CameraLocation, FRotator& CameraRotation)
{
	CameraLocation = FVector::ZeroVector;
	CameraRotation = FRotator::ZeroRotator;

	if (FLevelEditorViewportClient* LevelVC = GetViewportClient())
	{
		CameraLocation = LevelVC->GetViewLocation();
		CameraRotation = LevelVC->GetViewRotation();
		return true;
	}

	return false;
}

void UUnrealEditorSubsystem::SetLevelViewportCameraInfo(FVector CameraLocation, FRotator CameraRotation)
{
	if (FLevelEditorViewportClient* LevelVC = GetViewportClient())
	{
		LevelVC->SetViewLocationForOrbiting(CameraLocation);
		LevelVC->SetViewLocation(CameraLocation);
		LevelVC->SetViewRotation(CameraRotation);
	}
}

UWorld* UUnrealEditorSubsystem::GetEditorWorld()
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return nullptr;
	}

	return InternalUnrealEditorSubsystemLibrary::GetEditorWorld();
}

UWorld* UUnrealEditorSubsystem::GetGameWorld()
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	return InternalUnrealEditorSubsystemLibrary::GetGameWorld();
}

bool UUnrealEditorSubsystem::ScreenToWorld(const FVector2D& ScreenPosition, FVector& WorldPosition, FVector& WorldDirection) const
{	
	if (FLevelEditorViewportClient* LevelVC = GetViewportClient())
	{		
		FSceneViewFamily ViewFamily(FSceneViewFamily::ConstructionValues(
			LevelVC->Viewport,
			LevelVC->GetScene(),
			LevelVC->EngineShowFlags).SetRealtimeUpdate(LevelVC->IsRealtime()));
		TSharedPtr<FSceneView> SceneView = MakeShareable<>(LevelVC->CalcSceneView(&ViewFamily));
		const FMatrix InvViewProjectionMatrix = SceneView->ViewMatrices.GetClipToWorld();
		FSceneView::DeprojectScreenToWorld(ScreenPosition, FIntRect(FIntPoint(0, 0), LevelVC->Viewport->GetSizeXY()), InvViewProjectionMatrix, WorldPosition, WorldDirection);
		return true;
	}

	// something went wrong, zero things and return false
	WorldPosition = FVector::ZeroVector;
	WorldDirection = FVector::ZeroVector;
	return false;
}

bool UUnrealEditorSubsystem::WorldToScreen(const FVector& WorldPosition, FVector2D& ScreenPosition) const
{
	if (FLevelEditorViewportClient* LevelVC = GetViewportClient())
	{
		FSceneViewFamily ViewFamily(FSceneViewFamily::ConstructionValues(
			LevelVC->Viewport,
			LevelVC->GetScene(),
			LevelVC->EngineShowFlags).SetRealtimeUpdate(LevelVC->IsRealtime()));
		TSharedPtr<FSceneView> SceneView = MakeShareable<>(LevelVC->CalcSceneView(&ViewFamily));
		return FSceneView::ProjectWorldToScreen(WorldPosition, FIntRect(FIntPoint(0, 0), LevelVC->Viewport->GetSizeXY()), SceneView->ViewMatrices.GetWorldToClip(), ScreenPosition);
	}

	ScreenPosition = FVector2D::ZeroVector;
	return false;
}

UNREALED_API bool UUnrealEditorSubsystem::GetLevelViewportSize(FIntPoint& Size) const
{
	if (FLevelEditorViewportClient* LevelVC = GetViewportClient())
	{
		if (LevelVC->Viewport)
		{
			Size = LevelVC->Viewport->GetSizeXY();
			return true;
		}
	}

	Size = FIntPoint::ZeroValue;
	return false;
}

FLevelEditorViewportClient* UUnrealEditorSubsystem::GetViewportClient() const
{
	for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		if (LevelVC && LevelVC->IsPerspective())
		{
			return LevelVC;
		}
	}
	return nullptr;
}
