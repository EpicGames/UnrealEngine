// Copyright Epic Games, Inc. All Rights Reserved.

#include "TargetDomain/TargetDomainUtils.h"

#include "Algo/BinarySearch.h"
#include "Algo/IsSorted.h"
#include "Algo/Sort.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Async/Future.h"
#include "Compression/CompressedBuffer.h"
#include "Cooker/CookArtifact.h"
#include "EditorDomain/EditorDomain.h"
#include "EditorDomain/EditorDomainUtils.h"
#include "HAL/PlatformFileManager.h"
#include "IO/IoStatus.h"
#include "Logging/LogMacros.h"
#include "Misc/App.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Optional.h"
#include "Misc/Paths.h"
#include "Misc/ScopeRWLock.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"
#include "UObject/TopLevelAssetPath.h"

namespace UE::TargetDomain
{

bool IsIncrementalCookEnabled(FName PackageName, bool bAllowAllClasses)
{
	IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
	if (!AssetRegistry)
	{
		return false;
	}
	
	TArray<FName>* ImportedClasses = nullptr;
	TArray<FName> InMemoryImportedClasses;
	TOptional<FAssetPackageData> PackageDataOpt = AssetRegistry->GetAssetPackageDataCopy(PackageName);
	if (PackageDataOpt)
	{
		ImportedClasses = &PackageDataOpt->ImportedClasses;
	}
	else
	{
		TStringBuilder<256> PackageNameStr(EInPlace::InPlace, PackageName);
		const UPackage* Package = FindPackage(nullptr, *PackageNameStr);
		if (!Package)
		{
			return false;
		}

		bool bInMemoryPackage = Package->HasAnyPackageFlags(PKG_InMemoryOnly);
		if (!bInMemoryPackage)
		{
			// Strange case where the package does not exist on disk nor in memory.
			return false;
		}
		TSet<UClass*> PackageClasses;
		ForEachObjectWithPackage(Package, [&PackageClasses, Package](UObject* Object)
			{
				UClass* Class = Object->GetClass();
				if (!Class->IsInPackage(Package)) // Imported classes list does not include classes in the package
				{
					PackageClasses.Add(Object->GetClass());
				}
				return true;
			});
		InMemoryImportedClasses.Reserve(PackageClasses.Num());
		for (UClass* Class : PackageClasses)
		{
			TStringBuilder<256> ClassPath;
			Class->GetPathName(nullptr, ClassPath);
			InMemoryImportedClasses.Add(FName(ClassPath));
		}
		InMemoryImportedClasses.Sort(FNameLexicalLess());
		ImportedClasses = &InMemoryImportedClasses;
	}

	if (!bAllowAllClasses)
	{
		UE::EditorDomain::FClassDigestMap& ClassDigests = UE::EditorDomain::GetClassDigests();
		FReadScopeLock ClassDigestsScopeLock(ClassDigests.Lock);
		for (FName ClassName : *ImportedClasses)
		{
			FTopLevelAssetPath ClassPath(WriteToString<256>(ClassName).ToView());
			UE::EditorDomain::FClassDigestData* ExistingData = nullptr;
			if (ClassPath.IsValid())
			{
				ExistingData = ClassDigests.Map.Find(ClassPath);
			}
			else if (!ClassName.IsNone())
			{
				// All classes are top-level objects, but user-defined structs are not top-level
				// objects.  In this code we don't need to handle user-defined structs because
				// we do not support deny-listing user-defined structs.  So if the ClassName is
				// not a top level asset, then ignore it.
				continue;
			}

			if (!ExistingData)
			{
				// !ExistingData -> !allowed, because caller has already called CalculatePackageDigest, so all
				// existing classes in the package have been added to ClassDigests.
				UE_LOGF(LogEditorDomain, Verbose, "NonIterative Package %ls due to missing class %ls",
					*PackageName.ToString(), *ClassPath.ToString());
				return false;
			}

			UE::EditorDomain::FClassDigestData* DataToRead = ExistingData;
			FTopLevelAssetPath* ClassToRead = &ClassPath;
			if (!DataToRead->bNative)
			{
				// TODO: We need to add a way to mark non-native classes (there can be many of them) as allowed or denied.
				// Currently we are allowing them all, so long as their closest native is allowed. But this is not completely
				// safe to do, because non-native classes can add constructionevents that e.g. use the Random function.
				ClassToRead = &DataToRead->ClosestNative;
				DataToRead = ClassDigests.Map.Find(DataToRead->ClosestNative);
				if (!DataToRead)
				{
					UE_LOGF(LogEditorDomain, Verbose,
						"NonIterative Package %ls due to missing native super class %ls of non-native class %ls",
						*PackageName.ToString(), *ExistingData->ClosestNative.ToString(), *ClassPath.ToString());
					return false;
				}
			}
			if (!DataToRead->bTargetIterativeEnabled)
			{
				UE_LOGF(LogEditorDomain, Verbose,
					"NonIterative Package %ls due to bTargetIterativeEnabled=false on class %ls%ls",
					*PackageName.ToString(), *ClassToRead->ToString(),
					(ExistingData == DataToRead ? TEXT("")
						: *FString::Printf(TEXT(", the closest native superclass of non-native class %s"), *ClassPath.ToString())));
				return false;
			}
		}
	}
	return true;
}

TUniquePtr<FEditorDomainOplog> GEditorDomainOplog;
TArray<const UTF8CHAR*> FEditorDomainOplog::ReservedOplogKeys;

FEditorDomainOplog::FEditorDomainOplog()
{
	StaticInit();

	if (UE::Zen::FZenServiceInstance::GetAutoLaunchedPort() != 0)
	{
		// Connect to auto-launched service
		HttpClient = MakeUnique<UE::FZenStoreHttpClient>();
	}
	else
	{
		HttpClient = MakeUnique<UE::FZenStoreHttpClient>(TEXT("localhost"), 8558);
	}

	FString ProjectId = FApp::GetZenStoreProjectId();
	FString OplogId = TEXT("EditorDomain");

	FString RootDir = FPaths::RootDir();
	FString EngineDir = FPaths::EngineDir();
	FPaths::NormalizeDirectoryName(EngineDir);
	FString ProjectDir = FPaths::ProjectDir();
	FPaths::NormalizeDirectoryName(ProjectDir);
	FString ProjectPath = FPaths::GetProjectFilePath();
	FPaths::NormalizeFilename(ProjectPath);

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	FString AbsServerRoot = PlatformFile.ConvertToAbsolutePathForExternalAppForRead(*RootDir);
	FString AbsEngineDir = PlatformFile.ConvertToAbsolutePathForExternalAppForRead(*EngineDir);
	FString AbsProjectDir = PlatformFile.ConvertToAbsolutePathForExternalAppForRead(*ProjectDir);
	FString ProjectFilePath = PlatformFile.ConvertToAbsolutePathForExternalAppForRead(*ProjectPath);

	if (UE::Zen::IsDefaultServicePresent())
	{
		bool IsLocalConnection = HttpClient->GetZenServiceInstance().IsServiceRunningLocally();
		HttpClient->TryCreateProject(ProjectId, FStringView(), OplogId, AbsServerRoot, AbsEngineDir, AbsProjectDir, IsLocalConnection ? ProjectFilePath : FStringView());
		HttpClient->TryCreateOplog(ProjectId, OplogId, TEXT("") /*InOplogMarkerFile*/);
	}
}

void FEditorDomainOplog::InitializeRead()
{
	if (bInitializedRead)
	{
		return;
	}
	UE_LOGF(LogEditorDomain, Display, "Fetching EditorDomain oplog...");

	TFuture<FIoStatus> FutureOplogStatus = HttpClient->GetOplog().Next([this](TIoStatusOr<FCbObject> OplogStatus)
		{
			if (!OplogStatus.IsOk())
			{
				return OplogStatus.Status();
			}

			FCbObject Oplog = OplogStatus.ConsumeValueOrDie();

			for (FCbField& EntryObject : Oplog["entries"])
			{
				FUtf8StringView PackageName = EntryObject["key"].AsString();
				if (PackageName.IsEmpty())
				{
					continue;
				}
				FName PackageFName(PackageName);
				FOplogEntry& Entry = Entries.FindOrAdd(PackageFName);
				Entry.Attachments.Empty();

				for (FCbFieldView Field : EntryObject)
				{
					FUtf8StringView FieldName = Field.GetName();
					if (IsReservedOplogKey(FieldName))
					{
						continue;
					}
					if (Field.IsHash())
					{
						const UTF8CHAR* AttachmentId = UE::FZenStoreHttpClient::FindOrAddAttachmentId(FieldName);
						Entry.Attachments.Add({ AttachmentId, Field.AsHash() });
					}
				}
				Entry.Attachments.Shrink();
				check(Algo::IsSorted(Entry.Attachments, [](const FOplogEntry::FAttachment& A, const FOplogEntry::FAttachment& B)
					{
						return FUtf8StringView(A.Key).Compare(FUtf8StringView(B.Key), ESearchCase::IgnoreCase) < 0;
					}));
			}

			return FIoStatus::Ok;
		});
	FutureOplogStatus.Get();
	bInitializedRead = true;
}

FCbAttachment FEditorDomainOplog::CreateAttachment(FSharedBuffer AttachmentData)
{
	FCompressedBuffer CompressedBuffer = FCompressedBuffer::Compress(AttachmentData);
	check(!CompressedBuffer.IsNull());
	return FCbAttachment(CompressedBuffer);
}

void FEditorDomainOplog::StaticInit()
{
	if (ReservedOplogKeys.Num() > 0)
	{
		return;
	}

	ReservedOplogKeys.Append({ UTF8TEXT("key") });
	Algo::Sort(ReservedOplogKeys, [](const UTF8CHAR* A, const UTF8CHAR* B)
		{
			return FUtf8StringView(A).Compare(FUtf8StringView(B), ESearchCase::IgnoreCase) < 0;
		});;
}

bool FEditorDomainOplog::IsReservedOplogKey(FUtf8StringView Key)
{
	int32 Index = Algo::LowerBound(ReservedOplogKeys, Key,
		[](const UTF8CHAR* Existing, FUtf8StringView Key)
		{
			return FUtf8StringView(Existing).Compare(Key, ESearchCase::IgnoreCase) < 0;
		});
	return Index != ReservedOplogKeys.Num() &&
		FUtf8StringView(ReservedOplogKeys[Index]).Equals(Key, ESearchCase::IgnoreCase);
}

bool FEditorDomainOplog::IsValid() const
{
	return HttpClient->IsConnected();
}

void FEditorDomainOplog::CommitPackage(FName PackageName, UE::Cook::Artifact::FAppendPackageMetaDataContext& Context)
{
	using namespace UE::Cook::Artifact;

	FScopeLock ScopeLock(&Lock);

	FCbPackage Pkg;

	TArray<IPackageWriter::FCommitAttachmentInfo> Attachments;
	Attachments.Reserve(Context.Entries.Num());
	for (FAppendPackageMetaDataContext::FEntry& PackageMetaEntry : Context.Entries)
	{
		IPackageWriter::FCommitAttachmentInfo& NewAttachment = Attachments.Emplace_GetRef();
		NewAttachment.Key = PackageMetaEntry.Key;
		NewAttachment.Value = MoveTemp(PackageMetaEntry.Value);
		NewAttachment.FieldStorage = PackageMetaEntry.FieldStorage;
	}
	TArray<FCbAttachment, TInlineAllocator<2>> CbAttachments;
	int32 NumAttachments = Attachments.Num();
	FOplogEntry& Entry = Entries.FindOrAdd(PackageName);
	Entry.Attachments.Empty(NumAttachments);
	if (NumAttachments)
	{
		TArray<const IPackageWriter::FCommitAttachmentInfo*, TInlineAllocator<2>> SortedAttachments;
		SortedAttachments.Reserve(NumAttachments);
		for (const IPackageWriter::FCommitAttachmentInfo& AttachmentInfo : Attachments)
		{
			SortedAttachments.Add(&AttachmentInfo);
		}
		SortedAttachments.Sort([](const IPackageWriter::FCommitAttachmentInfo& A, const IPackageWriter::FCommitAttachmentInfo& B)
			{
				return A.Key.ToString().Compare(B.Key.ToString(), ESearchCase::IgnoreCase) < 0;
			});
		CbAttachments.Reserve(NumAttachments);
		for (const IPackageWriter::FCommitAttachmentInfo* AttachmentInfo : SortedAttachments)
		{
			// All attachment keys are ASCII; convert FName to UTF-8 string view for the HTTP client APIs.
			const FString KeyStr = AttachmentInfo->Key.ToString();
			auto KeyConversion = StringCast<UTF8CHAR>(*KeyStr);
			const FUtf8StringView KeyView(KeyConversion.Get(), KeyConversion.Length());

			const FCbAttachment& CbAttachment = CbAttachments.Add_GetRef(CreateAttachment(AttachmentInfo->Value));
			check(!IsReservedOplogKey(KeyView));
			Pkg.AddAttachment(CbAttachment);
			Entry.Attachments.Add(FOplogEntry::FAttachment{
				UE::FZenStoreHttpClient::FindOrAddAttachmentId(KeyView), CbAttachment.GetHash() });
		}
	}

	FCbWriter PackageObj;
	FString PackageNameKey = PackageName.ToString();
	PackageNameKey.ToLowerInline();
	PackageObj.BeginObject();
	PackageObj << "key" << PackageNameKey;
	for (int32 Index = 0; Index < NumAttachments; ++Index)
	{
		FCbAttachment& CbAttachment = CbAttachments[Index];
		FOplogEntry::FAttachment& EntryAttachment = Entry.Attachments[Index];
		PackageObj << EntryAttachment.Key << CbAttachment;
	}
	PackageObj.EndObject();

	FCbObject Obj = PackageObj.Save().AsObject();
	Pkg.SetObject(Obj);


	TIoStatusOr<uint64> Status = HttpClient->AppendOp(Pkg);
	if (!Status.IsOk())
	{
		UE_LOGF(LogEditorDomain, Warning, "Failed to commit oplog entry '%ls' to Zen, retrying: Reason: %ls", *PackageNameKey, *Status.Status().ToString());
		Status = HttpClient->AppendOp(MoveTemp(Pkg));
		if (!Status.IsOk())
		{
			UE_LOGF(LogEditorDomain, Error, "Failed to commit oplog entry '%ls' to Zen: Reason: %ls", *PackageNameKey, *Status.Status().ToString());
		}
		else
		{
			UE_LOGF(LogEditorDomain, Verbose, "Commit oplog entry '%ls' to Zen succeeded on retry", *PackageNameKey);
		}
	}
}

// Note that this is destructive - we yank out the buffer memory from the 
// IoBuffer into the FSharedBuffer
FSharedBuffer IoBufferToSharedBuffer(FIoBuffer& InBuffer)
{
	InBuffer.EnsureOwned();
	const uint64 DataSize = InBuffer.DataSize();
	uint8* DataPtr = InBuffer.Release().ValueOrDie();
	return FSharedBuffer{ FSharedBuffer::TakeOwnership(DataPtr, DataSize, FMemory::Free) };
};

void FEditorDomainOplog::GetOplogAttachments(TArrayView<FName> PackageNames,
	TArrayView<FUtf8StringView> AttachmentKeys,
	TUniqueFunction<void(FName PackageName, FUtf8StringView AttachmentKey, FCbObject&& Attachment)>&& Callback)
{
	const int MaximumHashCount = PackageNames.Num() * AttachmentKeys.Num();
	TArray<FIoHash> AttachmentHashes;
	AttachmentHashes.Reserve(MaximumHashCount);
	struct FAttachmentHashParam
	{
		FName PackageName;
		FUtf8StringView AttachmentKey;

		FAttachmentHashParam(const FName& InPackageName, FUtf8StringView InAttachmentKey)
		: PackageName(InPackageName), AttachmentKey(InAttachmentKey)
		{
		}
	};
	TMultiMap<FIoHash, FAttachmentHashParam> AttachmentHashParams;
	AttachmentHashParams.Reserve(MaximumHashCount);

	TArray<FAttachmentHashParam> InvalidAttachmentHashParams;
	InvalidAttachmentHashParams.Reserve(MaximumHashCount);

	TArray<const UTF8CHAR*, TInlineAllocator<2>> AttachmentIds;
	for (FUtf8StringView AttachmentKey : AttachmentKeys)
	{
		AttachmentIds.Add(UE::FZenStoreHttpClient::FindAttachmentId(AttachmentKey));
	}

	{
		FScopeLock _(&Lock);
		InitializeRead();

		for (FName PackageName : PackageNames)
		{
			FOplogEntry* Entry = Entries.Find(PackageName);

			for (int32 InputAttachmentIndex = 0; InputAttachmentIndex < AttachmentKeys.Num(); ++InputAttachmentIndex)
			{
				FUtf8StringView AttachmentKey = AttachmentKeys[InputAttachmentIndex];
				const UTF8CHAR* AttachmentId = AttachmentIds[InputAttachmentIndex];

				FIoHash AttachmentHash;
				ON_SCOPE_EXIT
				{
					if (AttachmentHash.IsZero())
					{
						InvalidAttachmentHashParams.Emplace(PackageName, AttachmentKey);
					}
					else
					{
						AttachmentHashes.Add(AttachmentHash);
						AttachmentHashParams.Emplace(AttachmentHash, FAttachmentHashParam{PackageName, AttachmentKey});
					}
				};

				if (!Entry || !AttachmentId)
				{
					continue;
				}

				FUtf8StringView AttachmentIdView(AttachmentId);

				int32 AttachmentIndex = Algo::LowerBound(Entry->Attachments, AttachmentIdView,
					[](const FOplogEntry::FAttachment& Existing, FUtf8StringView AttachmentIdView)
					{
						return FUtf8StringView(Existing.Key).Compare(AttachmentIdView, ESearchCase::IgnoreCase) < 0;
					});
				if (AttachmentIndex == Entry->Attachments.Num())
				{
					continue;
				}
				const FOplogEntry::FAttachment& Existing = Entry->Attachments[AttachmentIndex];
				if (!FUtf8StringView(Existing.Key).Equals(AttachmentIdView, ESearchCase::IgnoreCase))
				{
					continue;
				}
				AttachmentHash = Existing.Hash;
			}
		}
	}

	// Invoke the callback for all invalid attachment hashes
	for (FAttachmentHashParam& InvalidAttachmentHashParam : InvalidAttachmentHashParams)
	{
		Callback(InvalidAttachmentHashParam.PackageName, InvalidAttachmentHashParam.AttachmentKey, FCbObject());
	}
	
	if (AttachmentHashes.IsEmpty())
	{
		return;
	}

	HttpClient->ReadChunks(AttachmentHashes, [Callback = MoveTemp(Callback),&AttachmentHashParams](const FIoHash& RawHash, TIoStatusOr<FIoBuffer> Result)
	{
		for (auto It(AttachmentHashParams.CreateConstKeyIterator(RawHash)); It; ++It)
		{
			const FAttachmentHashParam& Param = It.Value();
			if (!Result.IsOk())
			{
				Callback(Param.PackageName, Param.AttachmentKey, FCbObject());
				continue;
			}

			FIoBuffer Buffer = Result.ConsumeValueOrDie();
			if (Buffer.DataSize() == 0)
			{
				Callback(Param.PackageName, Param.AttachmentKey, FCbObject());
				continue;
			}
			FSharedBuffer SharedBuffer = IoBufferToSharedBuffer(Buffer);
			Callback(Param.PackageName, Param.AttachmentKey, FCbObject(SharedBuffer));
		}
	});
}

void CommitEditorDomainCookAttachments(FName PackageName, UE::Cook::Artifact::FAppendPackageMetaDataContext& Context)
{
	if (!GEditorDomainOplog)
	{
		return;
	}
	GEditorDomainOplog->CommitPackage(PackageName, Context);
}

void CookInitialize()
{
	bool bCookAttachmentsEnabled = true;
	GConfig->GetBool(TEXT("EditorDomain"), TEXT("CookAttachmentsEnabled"), bCookAttachmentsEnabled, GEditorIni);
	if (bCookAttachmentsEnabled)
	{
		GEditorDomainOplog = MakeUnique<FEditorDomainOplog>();
		if (!GEditorDomainOplog->IsValid())
		{
			UE_LOGF(LogEditorDomain, Display, "Failed to connect to ZenServer; EditorDomain oplog is unavailable.");
			GEditorDomainOplog.Reset();
		}
	}
}

} // namespace UE::TargetDomain
