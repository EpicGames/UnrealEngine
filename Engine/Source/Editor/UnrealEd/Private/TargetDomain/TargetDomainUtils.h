// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/StringView.h"
#include "HAL/CriticalSection.h"
#include "IO/IoHash.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/PackageWriter.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"
#include "ZenStoreHttpClient.h"

class FAssetPackageData;
class FSharedBuffer;

namespace UE::Cook::Artifact { struct FAppendPackageMetaDataContext; }

namespace UE::TargetDomain
{

/** Call during Startup to initialize global data used by TargetDomain functions. */
void CookInitialize();

/** Return whether incremental cook is enabled for the given packagename, based on used-class allowlist/blocklist. */
bool IsIncrementalCookEnabled(FName PackageName, bool bAllowAllClasses);

/** Store extra information derived during save and used by the cooker for the given EditorDomain package. */
void CommitEditorDomainCookAttachments(FName PackageName, UE::Cook::Artifact::FAppendPackageMetaDataContext& Context);

/**
 * Reads / writes an oplog for EditorDomain BuildDefinitionLists.
 * TODO: Reduce duplication between this class and FZenStoreWriter
 */
class FEditorDomainOplog
{
public:
	FEditorDomainOplog();

	bool IsValid() const;
	void CommitPackage(FName PackageName, UE::Cook::Artifact::FAppendPackageMetaDataContext& Context);
	void GetOplogAttachments(TArrayView<FName> PackageName,
		TArrayView<FUtf8StringView> AttachmentKeys,
		TUniqueFunction<void(FName PackageName, FUtf8StringView AttachmentKey, FCbObject&& Attachment)>&& Callback);

private:
	struct FOplogEntry
	{
		struct FAttachment
		{
			const UTF8CHAR* Key;
			FIoHash Hash;
		};

		TArray<FAttachment> Attachments;
	};

	void InitializeRead();

	FCbAttachment CreateAttachment(FSharedBuffer AttachmentData);
	FCbAttachment CreateAttachment(FCbObject AttachmentData)
	{
		return CreateAttachment(AttachmentData.GetBuffer().ToShared());
	}

	static void StaticInit();
	static bool IsReservedOplogKey(FUtf8StringView Key);

	TUniquePtr<UE::FZenStoreHttpClient> HttpClient;
	FCriticalSection Lock;
	TMap<FName, FOplogEntry> Entries;
	bool bConnectSuccessful = false;
	bool bInitializedRead = false;

	static TArray<const UTF8CHAR*> ReservedOplogKeys;
};
extern TUniquePtr<FEditorDomainOplog> GEditorDomainOplog;

} // namespace UE::TargetDomain
