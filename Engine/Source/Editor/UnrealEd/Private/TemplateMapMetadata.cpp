// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TemplateMapMetadata.cpp: map template related code
=============================================================================*/

#include "Editor/TemplateMapMetadata.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TemplateMapMetadata)

/*-----------------------------------------------------------------------------
	UTemplateMapMetadata.
-----------------------------------------------------------------------------*/

UDEPRECATED_TemplateMapMetadata::UDEPRECATED_TemplateMapMetadata(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}
