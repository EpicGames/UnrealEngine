// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/AutomationEditorCommon.h"
#include "UObject/UnrealType.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "Serialization/FindReferencersArchive.h"
#include "Editor/UnrealEdEngine.h"
#include "Factories/Factory.h"
#include "Factories/TextureFactory.h"
#include "Framework/Application/SlateApplication.h"
#include "Engine/StaticMesh.h"
#include "Misc/ConfigCacheIni.h"
#include "Serialization/ArchiveReplaceObjectRef.h"
#include "NavigationSystem.h"
#include "Settings/LevelEditorPlaySettings.h"
#include "GameFramework/GameStateBase.h"
#include "GameFramework/WorldSettings.h"
#include "Engine/StaticMeshActor.h"
#include "LevelEditorViewport.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "FileHelpers.h"
#include "UnrealEdGlobals.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Tests/AutomationCommon.h"
#include "IAssetViewport.h"

#include "LevelEditor.h"
#include "Interfaces/IMainFrameModule.h"
#include "ShaderCompiler.h"
#include "AssetSelection.h"
#include "ITargetDeviceProxy.h"
#include "ITargetDeviceServicesModule.h"
#include "ILauncherWorker.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "LightingBuildOptions.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Bookmarks/IBookmarkTypeTools.h"
#include "GameMapsSettings.h"
#include "Editor/EditorPerformanceSettings.h"
#include "TextureCompiler.h"

#if WITH_AUTOMATION_TESTS

#define COOK_TIMEOUT 3600

DEFINE_LOG_CATEGORY_STATIC(LogAutomationEditorCommon, Log, All);


UWorld* FAutomationEditorCommonUtils::CreateNewMap()
{
	// Also change out of Landscape mode to ensure all references are cleared.
	if ( GLevelEditorModeTools().IsModeActive(FBuiltinEditorModes::EM_Landscape) )
	{
		GLevelEditorModeTools().DeactivateMode(FBuiltinEditorModes::EM_Landscape);
	}

	// Also change out of Foliage mode to ensure all references are cleared.
	if ( GLevelEditorModeTools().IsModeActive(FBuiltinEditorModes::EM_Foliage) )
	{
		GLevelEditorModeTools().DeactivateMode(FBuiltinEditorModes::EM_Foliage);
	}

	// Change out of mesh paint mode when opening a new map.
	if ( GLevelEditorModeTools().IsModeActive(FBuiltinEditorModes::EM_MeshPaint) )
	{
		GLevelEditorModeTools().DeactivateMode(FBuiltinEditorModes::EM_MeshPaint);
	}

	return GEditor->NewMap();
}

TUniquePtr<FScopedEditorWorld> FAutomationEditorCommonUtils::CreateScopedEditorWorld(
	const FString& TemplateNameOrPath,
	const UWorld::InitializationValues& InInitializationValues,
	EWorldType::Type InWorldType)
{
	// Get the list of template maps
	TArray<FTemplateMapInfo> TemplateMaps;
	FString TemplatePath;

	if (GUnrealEd)
	{
		// Get the list of template maps from the editor engine
		TemplateMaps = GUnrealEd->GetTemplateMapInfos();
	}

	// Search for the template by its display name
	for (const FTemplateMapInfo& Template : TemplateMaps)
	{
		if (Template.DisplayName.ToString() == TemplateNameOrPath)
		{
			TemplatePath = Template.Map.ToString();
			break;
		}
	}

	// Create a new FScopedEditorWorld using the template.
	if (!TemplatePath.IsEmpty())
	{
		return MakeUnique<FScopedEditorWorld>(TSoftObjectPtr<UWorld>(FSoftObjectPath(TemplatePath)), InInitializationValues, InWorldType);
	}

	// Create a new FScopedEditorWorld using the path to the level.
	TUniquePtr<FScopedEditorWorld> ScopedWorld = MakeUnique<FScopedEditorWorld>(TSoftObjectPtr<UWorld>(FSoftObjectPath(TemplateNameOrPath)), InInitializationValues, InWorldType);
	if (!ScopedWorld)
	{
		UE_LOGF(LogAutomationEditorCommon, Error, "Failed to create scoped editor world. Could not resolve template name or path: %ls", *TemplateNameOrPath);
	}
	return ScopedWorld;
}

/**
* Imports an object using a given factory
*
* @param ImportFactory - The factory to use to import the object
* @param ObjectName - The name of the object to create
* @param PackagePath - The full path of the package file to create
* @param ImportPath - The path to the object to import
*/
UObject* FAutomationEditorCommonUtils::ImportAssetUsingFactory(UFactory* ImportFactory, const FString& ObjectName, const FString& PackagePath, const FString& ImportPath)
{
	UObject* ImportedAsset = NULL;

	UPackage* Pkg = CreatePackage( *PackagePath);
	if (Pkg)
	{
		// Make sure the destination package is loaded
		Pkg->FullyLoad();

		UClass* ImportAssetType = ImportFactory->ResolveSupportedClass();
		bool bDummy = false;

		//If we are a texture factory suppress some warning dialog that we don't want
		if (ImportFactory->IsA(UTextureFactory::StaticClass()))
		{
			UTextureFactory::SuppressImportOverwriteDialog();
		}

		bool OutCanceled = false;
		ImportedAsset = ImportFactory->ImportObject(ImportAssetType, Pkg, FName(*ObjectName), RF_Public | RF_Standalone, ImportPath, nullptr, OutCanceled);

		if (ImportedAsset != nullptr)
		{
			UE_LOGF(LogAutomationEditorCommon, Display, "Imported %ls", *ImportPath);
		}
		else if (OutCanceled)
		{
			UE_LOGF(LogAutomationEditorCommon, Display, "Canceled import of %ls", *ImportPath);
		}
		else
		{
			UE_LOGF(LogAutomationEditorCommon, Error, "Failed to import asset using factory %ls!", *ImportFactory->GetName());
		}
	}
	else
	{
		UE_LOGF(LogAutomationEditorCommon, Error, "Failed to create a package!");
	}

	return ImportedAsset;
}

/**
* Nulls out references to a given object
*
* @param InObject - Object to null references to
*/
void FAutomationEditorCommonUtils::NullReferencesToObject(UObject* InObject)
{
	TArray<UObject*> ReplaceableObjects;
	TMap<UObject*, UObject*> ReplacementMap;
	ReplacementMap.Add(InObject, NULL);
	ReplacementMap.GenerateKeyArray(ReplaceableObjects);

	// Find all the properties (and their corresponding objects) that refer to any of the objects to be replaced
	TMap< UObject*, TArray<FProperty*> > ReferencingPropertiesMap;
	for (FThreadSafeObjectIterator ObjIter; ObjIter; ++ObjIter)
	{
		UObject* CurObject = *ObjIter;

		// Find the referencers of the objects to be replaced
		FFindReferencersArchive FindRefsArchive(CurObject, ReplaceableObjects);

		// Inform the object referencing any of the objects to be replaced about the properties that are being forcefully
		// changed, and store both the object doing the referencing as well as the properties that were changed in a map (so that
		// we can correctly call PostEditChange later)
		TMap<UObject*, int32> CurNumReferencesMap;
		TMultiMap<UObject*, FProperty*> CurReferencingPropertiesMMap;
		if (FindRefsArchive.GetReferenceCounts(CurNumReferencesMap, CurReferencingPropertiesMMap) > 0)
		{
			TArray<FProperty*> CurReferencedProperties;
			CurReferencingPropertiesMMap.GenerateValueArray(CurReferencedProperties);
			ReferencingPropertiesMap.Add(CurObject, CurReferencedProperties);
			for (TArray<FProperty*>::TConstIterator RefPropIter(CurReferencedProperties); RefPropIter; ++RefPropIter)
			{
				CurObject->PreEditChange(*RefPropIter);
			}
		}

	}

	// Iterate over the map of referencing objects/changed properties, forcefully replacing the references and then
	// alerting the referencing objects the change has completed via PostEditChange
	int32 NumObjsReplaced = 0;
	for (TMap< UObject*, TArray<FProperty*> >::TConstIterator MapIter(ReferencingPropertiesMap); MapIter; ++MapIter)
	{
		++NumObjsReplaced;

		UObject* CurReplaceObj = MapIter.Key();
		const TArray<FProperty*>& RefPropArray = MapIter.Value();

		FArchiveReplaceObjectRef<UObject> ReplaceAr(CurReplaceObj, ReplacementMap, EArchiveReplaceObjectFlags::IgnoreOuterRef);

		for (TArray<FProperty*>::TConstIterator RefPropIter(RefPropArray); RefPropIter; ++RefPropIter)
		{
			FPropertyChangedEvent PropertyEvent(*RefPropIter);
			CurReplaceObj->PostEditChangeProperty(PropertyEvent);
		}

		if (!CurReplaceObj->HasAnyFlags(RF_Transient) && CurReplaceObj->GetOutermost() != GetTransientPackage())
		{
			if (!CurReplaceObj->RootPackageHasAnyFlags(PKG_CompiledIn))
			{
				CurReplaceObj->MarkPackageDirty();
			}
		}
	}
}

/**
* gets a factory class based off an asset file extension
*
* @param AssetExtension - The file extension to use to find a supporting UFactory
*/
UClass* FAutomationEditorCommonUtils::GetFactoryClassForType(const FString& AssetExtension)
{
	// First instantiate one factory for each file extension encountered that supports the extension
	for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		if ((*ClassIt)->IsChildOf(UFactory::StaticClass()) && !((*ClassIt)->HasAnyClassFlags(CLASS_Abstract)))
		{
			UFactory* Factory = Cast<UFactory>((*ClassIt)->GetDefaultObject());
			if (Factory->bEditorImport)
			{
				TArray<FString> FactoryExtensions;
				Factory->GetSupportedFileExtensions(FactoryExtensions);

				// Case insensitive string compare with supported formats of this factory
				if (FactoryExtensions.Contains(AssetExtension))
				{
					return *ClassIt;
				}
			}
		}
	}

	return NULL;
}

/**
* Applies settings to an object by finding UProperties by name and calling ImportText
*
* @param InObject - The object to search for matching properties
* @param PropertyChain - The list FProperty names recursively to search through
* @param Value - The value to import on the found property
*/
void FAutomationEditorCommonUtils::ApplyCustomFactorySetting(UObject* InObject, TArray<FString>& PropertyChain, const FString& Value)
{
	const FString PropertyName = PropertyChain[0];
	PropertyChain.RemoveAt(0);

	FProperty* TargetProperty = FindFProperty<FProperty>(InObject->GetClass(), *PropertyName);
	if (TargetProperty)
	{
		if (PropertyChain.Num() == 0)
		{
			TargetProperty->ImportText_InContainer(*Value, InObject, InObject, 0);
		}
		else
		{
			FStructProperty* StructProperty = CastField<FStructProperty>(TargetProperty);
			FObjectProperty* ObjectProperty = CastField<FObjectProperty>(TargetProperty);

			UObject* SubObject = NULL;
			bool bValidPropertyType = true;

			if (StructProperty)
			{
				SubObject = StructProperty->Struct;
			}
			else if (ObjectProperty)
			{
				SubObject = ObjectProperty->GetObjectPropertyValue(ObjectProperty->ContainerPtrToValuePtr<UObject>(InObject));
			}
			else
			{
				//Unknown nested object type
				bValidPropertyType = false;
				UE_LOGF(LogAutomationEditorCommon, Error, "ERROR: Unknown nested object type for property: %ls", *PropertyName);
			}

			if (SubObject)
			{
				ApplyCustomFactorySetting(SubObject, PropertyChain, Value);
			}
			else if (bValidPropertyType)
			{
				UE_LOGF(LogAutomationEditorCommon, Error, "Error accessing null property: %ls", *PropertyName);
			}
		}
	}
	else
	{
		UE_LOGF(LogAutomationEditorCommon, Error, "ERROR: Could not find factory property: %ls", *PropertyName);
	}
}

/**
* Applies the custom factory settings
*
* @param InFactory - The factory to apply custom settings to
* @param FactorySettings - An array of custom settings to apply to the factory
*/
void FAutomationEditorCommonUtils::ApplyCustomFactorySettings(UFactory* InFactory, const TArray<FImportFactorySettingValues>& FactorySettings)
{
	bool bCallConfigureProperties = true;

	for (int32 i = 0; i < FactorySettings.Num(); ++i)
	{
		if (FactorySettings[i].SettingName.Len() > 0 && FactorySettings[i].Value.Len() > 0)
		{
			//Check if we are setting an FBX import type override.  If we are, we don't want to call ConfigureProperties because that enables bDetectImportTypeOnImport
			if (FactorySettings[i].SettingName.Contains(TEXT("MeshTypeToImport")))
			{
				bCallConfigureProperties = false;
			}

			TArray<FString> PropertyChain;
			FactorySettings[i].SettingName.ParseIntoArray(PropertyChain, TEXT("."), false);
			ApplyCustomFactorySetting(InFactory, PropertyChain, FactorySettings[i].Value);
		}
	}

	if (bCallConfigureProperties)
	{
		InFactory->ConfigureProperties();
	}
}

/**
* Writes a number to a text file.
* @param InTestName is the folder that has the same name as the test. (For Example: "Performance").
* @param InItemBeingTested is the name for the thing that is being tested. (For Example: "MapName").
* @param InFileName is the name of the file with an extension
* @param InEntry is the double-precision number that is expected to be written to the file.
* @param Delimiter is the delimiter to be used. TEXT(",")
*/
void FAutomationEditorCommonUtils::WriteToTextFile(const FString& InTestName, const FString& InTestItem, const FString& InFileName, const double& InEntry, const FString& Delimiter)
{
	//Performance file locations and setups.
	FString FileSaveLocation = FPaths::Combine(*FPaths::AutomationLogDir(), *InTestName, *InTestItem, *InFileName);

	if (FPaths::FileExists(FileSaveLocation))
	{
		//The text files existing content.
		FString TextFileContents;

		//Write to the text file the combined contents from the text file with the number to write.
		FFileHelper::LoadFileToString(TextFileContents, *FileSaveLocation);
		FString FileSetup = TextFileContents + Delimiter + FString::SanitizeFloat(InEntry);
		FFileHelper::SaveStringToFile(FileSetup, *FileSaveLocation);
		return;
	}

	FFileHelper::SaveStringToFile(FString::SanitizeFloat(InEntry), *FileSaveLocation);
}

/**
* Returns the sum of the numbers available in an array of float.
* @param InFloatArray is the name of the array intended to be used.
* @param bisAveragedInstead will return the average of the available numbers instead of the sum.
*/
float FAutomationEditorCommonUtils::TotalFromFloatArray(const TArray<float>& InFloatArray, bool bIsAveragedInstead)
{
	//Total Value holds the sum of all the numbers available in the array.
	float TotalValue = 0;

	//Get the sum of the array.
	for (int32 I = 0; I < InFloatArray.Num(); ++I)
	{
		TotalValue += InFloatArray[I];
	}

	//If bAverageInstead equals true then only the average is returned.
	if (bIsAveragedInstead)
	{
		UE_LOGF(LogEditorAutomationTests, VeryVerbose, "Average value of the Array is %f", (TotalValue / InFloatArray.Num()));
		return (TotalValue / InFloatArray.Num());
	}

	UE_LOGF(LogEditorAutomationTests, VeryVerbose, "Total Value of the Array is %f", TotalValue);
	return TotalValue;
}

/**
* Returns the largest value from an array of float numbers.
* @param InFloatArray is the name of the array intended to be used.
*/
float FAutomationEditorCommonUtils::LargestValueInFloatArray(const TArray<float>& InFloatArray)
{
	//Total Value holds the sum of all the numbers available in the array.
	float LargestValue = 0;

	//Find the largest value
	for (int32 I = 0; I < InFloatArray.Num(); ++I)
	{
		if (LargestValue < InFloatArray[I])
		{
			LargestValue = InFloatArray[I];
		}
	}
	UE_LOGF(LogEditorAutomationTests, VeryVerbose, "The Largest value of the array is %f", LargestValue);
	return LargestValue;
}

/**
* Returns the contents of a text file as an array of FString.
* @param InFileLocation -  is the location of the file.
* @param OutArray - The name of the array where the 
*/
void FAutomationEditorCommonUtils::CreateArrayFromFile(const FString& InFileLocation, TArray<FString>& OutArray)
{
	FString RawData;

	if (FPaths::FileExists(InFileLocation))
	{
		UE_LOGF(LogEditorAutomationTests, VeryVerbose, "Loading and parsing the data from '%ls' into an array.", *InFileLocation);
		FFileHelper::LoadFileToString(RawData, *InFileLocation);
		RawData.ParseIntoArray(OutArray, TEXT(","), false);
	}

	UE_LOGF(LogEditorAutomationTests, Warning, "Unable to create an array.  '%ls' does not exist.", *InFileLocation);
	RawData = TEXT("0");
	OutArray.Add(RawData);
}

/**
* Returns true if the archive/file can be written to otherwise false..
* @param InFilePath - is the location of the file.
* @param InArchiveName - is the name of the archive to be used.
*/
bool FAutomationEditorCommonUtils::IsArchiveWriteable(const FString& InFilePath, const FArchive* InArchiveName)
{
	if (!InArchiveName)
	{
		UE_LOGF(LogEditorAutomationTests, Error, "Failed to write to the csv file: %ls", *FPaths::ConvertRelativePathToFull(InFilePath));
		return false;
	}
	return true;
}


void FAutomationEditorCommonUtils::GetLaunchOnDeviceID(FString& OutDeviceID, const FString& InMapName)
{
	UAutomationTestSettings const* AutomationTestSettings = GetDefault<UAutomationTestSettings>();
	check(AutomationTestSettings);

	OutDeviceID = "None";

	FString LaunchOnDeviceId;
	for (auto LaunchIter = AutomationTestSettings->LaunchOnSettings.CreateConstIterator(); LaunchIter; LaunchIter++)
	{
		FString LaunchOnSettings = LaunchIter->DeviceID;
		FString LaunchOnMap = FPaths::GetBaseFilename(LaunchIter->LaunchOnTestmap.FilePath);
		if (LaunchOnMap.Equals(InMapName))
		{
			// shared devices section
			ITargetDeviceServicesModule* TargetDeviceServicesModule = static_cast<ITargetDeviceServicesModule*>(FModuleManager::Get().LoadModule(TEXT("TargetDeviceServices")));
			// for each platform...
			TArray<TSharedPtr<ITargetDeviceProxy>> DeviceProxies;
			TargetDeviceServicesModule->GetDeviceProxyManager()->GetProxies(FName(*LaunchOnSettings), true, DeviceProxies);
			// for each proxy...
			for (auto DeviceProxyIt = DeviceProxies.CreateIterator(); DeviceProxyIt; ++DeviceProxyIt)
			{
				TSharedPtr<ITargetDeviceProxy> DeviceProxy = *DeviceProxyIt;
				if (DeviceProxy->IsConnected())
				{
					OutDeviceID = DeviceProxy->GetTargetDeviceId((FName)*LaunchOnSettings);
					break;
				}
			}
		}
	}
}

void FAutomationEditorCommonUtils::GetLaunchOnDeviceID(FString& OutDeviceID, const FString& InMapName, const FString& InDeviceName)
{
	UAutomationTestSettings const* AutomationTestSettings = GetDefault<UAutomationTestSettings>();
	check(AutomationTestSettings);

	//Output device name will default to "None".
	OutDeviceID = "None";

	// shared devices section
	ITargetDeviceServicesModule* TargetDeviceServicesModule = static_cast<ITargetDeviceServicesModule*>(FModuleManager::Get().LoadModule(TEXT("TargetDeviceServices")));
	// for each platform...
	TArray<TSharedPtr<ITargetDeviceProxy>> DeviceProxies;
	TargetDeviceServicesModule->GetDeviceProxyManager()->GetProxies(FName(*InDeviceName), true, DeviceProxies);
	// for each proxy...
	for (auto DeviceProxyIt = DeviceProxies.CreateIterator(); DeviceProxyIt; ++DeviceProxyIt)
	{
		TSharedPtr<ITargetDeviceProxy> DeviceProxy = *DeviceProxyIt;
		if (DeviceProxy->IsConnected())
		{
			OutDeviceID = DeviceProxy->GetTargetDeviceId((FName)*InDeviceName);
			break;
		}
	}
}

bool FAutomationEditorCommonUtils::SetOrthoViewportView(const FVector& ViewLocation, const FRotator& ViewRotation)
{
	for (FLevelEditorViewportClient* ViewportClient : GEditor->GetLevelViewportClients())
	{
		if (!ViewportClient->IsOrtho())
		{
			ViewportClient->SetViewLocation(ViewLocation);
			ViewportClient->SetViewRotation(ViewRotation);
			return true;
		}
	}

	UE_LOGF(LogEditorAutomationTests, Log, "An ortho viewport was not found.  May affect the test results.");
	return false;
}

bool FAutomationEditorCommonUtils::SetPlaySessionStartToActiveViewport(FRequestPlaySessionParams& OutParams)
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
	TSharedPtr<IAssetViewport> ActiveLevelViewport = LevelEditorModule.GetFirstActiveViewport();
	// Make sure we can find a path to the view port.
	if (ActiveLevelViewport.IsValid() &&
		FSlateApplication::Get().FindWidgetWindow(ActiveLevelViewport->AsWidget()).IsValid())
	{
		// Start the player where the camera is if not forcing from player start
		OutParams.StartLocation = ActiveLevelViewport->GetAssetViewportClient().GetViewLocation();
		OutParams.StartRotation = ActiveLevelViewport->GetAssetViewportClient().GetViewRotation();
		return true;
	}

	return false;
}

//////////////////////////////////////////////////////////////////////
//Asset Path Commands

/**
* Converts a package path to an asset path
*
* @param PackagePath - The package path to convert
*/
FString FAutomationEditorCommonUtils::ConvertPackagePathToAssetPath(const FString& PackagePath)
{
	const FString Filename = FPaths::ConvertRelativePathToFull(PackagePath);
	FString EngineFileName = Filename;
	FString GameFileName = Filename;
	FString ProjectPluginFileName = Filename;
	FString EnginePluginFileName = Filename;
	if (FPaths::MakePathRelativeTo(EngineFileName, *FPaths::EngineContentDir()) && !EngineFileName.Contains(TEXT("../")))
	{
		const FString ShortName = FPaths::GetBaseFilename(EngineFileName);
		const FString PathName = FPaths::GetPath(EngineFileName);
		const FString AssetName = FString::Printf(TEXT("/Engine/%s/%s.%s"), *PathName, *ShortName, *ShortName);
		return AssetName;
	}
	else if (FPaths::MakePathRelativeTo(GameFileName, *FPaths::ProjectContentDir()) && !GameFileName.Contains(TEXT("../")))
	{
		const FString ShortName = FPaths::GetBaseFilename(GameFileName);
		const FString PathName = FPaths::GetPath(GameFileName);
		const FString AssetName = FString::Printf(TEXT("/Game/%s/%s.%s"), *PathName, *ShortName, *ShortName);
		return AssetName;
	}
	else if (FPaths::MakePathRelativeTo(ProjectPluginFileName, *FPaths::ProjectPluginsDir()) && !ProjectPluginFileName.Contains(TEXT("../")))
	{
		const FString ShortName = FPaths::GetBaseFilename(ProjectPluginFileName);
		const FString FullPathName = FPaths::GetPath(ProjectPluginFileName);
		const FString CleanedPathName = FullPathName.Replace(TEXT("Content/"), TEXT(""));
		const FString AssetName = FString::Printf(TEXT("/%s/%s.%s"), *CleanedPathName, *ShortName, *ShortName);
		return AssetName;
	}
	else if (FPaths::MakePathRelativeTo(EnginePluginFileName, *FPaths::EnginePluginsDir()) && !EnginePluginFileName.Contains(TEXT("../")))
	{
		const FString ShortName = FPaths::GetBaseFilename(EnginePluginFileName);
		const FString FullPathName = FPaths::GetPath(EnginePluginFileName);
		const FString CleanedPathName = FullPathName.Replace(TEXT("Content/"), TEXT(""));
		const FString AssetName = FString::Printf(TEXT("/%s/%s.%s"), *CleanedPathName, *ShortName, *ShortName);
		return AssetName;
	}
	else
	{
		UE_LOGF(LogAutomationEditorCommon, Error, "PackagePath (%ls) is invalid for the current project", *PackagePath);
		return TEXT("");
	}
}

/**
* Gets the asset data from a package path
*
* @param PackagePath - The package path used to look up the asset data
*/
FAssetData FAutomationEditorCommonUtils::GetAssetDataFromPackagePath(const FString& PackagePath)
{
	FString AssetPath = FAutomationEditorCommonUtils::ConvertPackagePathToAssetPath(PackagePath);
	if (AssetPath.Len() > 0)
	{
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
		return AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(AssetPath));
	}

	return FAssetData();
}

//////////////////////////////////////////////////////////////////////
//Find Asset Commands

/**
* Generates a list of assets from the ENGINE and the GAME by a specific type.
* This is to be used by the GetTest() function.
*/
void FAutomationEditorCommonUtils::CollectTestsByClass(UClass * Class, TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	TArray<FAssetData> ObjectList;
	AssetRegistryModule.Get().GetAssetsByClass(Class->GetClassPathName(), ObjectList);

	for (TObjectIterator<UClass> AllClassesIt; AllClassesIt; ++AllClassesIt)
	{
		UClass* ClassList = *AllClassesIt;
		FName ClassName = ClassList->GetFName();
	}

	for (auto ObjIter = ObjectList.CreateConstIterator(); ObjIter; ++ObjIter)
	{
		const FAssetData& Asset = *ObjIter;
		FString Filename = Asset.GetObjectPathString();
		//convert to full paths
		Filename = FPackageName::LongPackageNameToFilename(Filename);
		if (FAutomationTestFramework::Get().ShouldTestContent(Filename))
		{
			FString BeautifiedFilename = Asset.AssetName.ToString();
			OutBeautifiedNames.Add(BeautifiedFilename);
			OutTestCommands.Add(Asset.GetObjectPathString());
		}
	}
}

/**
* Generates a list of assets from the GAME by a specific type.
* This is to be used by the GetTest() function.
*/
void FAutomationEditorCommonUtils::CollectGameContentTestsByClass(UClass * Class, bool bRecursiveClass, TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands)
{
	//Setting the Asset Registry
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	//Variable setups
	TArray<FAssetData> ObjectList;
	FARFilter AssetFilter;

	//Generating the list of assets.
	//This list is being filtered by the game folder and class type.  The results are placed into the ObjectList variable.
	AssetFilter.ClassPaths.Add(Class->GetClassPathName());

	//removed path as a filter as it causes two large lists to be sorted.  Filtering on "game" directory on iteration
	//AssetFilter.PackagePaths.Add("/Game");
	AssetFilter.bRecursiveClasses = bRecursiveClass;
	AssetFilter.bRecursivePaths = true;
	AssetRegistryModule.Get().GetAssets(AssetFilter, ObjectList);

	//Loop through the list of assets, make their path full and a string, then add them to the test.
	for (auto ObjIter = ObjectList.CreateConstIterator(); ObjIter; ++ObjIter)
	{
		const FAssetData& Asset = *ObjIter;
		FString Filename = Asset.GetObjectPathString();

		if (Filename.StartsWith("/Game"))
		{
			//convert to full paths
			Filename = FPackageName::LongPackageNameToFilename(Filename);
			if (FAutomationTestFramework::Get().ShouldTestContent(Filename))
			{
				FString BeautifiedFilename = Asset.AssetName.ToString();
				OutBeautifiedNames.Add(BeautifiedFilename);
				OutTestCommands.Add(Asset.GetObjectPathString());
			}
		}
	}
}

void FAutomationEditorCommonUtils::LoadMap(const FString& MapName)
{
	bool bLoadAsTemplate = false;
	bool bShowProgress = false;
	FEditorFileUtils::LoadMap(MapName, bLoadAsTemplate, bShowProgress);
}

void FAutomationEditorCommonUtils::RunPIE(float PIEDuration)
{
	bool bInSimulateInEditor = true;
	//once in the editor
	ADD_LATENT_AUTOMATION_COMMAND(FStartPIECommand(true));
	ADD_LATENT_AUTOMATION_COMMAND(FWaitLatentCommand(PIEDuration));
	ADD_LATENT_AUTOMATION_COMMAND(FEndPlayMapCommand());

	//wait between tests
	ADD_LATENT_AUTOMATION_COMMAND(FWaitLatentCommand(1.0f));

	//once not in the editor
	ADD_LATENT_AUTOMATION_COMMAND(FStartPIECommand(false));
	ADD_LATENT_AUTOMATION_COMMAND(FWaitLatentCommand(PIEDuration));
	ADD_LATENT_AUTOMATION_COMMAND(FEndPlayMapCommand());
}

/**
* Generates a list of assets from the GAME by a specific type.
* This is to be used by the GetTest() function.
*/
void FAutomationEditorCommonUtils::CollectGameContentTests(TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands)
{
	//Setting the Asset Registry
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	//Variable setups
	TArray<FAssetData> ObjectList;
	FARFilter AssetFilter;
	
	//removed path as a filter as it causes two large lists to be sorted.  Filtering on "game" directory on iteration
	AssetFilter.PackagePaths.Add("/Game");
	AssetFilter.bRecursiveClasses = true;
	AssetFilter.bRecursivePaths = true;
	AssetRegistryModule.Get().GetAssets(AssetFilter, ObjectList);

	//Loop through the list of assets, make their path full and a string, then add them to the test.
	for (auto ObjIter = ObjectList.CreateConstIterator(); ObjIter; ++ObjIter)
	{
		const FAssetData& Asset = *ObjIter;
		if (Asset.GetClass() == nullptr)
		{
			// a nullptr class is bad !
			UE_LOGF(LogAutomationEditorCommon, Warning, "GetClass for %ls (%ls) returned nullptr. Asset ignored", *Asset.AssetName.ToString(), *Asset.GetObjectPathString());
		}
		else 
		{
			FString Filename = Asset.GetObjectPathString();

			if (Filename.StartsWith("/Game"))
			{
				//convert to full paths
				Filename = FPackageName::LongPackageNameToFilename(Filename);
				if (FAutomationTestFramework::Get().ShouldTestContent(Filename))
				{
					FString BeautifiedFilename = FString::Printf(TEXT("%s.%s"), *Asset.AssetClassPath.ToString(), *Asset.AssetName.ToString());
					OutBeautifiedNames.Add(BeautifiedFilename);
					OutTestCommands.Add(Asset.GetObjectPathString());
				}
			}
		}
	}
}








///////////////////////////////////////////////////////////////////////
// Common Latent commands

//Latent Undo and Redo command
//If bUndo is true then the undo action will occur otherwise a redo will happen.
bool FUndoRedoCommand::Update()
{
	if ( bUndo == true )
	{
		//Undo
		GEditor->UndoTransaction();
	}
	else
	{
		//Redo
		GEditor->RedoTransaction();
	}

	return true;
}

/**
* Open editor for a particular asset
*/
bool FOpenEditorForAssetCommand::Update()
{
	UObject* Object = StaticLoadObject(UObject::StaticClass(), NULL, *AssetName);
	if ( Object )
	{
		// Some assets (like UWorlds) may be destroyed and recreated as part of opening. To protect against this, keep the path to the asset and try to re-find it if it disappeared.
		TWeakObjectPtr<UObject> WeakObject = Object;

		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Object);

		// If the object was destroyed, attempt to find it if it was recreated
		if (!WeakObject.IsValid() && !AssetName.IsEmpty())
		{
			Object = FindObject<UObject>(nullptr, *AssetName);
		}

		//This checks to see if the asset sub editor is loaded.
		if ( GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(Object, true) != NULL )
		{
			UE_LOGF(LogEditorAutomationTests, Log, "Verified asset editor for: %ls.", *AssetName);
			UE_LOGF(LogEditorAutomationTests, Display, "The editor successfully loaded for: %ls.", *AssetName);
			return true;
		}
	}
	else
	{
		UE_LOGF(LogEditorAutomationTests, Error, "Failed to find object: %ls.", *AssetName);
	}
	return true;
}

/**
* Close all sub-editors
*/
bool FCloseAllAssetEditorsCommand::Update()
{
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllAssetEditors();

	//Get all assets currently being tracked with open editors and make sure they are not still opened.
	if ( GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->GetAllEditedAssets().Num() >= 1 )
	{
		UE_LOGF(LogEditorAutomationTests, Warning, "Not all of the editors were closed.");
		return true;
	}

	UE_LOGF(LogEditorAutomationTests, Log, "Verified asset editors were closed");
	UE_LOGF(LogEditorAutomationTests, Display, "The asset editors closed successfully");
	return true;
}

/**
* Start PIE session
*/
bool FStartPIECommand::Update()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::Get().GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));

	FRequestPlaySessionParams Params;
	Params.DestinationSlateViewport = LevelEditorModule.GetFirstActiveViewport();
	if (bSimulateInEditor)
	{
		Params.WorldType = EPlaySessionWorldType::SimulateInEditor;
	}

	// Make sure the player start location is a valid location.
	if (GUnrealEd->CheckForPlayerStart() == nullptr)
	{
		FAutomationEditorCommonUtils::SetPlaySessionStartToActiveViewport(Params);
	}

	GUnrealEd->RequestPlaySession(Params);
	return true;
}

/**
* End PlayMap session
*/
bool FEndPlayMapCommand::Update()
{
	GUnrealEd->RequestEndPlayMap();
	return true;
}

/**
* This this command loads a map into the editor.
*/
bool FEditorLoadMap::Update()
{
	//Get the base filename for the map that will be used.
	FString ShortMapName = FPaths::GetBaseFilename(MapName);

	//Get the current number of seconds before loading the map.
	double MapLoadStartTime = FPlatformTime::Seconds();

	//Load the map
	FAutomationEditorCommonUtils::LoadMap(MapName);

	//This is the time it took to load the map in the editor.
	double MapLoadTime = FPlatformTime::Seconds() - MapLoadStartTime;

	//Gets the main frame module to get the name of our current level.
	const IMainFrameModule& MainFrameModule = FModuleManager::GetModuleChecked< IMainFrameModule >("MainFrame");
	FString LoadedMapName = MainFrameModule.GetLoadedLevelName();

	UE_LOGF(LogEditorAutomationTests, Log, "%ls has been loaded.", *ShortMapName);

	//Log out to a text file the time it takes to load the map.
	FAutomationEditorCommonUtils::WriteToTextFile(TEXT("Performance"), LoadedMapName, TEXT("RAWMapLoadTime.txt"), MapLoadTime, TEXT(","));

	UE_LOGF(LogEditorAutomationTests, Display, "%ls took %.3f to load.", *LoadedMapName, MapLoadTime);

	return true;
}

/**
* This will cause the test to wait for the shaders to finish compiling before moving on.
*/
bool FWaitForShadersToFinishCompiling::Update()
{
	static double TimeShadersFinishedCompiling = 0;
	static double LastReportTime = FPlatformTime::Seconds();
	const double TimeToWaitForJobs = 2.0;

	bool ShadersCompiling = GShaderCompilingManager && GShaderCompilingManager->IsCompiling();
	bool TexturesCompiling = FTextureCompilingManager::Get().GetNumRemainingTextures() > 0;

	double TimeNow = FPlatformTime::Seconds();

	if (ShadersCompiling || TexturesCompiling)
	{
		if (TimeNow - LastReportTime > 5.0)
		{
			LastReportTime = TimeNow;

			if (ShadersCompiling)
			{
				UE_LOGF(LogEditorAutomationTests, Log, "Waiting for %i shaders to finish.", GShaderCompilingManager->GetNumRemainingJobs() + GShaderCompilingManager->GetNumPendingJobs());
			}

			if (TexturesCompiling)
			{
				UE_LOGF(LogEditorAutomationTests, Log, "Waiting for %i texures to finish.", FTextureCompilingManager::Get().GetNumRemainingTextures());
			}
		}

		TimeShadersFinishedCompiling = 0;

		return false;
	}

	// Current jobs are done, but things may still come in on subsequent frames..
	if (TimeShadersFinishedCompiling == 0)
	{
		TimeShadersFinishedCompiling = FPlatformTime::Seconds();
	}

	if (FPlatformTime::Seconds() - TimeShadersFinishedCompiling < TimeToWaitForJobs)
	{
		return false;
	}

	// may not be necessary, but just double-check everything is finished and ready
	GShaderCompilingManager->FinishAllCompilation();
	UE_LOGF(LogEditorAutomationTests, Log, "Done waiting for shaders to finish.");
	return true;
}

/**
* Latent command that changes the editor viewport to the first available bookmarked view.
*/
bool FChangeViewportToFirstAvailableBookmarkCommand::Update()
{
	uint32 ViewportIndex = 0;

	UE_LOGF(LogEditorAutomationTests, Log, "Attempting to change the editor viewports view to the first set bookmark.");

	//Move the perspective viewport view to show the test.
	for (FLevelEditorViewportClient* ViewportClient : GEditor->GetLevelViewportClients())
	{
		const uint32 NumberOfBookmarks = IBookmarkTypeTools::Get().GetMaxNumberOfBookmarks(ViewportClient);
		for ( ViewportIndex = 0; ViewportIndex <= NumberOfBookmarks; ViewportIndex++ )
		{
			if (IBookmarkTypeTools::Get().CheckBookmark(ViewportIndex, ViewportClient) )
			{
				UE_LOGF(LogEditorAutomationTests, VeryVerbose, "Changing a viewport view to the set bookmark %i", ViewportIndex);
				IBookmarkTypeTools::Get().JumpToBookmark(ViewportIndex, TSharedPtr<struct FBookmarkBaseJumpToSettings>(), ViewportClient);
				break;
			}
		}
	}
	return true;
}

/**
* Latent command that adds a static mesh to the worlds origin.
*/
bool FAddStaticMeshCommand::Update()
{
	//Gather assets.
	UObject* Cube = (UStaticMesh*)StaticLoadObject(UStaticMesh::StaticClass(), NULL, TEXT("/Engine/EngineMeshes/Cube.Cube"), NULL, LOAD_None, NULL);
	//Add Cube mesh to the world
	AStaticMeshActor* StaticMesh = Cast<AStaticMeshActor>(FActorFactoryAssetProxy::AddActorForAsset(Cube));
	StaticMesh->TeleportTo(FVector(0.0f, 0.0f, 0.0f), FRotator(0, 0, 0));
	StaticMesh->SetActorRelativeScale3D(FVector(1.0f, 1.0f, 1.0f));

	UE_LOGF(LogEditorAutomationTests, Log, "Static Mesh cube has been added to 0, 0, 0.")

		return true;
}

/**
* Latent command that builds lighting for the current level.
*/
bool FBuildLightingCommand::Update()
{
	//If we are running with -NullRHI then we have to skip this step.
	if ( GUsingNullRHI )
	{
		UE_LOGF(LogEditorAutomationTests, Log, "SKIPPED Build Lighting Step.  You're currently running with -NullRHI.");
		return true;
	}

	if ( GUnrealEd->WarnIfLightingBuildIsCurrentlyRunning() )
	{
		UE_LOGF(LogEditorAutomationTests, Warning, "Lighting is already being built.");
		return true;
	}

	UWorld* CurrentWorld = GEditor->GetEditorWorldContext().World();
	GUnrealEd->Exec(CurrentWorld, TEXT("MAP REBUILD"));

	FLightingBuildOptions LightingBuildOptions;

	// Retrieve settings from ini.
	GConfig->GetBool(TEXT("LightingBuildOptions"), TEXT("OnlyBuildSelected"), LightingBuildOptions.bOnlyBuildSelected, GEditorPerProjectIni);
	GConfig->GetBool(TEXT("LightingBuildOptions"), TEXT("OnlyBuildCurrentLevel"), LightingBuildOptions.bOnlyBuildCurrentLevel, GEditorPerProjectIni);
	GConfig->GetBool(TEXT("LightingBuildOptions"), TEXT("OnlyBuildSelectedLevels"), LightingBuildOptions.bOnlyBuildSelectedLevels, GEditorPerProjectIni);
	GConfig->GetBool(TEXT("LightingBuildOptions"), TEXT("OnlyBuildVisibility"), LightingBuildOptions.bOnlyBuildVisibility, GEditorPerProjectIni);
	GConfig->GetBool(TEXT("LightingBuildOptions"), TEXT("UseErrorColoring"), LightingBuildOptions.bUseErrorColoring, GEditorPerProjectIni);
	GConfig->GetBool(TEXT("LightingBuildOptions"), TEXT("ShowLightingBuildInfo"), LightingBuildOptions.bShowLightingBuildInfo, GEditorPerProjectIni);
	int32 QualityLevel;
	GConfig->GetInt(TEXT("LightingBuildOptions"), TEXT("QualityLevel"), QualityLevel, GEditorPerProjectIni);
	QualityLevel = FMath::Clamp<int32>(QualityLevel, Quality_Preview, Quality_Production);
	LightingBuildOptions.QualityLevel = Quality_Production;

	UE_LOGF(LogEditorAutomationTests, Log, "Building lighting in Production Quality.");
	GUnrealEd->BuildLighting(LightingBuildOptions);

	return true;
}


bool FSaveLevelCommand::Update()
{
	if ( !GUnrealEd->IsLightingBuildCurrentlyExporting() && !GUnrealEd->IsLightingBuildCurrentlyRunning() )
	{
		UWorld* World = GEditor->GetEditorWorldContext().World();
		ULevel* Level = World->GetCurrentLevel();
		MapName += TEXT("_Copy.umap");
		FString TempMapLocation = FPaths::Combine(*FPaths::ProjectContentDir(), TEXT("Maps"), TEXT("Automation_TEMP"), *MapName);
		FEditorFileUtils::SaveLevel(Level, TempMapLocation);

		return true;
	}
	return false;
}

bool FLaunchOnCommand::Update()
{
	FRequestPlaySessionParams::FLauncherDeviceInfo LaunchedDeviceInfo;
	LaunchedDeviceInfo.DeviceId = InLauncherDeviceID;
	LaunchedDeviceInfo.DeviceName = InLauncherDeviceID.Right(InLauncherDeviceID.Find(TEXT("@")));

	FRequestPlaySessionParams Params;
	Params.LauncherTargetDevice = LaunchedDeviceInfo;

	GUnrealEd->RequestPlaySession(Params);

	// Immediately start our requested play session
	GUnrealEd->StartQueuedPlaySessionRequest();

	return true;
}

bool FWaitToFinishCookByTheBookCommand::Update()
{
	if ( !GUnrealEd->CookServer->IsCookByTheBookRunning() )
	{
		if ( GUnrealEd->IsCookByTheBookInEditorFinished() )
		{
			UE_LOGF(LogEditorAutomationTests, Log, "The cook by the book operation has finished.");
		}
		return true;
	}
	else if ( ( FPlatformTime::Seconds() - StartTime ) == COOK_TIMEOUT )
	{
		GUnrealEd->CancelCookByTheBookInEditor();
		UE_LOGF(LogEditorAutomationTests, Error, "It has been an hour or more since the cook has started.");
		return false;
	}
	return false;
}

bool FDeleteDirCommand::Update()
{
FString FullFolderPath = FPaths::ConvertRelativePathToFull(*InFolderLocation);
if (IFileManager::Get().DirectoryExists(*FullFolderPath))
{
	IFileManager::Get().DeleteDirectory(*FullFolderPath, false, true);
}
return true;
}

bool FWaitToFinishBuildDeployCommand::Update()
{
	if (GEditor->LauncherWorker->GetStatus() == ELauncherWorkerStatus::Completed)
	{
		UE_LOGF(LogEditorAutomationTests, Log, "The build game and deploy operation has finished.");
		return true;
	}
	else if (GEditor->LauncherWorker->GetStatus() == ELauncherWorkerStatus::Canceled || GEditor->LauncherWorker->GetStatus() == ELauncherWorkerStatus::Canceling)
	{
		UE_LOGF(LogEditorAutomationTests, Warning, "The build was canceled.");
		return true;
	}
	return false;
}

bool FWaitForEditorNavMeshBuildCommand::Update()
{
	// Timeout to avoid hanging forever if the nav build never completes (e.g. build system stuck).
	// Configurable via UAutomationTestSettings::NavMeshBuildTimeoutSeconds in DefaultEngine.ini.
	const float ConfiguredTimeout = GetDefault<UAutomationTestSettings>()->NavMeshBuildTimeoutSeconds;
	const float NavMeshBuildTimeoutSeconds = FMath::Max(ConfiguredTimeout, 1.0f);
	if (ConfiguredTimeout < 1.0f)
	{
		static bool bWarnedAboutNavMeshTimeout = false;
		if (!bWarnedAboutNavMeshTimeout)
		{
			bWarnedAboutNavMeshTimeout = true;
			UE_LOGF(LogAutomationEditorCommon, Warning, "FWaitForEditorNavMeshBuildCommand: NavMeshBuildTimeoutSeconds is %.1f, which is invalid. Using %.1f instead.", ConfiguredTimeout, NavMeshBuildTimeoutSeconds);
		}
	}
	if (GetCurrentRunTime() > NavMeshBuildTimeoutSeconds)
	{
		UE_LOGF(LogAutomationEditorCommon, Warning, "FWaitForEditorNavMeshBuildCommand timed out after %.0f seconds waiting for editor navmesh build to complete.", NavMeshBuildTimeoutSeconds);
		return true;
	}

	for (const FWorldContext& Context : GEngine->GetWorldContexts())
	{
		if (Context.WorldType == EWorldType::Editor && Context.World())
		{
			UNavigationSystemV1* NavigationSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(Context.World());
			if (NavigationSys)
			{
				// Returns true immediately if nav is already clean — zero cost for maps where the navmesh
				// is pre-built or where the test doesn't care about navigation.
				return !NavigationSys->IsNavigationBuildInProgress() && !NavigationSys->HasDirtyAreasQueued();
			}
			break;
		}
	}

	return true; // No editor world or no nav system — returns immediately on first tick, zero cost for maps that don't use navigation.
}

FStartPIEForAutomationCommand::FStartPIEForAutomationCommand(FRequestPlaySessionParams InRequestParams)
	: RequestParams(InRequestParams)
	, bPIEStarted(false)
	, bPIEFailed(false)
	, bSessionRequested(false)
{
	FEditorDelegates::PostPIEStarted.AddRaw(this, &FStartPIEForAutomationCommand::OnPostPIEStarted);
	FEditorDelegates::EndPIE.AddRaw(this, &FStartPIEForAutomationCommand::OnEndPIE);
}

FStartPIEForAutomationCommand::~FStartPIEForAutomationCommand()
{
	FEditorDelegates::PostPIEStarted.RemoveAll(this);
	FEditorDelegates::EndPIE.RemoveAll(this);
	// Balance the AddToRoot() called in AutomationLoadMap. The destructor always runs regardless
	// of whether Update() was ever called, so this covers test abort and framework teardown paths.
	// The null check guards against the unlikely case where EditorPlaySettings was never set.
	if (ULevelEditorPlaySettings* EditorPlaySettings = RequestParams.EditorPlaySettings.Get())
	{
		EditorPlaySettings->RemoveFromRoot();
	}
}

bool FStartPIEForAutomationCommand::Update()
{
	// First tick: fire the PIE request
	if (!bSessionRequested)
	{
		bSessionRequested = true;
		GEditor->RequestPlaySession(RequestParams);
		GEditor->StartQueuedPlaySessionRequest();
		return false; // keep polling
	}

	const float ConfiguredPIETimeout = GetDefault<UAutomationTestSettings>()->PIEStartTimeoutSeconds;
	const float TimeoutSeconds = FMath::Max(ConfiguredPIETimeout, 1.0f);
	if (ConfiguredPIETimeout < 1.0f)
	{
		static bool bWarnedAboutPIETimeout = false;
		if (!bWarnedAboutPIETimeout)
		{
			bWarnedAboutPIETimeout = true;
			UE_LOGF(LogAutomationEditorCommon, Warning, "FStartPIEForAutomationCommand: PIEStartTimeoutSeconds is %.1f, which is invalid. Using %.1f instead.", ConfiguredPIETimeout, TimeoutSeconds);
		}
	}

	// PIE started successfully — wait for the world to be ready.
	// Matches the condition used by FWaitForMapToLoadCommand. HasMatchStarted() checks World->bMatchStarted,
	// which is set by AGameModeBase::StartPlay() -> GameState->HandleBeginPlay() -> NotifyMatchStarted()
	// for all AGameModeBase-derived game modes, including AFunctionalTestGameMode.
	if (bPIEStarted)
	{
		// PIE ended after starting — covers mid-initialization crashes (e.g. Blueprint load failure)
		if (bPIEFailed)
		{
			if (FAutomationTestBase* CurrentTest = FAutomationTestFramework::Get().GetCurrentTest())
			{
				CurrentTest->AddError(TEXT("FStartPIEForAutomationCommand: PIE ended before the world became ready."));
			}
			else
			{
				UE_LOGF(LogAutomationEditorCommon, Error, "FStartPIEForAutomationCommand: PIE ended before the world became ready.");
			}
			return true;
		}

		UWorld* GameWorld = AutomationCommon::GetAnyGameWorld();
		if (GameWorld && GameWorld->AreActorsInitialized())
		{
			AGameStateBase* GameState = GameWorld->GetGameState();
			if (GameState && GameState->HasMatchStarted())
			{
				return true; // ready
			}
		}

		// Timeout guards against the world stalling after PIE has started
		if (GetCurrentRunTime() > TimeoutSeconds)
		{
			const FText ErrorMsg = FText::Format(
				FText::FromString(TEXT("FStartPIEForAutomationCommand: world did not become ready after {0} seconds (PIEStartTimeoutSeconds = {1}).")),
				FText::AsNumber(FMath::RoundToInt(GetCurrentRunTime())),
				FText::AsNumber(TimeoutSeconds));
			if (FAutomationTestBase* CurrentTest = FAutomationTestFramework::Get().GetCurrentTest())
			{
				CurrentTest->AddError(ErrorMsg.ToString());
			}
			else
			{
				UE_LOGF(LogAutomationEditorCommon, Error, "%ls", *ErrorMsg.ToString());
			}
			return true;
		}

		return false; // PIE is up but world not ready yet, keep waiting
	}

	// PIE failed before it started (EndPIE fired before PostPIEStarted)
	if (bPIEFailed)
	{
		if (FAutomationTestBase* CurrentTest = FAutomationTestFramework::Get().GetCurrentTest())
		{
			CurrentTest->AddError(TEXT("FStartPIEForAutomationCommand: PIE failed to start (Blueprint compilation error or other startup failure)."));
		}
		else
		{
			UE_LOGF(LogAutomationEditorCommon, Error, "FStartPIEForAutomationCommand: PIE failed to start (Blueprint compilation error or other startup failure).");
		}
		return true;
	}

	// Timeout — guards against the session request silently failing before either delegate fires.
	if (GetCurrentRunTime() > TimeoutSeconds)
	{
		const FText ErrorMsg = FText::Format(
			FText::FromString(TEXT("FStartPIEForAutomationCommand: PIE did not start after {0} seconds (PIEStartTimeoutSeconds = {1}).")),
			FText::AsNumber(FMath::RoundToInt(GetCurrentRunTime())),
			FText::AsNumber(TimeoutSeconds));
		if (FAutomationTestBase* CurrentTest = FAutomationTestFramework::Get().GetCurrentTest())
		{
			CurrentTest->AddError(ErrorMsg.ToString());
		}
		else
		{
			UE_LOGF(LogAutomationEditorCommon, Error, "%ls", *ErrorMsg.ToString());
		}
		return true;
	}

	return false; // still waiting for PIE to come up or fail
}

void FStartPIEForAutomationCommand::OnPostPIEStarted(const bool bInSimulateInEditor)
{
	bPIEStarted = true;
}

void FStartPIEForAutomationCommand::OnEndPIE(const bool bInSimulateInEditor)
{
	// Set bPIEFailed unconditionally — covers both "PIE never started" and "PIE started but
	// ended before the world became ready" (e.g. mid-initialization crash).
	bPIEFailed = true;
}

// agrant-todo: Expose the version in AutomationCommon.cpp for 4.27
namespace EditorAutomationPrivate
{
	// @todo this is a temporary solution. Once we know how to get test's hands on a proper world
	// this function should be redone/removed
	UWorld* GetAnyGameWorld()
	{
		UWorld* TestWorld = nullptr;
		const TIndirectArray<FWorldContext>& WorldContexts = GEngine->GetWorldContexts();
		for (const FWorldContext& Context : WorldContexts)
		{
			if (((Context.WorldType == EWorldType::PIE) || (Context.WorldType == EWorldType::Game)) && (Context.World() != NULL))
			{
				TestWorld = Context.World();
				break;
			}
		}

		return TestWorld;
	}
}


// agrant-todo: Use the standard version in AutomationCommon.cpp for 4.27
DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FWaitForSpecifiedPIEMapToEndCommand, FString, MapName);

bool FWaitForSpecifiedPIEMapToEndCommand::Update()
{
	UWorld* TestWorld = EditorAutomationPrivate::GetAnyGameWorld();

	if (!TestWorld)
	{
		return true;
	}

	// remove any paths or extensions to match the name of the world
	FString ShortMapName = FPackageName::GetShortName(MapName);
	ShortMapName = FPaths::GetBaseFilename(ShortMapName);

	// Handle both ways the user may have specified this
	if (TestWorld->GetName() != ShortMapName)
	{
		return true;
	}

	return false;
}

// agrant-todo: Move this into BasicTests.cpp for 4.27
/**
 * Generic Pie Test for projects.
 * By default this test will PIE the lit of MapsToPIETest from automation settings. if that is empty it will PIE the default editor and game (if they're different)
 * maps.
 *
 * If the editor session was started with a map on the command line then that's the only map that will be PIE'd. This allows project to set up tests that PIE
 * a list of maps from an external source.
 */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FProjectMapsPIETest, "Project.Maps.PIE", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ProductFilter)

/**
 * Execute the loading of one map to verify PIE works
 *
 * @param Parameters - Unused for this test
 * @return	TRUE if the test was successful, FALSE otherwise
 */
	bool FProjectMapsPIETest::RunTest(const FString& Parameters)
{
	UAutomationTestSettings const* AutomationTestSettings = GetDefault<UAutomationTestSettings>();
	check(AutomationTestSettings);

	TArray<FString> PIEMaps;

	// If the user has specified a map on the command line then that is what we'll PIE

	const TCHAR* ParsedCmdLine = FCommandLine::Get();
	FString ParsedMapName;
	bool FirstMapAlreadyLoaded = false;

	// If there is an explicit list of maps on the command line via -map or -maps the use those.
	if (FParse::Value(FCommandLine::Get(), TEXT("-maps="), ParsedMapName) || FParse::Value(FCommandLine::Get(), TEXT("-map="), ParsedMapName))
	{
		ParsedMapName.ParseIntoArray(PIEMaps, TEXT("+"), true);

		UE_LOGF(LogEditorAutomationTests, Display, "Found Maps %ls on command line. PIE Test will use these maps", *ParsedMapName);
	}
	else if (FParse::Token(ParsedCmdLine, ParsedMapName, false) && ParsedMapName.StartsWith(TEXT("-")) == false)
	{
		// If the user specified a map as the first param after the project, we'll PIE that
		FString InitialMapName;

		// If the specified package exists
		if (FPackageName::SearchForPackageOnDisk(ParsedMapName, NULL, &InitialMapName) &&
			// and it's a valid map file
			FPaths::GetExtension(InitialMapName, /*bIncludeDot=*/true) == FPackageName::GetMapPackageExtension())
		{
			PIEMaps.Add(InitialMapName);
			FirstMapAlreadyLoaded = true;
			UE_LOGF(LogEditorAutomationTests, Display, "Found Map %ls on command line. PIE Test will be restricted to this map", *InitialMapName);
		}
	}

	// Ok, at this point there were no command line maps so default to the project settings. We PIE the editor startup map and the game startup map
	if (PIEMaps.Num() == 0)
	{
		// If the project has maps configured for PIE then use those
		if (AutomationTestSettings->MapsToPIETest.Num())
		{
			for (const FString& Map : AutomationTestSettings->MapsToPIETest)
			{
				PIEMaps.Add(Map);
			}
		}
		else
		{
			// Else pick the editor startup and game startup maps (if they are different).
			UE_LOGF(LogEditorAutomationTests, Display, "No MapsToPIE or MapsToTest specified in DefaultEngine.ini [/Script/Engine.AutomationTestSettings]. Using GameStartup or EditorStartup Map");

			UGameMapsSettings const* MapSettings = GetDefault<UGameMapsSettings>();

			if (MapSettings->EditorStartupMap.IsValid())
			{
				PIEMaps.Add(MapSettings->EditorStartupMap.GetLongPackageName());
			}

			if (MapSettings->GetGameDefaultMap().Len() && MapSettings->GetGameDefaultMap() != MapSettings->EditorStartupMap.GetLongPackageName())
			{
				PIEMaps.Add(MapSettings->GetGameDefaultMap());
			}
		}
	}

	// Uh-oh
	if (PIEMaps.Num() == 0)
	{
		UE_LOGF(LogEditorAutomationTests, Fatal, "No automation or default maps are configured for PIE!");
	}

	// Don't want these settings affecting metrics
	UEditorPerformanceSettings* Settings = GetMutableDefault<UEditorPerformanceSettings>();
	Settings->bThrottleCPUWhenNotForeground = false;
	Settings->bMonitorEditorPerformance = false;
	Settings->PostEditChange();
	
	for (const FString& Map : PIEMaps)
	{
		// Accept any of...
		// - MyMap
		// - /Game/MyMap
		// - /Game/MyMap.MyMap
		FString MapPackageName = Map;

		if (FPackageName::IsValidObjectPath(Map))
		{
			MapPackageName = FPackageName::ObjectPathToPackageName(Map);
		}

		if (!FPackageName::SearchForPackageOnDisk(Map, NULL, &MapPackageName))
		{
			UE_LOGF(LogEditorAutomationTests, Error, "Couldn't resolve map for PIE test from %ls to valid package name!", *MapPackageName);
			continue;
		}

		UE_LOGF(LogEditorAutomationTests, Display, "Queueing Map %ls for PIE Automation", *MapPackageName);
		
		AddCommand(new FEditorAutomationLogCommand(FString::Printf(TEXT("LoadMap-Begin: %s"), *MapPackageName)));
		if (!FirstMapAlreadyLoaded)
		{
			AddCommand(new FEditorLoadMap(MapPackageName));
		}
		AddCommand(new FWaitLatentCommand(1.0f));
		AddCommand(new FEditorAutomationLogCommand(FString::Printf(TEXT("LoadMap-End: %s"), *MapPackageName)));
		AddCommand(new FEditorAutomationLogCommand(FString::Printf(TEXT("PIE-Begin: %s"), *MapPackageName)));
		AddCommand(new FStartPIECommand(false));
		AddCommand(new FWaitForSpecifiedMapToLoadCommand(MapPackageName));  // need at least some frames before starting & ending PIE
		AddCommand(new FWaitForInteractiveFrameRate());	// wait until the editor reaches something vaguely usable
		AddCommand(new FWaitLatentCommand(AutomationTestSettings->PIETestDuration));
		AddCommand(new FEndPlayMapCommand());
		AddCommand(new FWaitForSpecifiedPIEMapToEndCommand(MapPackageName));  // need at least some frames before starting & ending PIE
		AddCommand(new FEditorAutomationLogCommand(FString::Printf(TEXT("PIE-End: %s"), *MapPackageName)));

		FirstMapAlreadyLoaded = false;
	}

	return true;
}

#endif
