// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/AutomationEditorPromotionCommon.h"
#include "Misc/AutomationTest.h"
#include "Modules/ModuleManager.h"
#include "UObject/UnrealType.h"
#include "UObject/PropertyPortFlags.h"
#include "Input/Events.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/InputBindingManager.h"
#include "Materials/Material.h"
#include "Editor/UnrealEdEngine.h"
#include "Factories/MaterialFactoryNew.h"
#include "UnrealEdGlobals.h"
#include "Tests/AutomationCommon.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/Texture.h"
#include "LevelEditor.h"
#include "ScopedTransaction.h"
#include "Interfaces/IMainFrameModule.h"

#if WITH_AUTOMATION_TESTS

#define LOCTEXT_NAMESPACE "EditorPromotionTestCommon"

DEFINE_LOG_CATEGORY_STATIC(LogEditorPromotionTests, Log, All);

/**
* Finds a visible widget by type.     SLOW!!!!!
*
* @param InParent - We search this widget and its children for a matching widget (recursive)
* @param InWidgetType - The widget type we are searching for
*/
TSharedPtr<SWidget> FEditorPromotionTestUtilities::FindFirstWidgetByClass(TSharedRef<SWidget> InParent, const FName& InWidgetType)
{
	if (InParent->GetType() == InWidgetType)
	{
		return InParent;
	}
	FChildren* Children = InParent->GetChildren();
	for (int32 i = 0; i < Children->Num(); ++i)
	{
		TSharedRef<SWidget> ChildWidget = Children->GetChildAt(i);
		TSharedPtr<SWidget> FoundWidget = FindFirstWidgetByClass(ChildWidget, InWidgetType);
		if (FoundWidget.IsValid())
		{
			return FoundWidget;
		}
	}
	return NULL;
}

/**
* Gets the base path for this asset
*/
FString FEditorPromotionTestUtilities::GetGamePath()
{
	return TEXT("/Game/BuildPromotionTest");
}

/**
* Creates a material from an existing texture
*
* @param InTexture - The texture to use as the diffuse for the new material
*/
UMaterial* FEditorPromotionTestUtilities::CreateMaterialFromTexture(UTexture* InTexture)
{
	// Create the factory used to generate the asset
	UMaterialFactoryNew* Factory = NewObject<UMaterialFactoryNew>();
	Factory->InitialTexture = InTexture;


	const FString AssetName = FString::Printf(TEXT("%s_Mat"), *InTexture->GetName());
	const FString PackageName = FEditorPromotionTestUtilities::GetGamePath() + TEXT("/") + AssetName;
	UPackage* AssetPackage = CreatePackage( *PackageName);
	EObjectFlags Flags = RF_Public | RF_Standalone;

	UObject* CreatedAsset = Factory->FactoryCreateNew(UMaterial::StaticClass(), AssetPackage, FName(*AssetName), Flags, NULL, GWarn);

	if (CreatedAsset)
	{
		// Notify the asset registry
		FAssetRegistryModule::AssetCreated(CreatedAsset);

		// Mark the package dirty...
		AssetPackage->MarkPackageDirty();
	}

	return Cast<UMaterial>(CreatedAsset);
}

/**
* Sets an editor keyboard shortcut
*
* @param CommandContext - The context of the command
* @param Command - The command name to set
* @param NewChord - The new input chord to assign
*/
bool FEditorPromotionTestUtilities::SetEditorKeybinding(const FString& CommandContext, const FString& Command, const FInputChord& NewChord, const FInputChord& NewAlternateChord)
{
	TSharedPtr<FUICommandInfo> UICommand = FInputBindingManager::Get().FindCommandInContext(*CommandContext, *Command);
	if (UICommand.IsValid())
	{
		UICommand->SetActiveChord(NewChord, EMultipleKeyBindingIndex::Primary);
		UICommand->SetActiveChord(NewAlternateChord, EMultipleKeyBindingIndex::Secondary);
		FInputBindingManager::Get().SaveInputBindings();
		return true;
	}
	else
	{
		return false;
	}
}

/**
* Gets an editor keyboard shortcut
*
* @param CommandContext - The context of the command
* @param Command - The command name to get
*/
FInputChord FEditorPromotionTestUtilities::GetEditorKeybinding(const FString& CommandContext, const FString& Command)
{
	FInputChord Keybinding;
	TSharedPtr<FUICommandInfo> UICommand = FInputBindingManager::Get().FindCommandInContext(*CommandContext, *Command);
	if (UICommand.IsValid())
	{
		Keybinding = UICommand->GetFirstValidChord().Get();
	}
	return Keybinding;
}

/**
* Gets the current input chord or sets a new one if it doesn't exist
*
* @param Context - The context of the UI Command
* @param Command - The name of the UI command
*/
FInputChord FEditorPromotionTestUtilities::GetOrSetUICommand(const FString& Context, const FString& Command)
{
	FInputChord CurrentChord = GetEditorKeybinding(Context, Command);

	//If there is no current keybinding, set one
	if (!CurrentChord.Key.IsValid())
	{
		FInputChord NewChord(EKeys::J, EModifierKey::Control);
		SetEditorKeybinding(Context, Command, NewChord);
		CurrentChord = NewChord;
	}

	return CurrentChord;
}

/**
* Sends a UI command to the active top level window after focusing on a widget of a given type
*
* @param InChord - The chord to send to the window
* @param WidgetTypeToFocus - The widget type to find and focus on
*/
void FEditorPromotionTestUtilities::SendCommandToCurrentEditor(const FInputChord& InChord, const FName& WidgetTypeToFocus)
{
	IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
	TSharedPtr<SWindow>	ParentWindow = MainFrame.GetParentWindow();
	if (ParentWindow.IsValid() == false)
	{
		UE_LOGF(LogEditorPromotionTests, Error, "Failed to find main editor window to send commands to.");
		return;
	}

	FSlateApplication::Get().ProcessWindowActivatedEvent(FWindowActivateEvent(FWindowActivateEvent::EA_Activate, ParentWindow.ToSharedRef()));

	TSharedPtr<SWidget> FocusWidget = FindFirstWidgetByClass(ParentWindow.ToSharedRef(), WidgetTypeToFocus);
	if (FocusWidget.IsValid())
	{
		FSlateApplication::Get().SetKeyboardFocus(FocusWidget.ToSharedRef(), EFocusCause::SetDirectly);

		//Send the command
		FModifierKeysState ModifierKeys(InChord.NeedsShift(), false, InChord.NeedsControl(), false, InChord.NeedsAlt(), false, InChord.NeedsCommand(), false, false);
		FKeyEvent KeyEvent(InChord.Key, ModifierKeys, 0/*UserIndex*/, false, 0, 0);
		FSlateApplication::Get().ProcessKeyDownEvent(KeyEvent);
		FSlateApplication::Get().ProcessKeyUpEvent(KeyEvent);
	}
	else
	{
		//UE_LOGF(LogParticleEditorPromotionTests, Error, "Could not find widget %ls to send UI command", *WidgetTypeToFocus.ToString());
	}
}

/**
* Gets an object property value by name
*
* @param TargetObject - The object to modify
* @param InVariableName - The name of the property
*/
FString FEditorPromotionTestUtilities::GetPropertyByName(UObject* TargetObject, const FString& InVariableName)
{
	FProperty* FoundProperty = FindFProperty<FProperty>(TargetObject->GetClass(), *InVariableName);
	if (FoundProperty)
	{
		FString ValueString;
		FoundProperty->ExportTextItem_InContainer(ValueString, TargetObject, NULL, NULL, PPF_None);
		return ValueString;
	}
	return TEXT("");
}


/**
* Sets an object property value by name
*
* @param TargetObject - The object to modify
* @param InVariableName - The name of the property
*/
void FEditorPromotionTestUtilities::SetPropertyByName(UObject* TargetObject, const FString& InVariableName, const FString& NewValueString)
{
	FProperty* FoundProperty = FindFProperty<FProperty>(TargetObject->GetClass(), *InVariableName);
	if (FoundProperty)
	{
		const FScopedTransaction PropertyChanged(LOCTEXT("PropertyChanged", "Object Property Change"));

		TargetObject->Modify();

		TargetObject->PreEditChange(FoundProperty);
		FoundProperty->ImportText_InContainer(*NewValueString, TargetObject, TargetObject, 0);
		FPropertyChangedEvent PropertyChangedEvent(FoundProperty, EPropertyChangeType::ValueSet);
		TargetObject->PostEditChangeProperty(PropertyChangedEvent);
	}
}

/**
* Starts a PIE session
*/
void FEditorPromotionTestUtilities::StartPIE(bool bSimulateInEditor)
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::Get().GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
	TSharedPtr<class IAssetViewport> ActiveLevelViewport = LevelEditorModule.GetFirstActiveViewport();

	FRequestPlaySessionParams SessionParams;
	SessionParams.DestinationSlateViewport = ActiveLevelViewport;
	if (bSimulateInEditor)
	{
		SessionParams.WorldType = EPlaySessionWorldType::SimulateInEditor;
	}

	GUnrealEd->RequestPlaySession(SessionParams);
}

/**
* Ends a PIE session
*/
void FEditorPromotionTestUtilities::EndPIE()
{
	GUnrealEd->RequestEndPlayMap();
}

/**
* Takes an automation screenshot
*
* @param ScreenshotName - The sub name to use for the screenshot
*/
void FEditorPromotionTestUtilities::TakeScreenshot(const FString& ScreenshotName, const FAutomationScreenshotOptions& Options, bool bUseTopWindow)
{
	TSharedPtr<SWindow> Window;

	if (bUseTopWindow)
	{
		Window = FSlateApplication::Get().GetActiveTopLevelWindow();
	}
	else
	{
		//Find the main editor window
		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		TSharedPtr<SWindow>	ParentWindow = MainFrame.GetParentWindow();
		if (ParentWindow.IsValid() == false)
		{
			UE_LOGF(LogEditorPromotionTests, Error, "ERROR: Could not find the main editor window.");
			return;
		}

		Window = ParentWindow;
	}

	if (Window.IsValid())
	{
		TSharedRef<SWidget> WindowRef = Window.ToSharedRef();

		TArray<FColor> OutImageData;
		FIntVector OutImageSize;
		if (FSlateApplication::Get().TakeScreenshot(WindowRef, OutImageData, OutImageSize))
		{
			FAutomationScreenshotData Data = AutomationCommon::BuildScreenshotData(TEXT("Editor"), TEXT("PromotionTest"), ScreenshotName, OutImageSize.X, OutImageSize.Y);

			// Copy the relevant data into the metadata for the screenshot.
			Data.bHasComparisonRules = true;
			Data.ToleranceRed = Options.ToleranceAmount.Red;
			Data.ToleranceGreen = Options.ToleranceAmount.Green;
			Data.ToleranceBlue = Options.ToleranceAmount.Blue;
			Data.ToleranceAlpha = Options.ToleranceAmount.Alpha;
			Data.ToleranceMinBrightness = Options.ToleranceAmount.MinBrightness;
			Data.ToleranceMaxBrightness = Options.ToleranceAmount.MaxBrightness;
			Data.bIgnoreAntiAliasing = Options.bIgnoreAntiAliasing;
			Data.bIgnoreColors = Options.bIgnoreColors;
			Data.MaximumLocalError = Options.MaximumLocalError;
			Data.MaximumGlobalError = Options.MaximumGlobalError;

			FAutomationTestFramework::Get().OnScreenshotCaptured().ExecuteIfBound(OutImageData, Data);

			// TODO Add comparison support.
			if ( GIsAutomationTesting )
			{
				//FAutomationTestFramework::Get().OnScreenshotCompared.AddRaw(this, &FAutomationScreenshotTaker::OnComparisonComplete);
			}
		}
	}
	else
	{
		UE_LOGF(LogEditorPromotionTests, Error, "Failed to find editor window for screenshot (%ls)", *ScreenshotName);
	}
}

#undef LOCTEXT_NAMESPACE

#endif