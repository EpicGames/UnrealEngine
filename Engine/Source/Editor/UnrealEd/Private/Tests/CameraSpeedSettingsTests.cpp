// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#if WITH_AUTOMATION_TESTS

#include "Settings/EditorViewportSettings.h"

namespace UE::Editor::Tests
{
	BEGIN_DEFINE_SPEC(FCameraSpeedSettingsTextFixture, "Editor.Settings.Camera", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	
	END_DEFINE_SPEC(FCameraSpeedSettingsTextFixture)
	
	void FCameraSpeedSettingsTextFixture::Define()
	{
		It("Constrains the current speed with the min and max", [this]
		{
			FEditorViewportCameraSpeedSettings Settings;
			
			Settings.SetCurrentSpeed(0.0f);
			AddErrorIfFalse(Settings.GetCurrentSpeed() >= Settings.GetAbsoluteMinSpeed(), TEXT("Setting the current speed below the min speed should constrain it."));
			
			Settings.SetCurrentSpeed(1000000000.0f);
			AddErrorIfFalse(Settings.GetCurrentSpeed() <= Settings.GetAbsoluteMaxSpeed(), TEXT("Setting the current speed below the min speed should constrain it."));
            
			Settings.SetCurrentSpeed(10.0f);
			AddErrorIfFalse(Settings.GetCurrentSpeed() == 10.0f, TEXT("Setting the current speed within the min & max should succeed."));
		});
		
		It("Updates UI & Current values when setting speed range", [this]
		{
			FEditorViewportCameraSpeedSettings Settings;
			Settings.SetAbsoluteSpeedRange(4.0f, 10.0f);
				
			AddErrorIfFalse(Settings.GetAbsoluteMinSpeed() == 4.0f, TEXT("Setting min speed to a valid value should not fail."));
			AddErrorIfFalse(Settings.GetAbsoluteMaxSpeed() == 10.0f, TEXT("Setting max speed to a valid value should not fail."));
			AddErrorIfFalse(Settings.GetMinUISpeed() >= Settings.GetAbsoluteMinSpeed(), TEXT("Setting the min speed above min UI should adjust the UI speed."));
			AddErrorIfFalse(Settings.GetMaxUISpeed() <= Settings.GetAbsoluteMaxSpeed(), TEXT("Setting the max speed below max UI should adjust the UI speed."));
			AddErrorIfFalse(Settings.GetCurrentSpeed() >= Settings.GetAbsoluteMinSpeed(), TEXT("Setting a min speed should adjust the current speed upwards."));
		});
		
		It("Constrains setting the UI speed range to the absolute range", [this]
		{
			FEditorViewportCameraSpeedSettings Settings;
			Settings.SetAbsoluteSpeedRange(4.0f, 10.0f);
			Settings.SetUISpeedRange(0.0f, 20.0f);
			
			AddErrorIfFalse(Settings.GetAbsoluteMinSpeed() == 4.0f, TEXT("Min speed should not change."));
            AddErrorIfFalse(Settings.GetAbsoluteMaxSpeed() == 10.0f, TEXT("Max speed should not change."));
			AddErrorIfFalse(Settings.GetAbsoluteMinSpeed() == Settings.GetMinUISpeed(), TEXT("Min UI speed is constrained to Min speed."));
			AddErrorIfFalse(Settings.GetAbsoluteMaxSpeed() == Settings.GetMaxUISpeed(), TEXT("Max UI speed is constrained to Max speed."));
			
			Settings.SetUISpeedRange(8.0f, 6.0f);
			AddErrorIfFalse(Settings.GetMinUISpeed() == 6.0f && Settings.GetMaxUISpeed() == 8.0f, TEXT("Swap poorly ordered min & max UI speeds."));
			
			Settings.SetUISpeedRange(20.0f, 20.0f);
			AddErrorIfFalse(Settings.GetMaxUISpeed() <= Settings.GetAbsoluteMaxSpeed(), TEXT("Max UI speed must be constrained."));
			AddErrorIfFalse(Settings.GetMinUISpeed() < Settings.GetMaxUISpeed(), TEXT("The Min UI speed must always be lower than max UI."));
			AddErrorIfFalse(Settings.GetMinUISpeed() < Settings.GetAbsoluteMaxSpeed(), TEXT("The Min UI speed must always be lower than max."));
			
			Settings.SetUISpeedRange(2.0f, 2.0f);
			AddErrorIfFalse(Settings.GetMinUISpeed() >= Settings.GetAbsoluteMinSpeed(), TEXT("Min UI speed must be constrained."));
			AddErrorIfFalse(Settings.GetMinUISpeed() < Settings.GetMaxUISpeed(), TEXT("The Min UI speed must always be lower than Max UI."));
			AddErrorIfFalse(Settings.GetAbsoluteMinSpeed() < Settings.GetMaxUISpeed(), TEXT("The Max UI speed must always be higher than min."));
		});
	}
}

#endif // WITH_AUTOMATION_TESTS
