// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#if WITH_AUTOMATION_TESTS

#include "Misc/ChunkDependencyInfo.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FChunkDependencyHighestSharedTest, "System.Core.ChunkDependency.HighestSharedChunk",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FChunkDependencyHighestSharedTest::RunTest(const FString& Parameters)
{
	UChunkDependencyInfo* ChunkDependencyInfo = NewObject<UChunkDependencyInfo>();
	ChunkDependencyInfo->DependencyArray.Empty();
	ChunkDependencyInfo->DependencyArray.Add(FChunkDependency(5, 0));
	ChunkDependencyInfo->DependencyArray.Add(FChunkDependency(10, 0));
	ChunkDependencyInfo->DependencyArray.Add(FChunkDependency(20, 0));
	ChunkDependencyInfo->DependencyArray.Add(FChunkDependency(30, 20));
	ChunkDependencyInfo->DependencyArray.Add(FChunkDependency(40, 20));
	ChunkDependencyInfo->DependencyArray.Add(FChunkDependency(50, 20));
	ChunkDependencyInfo->DependencyArray.Add(FChunkDependency(60, 40));
	ChunkDependencyInfo->BuildChunkDependencyGraph(60);
	//     0
	//   / |  \
	//  5  10  20
	//        / | \
	//       30 40 50
	//          | 
	//          60 

	UTEST_EQUAL(TEXT("Invalid"), ChunkDependencyInfo->FindHighestSharedChunk({INDEX_NONE}), INDEX_NONE);
	UTEST_EQUAL(TEXT("Non existent"), ChunkDependencyInfo->FindHighestSharedChunk({100}), INDEX_NONE);
	UTEST_EQUAL(TEXT("Partial Invalid"), ChunkDependencyInfo->FindHighestSharedChunk({0, INDEX_NONE}), INDEX_NONE);
	UTEST_EQUAL(TEXT("Empty"), ChunkDependencyInfo->FindHighestSharedChunk({}), INDEX_NONE);
	UTEST_EQUAL(TEXT("Single"), ChunkDependencyInfo->FindHighestSharedChunk({ 10 }), 10);
	UTEST_EQUAL(TEXT("Duplicate"), ChunkDependencyInfo->FindHighestSharedChunk({ 10, 10 }), 10);
	UTEST_EQUAL(TEXT("Two leaf nodes are able to find a parent"), ChunkDependencyInfo->FindHighestSharedChunk({ 5, 10 }), 0);
	UTEST_EQUAL(TEXT("Non leaf nodes are able to find a parent"), ChunkDependencyInfo->FindHighestSharedChunk({ 10, 20 }), 0);
	UTEST_EQUAL(TEXT("Parent and Leaf results in parent"), ChunkDependencyInfo->FindHighestSharedChunk({ 20, 30 }), 20);
	UTEST_EQUAL(TEXT("separatend leaves find parent"), ChunkDependencyInfo->FindHighestSharedChunk({ 30, 60 }), 20);
	UTEST_EQUAL(TEXT("Complex parent"), ChunkDependencyInfo->FindHighestSharedChunk({ 30, 40, 60 }), 20);
	UTEST_EQUAL(TEXT("(20, 40, 60) = 20"), ChunkDependencyInfo->FindHighestSharedChunk({ 20, 40, 60 }), 20);
	UTEST_EQUAL(TEXT("(20, 40, 30) = 20"), ChunkDependencyInfo->FindHighestSharedChunk({ 20, 40, 30 }), 20);
	UTEST_EQUAL(TEXT("(40, 60) = 40"), ChunkDependencyInfo->FindHighestSharedChunk({ 40, 60 }), 40);
	UTEST_EQUAL(TEXT("(5, 60) = 0"), ChunkDependencyInfo->FindHighestSharedChunk({ 5, 60 }), 0);
	UTEST_EQUAL(TEXT("(0, 5) = 0"), ChunkDependencyInfo->FindHighestSharedChunk({ 0, 5 }), 0);
	return true;
}

/**
 * multiple parents are not supported by the chunk dependencies.
 * Only the first parent will be considered the parent.
 */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FChunkDependencyMultipleParentTest, "System.Core.ChunkDependency.MultipleParent",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	bool FChunkDependencyMultipleParentTest::RunTest(const FString& Parameters)
{
	UChunkDependencyInfo* ChunkDependencyInfo = NewObject<UChunkDependencyInfo>();
	ChunkDependencyInfo->DependencyArray.Empty();
	ChunkDependencyInfo->DependencyArray.Add(FChunkDependency(5, 0));
	ChunkDependencyInfo->DependencyArray.Add(FChunkDependency(10, 0));
	ChunkDependencyInfo->DependencyArray.Add(FChunkDependency(20, 0));
	ChunkDependencyInfo->DependencyArray.Add(FChunkDependency(30, 20));
	ChunkDependencyInfo->DependencyArray.Add(FChunkDependency(40, 20));
	ChunkDependencyInfo->DependencyArray.Add(FChunkDependency(50, 20));
	ChunkDependencyInfo->DependencyArray.Add(FChunkDependency(60, 40));
	ChunkDependencyInfo->DependencyArray.Add(FChunkDependency(60, 30));
	ChunkDependencyInfo->BuildChunkDependencyGraph(60);
	//     0
	//   / |  \
	//  5  10  20
	//        / | \
	//       30 40 50
	//        \ | 
	//          60 


	UTEST_EQUAL(TEXT("Single"), ChunkDependencyInfo->FindHighestSharedChunk({ 10 }), 10);
	UTEST_EQUAL(TEXT("Duplicate"), ChunkDependencyInfo->FindHighestSharedChunk({ 10, 10 }), 10);
	UTEST_EQUAL(TEXT("Two leaf nodes are able to find a parent"), ChunkDependencyInfo->FindHighestSharedChunk({ 5, 10 }), 0);
	UTEST_EQUAL(TEXT("Non leaf nodes are able to find a parent"), ChunkDependencyInfo->FindHighestSharedChunk({ 10, 20 }), 0);
	UTEST_EQUAL(TEXT("Parent and Leaf results in parent"), ChunkDependencyInfo->FindHighestSharedChunk({ 20, 30 }), 20);
	UTEST_EQUAL(TEXT("separatend leaves find parent"), ChunkDependencyInfo->FindHighestSharedChunk({ 30, 60 }), 20);
	UTEST_EQUAL(TEXT("Complex parent"), ChunkDependencyInfo->FindHighestSharedChunk({ 30, 40, 60 }), 20);
	UTEST_EQUAL(TEXT("(20, 40, 60) = 20"), ChunkDependencyInfo->FindHighestSharedChunk({ 20, 40, 60 }), 20);
	UTEST_EQUAL(TEXT("(20, 40, 30) = 20"), ChunkDependencyInfo->FindHighestSharedChunk({ 20, 40, 30 }), 20);
	UTEST_EQUAL(TEXT("(40, 60) = 40"), ChunkDependencyInfo->FindHighestSharedChunk({ 40, 60 }), 40);
	UTEST_EQUAL(TEXT("(30, 40, 60) = 20"), ChunkDependencyInfo->FindHighestSharedChunk({ 30, 40, 60 }), 20);
	UTEST_EQUAL(TEXT("(30, 60) = 20 - multiple parents not supported."), ChunkDependencyInfo->FindHighestSharedChunk({ 30, 60 }), 20);
	UTEST_EQUAL(TEXT("(10, 60) = 0"), ChunkDependencyInfo->FindHighestSharedChunk({ 10, 60 }), 0);
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FChunkDependencyCycleTest, "System.Core.ChunkDependency.Cycle",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	bool FChunkDependencyCycleTest::RunTest(const FString& Parameters)
{
	UChunkDependencyInfo* ChunkDependencyInfo = NewObject<UChunkDependencyInfo>();
	ChunkDependencyInfo->DependencyArray.Empty();
	ChunkDependencyInfo->DependencyArray.Add(FChunkDependency(5, 0));
	ChunkDependencyInfo->DependencyArray.Add(FChunkDependency(10, 0));
	ChunkDependencyInfo->DependencyArray.Add(FChunkDependency(20, 0));
	ChunkDependencyInfo->DependencyArray.Add(FChunkDependency(30, 20));
	ChunkDependencyInfo->DependencyArray.Add(FChunkDependency(40, 20));
	ChunkDependencyInfo->DependencyArray.Add(FChunkDependency(60, 40));
	ChunkDependencyInfo->DependencyArray.Add(FChunkDependency(60, 20));
	ChunkDependencyInfo->BuildChunkDependencyGraph(60);
	//     0
	//   / |  \
	//  5  10  20
	//        / | 
	//       30 40 
	//          | 
	//          60 
	//          | 
	//          20 

	UTEST_EQUAL(TEXT("Two leaf nodes are able to find a parent"), ChunkDependencyInfo->FindHighestSharedChunk({ 5, 10 }), 0);
	UTEST_EQUAL(TEXT("separatend leaves find parent"), ChunkDependencyInfo->FindHighestSharedChunk({ 30, 60 }), 20);
	UTEST_EQUAL(TEXT("Complex parent"), ChunkDependencyInfo->FindHighestSharedChunk({ 30, 40, 60 }), 20);
	UTEST_EQUAL(TEXT("(20, 40, 60) = 20"), ChunkDependencyInfo->FindHighestSharedChunk({ 20, 40, 60 }), 20);
	UTEST_EQUAL(TEXT("(20, 60) = 20"), ChunkDependencyInfo->FindHighestSharedChunk({ 20, 60 }), 20);
	UTEST_EQUAL(TEXT("(40, 60) = 40"), ChunkDependencyInfo->FindHighestSharedChunk({ 40, 60 }), 40);
	return true;
}

#endif // WITH_AUTOMATION_TESTS