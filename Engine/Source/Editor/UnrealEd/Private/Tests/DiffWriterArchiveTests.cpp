// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#if WITH_AUTOMATION_TESTS

#include "Cooker/DiffWriterArchive.h"

#include "Memory/SharedBuffer.h"
#include "Serialization/LargeMemoryWriter.h"
#include "Serialization/MemoryReader.h"

namespace UE::DiffWriter::Tests
{

struct FBasicDiffState
{
	int32 FirstParam = 0;
	int32 SecondParam = 0;
	int32 ThirdParam = 0;

	FORCENOINLINE void Serialize(FArchive& Ar)
	{
		Ar << FirstParam;
		Ar << SecondParam;
		Ar << ThirdParam;
	}
};

ICookedPackageWriter::FPreviousCookedBytesData ToPackageData(FLargeMemoryWriter& Ar)
{
	int64 Size = Ar.TotalSize(); // Cache this before calling ReleaseOwnership
	return ICookedPackageWriter::FPreviousCookedBytesData
	{
		TUniquePtr<uint8, ICookedPackageWriter::FDeleteByFree>(Ar.ReleaseOwnership()),
			Size, 0 /* HeaderSize */, 0 /* StartOffset */
	};
}

TSharedPtr<FDiffOutputRecorder> MakeOutputRecorder()
{
	auto EmptyCallback = [](ELogVerbosity::Type Verbosity, FStringView Message) {};

	return MakeShared<FDiffOutputRecorder>(EmptyCallback, nullptr);
}

} // namespace UE::DiffWriter::Tests

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FDiffWriterArchiveTestsCallstacks, "System.Core.Cooker.DiffWriterArchive.Callstacks",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FDiffWriterArchiveTestsCallstacks::RunTest(const FString& Parameters)
{
	using namespace UE::DiffWriter;
	using namespace UE::DiffWriter::Tests;

	const int32 MaxDiffsToLog = 1000;
	bool bIgnoreHeaderDiffs = false;
	FName PackageName(TEXT("PackageName"));
	EPackageHeaderFormat HeaderFormat(EPackageHeaderFormat::PackageFileSummary);
	UE::DiffWriter::FAccumulatorGlobals AccumulatorGlobals;

	TRefCountPtr<FAccumulator> Accumulator = new FAccumulator(AccumulatorGlobals, nullptr, PackageName, MaxDiffsToLog,
		bIgnoreHeaderDiffs, UE::DiffWriter::Tests::MakeOutputRecorder(), HeaderFormat);
	FBasicDiffState State = FBasicDiffState{ 1,2,3 };
	FDiffArchiveForLinker Ar(*Accumulator);

	State.Serialize(Ar);

	// Accumulator->bFirstSaveComplete should be is false, so bCollectingCallstacks should be false.
	// We always collect a callstack struct, but in the !bCollectingCallstacks case the callstack struct
	// we collected should have null cpp callstack data.
	FCallstacks& Callstacks = Accumulator->LinkerCallstacks;
	TestTrueExpr(Callstacks.Num() == 3);
	TestTrueExpr(Callstacks.GetEndOffset() == sizeof(FBasicDiffState));
	TestTrueExpr(Callstacks.GetCallstack(0).Offset == 0);
	TestTrueExpr(Callstacks.GetCallstack(0).Callstack != nullptr
		&& Callstacks.GetCallstack(0).Callstack->Callstack == nullptr);
	TestTrueExpr(Callstacks.GetCallstack(0).bSuppressLogging == false);
	TestTrueExpr(Callstacks.GetCallstack(1).Offset == 4);
	TestTrueExpr(Callstacks.GetCallstack(1).Callstack != nullptr
		&& Callstacks.GetCallstack(1).Callstack->Callstack == nullptr);
	TestTrueExpr(Callstacks.GetCallstack(1).bSuppressLogging == false);
	TestTrueExpr(Callstacks.GetCallstack(2).Offset == 8);
	TestTrueExpr(Callstacks.GetCallstack(2).Callstack != nullptr
		&& Callstacks.GetCallstack(2).Callstack->Callstack == nullptr);
	TestTrueExpr(Callstacks.GetCallstack(2).bSuppressLogging == false);
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FDiffWriterArchiveTestsBasic, "System.Core.Cooker.DiffWriterArchive.Basic",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FDiffWriterArchiveTestsBasic::RunTest(const FString& Parameters)
{
	using namespace UE::DiffWriter;
	using namespace UE::DiffWriter::Tests;

	const int32 MaxDiffsToLog = 1000;
	bool bIgnoreHeaderDiffs = false;
	FName PackageName(TEXT("PackageName"));
	FString Filename(TEXT("Filename"));
	EPackageHeaderFormat HeaderFormat(EPackageHeaderFormat::PackageFileSummary);
	UE::DiffWriter::FAccumulatorGlobals AccumulatorGlobals;

	// SECTION("DiffMap - identical")
	{
		TRefCountPtr<FAccumulator> Accumulator = new FAccumulator(AccumulatorGlobals, nullptr, PackageName,
			MaxDiffsToLog, bIgnoreHeaderDiffs,
			UE::DiffWriter::Tests::MakeOutputRecorder(), HeaderFormat);

		FLargeMemoryWriter InitialState;
		{
			FBasicDiffState State = FBasicDiffState {1,2,3};
			State.Serialize(InitialState);
		}

		FDiffArchiveForLinker NewState(*Accumulator);
		{
			FBasicDiffState State = FBasicDiffState {1,2,3};
			State.Serialize(NewState);
		}

		ICookedPackageWriter::FPreviousCookedBytesData InitialData = ToPackageData(InitialState);
		Accumulator->OnFirstSaveComplete(Filename, 0, 0, MoveTemp(InitialData));

		TestTrueExpr(!Accumulator->HasDifferences());
	}

	// SECTION("DiffMap - mismatch")
	{
		TRefCountPtr<FAccumulator> Accumulator = new FAccumulator(AccumulatorGlobals, nullptr, PackageName,
			MaxDiffsToLog, bIgnoreHeaderDiffs,
			UE::DiffWriter::Tests::MakeOutputRecorder(), HeaderFormat);

		FLargeMemoryWriter InitialState;
		{
			FBasicDiffState State = FBasicDiffState {1,2,3};
			State.Serialize(InitialState);
		}

		FDiffArchiveForLinker NewState(*Accumulator);
		{
			FBasicDiffState State = FBasicDiffState {1,2,4};
			State.Serialize(NewState);
		}

		ICookedPackageWriter::FPreviousCookedBytesData InitialData = ToPackageData(InitialState);
		Accumulator->OnFirstSaveComplete(Filename, 0, 0, MoveTemp(InitialData));

		TestTrueExpr(Accumulator->HasDifferences());
	}

	// SECTION("Compare - mismatch")
	{
		TRefCountPtr<FAccumulator> Accumulator = new FAccumulator(AccumulatorGlobals, nullptr, PackageName,
			MaxDiffsToLog, bIgnoreHeaderDiffs,
			UE::DiffWriter::Tests::MakeOutputRecorder(), HeaderFormat);

		FBasicDiffState InitialState = FBasicDiffState {1,2,3};
		FLargeMemoryWriter InitialMemory;
		InitialMemory.SetByteSwapping(false);
		InitialState.Serialize(InitialMemory);
		ICookedPackageWriter::FPreviousCookedBytesData InitialData = ToPackageData(InitialMemory);

		FBasicDiffState NewState = FBasicDiffState {1,22,33};
		TUniquePtr<FDiffArchiveForLinker> NewMemory = MakeUnique<FDiffArchiveForLinker>(*Accumulator);
		NewMemory->SetByteSwapping(false);
		NewState.Serialize(*NewMemory);

		// Generate diff map
		{
			Accumulator->OnFirstSaveComplete(Filename, 0, 0, MoveTemp(InitialData));
			TestTrueExpr(Accumulator->HasDifferences());
		}
		
		NewMemory.Reset();
		NewMemory = MakeUnique<FDiffArchiveForLinker>(*Accumulator);
		NewMemory->SetByteSwapping(false);
		NewState.Serialize(*NewMemory);
		Accumulator->OnSecondSaveComplete(0);

		// Compare using callstacks and diffmap
		{
			TMap<FName, FArchiveDiffStats> DiffStats;
			Accumulator->CompareWithPrevious(TEXT("BasicDiffTest"), DiffStats);
			TestTrueExpr(DiffStats[NAME_None].NumDiffs == 2);
		}
	}
	return true;
}

#endif // WITH_AUTOMATION_TESTS
