// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/FbxAutomationCommon.h"

#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Factories/FbxAnimSequenceImportData.h"
#include "Factories/FbxImportUI.h"
#include "Factories/FbxSkeletalMeshImportData.h"
#include "Factories/FbxStaticMeshImportData.h"
#include "Factories/FbxTextureImportData.h"
#include "HAL/PlatformCrt.h"
#include "JsonObjectConverter.h"
#include "Misc/FileHelper.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "Templates/SharedPointer.h"
#include "UObject/Class.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FbxAutomationCommon)


namespace FbxAutomationTestsAPI
{
	void ReadFbxOptions(const FString &FileOptionAndResult, TArray<UFbxTestPlan*> &TestPlanArray)
	{
		FString ImportUIJsonString;
		TSharedPtr<FJsonObject> RootObject;
		if (FFileHelper::LoadFileToString(ImportUIJsonString, *FileOptionAndResult))
		{
			TSharedRef< TJsonReader<> > JsonReader = TJsonReaderFactory<>::Create(ImportUIJsonString);
			if (!FJsonSerializer::Deserialize(JsonReader, RootObject))
			{
				return;
			}
		}
		const TArray< TSharedPtr<FJsonValue> > TestPlanArrayJson = RootObject->GetArrayField(TEXT("TestPlanArray"));
		for (TSharedPtr<FJsonValue> TestplanValue : TestPlanArrayJson)
		{

			const TSharedPtr<FJsonObject> TestPlanObjectObject = TestplanValue->AsObject();
			if (!TestPlanObjectObject.IsValid())
			{
				continue;
			}
			UFbxTestPlan* FbxTestPlan = NewObject<UFbxTestPlan>();
			FbxTestPlan->AddToRoot();

			//FFbxTestPlan basic property
			const TSharedPtr<FJsonObject> TestPlanPropertiesObject = TestPlanObjectObject->GetObjectField(TEXT("TestPlanProperties"));
			FJsonObjectConverter::JsonObjectToUStruct(TestPlanPropertiesObject.ToSharedRef(), UFbxTestPlan::StaticClass(), FbxTestPlan, 0, 0);
			
			FbxTestPlan->ImportUI = NewObject<UFbxImportUI>();
			FbxTestPlan->ImportUI->AddToRoot();

			//StaticMesh basic property
			const TSharedPtr<FJsonObject> StaticMeshImportDataObject = TestPlanObjectObject->GetObjectField(UFbxStaticMeshImportData::StaticClass()->GetPathName());
			FJsonObjectConverter::JsonObjectToUStruct(StaticMeshImportDataObject.ToSharedRef(), UFbxStaticMeshImportData::StaticClass(), FbxTestPlan->ImportUI->StaticMeshImportData, 0, 0);

			//SkeletalMesh basic property
			const TSharedPtr<FJsonObject> SkeletalMeshImportDataObject = TestPlanObjectObject->GetObjectField(UFbxSkeletalMeshImportData::StaticClass()->GetPathName());
			FJsonObjectConverter::JsonObjectToUStruct(SkeletalMeshImportDataObject.ToSharedRef(), UFbxSkeletalMeshImportData::StaticClass(), FbxTestPlan->ImportUI->SkeletalMeshImportData, 0, 0);

			//AnimSequence basic property
			const TSharedPtr<FJsonObject> AnimSequenceImportDataObject = TestPlanObjectObject->GetObjectField(UFbxAnimSequenceImportData::StaticClass()->GetPathName());
			FJsonObjectConverter::JsonObjectToUStruct(AnimSequenceImportDataObject.ToSharedRef(), UFbxAnimSequenceImportData::StaticClass(), FbxTestPlan->ImportUI->AnimSequenceImportData, 0, 0);

			//Texture basic property
			const TSharedPtr<FJsonObject> TextureImportDataObject = TestPlanObjectObject->GetObjectField(UFbxTextureImportData::StaticClass()->GetPathName());
			FJsonObjectConverter::JsonObjectToUStruct(TextureImportDataObject.ToSharedRef(), UFbxTextureImportData::StaticClass(), FbxTestPlan->ImportUI->TextureImportData, 0, 0);

			//Store pointer to set it back after the json read
			UFbxStaticMeshImportData *SavedStaticMeshData = FbxTestPlan->ImportUI->StaticMeshImportData;
			UFbxSkeletalMeshImportData *SavedSkeletalMeshData = FbxTestPlan->ImportUI->SkeletalMeshImportData;
			UFbxAnimSequenceImportData *SavedAnimSequenceData = FbxTestPlan->ImportUI->AnimSequenceImportData;
			UFbxTextureImportData *SavedTextureData = FbxTestPlan->ImportUI->TextureImportData;
			//ImportUi basic property
			const TSharedPtr<FJsonObject> ImportUiObject = TestPlanObjectObject->GetObjectField(UFbxImportUI::StaticClass()->GetPathName());
			FJsonObjectConverter::JsonObjectToUStruct(ImportUiObject.ToSharedRef(), UFbxImportUI::StaticClass(), FbxTestPlan->ImportUI, 0, 0);
			//Put back the pointer
			FbxTestPlan->ImportUI->StaticMeshImportData = SavedStaticMeshData;
			FbxTestPlan->ImportUI->SkeletalMeshImportData = SavedSkeletalMeshData;
			FbxTestPlan->ImportUI->AnimSequenceImportData = SavedAnimSequenceData;
			FbxTestPlan->ImportUI->TextureImportData = SavedTextureData;

			TestPlanArray.Add(FbxTestPlan);
		}
	}

	void WriteFbxOptions(const FString &Filename, TArray<UFbxTestPlan*> &TestPlanArray)
	{
		TSharedRef<FJsonObject> RootObject = MakeShareable(new FJsonObject);
		TArray< TSharedPtr<FJsonValue> > TestPlanArrayJson;
		for (UFbxTestPlan* TestPlan : TestPlanArray)
		{
			TSharedRef<FJsonObject> TestPlanObject = MakeShareable(new FJsonObject);
			//FFbxTestPlan basic property
			TSharedRef<FJsonObject> TestPlanPropertiesObject = MakeShareable(new FJsonObject);
			if (FJsonObjectConverter::UStructToJsonObject(UFbxTestPlan::StaticClass(), TestPlan, TestPlanPropertiesObject, 0, 0))
			{
				TestPlanObject->SetField(TEXT("TestPlanProperties"), MakeShareable(new FJsonValueObject(TestPlanPropertiesObject)));
			}
			//ImportUi basic property
			TSharedRef<FJsonObject> ImportUiObject = MakeShareable(new FJsonObject);
			if (FJsonObjectConverter::UStructToJsonObject(TestPlan->ImportUI->GetClass(), TestPlan->ImportUI, ImportUiObject, 0, 0))
			{
				TestPlanObject->SetField(TestPlan->ImportUI->GetClass()->GetPathName(), MakeShareable(new FJsonValueObject(ImportUiObject)));
			}
			//StaticMesh basic property
			TSharedRef<FJsonObject> StaticMeshImportData = MakeShareable(new FJsonObject);
			if (FJsonObjectConverter::UStructToJsonObject(TestPlan->ImportUI->StaticMeshImportData->GetClass(), TestPlan->ImportUI->StaticMeshImportData, StaticMeshImportData, 0, 0))
			{
				TestPlanObject->SetField(TestPlan->ImportUI->StaticMeshImportData->GetClass()->GetPathName(), MakeShareable(new FJsonValueObject(StaticMeshImportData)));
			}
			//SkeletalMesh basic property
			TSharedRef<FJsonObject> SkeletalMeshImportData = MakeShareable(new FJsonObject);
			if (FJsonObjectConverter::UStructToJsonObject(TestPlan->ImportUI->SkeletalMeshImportData->GetClass(), TestPlan->ImportUI->SkeletalMeshImportData, SkeletalMeshImportData, 0, 0))
			{
				TestPlanObject->SetField(TestPlan->ImportUI->SkeletalMeshImportData->GetClass()->GetPathName(), MakeShareable(new FJsonValueObject(SkeletalMeshImportData)));
			}
			//AnimSequence basic property
			TSharedRef<FJsonObject> AnimSequenceImportData = MakeShareable(new FJsonObject);
			if (FJsonObjectConverter::UStructToJsonObject(TestPlan->ImportUI->AnimSequenceImportData->GetClass(), TestPlan->ImportUI->AnimSequenceImportData, AnimSequenceImportData, 0, 0))
			{
				TestPlanObject->SetField(TestPlan->ImportUI->AnimSequenceImportData->GetClass()->GetPathName(), MakeShareable(new FJsonValueObject(AnimSequenceImportData)));
			}
			//Texture basic property
			TSharedRef<FJsonObject> TextureImportData = MakeShareable(new FJsonObject);
			if (FJsonObjectConverter::UStructToJsonObject(TestPlan->ImportUI->TextureImportData->GetClass(), TestPlan->ImportUI->TextureImportData, TextureImportData, 0, 0))
			{
				TestPlanObject->SetField(TestPlan->ImportUI->TextureImportData->GetClass()->GetPathName(), MakeShareable(new FJsonValueObject(TextureImportData)));
			}
			TestPlanArrayJson.Add(MakeShareable(new FJsonValueObject(TestPlanObject)));
		}

		RootObject->SetArrayField(TEXT("TestPlanArray"), TestPlanArrayJson);

		//Write the json file
		FString Json;
		TSharedRef<TJsonWriter<> > JsonWriter = TJsonWriterFactory<>::Create(&Json, 0);
		if (FJsonSerializer::Serialize(RootObject, JsonWriter))
		{
			FFileHelper::SaveStringToFile(Json, *Filename);
		}
	}
}

UFbxTestPlan::UFbxTestPlan(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ImportUI = nullptr;
}
