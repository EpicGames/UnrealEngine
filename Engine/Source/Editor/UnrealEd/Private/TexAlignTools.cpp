// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TexAlignTools.cpp: Tools for aligning textures on surfaces
=============================================================================*/

#include "TexAlignTools.h"
#include "Engine/Level.h"
#include "Model.h"
#include "TexAligner/TexAlignerBox.h"
#include "TexAligner/TexAlignerDefault.h"
#include "TexAligner/TexAlignerFit.h"
#include "TexAligner/TexAlignerPlanar.h"
#include "Engine/Polys.h"
#include "Editor.h"
#include "BSPOps.h"

FTexAlignTools GTexAlignTools;

static int32 GetMajorAxis( FVector3f InNormal, int32 InForceAxis )
{
	// Figure out the major axis information.
	int32 Axis = TAXIS_X;
	if( FMath::Abs(InNormal.Y) >= 0.5f ) Axis = TAXIS_Y;
	else 
	{
		// Only check Z if we aren't aligned to walls
		if( InForceAxis != TAXIS_WALLS )
			if( FMath::Abs(InNormal.Z) >= 0.5f ) Axis = TAXIS_Z;
	}

	return Axis;

}

// Checks the normal of the major axis ... if it's negative, returns 1.
static bool ShouldFlipVectors( FVector3f InNormal, int32 InAxis )
{
	if( InAxis == TAXIS_X )
		if( InNormal.X < 0 ) return 1;
	if( InAxis == TAXIS_Y )
		if( InNormal.Y < 0 ) return 1;
	if( InAxis == TAXIS_Z )
		if( InNormal.Z < 0 ) return 1;

	return 0;

}


UTexAligner::UTexAligner(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}


void UTexAligner::PostInitProperties()
{
	Super::PostInitProperties();
	Desc = TEXT("N/A");
	TAxis = TAXIS_AUTO;
	UTile = VTile = 1.f;
	DefTexAlign = TEXALIGN_Default;
}

void UTexAligner::Align( UWorld* InWorld, ETexAlign InTexAlignType )
{
	for( int32 LevelIndex = 0; LevelIndex < InWorld->GetNumLevels(); ++LevelIndex )
	{
		ULevel* Level = InWorld->GetLevel(LevelIndex);
		Align( InWorld, InTexAlignType, Level->Model );
	}
}

void UTexAligner::Align( UWorld* InWorld, ETexAlign InTexAlignType, UModel* InModel )
{
	//
	// Build an initial list of BSP surfaces to be aligned.
	//
	
	FPoly EdPoly;
	TArray<FBspSurfIdx> InitialSurfList;

	for( int32 i = 0 ; i < InModel->Surfs.Num() ; i++ )
	{
		FBspSurf* Surf = &InModel->Surfs[i];

		if( Surf->PolyFlags & PF_Selected )
		{
			new(InitialSurfList)FBspSurfIdx( Surf, i );
		}
	}

	//
	// Create a final list of BSP surfaces ... 
	//
	// - allows for rejection of surfaces
	// - allows for specific ordering of faces
	//

	TArray<FBspSurfIdx> FinalSurfList;
	FVector Normal;

	for( int32 i = 0 ; i < InitialSurfList.Num() ; i++ )
	{
		FBspSurfIdx* Surf = &InitialSurfList[i];
//		Normal = InModel->Vectors[ Surf->Surf->vNormal ];
//		GEditor->polyFindBrush( InModel, Surf->Idx, EdPoly );

		bool bOK = 1;
		/*
		switch( InTexAlignType )
		{
		}
		*/

		if( bOK )
			new(FinalSurfList)FBspSurfIdx( Surf->Surf, Surf->Idx );
	}

	//
	// Align the final surfaces.
	//

	for( int32 i = 0 ; i < FinalSurfList.Num() ; i++ )
	{
		FBspSurfIdx* Surf = &FinalSurfList[i];
		GEditor->polyFindBrush( InModel, Surf->Idx, EdPoly );
		Normal = (FVector)InModel->Vectors[ Surf->Surf->vNormal ];

		AlignSurf( InTexAlignType == TEXALIGN_None ? (ETexAlign)DefTexAlign : InTexAlignType, InModel, Surf, &EdPoly, &Normal );

		const bool bUpdateTexCoords = true;
		const bool bOnlyRefreshSurfaceMaterials = true;
		GEditor->polyUpdateBrush(InModel, Surf->Idx, bUpdateTexCoords, bOnlyRefreshSurfaceMaterials);
	}

	GEditor->RedrawLevelEditingViewports();

	InWorld->MarkPackageDirty();
	ULevel::LevelDirtiedEvent.Broadcast();
}

void UTexAligner::AlignSurf( ETexAlign InTexAlignType, UModel* InModel, FBspSurfIdx* InSurfIdx, FPoly* InPoly, FVector* InNormal )
{
}

UTexAlignerPlanar::UTexAlignerPlanar(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UTexAlignerPlanar::PostInitProperties()
{
	Super::PostInitProperties();
	Desc = NSLOCTEXT("UnrealEd", "Planar", "Planar").ToString();
	DefTexAlign = TEXALIGN_Planar;
}

void UTexAlignerPlanar::AlignSurf( ETexAlign InTexAlignType, UModel* InModel, FBspSurfIdx* InSurfIdx, FPoly* InPoly, FVector* InNormal )
{
	if( InTexAlignType == TEXALIGN_PlanarAuto )
		TAxis = TAXIS_AUTO;
	else if( InTexAlignType == TEXALIGN_PlanarWall )
		TAxis = TAXIS_WALLS;
	else if( InTexAlignType == TEXALIGN_PlanarFloor )
		TAxis = TAXIS_Z;

	int32 Axis = GetMajorAxis( (FVector3f)*InNormal, TAxis );

	if( TAxis != TAXIS_AUTO && TAxis != TAXIS_WALLS )
		Axis = TAxis;

	bool bFlip = ShouldFlipVectors( (FVector3f)*InNormal, Axis );

	// Determine the texturing vectors.
	FVector U, V;
	if( Axis == TAXIS_X )
	{
		U = FVector(0, (bFlip ? 1 : -1) ,0);
		V = FVector(0,0,-1);
	}
	else if( Axis == TAXIS_Y )
	{
		U = FVector((bFlip ? -1 : 1),0,0);
		V = FVector(0,0,-1);
	}
	else
	{
		U = FVector((bFlip ? 1 : -1),0,0);
		V = FVector(0,-1,0);
	}

	FVector Base = FVector::ZeroVector;

	U *= UTile;
	V *= VTile;

	InSurfIdx->Surf->pBase = FBSPOps::bspAddPoint(InModel,&Base,0);
	InSurfIdx->Surf->vTextureU = FBSPOps::bspAddVector( InModel, &U, 0);
	InSurfIdx->Surf->vTextureV = FBSPOps::bspAddVector( InModel, &V, 0);

}

UTexAlignerDefault::UTexAlignerDefault(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UTexAlignerDefault::PostInitProperties()
{
	Super::PostInitProperties();
	Desc = NSLOCTEXT("UnrealEd", "Default", "Default").ToString();
	DefTexAlign = TEXALIGN_Default;
}

void UTexAlignerDefault::AlignSurf( ETexAlign InTexAlignType, UModel* InModel, FBspSurfIdx* InSurfIdx, FPoly* InPoly, FVector* InNormal )
{
	InPoly->Base = InPoly->Vertices[0];
	InPoly->TextureU = FVector3f::ZeroVector;
	InPoly->TextureV = FVector3f::ZeroVector;
	InPoly->Finalize( NULL, 0 );

	InPoly->TextureU *= UTile;
	InPoly->TextureV *= VTile;

	ABrush* Actor = InSurfIdx->Surf->Actor;
	const FVector PrePivot = Actor->GetPivotOffset();
	const FVector Location = Actor->GetActorLocation();
	const FRotator Rotation = Actor->GetActorRotation();
	const FVector Scale = Actor->GetActorScale();
	const FRotationMatrix RotMatrix(Rotation);

	FVector Base = RotMatrix.TransformVector(((FVector)InPoly->Base - PrePivot) * Scale) + Location;
	FVector TextureU = RotMatrix.TransformVector((FVector)InPoly->TextureU / Scale);
	FVector TextureV = RotMatrix.TransformVector((FVector)InPoly->TextureV / Scale);

	InSurfIdx->Surf->pBase = FBSPOps::bspAddPoint(InModel, &Base, 0);
	InSurfIdx->Surf->vTextureU = FBSPOps::bspAddVector( InModel, &TextureU, 0);
	InSurfIdx->Surf->vTextureV = FBSPOps::bspAddVector( InModel, &TextureV, 0);
}

UTexAlignerBox::UTexAlignerBox(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UTexAlignerBox::PostInitProperties()
{
	Super::PostInitProperties();
	Desc = NSLOCTEXT("UnrealEd", "Box", "Box").ToString();
	DefTexAlign = TEXALIGN_Box;
}

void UTexAlignerBox::AlignSurf( ETexAlign InTexAlignType, UModel* InModel, FBspSurfIdx* InSurfIdx, FPoly* InPoly, FVector* InNormal )
{
	FVector U, V;

	FVector Normal = *InNormal;
	Normal.FindBestAxisVectors( V, U );
	U *= -1.0;
	V *= -1.0;

	U *= UTile;
	V *= VTile;

	FVector	Base = FVector::ZeroVector;

	InSurfIdx->Surf->pBase = FBSPOps::bspAddPoint(InModel,&Base,0);
	InSurfIdx->Surf->vTextureU = FBSPOps::bspAddVector( InModel, &U, 0 );
	InSurfIdx->Surf->vTextureV = FBSPOps::bspAddVector( InModel, &V, 0 );

}

UTexAlignerFit::UTexAlignerFit(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UTexAlignerFit::PostInitProperties()
{
	Super::PostInitProperties();
	Desc = NSLOCTEXT("UnrealEd", "Fit", "Fit").ToString();
	DefTexAlign = TEXALIGN_Fit;
}

void UTexAlignerFit::AlignSurf( ETexAlign InTexAlignType, UModel* InModel, FBspSurfIdx* InSurfIdx, FPoly* InPoly, FVector* InNormal )
{
	// @todo: Support cycling between texture corners by FIT'ing again?  Each Ctrl+Shift+F would rotate texture.
	// @todo: Consider making initial FIT match the texture's current orientation as close as possible?
	// @todo: Handle subtractive brush polys differently?  (flip U texture direction)
	// @todo: Option to ignore pixel aspect for quads (e.g. stretch full texture non-uniformly over quad)


	// Compute world space vertex positions
	TArray< FVector > WorldSpacePolyVertices;
	for( int32 VertexIndex = 0; VertexIndex < InPoly->Vertices.Num(); ++VertexIndex )
	{
		WorldSpacePolyVertices.Add( InSurfIdx->Surf->Actor->ActorToWorld().TransformPosition( (FVector)InPoly->Vertices[ VertexIndex ] ) );
	}

			
	// Create an orthonormal basis for the polygon
	FMatrix WorldToPolyRotationMatrix;
	const FVector& FirstPolyVertex = WorldSpacePolyVertices[ 0 ];
	{
		const FVector& VertexA = FirstPolyVertex;
		const FVector& VertexB = WorldSpacePolyVertices[ 1 ];
		FVector UpVec = ( VertexB - VertexA ).GetSafeNormal();
		FVector RightVec = (FVector)InPoly->Normal ^ UpVec;
		WorldToPolyRotationMatrix.SetIdentity();
		FVector Normal = (FVector)InPoly->Normal;
		WorldToPolyRotationMatrix.SetAxes( &RightVec, &UpVec, &Normal );
	}


	// Find a corner of the polygon that's closest to a 90 degree angle.  When there are multiple corners with
	// similar angles, we'll use the one closest to the local space bottom-left along the polygon's plane
	const double DesiredAbsDotProduct = 0.0f;
	int32 BestVertexIndex = INDEX_NONE;
	double BestDotProductDiff = 10000.0f;
	double BestPositivity = 10000.0f;
	for( int32 VertexIndex = 0; VertexIndex < WorldSpacePolyVertices.Num(); ++VertexIndex )
	{
		// Compute the previous and next vertex in the winding
		const int32 PrevWindingVertexIndex = ( VertexIndex > 0 ) ? ( VertexIndex - 1 ) : ( WorldSpacePolyVertices.Num() - 1 );
		const int32 NextWindingVertexIndex = ( VertexIndex < WorldSpacePolyVertices.Num() - 1 ) ? ( VertexIndex + 1 ) : 0;

		const FVector& PrevVertex = WorldSpacePolyVertices[ PrevWindingVertexIndex ];
		const FVector& CurVertex = WorldSpacePolyVertices[ VertexIndex ];
		const FVector& NextVertex = WorldSpacePolyVertices[ NextWindingVertexIndex ];

		// Compute the corner angle
		double AbsDotProduct = FMath::Abs( ( PrevVertex - CurVertex ).GetSafeNormal() | ( NextVertex - CurVertex ).GetSafeNormal() );

		// Compute how 'positive' this vertex is relative to the bottom left position in the polygon's plane
		FVector PolySpaceVertex = WorldToPolyRotationMatrix.InverseTransformVector( CurVertex - FirstPolyVertex );
		const double Positivity = PolySpaceVertex.X + PolySpaceVertex.Y;

		// Is the corner angle closer to 90 degrees than our current best?
		const double DotProductDiff = FMath::Abs( AbsDotProduct - DesiredAbsDotProduct );
		if( FMath::IsNearlyEqual( DotProductDiff, BestDotProductDiff, 0.1f ) )
		{
			// This angle is just as good as the current best, so check to see which is closer to the local space
			// bottom-left along the polygon's plane
			if( Positivity < BestPositivity )
			{
				// This vertex is in a more suitable location for the bottom-left of the texture
				BestVertexIndex = VertexIndex;
				if( DotProductDiff < BestDotProductDiff )
				{
					// Only store the new dot product if it's actually better than the existing one
					BestDotProductDiff = DotProductDiff;
				}
				BestPositivity = Positivity;
			}
		}
		else if( DotProductDiff <= BestDotProductDiff )
		{
			// This angle is definitely better!
			BestVertexIndex = VertexIndex;
			BestDotProductDiff = DotProductDiff;
			BestPositivity = Positivity;
		}
	}


	// Compute orthonormal basis for the 'best corner' of the polygon.  The texture will be positioned at the corner
	// of the bounds of the poly in this coordinate system
	const FVector& BestVertex = WorldSpacePolyVertices[ BestVertexIndex ];
	const int32 NextWindingVertexIndex = ( BestVertexIndex < WorldSpacePolyVertices.Num() - 1 ) ? ( BestVertexIndex + 1 ) : 0;
	const FVector& NextVertex = WorldSpacePolyVertices[ NextWindingVertexIndex ];

	FVector TextureUpVec = ( NextVertex - BestVertex ).GetSafeNormal();
	FVector TextureRightVec = (FVector)InPoly->Normal ^ TextureUpVec;

	FMatrix WorldToTextureRotationMatrix;
	WorldToTextureRotationMatrix.SetIdentity();
	FVector PolyNormal = (FVector)InPoly->Normal;
	WorldToTextureRotationMatrix.SetAxes( &TextureRightVec, &TextureUpVec, &PolyNormal );

	

	// Compute bounds of polygon along plane
	double MinX = std::numeric_limits<double>::max();
	double MaxX = std::numeric_limits<double>::min();
	double MinY = std::numeric_limits<double>::max();
	double MaxY = std::numeric_limits<double>::min();
	for( int32 VertexIndex = 0; VertexIndex < WorldSpacePolyVertices.Num(); ++VertexIndex )
	{
		const FVector& CurVertex = WorldSpacePolyVertices[ VertexIndex ];

		// Transform vertex into the coordinate system of our texture
		FVector TextureSpaceVertex = WorldToTextureRotationMatrix.InverseTransformVector( CurVertex - BestVertex );

		if( TextureSpaceVertex.X < MinX )
		{
			MinX = TextureSpaceVertex.X;
		}
		if( TextureSpaceVertex.X > MaxX )
		{
			MaxX = TextureSpaceVertex.X;
		}

		if( TextureSpaceVertex.Y < MinY )
		{
			MinY = TextureSpaceVertex.Y;
		}
		if( TextureSpaceVertex.Y > MaxY )
		{
			MaxY = TextureSpaceVertex.Y;
		}
	}


	// We'll use the texture space corner of the bounds as the origin of the texture.  This ensures that
	// the texture fits over the entire polygon without revealing any tiling
	const FVector TextureSpaceBasePos( MinX, MinY, 0.0f );
	FVector WorldSpaceBasePos = WorldToTextureRotationMatrix.TransformVector( TextureSpaceBasePos ) + BestVertex;


	// Apply scale to UV vectors.  We incorporate the parameterized tiling rations and scale by our texture size
	const float WorldTexelScale = UModel::GetGlobalBSPTexelScale();
	const double TextureSizeU = FMath::Abs( MaxX - MinX );
	const double TextureSizeV = FMath::Abs( MaxY - MinY );
	FVector TextureUVector = UTile * TextureRightVec * WorldTexelScale / TextureSizeU;
	FVector TextureVVector = VTile * TextureUpVec * WorldTexelScale / TextureSizeV;

	// Flip the texture vertically if we want that
	const bool bFlipVertically = true;
	if( bFlipVertically )
	{
		WorldSpaceBasePos += TextureUpVec * TextureSizeV;
		TextureVVector *= -1.0f;
	}


	// Apply texture base position
	{
		const bool bExactMatch = false;
		InSurfIdx->Surf->pBase = FBSPOps::bspAddPoint( InModel, const_cast< FVector* >( &WorldSpaceBasePos ), bExactMatch );
	}

	// Apply texture UV vectors
	{
		const bool bExactMatch = false;
		InSurfIdx->Surf->vTextureU = FBSPOps::bspAddVector( InModel, const_cast< FVector* >( &TextureUVector ), bExactMatch );
		InSurfIdx->Surf->vTextureV = FBSPOps::bspAddVector( InModel, const_cast< FVector* >( &TextureVVector ), bExactMatch );
	}
}



/*------------------------------------------------------------------------------
	FTexAlignTools.

	A helper class to store the state of the various texture alignment tools.
------------------------------------------------------------------------------*/

void FTexAlignTools::Init()
{
	//Never call Init more then once except if Release was call
	check(!bIsInit);
	// Create the list of aligners.
	Aligners.Empty();
	Aligners.Add(NewObject<UTexAlignerDefault>(GetTransientPackage(), NAME_None, RF_Public | RF_Standalone));
	Aligners.Add(NewObject<UTexAlignerPlanar>(GetTransientPackage(), NAME_None, RF_Public | RF_Standalone));
	Aligners.Add(NewObject<UTexAlignerBox>(GetTransientPackage(), NAME_None, RF_Public | RF_Standalone));
	Aligners.Add(NewObject<UTexAlignerFit>(GetTransientPackage(), NAME_None, RF_Public | RF_Standalone));
	for (UObject* Aligner : Aligners)
	{
		Aligner->AddToRoot();
	}
	FEditorDelegates::FitTextureToSurface.AddRaw(this, &FTexAlignTools::OnEditorFitTextureToSurface);
	bIsInit = true;
}

void FTexAlignTools::Release()
{
	if (bIsInit)
	{
		for (UObject* Aligner : Aligners)
		{
			Aligner->RemoveFromRoot();
		}
		Aligners.Empty();
		FEditorDelegates::FitTextureToSurface.RemoveAll(this);
	}
	bIsInit = false;
}

FTexAlignTools::FTexAlignTools()
{
	bIsInit = false;
}


FTexAlignTools::~FTexAlignTools()
{
	Release();
}

// Returns the most appropriate texture aligner based on the type passed in.
UTexAligner* FTexAlignTools::GetAligner( ETexAlign InTexAlign )
{
	switch( InTexAlign )
	{
		case TEXALIGN_Planar:
		case TEXALIGN_PlanarAuto:
		case TEXALIGN_PlanarWall:
		case TEXALIGN_PlanarFloor:
			return Aligners[1];

		case TEXALIGN_Default:
			return Aligners[0];

		case TEXALIGN_Box:
			return Aligners[2];

		case TEXALIGN_Fit:
			return Aligners[3];
	}

	check(0);	// Unknown type!
	return NULL;

}

void FTexAlignTools::OnEditorFitTextureToSurface(UWorld* InWorld)
{
	UTexAligner* FitAligner = GTexAlignTools.Aligners[ 3 ];
	for ( int32 LevelIndex = 0; LevelIndex < InWorld->GetNumLevels() ; ++LevelIndex )
	{
		ULevel* Level = InWorld->GetLevel(LevelIndex);
		FitAligner->Align( InWorld, TEXALIGN_None, Level->Model );
	}
}
