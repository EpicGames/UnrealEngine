// Copyright Epic Games, Inc. All Rights Reserved.

#include "Text/HLSLSyntaxHighlighterMarshaller.h"
#include "Text/HLSLSyntaxTokenizer.h"
#include "Framework/Text/IRun.h"
#include "Framework/Text/TextLayout.h"
#include "Framework/Text/ISlateRun.h"
#include "Framework/Text/SlateTextRun.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"

class FWhiteSpaceTextRun : public FSlateTextRun
{
public:
	static TSharedRef<FWhiteSpaceTextRun> Create(
		const FRunInfo& InRunInfo,
		const TSharedRef<const FString>& InText,
		const FTextBlockStyle& Style,
		const FTextRange& InRange,
		int32 NumSpacesPerTab)
	{
		return MakeShareable(new FWhiteSpaceTextRun(InRunInfo, InText, Style, InRange, NumSpacesPerTab));
	}

public:
	virtual FVector2D Measure(
		int32 StartIndex,
		int32 EndIndex,
		float Scale,
		const FRunTextContext& TextContext
		) const override
	{
		const FVector2D ShadowOffsetToApply((EndIndex == Range.EndIndex) ? FMath::Abs(Style.ShadowOffset.X * Scale) : 0.0f, FMath::Abs(Style.ShadowOffset.Y * Scale));

		if (EndIndex - StartIndex == 0)
		{
			return FVector2D(ShadowOffsetToApply.X * Scale, GetMaxHeight(Scale));
		}

		// count tabs
		int32 TabCount = 0;
		for (int32 Index = StartIndex; Index < EndIndex; Index++)
		{
			if ((*Text)[Index] == TEXT('\t'))
			{
				TabCount++;
			}
		}

		const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		FVector2D Size = FontMeasure->Measure(**Text, StartIndex, EndIndex, Style.Font, true, Scale) + ShadowOffsetToApply;

		Size.X -= TabWidth * TabCount * Scale;
		Size.X += SpaceWidth * TabCount * NumSpacesPerTab * Scale;

		return Size;
	}

protected:
	FWhiteSpaceTextRun(
		const FRunInfo& InRunInfo, 
		const TSharedRef<const FString>& InText, 
		const FTextBlockStyle& InStyle, 
		const FTextRange& InRange, 
		int32 InNumSpacesPerTab) : 
		FSlateTextRun(InRunInfo, InText, InStyle, InRange), 
		NumSpacesPerTab(InNumSpacesPerTab)
	{
		// measure tab width
		const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		TabWidth = FontMeasure->Measure(TEXT("\t"), 0, 1, Style.Font, true, 1.0f).X;
		SpaceWidth = FontMeasure->Measure(TEXT(" "), 0, 1, Style.Font, true, 1.0f).X;
	}

private:
	int32 NumSpacesPerTab;

	double TabWidth;

	double SpaceWidth;
};


TSharedRef<FHLSLSyntaxHighlighterMarshaller> FHLSLSyntaxHighlighterMarshaller::Create(const FSyntaxTextStyle& InSyntaxTextStyle)
{
	return MakeShareable(new FHLSLSyntaxHighlighterMarshaller(CreateTokenizer(), InSyntaxTextStyle));
}

TSharedPtr<ISyntaxTokenizer> FHLSLSyntaxHighlighterMarshaller::CreateTokenizer()
{

	return FHlslSyntaxTokenizer::Create();
}

void FHLSLSyntaxHighlighterMarshaller::ParseTokens(const FString& SourceString, FTextLayout& TargetTextLayout, TArray<ISyntaxTokenizer::FTokenizedLine> TokenizedLines)
{
	TArray<FTextLayout::FNewLineData> LinesToAdd;
	LinesToAdd.Reserve(TokenizedLines.Num());

	// Parse the tokens, generating the styled runs for each line
	int32 LineNo = 0;
	EParseState ParseState = EParseState::None;
	for(const ISyntaxTokenizer::FTokenizedLine& TokenizedLine : TokenizedLines)
	{
		LinesToAdd.Add(ProcessTokenizedLine(TokenizedLine, LineNo, SourceString, ParseState));
		LineNo++;
	}

	TargetTextLayout.AddLines(LinesToAdd);
}

FTextLayout::FNewLineData FHLSLSyntaxHighlighterMarshaller::ProcessTokenizedLine(const ISyntaxTokenizer::FTokenizedLine& TokenizedLine, const int32& LineNumber, const FString& SourceString, EParseState& ParseState)
{
	TSharedRef<FString> ModelString = MakeShareable(new FString());
	TArray< TSharedRef< IRun > > Runs;

	for(const ISyntaxTokenizer::FToken& Token : TokenizedLine.Tokens)
	{
		const FString TokenText = SourceString.Mid(Token.Range.BeginIndex, Token.Range.Len());

		const FTextRange ModelRange(ModelString->Len(), ModelString->Len() + TokenText.Len());
		ModelString->Append(TokenText);

		FRunInfo RunInfo(TEXT("SyntaxHighlight.HLSL.Normal"));
		FTextBlockStyle TextBlockStyle = SyntaxTextStyle.NormalTextStyle;

		const bool bIsWhitespace = FString(TokenText).TrimEnd().IsEmpty();
		if(!bIsWhitespace)
		{
			bool bHasMatchedSyntax = false;
			if(Token.Type == ISyntaxTokenizer::ETokenType::Syntax)
			{
				if(ParseState == EParseState::None && TokenText == TEXT("\""))
				{
					RunInfo.Name = TEXT("SyntaxHighlight.HLSL.String");
					TextBlockStyle = SyntaxTextStyle.StringTextStyle;
					ParseState = EParseState::LookingForString;
					bHasMatchedSyntax = true;
				}
				else if(ParseState == EParseState::LookingForString && TokenText == TEXT("\""))
				{
					RunInfo.Name = TEXT("SyntaxHighlight.HLSL.Normal");
					TextBlockStyle = SyntaxTextStyle.NormalTextStyle;
					ParseState = EParseState::None;
				}
				else if(ParseState == EParseState::None && TokenText == TEXT("\'"))
				{
					RunInfo.Name = TEXT("SyntaxHighlight.HLSL.String");
					TextBlockStyle = SyntaxTextStyle.StringTextStyle;
					ParseState = EParseState::LookingForCharacter;
					bHasMatchedSyntax = true;
				}
				else if(ParseState == EParseState::LookingForCharacter && TokenText == TEXT("\'"))
				{
					RunInfo.Name = TEXT("SyntaxHighlight.HLSL.Normal");
					TextBlockStyle = SyntaxTextStyle.NormalTextStyle;
					ParseState = EParseState::None;
				}
				else if(ParseState == EParseState::None && TokenText.StartsWith(TEXT("#")))
				{
					RunInfo.Name = TEXT("SyntaxHighlight.HLSL.PreProcessorKeyword");
					TextBlockStyle = SyntaxTextStyle.PreProcessorKeywordTextStyle;
					ParseState = EParseState::None;
				}
				else if(ParseState == EParseState::None && TokenText == TEXT("//"))
				{
					RunInfo.Name = TEXT("SyntaxHighlight.HLSL.Comment");
					TextBlockStyle = SyntaxTextStyle.CommentTextStyle;
					ParseState = EParseState::LookingForSingleLineComment;
				}
				else if(ParseState == EParseState::None && TokenText == TEXT("/*"))
				{
					RunInfo.Name = TEXT("SyntaxHighlight.HLSL.Comment");
					TextBlockStyle = SyntaxTextStyle.CommentTextStyle;
					ParseState = EParseState::LookingForMultiLineComment;
				}
				else if(ParseState == EParseState::LookingForMultiLineComment && TokenText == TEXT("*/"))
				{
					RunInfo.Name = TEXT("SyntaxHighlight.HLSL.Comment");
					TextBlockStyle = SyntaxTextStyle.CommentTextStyle;
					ParseState = EParseState::None;
				}
				else if(ParseState == EParseState::None && TChar<TCHAR>::IsIdentifier(TokenText[0]))
				{
					RunInfo.Name = TEXT("SyntaxHighlight.HLSL.Keyword");
					TextBlockStyle = SyntaxTextStyle.KeywordTextStyle;
					ParseState = EParseState::None;
				}
				else if(ParseState == EParseState::None && !TChar<TCHAR>::IsIdentifier(TokenText[0]))
				{
					RunInfo.Name = TEXT("SyntaxHighlight.HLSL.Operator");
					TextBlockStyle = SyntaxTextStyle.OperatorTextStyle;
					ParseState = EParseState::None;
				}
			}
			
			// It's possible that we fail to match a syntax token if we're in a state where it isn't parsed
			// In this case, we treat it as a literal token
			if(Token.Type == ISyntaxTokenizer::ETokenType::Literal || !bHasMatchedSyntax)
			{
				if(ParseState == EParseState::LookingForString)
				{
					RunInfo.Name = TEXT("SyntaxHighlight.HLSL.String");
					TextBlockStyle = SyntaxTextStyle.StringTextStyle;
				}
				else if(ParseState == EParseState::LookingForCharacter)
				{
					RunInfo.Name = TEXT("SyntaxHighlight.HLSL.String");
					TextBlockStyle = SyntaxTextStyle.StringTextStyle;
				}
				else if(ParseState == EParseState::LookingForSingleLineComment)
				{
					RunInfo.Name = TEXT("SyntaxHighlight.HLSL.Comment");
					TextBlockStyle = SyntaxTextStyle.CommentTextStyle;
				}
				else if(ParseState == EParseState::LookingForMultiLineComment)
				{
					RunInfo.Name = TEXT("SyntaxHighlight.HLSL.Comment");
					TextBlockStyle = SyntaxTextStyle.CommentTextStyle;
				}
			}

			TSharedRef< ISlateRun > Run = FSlateTextRun::Create(RunInfo, ModelString, TextBlockStyle, ModelRange);
			Runs.Add(Run);
		}
		else
		{
			RunInfo.Name = TEXT("SyntaxHighlight.HLSL.WhiteSpace");
			TSharedRef< ISlateRun > Run = FWhiteSpaceTextRun::Create(RunInfo, ModelString, TextBlockStyle, ModelRange, 4);
			Runs.Add(Run);
		}
	}

	if (ParseState != EParseState::LookingForMultiLineComment)
	{
		ParseState = EParseState::None;
	}
	return FTextLayout::FNewLineData(MoveTemp(ModelString), MoveTemp(Runs));
}

FHLSLSyntaxHighlighterMarshaller::FHLSLSyntaxHighlighterMarshaller(TSharedPtr<ISyntaxTokenizer> InTokenizer, const FSyntaxTextStyle& InSyntaxTextStyle) :
	FSyntaxHighlighterTextLayoutMarshaller(MoveTemp(InTokenizer))
	, SyntaxTextStyle(InSyntaxTextStyle)
{
}
