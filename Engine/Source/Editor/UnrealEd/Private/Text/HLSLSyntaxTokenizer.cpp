// Copyright Epic Games, Inc. All Rights Reserved.

#include "Text/HLSLSyntaxTokenizer.h"

// NOTE: Since SyntaxTokenizer matches on a first-token-encountered basis, it's important that
// tokens with the same prefix are ordered by longest-prefix-first. Ideally SyntaxTokenizer 
// should be using a prefix tree structure for longest prefix matching.

// Type Keywords are copied from CrossCompiler::EHlslToken
const TCHAR* HlslKeywords[] =
{
	TEXT("while"),
	TEXT("volatile"),
	TEXT("void"),
	TEXT("vector"),
	TEXT("unorm"),
	TEXT("uniform"),
	TEXT("uint4x4"),
	TEXT("uint4x3"),
	TEXT("uint4x2"),
	TEXT("uint4x1"),
	TEXT("uint4"),
	TEXT("uint3x4"),
	TEXT("uint3x3"),
	TEXT("uint3x2"),
	TEXT("uint3x1"),
	TEXT("uint3"),
	TEXT("uint2x4"),
	TEXT("uint2x3"),
	TEXT("uint2x2"),
	TEXT("uint2x1"),
	TEXT("uint2"),
	TEXT("uint1x4"),
	TEXT("uint1x3"),
	TEXT("uint1x2"),
	TEXT("uint1x1"),
	TEXT("uint1"),
	TEXT("uint"),
	TEXT("true"),
	TEXT("switch"),
	TEXT("struct"),
	TEXT("static"),
	TEXT("snorm"),
	TEXT("shared"),
	TEXT("row_major"),
	TEXT("return"),
	TEXT("register"),
	TEXT("precise"),
	TEXT("packoffset"),
	TEXT("numthreads"),
	TEXT("nointerpolation"),
	TEXT("namespace"),
	TEXT("matrix"),
	TEXT("int4x4"),
	TEXT("int4x3"),
	TEXT("int4x2"),
	TEXT("int4x1"),
	TEXT("int4"),
	TEXT("int3x4"),
	TEXT("int3x3"),
	TEXT("int3x2"),
	TEXT("int3x1"),
	TEXT("int3"),
	TEXT("int2x4"),
	TEXT("int2x3"),
	TEXT("int2x2"),
	TEXT("int2x1"),
	TEXT("int2"),
	TEXT("int1x4"),
	TEXT("int1x3"),
	TEXT("int1x2"),
	TEXT("int1x1"),
	TEXT("int1"),
	TEXT("int"),
	TEXT("if"),
	TEXT("half4x4"),
	TEXT("half4x3"),
	TEXT("half4x2"),
	TEXT("half4x1"),
	TEXT("half4"),
	TEXT("half3x4"),
	TEXT("half3x3"),
	TEXT("half3x2"),
	TEXT("half3x1"),
	TEXT("half3"),
	TEXT("half2x4"),
	TEXT("half2x3"),
	TEXT("half2x2"),
	TEXT("half2x1"),
	TEXT("half2"),
	TEXT("half1x4"),
	TEXT("half1x3"),
	TEXT("half1x2"),
	TEXT("half1x1"),
	TEXT("half1"),
	TEXT("half"),
	TEXT("groupshared"),
	TEXT("goto"),
	TEXT("for"),
	TEXT("float4x4"),
	TEXT("float4x3"),
	TEXT("float4x2"),
	TEXT("float4x1"),
	TEXT("float4"),
	TEXT("float3x4"),
	TEXT("float3x3"),
	TEXT("float3x2"),
	TEXT("float3x1"),
	TEXT("float3"),
	TEXT("float2x4"),
	TEXT("float2x3"),
	TEXT("float2x2"),
	TEXT("float2x1"),
	TEXT("float2"),
	TEXT("float1x4"),
	TEXT("float1x3"),
	TEXT("float1x2"),
	TEXT("float1x1"),
	TEXT("float1"),
	TEXT("float"),
	TEXT("false"),
	TEXT("extern"),
	TEXT("export"),
	TEXT("enum"),
	TEXT("else"),
	TEXT("dword"),
	TEXT("double"),
	TEXT("do"),
	TEXT("default"),
	TEXT("continue"),
	TEXT("const"),
	TEXT("column_major"),
	TEXT("case"),
	TEXT("break"),
	TEXT("bool4x4"),
	TEXT("bool4x3"),
	TEXT("bool4x2"),
	TEXT("bool4x1"),
	TEXT("bool4"),
	TEXT("bool3x4"),
	TEXT("bool3x3"),
	TEXT("bool3x2"),
	TEXT("bool3x1"),
	TEXT("bool3"),
	TEXT("bool2x4"),
	TEXT("bool2x3"),
	TEXT("bool2x2"),
	TEXT("bool2x1"),
	TEXT("bool2"),
	TEXT("bool1x4"),
	TEXT("bool1x3"),
	TEXT("bool1x2"),
	TEXT("bool1x1"),
	TEXT("bool1"),
	TEXT("bool"),
	TEXT("Buffer"),
	TEXT("in"),
	TEXT("out"),
	TEXT("inout"),
};

const TCHAR* HlslOperators[] =
{
	TEXT("/*"),
	TEXT("*/"),
	TEXT("//"),
	TEXT("\""),
	TEXT("\'"),
	TEXT("::"),
	TEXT(":"),
	TEXT("+="),
	TEXT("++"),
	TEXT("+"),
	TEXT("--"),
	TEXT("-="),
	TEXT("-"),
	TEXT("("),
	TEXT(")"),
	TEXT("["),
	TEXT("]"),
	TEXT("."),
	TEXT("->"),
	TEXT("!="),
	TEXT("!"),
	TEXT("&="),
	TEXT("~"),
	TEXT("&"),
	TEXT("*="),
	TEXT("*"),
	TEXT("->"),
	TEXT("/="),
	TEXT("/"),
	TEXT("%="),
	TEXT("%"),
	TEXT("<<="),
	TEXT("<<"),
	TEXT("<="),
	TEXT("<"),
	TEXT(">>="),
	TEXT(">>"),
	TEXT(">="),
	TEXT(">"),
	TEXT("=="),
	TEXT("&&"),
	TEXT("&"),
	TEXT("^="),
	TEXT("^"),
	TEXT("|="),
	TEXT("||"),
	TEXT("|"),
	TEXT("?"),
	TEXT("="),
};

const TCHAR* HlslPreProcessorKeywords[] =
{
	TEXT("#include"),
	TEXT("#define"),
	TEXT("#ifndef"),
	TEXT("#ifdef"),
	TEXT("#if"),
	TEXT("#else"),
	TEXT("#endif"),
	TEXT("#pragma"),
	TEXT("#undef"),
};

const TCHAR* HlslSymbols[] =
{
	TEXT("abort"),
	TEXT("abs"),
	TEXT("acos"),
	TEXT("all"),
	TEXT("AllMemoryBarrier"),
	TEXT("AllMemoryBarrierWithGroupSync"),
	TEXT("any"),
	TEXT("asdouble"),
	TEXT("asfloat"),
	TEXT("asin"),
	TEXT("asint"),
	TEXT("asuint"),
	TEXT("asuint"),
	TEXT("atan"),
	TEXT("atan2"),
	TEXT("ceil"),
	TEXT("CheckAccessFullyMapped"),
	TEXT("clamp"),
	TEXT("clip"),
	TEXT("cos"),
	TEXT("cosh"),
	TEXT("countbits"),
	TEXT("cross"),
	TEXT("D3DCOLORtoUBYTE4"),
	TEXT("ddx"),
	TEXT("ddx_coarse"),
	TEXT("ddx_fine"),
	TEXT("ddy"),
	TEXT("ddy_coarse"),
	TEXT("ddy_fine"),
	TEXT("degrees"),
	TEXT("determinant"),
	TEXT("DeviceMemoryBarrier"),
	TEXT("DeviceMemoryBarrierWithGroupSync"),
	TEXT("distance"),
	TEXT("dot"),
	TEXT("dst"),
	TEXT("errorf"),
	TEXT("EvaluateAttributeCentroid"),
	TEXT("EvaluateAttributeAtSample"),
	TEXT("EvaluateAttributeSnapped"),
	TEXT("exp"),
	TEXT("exp2"),
	TEXT("f16tof32"),
	TEXT("f32tof16"),
	TEXT("faceforward"),
	TEXT("firstbithigh"),
	TEXT("firstbitlow"),
	TEXT("floor"),
	TEXT("fma"),
	TEXT("fmod"),
	TEXT("frac"),
	TEXT("frexp"),
	TEXT("fwidth"),
	TEXT("GetRenderTargetSampleCount"),
	TEXT("GetRenderTargetSamplePosition"),
	TEXT("GroupMemoryBarrier"),
	TEXT("GroupMemoryBarrierWithGroupSync"),
	TEXT("InterlockedAdd"),
	TEXT("InterlockedAnd"),
	TEXT("InterlockedCompareExchange"),
	TEXT("InterlockedCompareStore"),
	TEXT("InterlockedExchange"),
	TEXT("InterlockedMax"),
	TEXT("InterlockedMin"),
	TEXT("InterlockedOr"),
	TEXT("InterlockedXor"),
	TEXT("isfinite"),
	TEXT("isinf"),
	TEXT("isnan"),
	TEXT("ldexp"),
	TEXT("length"),
	TEXT("lerp"),
	TEXT("lit"),
	TEXT("log"),
	TEXT("log10"),
	TEXT("log2"),
	TEXT("mad"),
	TEXT("max"),
	TEXT("min"),
	TEXT("modf"),
	TEXT("msad4"),
	TEXT("mul"),
	TEXT("noise"),
	TEXT("normalize"),
	TEXT("pow"),
	TEXT("printf"),
	TEXT("Process2DQuadTessFactorsAvg"),
	TEXT("Process2DQuadTessFactorsMax"),
	TEXT("Process2DQuadTessFactorsMin"),
	TEXT("ProcessIsolineTessFactors"),
	TEXT("ProcessQuadTessFactorsAvg"),
	TEXT("ProcessQuadTessFactorsMax"),
	TEXT("ProcessQuadTessFactorsMin"),
	TEXT("ProcessTriTessFactorsAvg"),
	TEXT("ProcessTriTessFactorsMax"),
	TEXT("ProcessTriTessFactorsMin"),
	TEXT("radians"),
	TEXT("rcp"),
	TEXT("reflect"),
	TEXT("refract"),
	TEXT("reversebits"),
	TEXT("round"),
	TEXT("rsqrt"),
	TEXT("saturate"),
	TEXT("sign"),
	TEXT("sin"),
	TEXT("sincos"),
	TEXT("sinh"),
	TEXT("smoothstep"),
	TEXT("sqrt"),
	TEXT("step"),
	TEXT("tan"),
	TEXT("tanh"),
	TEXT("tex1D"),
	TEXT("tex1D"),
	TEXT("tex1Dbias"),
	TEXT("tex1Dgrad"),
	TEXT("tex1Dlod"),
	TEXT("tex1Dproj"),
	TEXT("tex2D"),
	TEXT("tex2D"),
	TEXT("tex2Dbias"),
	TEXT("tex2Dgrad"),
	TEXT("tex2Dlod"),
	TEXT("tex2Dproj"),
	TEXT("tex3D"),
	TEXT("tex3D"),
	TEXT("tex3Dbias"),
	TEXT("tex3Dgrad"),
	TEXT("tex3Dlod"),
	TEXT("tex3Dproj"),
	TEXT("texCUBE"),
	TEXT("texCUBE"),
	TEXT("texCUBEbias"),
	TEXT("texCUBEgrad"),
	TEXT("texCUBElod"),
	TEXT("texCUBEproj"),
	TEXT("transpose"),
	TEXT("trunc"),
	TEXT("SV_ClipDistance"),
	TEXT("SV_CullDistance"),
	TEXT("SV_Coverage"),
	TEXT("SV_Depth"),
	TEXT("SV_DepthGreaterEqual"),
	TEXT("SV_DepthLessEqual"),
	TEXT("SV_DispatchThreadID"),
	TEXT("SV_DomainLocation"),
	TEXT("SV_GroupID"),
	TEXT("SV_GroupIndex"),
	TEXT("SV_GroupThreadID"),
	TEXT("SV_GSInstanceID"),
	TEXT("SV_InnerCoverage"),
	TEXT("SV_InsideTessFactor"),
	TEXT("SV_InstanceID"),
	TEXT("SV_IsFrontFace"),
	TEXT("SV_OutputControlPointID"),
	TEXT("SV_Position"),
	TEXT("SV_PrimitiveID"),
	TEXT("SV_RenderTargetArrayIndex"),
	TEXT("SV_SampleIndex"),
	TEXT("SV_StencilRef"),
	TEXT("SV_Target"),
	TEXT("SV_TessFactor"),
	TEXT("SV_VertexID"),
	TEXT("SV_ViewportArrayIndex"),
	TEXT("SV_ShadingRate"),
};

TSharedRef<FHlslSyntaxTokenizer> FHlslSyntaxTokenizer::Create()
{
	return MakeShareable(new FHlslSyntaxTokenizer());
}

void FHlslSyntaxTokenizer::Process(TArray<FTokenizedLine>& OutTokenizedLines, const FString& Input)
{
#if UE_ENABLE_ICU
	TArray<FTextRange> LineRanges;
	FTextRange::CalculateLineRangesFromString(Input, LineRanges);
	TokenizeLineRanges(Input, LineRanges, OutTokenizedLines);
#else
	FTokenizedLine FakeTokenizedLine;
	FakeTokenizedLine.Range = FTextRange(0, Input.Len());
	FakeTokenizedLine.Tokens.Emplace(FToken(ETokenType::Literal, FakeTokenizedLine.Range));
	OutTokenizedLines.Add(FakeTokenizedLine);
#endif
}

FHlslSyntaxTokenizer::FHlslSyntaxTokenizer()
{
	// operators
	for(const auto& Operator : HlslOperators)
	{
		Operators.Emplace(Operator);
	}	

	// keywords
	for(const auto& Keyword : HlslKeywords)
	{
		Keywords.Emplace(Keyword);
	}

	// Pre-processor Keywords
	for(const auto& PreProcessorKeyword : HlslPreProcessorKeywords)
	{
		Keywords.Emplace(PreProcessorKeyword);
	}
	
	// Symbols
	for (const auto& Function : HlslSymbols)
	{
		Keywords.Emplace(Function);
	}
}

void FHlslSyntaxTokenizer::TokenizeLineRanges(const FString& Input, const TArray<FTextRange>& LineRanges, TArray<FTokenizedLine>& OutTokenizedLines)
{
	// Tokenize line ranges
	for(const FTextRange& LineRange : LineRanges)
	{
		FTokenizedLine TokenizedLine;
		TokenizedLine.Range = LineRange;

		if(TokenizedLine.Range.IsEmpty())
		{
			TokenizedLine.Tokens.Emplace(FToken(ETokenType::Literal, TokenizedLine.Range));
		}
		else
		{
			int32 CurrentOffset = LineRange.BeginIndex;
			
			while(CurrentOffset < LineRange.EndIndex)
			{
				const TCHAR* CurrentString = &Input[CurrentOffset];
				const TCHAR CurrentChar = Input[CurrentOffset];

				bool bHasMatchedSyntax = false;

				// Greedy matching for operators
				for(const FString& Operator : Operators)
				{
					if(FCString::Strncmp(CurrentString, *Operator, Operator.Len()) == 0)
					{
						const int32 SyntaxTokenEnd = CurrentOffset + Operator.Len();
						TokenizedLine.Tokens.Emplace(FToken(ETokenType::Syntax, FTextRange(CurrentOffset, SyntaxTokenEnd)));
					
						check(SyntaxTokenEnd <= LineRange.EndIndex);
					
						bHasMatchedSyntax = true;
						CurrentOffset = SyntaxTokenEnd;
						break;
					}
				}
			
				if(bHasMatchedSyntax)
				{
					continue;
				}

				int32 PeekOffset = CurrentOffset + 1;
				if (CurrentChar == TEXT('#'))
				{
					// Match PreProcessorKeywords
					// They only contain letters
					while(PeekOffset < LineRange.EndIndex)
					{
						const TCHAR PeekChar = Input[PeekOffset];

						if (!TChar<TCHAR>::IsAlpha(PeekChar))
						{
							break;
						}
						
						PeekOffset++;
					}
				}
				else if (TChar<TCHAR>::IsAlpha(CurrentChar))
				{
					// Match Identifiers,
					// They start with a letter and contain
					// letters or numbers
					while(PeekOffset < LineRange.EndIndex)
					{
						const TCHAR PeekChar = Input[PeekOffset];

						if (!TChar<TCHAR>::IsIdentifier(PeekChar))
						{
							break;
						}
						
						PeekOffset++;
					}
				}

				const int32 CurrentStringLength = PeekOffset - CurrentOffset;
				
				// Check if it is an reserved keyword
				for(const FString& Keyword : Keywords)
				{
					if (FCString::Strncmp(CurrentString, *Keyword, FMath::Max(CurrentStringLength, Keyword.Len())) == 0)
					{
						const int32 SyntaxTokenEnd = CurrentOffset + CurrentStringLength;
						TokenizedLine.Tokens.Emplace(FToken(ETokenType::Syntax, FTextRange(CurrentOffset, SyntaxTokenEnd)));
					
						check(SyntaxTokenEnd <= LineRange.EndIndex);
					
						bHasMatchedSyntax = true;
						CurrentOffset = SyntaxTokenEnd;
						break;
					}
				}

				if (bHasMatchedSyntax)
				{
					continue;
				}

				// If none matched, consume the character(s) as text
				const int32 TextTokenEnd = CurrentOffset + CurrentStringLength;
				TokenizedLine.Tokens.Emplace(FToken(ETokenType::Literal, FTextRange(CurrentOffset, TextTokenEnd)));
				CurrentOffset = TextTokenEnd;
			}
		}

		OutTokenizedLines.Add(TokenizedLine);
	}
}
