// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	Texture2DPreview.h: Implementation for previewing 2D Textures and normal maps.
==============================================================================*/

#include "Texture2DPreview.h"
#include "Shader.h"
#include "GlobalRenderResources.h"
#include "GlobalShader.h"
#include "SimpleElementShaders.h"
#include "ShaderParameterUtils.h"
#include "PipelineStateCache.h"
#include "TextureResource.h"
#include "RHIStaticStates.h"
#include "RenderCore.h"
#include "VirtualTexturing.h"
#include "Engine/Texture2DArray.h"
#include "ShaderParameterStruct.h"
#include "DataDrivenShaderPlatformInfo.h"

/*------------------------------------------------------------------------------
	Batched element shaders for previewing 2d textures.
------------------------------------------------------------------------------*/

class FSimpleElementTexture2DPreviewPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FSimpleElementTexture2DPreviewPS);
	SHADER_USE_PARAMETER_STRUCT(FSimpleElementTexture2DPreviewPS, FGlobalShader);

	class FVirtualTextureDim : SHADER_PERMUTATION_BOOL("SAMPLE_VIRTUAL_TEXTURE");
	class FTexture2DArrayDim : SHADER_PERMUTATION_BOOL("TEXTURE_ARRAY");

	using FPermutationDomain = TShaderPermutationDomain<FVirtualTextureDim, FTexture2DArrayDim>;

	/** Should the shader be cached? Always. */
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (!IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5))
		{
			return false;
		}

		if (IsConsolePlatform(Parameters.Platform))
		{
			return false;
		}

		FPermutationDomain PermutationVector(Parameters.PermutationId);

		// Both SAMPLE_VIRTUAL_TEXTURE and TEXTURE_ARRAY can't be set at the same time
		if (PermutationVector.Get<FVirtualTextureDim>() && PermutationVector.Get<FTexture2DArrayDim>())
		{
			return false;
		}

		return true;
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_ARRAY(FUintVector4, VTPackedPageTableUniform, [2])
		SHADER_PARAMETER(FUintVector4, VTPackedUniform)

		SHADER_PARAMETER_EX(FVector4f, TextureComponentReplicate, EShaderPrecisionModifier::Half)
		SHADER_PARAMETER_EX(FVector4f, TextureComponentReplicateAlpha, EShaderPrecisionModifier::Half)
		SHADER_PARAMETER(FMatrix44f, ColorWeights)
		SHADER_PARAMETER(FVector4f, PackedParams)

		SHADER_PARAMETER(float, NumSlices)
		SHADER_PARAMETER(float, SliceIndex)

		SHADER_PARAMETER_SRV(Texture2D, InPhysicalTexture)
		SHADER_PARAMETER_TEXTURE(Texture2D, InTexture)
		SHADER_PARAMETER_TEXTURE(Texture2DArray, InTextureArray)
		SHADER_PARAMETER_SAMPLER(SamplerState, InTextureSampler)
		SHADER_PARAMETER_TEXTURE(Texture2D, InPageTableTexture0)
		SHADER_PARAMETER_TEXTURE(Texture2D, InPageTableTexture1)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FSimpleElementTexture2DPreviewPS, "/Engine/Private/SimpleElementTexture2DPreviewPixelShader.usf", "Main", SF_Pixel);

/** Binds vertex and pixel shaders for this element */
void FBatchedElementTexture2DPreviewParameters::BindShaders(
	FRHICommandList& RHICmdList,
	FGraphicsPipelineStateInitializer& GraphicsPSOInit,
	ERHIFeatureLevel::Type InFeatureLevel,
	const FMatrix& InTransform,
	const float InGamma,
	const FMatrix& ColorWeights,
	const FTexture* Texture)
{
	TShaderMapRef<FSimpleElementVS> VertexShader(GetGlobalShaderMap(InFeatureLevel));

	FSimpleElementTexture2DPreviewPS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FSimpleElementTexture2DPreviewPS::FVirtualTextureDim>(bIsVirtualTexture);
	PermutationVector.Set<FSimpleElementTexture2DPreviewPS::FTexture2DArrayDim>(bIsTextureArray);
	TShaderMapRef<FSimpleElementTexture2DPreviewPS> PixelShader(GetGlobalShaderMap(InFeatureLevel), PermutationVector);

	FSimpleElementTexture2DPreviewPS::FParameters Parameters{};
	{
		if (bIsVirtualTexture)
		{
			FVirtualTexture2DResource* VirtualTextureValue = (FVirtualTexture2DResource*)Texture;
			IAllocatedVirtualTexture* AllocatedVT = VirtualTextureValue->AcquireAllocatedVT();

			Parameters.InPhysicalTexture = AllocatedVT->GetPhysicalTextureSRV((uint32)LayerIndex, Texture->bSRGB);
			Parameters.InTextureSampler = VirtualTextureValue->SamplerStateRHI.GetReference();

			Parameters.InPageTableTexture0 = AllocatedVT->GetPageTableTexture(0u);
			Parameters.InPageTableTexture1 = AllocatedVT->GetNumPageTableTextures() > 1u ? AllocatedVT->GetPageTableTexture(1u) : GBlackTexture->TextureRHI.GetReference();

			FUintVector4 VTPackedPageTableUniform[2];
			FUintVector4 VTPackedUniform;

			AllocatedVT->GetPackedPageTableUniform(VTPackedPageTableUniform);
			AllocatedVT->GetPackedUniform(&VTPackedUniform, (uint32)LayerIndex);

			Parameters.VTPackedPageTableUniform[0] = VTPackedPageTableUniform[0];
			Parameters.VTPackedPageTableUniform[1] = VTPackedPageTableUniform[1];
			Parameters.VTPackedUniform = VTPackedUniform;
		}
		else if (Texture != nullptr)
		{
			if (bIsTextureArray)
			{
				Parameters.InTextureArray = Texture->TextureRHI;
			}
			else
			{
				Parameters.InTexture = Texture->TextureRHI;
			}
			Parameters.InTextureSampler = Texture->SamplerStateRHI.GetReference();
		}

		if (bUsePointSampling)
		{
			Parameters.InTextureSampler = TStaticSamplerState<SF_Point>::GetRHI();
		}

		Parameters.ColorWeights = FMatrix44f(ColorWeights);
		Parameters.PackedParams = FVector4f(InGamma, MipLevel, bIsNormalMap ? 1.0f : -1.0f, bIsSingleVTPhysicalSpace ? 0 : LayerIndex);

		// Store slice count and selected slice index for texture array
		if (bIsTextureArray)
		{
			Parameters.NumSlices = Texture != nullptr ? static_cast<float>(Texture->GetSizeZ()) : 1.0f;
			Parameters.SliceIndex = SliceIndex;
		}

		Parameters.TextureComponentReplicate = (Texture && Texture->bGreyScaleFormat) ? FLinearColor(1, 0, 0, 0) : FLinearColor(0, 0, 0, 0);
		Parameters.TextureComponentReplicateAlpha = (Texture && Texture->bGreyScaleFormat) ? FLinearColor(1, 0, 0, 0) : FLinearColor(0, 0, 0, 1);
	}

	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GSimpleElementVertexDeclaration.VertexDeclarationRHI;
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;

	if (bIsSingleChannelFormat)
	{
		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
	}

	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

	SetShaderParametersLegacyVS(RHICmdList, VertexShader, InTransform);

	SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), Parameters);
}
