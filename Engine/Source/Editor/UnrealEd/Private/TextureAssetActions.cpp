// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextureAssetActions.h"

#include "ContentBrowserMenuContexts.h"
#include "Styling/AppStyle.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"

#include "SlateFwd.h"
#include "UObject/Object.h"
#include "Layout/Visibility.h"
#include "Editor.h"
#include "Misc/FeedbackContext.h"
#include "Framework/Application/SlateApplication.h"
#include "IAssetTools.h"
#include "AssetRegistry/AssetRegistryHelpers.h"

#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SNumericEntryBox.h"

#include "Engine/Texture.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "TextureSourceDataUtils.h"
#include "TextureImportSettings.h"
#include "ImageCoreUtils.h"
#include "Styling/SlateIconFinder.h"
#include "TextureAssetActionsBP.h"

#define LOCTEXT_NAMESPACE "TextureAssetActions"

namespace UE
{
namespace TextureAssetActionsInternal
{

enum class EAssetActionStatus
{
	Enabled = 0,
	NoSource,
	UnderSized,
	WrongType,
	HasLayers,
	HasMipsLeaveExisting,
	Already8bit,
	DontChangeJPEG,
	MustBe8BitForJPEG,
	AlreadyPow2
};

/**
* STextureActionDlg
*
* This class creates and launches a dialog then awaits the result to return to the user.
*/
class STextureActionDlg
{
public:
	enum EResult
	{
		Cancel = 0,			// No/Cancel, normal usage would stop the current action
		Confirm = 1,		// Yes/Ok/Etc, normal usage would continue with action
	};

	STextureActionDlg(const TArray<UTexture*>& Textures, ETextureAction Act);

	/**  Shows the dialog box and waits for the user to respond. */
	EResult ShowModal();

private:
	TSharedPtr<SWindow> DialogWindow;
	TSharedPtr<class STextureAssetList> DialogWidget;
};

class STextureAssetList : public SCompoundWidget
{
public:
	
public:

	SLATE_BEGIN_ARGS(STextureAssetList)
	{}
	/** Window in which this widget resides */
	SLATE_ATTRIBUTE(TSharedPtr<SWindow>, ParentWindow)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	void Init(const TArray<UTexture *> &Textures, ETextureAction Act);

	STextureActionDlg::EResult GetUserResponse() const;

private:

	void UpdateList();
	void DoAction();

	/**
	* Creates a single line showing an asset and it's status related to VT conversion
	*/
	TSharedRef<SWidget> CreateAssetLine(int index, const UTexture * Texture, const EAssetActionStatus Status,
										const FString & EnabledActionString,
										const FString & DescriptionString);

	FReply OnButtonClick(STextureActionDlg::EResult ButtonID);

	void OnThresholdChanged(TSharedPtr<int32> InSelectedItem, ESelectInfo::Type SelectInfo);

	FText GetThresholdText() const;

	TSharedRef<SWidget> OnGenerateThresholdWidget(TSharedPtr<int32> InItem);

	FReply OnExpanderClicked(int index);

	bool GetOkButtonEnabled() const;
	
	EVisibility GetDetailVisibility(int index) const;

	const FSlateBrush* GetExpanderImage(int index) const;

	EVisibility GetIntroMessageVisibility() const;

	EVisibility GetErrorMessageVisibility() const;

	EVisibility GetThresholdVisibility() const
	{
		return ( Action == ETextureAction::Resize || Action == ETextureAction::ResizePow2 || Action == ETextureAction::JPEG ) ? EVisibility::Visible : EVisibility::Collapsed;
	}

	EVisibility GetNMK16Visibility() const
	{
		return ( Action == ETextureAction::ConvertTo8bit ) ? EVisibility::Visible : EVisibility::Hidden;
	}
	ECheckBoxState HandleNMK16CheckBoxIsChecked() const
	{
		return bNormalMapsKeep16bits ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	void HandleNMK16CheckBoxCheckedStateChanged(ECheckBoxState InNewState)
	{
		bNormalMapsKeep16bits = ( InNewState == ECheckBoxState::Checked );
		UpdateList();
	}
	
	ECheckBoxState HandleShowExcludedCheckBoxIsChecked() const
	{
		return bShowExcluded ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	void HandleShowExcludedCheckBoxCheckedStateChanged(ECheckBoxState InNewState)
	{
		bShowExcluded = ( InNewState == ECheckBoxState::Checked );
		UpdateList();
	}

	FText GetIntroMessage() const;

	FText GetErrorMessage() const;
	
	TArray<int> ExpandedIndexes;

	FText IntroMessage;
	FText ErrorMessage;
	STextureActionDlg::EResult	 UserResponse = STextureActionDlg::EResult::Cancel;

	TSharedPtr<SVerticalBox>	 AssetListContainer;
	//TSharedPtr<STextBlock>		 MessageTextBlock;
	//TSharedPtr<SHorizontalBox>	 ThresholdContainer;
	//TSharedPtr<SHorizontalBox>	 ErrorContainer;
	//TSharedPtr<SButton>			 OkButton;

	/** Pointer to the window which holds this Widget, required for modal control */
	TSharedPtr<SWindow>			 ParentWindow;

	struct TextureListEntry
	{
		UTexture * Texture;
		bool IsEnabled;
	};

	TArray<TextureListEntry> TextureList;

	int ThresholdValue = 0;
	bool bNormalMapsKeep16bits = true;
	bool bShowExcluded = true;

	TArray<TSharedPtr<int32>> TextureSizes;

	ETextureAction Action = ETextureAction::Invalid;
};


static FText TAA_DialogTitle(ETextureAction Act)
{
	switch(Act)
	{
	case ETextureAction::Resize: return LOCTEXT("TAA_Title_Resize", "Texture Asset : Resize Source");
	case ETextureAction::ResizePow2: return LOCTEXT("TAA_Title_ResizePow2", "Texture Asset : Resize Source To Power of Two");
	case ETextureAction::ConvertTo8bit: return LOCTEXT("TAA_Title_Convert", "Texture Asset : Convert To 8 bit or minimum viable bit depth");
	case ETextureAction::JPEG: return LOCTEXT("TAA_Title_JPEG", "Texture Asset : Compress with JPEG");
	default: check(0); return FText();
	}
}

static FText TAA_Intro(ETextureAction Act)
{
	switch(Act)
	{
	case ETextureAction::Resize: return LOCTEXT("TAA_Intro_Resize", "Reduce size of Texture Source to compact uassets.  Resizing is done using mip filter, in power of two steps.  LODBias is adjusted but platform built size may change.");
	case ETextureAction::ResizePow2: return LOCTEXT("TAA_Intro_ResizePow2", "Change Texture Source dimensions to the nearest power or two.  May shrink or enlarge depending on which is closer.  Only textures larger than threshold are changed.  Does not resize to be <= threshold size, that's just a selection filter.");
	case ETextureAction::ConvertTo8bit: return LOCTEXT("TAA_Intro_Convert", "Convert Texture Source to 8 bit, Normals to 8 or 16, HDR to 16F.  Only converts if bits per pixel goes down.  Output built texture may change.  Make sure CompressionSetting and SRGB are set correctly first!");
	case ETextureAction::JPEG: return LOCTEXT("TAA_Intro_JPEG", "Compress Texture Source with JPEG.  Only works on 8 bit, 2D simple textures.  Greatly reduces uasset size with some loss of quality.  Does not affect in-game size.");
	default: check(0); return FText();
	}
}

void STextureAssetList::Construct(const FArguments& InArgs)
{
	// BEWARE : Action is not set yet (it comes in Init)
	//	cannot do per-Action setup here

	UserResponse = STextureActionDlg::Cancel;
	ParentWindow = InArgs._ParentWindow.Get();
	static FName ErrorIcon = "MessageLog.Error";

	for (int i = 0; i < 16; i++)
	{
		TextureSizes.Add(MakeShareable(new int32(1 << i)));
	}
	ThresholdValue = *TextureSizes[10]; // 1024

	this->ChildSlot[
		SNew(SVerticalBox)
		// Textbox at the top giving an introductory message
		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(8.0f, 4.0f, 8.0f, 4.0f)
			[
				SNew(STextBlock)
				.AutoWrapText(true)
				.Visibility(this, &STextureAssetList::GetIntroMessageVisibility)
				.Text(this, &STextureAssetList::GetIntroMessage)
			]
		// Error message at the top giving a common error message
		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(8.0f, 4.0f, 8.0f, 4.0f)
			[
				SNew(SHorizontalBox)
				.Visibility(this, &STextureAssetList::GetErrorMessageVisibility)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush(ErrorIcon))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.AutoWrapText(true)
					.Text(this, &STextureAssetList::GetErrorMessage)
				]
			]
		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(8.0f, 4.0f, 8.0f, 4.0f)
			[
				SNew(SSeparator)
			]
		// The actual list of assets
		+ SVerticalBox::Slot()
			.Padding(8.0f, 4.0f, 8.0f, 4.0f)
			[
				SNew(SBorder)
				[
					SNew(SScrollBox)
					+ SScrollBox::Slot()
					[
						SAssignNew(AssetListContainer, SVerticalBox)
					]
				]
			]
		// The bottom row of widgets: texture size selector
		+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Left)
			.Padding(8.0f, 4.0f, 8.0f, 4.0f)
			[
				SNew(SHorizontalBox)
				.Visibility(this, &STextureAssetList::GetThresholdVisibility)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("TAA_Size", "Texture size threshold: "))
					.AutoWrapText(true)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SComboBox<TSharedPtr<int32>>)
					.OptionsSource(&TextureSizes)
					.OnSelectionChanged(this, &STextureAssetList::OnThresholdChanged)
					.OnGenerateWidget(this, &STextureAssetList::OnGenerateThresholdWidget)
					.InitiallySelectedItem(TextureSizes[10])
					[
						SNew(STextBlock)
						.Text(this, &STextureAssetList::GetThresholdText)
					]
				]
				/*
				// cannot do Action-dependent setup here :(
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text( Action == ETextureAction::JPEG ? 
						LOCTEXT("TAA_Threshold_JPEG", "included if pixel count >= threshold^2") :
						LOCTEXT("TAA_Threshold_Resize", "resizes so larger dimension is <= threshold")
					)
					.AutoWrapText(true)
				]
				*/
			]
		+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Left)
			.Padding(8.0f, 4.0f, 8.0f, 4.0f)
			[
				SNew(SHorizontalBox)
				.Visibility(this, &STextureAssetList::GetNMK16Visibility)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("TAA_NormalMapsKeep16bits", "Normal Maps keep 16 bits: "))
					.AutoWrapText(true)
				]
				+ SHorizontalBox::Slot()
				.Padding(4.0f, 0.0f, 2.0f, 0.0f)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SCheckBox)
					.IsChecked(this, &STextureAssetList::HandleNMK16CheckBoxIsChecked)
					.OnCheckStateChanged(this, &STextureAssetList::HandleNMK16CheckBoxCheckedStateChanged)
				]
			]
		+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Left)
			.Padding(8.0f, 4.0f, 8.0f, 4.0f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("TAA_ShowExcluded", "Show Excluded Textures: "))
					.AutoWrapText(true)
				]
				+ SHorizontalBox::Slot()
				.Padding(4.0f, 0.0f, 2.0f, 0.0f)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SCheckBox)
					.IsChecked(this, &STextureAssetList::HandleShowExcludedCheckBoxIsChecked)
					.OnCheckStateChanged(this, &STextureAssetList::HandleShowExcludedCheckBoxCheckedStateChanged)
				]
			]
		// Separator
		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(8.0f, 4.0f, 8.0f, 4.0f)
			[
				SNew(SSeparator)
			]
		// Dialog ok/cancel buttons
		+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(8.0f, 4.0f, 8.0f, 4.0f)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
				.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
				.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
				+ SUniformGridPanel::Slot(0, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.OnClicked(this, &STextureAssetList::OnButtonClick, STextureActionDlg::Confirm)
					.IsEnabled(this, &STextureAssetList::GetOkButtonEnabled)
					.Text(LOCTEXT("TAA_OK", "OK"))
				]
				+ SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.OnClicked(this, &STextureAssetList::OnButtonClick, STextureActionDlg::Cancel)
					.Text(LOCTEXT("TAA_Cancel", "Cancel"))
				]
			]
	];

	// will be done by Init :
	//UpdateList();

}

void STextureAssetList::Init(const TArray<UTexture *> &Textures, ETextureAction InAction)
{
	Action = InAction;
	
	IntroMessage = TAA_Intro(Action);

	TextureList.SetNum(Textures.Num());
	for(int i=0;i<Textures.Num();i++)
	{
		TextureList[i].Texture = Textures[i];
		TextureList[i].IsEnabled = true;
	}

	UpdateList();
}

STextureActionDlg::EResult STextureAssetList::GetUserResponse() const
{
	return UserResponse;
}

static FText AssetActionStatus_Text(EAssetActionStatus Status)
{
	switch(Status)
	{
	case EAssetActionStatus::Enabled: return FText();
	case EAssetActionStatus::NoSource: return LOCTEXT("TAAStatus_NoSource", "The texture has no source data; cooked Editor?");
	case EAssetActionStatus::WrongType: return LOCTEXT("TAAStatus_WrongType", "The texture is not a supported type.");
	case EAssetActionStatus::UnderSized: return LOCTEXT("TAAStatus_UnderSized", "The texture was under or equal the threshold size.");
	case EAssetActionStatus::AlreadyPow2: return LOCTEXT("TAAStatus_AlreadyPow2", "The texture is already power of two.");
	case EAssetActionStatus::HasLayers: return LOCTEXT("TAAStatus_HasLayers", "Textures with more than 1 layer not supported.");
	case EAssetActionStatus::HasMipsLeaveExisting: return LOCTEXT("TAAStatus_HasMipsLeaveExisting", "Textures has imported mips and LeaveExisting, will not change. (change MipGen if wanted)");
	case EAssetActionStatus::Already8bit: return LOCTEXT("TAAStatus_Already8", "Texture format is already minimum bit depth.");
	case EAssetActionStatus::DontChangeJPEG: return LOCTEXT("TAAStatus_JPEG", "Texture source is JPEG compressed, will not change."); 
	case EAssetActionStatus::MustBe8BitForJPEG: return LOCTEXT("TAAStatus_MustBe8BitForJPEG", "Texture source must be 8 bit for JPEG; change bit depth first."); 
	default:
		check(0);
		return FText();
	}	
}

static FName GetTextureClassIconName( ETextureClass TC )
{
	// see SlateEditStyle + StarshipStyle

	static const FName NameTexture2d("ClassIcon.Texture2D");
	static const FName NameTexture2dArray("ClassIcon.Texture2DArray");
	static const FName NameTextureCube("ClassIcon.TextureCube");
	static const FName NameTextureCubeArray("ClassIcon.TextureCubeArray");
	static const FName NameVolume("ClassIcon.VolumeTexture");
	static const FName NameRenderTarget("ClassIcon.TextureRenderTarget2D");
	static const FName NameDefault("ClassIcon.Default");

	switch(TC)
	{
	case ETextureClass::TwoD: return NameTexture2d;
	case ETextureClass::Cube: return NameTextureCube;
	case ETextureClass::Array: return NameTexture2dArray;
	case ETextureClass::CubeArray: return NameTextureCubeArray;
	case ETextureClass::Volume: return NameVolume;
	case ETextureClass::TwoDDynamic: return NameTexture2d;
	case ETextureClass::RenderTarget: return NameRenderTarget;
	case ETextureClass::Other2DNoSource: return NameTexture2d;

	case ETextureClass::Invalid:
	case ETextureClass::OtherUnknown:
		return NameDefault;

	default:
		check(0);
		return NameDefault;	
	}
}

TSharedRef<SWidget> STextureAssetList::CreateAssetLine(int index, const UTexture * Texture, const EAssetActionStatus Status,
										const FString & EnabledActionString,
										const FString & DescriptionString)
{
	const FAssetData Asset(Texture);

	FSlateIcon ClassIcon = FSlateIconFinder::FindIconForClass(Texture->GetClass());
	if (!ClassIcon.IsSet())
	{
		FName TextureClassIconName = GetTextureClassIconName(Texture->GetTextureClass());
		ClassIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), TextureClassIconName);
	}

	FName SeverityIcon = NAME_None;
	FText DetailedInfoText;

	bool IsEnabled = ( Status == EAssetActionStatus::Enabled );

	if ( IsEnabled )
	{
		// no error status, show what action will do in drop-down text
		DetailedInfoText = FText::FromString(EnabledActionString);
	}
	else
	{
		// get description of status problem
		DetailedInfoText = AssetActionStatus_Text(Status);

		if ( Status == EAssetActionStatus::UnderSized ||
			Status == EAssetActionStatus::Already8bit ||
			Status == EAssetActionStatus::AlreadyPow2 )
		{
			SeverityIcon = "MessageLog.Note";
		}
		else
		{
			SeverityIcon = "MessageLog.Error";
		}
	}

	/*
	//const bool bEngineAsset = Asset.PackagePath.ToString().StartsWith(TEXT("/Engine/")); // use FPackageName::SplitPackageNameRoot
	// ConvertToVT does this, currently we do not
	else if (bEngineAsset) // ??
	{
		SeverityIcon = "MessageLog.Note";
		DetailedInfoText = LOCTEXT("TAA_EngineAsset", "The texture is an engine asset, a copy will be created in the current project.");
	}
	*/

	auto Result =
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Fill) // use the whole line
		[
			// Class icon
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(SBox)
				.Padding(2.0f)
				[
					SNew(SImage)
					.Image(ClassIcon.GetIcon())
				]
			]
			// Error/warning icon
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.Padding(2.0f)
				[
					(SeverityIcon == NAME_None) ? SNullWidget::NullWidget :
					static_cast<TSharedRef<SWidget>>(SNew(SImage).Image(FAppStyle::GetBrush(SeverityIcon)))
				]
			]
			// Fold out button with asset name
			+ SHorizontalBox::Slot()
			.FillWidth(500.f) // take all the remaining space for this slot
			//.AutoWidth()
			.HAlign(HAlign_Left) // this applies to the stuff inside this slot, not the slot itself
			.VAlign(VAlign_Center)
			[
					(DetailedInfoText.IsEmpty())
					? static_cast<TSharedRef<SWidget>>(
						SNew(STextBlock)
						.Text(FText::FromString(Asset.GetObjectPathString()))
						.Clipping(EWidgetClipping::ClipToBounds)
						)
				: SNew(SButton)
				.ButtonStyle(FCoreStyle::Get(), "NoBorder")
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.ClickMethod(EButtonClickMethod::MouseDown)
				.OnClicked(this, &STextureAssetList::OnExpanderClicked, index)
				.ContentPadding(0.f)
				.ForegroundColor(FSlateColor::UseForeground())
				.IsFocusable(false)
				.Clipping(EWidgetClipping::ClipToBounds)
				//.Text(FText::FromName(Asset.GetObjectPathString()))
				[
					// Fold out icon
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					[
						SNew(SImage)
						.Image(this, &STextureAssetList::GetExpanderImage, index)
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
					// Fold out text
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(FText::FromString(Asset.GetObjectPathString()))
						.ColorAndOpacity( IsEnabled ? FSlateColor::UseForeground() : FSlateColor::UseSubduedForeground())
					] // change color for enabled/not
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Justification(ETextJustify::Right)
					.Text(FText::FromString(DescriptionString))
					.ColorAndOpacity( IsEnabled ? FSlateColor::UseForeground() : FSlateColor::UseSubduedForeground())
				]
			] 
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		[
			SNew(STextBlock)
			.Text(DetailedInfoText)
			.Visibility(this, &STextureAssetList::GetDetailVisibility, index)
		]
		;
	return Result;
}

bool ResizeTextureSourceToPowerOfTwo(UTexture * Texture)
{
	if ( Texture->Source.AreAllBlocksPowerOfTwo() )
	{
		return false;
	}

	const FIntPoint BeforeSourceSize = Texture->Source.GetLogicalSize();

	if ( ! UE::TextureUtilitiesCommon::Experimental::ResizeTextureSourceDataToNearestPowerOfTwo(Texture) )
	{
		UE_LOGF(LogTexture, Display, "Texture (%ls) did not resize.", *Texture->GetName());

		// did not resize, but may have done PreEditChange
		return false;
	}
	
	const FIntPoint AfterSourceSize = Texture->Source.GetLogicalSize();

	UE_LOGF(LogTexture, Display, "Texture (%ls) did resize Before Size=%dx%d After Size=%dx%d", *Texture->GetName(),
		BeforeSourceSize.X,BeforeSourceSize.Y,AfterSourceSize.X,AfterSourceSize.Y);

	// ?? if Texture was set to stretch to Pow2, we could remove that now, but leaving it is harmless
	//Texture->PowerOfTwoMode = ETexturePowerOfTwoSetting::None;

	// if Texture was set to MipGen = NoMipMaps, change to FromTextureGroup ?
	// assume that since we changed to Pow2, we want mips and streaming
	if ( Texture->MipGenSettings == TMGS_NoMipmaps )
	{
		Texture->MipGenSettings = TMGS_FromTextureGroup;
	}
	Texture->NeverStream = false;
	
	// it it was set to UI (uncompressed) change to default :
	if ( Texture->CompressionSettings == TC_EditorIcon )
	{
		Texture->CompressionSettings = TC_Default;
	}

	// this counts as a reimport :
	UE::TextureUtilitiesCommon::ApplyDefaultsForNewlyImportedTextures(Texture,true);

	// ResizeTextureSourceDataToNearestPowerOfTwo did the PreEditChange
	Texture->PostEditChange();

	return true;
}

// resizes to <= TargetSize
//	if size was already <= TargetSize , no change is made and we return false
bool ResizeTextureSource(UTexture * Texture,int TargetSize)
{
	// we do the resizing considering only mip/LOD/build settings for the running Editor platform (eg. Windows)
	const ITargetPlatform* RunningPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
	
	int32 BeforeSizeX;
	int32 BeforeSizeY;
	Texture->GetBuiltTextureSize(RunningPlatform, BeforeSizeX, BeforeSizeY);
	
	const FIntPoint BeforeSourceSize = Texture->Source.GetLogicalSize();
	UE_LOGF(LogTexture, Display, "Texture (%ls) Resizing to <= %d Source Size=%dx%d Built Size=%dx%d", *Texture->GetName() , TargetSize,
		BeforeSourceSize.X,BeforeSourceSize.Y,BeforeSizeX,BeforeSizeY);

	if ( ! UE::TextureUtilitiesCommon::Experimental::DownsizeTextureSourceData(Texture, TargetSize, RunningPlatform))
	{
		UE_LOGF(LogTexture, Display, "Texture (%ls) did not resize.", *Texture->GetName());

		// did not resize, but may have done PreEditChange
		return false;
	}
	
	const FIntPoint AfterSourceSize = Texture->Source.GetLogicalSize();

	Texture->LODBias = 0;
	
	int32 AfterSizeX;
	int32 AfterSizeY;
	Texture->GetBuiltTextureSize(RunningPlatform, AfterSizeX, AfterSizeY);

	// if AfterSize > BeforeSize , kick up LODBias
	//	to try to preserve GetBuiltTextureSize
	Texture->LODBias = FMath::RoundToInt32( FMath::Log2( (double) FMath::Max(AfterSizeX,AfterSizeY) / FMath::Max(BeforeSizeX,BeforeSizeY) ) );
	Texture->LODBias = FMath::Max(0,Texture->LODBias);
		
	// recompute AfterSize if we changed LODBias
	if ( Texture->LODBias != 0 )
	{
		Texture->GetBuiltTextureSize(RunningPlatform, AfterSizeX, AfterSizeY);
	}
	
	UE_LOGF(LogTexture, Display, "Texture (%ls) did resize Source Size=%dx%d Built Size=%dx%d", *Texture->GetName(),
		AfterSourceSize.X,AfterSourceSize.Y,AfterSizeX,AfterSizeY);

	if ( BeforeSizeX != AfterSizeX || BeforeSizeY != AfterSizeY )
	{
		// not a warning, just FYI
		// changing built size is totally possible and expected to happen sometimes
		//	basically any time you resize smaller than the previous in-game size
		UE_LOGF(LogTexture,Display,"DoResizeTextureSource failed to preserve built size; was: %dx%d now: %dx%d on [%ls]",
			BeforeSizeX,BeforeSizeY,
			AfterSizeX,AfterSizeY,
			*Texture->GetFullName());
	}

	// this counts as a reimport :
	UE::TextureUtilitiesCommon::ApplyDefaultsForNewlyImportedTextures(Texture,true);

	// DownsizeTextureSourceData did the PreEditChange
	Texture->PostEditChange();

	return true;
}

static ETextureSourceFormat GetReducedTextureSourceFormat(const TextureCompressionSettings TC,const ETextureSourceFormat InTSF,const bool NormalMapsKeep16bits)
{
	if ( InTSF == TSF_BGRE8 )
	{
		// don't change BGRE
		return InTSF;
	}

	bool Out8bit = false;
	bool OutBC45 = false;
	bool OutHDR = false;
	bool OutSingleChannel = false;

	switch(TC)
	{
	case TC_Grayscale				: //"Grayscale (G8/16, RGB8 sRGB)"),
	case TC_Displacementmap			: //"Displacementmap (G8/16)"),
		// Gray and Displacement pass through G16 ; note they do not do that for RGBA16 (see GetDefaultTextureFormatName)
		//	do this conditional on NormalMapsKeep16bits just so we have a way to toggle it, even though it's not really a normal map
		if ( InTSF == TSF_G16 && NormalMapsKeep16bits ) return InTSF;
		// otherwise we will convert to G8
		// [[fallthrough]];
	case TC_DistanceFieldFont		: //"DistanceFieldFont (G8)"),
		Out8bit = true;
		OutSingleChannel = true;
		break;

	case TC_Default					: //"Default (DXT1/5, BC1/3 on DX11)"),
	case TC_Masks					: //"Masks (no sRGB)"),
	case TC_VectorDisplacementmap	: //"VectorDisplacementmap (RGBA8)"),
	case TC_EditorIcon				: //"UserInterface2D (RGBA)"),
	case TC_BC7						: //"BC7 (DX11, optional A)"),
	case TC_LQ				        : // "Low Quality (BGR565/BGR555A1)", ToolTip = "BGR565/BGR555A1, fallback to DXT1/DXT5 on Mac platform"),
		Out8bit = true;
		break;

	case TC_Normalmap				: //"Normalmap (DXT5, BC5 on DX11)"),
		OutBC45 = true;
		break;

	case TC_Alpha					: //"Alpha (no sRGB, BC4 on DX11)"),
		OutBC45 = true;
		OutSingleChannel = true;
		break;
		
	case TC_HDR						: //"HDR (RGBA16F, no sRGB)"),
	case TC_HDR_Compressed			: //"HDR Compressed (RGB, BC6H, DX11)"),
		OutHDR = true;
		break;

	case TC_HalfFloat				: //"Half Float (R16F)"),
	case TC_SingleFloat				: //"Single Float (R32F)"),
		OutHDR = true;
		OutSingleChannel = true;
		break;

	case TC_EncodedReflectionCapture: 
	case TC_HDR_F32					: //"HDR High Precision (RGBA32F)"),
		// don't change :
		return InTSF;

	default:
		check(0);
		return InTSF;
	}

	if ( OutBC45 && ! NormalMapsKeep16bits )
	{
		// if NormalMaps don't keep 16 bit sources, then just treat them like 8 bit :
		Out8bit = true;
	}

	ETextureSourceFormat OutTSF = InTSF;

	if ( Out8bit )
	{
		OutTSF = ( OutSingleChannel ) ? TSF_G8 : TSF_BGRA8;		
	}
	else if ( OutBC45 )
	{
		check( NormalMapsKeep16bits );

		// just choose a 16 bit output format, even if source was 8 bit
		//	we will only convert if bytes per pixel goes down
		OutTSF = ( OutSingleChannel ) ? TSF_G16 : TSF_RGBA16;		

		if ( OutSingleChannel && InTSF == TSF_BGRA8 )
		{
			// don't do BGRA8 -> G16 , use G8 instead
			OutTSF = TSF_G8;
		}
	}
	else
	{
		check(OutHDR);
		check( TC != TC_HDR_F32 ); // already handled

		if ( TC == TC_SingleFloat &&
			(InTSF == TSF_RGBA32F || InTSF == TSF_R32F) )
		{
			// 32 bit output, 32 bit input, keep it 32 bit
			OutTSF = TSF_R32F;
		}
		else
		{
			OutTSF = ( OutSingleChannel ) ? TSF_R16F : TSF_RGBA16F;	
		}
	}

	int64 InBPP  = FTextureSource::GetBytesPerPixel(InTSF);
	int64 OutBPP = FTextureSource::GetBytesPerPixel(OutTSF);

	if ( InBPP <= OutBPP )
	{
		// if bytes per pixel didn't go down, don't change
		return InTSF;
	}
	else
	{
		// reducing
		return OutTSF;
	}
}

bool ConvertTo8bitTextureSource(UTexture * Texture,bool NormalMapsKeep16bits)
{
	if ( Texture->Source.GetNumLayers() > 1 )
	{
		return false;
	}

	ETextureSourceFormat InTSF = Texture->Source.GetFormat();
	ETextureSourceFormat OutTSF = GetReducedTextureSourceFormat(Texture->CompressionSettings,InTSF,NormalMapsKeep16bits);
	if ( InTSF == OutTSF )
	{
		return false;
	}
	
	UE_LOGF(LogTexture, Display, "Texture (%ls) changing format from %ls to %ls, TC = %ls", 
		*Texture->GetName() ,
		*StaticEnum<ETextureSourceFormat>()->GetDisplayNameTextByValue(InTSF).ToString(), // these have TSF_ on the strings
		*StaticEnum<ETextureSourceFormat>()->GetDisplayNameTextByValue(OutTSF).ToString(),
		*StaticEnum<TextureCompressionSettings>()->GetDisplayNameTextByValue(Texture->CompressionSettings).ToString()
		);

	// calls Pre/PostEditChange :
	UE::TextureUtilitiesCommon::Experimental::ChangeTextureSourceFormat(Texture,OutTSF);

	return true;
}

bool CompressTextureSourceWithJPEG(UTexture * Texture)
{
	// calls Pre/PostEditChange :
	bool bDid = UE::TextureUtilitiesCommon::Experimental::CompressTextureSourceWithJPEG(Texture);
	
	UE_LOGF(LogTexture, Display, "Texture (%ls) %ls", 
		*Texture->GetName() ,
		bDid ? TEXT("was changed to JPEG compression.") : TEXT("was not changed.")
		);

	return bDid;
}

void STextureAssetList::DoAction()
{
	int NumEnabled = 0;
	for (TextureListEntry & Entry : TextureList)
	{
		UTexture * Texture = Entry.Texture;

		UE_LOGF(LogTexture, Display, "Texture (%ls) Enabled=%d", *Texture->GetName() , (int)Entry.IsEnabled);

		if ( Entry.IsEnabled )
		{
			NumEnabled ++;	
		}
	}

	if ( NumEnabled == 0 )
	{
		return;
	}

	FScopedSlowTask Progress(NumEnabled, LOCTEXT("TAA_Progress", "Applying action to TextureSources ..."));
	Progress.MakeDialog(/*ShowCancelButton*/true);

	for (TextureListEntry & Entry : TextureList)
	{
		if ( ! Entry.IsEnabled ) continue;

		UTexture * Texture = Entry.Texture;

		Progress.EnterProgressFrame(1.f);
		if (Progress.ShouldCancel())
		{
			break;
		}

		switch(Action)
		{
		case ETextureAction::Resize:
			UE::TextureAssetActionsInternal::ResizeTextureSource(Texture,ThresholdValue);
			break;
		case ETextureAction::ResizePow2:
			UE::TextureAssetActionsInternal::ResizeTextureSourceToPowerOfTwo(Texture);
			break;
		case ETextureAction::ConvertTo8bit:
			UE::TextureAssetActionsInternal::ConvertTo8bitTextureSource(Texture,bNormalMapsKeep16bits);
			break;
		case ETextureAction::JPEG:
			UE::TextureAssetActionsInternal::CompressTextureSourceWithJPEG(Texture);
			break;
		default:
			check(0);
			break;
		}
	}
}

static EAssetActionStatus GetTextureFilterStatus(UTexture * Texture, ETextureAction Action, int ThresholdValue,
		bool bNormalMapsKeep16bits,
		FString & EnabledActionString,
		FString & DescriptionString )
{
	EAssetActionStatus Status = EAssetActionStatus::Enabled; 
				
	ETextureClass Class = Texture->GetTextureClass();
		
	// first filter for exclusions that apply to all actions
	//	don't query anything in Source until you check IsValid
	if ( Class != ETextureClass::TwoD &&
		Class != ETextureClass::Cube &&
		Class != ETextureClass::Array &&
		Class != ETextureClass::CubeArray &&
		Class != ETextureClass::Volume )
	{
		// rendertarget, dynamic, or unknown
		Status = EAssetActionStatus::WrongType;
	}
	else if ( ! Texture->Source.IsValid() )
	{
		Status = EAssetActionStatus::NoSource;
	}
	else if ( Texture->Source.GetNumLayers() != 1 ) // only 1 layer textures
	{
		Status = EAssetActionStatus::HasLayers;
	}
	else if ( Texture->Source.GetNumMips() != 1 && Texture->MipGenSettings == TMGS_LeaveExistingMips )
	{
		Status = EAssetActionStatus::HasMipsLeaveExisting;
	}
	else
	{
		// don't prep Description text until after Source IsValid check

		// GetLogicalSize = sum of udim blocks
		const FIntPoint SourceSize = Texture->Source.GetLogicalSize();
		const ETextureSourceFormat InTSF = Texture->Source.GetFormat();

		DescriptionString = FString::Printf(TEXT("%dx%d %s"),SourceSize.X,SourceSize.Y,
			ERawImageFormat::GetName( FImageCoreUtils::ConvertToRawImageFormat(InTSF) ));
		if ( Texture->VirtualTextureStreaming )
		{
			DescriptionString += FString(TEXT(" VT"));
		}
		if ( Texture->Source.GetSourceCompression() == ETextureSourceCompressionFormat::TSCF_JPEG )
		{
			DescriptionString += FString(TEXT(" JPEG"));
		}
		
		// per-Action filters :
		if( Action == ETextureAction::Resize || Action == ETextureAction::ResizePow2 )
		{
			int MaxSize = FMath::Max(SourceSize.X,SourceSize.Y);

			if ( Class != ETextureClass::TwoD && Class != ETextureClass::Cube )
			{
				// Resize only supports 2d and Cube for now, no arrays or volumes
				Status = EAssetActionStatus::WrongType;
			}
			else if ( Action == ETextureAction::ResizePow2 && Texture->Source.AreAllBlocksPowerOfTwo() )
			{
				Status = EAssetActionStatus::AlreadyPow2;
			}
			else if ( MaxSize <= ThresholdValue )
			{
				Status = EAssetActionStatus::UnderSized;
			}
			else
			{
				// could fill EnabledActionString
				// hard to get size we will output exactly right
			}
		}
		else if ( Action == ETextureAction::ConvertTo8bit )
		{
			ETextureSourceFormat OutTSF = GetReducedTextureSourceFormat(Texture->CompressionSettings,InTSF,bNormalMapsKeep16bits);
			if ( InTSF == OutTSF )
			{
				Status = EAssetActionStatus::Already8bit;
			}
			else if ( Texture->Source.GetSourceCompression() == ETextureSourceCompressionFormat::TSCF_JPEG )
			{
				// JPEG is already 8 bit; we don't want to try to change RGB JPEG to G8, just leave it alone
				Status = EAssetActionStatus::DontChangeJPEG;
			}
			else
			{
				// @@ NOTE : ThresholdValue not checked!

				// Enabled!
				EnabledActionString = FString::Printf(TEXT("TC=%s will convert to %s"),
					*StaticEnum<TextureCompressionSettings>()->GetDisplayNameTextByValue(Texture->CompressionSettings).ToString(),
					ERawImageFormat::GetName( FImageCoreUtils::ConvertToRawImageFormat( OutTSF ) ) );
			}
		}
		else if ( Action == ETextureAction::JPEG )
		{
			if ( Class != ETextureClass::TwoD || Texture->Source.GetNumBlocks() > 1 || Texture->Source.GetNumSlices() > 1 )
			{
				// JPEG only supports 2d
				// Blocked/UDIM doesn't support JPEG
				Status = EAssetActionStatus::WrongType;
			}
			else if ( Texture->Source.GetSourceCompression() == ETextureSourceCompressionFormat::TSCF_JPEG )
			{
				// Already JPEG
				Status = EAssetActionStatus::DontChangeJPEG;
			}
			else if ( InTSF != TSF_BGRA8 && InTSF != TSF_G8 )
			{
				Status = EAssetActionStatus::MustBe8BitForJPEG;
			}
			else if ( SourceSize.X < 16 || SourceSize.Y < 16 )
			{
				// hard-coded requirement of at least 16 in both dimensions for JPEG
				Status = EAssetActionStatus::UnderSized;
			}
			else if ( (int64)SourceSize.X*SourceSize.Y < (int64)ThresholdValue*ThresholdValue ) // pixel count
			{
				Status = EAssetActionStatus::UnderSized;
			}
			else
			{
				// enabled!
			}				
		}
		else
		{
			check(0);
		}
	}

	return Status;
}

bool GetTextureFilterEnabled( UTexture * Texture, ETextureAction Action, int ThresholdValue,
		bool bNormalMapsKeep16bits , bool * bOutExcludedBecauseAlreadySatisfied)
{
	FString EnabledActionString;
	FString DescriptionString;

	EAssetActionStatus Status = GetTextureFilterStatus( Texture, Action, ThresholdValue, bNormalMapsKeep16bits,
		EnabledActionString, DescriptionString );

	// Special returns for the exclusions that are due to the target already being in the desired state
	//  return another bool or trinary enum to indicate these

	bool bDummy;
	if ( bOutExcludedBecauseAlreadySatisfied == nullptr )
	{
		bOutExcludedBecauseAlreadySatisfied = &bDummy;
	}

	if ( Action == ETextureAction::Resize && Status == EAssetActionStatus::UnderSized )
	{
		*bOutExcludedBecauseAlreadySatisfied = true;
		return false;
	}
	if ( Action == ETextureAction::ResizePow2 && Status == EAssetActionStatus::AlreadyPow2 )
	{
		*bOutExcludedBecauseAlreadySatisfied = true;
		return false;
	}
	if ( Action == ETextureAction::ConvertTo8bit && Status == EAssetActionStatus::Already8bit )
	{
		*bOutExcludedBecauseAlreadySatisfied = true;
		return false;
	}
	if ( Action == ETextureAction::JPEG && Status == EAssetActionStatus::DontChangeJPEG )
	{
		*bOutExcludedBecauseAlreadySatisfied = true;
		return false;
	}
	
	*bOutExcludedBecauseAlreadySatisfied = false;

	return ( Status == EAssetActionStatus::Enabled );
}

void STextureAssetList::UpdateList()
{
	ExpandedIndexes.SetNum(0);

	AssetListContainer->ClearChildren();
	int32 AssetListContainerIndex = 0;

	// filter select textures to see if they should be acted on
	for (int Index=0;Index<TextureList.Num();Index++)
	{
		TextureListEntry & Entry = TextureList[Index];
		UTexture * Texture = Entry.Texture;
		
		FString EnabledActionString;
		FString DescriptionString;

		EAssetActionStatus Status = GetTextureFilterStatus( Texture, Action, ThresholdValue, bNormalMapsKeep16bits,
			EnabledActionString, DescriptionString );

		Entry.IsEnabled = ( Status == EAssetActionStatus::Enabled );

		if ( Entry.IsEnabled || bShowExcluded )
		{
			AssetListContainer->AddSlot()
				.AutoHeight()
				[
					CreateAssetLine(AssetListContainerIndex, Texture, Status, EnabledActionString, DescriptionString)
				];
			AssetListContainerIndex++;
		}
	}

	ErrorMessage = FText();
}

FReply STextureAssetList::OnButtonClick(STextureActionDlg::EResult ButtonID)
{
	ParentWindow->RequestDestroyWindow();
	UserResponse = ButtonID;

	if (ButtonID == STextureActionDlg::EResult::Confirm)
	{
		DoAction();
	}

	return FReply::Handled();
}

void STextureAssetList::OnThresholdChanged(TSharedPtr<int32> InSelectedItem, ESelectInfo::Type SelectInfo)
{
	ThresholdValue = *InSelectedItem;
	UpdateList();
}

FText STextureAssetList::GetThresholdText() const
{
	return FText::FromString(FString::Format(TEXT("{0}"), TArray<FStringFormatArg>({ ThresholdValue })));
}

TSharedRef<SWidget> STextureAssetList::OnGenerateThresholdWidget(TSharedPtr<int32> InItem)
{
	return SNew(STextBlock)
		.Text(FText::FromString(FString::Format(TEXT("{0}"), TArray<FStringFormatArg>({ *InItem }))));
}

FReply STextureAssetList::OnExpanderClicked(int index)
{
	if (ExpandedIndexes.Contains(index))
	{
		ExpandedIndexes.Remove(index);
	}
	else
	{
		ExpandedIndexes.Add(index);
	}
	return FReply::Handled();
}

bool STextureAssetList::GetOkButtonEnabled() const
{
	return ErrorMessage.IsEmpty();
}

EVisibility STextureAssetList::GetDetailVisibility(int index) const
{
	return (ExpandedIndexes.Contains(index)) ? EVisibility::Visible : EVisibility::Collapsed;
}

const FSlateBrush* STextureAssetList::GetExpanderImage(int index) const
{
	FName ResourceName;
	if (GetDetailVisibility(index) == EVisibility::Visible)
	{
		static FName ExpandedName = "TreeArrow_Expanded";
		ResourceName = ExpandedName;
	}
	else
	{
		static FName CollapsedName = "TreeArrow_Collapsed";
		ResourceName = CollapsedName;
	}
	return FCoreStyle::Get().GetBrush(ResourceName);
}

EVisibility STextureAssetList::GetIntroMessageVisibility() const
{
	return (IntroMessage.IsEmpty()) ? EVisibility::Collapsed : EVisibility::Visible;
}

EVisibility STextureAssetList::GetErrorMessageVisibility() const
{
	return (ErrorMessage.IsEmpty()) ? EVisibility::Collapsed : EVisibility::Visible;
}

FText STextureAssetList::GetIntroMessage() const
{
	return IntroMessage;
}

FText STextureAssetList::GetErrorMessage() const
{
	return ErrorMessage;
}

STextureActionDlg::STextureActionDlg(const TArray<UTexture *> &Textures, ETextureAction Act)
{
	if (FSlateApplication::IsInitialized())
	{
		DialogWindow = SNew(SWindow)
			.Title(TAA_DialogTitle(Act))
			.SupportsMinimize(false).SupportsMaximize(false)
			.ClientSize(FVector2D(1200, 600));

		TSharedPtr<SBorder> DialogWrapper =
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(4.0f)
			[
				SAssignNew(DialogWidget, STextureAssetList)
				.ParentWindow(DialogWindow)
			];

		// Init will do UpdateList :
		DialogWidget->Init(Textures,Act);
		DialogWindow->SetContent(DialogWrapper.ToSharedRef());
	}
}

STextureActionDlg::EResult STextureActionDlg::ShowModal()
{
	//Show Dialog
	GEditor->EditorAddModalWindow(DialogWindow.ToSharedRef());
	EResult UserResponse = (EResult)DialogWidget->GetUserResponse();
	DialogWindow->GetParentWindow()->RemoveDescendantWindow(DialogWindow.ToSharedRef());
	return UserResponse;
}

}; // namespace
}; // namespace

void UE::TextureAssetActions::TextureSource_Resize_WithDialog(const TArray<UTexture*> & InTextures)
{
	UE::TextureAssetActionsInternal::STextureActionDlg Dlg(InTextures,UE::TextureAssetActionsInternal::ETextureAction::Resize);
	Dlg.ShowModal();
}

void UE::TextureAssetActions::TextureSource_ConvertTo8bit_WithDialog(const TArray<UTexture*> & InTextures)
{
	UE::TextureAssetActionsInternal::STextureActionDlg Dlg(InTextures,UE::TextureAssetActionsInternal::ETextureAction::ConvertTo8bit);
	Dlg.ShowModal();
}

void UE::TextureAssetActions::TextureSource_JPEG_WithDialog(const TArray<UTexture*> & InTextures)
{
	UE::TextureAssetActionsInternal::STextureActionDlg Dlg(InTextures,UE::TextureAssetActionsInternal::ETextureAction::JPEG);
	Dlg.ShowModal();
}

void UE::TextureAssetActions::TextureSource_ResizeToPowerOfTwo_WithDialog(const TArray<UTexture*> & InTextures)
{
	UE::TextureAssetActionsInternal::STextureActionDlg Dlg(InTextures,UE::TextureAssetActionsInternal::ETextureAction::ResizePow2);
	Dlg.ShowModal();
}

#undef LOCTEXT_NAMESPACE
