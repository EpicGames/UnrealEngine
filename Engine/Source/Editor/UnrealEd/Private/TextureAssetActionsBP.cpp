// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextureAssetActionsBP.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextureAssetActionsBP)




UNREALED_API ETextureAssetActionsBlueprintReturn UTextureAssetActionsBlueprintFns::ResizeTextureSourceToPowerOfTwo(UTexture * Texture, int FilterThresholdValue)
{
	bool bExcludedBecauseAlreadySatisfied = false;

	if ( ! UE::TextureAssetActionsInternal::GetTextureFilterEnabled(Texture,UE::TextureAssetActionsInternal::ETextureAction::ResizePow2,FilterThresholdValue,false,&bExcludedBecauseAlreadySatisfied) )
	{
		return bExcludedBecauseAlreadySatisfied ? ETextureAssetActionsBlueprintReturn::AlreadyDone : ETextureAssetActionsBlueprintReturn::Error;
	}

	bool bSuccess = UE::TextureAssetActionsInternal::ResizeTextureSourceToPowerOfTwo(Texture);
	return bSuccess ? ETextureAssetActionsBlueprintReturn::Success : ETextureAssetActionsBlueprintReturn::Error;
}

UNREALED_API ETextureAssetActionsBlueprintReturn UTextureAssetActionsBlueprintFns::ConvertTo8bitTextureSource(UTexture * Texture,bool bNormalMapsKeep16bits )
{
	bool bExcludedBecauseAlreadySatisfied = false;
	int FilterThresholdValue = 1;

	if ( ! UE::TextureAssetActionsInternal::GetTextureFilterEnabled(Texture,UE::TextureAssetActionsInternal::ETextureAction::ConvertTo8bit,FilterThresholdValue,bNormalMapsKeep16bits,&bExcludedBecauseAlreadySatisfied) )
	{
		return bExcludedBecauseAlreadySatisfied ? ETextureAssetActionsBlueprintReturn::AlreadyDone : ETextureAssetActionsBlueprintReturn::Error;
	}

	bool bSuccess = UE::TextureAssetActionsInternal::ConvertTo8bitTextureSource(Texture,bNormalMapsKeep16bits);
	return bSuccess ? ETextureAssetActionsBlueprintReturn::Success : ETextureAssetActionsBlueprintReturn::Error;
}
	
UNREALED_API ETextureAssetActionsBlueprintReturn UTextureAssetActionsBlueprintFns::ResizeTextureSource(UTexture * Texture,int TargetSize )
{
	bool bExcludedBecauseAlreadySatisfied = false;
	int FilterThresholdValue = TargetSize;

	if ( ! UE::TextureAssetActionsInternal::GetTextureFilterEnabled(Texture,UE::TextureAssetActionsInternal::ETextureAction::Resize,FilterThresholdValue,false,&bExcludedBecauseAlreadySatisfied) )
	{
		return bExcludedBecauseAlreadySatisfied ? ETextureAssetActionsBlueprintReturn::AlreadyDone : ETextureAssetActionsBlueprintReturn::Error;
	}

	bool bSuccess = UE::TextureAssetActionsInternal::ResizeTextureSource(Texture,TargetSize);
	return bSuccess ? ETextureAssetActionsBlueprintReturn::Success : ETextureAssetActionsBlueprintReturn::Error;
}

UNREALED_API ETextureAssetActionsBlueprintReturn UTextureAssetActionsBlueprintFns::CompressTextureSourceWithJPEG(UTexture * Texture, int FilterThresholdValue )
{
	bool bExcludedBecauseAlreadySatisfied = false;

	if ( ! UE::TextureAssetActionsInternal::GetTextureFilterEnabled(Texture,UE::TextureAssetActionsInternal::ETextureAction::JPEG,FilterThresholdValue,false,&bExcludedBecauseAlreadySatisfied) )
	{
		return bExcludedBecauseAlreadySatisfied ? ETextureAssetActionsBlueprintReturn::AlreadyDone : ETextureAssetActionsBlueprintReturn::Error;
	}

	bool bSuccess = UE::TextureAssetActionsInternal::CompressTextureSourceWithJPEG(Texture);
	return bSuccess ? ETextureAssetActionsBlueprintReturn::Success : ETextureAssetActionsBlueprintReturn::Error;
}