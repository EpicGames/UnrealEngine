// Copyright Epic Games, Inc. All Rights Reserved.


#include "CoreMinimal.h"
#include "Editor.h"
#include "GlobalEditorNotification.h"
#include "ContentStreaming.h"

/** Notification class for texture streaming. */
class FTextureStreamingNotificationImpl : public FGlobalEditorProgressNotification
{
public:
	FTextureStreamingNotificationImpl()
		: FGlobalEditorProgressNotification(NSLOCTEXT("StreamingTextures", "StreamingTexturesInProgress", "Streaming Textures"))
	{}

protected:
	/** FGlobalEditorNotification interface */
	virtual bool AllowedToStartNotification() const override
	{
		// We only want to show the notification initially if there's still enough work left to do to warrant it
		// Never show these notifications during PIE
		const int32 NumStreamingTextures = GetNumStreamingTextures();
		return !GEditor->PlayWorld && NumStreamingTextures > 300;
	}

	virtual int32 UpdateProgress() override
	{
		const int32 NumStreamingTextures = GetNumStreamingTextures();
		if (NumStreamingTextures > 0)
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("NumTextures"), FText::AsNumber(NumStreamingTextures));
			UpdateProgressMessage(FText::Format(NSLOCTEXT("StreamingTextures", "StreamingTexturesInProgressFormat", "Streaming Textures ({NumTextures})"), Args));
		}

		return NumStreamingTextures;
	}

private:
	static int32 GetNumStreamingTextures();
};

/** Global notification object. */
FTextureStreamingNotificationImpl GTextureStreamingNotification;

int32 FTextureStreamingNotificationImpl::GetNumStreamingTextures()
{
	FStreamingManagerCollection& StreamingManagerCollection = IStreamingManager::Get();

	if (StreamingManagerCollection.IsTextureStreamingEnabled())
	{
		IRenderAssetStreamingManager& TextureStreamingManager = StreamingManagerCollection.GetTextureStreamingManager();
		return TextureStreamingManager.GetNumWantingResources();
	}

	return 0;
}
