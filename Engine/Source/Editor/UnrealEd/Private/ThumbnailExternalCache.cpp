// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailExternalCache.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "AssetThumbnail.h"
#include "Misc/ObjectThumbnail.h"
#include "ObjectTools.h"
#include "Serialization/Archive.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/ScopedSlowTask.h"
#include "Interfaces/IPluginManager.h"
#include "ImageUtils.h"
#include "Hash/CityHash.h"
#include "Async/Async.h"
#include "Async/ParallelFor.h"

#include "Misc/ObjectThumbnail.inl"

#define LOCTEXT_NAMESPACE "ThumbnailExternalCache"

DEFINE_LOG_CATEGORY_STATIC(LogThumbnailExternalCache, Log, All);

namespace ThumbnailExternalCache
{
	const int64 LatestVersion = 0;
	const uint64 ExpectedHeaderId = 0x424d5548545f4555; // "UE_THUMB"
	const FString ThumbnailImageFormatName(TEXT(""));

	void ResizeThumbnailImage(FObjectThumbnail& Thumbnail, const int32 NewWidth, const int32 NewHeight)
	{
		FImageView SrcImage = Thumbnail.GetImage();

		FImage DestImage;
		FImageCore::ResizeImageAllocDest(SrcImage,DestImage,NewWidth,NewHeight);

		Thumbnail.SetImage( MoveTemp(DestImage) );
	}

	// Return true if was resized
	bool ResizeThumbnailIfNeeded(FObjectThumbnail& Thumbnail, const int32 MaxImageSize)
	{
		const int32 Width  = Thumbnail.GetImageWidth();
		const int32 Height = Thumbnail.GetImageHeight();

		// Resize if larger than maximum size
		if (Width > MaxImageSize || Height > MaxImageSize)
		{
			const double ShrinkModifier = (double)FMath::Max<int32>(Width, Height) / (double)MaxImageSize;
			const int32 NewWidth  = FMath::RoundToInt((double)Width / ShrinkModifier);
			const int32 NewHeight = FMath::RoundToInt((double)Height / ShrinkModifier);
			ResizeThumbnailImage(Thumbnail, NewWidth, NewHeight);

			return true;
		}

		return false;
	}
}

struct FPackageThumbnailRecord
{
	FName Name;
	int64 Offset = 0;
};

class FSaveThumbnailCache
{
public:
	FSaveThumbnailCache();
	~FSaveThumbnailCache();

	void Save(FArchive& Ar, const TArrayView<FAssetData> InAssetDatas, const FThumbnailExternalCacheSettings& InSettings);
	void Save(FArchive& Ar, FCombinedThumbnailCacheToSave& CombinedCache, bool bSort);
};

FSaveThumbnailCache::FSaveThumbnailCache()
{
}

FSaveThumbnailCache::~FSaveThumbnailCache()
{
}

void FThumbnailExternalCache::LoadCompressAndAppend(const TArrayView<FAssetData> InAssetDatas, FCombinedThumbnailCacheToSave& CombinedCache)
{
	const double StartTime = FPlatformTime::Seconds();

	struct FAssetToProcess
	{
		const FAssetData* AssetData = nullptr;
		TSharedPtr<FSaveThumbnailCacheTask> Task;
	};

	TArray<FAssetToProcess> AssetsToProcess;
	AssetsToProcess.Reserve(InAssetDatas.Num());

	CombinedCache.Tasks.Reserve(CombinedCache.Tasks.Num() + InAssetDatas.Num());

	for (const FAssetData& AssetData : InAssetDatas)
	{
		FNameBuilder ObjectFullNameBuilder;
		AssetData.GetFullName(ObjectFullNameBuilder);
		FName ObjectFullName(ObjectFullNameBuilder);

		if (!CombinedCache.Tasks.Contains(ObjectFullName))
		{
			FAssetToProcess& AssetToProcess = AssetsToProcess.AddDefaulted_GetRef();
			AssetToProcess.AssetData = &AssetData;			 
			AssetToProcess.Task = MakeShared<FSaveThumbnailCacheTask>();
			AssetToProcess.Task->Name = ObjectFullName;

			CombinedCache.Tasks.Add(ObjectFullName, AssetToProcess.Task);
		}
	}

	// Init() is not threadsafe, so call it here before doing ParallelFor below in case it hasn't already been previously called.
	Init();

	// Load and compress
	ParallelFor(AssetsToProcess.Num(), [&AssetsToProcess, &CombinedCache](int32 Index)
	{
		FAssetToProcess& AssetToProcess = AssetsToProcess[Index];
		if (ThumbnailTools::LoadThumbnailFromPackage(*AssetToProcess.AssetData, AssetToProcess.Task->ObjectThumbnail) && !AssetToProcess.Task->ObjectThumbnail.IsEmpty())
		{
			AssetToProcess.Task->Compress(CombinedCache.Settings);
		}
		else
		{
			AssetToProcess.Task.Reset();
		}
	});

	// Deduplicate to free up memory
	// Assumes compression method is deterministic (refactor if not deterministic to hash decompressed thumbnails)
	CombinedCache.DeduplicateMap.Reserve(CombinedCache.DeduplicateMap.Num() + AssetsToProcess.Num());
	for (FAssetToProcess& AssetToProcess : AssetsToProcess)
	{
		if (AssetToProcess.Task.IsValid() && !AssetToProcess.Task->ObjectThumbnail.IsEmpty())
		{
			FSaveThumbnailCacheDeduplicateKey DeduplicateKey(AssetToProcess.Task->CompressedBytesHash, AssetToProcess.Task->ObjectThumbnail.GetCompressedDataSize());
			if (TSharedPtr<FSaveThumbnailCacheTask>* ExistingData = CombinedCache.DeduplicateMap.Find(DeduplicateKey))
			{
				// Point to the other object's thumbnail
				CombinedCache.Tasks[AssetToProcess.Task->Name] = *ExistingData;
			}
			else
			{
				CombinedCache.DeduplicateMap.Add(DeduplicateKey, AssetToProcess.Task);
			}
		}
	}

	CombinedCache.AccumlatedLoadTime += FPlatformTime::Seconds() - StartTime;
}

void FSaveThumbnailCache::Save(FArchive& Ar, FCombinedThumbnailCacheToSave& CombinedCache, bool bSort)
{
	const double TimeStart = FPlatformTime::Seconds();

	// Sorting is often done to reduce size of patches
	if (bSort)
	{
		CombinedCache.Tasks.KeyStableSort(FNameLexicalLess());
	}

	// Only write information about assets that contain thumbnails
	TArray<TPair<FName, TSharedPtr<FSaveThumbnailCacheTask>>> AssetsToWrite;
	AssetsToWrite.Reserve(CombinedCache.Tasks.Num());
	for (TPair<FName, TSharedPtr<FSaveThumbnailCacheTask>>& It : CombinedCache.Tasks)
	{
		if (It.Value.IsValid() && !It.Value->ObjectThumbnail.IsEmpty())
		{
			AssetsToWrite.Add(It);
		}
	}

	const int32 NumAssetDatas = AssetsToWrite.Num();

	UE_LOGF(LogThumbnailExternalCache, Log, "Saving thumbnails for %d/%d assets (%d unique thumbnails) to %ls", NumAssetDatas, CombinedCache.Tasks.Num(), CombinedCache.DeduplicateMap.Num(), *Ar.GetArchiveName());

	FText StatusText = LOCTEXT("SaveStatus", "Saving Thumbnails: {0}");
	FScopedSlowTask SlowTask((float)NumAssetDatas, FText::Format(StatusText, FText::AsNumber(NumAssetDatas)));
	SlowTask.MakeDialog(/*bShowCancelButton*/ false);

	const double SaveTimeStart = FPlatformTime::Seconds();

	TArray<FPackageThumbnailRecord> PackageThumbnailRecords;
	PackageThumbnailRecords.Reset();
	PackageThumbnailRecords.Reserve(NumAssetDatas);

	TMap<FSaveThumbnailCacheDeduplicateKey, int64> DeduplicateMap;
	DeduplicateMap.Reset();
	DeduplicateMap.Reserve(CombinedCache.DeduplicateMap.Num());
	int32 NumDuplicates = 0;
	int64 DuplicateBytesSaved = 0;

	int64 TotalCompressedBytes = 0;

	// Write Header
	{
		FThumbnailExternalCache::FThumbnailExternalCacheHeader Header;
		Header.HeaderId = ThumbnailExternalCache::ExpectedHeaderId;
		Header.Version = ThumbnailExternalCache::LatestVersion;
		Header.Flags = 0;
		Header.ImageFormatName = ThumbnailExternalCache::ThumbnailImageFormatName;
		Header.Serialize(Ar);
	}
	const int64 ThumbnailTableOffsetPos = Ar.Tell() - sizeof(int64);

	// Write compressed image data
	for (TPair<FName, TSharedPtr<FSaveThumbnailCacheTask>>& It : AssetsToWrite)
	{
		// Add table of contents entry
		FPackageThumbnailRecord& PackageThumbnailRecord = PackageThumbnailRecords.AddDefaulted_GetRef();
		PackageThumbnailRecord.Name = It.Key;

		// Image data
		FSaveThumbnailCacheTask& Task = *It.Value;
		FSaveThumbnailCacheDeduplicateKey DeduplicateKey(Task.CompressedBytesHash, Task.ObjectThumbnail.GetCompressedDataSize());
		if (const int64* ExistingOffset = DeduplicateMap.Find(DeduplicateKey))
		{
			// Reference existing compressed image data
			PackageThumbnailRecord.Offset = *ExistingOffset;
			DuplicateBytesSaved += DeduplicateKey.NumBytes;
			++NumDuplicates;
		}
		else
		{
			// Save compressed image data
			PackageThumbnailRecord.Offset = Ar.Tell();
			Task.ObjectThumbnail.Serialize(Ar);
			DeduplicateMap.Add(DeduplicateKey, PackageThumbnailRecord.Offset);
			TotalCompressedBytes += DeduplicateKey.NumBytes;
		}

		// Free memory
		Task.ObjectThumbnail.AccessCompressedImageData().Empty();
	}

	// Save table of contents
	int64 NewThumbnailTableOffset = Ar.Tell();

	int64 NumThumbnails = PackageThumbnailRecords.Num();
	Ar << NumThumbnails;
	{
		FString ThumbnailNameString;
		int64 Index = 0;
		for (FPackageThumbnailRecord& PackageThumbnailRecord : PackageThumbnailRecords)
		{
			ThumbnailNameString.Reset();
			PackageThumbnailRecord.Name.AppendString(ThumbnailNameString);
			UE_LOGF(LogThumbnailExternalCache, Verbose, "\t[%lld] %ls", Index++, *ThumbnailNameString);
			Ar << ThumbnailNameString;
			Ar << PackageThumbnailRecord.Offset;
		}
	}

	// Modify top of archive to know where table of contents is located
	Ar.Seek(ThumbnailTableOffsetPos);
	Ar << NewThumbnailTableOffset;

	const double SaveTime = FPlatformTime::Seconds() - SaveTimeStart;

	UE_LOGF(LogThumbnailExternalCache, Log, "Load Time: %f secs, Save Time: %f secs, Total Time: %f secs", CombinedCache.AccumlatedLoadTime, SaveTime, (FPlatformTime::Seconds() - TimeStart) + CombinedCache.AccumlatedLoadTime);
	UE_LOGF(LogThumbnailExternalCache, Log, "Thumbnails: %d, %f MB", PackageThumbnailRecords.Num(), (TotalCompressedBytes / (1024.0 * 1024.0)));
	UE_LOGF(LogThumbnailExternalCache, Log, "Duplicates: %d, %f MB", NumDuplicates, (DuplicateBytesSaved / (1024.0 * 1024.0)));
}

void FSaveThumbnailCache::Save(FArchive& Ar, const TArrayView<FAssetData> InAssetDatas, const FThumbnailExternalCacheSettings& InSettings)
{
	FCombinedThumbnailCacheToSave CombinedCache;
	CombinedCache.Settings = InSettings;

	FThumbnailExternalCache::Get().LoadCompressAndAppend(InAssetDatas, CombinedCache);
	Save(Ar, CombinedCache, true);
}

void FSaveThumbnailCacheTask::Compress(const FThumbnailExternalCacheSettings& InSettings)
{
	ThumbnailExternalCache::ResizeThumbnailIfNeeded(ObjectThumbnail, InSettings.MaxImageSize);

	if (ObjectThumbnail.GetCompressedDataSize() > 0)
	{
		if (InSettings.bRecompressLossless)
		{
			// See if compressor would change
			FThumbnailCompressionInterface* SourceCompressor = ObjectThumbnail.GetCompressor();
			FThumbnailCompressionInterface* DestCompressor = ObjectThumbnail.ChooseNewCompressor();
			if (SourceCompressor != DestCompressor && SourceCompressor && DestCompressor)
			{
				// Do not recompress lossy images because they are already likely small and artifacts in the image would increase
				if (SourceCompressor->IsLosslessCompression())
				{
					// Force decompress if needed so we can compress again
					ObjectThumbnail.GetUncompressedImageData();

					// Delete existing compressed image data and compress again
					ObjectThumbnail.CompressImageData();
				}
			}
		}
	}
	else
	{
		ObjectThumbnail.CompressImageData();
	}

	CompressedBytesHash = CityHash64(reinterpret_cast<const char*>(ObjectThumbnail.AccessCompressedImageData().GetData()), ObjectThumbnail.GetCompressedDataSize());

	// Release uncompressed image memory
	ObjectThumbnail.AccessImageData().Empty();
}

FThumbnailExternalCache::FThumbnailExternalCache()
{
}

FThumbnailExternalCache::~FThumbnailExternalCache()
{
	Cleanup();
}

FThumbnailExternalCache& FThumbnailExternalCache::Get()
{
	static FThumbnailExternalCache ThumbnailExternalCache;
	return ThumbnailExternalCache;
}

const FString& FThumbnailExternalCache::GetCachedEditorThumbnailsFilename()
{
	static const FString Filename = TEXT("CachedEditorThumbnails.bin");
	return Filename;
}

void FThumbnailExternalCache::Init()
{
	if (!bHasInit)
	{
		bHasInit = true;

		// Load file for project
		LoadCacheFileIndex(FPaths::ProjectDir() / FThumbnailExternalCache::GetCachedEditorThumbnailsFilename());

		// Load any thumbnail files for content plugins
		TArray<TSharedRef<IPlugin>> ContentPlugins = IPluginManager::Get().GetEnabledPluginsWithContent();
		for (const TSharedRef<IPlugin>& ContentPlugin : ContentPlugins)
		{
			LoadCacheFileIndexForPlugin(ContentPlugin);
		}

		// Look for cache file when a new path is mounted
		FPackageName::OnContentPathMounted().AddRaw(this, &FThumbnailExternalCache::OnContentPathMounted);

		// Unload cache file when path is unmounted
		FPackageName::OnContentPathDismounted().AddRaw(this, &FThumbnailExternalCache::OnContentPathDismounted);
	}
}

void FThumbnailExternalCache::Cleanup()
{
	if (bHasInit)
	{
		FPackageName::OnContentPathMounted().RemoveAll(this);
		FPackageName::OnContentPathDismounted().RemoveAll(this);
	}
}

bool FThumbnailExternalCache::LoadThumbnailsFromExternalCache(const TSet<FName>& InObjectFullNames, FThumbnailMap& InOutThumbnails)
{
	if (bIsSavingCache)
	{
		return false;
	}

	Init();

	if (CacheFiles.Num() == 0)
	{
		return false;
	}

	static const FString BlueprintGeneratedClassPrefix = TEXT("/Script/Engine.BlueprintGeneratedClass ");

	int32 NumLoaded = 0;
	for (const FName ObjectFullName : InObjectFullNames)
	{
		FName ThumbnailName = ObjectFullName;

		FNameBuilder NameBuilder(ObjectFullName);
		FStringView NameView(NameBuilder);

		// BlueprintGeneratedClass assets can be displayed in content browser but thumbnails are usually not saved to package file for them
		if (NameView.StartsWith(BlueprintGeneratedClassPrefix) && NameView.EndsWith(TEXT("_C")))
		{
			// Look for the thumbnail of the Blueprint version of this object instead
			FNameBuilder ModifiedNameBuilder;
			ModifiedNameBuilder.Append(TEXT("/Script/Engine.Blueprint "));
			FStringView ViewToAppend = NameView;
			ViewToAppend.RightChopInline(BlueprintGeneratedClassPrefix.Len());
			ViewToAppend.LeftChopInline(2);
			ModifiedNameBuilder.Append(ViewToAppend);
			ThumbnailName = FName(ModifiedNameBuilder.ToView());
		}

		for (TPair<FString, TSharedPtr<FThumbnailCacheFile>>& It : CacheFiles)
		{
			TSharedPtr<FThumbnailCacheFile>& ThumbnailCacheFile = It.Value;
			if (FThumbnailEntry* Found = ThumbnailCacheFile->NameToEntry.Find(ThumbnailName))
			{
				if (ThumbnailCacheFile->bUnableToOpenFile == false)
				{
					if (TUniquePtr<FArchive> FileReader = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*ThumbnailCacheFile->Filename)))
					{
						FileReader->Seek(Found->Offset);

						if (ensure(!FileReader->IsError()))
						{
							FObjectThumbnail ObjectThumbnail;
							(*FileReader) << ObjectThumbnail;
							
							InOutThumbnails.Add(ObjectFullName, ObjectThumbnail);
							++NumLoaded;
						}
					}
					else
					{
						// Avoid retrying if file no longer exists
						ThumbnailCacheFile->bUnableToOpenFile = true;
					}
				}
			}
		}
	}

	return NumLoaded > 0;
}

void FThumbnailExternalCache::SortAssetDatas(TArray<FAssetData>& AssetDatas)
{
	Algo::SortBy(AssetDatas, [](const FAssetData& Data) { return Data.PackageName; }, FNameLexicalLess());
}

bool FThumbnailExternalCache::SaveExternalCache(const FString& InFilename, FCombinedThumbnailCacheToSave& InCache, const bool bSort)
{
	bIsSavingCache = true;
	if (TUniquePtr<FArchive> FileWriter = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*InFilename)))
	{
		FSaveThumbnailCache SaveJob;
		SaveJob.Save(*FileWriter, InCache, bSort);
		bIsSavingCache = false;
		return true;
	}

	bIsSavingCache = false;

	return false;
}

bool FThumbnailExternalCache::SaveExternalCache(const FString& InFilename, const TArrayView<FAssetData> InAssetDatas, const FThumbnailExternalCacheSettings& InSettings)
{
	bIsSavingCache = true;
	if (TUniquePtr<FArchive> FileWriter = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*InFilename)))
	{
		SaveExternalCache(*FileWriter, InAssetDatas, InSettings);
		bIsSavingCache = false;
		return true;
	}

	bIsSavingCache = false;

	return false;
}

void FThumbnailExternalCache::SaveExternalCache(FArchive& Ar, const TArrayView<FAssetData> InAssetDatas, const FThumbnailExternalCacheSettings& InSettings)
{
	bIsSavingCache = true;
	FSaveThumbnailCache SaveJob;
	SaveJob.Save(Ar, InAssetDatas, InSettings);
	bIsSavingCache = false;
}

void FThumbnailExternalCache::OnContentPathMounted(const FString& InAssetPath, const FString& InFilesystemPath)
{
	if (TSharedPtr<IPlugin> FoundPlugin = IPluginManager::Get().FindPluginFromPath(InAssetPath))
	{
		LoadCacheFileIndexForPlugin(FoundPlugin);
	}
}

void FThumbnailExternalCache::OnContentPathDismounted(const FString& InAssetPath, const FString& InFilesystemPath)
{
	if (TSharedPtr<IPlugin> FoundPlugin = IPluginManager::Get().FindPluginFromPath(InAssetPath))
	{
		if (FoundPlugin->CanContainContent())
		{
			const FString Filename = FoundPlugin->GetBaseDir() / FThumbnailExternalCache::GetCachedEditorThumbnailsFilename();
			CacheFiles.Remove(Filename);
		}
	}
}

void FThumbnailExternalCache::LoadCacheFileIndexForPlugin(const TSharedPtr<IPlugin> InPlugin)
{
	if (InPlugin && InPlugin->CanContainContent())
	{
		const FString Filename = InPlugin->GetBaseDir() / FThumbnailExternalCache::GetCachedEditorThumbnailsFilename();
		if (IFileManager::Get().FileExists(*Filename))
		{
			LoadCacheFileIndex(Filename);
		}
	}
}

bool FThumbnailExternalCache::LoadCacheFileIndex(const FString& Filename)
{
	// Stop if attempt to load already made
	if (CacheFiles.Contains(Filename))
	{
		return true;
	}

	// Track file
	TSharedPtr<FThumbnailCacheFile> ThumbnailCacheFile = MakeShared<FThumbnailCacheFile>();
	ThumbnailCacheFile->Filename = Filename;
	ThumbnailCacheFile->bUnableToOpenFile = true;
	CacheFiles.Add(Filename, ThumbnailCacheFile);

	// Attempt load index of file
	if (TUniquePtr<FArchive> FileReader = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*Filename)))
	{
		if (LoadCacheFileIndex(*FileReader, ThumbnailCacheFile))
		{
			ThumbnailCacheFile->bUnableToOpenFile = false;
			return true;
		}
	}

	return false;
}

bool FThumbnailExternalCache::LoadCacheFileIndex(FArchive& Ar, const TSharedPtr<FThumbnailCacheFile>& CacheFile)
{
	FThumbnailExternalCacheHeader& Header = CacheFile->Header;
	Header.Serialize(Ar);

	if (Header.HeaderId != ThumbnailExternalCache::ExpectedHeaderId)
	{
		return false;
	}

	if (Header.Version != 0)
	{
		return false;
	}

	Ar.Seek(Header.ThumbnailTableOffset);

	int64 NumPackages = 0;
	Ar << NumPackages;

	CacheFile->NameToEntry.Reserve(IntCastChecked<int32>(NumPackages));

	FString PackageNameString;
	for (int64 i=0; i < NumPackages; ++i)
	{
		PackageNameString.Reset();
		Ar << PackageNameString;

		FThumbnailEntry NewEntry;
		Ar << NewEntry.Offset;

		CacheFile->NameToEntry.Add(FName(PackageNameString), NewEntry);
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
