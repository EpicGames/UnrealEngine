// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailHelpers.h"
#include "Engine/Level.h"
#include "FinalPostProcessSettings.h"
#include "SceneView.h"
#include "Components/PrimitiveComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Editor/UnrealEdEngine.h"
#include "Animation/AnimBlueprint.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "ThumbnailRendering/SceneThumbnailInfoWithPrimitive.h"
#include "Particles/ParticleSystemComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"
#include "Components/DirectionalLightComponent.h"
#include "UnrealEdGlobals.h"
#include "FXSystem.h"
#include "ContentStreaming.h"
#include "Materials/Material.h"
#include "MaterialShared.h"
#include "Animation/AnimSingleNodeInstance.h"
#include "Engine/TextureCube.h"
#include "Animation/BlendSpace1D.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "SceneInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ThumbnailHelpers)

/*
***************************************************************
  FThumbnailPreviewScene
***************************************************************
*/

FThumbnailPreviewScene::FThumbnailPreviewScene(const FConstructionValues& InCVS)
	: FPreviewScene(FPreviewScene::ConstructionValues()
						.SetLightRotation( FRotator(304.736, 39.84, 0) )
						.SetSkyBrightness(1.69f)
						.SetCreatePhysicsScene(false)
						.SetTransactional(false))
{
	// A background sky sphere
	if (InCVS.bCreateSkySphere)
	{
		UStaticMeshComponent* BackgroundComponent = NewObject<UStaticMeshComponent>();
 		BackgroundComponent->SetStaticMesh( GUnrealEd->GetThumbnailManager()->EditorSkySphere );
 		const float SkySphereScale = 2000.0f;
 		const FTransform BackgroundTransform(FRotator(0,0,0), FVector(0,0,0), FVector(SkySphereScale));
 		FPreviewScene::AddComponent(BackgroundComponent, BackgroundTransform);
	}

	// Adjust the default light
	DirectionalLight->Intensity = 0.2f;

	if (InCVS.bDefaultLightingThumbnailScene)
	{
		// Add additional lights
		UDirectionalLightComponent* DirectionalLight2 = NewObject<UDirectionalLightComponent>();
		DirectionalLight2->Intensity = InCVS.LightBrightness2;
		DirectionalLight2->ForwardShadingPriority = 1;
		AddComponent(DirectionalLight2, FTransform(InCVS.LightRotation2));

		UDirectionalLightComponent* DirectionalLight3 = NewObject<UDirectionalLightComponent>();
		DirectionalLight3->Intensity = InCVS.LightBrightness3;
		DirectionalLight2->ForwardShadingPriority = 2;
		AddComponent(DirectionalLight3, FTransform(InCVS.LightRotation3));
	}

	if (InCVS.bSetSkyCubeMap)
	{
		SetSkyCubemap(GUnrealEd->GetThumbnailManager()->AmbientCubemap);
	}

	// Add an infinite plane
	if (InCVS.bCreateFloorPlane)
	{
		const float FloorPlaneScale = 10000.f;
		const FTransform FloorPlaneTransform(FRotator(-90.f,0,0), FVector::ZeroVector, FVector(FloorPlaneScale));
		UStaticMeshComponent* FloorPlaneComponent = NewObject<UStaticMeshComponent>();
		FloorPlaneComponent->SetStaticMesh( GUnrealEd->GetThumbnailManager()->EditorPlane );
		FloorPlaneComponent->SetMaterial( 0, GUnrealEd->GetThumbnailManager()->FloorPlaneMaterial );
		FPreviewScene::AddComponent(FloorPlaneComponent, FloorPlaneTransform);
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FThumbnailPreviewScene::GetView(FSceneViewFamily* ViewFamily, int32 X, int32 Y, uint32 SizeX, uint32 SizeY) const
{
	// CreateView allocates a FSceneView, which is only accessible as a const pointer in FSceneViewFamily afterwards so CreateView is the new way and is marked as [[nodiscard]], hence the static_cast<void>, to avoid a compiler warning :
	static_cast<void>(CreateView(ViewFamily, X, Y, SizeX, SizeY)); 
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FSceneView* FThumbnailPreviewScene::CreateView(FSceneViewFamily * ViewFamily, int32 X, int32 Y, uint32 SizeX, uint32 SizeY) const
{
	check(ViewFamily);

	FIntRect ViewRect(
		FMath::Max<int32>(X,0),
		FMath::Max<int32>(Y,0),
		FMath::Max<int32>(X+SizeX,0),
		FMath::Max<int32>(Y+SizeY,0));

	if (ViewRect.Area() <= 0)
	{
		return nullptr;
	}

	const float FOVDegrees = GetFOV();
	const float HalfFOVRadians = FMath::DegreesToRadians<float>(FOVDegrees) * 0.5f;
	const float NearPlane = 1.0f;
	FMatrix ProjectionMatrix = FReversedZPerspectiveMatrix(
		HalfFOVRadians,
		1.0f,
		1.0f,
		NearPlane
		);

	FVector Origin(0);
	float OrbitPitch = 0;
	float OrbitYaw = 0;
	float OrbitZoom = 0;
	GetViewMatrixParameters(FOVDegrees, Origin, OrbitPitch, OrbitYaw, OrbitZoom);

	// Ensure a minimum camera distance to prevent problems with really small objects
	if (ShouldClampOrbitZoom())
	{
		const float MinCameraDistance = 48;
		OrbitZoom = FMath::Max<float>(MinCameraDistance, OrbitZoom);
	}

	const FRotator RotationOffsetToViewCenter(0.f, 90.f, 0.f);
	FMatrix ViewRotationMatrix = FRotationMatrix( FRotator(0, OrbitYaw, 0) ) * 
		FRotationMatrix( FRotator(0, 0, OrbitPitch) ) *
		FTranslationMatrix( FVector(0, OrbitZoom, 0) ) *
		FInverseRotationMatrix( RotationOffsetToViewCenter );

	ViewRotationMatrix = ViewRotationMatrix * FMatrix(
		FPlane(0,	0,	1,	0),
		FPlane(1,	0,	0,	0),
		FPlane(0,	1,	0,	0),
		FPlane(0,	0,	0,	1));

	Origin -= ViewRotationMatrix.InverseTransformPosition(FVector::ZeroVector);
	ViewRotationMatrix = ViewRotationMatrix.RemoveTranslation();

	FSceneViewInitOptions ViewInitOptions;
	ViewInitOptions.ViewFamily = ViewFamily;
	ViewInitOptions.SetViewRectangle(ViewRect);
	ViewInitOptions.ViewOrigin = -Origin;
	ViewInitOptions.ViewRotationMatrix = ViewRotationMatrix;
	ViewInitOptions.ProjectionMatrix = ProjectionMatrix;
	ViewInitOptions.BackgroundColor = FLinearColor::Black;

	FSceneView* NewView = new FSceneView(ViewInitOptions);

	ViewFamily->Views.Add(NewView);

	NewView->StartFinalPostprocessSettings( ViewInitOptions.ViewOrigin );
	NewView->EndFinalPostprocessSettings(ViewInitOptions);
		
	// Tell the texture streaming system about this thumbnail view, so the textures will stream in as needed
	// NOTE: Sizes may not actually be in screen space depending on how the thumbnail ends up stretched by the UI.  Not a big deal though.
	// NOTE: Textures still take a little time to stream if the view has not been re-rendered recently, so they may briefly appear blurry while mips are prepared
	// NOTE: Content Browser only renders thumbnails for loaded assets, and only when the mouse is over the panel. They'll be frozen in their last state while the mouse cursor is not over the panel.  This is for performance reasons
	float ScreenSize = static_cast<float>(SizeX);
	float FOVScreenSize = static_cast<float>(SizeX) / FMath::Tan(FOVDegrees);
	IStreamingManager::Get().AddViewInformation(Origin, ScreenSize, FOVScreenSize);

	return NewView;
}

void FThumbnailPreviewScene::Tick(float DeltaTime)
{
	UpdateCaptureContents();
}

TStatId FThumbnailPreviewScene::GetStatId() const
{
	return TStatId();
}

float FThumbnailPreviewScene::GetBoundsZOffset(const FBoxSphereBounds& Bounds) const
{
	// Return half the height of the bounds plus one to avoid ZFighting with the floor plane
	return static_cast<float>(Bounds.BoxExtent.Z + 1.0);
}

float FThumbnailPreviewScene::GetFOV() const
{
	// Default FOV value
	return 30.0f;
}

/*
***************************************************************
  FParticleSystemThumbnailScene
***************************************************************
*/

FParticleSystemThumbnailScene::FParticleSystemThumbnailScene()
	: FThumbnailPreviewScene()
{
	bForceAllUsedMipsResident = false;
	PartComponent = NULL;

	ThumbnailFXSystem = FFXSystemInterface::Create(GetScene()->GetFeatureLevel(), GetScene());
	GetScene()->SetFXSystem( ThumbnailFXSystem.Get());
}

FParticleSystemThumbnailScene::~FParticleSystemThumbnailScene()
{
	FFXSystemInterface::MarkPendingKill( ThumbnailFXSystem.Get() );
}

void FParticleSystemThumbnailScene::SetParticleSystem(UParticleSystem* ParticleSystem)
{
	bool bNewComponent = false;

	// If no preview component currently existing - create it now and warm it up.
	if ( ParticleSystem && !ParticleSystem->PreviewComponent)
	{
		ParticleSystem->PreviewComponent = NewObject<UParticleSystemComponent>();
		ParticleSystem->PreviewComponent->Template = ParticleSystem;

		ParticleSystem->PreviewComponent->SetComponentToWorld(FTransform::Identity);

		bNewComponent = true;
	}

	if ( ParticleSystem == NULL || PartComponent != ParticleSystem->PreviewComponent )
	{
		if ( PartComponent != NULL )
		{
			PartComponent->ResetParticles(/*bEmptyInstances=*/ true);
			FPreviewScene::RemoveComponent(PartComponent);
		}

		if ( ParticleSystem != NULL )
		{
			PartComponent = ParticleSystem->PreviewComponent;
			check(PartComponent);

			// Add Particle component to this scene.
			FPreviewScene::AddComponent(PartComponent,FTransform::Identity);

			PartComponent->InitializeSystem();
			PartComponent->ActivateSystem();

			// If its new - tick it so its at the warmup time.
			//		if (bNewComponent && (PartComponent->WarmupTime == 0.0f))
			if (PartComponent->WarmupTime == 0.0f)
			{
				ParticleSystem->PreviewComponent->ResetBurstLists();

				float WarmupElapsed = 0.f;
				float WarmupTimestep = 0.02f;
				while(WarmupElapsed < ParticleSystem->ThumbnailWarmup)
				{
					ParticleSystem->PreviewComponent->TickComponent(WarmupTimestep, LEVELTICK_All, NULL);
					WarmupElapsed += WarmupTimestep;
					ThumbnailFXSystem->Tick(ParticleSystem->PreviewComponent->GetWorld(), WarmupTimestep);
				}
			}
		}
	}
}

void FParticleSystemThumbnailScene::GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const
{
	check(PartComponent);
	check(PartComponent->Template);

	UParticleSystem* ParticleSystem = PartComponent->Template;

	OutOrigin = FVector::ZeroVector;
	OutOrbitPitch = -11.25f;
	OutOrbitYaw = -157.5f;
	OutOrbitZoom = ParticleSystem->ThumbnailDistance;
}


/*
***************************************************************
  FMaterialThumbnailScene
***************************************************************
*/

FMaterialThumbnailScene::FMaterialThumbnailScene()
	: FThumbnailPreviewScene()
{
	bForceAllUsedMipsResident = false;

	// Create preview actor
	// checked
	FActorSpawnParameters SpawnInfo;
	SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnInfo.bNoFail = true;
	SpawnInfo.ObjectFlags = RF_Transient;
	PreviewActor = GetWorld()->SpawnActor<AStaticMeshActor>( SpawnInfo );

	PreviewActor->GetStaticMeshComponent()->SetCanEverAffectNavigation(false);
	PreviewActor->GetStaticMeshComponent()->SetMobility(EComponentMobility::Movable);
	PreviewActor->GetStaticMeshComponent()->bSelectable = false;	// avoid generating hit proxies
	PreviewActor->SetActorEnableCollision(false);
}

void FMaterialThumbnailScene::SetMaterialInterface(UMaterialInterface* InMaterial)
{
	check(PreviewActor);
	check(PreviewActor->GetStaticMeshComponent());

	bForcePlaneThumbnail = false;

	if ( InMaterial )
	{
		// Transform the preview mesh as necessary
		FTransform Transform = FTransform::Identity;

		const USceneThumbnailInfoWithPrimitive* ThumbnailInfo = Cast<USceneThumbnailInfoWithPrimitive>(InMaterial->ThumbnailInfo);
		if ( !ThumbnailInfo )
		{
			ThumbnailInfo = USceneThumbnailInfoWithPrimitive::StaticClass()->GetDefaultObject<USceneThumbnailInfoWithPrimitive>();
		}

		UMaterial* BaseMaterial = InMaterial->GetBaseMaterial();

		if(BaseMaterial)
		{
			bForcePlaneThumbnail = BaseMaterial->ShouldForcePlanePreview();
		}
		else
		{
			bForcePlaneThumbnail = InMaterial->ShouldForcePlanePreview();
		}

		EThumbnailPrimType PrimitiveType = bForcePlaneThumbnail ? TPT_Plane : ThumbnailInfo->PrimitiveType.GetValue();

		switch( PrimitiveType )
		{
		case TPT_None:
			{
				bool bFoundCustomMesh = false;
				if ( ThumbnailInfo->PreviewMesh.IsValid() )
				{
					UStaticMesh* MeshToUse = Cast<UStaticMesh>(ThumbnailInfo->PreviewMesh.ResolveObject());
					if ( MeshToUse )
					{
						PreviewActor->GetStaticMeshComponent()->SetStaticMesh(MeshToUse);
						bFoundCustomMesh = true;
					}
				}
				
				if ( !bFoundCustomMesh )
				{
					// Just use a plane if the mesh was not found
					Transform.SetRotation(FQuat(FRotator(0, -90, 0)));
					PreviewActor->GetStaticMeshComponent()->SetStaticMesh(GUnrealEd->GetThumbnailManager()->EditorPlane);
				}
			}
			break;

		case TPT_Cube:
			PreviewActor->GetStaticMeshComponent()->SetStaticMesh(GUnrealEd->GetThumbnailManager()->EditorCube);
			break;

		case TPT_Sphere:
			// The sphere is a little big, scale it down to 256x256x256
			Transform.SetScale3D( FVector(0.8f) );
			PreviewActor->GetStaticMeshComponent()->SetStaticMesh(GUnrealEd->GetThumbnailManager()->EditorSphere);
			break;

		case TPT_Cylinder:
			PreviewActor->GetStaticMeshComponent()->SetStaticMesh(GUnrealEd->GetThumbnailManager()->EditorCylinder);
			break;

		case TPT_Plane:
			// The plane needs to be rotated 90 degrees to face the camera
			Transform.SetRotation(FQuat(FRotator(0, -90, 0)));
			PreviewActor->GetStaticMeshComponent()->SetStaticMesh(GUnrealEd->GetThumbnailManager()->EditorPlane);
			break;
			
		case TPT_ShaderBall:
			PreviewActor->GetStaticMeshComponent()->SetStaticMesh(GUnrealEd->GetThumbnailManager()->EditorShaderBall);
			break;

		default:
			check(0);
		}

		PreviewActor->GetStaticMeshComponent()->SetRelativeTransform(Transform);
		PreviewActor->GetStaticMeshComponent()->UpdateBounds();

		// Center the mesh at the world origin then offset to put it on top of the plane
		const float BoundsZOffset = GetBoundsZOffset(PreviewActor->GetStaticMeshComponent()->Bounds);
		Transform.SetLocation(-PreviewActor->GetStaticMeshComponent()->Bounds.Origin + FVector(0, 0, BoundsZOffset));

		PreviewActor->GetStaticMeshComponent()->SetRelativeTransform(Transform);
	}

	PreviewActor->GetStaticMeshComponent()->SetMaterial(0, InMaterial);
	PreviewActor->GetStaticMeshComponent()->RecreateRenderState_Concurrent();
}

bool FMaterialThumbnailScene::ShouldSetSeparateTranslucency(class UMaterialInterface* InMaterial) const
{
	return InMaterial->GetMaterialResource(GMaxRHIShaderPlatform) != nullptr ? InMaterial->GetMaterialResource(GMaxRHIShaderPlatform)->IsTranslucencyAfterDOFEnabled() : false;
}

void FMaterialThumbnailScene::GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const
{
	check(PreviewActor);
	check(PreviewActor->GetStaticMeshComponent());
	check(PreviewActor->GetStaticMeshComponent()->GetMaterial(0));

	// Fit the mesh in the view using the following formula
	// tan(HalfFOV) = Width/TargetCameraDistance
	const float HalfFOVRadians = FMath::DegreesToRadians<float>(InFOVDegrees) * 0.5f;
	// Add extra size to view slightly outside of the bounds to compensate for perspective
	const float BoundsMultiplier = 1.15f;
	const float HalfMeshSize = static_cast<float>(PreviewActor->GetStaticMeshComponent()->Bounds.SphereRadius * BoundsMultiplier);
	const float BoundsZOffset = GetBoundsZOffset(PreviewActor->GetStaticMeshComponent()->Bounds);
	const float TargetDistance = HalfMeshSize / FMath::Tan(HalfFOVRadians);

	// Since we're using USceneThumbnailInfoWithPrimitive in SetMaterialInterface, we should use it here instead of USceneThumbnailInfoWithPrimitive for consistency.
	USceneThumbnailInfoWithPrimitive* ThumbnailInfo = Cast<USceneThumbnailInfoWithPrimitive>(PreviewActor->GetStaticMeshComponent()->GetMaterial(0)->ThumbnailInfo);
	if ( ThumbnailInfo )
	{
		if ( TargetDistance + ThumbnailInfo->OrbitZoom < 0 )
		{
			ThumbnailInfo->OrbitZoom = -TargetDistance;
		}
	}
	else
	{
		ThumbnailInfo = USceneThumbnailInfoWithPrimitive::StaticClass()->GetDefaultObject<USceneThumbnailInfoWithPrimitive>();
	}

	OutOrigin = FVector(0, 0, -BoundsZOffset);
	OutOrbitPitch = bForcePlaneThumbnail ? 0.0f : ThumbnailInfo->OrbitPitch;
	OutOrbitYaw = ThumbnailInfo->OrbitYaw;
	OutOrbitZoom = TargetDistance + ThumbnailInfo->OrbitZoom;
}


/*
***************************************************************
  FSkeletalMeshThumbnailScene
***************************************************************
*/

FSkeletalMeshThumbnailScene::FSkeletalMeshThumbnailScene()
	: FThumbnailPreviewScene()
{
	bForceAllUsedMipsResident = false;
	// Create preview actor
	// checked
	FActorSpawnParameters SpawnInfo;
	SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnInfo.bNoFail = true;
	SpawnInfo.ObjectFlags = RF_Transient;
	PreviewActor = GetWorld()->SpawnActor<ASkeletalMeshActor>( SpawnInfo );

	PreviewActor->SetActorEnableCollision(false);
}

void FSkeletalMeshThumbnailScene::SetSkeletalMesh(USkeletalMesh* InSkeletalMesh)
{
	PreviewActor->GetSkeletalMeshComponent()->OverrideMaterials.Empty();
	PreviewActor->GetSkeletalMeshComponent()->SetSkeletalMesh(InSkeletalMesh, false);
	PreviewActor->GetSkeletalMeshComponent()->SetDrawDebugSkeleton(bDrawDebugSkeleton);
	PreviewActor->GetSkeletalMeshComponent()->SetDebugDrawColor(DrawDebugColor);

	if ( InSkeletalMesh )
	{
		FTransform MeshTransform = FTransform::Identity;

		PreviewActor->SetActorLocation(FVector(0,0,0), false);
		PreviewActor->GetSkeletalMeshComponent()->UpdateBounds();

		// Center the mesh at the world origin then offset to put it on top of the plane
		const float BoundsZOffset = GetBoundsZOffset(PreviewActor->GetSkeletalMeshComponent()->Bounds);
		PreviewActor->SetActorLocation( -PreviewActor->GetSkeletalMeshComponent()->Bounds.Origin + FVector(0, 0, BoundsZOffset), false );
		PreviewActor->GetSkeletalMeshComponent()->RecreateRenderState_Concurrent();
	}
	else
	{
		PreviewActor->GetSkeletalMeshComponent()->ClearAnimScriptInstance();
	}
}

void FSkeletalMeshThumbnailScene::SetDrawDebugSkeleton(bool bInDrawDebugSkeleton, const FLinearColor& InSkeletonColor)
{
 	bDrawDebugSkeleton = bInDrawDebugSkeleton;
	DrawDebugColor = InSkeletonColor;
	PreviewActor->GetSkeletalMeshComponent()->SetDrawDebugSkeleton(bDrawDebugSkeleton);
	PreviewActor->GetSkeletalMeshComponent()->SetDebugDrawColor(DrawDebugColor);
	PreviewActor->GetSkeletalMeshComponent()->RecreateRenderState_Concurrent();
}

void FSkeletalMeshThumbnailScene::GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const
{
	check(PreviewActor->GetSkeletalMeshComponent());

	const float HalfFOVRadians = FMath::DegreesToRadians<float>(InFOVDegrees) * 0.5f;
	// No need to add extra size to view slightly outside of the sphere to compensate for perspective since skeletal meshes already buffer bounds.
	const float HalfMeshSize = static_cast<float>(PreviewActor->GetSkeletalMeshComponent()->Bounds.SphereRadius); 
	const float BoundsZOffset = GetBoundsZOffset(PreviewActor->GetSkeletalMeshComponent()->Bounds);
	const float TargetDistance = HalfMeshSize / FMath::Tan(HalfFOVRadians);

	USceneThumbnailInfo* ThumbnailInfo = nullptr;
	if(PreviewActor->GetSkeletalMeshComponent()->GetSkeletalMeshAsset())
	{
		ThumbnailInfo = Cast<USceneThumbnailInfo>(PreviewActor->GetSkeletalMeshComponent()->GetSkeletalMeshAsset()->GetThumbnailInfo());
	}
	
	if ( ThumbnailInfo )
	{
		if ( TargetDistance + ThumbnailInfo->OrbitZoom < 0 )
		{
			ThumbnailInfo->OrbitZoom = -TargetDistance;
		}
	}
	else
	{
		ThumbnailInfo = USceneThumbnailInfo::StaticClass()->GetDefaultObject<USceneThumbnailInfo>();
	}

	OutOrigin = FVector(0, 0, -BoundsZOffset);
	OutOrbitPitch = ThumbnailInfo->OrbitPitch;
	OutOrbitYaw = ThumbnailInfo->OrbitYaw;
	OutOrbitZoom = TargetDistance + ThumbnailInfo->OrbitZoom;
}

/*
***************************************************************
  FStaticMeshThumbnailScene
***************************************************************
*/

FStaticMeshThumbnailScene::FStaticMeshThumbnailScene()
	: FThumbnailPreviewScene()
{
	bForceAllUsedMipsResident = false;

	// Create preview actor
	// checked
	FActorSpawnParameters SpawnInfo;
	SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnInfo.bNoFail = true;
	SpawnInfo.ObjectFlags = RF_Transient;
	PreviewActor = GetWorld()->SpawnActor<AStaticMeshActor>( SpawnInfo );

	PreviewActor->GetStaticMeshComponent()->SetCanEverAffectNavigation(false);
	PreviewActor->GetStaticMeshComponent()->SetMobility(EComponentMobility::Movable);
	PreviewActor->SetActorEnableCollision(false);
}

void FStaticMeshThumbnailScene::SetStaticMesh(UStaticMesh* StaticMesh)
{
	PreviewActor->GetStaticMeshComponent()->SetStaticMesh(StaticMesh);

	if ( StaticMesh )
	{
		FTransform MeshTransform = FTransform::Identity;

		PreviewActor->SetActorLocation(FVector(0,0,0), false);
		
		//Force LOD 0
		PreviewActor->GetStaticMeshComponent()->ForcedLodModel = 1;

		PreviewActor->GetStaticMeshComponent()->UpdateBounds();

		// Center the mesh at the world origin then offset to put it on top of the plane
		const float BoundsZOffset = GetBoundsZOffset(PreviewActor->GetStaticMeshComponent()->Bounds);
		PreviewActor->SetActorLocation( -PreviewActor->GetStaticMeshComponent()->Bounds.Origin + FVector(0, 0, BoundsZOffset), false );
	}

	PreviewActor->GetStaticMeshComponent()->RecreateRenderState_Concurrent();
}

void FStaticMeshThumbnailScene::SetOverrideMaterials(const TArray<class UMaterialInterface*>& OverrideMaterials)
{
	PreviewActor->GetStaticMeshComponent()->OverrideMaterials = OverrideMaterials;
	PreviewActor->GetStaticMeshComponent()->MarkRenderStateDirty();
}

void FStaticMeshThumbnailScene::GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const
{
	check(PreviewActor);
	check(PreviewActor->GetStaticMeshComponent());
	check(PreviewActor->GetStaticMeshComponent()->GetStaticMesh());

	const float HalfFOVRadians = FMath::DegreesToRadians<float>(InFOVDegrees) * 0.5f;
	// Add extra size to view slightly outside of the sphere to compensate for perspective
	const float HalfMeshSize = static_cast<float>(PreviewActor->GetStaticMeshComponent()->Bounds.SphereRadius * 1.15);
	const float BoundsZOffset = GetBoundsZOffset(PreviewActor->GetStaticMeshComponent()->Bounds);
	const float TargetDistance = HalfMeshSize / FMath::Tan(HalfFOVRadians);

	USceneThumbnailInfo* ThumbnailInfo = Cast<USceneThumbnailInfo>(PreviewActor->GetStaticMeshComponent()->GetStaticMesh()->GetThumbnailInfo());
	if ( ThumbnailInfo )
	{
		if ( TargetDistance + ThumbnailInfo->OrbitZoom < 0 )
		{
			ThumbnailInfo->OrbitZoom = -TargetDistance;
		}
	}
	else
	{
		ThumbnailInfo = USceneThumbnailInfo::StaticClass()->GetDefaultObject<USceneThumbnailInfo>();
	}

	OutOrigin = FVector(0, 0, -BoundsZOffset);
	OutOrbitPitch = ThumbnailInfo->OrbitPitch;
	OutOrbitYaw = ThumbnailInfo->OrbitYaw;
	OutOrbitZoom = TargetDistance + ThumbnailInfo->OrbitZoom;
}

/*
***************************************************************
FAnimationThumbnailScene
***************************************************************
*/
AAnimationThumbnailSkeletalMeshActor::AAnimationThumbnailSkeletalMeshActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.SetDefaultSubobjectClass<UDebugSkelMeshComponent>(TEXT("SkeletalMeshComponent0")))
{

}

FAnimationSequenceThumbnailScene::FAnimationSequenceThumbnailScene()
	: FThumbnailPreviewScene()
{
	bForceAllUsedMipsResident = false;
	// Create preview actor
	// checked
	FActorSpawnParameters SpawnInfo;
	SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnInfo.bNoFail = true;
	SpawnInfo.ObjectFlags = RF_Transient;
	PreviewActor = GetWorld()->SpawnActor<AAnimationThumbnailSkeletalMeshActor>(SpawnInfo);

	PreviewActor->SetActorEnableCollision(false);
	if (USkeletalMeshComponent* SkelMeshComp = PreviewActor->GetSkeletalMeshComponent())
	{
		SkelMeshComp->bSuppressNotifyEventDispatch = true;
	}
}

bool FAnimationSequenceThumbnailScene::SetAnimation(UAnimSequenceBase* InAnimation)
{
	PreviewActor->GetSkeletalMeshComponent()->OverrideMaterials.Empty();

	bool bSetSucessfully = false;

	PreviewAnimation = InAnimation;

	if (InAnimation)
	{
		if (USkeleton* Skeleton = InAnimation->GetSkeleton())
		{
			USkeletalMesh* PreviewSkeletalMesh = Skeleton->GetAssetPreviewMesh(InAnimation);
			if (!PreviewSkeletalMesh)
			{
				PreviewSkeletalMesh = Skeleton->FindCompatibleMesh();
			}
			PreviewActor->GetSkeletalMeshComponent()->SetSkeletalMesh(PreviewSkeletalMesh);

			if (PreviewSkeletalMesh)
			{
				bSetSucessfully = true;

				if (InAnimation->IsValidToPlay())
				{
					// Handle posing the mesh at the middle of the animation
					const float AnimPosition = InAnimation->GetPlayLength() / 2.f;

					UDebugSkelMeshComponent* MeshComponent = CastChecked<UDebugSkelMeshComponent>(PreviewActor->GetSkeletalMeshComponent());

					MeshComponent->EnablePreview(true, InAnimation);
					MeshComponent->Play(false);
					MeshComponent->Stop();
					MeshComponent->SetPosition(AnimPosition, false);

					UAnimSingleNodeInstance* SingleNodeInstance = PreviewActor->GetSkeletalMeshComponent()->GetSingleNodeInstance();
					if (SingleNodeInstance)
					{
						SingleNodeInstance->UpdateMontageWeightForTimeSkip(AnimPosition);
					}

					PreviewActor->GetSkeletalMeshComponent()->RefreshBoneTransforms(nullptr);
				}

				PreviewActor->SetActorLocation(FVector(0, 0, 0), false);
				PreviewActor->GetSkeletalMeshComponent()->UpdateBounds();

				// Center the mesh at the world origin then offset to put it on top of the plane
				const float BoundsZOffset = GetBoundsZOffset(PreviewActor->GetSkeletalMeshComponent()->Bounds);
				PreviewActor->SetActorLocation(-PreviewActor->GetSkeletalMeshComponent()->Bounds.Origin + FVector(0, 0, BoundsZOffset), false);
				PreviewActor->GetSkeletalMeshComponent()->RecreateRenderState_Concurrent();
			}
		}
	}
	
	if(!bSetSucessfully)
	{
		CleanupComponentChildren(PreviewActor->GetSkeletalMeshComponent());
		PreviewActor->GetSkeletalMeshComponent()->SetAnimation(NULL);
		PreviewActor->GetSkeletalMeshComponent()->SetSkeletalMesh(nullptr);
		PreviewActor->GetSkeletalMeshComponent()->ClearAnimScriptInstance();
	}

	return bSetSucessfully;
}

void FAnimationSequenceThumbnailScene::CleanupComponentChildren(USceneComponent* Component)
{
	if (Component)
	{
		for(int32 ComponentIdx = Component->GetAttachChildren().Num() - 1 ; ComponentIdx >= 0 ; --ComponentIdx)
		{
			CleanupComponentChildren(Component->GetAttachChildren()[ComponentIdx]);
			Component->GetAttachChildren()[ComponentIdx]->DestroyComponent();
		}
		check(Component->GetAttachChildren().Num() == 0);
	}
}

void FAnimationSequenceThumbnailScene::GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const
{
	check(PreviewAnimation);
	check(PreviewActor->GetSkeletalMeshComponent());
	check(PreviewActor->GetSkeletalMeshComponent()->GetSkeletalMeshAsset());

	const float HalfFOVRadians = FMath::DegreesToRadians<float>(InFOVDegrees) * 0.5f;
	// No need to add extra size to view slightly outside of the sphere to compensate for perspective since skeletal meshes already buffer bounds.
	const float HalfMeshSize = static_cast<float>(PreviewActor->GetSkeletalMeshComponent()->Bounds.SphereRadius);
	const float BoundsZOffset = GetBoundsZOffset(PreviewActor->GetSkeletalMeshComponent()->Bounds);
	const float TargetDistance = HalfMeshSize / FMath::Tan(HalfFOVRadians);

	USceneThumbnailInfo* ThumbnailInfo = Cast<USceneThumbnailInfo>(PreviewAnimation->ThumbnailInfo);
	if (ThumbnailInfo)
	{
		if (TargetDistance + ThumbnailInfo->OrbitZoom < 0)
		{
			ThumbnailInfo->OrbitZoom = -TargetDistance;
		}
	}
	else
	{
		ThumbnailInfo = USceneThumbnailInfo::StaticClass()->GetDefaultObject<USceneThumbnailInfo>();
	}

	OutOrigin = FVector(0, 0, -BoundsZOffset);
	OutOrbitPitch = ThumbnailInfo->OrbitPitch;
	OutOrbitYaw = ThumbnailInfo->OrbitYaw;
	OutOrbitZoom = TargetDistance + ThumbnailInfo->OrbitZoom;
}

/*
***************************************************************
FBlendSpaceThumbnailScene
***************************************************************
*/

FBlendSpaceThumbnailScene::FBlendSpaceThumbnailScene()
	: FThumbnailPreviewScene()
{
	bForceAllUsedMipsResident = false;
	// Create preview actor
	// checked
	FActorSpawnParameters SpawnInfo;
	SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnInfo.bNoFail = true;
	SpawnInfo.ObjectFlags = RF_Transient;
	PreviewActor = GetWorld()->SpawnActor<AAnimationThumbnailSkeletalMeshActor>(SpawnInfo);

	PreviewActor->SetActorEnableCollision(false);
	if (USkeletalMeshComponent* SkelMeshComp = PreviewActor->GetSkeletalMeshComponent())
	{
		SkelMeshComp->bSuppressNotifyEventDispatch = true;
	}
}

bool FBlendSpaceThumbnailScene::SetBlendSpace(class UBlendSpace* InBlendSpace)
{
	PreviewActor->GetSkeletalMeshComponent()->OverrideMaterials.Empty();

	bool bSetSucessfully = false;

	PreviewAnimation = InBlendSpace;

	if (InBlendSpace)
	{
		if (USkeleton* Skeleton = InBlendSpace->GetSkeleton())
		{
			USkeletalMesh* PreviewSkeletalMesh = Skeleton->GetAssetPreviewMesh(InBlendSpace);
			if (!PreviewSkeletalMesh)
			{
				PreviewSkeletalMesh = Skeleton->FindCompatibleMesh();
			}
			PreviewActor->GetSkeletalMeshComponent()->SetSkeletalMesh(PreviewSkeletalMesh);

			if (PreviewSkeletalMesh)
			{
				bSetSucessfully = true;

				UDebugSkelMeshComponent* MeshComponent = CastChecked<UDebugSkelMeshComponent>(PreviewActor->GetSkeletalMeshComponent());

				// Handle posing the mesh at the middle of the animation
				MeshComponent->EnablePreview(true, InBlendSpace);
				MeshComponent->Play(false);
				MeshComponent->Stop();

				UAnimSingleNodeInstance* AnimInstance = MeshComponent->GetSingleNodeInstance();
				if (AnimInstance)
				{
					FVector BlendInput(0.f);
					const int32 NumDimensions = InBlendSpace->IsA<UBlendSpace1D>() ? 1 : 2;
					for (int32 i = 0; i < NumDimensions; ++i)
					{
						const FBlendParameter& Param = InBlendSpace->GetBlendParameter(i);
						BlendInput[i] = (Param.GetRange() / 2.f) + Param.Min;
					}
					AnimInstance->UpdateBlendspaceSamples(BlendInput);
				}

				MeshComponent->TickAnimation(0.f, false);
				MeshComponent->RefreshBoneTransforms(nullptr);

				FTransform MeshTransform = FTransform::Identity;

				PreviewActor->SetActorLocation(FVector(0, 0, 0), false);
				PreviewActor->GetSkeletalMeshComponent()->UpdateBounds();

				// Center the mesh at the world origin then offset to put it on top of the plane
				const float BoundsZOffset = GetBoundsZOffset(PreviewActor->GetSkeletalMeshComponent()->Bounds);
				PreviewActor->SetActorLocation(-PreviewActor->GetSkeletalMeshComponent()->Bounds.Origin + FVector(0, 0, BoundsZOffset), false);
				PreviewActor->GetSkeletalMeshComponent()->RecreateRenderState_Concurrent();
			}
		}
	}

	if (!bSetSucessfully)
	{
		CleanupComponentChildren(PreviewActor->GetSkeletalMeshComponent());
		PreviewActor->GetSkeletalMeshComponent()->SetAnimation(NULL);
		PreviewActor->GetSkeletalMeshComponent()->SetSkeletalMesh(nullptr);
	}

	return bSetSucessfully;
}

void FBlendSpaceThumbnailScene::CleanupComponentChildren(USceneComponent* Component)
{
	if (Component)
	{
		for (int32 ComponentIdx = Component->GetAttachChildren().Num() - 1; ComponentIdx >= 0; --ComponentIdx)
		{
			CleanupComponentChildren(Component->GetAttachChildren()[ComponentIdx]);
			Component->GetAttachChildren()[ComponentIdx]->DestroyComponent();
		}
		check(Component->GetAttachChildren().Num() == 0);
	}
}

void FBlendSpaceThumbnailScene::GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const
{
	check(PreviewAnimation);
	check(PreviewActor->GetSkeletalMeshComponent());
	check(PreviewActor->GetSkeletalMeshComponent()->GetSkeletalMeshAsset());

	const float HalfFOVRadians = FMath::DegreesToRadians<float>(InFOVDegrees) * 0.5f;
	// No need to add extra size to view slightly outside of the sphere to compensate for perspective since skeletal meshes already buffer bounds.
	const float HalfMeshSize = static_cast<float>(PreviewActor->GetSkeletalMeshComponent()->Bounds.SphereRadius);
	const float BoundsZOffset = GetBoundsZOffset(PreviewActor->GetSkeletalMeshComponent()->Bounds);
	const float TargetDistance = HalfMeshSize / FMath::Tan(HalfFOVRadians);

	USceneThumbnailInfo* ThumbnailInfo = Cast<USceneThumbnailInfo>(PreviewAnimation->ThumbnailInfo);
	if (ThumbnailInfo)
	{
		if (TargetDistance + ThumbnailInfo->OrbitZoom < 0)
		{
			ThumbnailInfo->OrbitZoom = -TargetDistance;
		}
	}
	else
	{
		ThumbnailInfo = USceneThumbnailInfo::StaticClass()->GetDefaultObject<USceneThumbnailInfo>();
	}

	OutOrigin = FVector(0, 0, -BoundsZOffset);
	OutOrbitPitch = ThumbnailInfo->OrbitPitch;
	OutOrbitYaw = ThumbnailInfo->OrbitYaw;
	OutOrbitZoom = TargetDistance + ThumbnailInfo->OrbitZoom;
}

/*
***************************************************************
FAnimBlueprintThumbnailScene
***************************************************************
*/

FAnimBlueprintThumbnailScene::FAnimBlueprintThumbnailScene()
	: FThumbnailPreviewScene()
{
	bForceAllUsedMipsResident = false;
	
	// Create preview actor
	FActorSpawnParameters SpawnInfo;
	SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnInfo.bNoFail = true;
	SpawnInfo.ObjectFlags = RF_Transient;
	PreviewActor = GetWorld()->SpawnActor<ASkeletalMeshActor>(SpawnInfo);

	PreviewActor->SetActorEnableCollision(false);
	if (USkeletalMeshComponent* SkelMeshComp = PreviewActor->GetSkeletalMeshComponent())
	{
		SkelMeshComp->bSuppressNotifyEventDispatch = true;
	}
}

bool FAnimBlueprintThumbnailScene::SetAnimBlueprint(class UAnimBlueprint* InBlueprint)
{
	PreviewActor->GetSkeletalMeshComponent()->OverrideMaterials.Empty();

	bool bSetSucessfully = false;

	PreviewBlueprint = InBlueprint;

	if (InBlueprint)
	{
		if (USkeleton* Skeleton = InBlueprint->TargetSkeleton)
		{
			USkeletalMesh* PreviewSkeletalMesh = Skeleton->GetAssetPreviewMesh(InBlueprint);
			if (!PreviewSkeletalMesh)
			{
				PreviewSkeletalMesh = Skeleton->FindCompatibleMesh();
			}
			PreviewActor->GetSkeletalMeshComponent()->SetSkeletalMesh(PreviewSkeletalMesh);

			if (PreviewSkeletalMesh)
			{
				bSetSucessfully = true;

				PreviewActor->GetSkeletalMeshComponent()->SetAnimInstanceClass(InBlueprint->GeneratedClass);

				FTransform MeshTransform = FTransform::Identity;

				PreviewActor->SetActorLocation(FVector(0, 0, 0), false);
				PreviewActor->GetSkeletalMeshComponent()->UpdateBounds();

				// Center the mesh at the world origin then offset to put it on top of the plane
				const float BoundsZOffset = GetBoundsZOffset(PreviewActor->GetSkeletalMeshComponent()->Bounds);
				PreviewActor->SetActorLocation(-PreviewActor->GetSkeletalMeshComponent()->Bounds.Origin + FVector(0, 0, BoundsZOffset), false);
				PreviewActor->GetSkeletalMeshComponent()->RecreateRenderState_Concurrent();
			}
		}
	}

	if (!bSetSucessfully)
	{
		CleanupComponentChildren(PreviewActor->GetSkeletalMeshComponent());
		PreviewActor->GetSkeletalMeshComponent()->SetSkeletalMesh(nullptr);
		PreviewActor->GetSkeletalMeshComponent()->SetAnimInstanceClass(nullptr);
	}

	return bSetSucessfully;
}

void FAnimBlueprintThumbnailScene::CleanupComponentChildren(USceneComponent* Component)
{
	if (Component)
	{
		for (int32 ComponentIdx = Component->GetAttachChildren().Num() - 1; ComponentIdx >= 0; --ComponentIdx)
		{
			CleanupComponentChildren(Component->GetAttachChildren()[ComponentIdx]);
			Component->GetAttachChildren()[ComponentIdx]->DestroyComponent();
		}
		check(Component->GetAttachChildren().Num() == 0);
	}
}

void FAnimBlueprintThumbnailScene::GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const
{
	check(PreviewBlueprint);
	check(PreviewActor->GetSkeletalMeshComponent());
	check(PreviewActor->GetSkeletalMeshComponent()->GetSkeletalMeshAsset());

	const float HalfFOVRadians = FMath::DegreesToRadians<float>(InFOVDegrees) * 0.5f;
	// No need to add extra size to view slightly outside of the sphere to compensate for perspective since skeletal meshes already buffer bounds.
	const float HalfMeshSize = static_cast<float>(PreviewActor->GetSkeletalMeshComponent()->Bounds.SphereRadius);
	const float BoundsZOffset = GetBoundsZOffset(PreviewActor->GetSkeletalMeshComponent()->Bounds);
	const float TargetDistance = HalfMeshSize / FMath::Tan(HalfFOVRadians);

	USceneThumbnailInfo* ThumbnailInfo = Cast<USceneThumbnailInfo>(PreviewBlueprint->ThumbnailInfo);
	if (ThumbnailInfo)
	{
		if (TargetDistance + ThumbnailInfo->OrbitZoom < 0)
		{
			ThumbnailInfo->OrbitZoom = -TargetDistance;
		}
	}
	else
	{
		ThumbnailInfo = USceneThumbnailInfo::StaticClass()->GetDefaultObject<USceneThumbnailInfo>();
	}

	OutOrigin = FVector(0, 0, -BoundsZOffset);
	OutOrbitPitch = ThumbnailInfo->OrbitPitch;
	OutOrbitYaw = ThumbnailInfo->OrbitYaw;
	OutOrbitZoom = TargetDistance + ThumbnailInfo->OrbitZoom;
}

/*
***************************************************************
FPhysicsAssetThumbnailScene
***************************************************************
*/

FPhysicsAssetThumbnailScene::FPhysicsAssetThumbnailScene()
	: FThumbnailPreviewScene()
{
	bForceAllUsedMipsResident = false;
	// Create preview actor
	// checked
	FActorSpawnParameters SpawnInfo;
	SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnInfo.bNoFail = true;
	SpawnInfo.ObjectFlags = RF_Transient;
	PreviewActor = GetWorld()->SpawnActor<ASkeletalMeshActor>( SpawnInfo );

	PreviewActor->SetActorEnableCollision(false);
}

void FPhysicsAssetThumbnailScene::SetPhysicsAsset(UPhysicsAsset* InPhysicsAsset)
{
	PreviewActor->GetSkeletalMeshComponent()->OverrideMaterials.Empty();
	PreviewActor->SetActorEnableCollision(true);

	if(InPhysicsAsset)
	{
		USkeletalMesh* SkeletalMesh = InPhysicsAsset->PreviewSkeletalMesh.LoadSynchronous();
		if (SkeletalMesh)
		{
			PreviewActor->GetSkeletalMeshComponent()->SetSkeletalMesh(SkeletalMesh);

			FTransform MeshTransform = FTransform::Identity;

			PreviewActor->SetActorLocation(FVector(0,0,0), false);
			PreviewActor->GetSkeletalMeshComponent()->UpdateBounds();

			// Center the mesh at the world origin then offset to put it on top of the plane
			const float BoundsZOffset = GetBoundsZOffset(PreviewActor->GetSkeletalMeshComponent()->Bounds);
			PreviewActor->SetActorLocation( -PreviewActor->GetSkeletalMeshComponent()->Bounds.Origin + FVector(0, 0, BoundsZOffset), false );
			PreviewActor->GetSkeletalMeshComponent()->RecreateRenderState_Concurrent();
		}
	}
	else
	{
		PreviewActor->GetSkeletalMeshComponent()->SetSkeletalMesh(nullptr);
		PreviewActor->GetSkeletalMeshComponent()->ClearAnimScriptInstance();
	}
}

void FPhysicsAssetThumbnailScene::GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const
{
	check(PreviewActor->GetSkeletalMeshComponent());

	const float HalfFOVRadians = FMath::DegreesToRadians<float>(InFOVDegrees) * 0.5f;
	// No need to add extra size to view slightly outside of the sphere to compensate for perspective since skeletal meshes already buffer bounds.
	const float HalfMeshSize = static_cast<float>(PreviewActor->GetSkeletalMeshComponent()->Bounds.SphereRadius);
	const float BoundsZOffset = GetBoundsZOffset(PreviewActor->GetSkeletalMeshComponent()->Bounds);
	const float TargetDistance = HalfMeshSize / FMath::Tan(HalfFOVRadians);

	USceneThumbnailInfo* ThumbnailInfo = USceneThumbnailInfo::StaticClass()->GetDefaultObject<USceneThumbnailInfo>();
	if(PreviewActor->GetSkeletalMeshComponent()->GetSkeletalMeshAsset() && PreviewActor->GetSkeletalMeshComponent()->GetSkeletalMeshAsset()->GetPhysicsAsset())
	{
		if ( USceneThumbnailInfo* InteralThumbnailInfo = Cast<USceneThumbnailInfo>(PreviewActor->GetSkeletalMeshComponent()->GetSkeletalMeshAsset()->GetPhysicsAsset()->ThumbnailInfo) )
		{
			ThumbnailInfo = InteralThumbnailInfo;
			if ( TargetDistance + InteralThumbnailInfo->OrbitZoom < 0 )
			{
				InteralThumbnailInfo->OrbitZoom = -TargetDistance;
			}
		}
	}

	OutOrigin = FVector(0, 0, -BoundsZOffset);
	OutOrbitPitch = ThumbnailInfo->OrbitPitch;
	OutOrbitYaw = ThumbnailInfo->OrbitYaw;
	OutOrbitZoom = TargetDistance + ThumbnailInfo->OrbitZoom;
}

/*
***************************************************************
  FClassActorThumbnailScene
***************************************************************
*/

FClassActorThumbnailScene::FClassActorThumbnailScene()
	: FThumbnailPreviewScene()
	, NumStartingActors(0)
	, PreviewActor(nullptr)
{
	NumStartingActors = GetWorld()->GetCurrentLevel()->Actors.Num();
}

void FClassActorThumbnailScene::SpawnPreviewActor(UClass* InClass)
{
	if (PreviewActor.IsStale())
	{
		PreviewActor = nullptr;
		ClearStaleActors();
	}

	if (PreviewActor.IsValid())
	{
		if (PreviewActor->GetClass() == InClass)
		{
			return;
		}

		PreviewActor->Destroy();
		PreviewActor = nullptr;
	}
	if (InClass && !InClass->HasAnyClassFlags(CLASS_Deprecated | CLASS_Abstract))
	{
		// Create preview actor
		FActorSpawnParameters SpawnInfo;
		SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
		SpawnInfo.bNoFail = true;
		SpawnInfo.ObjectFlags = RF_Transient;
		PreviewActor = GetWorld()->SpawnActor<AActor>(InClass, SpawnInfo);

		if (PreviewActor.IsValid())
		{
			const FBoxSphereBounds Bounds = GetPreviewActorBounds();
			const float BoundsZOffset = GetBoundsZOffset(Bounds);
			const FVector TargetLocation(-Bounds.Origin + FVector(0, 0, BoundsZOffset));

			PreviewActor->SetActorLocation(TargetLocation);
		}
	}
}

void FClassActorThumbnailScene::ClearStaleActors()
{
	ULevel* Level = GetWorld()->GetCurrentLevel();

	for (int32 i = NumStartingActors; i < Level->Actors.Num(); ++i)
	{
		if (Level->Actors[i])
		{
			Level->Actors[i]->Destroy();
		}
	}
}

bool FClassActorThumbnailScene::IsValidComponentForVisualization(const UActorComponent* Component)
{
	const UPrimitiveComponent* PrimComp = Cast<UPrimitiveComponent>(Component);
	if ( PrimComp && PrimComp->IsVisible() && !PrimComp->bHiddenInGame )
	{
		const UStaticMeshComponent* StaticMeshComp = Cast<UStaticMeshComponent>(Component);
		if ( StaticMeshComp && StaticMeshComp->GetStaticMesh())
		{
			return true;
		}

		const USkeletalMeshComponent* SkelMeshComp = Cast<USkeletalMeshComponent>(Component);
		if ( SkelMeshComp && SkelMeshComp->GetSkeletalMeshAsset())
		{
			return true;
		}

		// we cannot include the geomety collection component in this module because of circular dependency 
		// so we need to check using the name of the class instead 
		const FName ClassName = Component->GetClass()->GetFName();
		const FName GeometryCollectionClassName("GeometryCollectionComponent");
		if (ClassName == GeometryCollectionClassName)
		{
			return true;
		}
	}

	return false;
}

FBoxSphereBounds FClassActorThumbnailScene::GetPreviewActorBounds() const
{
	FBoxSphereBounds::Builder BoundsBuilder;
	if (PreviewActor.IsValid() && PreviewActor->GetRootComponent())
	{
		TArray<USceneComponent*> PreviewComponents;
		PreviewActor->GetRootComponent()->GetChildrenComponents(true, PreviewComponents);
		PreviewComponents.Add(PreviewActor->GetRootComponent());

		for (USceneComponent* PreviewComponent : PreviewComponents)
		{
			if (IsValidComponentForVisualization(PreviewComponent))
			{
				BoundsBuilder += PreviewComponent->Bounds;
			}
		}
	}

	return BoundsBuilder;
}

void FClassActorThumbnailScene::GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const
{
	const float HalfFOVRadians = FMath::DegreesToRadians<float>(InFOVDegrees) * 0.5f;
	// Add extra size to view slightly outside of the sphere to compensate for perspective
	const FBoxSphereBounds Bounds = GetPreviewActorBounds();

	const float HalfMeshSize = static_cast<float>(Bounds.SphereRadius * 1.15);
	const float BoundsZOffset = GetBoundsZOffset(Bounds);
	const float TargetDistance = HalfMeshSize / FMath::Tan(HalfFOVRadians);

	USceneThumbnailInfo* ThumbnailInfo = GetSceneThumbnailInfo(TargetDistance);
	check(ThumbnailInfo);

	OutOrigin = FVector(0, 0, -BoundsZOffset);
	OutOrbitPitch = ThumbnailInfo->OrbitPitch;
	OutOrbitYaw = ThumbnailInfo->OrbitYaw;
	OutOrbitZoom = TargetDistance + ThumbnailInfo->OrbitZoom;
}

/*
***************************************************************
  FBlueprintThumbnailScene
***************************************************************
*/

FBlueprintThumbnailScene::FBlueprintThumbnailScene()
	: FClassActorThumbnailScene()
	, CurrentBlueprint(nullptr)
{
}

void FBlueprintThumbnailScene::SetBlueprint(UBlueprint* Blueprint)
{
	CurrentBlueprint = Blueprint;
	UClass* BPClass = (Blueprint ? *Blueprint->GeneratedClass : nullptr);
	SpawnPreviewActor(BPClass);
}

void FBlueprintThumbnailScene::BlueprintChanged(UBlueprint* Blueprint)
{
	if (CurrentBlueprint == Blueprint)
	{
		UClass* BPClass = (Blueprint ? *Blueprint->GeneratedClass : nullptr);
		SpawnPreviewActor(BPClass);
	}
}

USceneThumbnailInfo* FBlueprintThumbnailScene::GetSceneThumbnailInfo(const float TargetDistance) const
{
	UBlueprint* Blueprint = CurrentBlueprint.Get();
	check(Blueprint);

	USceneThumbnailInfo* ThumbnailInfo = Cast<USceneThumbnailInfo>(Blueprint->ThumbnailInfo);
	if ( ThumbnailInfo )
	{
		if ( TargetDistance + ThumbnailInfo->OrbitZoom < 0 )
		{
			ThumbnailInfo->OrbitZoom = -TargetDistance;
		}
	}
	else
	{
		ThumbnailInfo = USceneThumbnailInfo::StaticClass()->GetDefaultObject<USceneThumbnailInfo>();
	}

	return ThumbnailInfo;
}

/*
***************************************************************
  FClassThumbnailScene
***************************************************************
*/

FClassThumbnailScene::FClassThumbnailScene()
	: FClassActorThumbnailScene()
	, CurrentClass(nullptr)
{
}

void FClassThumbnailScene::SetClass(UClass* Class)
{
	CurrentClass = Class;
	SpawnPreviewActor(CurrentClass);
}

USceneThumbnailInfo* FClassThumbnailScene::GetSceneThumbnailInfo(const float TargetDistance) const
{
	// todo: jdale - CLASS - Needs proper thumbnail info for class (see FAssetTypeActions_Class::GetThumbnailInfo)
	USceneThumbnailInfo* ThumbnailInfo = USceneThumbnailInfo::StaticClass()->GetDefaultObject<USceneThumbnailInfo>();
	return ThumbnailInfo;
}
