// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailRendering/ThumbnailManager.h"

#include "Editor.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "ObjectTools.h"
#include "UObject/ConstructorHelpers.h"
#include "Materials/Material.h"
#include "ISourceControlOperation.h"
#include "SourceControlOperations.h"
#include "ISourceControlProvider.h"
#include "ISourceControlModule.h"
#include "Engine/Blueprint.h"
#include "Engine/StaticMesh.h"
#include "UnrealClient.h"
#include "Engine/TextureCube.h"
#include "Engine/Texture2DArray.h"
#include "ImageUtils.h"
#include "AssetThumbnail.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ThumbnailManager)


DEFINE_LOG_CATEGORY_STATIC(LogThumbnailManager, Log, All);

//////////////////////////////////////////////////////////////////////////

UThumbnailManager* UThumbnailManager::ThumbnailManagerSingleton = nullptr;

UThumbnailManager::UThumbnailManager(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if (!IsRunningCommandlet())
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinder<UStaticMesh> EditorCubeMesh;
			ConstructorHelpers::FObjectFinder<UStaticMesh> EditorSphereMesh;
			ConstructorHelpers::FObjectFinder<UStaticMesh> EditorCylinderMesh;
			ConstructorHelpers::FObjectFinder<UStaticMesh> EditorPlaneMesh;
			ConstructorHelpers::FObjectFinder<UStaticMesh> EditorShaderBallMesh;
			ConstructorHelpers::FObjectFinder<UStaticMesh> EditorSkySphereMesh;
			ConstructorHelpers::FObjectFinder<UMaterial> FloorPlaneMaterial;
			ConstructorHelpers::FObjectFinder<UTextureCube> DaylightAmbientCubemap;
			
			FConstructorStatics()
				: EditorCubeMesh(TEXT("/Engine/EditorMeshes/EditorCube"))
				, EditorSphereMesh(TEXT("/Engine/EditorMeshes/EditorSphere"))
				, EditorCylinderMesh(TEXT("/Engine/EditorMeshes/EditorCylinder"))
				, EditorPlaneMesh(TEXT("/Engine/EditorMeshes/EditorPlane"))
				, EditorShaderBallMesh(TEXT("/Engine/EditorMeshes/EditorShaderBall"))
				, EditorSkySphereMesh(TEXT("/Engine/EditorMeshes/EditorSkySphere"))
				, FloorPlaneMaterial(TEXT("/Engine/EditorMaterials/Thumbnails/FloorPlaneMaterial"))
				, DaylightAmbientCubemap(TEXT("/Engine/MapTemplates/Sky/DaylightAmbientCubemap"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		EditorCube = ConstructorStatics.EditorCubeMesh.Object;
		EditorSphere = ConstructorStatics.EditorSphereMesh.Object;
		EditorCylinder = ConstructorStatics.EditorCylinderMesh.Object;
		EditorPlane = ConstructorStatics.EditorPlaneMesh.Object;
		EditorShaderBall = ConstructorStatics.EditorShaderBallMesh.Object;
		EditorSkySphere = ConstructorStatics.EditorSkySphereMesh.Object;
		FloorPlaneMaterial = ConstructorStatics.FloorPlaneMaterial.Object;
		AmbientCubemap = ConstructorStatics.DaylightAmbientCubemap.Object;

		SetupCheckerboardTexture();
	}
}

void UThumbnailManager::Initialize(void)
{
	if (bIsInitialized == false)
	{
		InitializeRenderTypeArray(RenderableThumbnailTypes);

		// The size of the pool is a bit large to allow the asset views to use it (each asset view used 1024 by default)
		SharedThumbnailPool = MakeShared<FAssetThumbnailPool>(4096);

		FCoreUObjectDelegates::OnObjectPropertyChanged.AddUObject(this, &UThumbnailManager::OnObjectPropertyChanged);

		if (GEditor)
		{
			GEditor->OnActorMoved().AddUObject(this, &UThumbnailManager::OnActorPostEditMove);
		}

		bIsInitialized = true;
	}
}

void UThumbnailManager::InitializeRenderTypeArray(TArray<FThumbnailRenderingInfo>& ThumbnailRendererTypes)
{
	// Loop through setting up each thumbnail entry
	for (int32 Index = 0; Index < ThumbnailRendererTypes.Num(); Index++)
	{
		FThumbnailRenderingInfo& RenderInfo = ThumbnailRendererTypes[Index];

		// Load the class that this is for
		if (RenderInfo.ClassNeedingThumbnailName.Len() > 0)
		{
			// Try to load the specified class
			RenderInfo.ClassNeedingThumbnail = LoadObject<UClass>(nullptr, *RenderInfo.ClassNeedingThumbnailName, nullptr, LOAD_None, nullptr);
		}

		if (RenderInfo.RendererClassName.Len() > 0)
		{
			// Try to create the renderer object by loading its class and
			// constructing one
			UClass* RenderClass = LoadObject<UClass>(nullptr, *RenderInfo.RendererClassName, nullptr, LOAD_None, nullptr);
			if (RenderClass != nullptr)
			{
				RenderInfo.Renderer = NewObject<UThumbnailRenderer>(GetTransientPackage(), RenderClass);
			}
		}

		// Add this to the map if it created the renderer component
		if (RenderInfo.Renderer != nullptr)
		{
			RenderInfoMap.Add(RenderInfo.ClassNeedingThumbnail, &RenderInfo);
		}
	}
}

void UThumbnailManager::BeginDestroy()
{
	Super::BeginDestroy();

	SharedThumbnailPool.Reset();
}

FThumbnailRenderingInfo* UThumbnailManager::GetRenderingInfo(UObject* Object)
{
	// If something may have been GCed, empty the map so we don't crash
	if (bMapNeedsUpdate == true)
	{
		RenderInfoMap.Empty();
		bMapNeedsUpdate = false;
	}

	check(Object);

	TArray<FThumbnailRenderingInfo>& ThumbnailTypes = RenderableThumbnailTypes;

	auto FindThumbnailRendererForClass = [&](UObject* Object, UClass* ClassToCheck) -> FThumbnailRenderingInfo*
	{
		// Search for the cached entry and do the slower if not found
		FThumbnailRenderingInfo* RenderInfo = RenderInfoMap.FindRef(ClassToCheck);
		if (RenderInfo == nullptr)
		{
			// Loop through searching for the right thumbnail entry
			for (int32 Index = ThumbnailTypes.Num() - 1; (Index >= 0) && (RenderInfo == nullptr); Index--)
			{
				RenderInfo = &ThumbnailTypes[Index];

				// See if this thumbnail renderer will work for the specified class or
				// if there is some data reason not to render the thumbnail
				if ((ClassToCheck->IsChildOf(RenderInfo->ClassNeedingThumbnail) == false) || (RenderInfo->Renderer == nullptr))
				{
					RenderInfo = nullptr;
				}
			}

			// Make sure to add it to the cache if it is missing
			RenderInfoMap.Add(ClassToCheck, (RenderInfo != nullptr) ? RenderInfo : &NotSupported);
		}

		if (RenderInfo && RenderInfo->Renderer && !RenderInfo->Renderer->CanVisualizeAsset(Object))
		{
			// This is an asset with a thumbnail renderer, but it can't visualized (i.e it is something like a blueprint that doesn't contain any visible primitive components)
			RenderInfo = nullptr;
		}

		return RenderInfo;
	};

	FThumbnailRenderingInfo* RenderInfo = FindThumbnailRendererForClass(Object, Object->GetClass());

	// For blueprints that the UBlueprint subclass renderer didn't want to deal with, ask the class itself
	if (RenderInfo == nullptr)
	{
		if (UBlueprint* Blueprint = Cast<UBlueprint>(Object))
		{
			if (Blueprint->GeneratedClass != nullptr)
			{
				UObject* ClassCDO = Blueprint->GeneratedClass->GetDefaultObject(false);
				RenderInfo = FindThumbnailRendererForClass(ClassCDO, Blueprint->GeneratedClass);
				if (RenderInfo != nullptr)
				{
					RenderInfo->bUseClassDefaultObject = true;
				}
			}
		}
	}

	// Check to see if this object is the "not supported" type or not
	if (RenderInfo == &NotSupported)
	{
		RenderInfo = nullptr;
	}
	
	return RenderInfo;
}

void UThumbnailManager::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	// Just mark us as dirty so that the cache is rebuilt
	bMapNeedsUpdate = true;
}

void UThumbnailManager::RegisterCustomRenderer(UClass* Class, TSubclassOf<UThumbnailRenderer> RendererClass)
{
	check(Class != nullptr);
	check(*RendererClass != nullptr);

	const FString NewClassPathName = Class->GetPathName();

	// Verify that this class isn't already registered
	for (int32 Index = 0; Index < RenderableThumbnailTypes.Num(); ++Index)
	{
		if (ensure(RenderableThumbnailTypes[Index].ClassNeedingThumbnailName != NewClassPathName))
		{
		}
		else
		{
			return;
		}
	}

	// Register the new class
	FThumbnailRenderingInfo& Info = *(new (RenderableThumbnailTypes) FThumbnailRenderingInfo());
	Info.ClassNeedingThumbnailName = NewClassPathName;
	Info.ClassNeedingThumbnail = Class;
	if (FApp::CanEverRender())
	{
		Info.Renderer = NewObject<UThumbnailRenderer>(GetTransientPackage(), RendererClass);
	}
	else
	{
		Info.Renderer = nullptr;
	}
	Info.RendererClassName = RendererClass->GetPathName();

	bMapNeedsUpdate = true;
}

void UThumbnailManager::UnregisterCustomRenderer(UClass* Class)
{
	check(Class != nullptr);

	const FString OldClassPathName = Class->GetPathName();

	for (int32 Index = 0; Index < RenderableThumbnailTypes.Num(); )
	{
		if (RenderableThumbnailTypes[Index].ClassNeedingThumbnailName == OldClassPathName)
		{
			RenderableThumbnailTypes.RemoveAtSwap(Index);
		}
		else
		{
			++Index;
		}
	}

	bMapNeedsUpdate = true;
}

UThumbnailManager& UThumbnailManager::Get()
{
	// Create it if we need to
	if (ThumbnailManagerSingleton == nullptr)
	{
		FString ClassName = GetDefault<UThumbnailManager>()->ThumbnailManagerClassName;
		if (!ClassName.IsEmpty())
		{
			// Try to load the specified class
			UClass* Class = LoadObject<UClass>(nullptr, *ClassName, nullptr, LOAD_None, nullptr);
			if (Class != nullptr)
			{
				// Create an instance of this class
				ThumbnailManagerSingleton = NewObject<UThumbnailManager>(GetTransientPackage(), Class);
			}
		}

		// If the class couldn't be loaded or is the wrong type, fallback to the default
		if (ThumbnailManagerSingleton == nullptr)
		{
			ThumbnailManagerSingleton = NewObject<UThumbnailManager>();
		}

		// Keep the singleton alive
		ThumbnailManagerSingleton->AddToRoot();

		// Tell it to load all of its classes
		ThumbnailManagerSingleton->Initialize();
	}

	return *ThumbnailManagerSingleton;
}

UThumbnailManager* UThumbnailManager::TryGet()
{
	return ThumbnailManagerSingleton;
}

void UThumbnailManager::SetupCheckerboardTexture()
{
	if (CheckerboardTexture)
	{
		return;
	}

	CheckerboardTexture = FImageUtils::CreateCheckerboardTexture(FColor(128, 128, 128), FColor(64, 64, 64), 32);
}

bool UThumbnailManager::CaptureProjectThumbnail(FViewport* Viewport, const FString& OutputFilename, bool bUseSCCIfPossible)
{
	const uint32 AutoScreenshotSize = 192;

	//capture the thumbnail
	uint32 SrcWidth = Viewport->GetSizeXY().X;
	uint32 SrcHeight = Viewport->GetSizeXY().Y;
	// Read the contents of the viewport into an array.
	TArray<FColor> OrigBitmap;
	if (Viewport->ReadPixels(OrigBitmap))
	{
		check(OrigBitmap.Num() == SrcWidth * SrcHeight);

		//pin to smallest value
		int32 CropSize = FMath::Min<uint32>(SrcWidth, SrcHeight);
		//pin to max size
		int32 ScaledSize  = FMath::Min<uint32>(AutoScreenshotSize, CropSize);

		//calculations for cropping
		TArray64<FColor> CroppedBitmap;
		CroppedBitmap.AddUninitialized(CropSize*CropSize);

		//Crop the image
		int32 CroppedSrcTop  = (SrcHeight - CropSize) / 2;
		int32 CroppedSrcLeft = (SrcWidth - CropSize) / 2;

		for (int32 Row = 0; Row < CropSize; ++Row)
		{
			//Row*Side of a row*byte per color
			int32 SrcPixelIndex = (CroppedSrcTop+Row) * SrcWidth + CroppedSrcLeft;
			const void* SrcPtr = &(OrigBitmap[SrcPixelIndex]);
			void* DstPtr = &(CroppedBitmap[Row * CropSize]);
			FMemory::Memcpy(DstPtr, SrcPtr, CropSize * 4);
		}

		FImageView CroppedImage(CroppedBitmap.GetData(),CropSize,CropSize);
		//Viewport ReadPixels seems to have A = 0, make sure it is set to opaque for image save
		FImageCore::SetAlphaOpaque(CroppedImage);

		//Scale image down if needed
		FImage ScaledImage;
		FImageView SaveImage;
		if (ScaledSize < CropSize)
		{
			FImageCore::ResizeTo(CroppedImage,ScaledImage,ScaledSize,ScaledSize,ERawImageFormat::BGRA8,EGammaSpace::sRGB);
			SaveImage = ScaledImage;
		}
		else
		{
			//just copy the data over. sizes are the same
			SaveImage = CroppedImage;
		}

		// Compress the scaled image
		// OutputFilename is a .png in current use
		TArray64<uint8> ScaledPng;
		if ( ! FImageUtils::CompressImage(ScaledPng, *OutputFilename, SaveImage) )
		{
			return false;
		}

		// Save to file
		const FString ScreenShotPath = FPaths::GetPath(OutputFilename);
		if ( IFileManager::Get().MakeDirectory(*ScreenShotPath, true) )
		{
			// If source control is available, try to check out the file if necessary.
			// If not, silently continue. This is just a courtesy.
			bool bMarkFileForAdd = false;
			FString AbsoluteFilename = FPaths::ConvertRelativePathToFull(OutputFilename);
			TArray<FString> FilesToBeCheckedOut;
			FilesToBeCheckedOut.Add(AbsoluteFilename);

			ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
			if ( bUseSCCIfPossible && ISourceControlModule::Get().IsEnabled() && SourceControlProvider.IsAvailable() )
			{
				FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(AbsoluteFilename, EStateCacheUsage::ForceUpdate);
				if(SourceControlState.IsValid())
				{
					if ( SourceControlState->CanCheckout() )
					{
						SourceControlProvider.Execute(ISourceControlOperation::Create<FCheckOut>(), FilesToBeCheckedOut);
					}
					else if ( !SourceControlState->IsSourceControlled() )
					{
						bMarkFileForAdd = true;
					}
				}
			}

			if ( FFileHelper::SaveArrayToFile( ScaledPng, *OutputFilename ) )
			{
				if ( bMarkFileForAdd )
				{
					SourceControlProvider.Execute(ISourceControlOperation::Create<FMarkForAdd>(), FilesToBeCheckedOut);
				}

				return true;
			}
		}
		else
		{
			// failed to make output dir?
		}
	}

	return false;
}

void UThumbnailManager::OnObjectPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		DirtyThumbnailForObject(ObjectBeingModified);
	}
}

void UThumbnailManager::OnActorPostEditMove(AActor* Actor)
{
	DirtyThumbnailForObject(Actor);
}

void UThumbnailManager::DirtyThumbnailForObject(UObject* ObjectBeingModified)
{
	if (!ObjectBeingModified)
	{
		return;
	}

	if (ObjectBeingModified->HasAnyFlags(RF_ClassDefaultObject))
	{
		if (ObjectBeingModified->GetClass()->ClassGeneratedBy != nullptr)
		{
			// This is a blueprint modification. Check to see if this thumbnail is the blueprint of the modified CDO
			ObjectBeingModified = ObjectBeingModified->GetClass()->ClassGeneratedBy;
		}
	}
	else if (AActor* ActorBeingModified = Cast<AActor>(ObjectBeingModified))
	{
		// This is a non CDO actor getting modified. Update the actor's world's thumbnail.
		ObjectBeingModified = ActorBeingModified->GetWorld();
	}

	if (ObjectBeingModified)
	{
		// An object in memory was modified.  We'll mark its thumbnail as dirty so that it'll be
		// regenerated on demand later. (Before being displayed in the browser, or package saves, etc.)
		FObjectThumbnail* Thumbnail = ThumbnailTools::GetThumbnailForObject(ObjectBeingModified);

		// If we don't yet have a thumbnail map, load one from disk if possible
		if (Thumbnail == nullptr)
		{
			UPackage* ObjectPackage = ObjectBeingModified->GetOutermost();

			const bool bMemoryPackage = FPackageName::IsMemoryPackage(ObjectBeingModified->GetPathName());  // Don't try to load from disk if the package is a memory package
			const bool bUnsavedPackage = ObjectPackage->HasAnyPackageFlags(PKG_NewlyCreated);               // Don't try loading thumbnails for package that have never been saved
			const bool bPackageDirty = ObjectPackage->IsDirty();                                            // Don't try loading thumbnails for package that we have no intention of saving
			const bool bIsGarbageCollecting = IsGarbageCollecting();                                        // Don't attempt to do this while garbage collecting since loading or finding objects during GC is illegal
			const bool bUsesGenericThumbnail = [ObjectBeingModified, this]() -> bool                        // No need to dirty generic thumbnails
			{
				if (FThumbnailRenderingInfo* RenderingInfo = GetRenderingInfo(ObjectBeingModified))
				{
					return RenderingInfo->Renderer == nullptr;
				}
				else
				{
					return true;
				}
			}();

			const bool bTryLoadThumbnailFromDisk = !bIsGarbageCollecting && !bMemoryPackage && !bUnsavedPackage && !bUsesGenericThumbnail && bPackageDirty;
			if (bTryLoadThumbnailFromDisk)
			{
				FName ObjectFullName = FName(*ObjectBeingModified->GetFullName());

				FThumbnailMap LoadedThumbnails;
				if (ThumbnailTools::ConditionallyLoadThumbnailsForObjects({ ObjectFullName }, LoadedThumbnails))
				{
					Thumbnail = LoadedThumbnails.Find(ObjectFullName);

					if (Thumbnail != nullptr)
					{
						Thumbnail = ThumbnailTools::CacheThumbnail(ObjectBeingModified->GetFullName(), Thumbnail, ObjectPackage);
					}
				}
			}
		}

		if (Thumbnail != nullptr)
		{
			// Mark the thumbnail as dirty
			Thumbnail->MarkAsDirty();
		}

		OnThumbnailDirtied.Broadcast(FSoftObjectPath(ObjectBeingModified));
	}
}
