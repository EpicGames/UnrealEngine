// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailRendering/AnimBlueprintThumbnailRenderer.h"
#include "ShowFlags.h"
#include "SceneView.h"
#include "Misc/App.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimBlueprintThumbnailRenderer)

UAnimBlueprintThumbnailRenderer::UAnimBlueprintThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UAnimBlueprintThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(Object);

	if (AnimBlueprint)
	{
		if (USkeleton* Skeleton = AnimBlueprint->TargetSkeleton)
		{
			USkeletalMesh* PreviewSkeletalMesh = Skeleton->GetAssetPreviewMesh(AnimBlueprint);
			if (PreviewSkeletalMesh == nullptr)
			{
				PreviewSkeletalMesh = Skeleton->FindCompatibleMesh();
			}

			if (PreviewSkeletalMesh && (PreviewSkeletalMesh->IsCompiling() || PreviewSkeletalMesh->GetResourceForRendering() == nullptr))
			{
				return false;
			}
		}
	}

	return AnimBlueprint && AnimBlueprint->BlueprintType != BPTYPE_Interface && !AnimBlueprint->bIsTemplate;
}

void UAnimBlueprintThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(Object);
	if (AnimBlueprint && AnimBlueprint->GeneratedClass)
	{
		if (!ThumbnailScene)
		{
			ThumbnailScene.Emplace();
		}

		if(ThumbnailScene->SetAnimBlueprint(AnimBlueprint))
		{
			FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(RenderTarget, ThumbnailScene->GetScene(), FEngineShowFlags(ESFIM_Game))
				.SetTime(UThumbnailRenderer::GetTime())
				.SetAdditionalViewFamily(bAdditionalViewFamily));

			ViewFamily.EngineShowFlags.DisableAdvancedFeatures();
			ViewFamily.EngineShowFlags.MotionBlur = 0;
			ViewFamily.EngineShowFlags.LOD = 0;

			RenderViewFamily(Canvas, &ViewFamily, ThumbnailScene->CreateView(&ViewFamily, X, Y, Width, Height));
			ThumbnailScene->SetAnimBlueprint(nullptr);
		}
	}
}

void UAnimBlueprintThumbnailRenderer::BeginDestroy()
{
	ThumbnailScene.Reset();

	Super::BeginDestroy();
}
