// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailRendering/AnimSequenceThumbnailRenderer.h"
#include "Misc/App.h"
#include "ShowFlags.h"
#include "SceneView.h"
#include "Animation/AnimSequenceBase.h"
#include "ThumbnailHelpers.h"
#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimSequenceThumbnailRenderer)

UAnimSequenceThumbnailRenderer::UAnimSequenceThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ThumbnailScene = nullptr;
}

bool UAnimSequenceThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	UAnimSequenceBase* AnimSequence = Cast<UAnimSequenceBase>(Object);
	if (AnimSequence != nullptr)
	{
		if (USkeleton* Skeleton = AnimSequence->GetSkeleton())
		{
			USkeletalMesh* PreviewSkeletalMesh = Skeleton->GetAssetPreviewMesh(AnimSequence);
			if (PreviewSkeletalMesh == nullptr)
			{
				PreviewSkeletalMesh = Skeleton->FindCompatibleMesh();
			}

			if (PreviewSkeletalMesh && (PreviewSkeletalMesh->IsCompiling() || PreviewSkeletalMesh->GetResourceForRendering() == nullptr))
			{
				return false;
			}
		}
	}

	return true;
}

void UAnimSequenceThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UAnimSequenceBase* Anim = Cast<UAnimSequenceBase>(Object);
	if (Anim != nullptr)
	{
		if ( ThumbnailScene == nullptr )
		{
			ThumbnailScene = new FAnimationSequenceThumbnailScene();
		}

		if(ThumbnailScene->SetAnimation(Anim))
		{
			FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(RenderTarget, ThumbnailScene->GetScene(), FEngineShowFlags(ESFIM_Game))
				.SetTime(UThumbnailRenderer::GetTime())
				.SetAdditionalViewFamily(bAdditionalViewFamily));

			ViewFamily.EngineShowFlags.DisableAdvancedFeatures();
			ViewFamily.EngineShowFlags.MotionBlur = 0;
			ViewFamily.EngineShowFlags.LOD = 0;

			RenderViewFamily(Canvas, &ViewFamily, ThumbnailScene->CreateView(&ViewFamily, X, Y, Width, Height));
			ThumbnailScene->SetAnimation(nullptr);
		}
	}
}

void UAnimSequenceThumbnailRenderer::BeginDestroy()
{
	if ( ThumbnailScene != nullptr )
	{
		delete ThumbnailScene;
		ThumbnailScene = nullptr;
	}

	Super::BeginDestroy();
}
