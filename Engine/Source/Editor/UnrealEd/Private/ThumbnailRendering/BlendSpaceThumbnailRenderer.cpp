// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailRendering/BlendSpaceThumbnailRenderer.h"
#include "Misc/App.h"
#include "ShowFlags.h"
#include "SceneView.h"
#include "ThumbnailHelpers.h"
#include "Animation/BlendSpace.h"
#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlendSpaceThumbnailRenderer)

UBlendSpaceThumbnailRenderer::UBlendSpaceThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ThumbnailScene = nullptr;
}

bool UBlendSpaceThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	UBlendSpace* BlendSpace = Cast<UBlendSpace>(Object);
	if (BlendSpace != nullptr)
	{
		if (USkeleton* Skeleton = BlendSpace->GetSkeleton())
		{
			USkeletalMesh* PreviewSkeletalMesh = Skeleton->GetAssetPreviewMesh(BlendSpace);
			if (PreviewSkeletalMesh == nullptr)
			{
				PreviewSkeletalMesh = Skeleton->FindCompatibleMesh();
			}

			if (PreviewSkeletalMesh && (PreviewSkeletalMesh->IsCompiling() || PreviewSkeletalMesh->GetResourceForRendering() == nullptr))
			{
				return false;
			}
		}
	}

	return true;
}

void UBlendSpaceThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UBlendSpace* BlendSpace = Cast<UBlendSpace>(Object);
	if (BlendSpace != nullptr)
	{
		if (ThumbnailScene == nullptr)
		{
			ThumbnailScene = new FBlendSpaceThumbnailScene();
		}

		if (ThumbnailScene->SetBlendSpace(BlendSpace))
		{
			FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(RenderTarget, ThumbnailScene->GetScene(), FEngineShowFlags(ESFIM_Game))
				.SetTime(UThumbnailRenderer::GetTime())
				.SetAdditionalViewFamily(bAdditionalViewFamily));

			ViewFamily.EngineShowFlags.DisableAdvancedFeatures();
			ViewFamily.EngineShowFlags.MotionBlur = 0;
			ViewFamily.EngineShowFlags.LOD = 0;

			RenderViewFamily(Canvas, &ViewFamily, ThumbnailScene->CreateView(&ViewFamily, X, Y, Width, Height));
			ThumbnailScene->SetBlendSpace(nullptr);
		}
	}
}

void UBlendSpaceThumbnailRenderer::BeginDestroy()
{
	if ( ThumbnailScene != nullptr )
	{
		delete ThumbnailScene;
		ThumbnailScene = nullptr;
	}

	Super::BeginDestroy();
}
