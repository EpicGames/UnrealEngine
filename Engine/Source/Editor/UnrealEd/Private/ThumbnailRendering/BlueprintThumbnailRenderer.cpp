// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailRendering/BlueprintThumbnailRenderer.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "ShowFlags.h"
#include "SceneView.h"
#include "Misc/App.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintThumbnailRenderer)


UBlueprintThumbnailRenderer::UBlueprintThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	FKismetEditorUtilities::OnBlueprintUnloaded.AddUObject(this, &UBlueprintThumbnailRenderer::OnBlueprintUnloaded);
}

bool UBlueprintThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	UBlueprint* Blueprint = Cast<UBlueprint>(Object);

	bool bCanVisualizeAsset = false;

	// Only visualize actor based blueprints
	if (Blueprint && Blueprint->GeneratedClass && Blueprint->GeneratedClass->IsChildOf(AActor::StaticClass()))
	{
		AActor::ForEachComponentOfActorClassDefault<UActorComponent>(TSubclassOf<AActor>(Blueprint->GeneratedClass), [&](const UActorComponent* Component)
		{
			bCanVisualizeAsset = FBlueprintThumbnailScene::IsValidComponentForVisualization(Component);
			return !bCanVisualizeAsset;
		});
	}

	return bCanVisualizeAsset;
}

void UBlueprintThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UBlueprint* Blueprint = Cast<UBlueprint>(Object);

	// Strict validation - it may hopefully fix UE-35705.
	const bool bIsBlueprintValid = IsValid(Blueprint)
		&& IsValid(Blueprint->GeneratedClass)
		&& Blueprint->bHasBeenRegenerated
		//&& Blueprint->IsUpToDate() - This condition makes the thumbnail blank whenever the BP is dirty. It seems too strict.
		&& !Blueprint->bBeingCompiled
		&& !Blueprint->HasAnyFlags(RF_Transient);
	if (bIsBlueprintValid)
	{
		if (!ThumbnailScene)
		{
			ThumbnailScene.Emplace();
		}

		ThumbnailScene->SetBlueprint(Blueprint);
		FSceneViewFamilyContext ViewFamily( FSceneViewFamily::ConstructionValues( RenderTarget, ThumbnailScene->GetScene(), FEngineShowFlags(ESFIM_Game) )
			.SetTime(UThumbnailRenderer::GetTime())
			.SetAdditionalViewFamily(bAdditionalViewFamily));

		ViewFamily.EngineShowFlags.DisableAdvancedFeatures();
		ViewFamily.EngineShowFlags.MotionBlur = 0;

		RenderViewFamily(Canvas, &ViewFamily, ThumbnailScene->CreateView(&ViewFamily, X, Y, Width, Height));
	}
}

void UBlueprintThumbnailRenderer::BeginDestroy()
{
	FKismetEditorUtilities::OnBlueprintUnloaded.RemoveAll(this);
	ThumbnailScene.Reset();

	Super::BeginDestroy();
}

void UBlueprintThumbnailRenderer::BlueprintChanged(UBlueprint* Blueprint)
{
	if (Blueprint && Blueprint->GeneratedClass)
	{
		if (ThumbnailScene)
		{
			ThumbnailScene->BlueprintChanged(Blueprint);
		}
	}
}

void UBlueprintThumbnailRenderer::OnBlueprintUnloaded(UBlueprint* Blueprint)
{
	if (Blueprint && Blueprint->GeneratedClass)
	{
		if (ThumbnailScene && ThumbnailScene->GetCurrentBlueprint() == Blueprint)
		{
			ThumbnailScene.Reset();
		}
	}
}
