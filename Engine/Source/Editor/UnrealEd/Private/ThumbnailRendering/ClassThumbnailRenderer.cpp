// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailRendering/ClassThumbnailRenderer.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "ShowFlags.h"
#include "SceneView.h"
#include "Misc/App.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClassThumbnailRenderer)

UClassThumbnailRenderer::UClassThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	FKismetEditorUtilities::OnBlueprintGeneratedClassUnloaded.AddUObject(this, &UClassThumbnailRenderer::OnBlueprintGeneratedClassUnloaded);
}

bool UClassThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	UClass* Class = Cast<UClass>(Object);

	// Only visualize actor based classes
	if (Class && Class->IsChildOf(AActor::StaticClass()))
	{
		// Try to find any visible primitive components in the class' CDO
		AActor* CDO = Class->GetDefaultObject<AActor>();

		for (UActorComponent* Component : CDO->GetComponents())
		{
			if (FClassThumbnailScene::IsValidComponentForVisualization(Component))
			{
				return true;
			}
		}
	}

	return false;
}

void UClassThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UClass* Class = Cast<UClass>(Object);
	if (Class != nullptr)
	{
		if (!ThumbnailScene)
		{
			ThumbnailScene.Emplace();
		}

		ThumbnailScene->SetClass(Class);
		FSceneViewFamilyContext ViewFamily( FSceneViewFamily::ConstructionValues( RenderTarget, ThumbnailScene->GetScene(), FEngineShowFlags(ESFIM_Game) )
			.SetTime(UThumbnailRenderer::GetTime())
			.SetAdditionalViewFamily(bAdditionalViewFamily));

		ViewFamily.EngineShowFlags.DisableAdvancedFeatures();
		ViewFamily.EngineShowFlags.MotionBlur = 0;

		RenderViewFamily(Canvas, &ViewFamily, ThumbnailScene->CreateView(&ViewFamily, X, Y, Width, Height));
	}
}

void UClassThumbnailRenderer::BeginDestroy()
{
	FKismetEditorUtilities::OnBlueprintGeneratedClassUnloaded.RemoveAll(this);
	ThumbnailScene.Reset();

	Super::BeginDestroy();
}

void UClassThumbnailRenderer::OnBlueprintGeneratedClassUnloaded(UBlueprintGeneratedClass* BPGC)
{
	if (ThumbnailScene && ThumbnailScene->GetCurrentClass() == BPGC)
	{
		ThumbnailScene.Reset();
	}
}
