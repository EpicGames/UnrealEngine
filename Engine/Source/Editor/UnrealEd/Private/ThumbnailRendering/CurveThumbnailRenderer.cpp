// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailRendering/CurveThumbnailRenderer.h"

#include "CanvasItem.h"
#include "Misc/App.h"
#include "ShowFlags.h"
#include "SceneView.h"

#include "Curves/CurveFloat.h"
#include "Curves/CurveVector.h"
#include "Curves/CurveLinearColor.h"
#include "CanvasTypes.h"
#include "UnrealClient.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CurveThumbnailRenderer)

UCurveLinearColorThumbnailRenderer::UCurveLinearColorThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UCurveFloatThumbnailRenderer::UCurveFloatThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UCurveVector3ThumbnailRenderer::UCurveVector3ThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UCurveFloatThumbnailRenderer::GetThumbnailSize(UObject* Object, float Zoom, uint32& OutWidth, uint32& OutHeight) const
{
	UCurveFloat* Curve = Cast<UCurveFloat>(Object);
	if (Curve && !Curve->FloatCurve.IsEmpty())
	{
		OutWidth = 255;
		OutHeight = 255;
	}
	else
	{
		OutWidth = 0;
		OutHeight = 0;
	}
}

void DrawCurve(const FRichCurve& Curve, FCanvas* Canvas, const FLinearColor& Color, float MinTime, float MaxTime, float MinValue, float MaxValue)
{
	FVector2D TextureSize = Canvas->GetRenderTarget()->GetSizeXY();
	if (TextureSize.X == 0 || TextureSize.Y == 0)
	{
		return;
	}
	
	//Render the curve to the canvas with line segments
	FVector2D PrevPos;
	int Samples = TextureSize.X / 2.5;
	float Padding = Samples / 5.0;
	for (int i = 0; i < Samples; i++)
	{
		float Time;
		if (i < Padding / 2)
		{
			Time =  MinTime;
		}
		else if (i > (Samples - Padding / 2))
		{
			Time =  MaxTime;
		}
		else
		{
			float TimeRange = (Samples - Padding) * (MaxTime - MinTime);
			Time = TimeRange == 0 ? 0 : (i - Padding / 2) / TimeRange + MinTime;
		}
			
		float Value = Curve.Eval(Time);
		FVector2D Pos;
		Pos.X = i * TextureSize.X / Samples;
		float NormalizedValue = 1 - (Value - MinValue) / (MaxValue - MinValue);
		Pos.Y = NormalizedValue * (TextureSize.Y - Padding * 2) + Padding;

		if (i > 0)
		{
			FCanvasLineItem Line(PrevPos, Pos);
			Line.LineThickness = 1.5;
			Line.SetColor(Color);
			Line.Draw(Canvas);
		}
		PrevPos = Pos;
	}
}

void UCurveFloatThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	if (UCurveFloat* Curve = Cast<UCurveFloat>(Object))
	{
		FVector2D TextureSize = Canvas->GetRenderTarget()->GetSizeXY();
		if (!bAdditionalViewFamily)
		{
			Canvas->Clear(FLinearColor::Black);
		}
		
		if (Curve->FloatCurve.GetNumKeys() < 2)
		{
			FCanvasLineItem Line(FVector2D(0, TextureSize.Y / 2), FVector2D(TextureSize.X, TextureSize.Y / 2));
			Line.LineThickness = 1.5;
			Line.SetColor(FLinearColor::Gray);
			Line.Draw(Canvas);
			return;
		}

		float MinTime, MaxTime;
		Curve->FloatCurve.GetTimeRange(MinTime, MaxTime);

		float MinValue, MaxValue;
		Curve->FloatCurve.GetValueRange( MinValue, MaxValue);
		DrawCurve(Curve->FloatCurve, Canvas, FLinearColor::Gray, MinTime, MaxTime, MinValue, MaxValue);
	}
}

bool UCurveFloatThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	return Cast<UCurveFloat>(Object) != nullptr;
}

void UCurveVector3ThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	if (UCurveVector* VectorCurve = Cast<UCurveVector>(Object))
	{
		FVector2D TextureSize = Canvas->GetRenderTarget()->GetSizeXY();
		if (!bAdditionalViewFamily)
		{
			Canvas->Clear(FLinearColor::Black);
		}

		static FLinearColor Colors[3] = {FLinearColor::Red, FLinearColor::Green, FLinearColor(0.2f, 0.2f, 1)};
		
		float MinTime, MaxTime;
		VectorCurve->GetTimeRange(MinTime, MaxTime);

		float MinValue, MaxValue;
		VectorCurve->GetValueRange( MinValue, MaxValue);

		for (int i = 0; i < 3; i++)
		{
			FRichCurve& SubCurve = VectorCurve->FloatCurves[i];
			if (SubCurve.GetNumKeys() == 0)
			{
				
				FCanvasLineItem Line(FVector2D(0, TextureSize.Y / 2), FVector2D(TextureSize.X, TextureSize.Y / 2));
				Line.LineThickness = 1.5;
				Line.SetColor(Colors[i]);
				Line.Draw(Canvas);
				continue;
			}
			DrawCurve(SubCurve, Canvas, Colors[i], MinTime, MaxTime, MinValue, MaxValue);
		}
	}
}

void UCurveVector3ThumbnailRenderer::GetThumbnailSize(UObject* Object, float Zoom, uint32& OutWidth, uint32& OutHeight) const
{
	if (UCurveVector* Curve = Cast<UCurveVector>(Object))
	{
		OutWidth = 255;
		OutHeight = 255;
	}
	else
	{
		OutWidth = 0;
		OutHeight = 0;
	}
}

bool UCurveVector3ThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	return Cast<UCurveVector>(Object) != nullptr;
}

void UCurveLinearColorThumbnailRenderer::GetThumbnailSize(UObject* Object, float Zoom, uint32& OutWidth, uint32& OutHeight) const
{
	if (UCurveLinearColor* GradientCurve = Cast<UCurveLinearColor>(Object))
	{
		OutWidth = 255;
		OutHeight = 255;
	}
	else
	{
		OutWidth = 0;
		OutHeight = 0;
	}
}

void UCurveLinearColorThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* Viewport, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	if (UCurveLinearColor* GradientCurve = Cast<UCurveLinearColor>(Object))
	{
		FVector2D TextureSize = Canvas->GetRenderTarget()->GetSizeXY();
		GradientCurve->DrawThumbnail(Canvas, FVector2D(0.f, 0.f), TextureSize);
	}
}

bool UCurveLinearColorThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	return Cast<UCurveLinearColor>(Object) != nullptr;
}
