// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailRendering/DefaultSizedThumbnailRenderer.h"

#include "Math/UnrealMathSSE.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DefaultSizedThumbnailRenderer)

class UObject;

UDefaultSizedThumbnailRenderer::UDefaultSizedThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UDefaultSizedThumbnailRenderer::GetThumbnailSize(UObject*, float Zoom, uint32& OutWidth, uint32& OutHeight) const
{
	OutWidth = FMath::TruncToInt(DefaultSizeX * Zoom);
	OutHeight = FMath::TruncToInt(DefaultSizeY * Zoom);
}
