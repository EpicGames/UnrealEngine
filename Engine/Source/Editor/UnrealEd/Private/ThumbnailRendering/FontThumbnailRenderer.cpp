// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailRendering/FontThumbnailRenderer.h"
#include "Fonts/SlateFontInfo.h"
#include "Fonts/FontCache.h"
#include "Framework/Application/SlateApplication.h"
#include "Engine/Font.h"
#include "CanvasItem.h"
#include "Engine/Texture2D.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "TextureResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FontThumbnailRenderer)

UFontThumbnailRenderer::UFontThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UFontThumbnailRenderer::GetThumbnailSize(UObject* Object, float Zoom, uint32& OutWidth, uint32& OutHeight) const
{
	UFont* const Font = Cast<UFont>(Object);

	if(Font)
	{
		switch(Font->FontCacheType)
		{
		case EFontCacheType::Offline:
			if(Font->Textures.Num() > 0 && Font->Textures[0])
			{
				// Get the texture interface for the font text
				UTexture2D* const Tex = Font->Textures[0];
				OutWidth = FMath::TruncToInt(Zoom * (float)Tex->GetSurfaceWidth());
				OutHeight = FMath::TruncToInt(Zoom * (float)Tex->GetSurfaceHeight());
				return;
			}
			break;

		case EFontCacheType::Runtime:
			if (const FCompositeFont* CompositeFont = Font->GetCompositeFont();
				CompositeFont && CompositeFont->DefaultTypeface.Fonts.Num() > 0)
			{
				OutWidth = OutHeight = static_cast<uint32>(Zoom * 256.0f);
			}
			break;

		default:
			break;
		};
	}

	OutWidth = OutHeight = 0;
}

void UFontThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UFont* const Font = Cast<UFont>(Object);

	if(Font)
	{
		switch(Font->FontCacheType)
		{
		case EFontCacheType::Offline:
			if (Font->Textures.Num() > 0 && Font->Textures[0] && Font->Textures[0]->GetResource())
			{
				FCanvasTileItem TileItem(FVector2D(X, Y), Font->Textures[0]->GetResource(), FLinearColor::White);
				TileItem.BlendMode = (Font->ImportOptions.bUseDistanceFieldAlpha) ? SE_BLEND_TranslucentDistanceField : SE_BLEND_Translucent;
				Canvas->DrawItem(TileItem);
			}
			break;

		case EFontCacheType::Runtime:
			if (const FCompositeFont* CompositeFont = Font->GetCompositeFont();
				CompositeFont && CompositeFont->DefaultTypeface.Fonts.Num() > 0)
			{
				static const int32 FontSize = 28;
				static const float FontScale = 1.0f;

				const FString FontName = Object->GetName();
				const TextBiDi::ETextDirection FontNameBaseTextDirection = TextBiDi::ComputeBaseDirection(FontName);

				TSharedRef<FSlateFontCache> FontCache = FSlateApplication::Get().GetRenderer()->GetFontCache();

				// Draw the object name using each of the default fonts
				FVector2D CurPos(X, Y);
				for(const FTypefaceEntry& TypefaceEntry : CompositeFont->DefaultTypeface.Fonts)
				{
					FSlateFontInfo FontInfo(Font, FontSize, TypefaceEntry.Name);
					FShapedGlyphSequenceRef FontNameShapedText = FontCache->ShapeBidirectionalText(FontName, FontInfo, FontScale, FontNameBaseTextDirection, ETextShapingMethod::Auto);

					FVector2D TextDrawPos = CurPos;
					if (FontNameBaseTextDirection == TextBiDi::ETextDirection::RightToLeft)
					{
						TextDrawPos.X = Width - CurPos.X - FontNameShapedText->GetMeasuredWidth();
					}

					FCanvasShapedTextItem TextItem(TextDrawPos, FontNameShapedText, FLinearColor::White);
					Canvas->DrawItem(TextItem);

					CurPos.Y += FontNameShapedText->GetMaxTextHeight();
				}
			}
			break;

		default:
			break;
		};
	}
}
