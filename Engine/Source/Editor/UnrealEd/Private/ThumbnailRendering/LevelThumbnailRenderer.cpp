// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailRendering/LevelThumbnailRenderer.h"
#include "EngineDefines.h"
#include "Misc/App.h"
#include "Engine/Level.h"
#include "ShowFlags.h"
#include "SceneView.h"
#include "SceneViewExtension.h"
#include "Engine/LevelBounds.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelThumbnailRenderer)

ULevelThumbnailRenderer::ULevelThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void ULevelThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	ULevel* Level = Cast<ULevel>(Object);
	if (Level != nullptr)
	{
		FSceneViewFamilyContext ViewFamily( FSceneViewFamily::ConstructionValues( RenderTarget, Level->OwningWorld->Scene, FEngineShowFlags(ESFIM_Game) )
			.SetTime(UThumbnailRenderer::GetTime())
			.SetAdditionalViewFamily(bAdditionalViewFamily));

		ViewFamily.EngineShowFlags.DisableAdvancedFeatures();
		ViewFamily.EngineShowFlags.MotionBlur = 0;
		ViewFamily.EngineShowFlags.SetDistanceCulledPrimitives(true); // show distance culled objects
		ViewFamily.EngineShowFlags.SetPostProcessing(false);

		RenderViewFamily(Canvas, &ViewFamily, CreateView(Level, &ViewFamily, X, Y, Width, Height));
	}
}

FSceneView* ULevelThumbnailRenderer::CreateView(ULevel* Level, FSceneViewFamily* ViewFamily, int32 X, int32 Y, uint32 SizeX, uint32 SizeY) const 
{
	check(ViewFamily);

	FIntRect ViewRect(
		FMath::Max<int32>(X,0),
		FMath::Max<int32>(Y,0),
		FMath::Max<int32>(X+SizeX,0),
		FMath::Max<int32>(Y+SizeY,0));

	FBox LevelBox(ForceInit);

	if (Level->LevelBoundsActor.IsValid())
	{
		LevelBox = Level->LevelBoundsActor.Get()->GetComponentsBoundingBox();
	}
	else
	{
		LevelBox = ALevelBounds::CalculateLevelBounds(Level);
	}

	if (ViewRect.Area() <= 0)
	{
		return nullptr;
	}


	FSceneViewInitOptions ViewInitOptions;

	ViewInitOptions.SetViewRectangle(ViewRect);
	ViewInitOptions.ViewFamily = ViewFamily;

	const FVector ViewPoint = LevelBox.GetCenter();
	ViewInitOptions.ViewOrigin = FVector(ViewPoint.X, ViewPoint.Y, 0);
	ViewInitOptions.ViewRotationMatrix = FMatrix(
		FPlane(1,				0,				0,		0),
		FPlane(0,				-1,				0,		0),
		FPlane(0,				0,				-1,		0),
		FPlane(0,				0,				0,		1));

	const FMatrix::FReal ZOffset = UE_OLD_WORLD_MAX;
	ViewInitOptions.ProjectionMatrix =  FReversedZOrthoMatrix(
		LevelBox.GetSize().X/2.f,
		LevelBox.GetSize().Y/2.f,
		0.5f / ZOffset,
		ZOffset
		);
	FSceneView* NewView = new FSceneView(ViewInitOptions);
	ViewFamily->Views.Add(NewView);
	return NewView;
}
