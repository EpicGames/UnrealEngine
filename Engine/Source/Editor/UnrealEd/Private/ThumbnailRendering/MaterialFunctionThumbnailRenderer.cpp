// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailRendering/MaterialFunctionThumbnailRenderer.h"
#include "Misc/App.h"
#include "ShowFlags.h"
#include "SceneView.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialFunctionInstance.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceConstant.h"
#include "ThumbnailHelpers.h"
#include "Slate/WidgetRenderer.h"
#include "Widgets/Images/SImage.h"
#include "SlateMaterialBrush.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Settings/ContentBrowserSettings.h"
#include "Engine/Texture2D.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialFunctionThumbnailRenderer)


UMaterialFunctionThumbnailRenderer::UMaterialFunctionThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ThumbnailScene(nullptr)
	, WidgetRenderer(nullptr)
	, UIMaterialBrush(nullptr)
{
	ThumbnailScene = nullptr;
}

void UMaterialFunctionThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UMaterialFunctionInterface* MatFunc = Cast<UMaterialFunctionInterface>(Object);
	UMaterialFunctionInstance* MatFuncInst = Cast<UMaterialFunctionInstance>(Object);
	const bool bIsFunctionInstancePreview = MatFuncInst && MatFuncInst->GetBaseFunction();

	if (MatFunc || bIsFunctionInstancePreview)
	{
		UMaterialInterface* PreviewMaterial = bIsFunctionInstancePreview ? MatFuncInst->GetPreviewMaterial() : MatFunc->GetPreviewMaterial();
		UMaterial* Mat = PreviewMaterial ? PreviewMaterial->GetMaterial() : nullptr;

		if (Mat != nullptr && Mat->IsUIMaterial())
		{
			if (WidgetRenderer == nullptr)
			{
				const bool bUseGammaCorrection = true;
				WidgetRenderer = new FWidgetRenderer(bUseGammaCorrection);
				check(WidgetRenderer);
			}

			if (UIMaterialBrush == nullptr)
			{
				UIMaterialBrush = new FSlateMaterialBrush(FVector2D(SlateBrushDefs::DefaultImageSize, SlateBrushDefs::DefaultImageSize));
				check(UIMaterialBrush);
			}

			UIMaterialBrush->SetMaterial(PreviewMaterial);

			UTexture2D* CheckerboardTexture = UThumbnailManager::Get().CheckerboardTexture;

			FSlateBrush CheckerboardBrush;
			CheckerboardBrush.SetResourceObject(CheckerboardTexture);
			CheckerboardBrush.ImageSize = FVector2D(CheckerboardTexture->GetSizeX(), CheckerboardTexture->GetSizeY());
			CheckerboardBrush.Tiling = ESlateBrushTileType::Both;

			TSharedRef<SWidget> Thumbnail =
				SNew(SOverlay)

				// Checkerboard
				+ SOverlay::Slot()
				[
					SNew(SImage)
					.Image(&CheckerboardBrush)
				]

				+ SOverlay::Slot()
				[
					SNew(SImage)
					.Image(UIMaterialBrush)
				];

			const FVector2D DrawSize((float)Width, (float)Height);
			const float DeltaTime = 0.f;
			WidgetRenderer->DrawWidget(RenderTarget, Thumbnail, DrawSize, DeltaTime);

			UIMaterialBrush->SetMaterial(nullptr);
		}
		else
		{
			if (ThumbnailScene == nullptr || ensure(ThumbnailScene->GetWorld() != nullptr) == false)
			{
				if (ThumbnailScene)
				{
					FlushRenderingCommands();
					delete ThumbnailScene;
				}

				ThumbnailScene = new FMaterialThumbnailScene();
			}

			EMaterialFunctionUsage FunctionUsage = bIsFunctionInstancePreview ? MatFuncInst->GetMaterialFunctionUsage() : MatFunc->GetMaterialFunctionUsage();
			UThumbnailInfo* ThumbnailInfo = bIsFunctionInstancePreview ? MatFuncInst->ThumbnailInfo : MatFunc->ThumbnailInfo;

			if (PreviewMaterial)
			{
				PreviewMaterial->ThumbnailInfo = ThumbnailInfo;
				if (FunctionUsage == EMaterialFunctionUsage::MaterialLayerBlend)
				{
					PreviewMaterial->SetShouldForcePlanePreview(true);
				}
				ThumbnailScene->SetMaterialInterface(PreviewMaterial);

				FSceneViewFamilyContext ViewFamily( FSceneViewFamily::ConstructionValues( RenderTarget, ThumbnailScene->GetScene(), FEngineShowFlags(ESFIM_Game) )
					.SetTime(UThumbnailRenderer::GetTime())
					.SetAdditionalViewFamily(bAdditionalViewFamily));

				ViewFamily.EngineShowFlags.DisableAdvancedFeatures();
				ViewFamily.EngineShowFlags.MotionBlur = 0;

				RenderViewFamily(Canvas, &ViewFamily, ThumbnailScene->CreateView(&ViewFamily, X, Y, Width, Height));

				ThumbnailScene->SetMaterialInterface(nullptr);
			}
		}
	}
}

bool UMaterialFunctionThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	return GetDefault<UContentBrowserSettings>()->bEnableRealtimeMaterialInstanceThumbnails;
}

void UMaterialFunctionThumbnailRenderer::BeginDestroy()
{
	if (ThumbnailScene != nullptr)
	{
		delete ThumbnailScene;
		ThumbnailScene = nullptr;
	}

	if (WidgetRenderer != nullptr)
	{
		BeginCleanup(WidgetRenderer);
		WidgetRenderer = nullptr;
	}

	if (UIMaterialBrush != nullptr)
	{
		delete UIMaterialBrush;
		UIMaterialBrush = nullptr;
	}

	Super::BeginDestroy();
}
