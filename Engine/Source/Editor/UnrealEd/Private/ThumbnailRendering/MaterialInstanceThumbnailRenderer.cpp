// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailRendering/MaterialInstanceThumbnailRenderer.h"
#include "Misc/App.h"
#include "ShowFlags.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialInterface.h"
#include "Materials/Material.h"
#include "SceneView.h"
#include "ThumbnailHelpers.h"
#include "Slate/WidgetRenderer.h"
#include "Widgets/Images/SImage.h"
#include "SlateMaterialBrush.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Engine/Texture2D.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialInstanceThumbnailRenderer)

UMaterialInstanceThumbnailRenderer::UMaterialInstanceThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ThumbnailScene(nullptr)
	, WidgetRenderer(nullptr)
	, UIMaterialBrush(nullptr)
{
}

void UMaterialInstanceThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(Object);
	if (MaterialInterface != nullptr)
	{
		UMaterial* BaseMaterial = MaterialInterface->GetMaterial();
		if (BaseMaterial != nullptr && BaseMaterial->IsUIMaterial())
		{
			if (WidgetRenderer == nullptr)
			{
				const bool bUseGammaCorrection = true;
				WidgetRenderer = new FWidgetRenderer(bUseGammaCorrection);
				check(WidgetRenderer);
			}

			if (UIMaterialBrush == nullptr)
			{
				UIMaterialBrush = new FSlateMaterialBrush(FVector2D(SlateBrushDefs::DefaultImageSize, SlateBrushDefs::DefaultImageSize));
				check(UIMaterialBrush);
			}
		
			UIMaterialBrush->SetMaterial(MaterialInterface);

			UTexture2D* CheckerboardTexture = UThumbnailManager::Get().CheckerboardTexture;
			
			FSlateBrush CheckerboardBrush;
			CheckerboardBrush.SetResourceObject(CheckerboardTexture);
			CheckerboardBrush.ImageSize = FVector2D(CheckerboardTexture->GetSizeX(), CheckerboardTexture->GetSizeY());
			CheckerboardBrush.Tiling = ESlateBrushTileType::Both;

			TSharedRef<SWidget> Thumbnail =
				SNew(SOverlay)

				// Checkerboard
				+ SOverlay::Slot()
				[
					SNew(SImage)
					.Image(&CheckerboardBrush)
				]

				+ SOverlay::Slot()
				[
					SNew(SImage)
					.Image(UIMaterialBrush)
				];

			const FVector2D DrawSize((float)Width, (float)Height);
			const float DeltaTime = 0.f;
			WidgetRenderer->DrawWidget(RenderTarget, Thumbnail, DrawSize, DeltaTime);

			UIMaterialBrush->SetMaterial(nullptr);
		}
		else
		{
			if ( ThumbnailScene == nullptr || ensure(ThumbnailScene->GetWorld() != nullptr) == false )
			{
				if (ThumbnailScene)
				{
					FlushRenderingCommands();
					delete ThumbnailScene;
				}
				ThumbnailScene = new FMaterialThumbnailScene();
			}

			// If the material doesn't have the static mesh usage flag then create a child
			// instance with the usage flag set and use that for the thumbnail rendering.
			if (!MaterialInterface->GetUsageByFlag(MATUSAGE_StaticMesh))
			{
				if (!OverrideMaterialInstance || OverrideMaterialInstance->Parent != MaterialInterface)
				{
					OverrideMaterialInstance = NewObject<UMaterialInstanceConstant>();
					OverrideMaterialInstance->SetParentEditorOnly(MaterialInterface);
					OverrideMaterialInstance->BasePropertyOverrides.bOverride_UsageFlags = (1 << MATUSAGE_StaticMesh);
					OverrideMaterialInstance->BasePropertyOverrides.UsageFlags = (1 << MATUSAGE_StaticMesh);
					OverrideMaterialInstance->UpdateStaticPermutation();
				}
				ThumbnailScene->SetMaterialInterface(OverrideMaterialInstance);
			}
			else
			{
				ThumbnailScene->SetMaterialInterface(MaterialInterface);
			}

			FSceneViewFamilyContext ViewFamily( FSceneViewFamily::ConstructionValues( RenderTarget, ThumbnailScene->GetScene(), FEngineShowFlags(ESFIM_Game) )
				.SetTime(UThumbnailRenderer::GetTime())
				.SetAdditionalViewFamily(bAdditionalViewFamily));

			ViewFamily.EngineShowFlags.DisableAdvancedFeatures();
			ViewFamily.EngineShowFlags.SetSeparateTranslucency(ThumbnailScene->ShouldSetSeparateTranslucency(MaterialInterface));
			ViewFamily.EngineShowFlags.MotionBlur = 0;
			ViewFamily.EngineShowFlags.AntiAliasing = 0;

			RenderViewFamily(Canvas, &ViewFamily, ThumbnailScene->CreateView(&ViewFamily, X, Y, Width, Height));

			ThumbnailScene->SetMaterialInterface(nullptr);
		}
	}
}

bool UMaterialInstanceThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	if (UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(Object))
	{
		if (MaterialInterface->IsCompiling())
		{
			return false;
		}
	}
	return true;
}

void UMaterialInstanceThumbnailRenderer::BeginDestroy()
{
	if (ThumbnailScene != nullptr)
	{
		delete ThumbnailScene;
		ThumbnailScene = nullptr;
	}

	if (WidgetRenderer != nullptr)
	{
		BeginCleanup(WidgetRenderer);
		WidgetRenderer = nullptr;
	}

	if (UIMaterialBrush != nullptr)
	{
		delete UIMaterialBrush;
		UIMaterialBrush = nullptr;
	}

	Super::BeginDestroy();
}
