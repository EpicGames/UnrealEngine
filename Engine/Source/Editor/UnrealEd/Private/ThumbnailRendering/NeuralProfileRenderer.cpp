// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailRendering/NeuralProfileRenderer.h"
#include "CanvasItem.h"
#include "Engine/Engine.h"
#include "EngineGlobals.h"
#include "Engine/NeuralProfile.h"
#include "CanvasTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NeuralProfileRenderer)

UNeuralProfileRenderer::UNeuralProfileRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UNeuralProfileRenderer::GetThumbnailSize(UObject* Object, float Zoom, uint32& OutWidth, uint32& OutHeight) const
{
	OutWidth = 128;
	OutHeight = 128;
}

void UNeuralProfileRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UNeuralProfile* LocalNeuralProfile = Cast<UNeuralProfile>(Object);
	if (LocalNeuralProfile)
	{
		const bool bHasProfile = LocalNeuralProfile->Settings.NNEModelData != nullptr;
		FLinearColor Col;

		Col = bHasProfile? FLinearColor(0.0f,1.0f,0.0f):FLinearColor(0.0f,0.0f,0.0f); Col.A = 1;
		Canvas->DrawTile(0,          0, Width, Height / 2, 0, 0, 1, 1, Col);
		Col = FLinearColor(0.0f, 0.0f, 0.0f); Col.A = 1;
		Canvas->DrawTile(0, Height / 2, Width, Height / 2, 0, 0, 1, 1, Col);

		FText ChannelText = FText::AsNumber(LocalNeuralProfile->Settings.InputDimension.Y);
		//FText DimenionHeight = FText::AsNumber(LocalNeuralProfile->Settings.Dimension.Z);
		//FText DimenionWidth = FText::AsNumber(LocalNeuralProfile->Settings.Dimension.W);
		
		FCanvasTextItem TextItem(FVector2D(5.0f, 5.0f), ChannelText, GEngine->GetLargeFont(), FLinearColor::White);
		TextItem.EnableShadow(FLinearColor::Black);
		TextItem.Scale = FVector2D(Width / 128.0f, Height / 128.0f);
		TextItem.Draw(Canvas);

	}
}
