// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailRendering/ParticleSystemThumbnailRenderer.h"
#include "Misc/App.h"
#include "UObject/ConstructorHelpers.h"
#include "ShowFlags.h"
#include "SceneView.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/Canvas.h"
#include "Particles/ParticleSystem.h"
#include "Engine/Texture2D.h"
#include "TextureResource.h"
#include "ThumbnailHelpers.h"
#include "UnrealEdGlobals.h"

// FPreviewScene derived helpers for rendering

#include "CanvasTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ParticleSystemThumbnailRenderer)

UParticleSystemThumbnailRenderer::UParticleSystemThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		ConstructorHelpers::FObjectFinder<UTexture2D> PSysThumbnail_NoImage;
		ConstructorHelpers::FObjectFinder<UTexture2D> PSysThumbnail_OOD;
		FConstructorStatics()
			: PSysThumbnail_NoImage(TEXT("/Engine/EditorMaterials/ParticleSystems/PSysThumbnail_NoImage"))
			, PSysThumbnail_OOD(TEXT("/Engine/EditorMaterials/ParticleSystems/PSysThumbnail_OOD"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	NoImage = ConstructorStatics.PSysThumbnail_NoImage.Object;
	OutOfDate = ConstructorStatics.PSysThumbnail_OOD.Object;
	ThumbnailScene = nullptr;
}

void UParticleSystemThumbnailRenderer::GetThumbnailSize(UObject* Object, float Zoom, uint32& OutWidth, uint32& OutHeight) const
{
	// Particle system thumbnails will be 1024x1024 at 100%.
	UParticleSystem* PSys = Cast<UParticleSystem>(Object);
	if (PSys != nullptr)
	{
		if ((PSys->bUseRealtimeThumbnail) ||
			(PSys->ThumbnailImage) || 
			(NoImage))
		{
			OutWidth = FMath::TruncToInt(1024 * Zoom);
			OutHeight = FMath::TruncToInt(1024 * Zoom);
		}
		else
		{
			// Nothing valid to display
			OutWidth = OutHeight = 0;
		}
	}
	else
	{
		OutWidth = OutHeight = 0;
	}
}

void UParticleSystemThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	if (GUnrealEd->GetThumbnailManager())
	{
		UParticleSystem* ParticleSystem = Cast<UParticleSystem>(Object);
		if (ParticleSystem != nullptr)
		{
			if ( ParticleSystem->bUseRealtimeThumbnail )
			{
				if ( ThumbnailScene == nullptr )
				{
					ThumbnailScene = new FParticleSystemThumbnailScene();
				}

				ThumbnailScene->SetParticleSystem(ParticleSystem);
				FSceneViewFamilyContext ViewFamily( FSceneViewFamily::ConstructionValues( RenderTarget, ThumbnailScene->GetScene(), FEngineShowFlags(ESFIM_Game))
					.SetTime(UThumbnailRenderer::GetTime())
					.SetAdditionalViewFamily(bAdditionalViewFamily));
			
				ViewFamily.EngineShowFlags.DisableAdvancedFeatures();
				ViewFamily.EngineShowFlags.MotionBlur = 0;

				RenderViewFamily(Canvas, &ViewFamily, ThumbnailScene->CreateView(&ViewFamily, X, Y, Width, Height));
				ThumbnailScene->SetParticleSystem(nullptr);
			}
			else if (ParticleSystem->ThumbnailImage)
			{
				Canvas->DrawTile(X,Y,Width,Height,0.f,0.f,1.f,1.f,FLinearColor::White,
					ParticleSystem->ThumbnailImage->GetResource(),false);
				if (ParticleSystem->ThumbnailImageOutOfDate == true)
				{
					Canvas->DrawTile(X,Y,Width/2,Height/2,0.f,0.f,1.f,1.f,FLinearColor::White,
						OutOfDate->GetResource(),true);
				}
			}
			else if (NoImage)
			{
				// Use the texture interface to draw
				Canvas->DrawTile(X,Y,Width,Height,0.f,0.f,1.f,1.f,FLinearColor::White,
					NoImage->GetResource(),false);
			}
		}
	}
}

void UParticleSystemThumbnailRenderer::BeginDestroy()
{
	if ( ThumbnailScene != nullptr )
	{
		delete ThumbnailScene;
		ThumbnailScene = nullptr;
	}

	Super::BeginDestroy();
}
