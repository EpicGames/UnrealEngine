// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailRendering/PhysicalMaterialMaskThumbnailRenderer.h"
#include "CanvasItem.h"
#include "Engine/Engine.h"
#include "EngineGlobals.h"
#include "PhysicalMaterials/PhysicalMaterialMask.h"
#include "CanvasTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicalMaterialMaskThumbnailRenderer)

UPhysicalMaterialMaskThumbnailRenderer::UPhysicalMaterialMaskThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UPhysicalMaterialMaskThumbnailRenderer::GetThumbnailSize(UObject* Object, float Zoom, uint32& OutWidth, uint32& OutHeight) const
{
	UPhysicalMaterialMask* PhysMatMask = Cast<UPhysicalMaterialMask>(Object);
	if (PhysMatMask->MaskTexture != nullptr)
	{
		Super::GetThumbnailSize(PhysMatMask->MaskTexture, Zoom, OutWidth, OutHeight);
	}
}

void UPhysicalMaterialMaskThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UPhysicalMaterialMask* PhysMatMask = Cast<UPhysicalMaterialMask>(Object);
	if (PhysMatMask->MaskTexture != nullptr)
	{
		Super::Draw(PhysMatMask->MaskTexture, X, Y, Width, Height, RenderTarget, Canvas, bAdditionalViewFamily);
	}
}
