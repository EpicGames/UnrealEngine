// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailRendering/PhysicsAssetThumbnailRenderer.h"
#include "Misc/App.h"
#include "ShowFlags.h"
#include "SceneView.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "ThumbnailHelpers.h"
#include "Engine/SkeletalMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsAssetThumbnailRenderer)

UPhysicsAssetThumbnailRenderer::UPhysicsAssetThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ThumbnailScene = nullptr;
}

bool UPhysicsAssetThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	if (!Super::CanVisualizeAsset(Object))
	{
		return false;
	}
	UPhysicsAsset* PhysicsAsset = Cast<UPhysicsAsset>(Object);
	if (!PhysicsAsset)
	{
		return false;
	}
	USkeletalMesh* SkeletalMesh = PhysicsAsset->PreviewSkeletalMesh.LoadSynchronous();
	const bool bValidRenderData = SkeletalMesh && SkeletalMesh->IsReadyToRenderInThumbnail();
	if (!bValidRenderData)
	{
		return false;
	}
	return true;
}

void UPhysicsAssetThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UPhysicsAsset* PhysicsAsset = Cast<UPhysicsAsset>(Object);
	if (PhysicsAsset)
	{
		USkeletalMesh* SkeletalMesh = PhysicsAsset->PreviewSkeletalMesh.LoadSynchronous();
		const bool bValidRenderData = SkeletalMesh && SkeletalMesh->IsReadyToRenderInThumbnail();
		if (!bValidRenderData)
		{
			return;
		}
		if (!ThumbnailScene)
		{
			ThumbnailScene = new FPhysicsAssetThumbnailScene();
		}

		ThumbnailScene->SetPhysicsAsset(PhysicsAsset);
		FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(RenderTarget, ThumbnailScene->GetScene(), FEngineShowFlags(ESFIM_Game))
			.SetTime(UThumbnailRenderer::GetTime())
			.SetAdditionalViewFamily(bAdditionalViewFamily));

		ViewFamily.EngineShowFlags.DisableAdvancedFeatures();
		ViewFamily.EngineShowFlags.MotionBlur = 0;
		ViewFamily.EngineShowFlags.LOD = 0;

		RenderViewFamily(Canvas, &ViewFamily, ThumbnailScene->CreateView(&ViewFamily, X, Y, Width, Height));
		ThumbnailScene->SetPhysicsAsset(nullptr);
	}
}

void UPhysicsAssetThumbnailRenderer::BeginDestroy()
{
	if ( ThumbnailScene != nullptr )
	{
		delete ThumbnailScene;
		ThumbnailScene = nullptr;
	}

	Super::BeginDestroy();
}
