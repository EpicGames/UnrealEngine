// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailRendering/SceneThumbnailInfo.h"

#include "Serialization/Archive.h"
#include "UObject/AnimPhysObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SceneThumbnailInfo)


USceneThumbnailInfo::USceneThumbnailInfo(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	OrbitPitch = -11.25;
	OrbitYaw = -157.5;
	OrbitZoom = 0;
}

void USceneThumbnailInfo::ResetToDefault()
{
	const USceneThumbnailInfo* Default = GetDefault<USceneThumbnailInfo>();
	OrbitPitch = Default->OrbitPitch;
	OrbitYaw = Default->OrbitYaw;
	OrbitZoom = Default->OrbitZoom;
}

bool USceneThumbnailInfo::DiffersFromDefault() const
{
	const USceneThumbnailInfo* Default = GetDefault<USceneThumbnailInfo>();
	return OrbitPitch != Default->OrbitPitch ||
		OrbitYaw != Default->OrbitYaw ||
		OrbitZoom != Default->OrbitZoom;
}

void USceneThumbnailInfo::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FAnimPhysObjectVersion::GUID);

	if (Ar.CustomVer(FAnimPhysObjectVersion::GUID) < FAnimPhysObjectVersion::ThumbnailSceneInfoAndAssetImportDataAreTransactional)
	{
		SetFlags(RF_Transactional);
	}
}
