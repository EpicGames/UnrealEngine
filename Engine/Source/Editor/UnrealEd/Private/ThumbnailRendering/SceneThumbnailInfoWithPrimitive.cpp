// Copyright Epic Games, Inc. All Rights Reserved.


#include "ThumbnailRendering/SceneThumbnailInfoWithPrimitive.h"

#include "Serialization/Archive.h"
#include "UObject/EditorObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SceneThumbnailInfoWithPrimitive)

USceneThumbnailInfoWithPrimitive::USceneThumbnailInfoWithPrimitive(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimitiveType = TPT_Sphere;
	OrbitPitch = -35;
	OrbitYaw = -180.f;
	OrbitZoom = 0.f;
	bUserModifiedShape = false;
}

void USceneThumbnailInfoWithPrimitive::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);
}

void USceneThumbnailInfoWithPrimitive::PostLoad()
{
	Super::PostLoad();

	if (GetLinkerCustomVersion(FEditorObjectVersion::GUID) < FEditorObjectVersion::MaterialThumbnailRenderingChanges)
	{
		bUserModifiedShape = PrimitiveType != TPT_Sphere;
	}
}

void USceneThumbnailInfoWithPrimitive::ResetToDefault()
{
	const USceneThumbnailInfoWithPrimitive* Default = GetDefault<USceneThumbnailInfoWithPrimitive>();
	PrimitiveType = Default->PrimitiveType;
	OrbitPitch = Default->OrbitPitch;
	OrbitYaw = Default->OrbitYaw;
	OrbitZoom = Default->OrbitZoom;
	bUserModifiedShape = false;
}

bool USceneThumbnailInfoWithPrimitive::DiffersFromDefault() const
{
	const USceneThumbnailInfoWithPrimitive* Default = GetDefault<USceneThumbnailInfoWithPrimitive>();
	return
		PrimitiveType != Default->PrimitiveType ||
		OrbitPitch != Default->OrbitPitch ||
		OrbitYaw != Default->OrbitYaw ||
		OrbitZoom != Default->OrbitZoom;

}
