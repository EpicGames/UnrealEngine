// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailRendering/SkeletalMeshThumbnailRenderer.h"
#include "Misc/App.h"
#include "ShowFlags.h"
#include "SceneView.h"
#include "Engine/SkeletalMesh.h"
#include "ThumbnailHelpers.h"
#include "UObject/PackageReload.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalMeshThumbnailRenderer)

USkeletalMeshThumbnailRenderer::USkeletalMeshThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PackageReloadedHandle = FCoreUObjectDelegates::OnPackageReloaded.AddUObject(this, &USkeletalMeshThumbnailRenderer::HandlePackageReloaded);
}

void USkeletalMeshThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Object);
	TSharedRef<FSkeletalMeshThumbnailScene> ThumbnailScene = ThumbnailSceneCache.EnsureThumbnailScene(Object);

	if(SkeletalMesh)
	{
		ThumbnailScene->SetSkeletalMesh(SkeletalMesh);
	}
	AddAdditionalPreviewSceneContent(Object, ThumbnailScene->GetWorld());

	FSceneViewFamilyContext ViewFamily( FSceneViewFamily::ConstructionValues( RenderTarget, ThumbnailScene->GetScene(), FEngineShowFlags(ESFIM_Game) )
		.SetTime(UThumbnailRenderer::GetTime())
		.SetAdditionalViewFamily(bAdditionalViewFamily));

	ViewFamily.EngineShowFlags.DisableAdvancedFeatures();
	ViewFamily.EngineShowFlags.MotionBlur = 0;
	ViewFamily.EngineShowFlags.LOD = 0;

	RenderViewFamily(Canvas, &ViewFamily, ThumbnailScene->CreateView(&ViewFamily, X, Y, Width, Height));
	ThumbnailScene->SetSkeletalMesh(nullptr);
}

EThumbnailRenderFrequency USkeletalMeshThumbnailRenderer::GetThumbnailRenderFrequency(UObject* Object) const
{
	USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Object);
	return SkeletalMesh && SkeletalMesh->GetResourceForRendering() 
		? EThumbnailRenderFrequency::Realtime
		: EThumbnailRenderFrequency::OnPropertyChange;
}

bool USkeletalMeshThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	if (!Super::CanVisualizeAsset(Object))
	{
		return false;
	}
	USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Object);
	const bool bValidRenderData = SkeletalMesh && SkeletalMesh->IsReadyToRenderInThumbnail();
	return bValidRenderData;
}

void USkeletalMeshThumbnailRenderer::BeginDestroy()
{
	FCoreUObjectDelegates::OnPackageReloaded.Remove(PackageReloadedHandle);

	ThumbnailSceneCache.Clear();

	Super::BeginDestroy();
}

void USkeletalMeshThumbnailRenderer::HandlePackageReloaded(EPackageReloadPhase Phase, FPackageReloadedEvent* Event)
{
	if (Phase != EPackageReloadPhase::PrePackageFixup || Event == nullptr)
	{
		return;
	}

	for (const TPair<UObject*, UObject*>& Pair : Event->GetRepointedObjects())
	{
		// We use the Value to check for the new skeletal mesh because the thumbnail scene caches using the object path
		// and during a reload the old asset package is renamed, changing its path
		if (Pair.Value && Pair.Value->IsA<USkeletalMesh>())
		{
			ThumbnailSceneCache.RemoveThumbnailScene(Pair.Value);
		}
	}
}
