// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailRendering/SkeletonThumbnailRenderer.h"

#include "AssetToolsModule.h"
#include "Misc/App.h"
#include "ShowFlags.h"
#include "SceneView.h"
#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"
#include "ThumbnailHelpers.h"
#include "UObject/PackageReload.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletonThumbnailRenderer)

USkeletonThumbnailRenderer::USkeletonThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PackageReloadedHandle = FCoreUObjectDelegates::OnPackageReloaded.AddUObject(this, &USkeletonThumbnailRenderer::HandlePackageReloaded);
}

bool USkeletonThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	//The const version of GetPreviewMesh function simply return the preview mesh, it wont verify the skeleton compatibility.
	//If we are not compiling call the function that are verifying the compatibility.
	const USkeleton* ConstSkeleton = Cast<USkeleton>(Object);
	USkeletalMesh* SkeletalMesh = ConstSkeleton->GetPreviewMesh();
	if (!SkeletalMesh || !SkeletalMesh->IsCompiling())
	{
		USkeleton* Skeleton = Cast<USkeleton>(Object);
		//This case can stall this thread (GameThread) since it will verify this skeleton compatibility with the preview skeletal mesh.
		//If the skeletal mesh is compiling a wait until compile will happen when doing USkeletalMesh::GetSkeleton()
		constexpr bool bFindIfNotSet = true;
		SkeletalMesh = Skeleton->GetPreviewMesh(bFindIfNotSet);
	}
	//Only visualize thumbnail if the skeletal mesh preview exist and is not compiling and have valid render data
	return (SkeletalMesh && SkeletalMesh->IsReadyToRenderInThumbnail());
}

void USkeletonThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	USkeleton* Skeleton = Cast<USkeleton>(Object);
	TSharedRef<FSkeletalMeshThumbnailScene> ThumbnailScene = ThumbnailSceneCache.EnsureThumbnailScene(Object);

	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
	const TSharedPtr<IAssetTypeActions> AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(USkeleton::StaticClass()).Pin();
	ThumbnailScene->SetDrawDebugSkeleton(true, AssetTypeActions->GetTypeColor());
	
	if(Skeleton)
	{
		constexpr bool bFindIfNotSet = true; 
		if(USkeletalMesh* SkeletalMesh = Skeleton->GetPreviewMesh(bFindIfNotSet))
		{
			ThumbnailScene->SetSkeletalMesh(SkeletalMesh);
		}
	}
	AddAdditionalPreviewSceneContent(Object, ThumbnailScene->GetWorld());

	FSceneViewFamilyContext ViewFamily( FSceneViewFamily::ConstructionValues( RenderTarget, ThumbnailScene->GetScene(), FEngineShowFlags(ESFIM_Game) )
		.SetTime(UThumbnailRenderer::GetTime())
		.SetAdditionalViewFamily(bAdditionalViewFamily));

	ViewFamily.EngineShowFlags.DisableAdvancedFeatures();
	ViewFamily.EngineShowFlags.MotionBlur = 0;
	ViewFamily.EngineShowFlags.LOD = 0;

	RenderViewFamily(Canvas, &ViewFamily, ThumbnailScene->CreateView(&ViewFamily, X, Y, Width, Height));
	ThumbnailScene->SetSkeletalMesh(nullptr);
}

EThumbnailRenderFrequency USkeletonThumbnailRenderer::GetThumbnailRenderFrequency(UObject* Object) const
{
	USkeleton* Skeleton = Cast<USkeleton>(Object);
	if(Skeleton)
	{
		if(USkeletalMesh* SkeletalMesh = Skeleton->GetPreviewMesh())
		{
			return SkeletalMesh->GetResourceForRendering() != nullptr 
				? EThumbnailRenderFrequency::Realtime
				: EThumbnailRenderFrequency::OnPropertyChange;
		}
	}
	return EThumbnailRenderFrequency::OnPropertyChange;
}

void USkeletonThumbnailRenderer::BeginDestroy()
{
	FCoreUObjectDelegates::OnPackageReloaded.Remove(PackageReloadedHandle);

	ThumbnailSceneCache.Clear();

	Super::BeginDestroy();
}

void USkeletonThumbnailRenderer::HandlePackageReloaded(EPackageReloadPhase Phase, FPackageReloadedEvent* Event)
{
	if (Phase != EPackageReloadPhase::PrePackageFixup || Event == nullptr)
	{
		return;
	}

	for (const TPair<UObject*, UObject*>& Pair : Event->GetRepointedObjects())
	{
		// We use the Value to check for the new skeletal mesh because the thumbnail scene caches using the object path
		// and during a reload the old asset package is renamed, changing its path
		if (Pair.Value)
		{
			if (Pair.Value->IsA<USkeleton>())
			{
				ThumbnailSceneCache.RemoveThumbnailScene(Pair.Value);
			}
			else if (Pair.Value->IsA<USkeletalMesh>())
			{
				// We can't easily tell if our preview skeletal mesh is reloaded
				// Since asset reload is rare, just clear any cached data
				ThumbnailSceneCache.Clear();
			}
		}
	}
}
