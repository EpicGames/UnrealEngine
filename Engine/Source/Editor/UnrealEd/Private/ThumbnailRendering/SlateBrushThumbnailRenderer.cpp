// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailRendering/SlateBrushThumbnailRenderer.h"
#include "Styling/SlateBrush.h"
#include "CanvasItem.h"
#include "Engine/Canvas.h"
#include "Engine/Texture2D.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Slate/SlateBrushAsset.h"
#include "TextureResource.h"
#include "Materials/MaterialInterface.h"
#include "Materials/Material.h"
// FPreviewScene derived helpers for rendering
#include "CanvasItem.h"
#include "CanvasTypes.h"
// helpers for rendering RoundedBox thumbnail as an image widget
#include "Slate/WidgetRenderer.h"
#include "Input/HittestGrid.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SVirtualWindow.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SlateBrushThumbnailRenderer)

USlateBrushThumbnailRenderer::USlateBrushThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bIsLastFrequencyRealTime(false)
{
}

void USlateBrushThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	USlateBrushAsset* SlateBrushAsset = Cast<USlateBrushAsset>(Object);
	if (SlateBrushAsset)
	{
		FSlateBrush Brush = SlateBrushAsset->Brush;
		UTexture2D* Texture = Cast<UTexture2D>( Brush.GetResourceObject() );
		UMaterialInterface* Material = Cast<UMaterialInterface>(Brush.GetResourceObject());

		if (bIsLastFrequencyRealTime)
		{
			bIsLastFrequencyRealTime = false;
		}

		if (Texture)
		{
			switch(Brush.DrawAs)
			{
			case ESlateBrushDrawType::Image:
			case ESlateBrushDrawType::Border:
			case ESlateBrushDrawType::NoDrawType:
				{
					CreateTextureThumbnailOnCanvas(X, Y, Width, Height, RenderTarget, Brush, Canvas, Texture);
				}
				break;
			case ESlateBrushDrawType::Box:
				{
					// Draw the checkerboard background
					const int32 CheckerDensity = 8;
					auto Checker = ToRawPtr(UThumbnailManager::Get().CheckerboardTexture);
					Canvas->DrawTile(
						0.0f, 0.0f, Width, Height,							// Dimensions
						0.0f, 0.0f, CheckerDensity, CheckerDensity,			// UVs
						FLinearColor::White, Checker->GetResource(),		// Tint & Texture
						true);
					
					float NaturalWidth = Texture->GetSurfaceWidth();
					float NaturalHeight = Texture->GetSurfaceHeight();

					float TopPx = FMath::Clamp<float>(NaturalHeight * Brush.Margin.Top, 0, Height);
					float BottomPx = FMath::Clamp<float>(NaturalHeight * Brush.Margin.Bottom, 0, Height);
					float VerticalCenterPx = FMath::Clamp<float>(Height - TopPx - BottomPx, 0, Height);
					float LeftPx = FMath::Clamp<float>(NaturalWidth * Brush.Margin.Left, 0, Width);
					float RightPx = FMath::Clamp<float>(NaturalWidth * Brush.Margin.Right, 0, Width);
					float HorizontalCenterPx = FMath::Clamp<float>(Width - LeftPx - RightPx, 0, Width);

					// Top-Left
					FVector2D TopLeftSize( LeftPx, TopPx );
					{
						FVector2D UV0( 0, 0 );
						FVector2D UV1( Brush.Margin.Left, Brush.Margin.Top );

						FCanvasTileItem CanvasTile( FVector2D( X, Y ), Texture->GetResource(), TopLeftSize, UV0, UV1, Brush.TintColor.GetSpecifiedColor() );
						CanvasTile.BlendMode = SE_BLEND_Translucent;
						CanvasTile.Draw( Canvas );
					}

					// Bottom-Left
					FVector2D BottomLeftSize( LeftPx, BottomPx );
					{
						FVector2D UV0( 0, 1 - Brush.Margin.Bottom );
						FVector2D UV1( Brush.Margin.Left, 1 );

						FCanvasTileItem CanvasTile( FVector2D( X, Y + Height - BottomPx ), Texture->GetResource(), BottomLeftSize, UV0, UV1, Brush.TintColor.GetSpecifiedColor() );
						CanvasTile.BlendMode = SE_BLEND_Translucent;
						CanvasTile.Draw( Canvas );
					}

					// Top-Right
					FVector2D TopRightSize( RightPx, TopPx );
					{
						FVector2D UV0( 1 - Brush.Margin.Right, 0 );
						FVector2D UV1( 1, Brush.Margin.Top );

						FCanvasTileItem CanvasTile( FVector2D( X + Width - RightPx, Y ), Texture->GetResource(), TopRightSize, UV0, UV1, Brush.TintColor.GetSpecifiedColor() );
						CanvasTile.BlendMode = SE_BLEND_Translucent;
						CanvasTile.Draw( Canvas );
					}

					// Bottom-Right
					FVector2D BottomRightSize( RightPx, BottomPx );
					{
						FVector2D UV0( 1 - Brush.Margin.Right, 1 - Brush.Margin.Bottom );
						FVector2D UV1( 1, 1 );

						FCanvasTileItem CanvasTile( FVector2D( X + Width - RightPx, Y + Height - BottomPx ), Texture->GetResource(), BottomRightSize, UV0, UV1, Brush.TintColor.GetSpecifiedColor() );
						CanvasTile.BlendMode = SE_BLEND_Translucent;
						CanvasTile.Draw( Canvas );
					}

					//-----------------------------------------------------------------------

					// Center-Vertical-Left
					FVector2D CenterVerticalLeftSize( LeftPx, VerticalCenterPx );
					{
						FVector2D UV0( 0, Brush.Margin.Top );
						FVector2D UV1( Brush.Margin.Left, 1 - Brush.Margin.Bottom );

						FCanvasTileItem CanvasTile( FVector2D( X, Y + TopPx), Texture->GetResource(), CenterVerticalLeftSize, UV0, UV1, Brush.TintColor.GetSpecifiedColor() );
						CanvasTile.BlendMode = SE_BLEND_Translucent;
						CanvasTile.Draw( Canvas );
					}

					// Center-Vertical-Right
					FVector2D CenterVerticalRightSize( RightPx, VerticalCenterPx );
					{
						FVector2D UV0( 1 - Brush.Margin.Right, Brush.Margin.Top );
						FVector2D UV1( 1, 1 - Brush.Margin.Bottom );

						FCanvasTileItem CanvasTile( FVector2D( X + Width - RightPx, Y + TopPx), Texture->GetResource(), CenterVerticalRightSize, UV0, UV1, Brush.TintColor.GetSpecifiedColor() );
						CanvasTile.BlendMode = SE_BLEND_Translucent;
						CanvasTile.Draw( Canvas );
					}

					//-----------------------------------------------------------------------

					// Center-Horizontal-Top
					FVector2D CenterHorizontalTopSize( HorizontalCenterPx, TopPx );
					{
						FVector2D UV0( Brush.Margin.Left, 0 );
						FVector2D UV1( 1 - Brush.Margin.Right, Brush.Margin.Top );

						FCanvasTileItem CanvasTile( FVector2D( X + LeftPx, Y), Texture->GetResource(), CenterHorizontalTopSize, UV0, UV1, Brush.TintColor.GetSpecifiedColor() );
						CanvasTile.BlendMode = SE_BLEND_Translucent;
						CanvasTile.Draw( Canvas );
					}

					// Center-Horizontal-Bottom
					FVector2D CenterHorizontalBottomSize( HorizontalCenterPx, BottomPx );
					{
						FVector2D UV0( Brush.Margin.Left, 1 - Brush.Margin.Bottom );
						FVector2D UV1( 1 - Brush.Margin.Right, 1 );

						FCanvasTileItem CanvasTile( FVector2D( X + LeftPx, Y + Height - BottomPx ), Texture->GetResource(), CenterHorizontalBottomSize, UV0, UV1, Brush.TintColor.GetSpecifiedColor() );
						CanvasTile.BlendMode = SE_BLEND_Translucent;
						CanvasTile.Draw( Canvas );
					}

					//-----------------------------------------------------------------------

					// Center
					FVector2D CenterSize( HorizontalCenterPx, VerticalCenterPx );
					{
						FVector2D UV0( Brush.Margin.Left, Brush.Margin.Top );
						FVector2D UV1( 1 - Brush.Margin.Right, 1 - Brush.Margin.Bottom );

						FCanvasTileItem CanvasTile( FVector2D( X + LeftPx, Y + TopPx), Texture->GetResource(), CenterSize, UV0, UV1, Brush.TintColor.GetSpecifiedColor() );
						CanvasTile.BlendMode = SE_BLEND_Translucent;
						CanvasTile.Draw( Canvas );
					}
				}
				break;
			case ESlateBrushDrawType::RoundedBox:
				{
					CreateThumbnailAsImage(Width, Height, RenderTarget, Brush);
				}
				break;
			default:

				check(false);
			}
		}
		else if (Material)
		{
			UMaterial* Mat = Material->GetMaterial();
			if (Mat && Mat->IsUIMaterial())
			{
				switch (Brush.DrawAs)
				{
				case ESlateBrushDrawType::Image:
				case ESlateBrushDrawType::Border:
				case ESlateBrushDrawType::Box:
				case ESlateBrushDrawType::NoDrawType:
				case ESlateBrushDrawType::RoundedBox:
				{
					bIsLastFrequencyRealTime = true;
					CreateThumbnailAsImage(Width, Height, RenderTarget, Brush);
				}
				break;
				default:

					check(false);
				}
			}
		}
	}
	
}

EThumbnailRenderFrequency USlateBrushThumbnailRenderer::GetThumbnailRenderFrequency(UObject* Object) const
{ 
	if (USlateBrushAsset* SlateBrushAsset = Cast<USlateBrushAsset>(Object))
	{
		FSlateBrush Brush = SlateBrushAsset->Brush;
		if (UMaterialInterface* Material = Cast<UMaterialInterface>(Brush.GetResourceObject()))
		{
			UMaterial* Mat = Material->GetMaterial();
			if (Mat && Mat->IsUIMaterial())
			{
				return EThumbnailRenderFrequency::Realtime;
			}
		}
	}

	// If we are switching from Realtime to OnAssetSave, we need to cache the realtime thumbnail first to use it until the next time we save the asset.
	// So we set the frequency to Once for the very first thumbnail rendering call after we have switched from Realtime.
	return bIsLastFrequencyRealTime ? EThumbnailRenderFrequency::Once : EThumbnailRenderFrequency::OnAssetSave;

}
void USlateBrushThumbnailRenderer::CreateThumbnailAsImage(uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FSlateBrush& Brush)
{
	const bool bUseGammaCorrection = true;
	FWidgetRenderer* WidgetRenderer = new FWidgetRenderer(bUseGammaCorrection, true);

	UTexture2D* CheckerboardTexture = UThumbnailManager::Get().CheckerboardTexture;
	FSlateBrush CheckerboardBrush;
	CheckerboardBrush.SetResourceObject(CheckerboardTexture);
	CheckerboardBrush.ImageSize = FVector2D(CheckerboardTexture->GetSizeX(), CheckerboardTexture->GetSizeY());
	CheckerboardBrush.Tiling = ESlateBrushTileType::Both;

	const FVector2D DrawSize((float)Width, (float)Height);
	const float DeltaTime = 0.f;

	if (Brush.GetResourceObject())
	{
		TSharedRef<SWidget> Thumbnail =
			SNew(SOverlay)

			// Checkerboard
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Image(&CheckerboardBrush)
			]

		+ SOverlay::Slot()
			[
				SNew(SImage)
				.Image(&Brush)
			];

		WidgetRenderer->DrawWidget(RenderTarget, Thumbnail, DrawSize, DeltaTime);
	}
	else
	{
		TSharedRef<SWidget> Thumbnail =
			SNew(SOverlay)

			// Checkerboard
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Image(&CheckerboardBrush)
			];

		WidgetRenderer->DrawWidget(RenderTarget, Thumbnail, DrawSize, DeltaTime);
	}

	if (WidgetRenderer)
	{
		BeginCleanup(WidgetRenderer);
		WidgetRenderer = nullptr;
	}
}

void USlateBrushThumbnailRenderer::CreateTextureThumbnailOnCanvas(int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FSlateBrush& Brush, FCanvas* Canvas, UTexture2D* Texture)
{
	// Draw the background checkboard pattern
	const int32 CheckerDensity = 8;
	auto Checker = ToRawPtr(UThumbnailManager::Get().CheckerboardTexture);
	Canvas->DrawTile(
		0.0f, 0.0f, Width, Height,							// Dimensions
		0.0f, 0.0f, CheckerDensity, CheckerDensity,			// UVs
		FLinearColor::White, Checker->GetResource(),		// Tint & Texture
		true);												// Alpha Blend
	FCanvasTileItem CanvasTile(FVector2D(X, Y), Texture->GetResource(), FVector2D(Width, Height), Brush.TintColor.GetSpecifiedColor());
	CanvasTile.BlendMode = SE_BLEND_Translucent;
	CanvasTile.Draw(Canvas);
}
