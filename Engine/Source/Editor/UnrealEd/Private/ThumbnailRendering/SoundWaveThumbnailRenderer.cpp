// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailRendering/SoundWaveThumbnailRenderer.h"
#include "CanvasItem.h"
#include "Audio.h"
#include "Sound/SoundWave.h"
#include "Sound/SoundSourceBus.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundWaveThumbnailRenderer)
 
USoundWaveThumbnailRenderer::USoundWaveThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool USoundWaveThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	USoundWave* SoundWave = Cast<USoundWave>(Object);
	if (!SoundWave)
	{
		return false;
	}

	return SoundWave->CanVisualizeAsset();
}

void USoundWaveThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	USoundWave* SoundWave = Cast<USoundWave>(Object);
	if (!SoundWave)
	{
		return;
	}

	SoundWave->SetRedrawThumbnail(false);
	if (SoundWave->NumChannels == 0 || !SoundWave->RawData.HasPayloadData())
	{
		return;
	}

	// Canvas line item to draw with
	FCanvasLineItem LineItem;
	LineItem.SetColor(FLinearColor::White);

	TFuture<FSharedBuffer> BufferFuture = SoundWave->RawData.GetPayload();
	const uint8* RawWaveData = (const uint8 *)BufferFuture.Get().GetData();  // Bulk
	int32 RawDataSize = BufferFuture.Get().GetSize();

	if (!RawWaveData || RawDataSize <= 0)
	{
		return;
	}
	 
	// Compute the scaled y-value used to render the channel data
	const float SampleYScale = Height / (2.f * 32767 * SoundWave->NumChannels);

	// How many wave files are we going to parse. Note: mono and stereo files are just one file, multi channel files have one mono file per channel.
	const uint32 NumPackedWaveFiles = SoundWave->NumChannels > 2 ? SoundWave->NumChannels : 1;

	// Loop through each packed wave file and render their data
	uint32 CurrentRawWaveByteIndex = 0;
	for (uint32 ChannelWaveFileIndex = 0; ChannelWaveFileIndex < NumPackedWaveFiles; ++ChannelWaveFileIndex)
	{
		uint32 RawChannelFileByteSize = 0;
		if (SoundWave->NumChannels > 2)
		{
			// For multi-channel files, we store the packed sound wave data sizes in the sound wave.
			check((int32)ChannelWaveFileIndex < SoundWave->ChannelSizes.Num());
			RawChannelFileByteSize = SoundWave->ChannelSizes[ChannelWaveFileIndex];
		}
		else
		{
			// For mono and stereo files, the sound file size is just the raw data size
			RawChannelFileByteSize = RawDataSize;
		}

		// If we have no data, then nothing to render
		if (RawChannelFileByteSize == 0)
		{
			continue;
		}

		// Read the packed wave file data
		FWaveModInfo WaveInfo;				
		if (WaveInfo.ReadWaveHeader(RawWaveData + CurrentRawWaveByteIndex, RawChannelFileByteSize, 0))
		{
			// Make sure we have a mono file here
			if (SoundWave->NumChannels > 2)
			{
				bool bLooksSane = (WaveInfo.pChannels && *WaveInfo.pChannels == 1);
				if (!bLooksSane)
				{
					UE_LOGF(LogAudio, Warning, "USoundWaveThumbnailRenderer::Draw: Not rendering thumbnail for Wave '%ls', as its supposed to be mono and has '%d' channels.", 
						*GetNameSafe(SoundWave), WaveInfo.pChannels ? *WaveInfo.pChannels : -1);

					continue;
				}
			}
			else
			{
				bool bLooksSane = WaveInfo.pChannels && *WaveInfo.pChannels == SoundWave->NumChannels;
				if (!bLooksSane)
				{
					UE_LOGF(LogAudio, Warning, "USoundWaveThumbnailRenderer::Draw: Not rendering thumbnail for Wave '%ls', as pChannels('%d') != SoundWave->NumChannels('%d')",
						*GetNameSafe(SoundWave), WaveInfo.pChannels ? *WaveInfo.pChannels : -1, SoundWave->NumChannels);
							
					continue;
				}
			}

			// Sample count
			const uint32 TotalSampleCount = WaveInfo.SampleDataSize / sizeof(int16);
			const uint32 TotalFrameCount = SoundWave->NumChannels == 2 ? TotalSampleCount / 2 : TotalSampleCount;
			const uint32 NumChannelsInWaveFile = *WaveInfo.pChannels;
			const uint32 FramesPerPixel = TotalFrameCount / Width;

			// Get the sample data of this file
			const int16* SamplePtr = reinterpret_cast<const int16*>(WaveInfo.SampleDataStart);

			// Render each channel separately so outer loop is the sound wave channel index.
			// Note: for multi-channel files this will always be 1-channel (mono).
			for (uint32 ChannelIndex = 0; ChannelIndex < NumChannelsInWaveFile; ++ChannelIndex)
			{
				// Reset the current frame count as we're starting from the beginning of the file to
				// render the channel data
				uint32 CurrentFrameCount = 0;

				// Loop through each pixel (in x direction)
				for (uint32 PixelIndex = 0; PixelIndex < Width; ++PixelIndex)
				{
					// reset the sample sum and num samples in pixel for each pixel
					int64 SampleSum = 0;
					int32 NumSamplesInPixel = 0;

					// Loop through all pixels in this x-frame, sum all audio data. Track total frames rendered to avoid writing past buffer boundary
					for (uint32 PixelFrameIndex = 0; PixelFrameIndex < FramesPerPixel && CurrentFrameCount < TotalFrameCount; ++PixelFrameIndex)
					{
						// Get the sample value of the wave file
						const uint32 SampleIndex = CurrentFrameCount * NumChannelsInWaveFile + ChannelIndex;
						check(SampleIndex < TotalSampleCount);
						int16 SampleValue = SamplePtr[SampleIndex];

						// Sum the sample value with the running sum
						SampleSum += FMath::Abs(SampleValue);

						// Track the number of samples we're actually summing to get an accurate average
						++NumSamplesInPixel;

						// Increment the frame after processing channels
						++CurrentFrameCount;
					}

					// If we actually added any audio data in this pixel
					if (NumSamplesInPixel > 0)
					{
						const float AverageSampleValue = (float)SampleSum / NumSamplesInPixel;
						const float AverageSampleValueScaled = AverageSampleValue * SampleYScale;

						// Don't try to draw anything if the audio data was too quiet
						if (AverageSampleValueScaled > 0.001f)
						{
							// What channel we're rendering is going to be one of the interleaved channels (ChannelIndex) in the case
							// of a stereo file, or the channel wave file index for multi-channel (or mono) files.
							uint32 Channel = SoundWave->NumChannels == 2 ? ChannelIndex : ChannelWaveFileIndex;

							// Draw vertical line mirrored around x-axis for channel equal to average sample value height
							const float YCenter = Y + ((2 * Channel) + 1) * Height / (2.f * SoundWave->NumChannels);
							LineItem.Draw(Canvas, FVector2D(X + PixelIndex, YCenter - AverageSampleValueScaled), FVector2D(X + PixelIndex, YCenter + AverageSampleValueScaled));
						}
					}
				}
			}
		}
		// Offset the raw wave data byte index by the current mono file byte size
		CurrentRawWaveByteIndex += RawChannelFileByteSize;
	}

}

EThumbnailRenderFrequency USoundWaveThumbnailRenderer::GetThumbnailRenderFrequency(UObject* Object) const
{
	if (USoundWave* SoundWave = Cast<USoundWave>(Object))
	{
		return SoundWave->CanVisualizeAsset()
			? EThumbnailRenderFrequency::Realtime
			: EThumbnailRenderFrequency::OnPropertyChange;
	}

	return EThumbnailRenderFrequency::OnPropertyChange;
}
