// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailRendering/SpecularProfileRenderer.h"
#include "CanvasItem.h"
#include "Engine/Engine.h"
#include "EngineGlobals.h"
#include "Engine/SpecularProfile.h"
#include "CanvasTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SpecularProfileRenderer)

USpecularProfileRenderer::USpecularProfileRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void USpecularProfileRenderer::GetThumbnailSize(UObject* Object, float Zoom, uint32& OutWidth, uint32& OutHeight) const
{
	OutWidth = 128;
	OutHeight = 128;
}

void USpecularProfileRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	USpecularProfile* LocalSpecularProfile = Cast<USpecularProfile>(Object);
	if (LocalSpecularProfile)
	{
		FLinearColor Col;
		Col = LocalSpecularProfile->Settings.ViewColor.GetLinearColorValue(1.0f); Col.A = 1;
		Canvas->DrawTile(0,          0, Width, Height / 2, 0, 0, 1, 1, Col);
		Col = LocalSpecularProfile->Settings.LightColor.GetLinearColorValue(1.0f); Col.A = 1;
		Canvas->DrawTile(0, Height / 2, Width, Height / 2, 0, 0, 1, 1, Col);

		FCanvasTextItem TextItem(FVector2D(5.0f, 5.0f), FText::FromString(LocalSpecularProfile->Settings.IsProcedural() ? TEXT("Procedural") : TEXT("Texture")), GEngine->GetLargeFont(), FLinearColor::White);
		TextItem.EnableShadow(FLinearColor::Black);
		TextItem.Scale = FVector2D(Width / 128.0f, Height / 128.0f);
		TextItem.Draw(Canvas);
	}
}

bool USpecularProfileRenderer::CanVisualizeAsset(UObject* Object)
{
	return true;
}

EThumbnailRenderFrequency USpecularProfileRenderer::GetThumbnailRenderFrequency(UObject* Object) const
{
	return EThumbnailRenderFrequency::OnPropertyChange;
}
