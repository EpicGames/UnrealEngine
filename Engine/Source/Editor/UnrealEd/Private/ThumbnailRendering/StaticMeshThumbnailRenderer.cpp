// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailRendering/StaticMeshThumbnailRenderer.h"
#include "Misc/App.h"
#include "ShowFlags.h"
#include "SceneInterface.h"
#include "SceneView.h"
#include "ThumbnailHelpers.h"
#include "Engine/StaticMesh.h"
#include "RenderingThread.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StaticMeshThumbnailRenderer)

UStaticMeshThumbnailRenderer::UStaticMeshThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ThumbnailScene = nullptr;
}

void UStaticMeshThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UStaticMesh* StaticMesh = Cast<UStaticMesh>(Object);
	if (IsValid(StaticMesh))
	{
		if (ThumbnailScene == nullptr || ensure(ThumbnailScene->GetWorld() != nullptr) == false)
		{
			if (ThumbnailScene)
			{
				FlushRenderingCommands();
				delete ThumbnailScene;
			}
			ThumbnailScene = new FStaticMeshThumbnailScene();
		}

		ThumbnailScene->SetStaticMesh(StaticMesh);
		ThumbnailScene->GetScene()->UpdateSpeedTreeWind(0.0);

		FSceneViewFamilyContext ViewFamily( FSceneViewFamily::ConstructionValues( RenderTarget, ThumbnailScene->GetScene(), FEngineShowFlags(ESFIM_Game) )
			.SetTime(UThumbnailRenderer::GetTime())
			.SetAdditionalViewFamily(bAdditionalViewFamily));

		ViewFamily.EngineShowFlags.DisableAdvancedFeatures();
		ViewFamily.EngineShowFlags.MotionBlur = 0;
		ViewFamily.EngineShowFlags.LOD = 0;

		RenderViewFamily(Canvas, &ViewFamily, ThumbnailScene->CreateView(&ViewFamily, X, Y, Width, Height));
		ThumbnailScene->SetStaticMesh(nullptr);
	}
}

void UStaticMeshThumbnailRenderer::BeginDestroy()
{
	if ( ThumbnailScene != nullptr )
	{
		delete ThumbnailScene;
		ThumbnailScene = nullptr;
	}

	Super::BeginDestroy();
}
