// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailRendering/SubsurfaceProfileRenderer.h"
#include "CanvasItem.h"
#include "Engine/Engine.h"
#include "EngineGlobals.h"
#include "Engine/SubsurfaceProfile.h"
#include "CanvasTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubsurfaceProfileRenderer)

USubsurfaceProfileRenderer::USubsurfaceProfileRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void USubsurfaceProfileRenderer::GetThumbnailSize(UObject* Object, float Zoom, uint32& OutWidth, uint32& OutHeight) const
{
	OutWidth = 128;
	OutHeight = 128;
}

void USubsurfaceProfileRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	USubsurfaceProfile* LocalSubsurfaceProfile = Cast<USubsurfaceProfile>(Object);
	if (LocalSubsurfaceProfile)
	{
		FLinearColor Col;

		Col = LocalSubsurfaceProfile->Settings.Tint; Col.A = 1;
		Canvas->DrawTile(0,          0, Width, Height / 2, 0, 0, 1, 1, Col);
		Col = LocalSubsurfaceProfile->Settings.SurfaceAlbedo; Col.A = 1;
		Canvas->DrawTile(0, Height / 2, Width, Height / 2, 0, 0, 1, 1, Col);

		FText MeanFreePathDistanceText = FText::AsNumber(LocalSubsurfaceProfile->Settings.MeanFreePathDistance);
		FCanvasTextItem TextItem(FVector2D(5.0f, 5.0f), MeanFreePathDistanceText, GEngine->GetLargeFont(), FLinearColor::White);
		TextItem.EnableShadow(FLinearColor::Black);
		TextItem.Scale = FVector2D(Width / 128.0f, Height / 128.0f);
		TextItem.Draw(Canvas);
	}
}

bool USubsurfaceProfileRenderer::CanVisualizeAsset(UObject* Object)
{
	return true;
}

EThumbnailRenderFrequency USubsurfaceProfileRenderer::GetThumbnailRenderFrequency(UObject* Object) const
{
	return EThumbnailRenderFrequency::OnPropertyChange;
}
