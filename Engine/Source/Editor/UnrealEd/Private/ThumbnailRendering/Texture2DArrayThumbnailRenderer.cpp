// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/Texture2dArrayThumbnailRenderer.h"
#include "Engine/Texture2DArray.h"
#include "ThumbnailRendering/ThumbnailManager.h"

UTexture2DArrayThumbnailRenderer::UTexture2DArrayThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UTexture2DArrayThumbnailRenderer::GetThumbnailSize(UObject* Object, float Zoom, uint32& OutWidth, uint32& OutHeight) const
{
	OutWidth = 0;
	OutHeight = 0;

	UTexture2DArray* TextureArray = Cast<UTexture2DArray>(Object);
	if (TextureArray != nullptr)
	{
		// Let the base class get the size of a face
		Super::GetThumbnailSize(TextureArray, Zoom, OutWidth, OutHeight);
	}

}

void UTexture2DArrayThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UTexture2DArray* TextureArray = Cast<UTexture2DArray>(Object);
	if (TextureArray != nullptr)
	{
		Super::Draw(TextureArray, X, Y, Width, Height, nullptr, Canvas, bAdditionalViewFamily);
	}
}
