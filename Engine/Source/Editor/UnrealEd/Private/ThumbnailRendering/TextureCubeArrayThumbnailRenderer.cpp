// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/TextureCubeArrayThumbnailRenderer.h"
#include "Engine/TextureCubeArray.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextureCubeArrayThumbnailRenderer)

UTextureCubeArrayThumbnailRenderer::UTextureCubeArrayThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UTextureCubeArrayThumbnailRenderer::GetThumbnailSize(UObject* Object, float Zoom, uint32& OutWidth, uint32& OutHeight) const
{
	OutWidth = 0;
	OutHeight = 0;

	UTextureCubeArray* CubeMap = Cast<UTextureCubeArray>(Object);
	if (CubeMap != nullptr)
	{
		// Let the base class get the size of a face
		Super::GetThumbnailSize(CubeMap, Zoom, OutWidth, OutHeight);
	}
}

void UTextureCubeArrayThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UTextureCubeArray* CubeMap = Cast<UTextureCubeArray>(Object);
	if (CubeMap != nullptr)
	{
		Super::Draw(CubeMap, X, Y, Width, Height, nullptr, Canvas, bAdditionalViewFamily);
	}
}
