// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/TextureCubeThumbnailRenderer.h"
#include "Engine/TextureCube.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextureCubeThumbnailRenderer)

UTextureCubeThumbnailRenderer::UTextureCubeThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UTextureCubeThumbnailRenderer::GetThumbnailSize(UObject* Object, float Zoom, uint32& OutWidth, uint32& OutHeight) const
{
	OutWidth = 0;
	OutHeight = 0;

	UTextureCube* CubeMap = Cast<UTextureCube>(Object);
	if (CubeMap != nullptr)
	{
		// Let the base class get the size of a face
		Super::GetThumbnailSize(CubeMap, Zoom, OutWidth, OutHeight);
	}
}

void UTextureCubeThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UTextureCube* CubeMap = Cast<UTextureCube>(Object);
	if (CubeMap != nullptr)
	{
		Super::Draw(CubeMap, X, Y, Width, Height, nullptr, Canvas, bAdditionalViewFamily);
	}
}
