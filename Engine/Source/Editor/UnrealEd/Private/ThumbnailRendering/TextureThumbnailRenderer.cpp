// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailRendering/TextureThumbnailRenderer.h"
#include "CanvasItem.h"
#include "Engine/Canvas.h"
#include "Engine/Engine.h"
#include "Engine/Texture.h"
#include "Engine/Texture2D.h"
#include "Engine/TextureLightProfile.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "EngineGlobals.h"
#include "Engine/TextureCube.h"
#include "Engine/Texture2DArray.h"
#include "Engine/TextureCubeArray.h"
#include "Texture2DPreview.h"
#include "Engine/TextureRenderTargetCube.h"

#include "CubemapUnwrapUtils.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "TextureResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextureThumbnailRenderer)

UTextureThumbnailRenderer::UTextureThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UTextureThumbnailRenderer::GetThumbnailSize(UObject* Object, float Zoom, uint32& OutWidth, uint32& OutHeight) const
{
	UTexture* Texture = Cast<UTexture>(Object);
	UTextureLightProfile* TextureLightProfile = Cast<UTextureLightProfile>(Object);

	if (TextureLightProfile)
	{
		// otherwise a 1D texture would result in a very boring thumbnail
		OutWidth = 192;
		OutHeight = 192;
		return;
	}

	if (Texture != nullptr)
	{
		UTexture2D* Texture2D = Cast<UTexture2D>(Texture);
		FSharedImageConstRef CpuCopy;
		if (Texture2D)
		{
			CpuCopy = Texture2D->GetCPUCopy();
		}

		if (CpuCopy.IsValid())
		{
			OutWidth = FMath::TruncToInt(Zoom * (float)CpuCopy->GetWidth());
			OutHeight = FMath::TruncToInt(Zoom * (float)CpuCopy->GetHeight());
		}
		else
		{
			OutWidth = FMath::TruncToInt(Zoom * (float)Texture->GetSurfaceWidth());
			OutHeight = FMath::TruncToInt(Zoom * (float)Texture->GetSurfaceHeight());
		}
	}
	else
	{
		OutWidth = OutHeight = 0;
	}
}

void UTextureThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UTexture* Texture = Cast<UTexture>(Object);
	if (Texture != nullptr && Texture->GetResource() != nullptr)
	{
		UTexture2D* Texture2D = Cast<UTexture2D>(Texture);
		UTexture* CpuCopyTexture = nullptr;
		if (Texture2D)
		{
			CpuCopyTexture = Texture2D->GetCPUCopyTexture();
		}

		// behavior here should match FTextureEditorViewportClient as much as possible

		if (CpuCopyTexture == nullptr)
		{
			// Take the alpha channel into account for textures that have one.
			// This provides a much better preview than just showing RGB,
			// Because the RGB content in areas with an alpha of 0 is often garbage that will not be seen in normal conditions.
			// Non-UI textures often have uncorrelated data in the alpha channel (like a skin mask, specular power, etc) so we only preview UI textures this way.
			// @todo : this logic depending on LODGroup to decide whether alpha thumbnails should blend is very odd and unexpected for the user.  Consider changing.
			const bool bUseTranslucentBlend = Texture2D && Texture2D->HasAlphaChannel() && ((Texture2D->LODGroup == TEXTUREGROUP_UI) || (Texture2D->LODGroup == TEXTUREGROUP_Pixels2D));

			UTextureCube* TextureCube = Cast<UTextureCube>(Texture);
			UTexture2DArray* Texture2DArray = Cast<UTexture2DArray>(Texture);
			UTextureCubeArray* TextureCubeArray = Cast<UTextureCubeArray>(Texture);
			UTextureRenderTargetCube* RTTextureCube = Cast<UTextureRenderTargetCube>(Texture);
			UTextureLightProfile* TextureLightProfile = Cast<UTextureLightProfile>(Texture);
			const bool bIsVirtualTexture = Texture->IsCurrentlyVirtualTextured();

			TRefCountPtr<FBatchedElementParameters> BatchedElementParameters;
			// BatchedElementParameters is released by the render thread when it was rendered

			if(TextureCube || TextureCubeArray || RTTextureCube)
			{
				// show LongLat Unwrap
				float MipLevel = -1.f;
				float SliceIndex = -1.f;
				bool bShowLongLatUnwrap = true;
				BatchedElementParameters = new FMipLevelBatchedElementParameters(MipLevel, SliceIndex, TextureCubeArray != nullptr, FMatrix44f::Identity, bShowLongLatUnwrap, false, false);
			
				// If the thumbnail is square then make it 2:1 for cubes.
				if(Width == Height)
				{
					Height = Width / 2;
					Y += Height / 2;
				}
			}
			else if (TextureLightProfile)
			{
				BatchedElementParameters = new FIESLightProfileBatchedElementParameters(TextureLightProfile->Brightness);
			}
			else if (Texture2D || Texture2DArray)
			{
				bool bIsNormalMap = Texture->IsNormalMap();
				bool bIsSingleChannel = Texture->CompressionSettings == TC_Grayscale || Texture->CompressionSettings == TC_Alpha;
				bool bSingleVTPhysicalSpace = Texture2D && Texture2D->IsVirtualTexturedWithSinglePhysicalSpace();
				float MipLevel = -1.f;
				float LayerIndex = 0;
				float SliceIndex = -1.f;
				bool bIsTextureArray = (Texture2DArray != nullptr);
				bool bUsePointSampling = false;

				// if you correctly tell FBatchedElementTexture2DPreviewParameters that the texture is a VT
				//	then you only get a solid color thumbnail for the initial render
				//	but if it is refreshes or the VT is examined, the thumbnail will become correct
				//bool bSampleAsVirtualTexture = bIsVirtualTexture;

				// if you just always lie and say you are not a VT, you get a good thumbnail :
				//	this makes use of the non-VT Editor-only texture that VT makes in FVirtualTexture2DResource::InitializeEditorResources
				bool bSampleAsVirtualTexture = false;

				BatchedElementParameters = new FBatchedElementTexture2DPreviewParameters(MipLevel, LayerIndex, SliceIndex, bIsNormalMap, bIsSingleChannel, bSingleVTPhysicalSpace, bSampleAsVirtualTexture, bIsTextureArray, bUsePointSampling);
			}
			else
			{
				// BatchedElementParameters is not set
				// default Canvas will be used, which just shows the texture
				// (what shader does this use??)

				// some UTexture types can hit this
				// UTextureRenderTarget

				// ?? maybe get rid of this and just always use the Texture2D branch above
			}

			if (bUseTranslucentBlend)
			{
				// If using alpha, draw a checkerboard underneath first.
				const int32 CheckerDensity = 8;
				auto Checker = UThumbnailManager::Get().CheckerboardTexture;
				Canvas->DrawTile(
					0.0f, 0.0f, Width, Height,							// Dimensions
					0.0f, 0.0f, CheckerDensity, CheckerDensity,			// UVs
					FLinearColor::White, Checker->GetResource());			// Tint & Texture
			}

			// Use A canvas tile item to draw
			FCanvasTileItem CanvasTile( FVector2D( X, Y ), Texture->GetResource(), FVector2D( Width,Height ), FLinearColor::White );
			if ( BatchedElementParameters == nullptr )
			{
				// old style BatchedElementParameters == null wants you to use "Translucent" :
				CanvasTile.BlendMode = bUseTranslucentBlend ? SE_BLEND_Translucent : SE_BLEND_Opaque;
			}
			else
			{
				// matches the behavior of FTextureEditorViewportClient::Draw and FTextureEditorToolkit::GetColourChannelBlendMode
				// I think it may be a bug in BatchedElement that SE_BLEND_Translucent doesn't work here (failing to set up TextureComponentReplicate?)
				CanvasTile.BlendMode = bUseTranslucentBlend ? (ESimpleElementBlendMode)(SE_BLEND_RGBA_MASK_START+0xF) : SE_BLEND_Opaque;
			}
			CanvasTile.BatchedElementParameters = BatchedElementParameters;
			if (bIsVirtualTexture && Texture->Source.GetNumBlocks() > 1)
			{
				// Adjust UVs to display entire UDIM range, acounting for UE inverted V-axis
				// We're not actually rendering a VT here, but the editor-only texture we're using is still using the UDIM tile layout
				// So we use inverted Y-axis, but then normalize back to [0,1)
				const FIntPoint BlockSize = Texture->Source.GetSizeInBlocks();
				const float RcpBlockSizeY = 1.0f / (float)BlockSize.Y;
				CanvasTile.UV0.Y = (1.0f - (float)BlockSize.Y) * RcpBlockSizeY;
				CanvasTile.UV1.Y = RcpBlockSizeY;
			}
			CanvasTile.Draw( Canvas );

			if (TextureLightProfile)
			{
				float Brightness = TextureLightProfile->Brightness;

				// Brightness in Lumens
				FText BrightnessText = FText::AsNumber( Brightness );
				FCanvasTextItem TextItem( FVector2D( 5.0f, 5.0f ), BrightnessText, GEngine->GetLargeFont(), FLinearColor::White );
				TextItem.EnableShadow(FLinearColor::Black);
				TextItem.Scale = FVector2D(Width / 128.0f, Height / 128.0f);
				TextItem.Draw(Canvas);
			}

			if (bIsVirtualTexture)
			{
				auto VTChars = TEXT("VT");
				int32 VTWidth = 0;
				int32 VTHeight = 0;
				StringSize(GEngine->GetLargeFont(), VTWidth, VTHeight, VTChars);
				float PaddingX = Width / 128.0f;
				float PaddingY = Height / 128.0f;
				float ScaleX = Width / 64.0f; //Text is 1/64'th of the size of the thumbnails
				float ScaleY = Height / 64.0f;
				// VT overlay
				FCanvasTextItem TextItem(FVector2D(Width - PaddingX - VTWidth * ScaleX, Height - PaddingY - VTHeight * ScaleY), FText::FromString(VTChars), GEngine->GetLargeFont(), FLinearColor::White);
				TextItem.EnableShadow(FLinearColor::Black);
				TextItem.Scale = FVector2D(ScaleX, ScaleY);
				TextItem.Draw(Canvas);
			}
		}
		else
		{
			// Render a tile using a transient texture that holds the cpu copy.
			Canvas->DrawTile(
				0.0f, 0.0f, Width, Height,							// Dimensions
				0.0f, 0.0f, 1.0f, 1.0f,								// UVs
				FLinearColor::White, CpuCopyTexture->GetResource());			// Tint & Texture

			const TCHAR* Chars = TEXT("CPU");
			int32 TextWidth = 0;
			int32 TextHeight = 0;
			StringSize(GEngine->GetLargeFont(), TextWidth, TextHeight, Chars);
			float PaddingX = Width / 128.0f;
			float PaddingY = Height / 128.0f;
			float ScaleX = Width / 64.0f; //Text is 1/64'th of the size of the thumbnails
			float ScaleY = Height / 64.0f;
			// overlay
			FCanvasTextItem TextItem(FVector2D(Width - PaddingX - TextWidth * ScaleX, Height - PaddingY - TextHeight * ScaleY), FText::FromString(Chars), GEngine->GetLargeFont(), FLinearColor::White);
			TextItem.EnableShadow(FLinearColor::Black);
			TextItem.Scale = FVector2D(ScaleX, ScaleY);
			TextItem.Draw(Canvas);
		}
	}
}

bool UTextureThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	UTexture* Texture = Cast<UTexture>(Object);
	return (Texture && !Texture->IsCompiling());
}
