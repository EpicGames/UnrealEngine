// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailRendering/ThumbnailRenderer.h"

#include "RendererInterface.h"
#include "EngineModule.h"
#include "SceneView.h"
#include "SceneViewExtension.h"
#include "LegacyScreenPercentageDriver.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ThumbnailRenderer)

UThumbnailRenderer::UThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

// static
void UThumbnailRenderer::RenderViewFamily(FCanvas* Canvas, FSceneViewFamily* ViewFamily, FSceneView* View)
{
	if ((ViewFamily == nullptr) || (View == nullptr))
	{ 
		return;
	}

	check(ViewFamily->Views.Num() == 1 && (ViewFamily->Views[0] == View));

	ViewFamily->EngineShowFlags.ScreenPercentage = false;
	ViewFamily->SetScreenPercentageInterface(new FLegacyScreenPercentageDriver(
		*ViewFamily, /* GlobalResolutionFraction = */ 1.0f));

	ViewFamily->ViewExtensions = GEngine->ViewExtensions->GatherActiveExtensions(FSceneViewExtensionContext(ViewFamily->Scene));

	// View extensions should have a chance at changing ViewFamily and View (while it's still mutable, since ViewFamily->Views contains const FSceneView pointers) before rendering :
	for (const FSceneViewExtensionRef& Extension : ViewFamily->ViewExtensions)
	{
		Extension->SetupViewFamily(*ViewFamily);
		Extension->SetupView(*ViewFamily, *View);
	}

	GetRendererModule().BeginRenderingViewFamily(Canvas, ViewFamily);
}

// static
FGameTime UThumbnailRenderer::GetTime()
{
	return FGameTime::GetTimeSinceAppStart();
}
