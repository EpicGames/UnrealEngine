// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailRendering/ToonProfileRenderer.h"
#include "CanvasItem.h"
#include "Engine/Engine.h"
#include "EngineGlobals.h"
#include "Engine/ToonProfile.h"
#include "CanvasTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ToonProfileRenderer)

UToonProfileRenderer::UToonProfileRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UToonProfileRenderer::GetThumbnailSize(UObject* Object, float Zoom, uint32& OutWidth, uint32& OutHeight) const
{
	OutWidth = 128;
	OutHeight = 128;
}

void UToonProfileRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UToonProfile* LocalToonProfile = Cast<UToonProfile>(Object);
	if (LocalToonProfile)
	{
		FLinearColor Col; 

		const float TimeStep = 0.1f;
		for (float Time = 0.0f; Time < 1.0f; Time += TimeStep)
		{
			Col = LocalToonProfile->Settings.DiffuseRamp.GetLinearColorValue(Time);
			Col.A = 1;
			Canvas->DrawTile(float(Width) * Time, 0, float(Width) * (Time + TimeStep), Height/2, 0, 0, 1, 1, Col);

			Col = LocalToonProfile->Settings.SpecularRamp.GetLinearColorValue(Time);
			Col.A = 1;
			Canvas->DrawTile(float(Width) * Time, Height / 2, float(Width) * (Time + TimeStep), Height, 0, 0, 1, 1, Col);
		}

		FCanvasTextItem TextItem(FVector2D(5.0f, 5.0f), FText::FromString(TEXT("TOON")), GEngine->GetLargeFont(), FLinearColor::White);
		TextItem.EnableShadow(FLinearColor::Black);
		TextItem.Scale = FVector2D(Width / 128.0f, Height / 128.0f);
		TextItem.Draw(Canvas);
	}
}

bool UToonProfileRenderer::CanVisualizeAsset(UObject* Object)
{
	return true;
}

EThumbnailRenderFrequency UToonProfileRenderer::GetThumbnailRenderFrequency(UObject* Object) const
{
	return EThumbnailRenderFrequency::OnPropertyChange;
}
