// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailRendering/VolumeTextureThumbnailRenderer.h"
#include "Misc/App.h"
#include "ShowFlags.h"
#include "Materials/Material.h"
#include "Engine/VolumeTexture.h"
#include "SceneView.h"
#include "ThumbnailHelpers.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"
#include "UnrealEdGlobals.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "ThumbnailRendering/SceneThumbnailInfoWithPrimitive.h"
#include "Editor/UnrealEdEngine.h"

// FPreviewScene derived helpers for rendering
#include "RendererInterface.h"
#include "EngineModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VolumeTextureThumbnailRenderer)

/*
***************************************************************
  FVolumeTextureThumbnailScene
***************************************************************
*/

class UNREALED_API FVolumeTextureThumbnailScene : public FThumbnailPreviewScene
{
public:	
	/** Constructor */
	FVolumeTextureThumbnailScene();

	/** Sets the material to use in the next CreateView() */
	void SetMaterialInterface(UMaterialInstance* InMaterial);

protected:
	// FThumbnailPreviewScene implementation
	virtual void GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const override;

protected:

	/** The static mesh actor used to display all material thumbnails */
	class AStaticMeshActor* PreviewActor;
};


FVolumeTextureThumbnailScene::FVolumeTextureThumbnailScene()
	: FThumbnailPreviewScene()
{
	bForceAllUsedMipsResident = false;

	// Create preview actor
	// checked
	FActorSpawnParameters SpawnInfo;
	SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnInfo.bNoFail = true;
	SpawnInfo.ObjectFlags = RF_Transient;
	PreviewActor = GetWorld()->SpawnActor<AStaticMeshActor>( SpawnInfo );

	PreviewActor->GetStaticMeshComponent()->SetMobility(EComponentMobility::Movable);
	PreviewActor->SetActorEnableCollision(false);
}

void FVolumeTextureThumbnailScene::SetMaterialInterface(UMaterialInstance* InMaterial)
{
	check(PreviewActor);
	check(PreviewActor->GetStaticMeshComponent());

	if (InMaterial)
	{
		// Transform the preview mesh as necessary
		FTransform Transform = FTransform::Identity;

		const USceneThumbnailInfoWithPrimitive* ThumbnailInfo = Cast<USceneThumbnailInfoWithPrimitive>(InMaterial->ThumbnailInfo);
		if ( !ThumbnailInfo )
		{
			ThumbnailInfo = USceneThumbnailInfoWithPrimitive::StaticClass()->GetDefaultObject<USceneThumbnailInfoWithPrimitive>();
		}

		PreviewActor->GetStaticMeshComponent()->SetStaticMesh(GUnrealEd->GetThumbnailManager()->EditorCube);
		PreviewActor->GetStaticMeshComponent()->SetRelativeTransform(Transform);
		PreviewActor->GetStaticMeshComponent()->UpdateBounds();

		// Center the mesh at the world origin then offset to put it on top of the plane
		const float BoundsZOffset = GetBoundsZOffset(PreviewActor->GetStaticMeshComponent()->Bounds);
		Transform.SetLocation(-PreviewActor->GetStaticMeshComponent()->Bounds.Origin + FVector(0, 0, BoundsZOffset));

		PreviewActor->GetStaticMeshComponent()->SetRelativeTransform(Transform);
	}

	PreviewActor->GetStaticMeshComponent()->SetMaterial(0, InMaterial);
	PreviewActor->GetStaticMeshComponent()->RecreateRenderState_Concurrent();
}

void FVolumeTextureThumbnailScene::GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const
{
	check(PreviewActor);
	check(PreviewActor->GetStaticMeshComponent());
	check(PreviewActor->GetStaticMeshComponent()->GetMaterial(0));

	// Fit the mesh in the view using the following formula
	// tan(HalfFOV) = Width/TargetCameraDistance
	const float HalfFOVRadians = FMath::DegreesToRadians<float>(InFOVDegrees) * 0.5f;
	// Add extra size to view slightly outside of the bounds to compensate for perspective
	const float BoundsMultiplier = 1.15f;
	const float HalfMeshSize = PreviewActor->GetStaticMeshComponent()->Bounds.SphereRadius * BoundsMultiplier;
	const float BoundsZOffset = GetBoundsZOffset(PreviewActor->GetStaticMeshComponent()->Bounds);
	const float TargetDistance = HalfMeshSize / FMath::Tan(HalfFOVRadians);

	// Since we're using USceneThumbnailInfoWithPrimitive in SetMaterialInterface, we should use it here instead of USceneThumbnailInfoWithPrimitive for consistency.
	USceneThumbnailInfoWithPrimitive* ThumbnailInfo = Cast<USceneThumbnailInfoWithPrimitive>(PreviewActor->GetStaticMeshComponent()->GetMaterial(0)->ThumbnailInfo);
	if ( ThumbnailInfo )
	{
		if ( TargetDistance + ThumbnailInfo->OrbitZoom < 0 )
		{
			ThumbnailInfo->OrbitZoom = -TargetDistance;
		}
	}
	else
	{
		ThumbnailInfo = USceneThumbnailInfoWithPrimitive::StaticClass()->GetDefaultObject<USceneThumbnailInfoWithPrimitive>();
	}

	OutOrigin = FVector(0, 0, -BoundsZOffset);
	OutOrbitPitch = ThumbnailInfo->OrbitPitch;
	OutOrbitYaw = ThumbnailInfo->OrbitYaw;
	OutOrbitZoom = TargetDistance + ThumbnailInfo->OrbitZoom;
}

UVolumeTextureThumbnailRenderer::UVolumeTextureThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ThumbnailScene(nullptr)
{
}

void UVolumeTextureThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UVolumeTexture* VolumeTexture = Cast<UVolumeTexture>(Object);
	if (VolumeTexture != nullptr)
	{
		if (!ThumbnailScene)
		{
			ThumbnailScene = new FVolumeTextureThumbnailScene();
		}

		if (!MaterialInstance)
		{
			UMaterial* BaseMaterial = LoadObject<UMaterial>(nullptr, TEXT("/Engine/EngineMaterials/VolumeTextureThumbnailMaterial.VolumeTextureThumbnailMaterial"), nullptr, LOAD_None, nullptr);
			if (BaseMaterial)
			{
				MaterialInstance = NewObject<UMaterialInstanceConstant>(GetTransientPackage());
				MaterialInstance->SetParentEditorOnly(BaseMaterial);
			}
		}

		if (MaterialInstance)
		{
			MaterialInstance->SetTextureParameterValueEditorOnly(FName("PreviewVolume"), VolumeTexture);
			MaterialInstance->PostEditChange();

			ThumbnailScene->SetMaterialInterface(MaterialInstance);
			FSceneViewFamilyContext ViewFamily( FSceneViewFamily::ConstructionValues( RenderTarget, ThumbnailScene->GetScene(), FEngineShowFlags(ESFIM_Game) )
				.SetTime(UThumbnailRenderer::GetTime())
				.SetAdditionalViewFamily(bAdditionalViewFamily));

			ViewFamily.EngineShowFlags.DisableAdvancedFeatures();
			ViewFamily.EngineShowFlags.SetSeparateTranslucency(true);
			ViewFamily.EngineShowFlags.MotionBlur = 0;
			ViewFamily.EngineShowFlags.AntiAliasing = 0;

			RenderViewFamily(Canvas, &ViewFamily, ThumbnailScene->CreateView(&ViewFamily, X, Y, Width, Height));
		}

		ThumbnailScene->SetMaterialInterface(nullptr);
	}
}

void UVolumeTextureThumbnailRenderer::BeginDestroy()
{
	if ( ThumbnailScene != nullptr )
	{
		delete ThumbnailScene;
		ThumbnailScene = nullptr;
	}

	Super::BeginDestroy();
}
