// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailRendering/WorldThumbnailInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldThumbnailInfo)

UWorldThumbnailInfo::UWorldThumbnailInfo(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	CameraMode = ECameraProjectionMode::Perspective;
	OrthoDirection = EOrthoThumbnailDirection::Top;
}
