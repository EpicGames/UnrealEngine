// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailRendering/WorldThumbnailRenderer.h"
#include "EngineDefines.h"
#include "Misc/App.h"
#include "ShowFlags.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "SceneView.h"
#include "SceneViewExtension.h"
#include "SceneViewExtensionContext.h"
#include "ThumbnailRendering/WorldThumbnailInfo.h"
#include "Engine/LevelBounds.h"
#include "ContentStreaming.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldThumbnailRenderer)

UWorldThumbnailRenderer::UWorldThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	GlobalOrbitPitchOffset = 0.f;
	GlobalOrbitYawOffset = 0.f;
	bUseUnlitScene = false;
	bAllowWorldThumbnails = false;
}

bool UWorldThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	if (bAllowWorldThumbnails)
	{
		UWorld* World = Cast<UWorld>(Object);
		if (World && World->PersistentLevel)
		{
			// If this is a world, only render the current persistent editor world. Other worlds don't have an initialized scene to render.
			return World->bIsWorldInitialized;
		}
	}

	return false;
}

void UWorldThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UWorld* World = Cast<UWorld>(Object);
	if (World != nullptr && World->Scene)
	{
		FSceneViewFamilyContext ViewFamily( FSceneViewFamily::ConstructionValues( RenderTarget, World->Scene, FEngineShowFlags(ESFIM_All0) )
			.SetTime(UThumbnailRenderer::GetTime())
			.SetAdditionalViewFamily(bAdditionalViewFamily));

		ViewFamily.bThumbnailRendering = true;

		ViewFamily.EngineShowFlags.SetDiffuse(true);
		ViewFamily.EngineShowFlags.SetSkeletalMeshes(true);
		ViewFamily.EngineShowFlags.SetTranslucency(true);
		ViewFamily.EngineShowFlags.SetBillboardSprites(true);
		ViewFamily.EngineShowFlags.SetLOD(true);
		ViewFamily.EngineShowFlags.SetMaterials(true);
		ViewFamily.EngineShowFlags.SetStaticMeshes(true);
		ViewFamily.EngineShowFlags.SetNaniteMeshes(true);
		ViewFamily.EngineShowFlags.SetLandscape(true);
		ViewFamily.EngineShowFlags.SetGame(true);
		ViewFamily.EngineShowFlags.SetBSP(true);
		ViewFamily.EngineShowFlags.SetRendering(true);
		ViewFamily.EngineShowFlags.SetPaper2DSprites(true);
		ViewFamily.EngineShowFlags.SetDistanceCulledPrimitives(true);

		if ( !bUseUnlitScene )
		{
			ViewFamily.EngineShowFlags.SetSpecular(true);
			ViewFamily.EngineShowFlags.SetLighting(true);
			ViewFamily.EngineShowFlags.SetDirectLighting(true);
			ViewFamily.EngineShowFlags.SetIndirectLightingCache(true);
			ViewFamily.EngineShowFlags.SetDeferredLighting(true);
			ViewFamily.EngineShowFlags.SetDirectionalLights(true);
			ViewFamily.EngineShowFlags.SetGlobalIllumination(true);
			ViewFamily.EngineShowFlags.SetPointLights(true);
			ViewFamily.EngineShowFlags.SetSpotLights(true);
			ViewFamily.EngineShowFlags.SetSkyLighting(true);
			ViewFamily.EngineShowFlags.SetReflectionEnvironment(true);
		}

		RenderViewFamily(Canvas, &ViewFamily, CreateView(World, &ViewFamily, X, Y, Width, Height));
	}
}

FSceneView* UWorldThumbnailRenderer::CreateView(UWorld* World, FSceneViewFamily* ViewFamily, int32 X, int32 Y, uint32 SizeX, uint32 SizeY) const 
{
	check(ViewFamily);
	check(World);
	check(World->PersistentLevel);

	FIntRect ViewRect(
		FMath::Max<int32>(X, 0),
		FMath::Max<int32>(Y, 0),
		FMath::Max<int32>(X + SizeX, 0),
		FMath::Max<int32>(Y + SizeY, 0));

	if (ViewRect.Area() <= 0)
	{
		return nullptr;
	}

	FBox WorldBox(ForceInit);
	TArray<ULevel*> LevelsToRender = World->GetLevels();
	for ( ULevel* Level : LevelsToRender )
	{
		if (Level && Level->bIsVisible)
		{
			ALevelBounds* LevelBounds = Level->LevelBoundsActor.Get();
				
			if (!LevelBounds && !Level->HasAllFlags(RF_Transient))
			{
				// Ensure a Level Bounds Actor exists for future renders
				FActorSpawnParameters SpawnParameters;
				SpawnParameters.ObjectFlags |= RF_Transient;
				SpawnParameters.OverrideLevel = Level;
				LevelBounds = World->SpawnActor<ALevelBounds>(SpawnParameters);
				LevelBounds->UpdateLevelBoundsImmediately();
				Level->LevelBoundsActor = LevelBounds;
			}

			if (LevelBounds)
			{
				if(!LevelBounds->IsUsingDefaultBounds())
				{
					WorldBox += LevelBounds->GetComponentsBoundingBox();
				}
			}
			else
			{
				FBox LevelBox = ALevelBounds::CalculateLevelBounds(Level);
				if (LevelBox.IsValid)
				{
					WorldBox += LevelBox;
				}
			}
		}
	}

	UWorldThumbnailInfo* ThumbnailInfo = Cast<UWorldThumbnailInfo>(World->ThumbnailInfo);
	if (!ThumbnailInfo)
	{
		ThumbnailInfo = UWorldThumbnailInfo::StaticClass()->GetDefaultObject<UWorldThumbnailInfo>();
	}

	const FVector Origin = WorldBox.GetCenter();
	FMatrix ViewRotationMatrix = FMatrix::Identity;
	FMatrix ProjectionMatrix = FMatrix::Identity;
	float FOVScreenSize = 0; // Screen size taking FOV into account
	if (ThumbnailInfo->CameraMode == ECameraProjectionMode::Perspective)
	{
		const float FOVDegrees = 30.f;
		const float HalfFOVRadians = FMath::DegreesToRadians<float>(FOVDegrees) * 0.5f;
		const float WorldRadius = WorldBox.GetSize().Size() / 2.f;
		float TargetDistance = WorldRadius / FMath::Tan(HalfFOVRadians);

		if (TargetDistance + ThumbnailInfo->OrbitZoom < 0)
		{
			ThumbnailInfo->OrbitZoom = -TargetDistance;
		}

		float OrbitPitch = GlobalOrbitPitchOffset + ThumbnailInfo->OrbitPitch;
		float OrbitYaw = GlobalOrbitYawOffset + ThumbnailInfo->OrbitYaw;
		float OrbitZoom = TargetDistance + ThumbnailInfo->OrbitZoom;

		// Ensure a minimum camera distance to prevent problems with really small objects
		const float MinCameraDistance = 48;
		OrbitZoom = FMath::Max<float>(MinCameraDistance, OrbitZoom);

		const FRotator RotationOffsetToViewCenter(0.f, 90.f, 0.f);
		ViewRotationMatrix = FRotationMatrix(FRotator(0, OrbitYaw, 0)) *
			FRotationMatrix(FRotator(0, 0, OrbitPitch)) *
			FTranslationMatrix(FVector(0, OrbitZoom, 0)) *
			FInverseRotationMatrix(RotationOffsetToViewCenter);

		ViewRotationMatrix = ViewRotationMatrix * FMatrix(
			FPlane(0, 0, 1, 0),
			FPlane(1, 0, 0, 0),
			FPlane(0, 1, 0, 0),
			FPlane(0, 0, 0, 1));

		const float NearPlane = 1.0f;
		ProjectionMatrix = FReversedZPerspectiveMatrix(
			HalfFOVRadians,
			1.0f,
			1.0f,
			NearPlane
			);

		FOVScreenSize = SizeX / FMath::Tan(FOVDegrees);
	}
	else if (ThumbnailInfo->CameraMode == ECameraProjectionMode::Orthographic)
	{
		FVector2D WorldSizeMin2D;
		FVector2D WorldSizeMax2D;
		switch (ThumbnailInfo->OrthoDirection)
		{
			case EOrthoThumbnailDirection::Top:
				ViewRotationMatrix = FMatrix(
					FPlane(1, 0, 0, 0),
					FPlane(0, -1, 0, 0),
					FPlane(0, 0, -1, 0),
					FPlane(0, 0, Origin.Z, 1));
				WorldSizeMin2D = FVector2D(WorldBox.Min.X,WorldBox.Min.Y);
				WorldSizeMax2D = FVector2D(WorldBox.Max.X,WorldBox.Max.Y);
				break;
			case EOrthoThumbnailDirection::Bottom:
				ViewRotationMatrix = FMatrix(
					FPlane(1, 0, 0, 0),
					FPlane(0, -1, 0, 0),
					FPlane(0, 0, 1, 0),
					FPlane(0, 0, Origin.Z, 1));
				WorldSizeMin2D = FVector2D(WorldBox.Min.X, WorldBox.Min.Y);
				WorldSizeMax2D = FVector2D(WorldBox.Max.X, WorldBox.Max.Y);
				break;
			case EOrthoThumbnailDirection::Front:
				ViewRotationMatrix = FMatrix(
					FPlane(1, 0, 0, 0),
					FPlane(0, 0, -1, 0),
					FPlane(0, 1, 0, 0),
					FPlane(0, 0, Origin.Y, 1));
				WorldSizeMin2D = FVector2D(WorldBox.Min.X, WorldBox.Min.Z);
				WorldSizeMax2D = FVector2D(WorldBox.Max.X, WorldBox.Max.Z);
				break;
			case EOrthoThumbnailDirection::Back:
				ViewRotationMatrix = FMatrix(
					FPlane(-1, 0, 0, 0),
					FPlane(0, 0, 1, 0),
					FPlane(0, 1, 0, 0),
					FPlane(0, 0, Origin.Y, 1));
				WorldSizeMin2D = FVector2D(WorldBox.Min.X, WorldBox.Min.Z);
				WorldSizeMax2D = FVector2D(WorldBox.Max.X, WorldBox.Max.Z);
				break;
			case EOrthoThumbnailDirection::Left:
				ViewRotationMatrix = FMatrix(
					FPlane(0, 0, -1, 0),
					FPlane(-1, 0, 0, 0),
					FPlane(0, 1, 0, 0),
					FPlane(0, 0, Origin.X, 1));
				WorldSizeMin2D = FVector2D(WorldBox.Min.Y, WorldBox.Min.Z);
				WorldSizeMax2D = FVector2D(WorldBox.Max.Y, WorldBox.Max.Z);
				break;
			case EOrthoThumbnailDirection::Right:
				ViewRotationMatrix = FMatrix(
					FPlane(0, 0, 1, 0),
					FPlane(1, 0, 0, 0),
					FPlane(0, 1, 0, 0),
					FPlane(0, 0, Origin.X, 1));
				WorldSizeMin2D = FVector2D(WorldBox.Min.Y, WorldBox.Min.Z);
				WorldSizeMax2D = FVector2D(WorldBox.Max.Y, WorldBox.Max.Z);
				break;
			default:
				// Unknown OrthoDirection
				ensureMsgf(false, TEXT("Unknown thumbnail OrthoDirection"));
				break;
		}

		FVector2D WorldSize2D = (WorldSizeMax2D - WorldSizeMin2D);
		WorldSize2D.X = FMath::Abs(WorldSize2D.X);
		WorldSize2D.Y = FMath::Abs(WorldSize2D.Y);
		const bool bUseXAxis = (WorldSize2D.X / WorldSize2D.Y) > 1.f;
		const FVector2D::FReal WorldAxisSize = bUseXAxis ? WorldSize2D.X : WorldSize2D.Y;
		const uint32 ViewportAxisSize = bUseXAxis ? SizeX : SizeY;
		const FVector2D::FReal OrthoZoom = WorldAxisSize / ViewportAxisSize / 2.f;
		const float OrthoWidth = FMath::Max(1.f, SizeX * OrthoZoom);
		const float OrthoHeight = FMath::Max(1.f, SizeY * OrthoZoom);

		const FMatrix::FReal ZOffset = UE_OLD_HALF_WORLD_MAX;
		ProjectionMatrix = FReversedZOrthoMatrix(
			OrthoWidth,
			OrthoHeight,
			0.5f / ZOffset,
			ZOffset
			);

		FOVScreenSize = SizeX;
	}
	else
	{
		// Unknown CameraMode
		ensureMsgf(false, TEXT("Unknown thumbnail CameraMode"));
	}

	ensureMsgf(!ViewRotationMatrix.ContainsNaN(), TEXT("Nans found on ViewRotationMatrix"));
	if (ViewRotationMatrix.ContainsNaN())
	{
		ViewRotationMatrix.SetIdentity();
	}

	ensureMsgf(!ProjectionMatrix.ContainsNaN(), TEXT("Nans found on ProjectionMatrix"));
	if (ProjectionMatrix.ContainsNaN())
	{
		ProjectionMatrix.SetIdentity();
	}

	FSceneViewInitOptions ViewInitOptions;
	ViewInitOptions.ViewFamily = ViewFamily;
	ViewInitOptions.SetViewRectangle(ViewRect);
	ViewInitOptions.BackgroundColor = FLinearColor::Black;
	ViewInitOptions.ViewOrigin = Origin;
	ViewInitOptions.ViewRotationMatrix = ViewRotationMatrix;
	ViewInitOptions.ProjectionMatrix = ProjectionMatrix;

	FSceneView* NewView = new FSceneView(ViewInitOptions);

	ViewFamily->Views.Add(NewView);

	// Tell the texture streaming system about this thumbnail view, so the textures will stream in as needed
	// NOTE: Sizes may not actually be in screen space depending on how the thumbnail ends up stretched by the UI.  Not a big deal though.
	// NOTE: Textures still take a little time to stream if the view has not been re-rendered recently, so they may briefly appear blurry while mips are prepared
	// NOTE: Content Browser only renders thumbnails for loaded assets, and only when the mouse is over the panel. They'll be frozen in their last state while the mouse cursor is not over the panel.  This is for performance reasons
	IStreamingManager::Get().AddViewInformation(Origin, SizeX, FOVScreenSize);

	return NewView;
}
