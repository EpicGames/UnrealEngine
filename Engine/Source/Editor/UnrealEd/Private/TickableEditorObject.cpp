// Copyright Epic Games, Inc. All Rights Reserved.

#include "TickableEditorObject.h"
#include "Tickable.h"

FTickableObjectBase::FTickableStatics& FTickableEditorObject::GetStatics()
{
	static FTickableStatics Singleton;
	return Singleton;
}

FTickableObjectBase::FTickableStatics& FTickableCookObject::GetStatics()
{
	static FTickableStatics Singleton;
	return Singleton;
}

FTickableObjectBase* FTickableEditorObject::ObjectBeingTicked = nullptr;
FTickableObjectBase* FTickableCookObject::ObjectBeingTicked = nullptr;

