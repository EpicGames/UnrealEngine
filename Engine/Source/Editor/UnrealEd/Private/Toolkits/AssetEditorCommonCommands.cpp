// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/AssetEditorCommonCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "GenericPlatform/GenericApplication.h"
#include "InputCoreTypes.h"

#define LOCTEXT_NAMESPACE "AssetEditorCommonCommands"

void FAssetEditorCommonCommands::RegisterCommands()
{
	UI_COMMAND( SaveAsset, "Save", "Saves this asset", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::S) );
	UI_COMMAND( SaveAssetAs, "Save As...", "Saves this asset under a different name", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Alt, EKeys::S) );
	UI_COMMAND( ReimportAsset, "Reimport", "Reimports the asset being edited", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND( SwitchToStandaloneEditor, "Switch to Standalone Editor", "Closes the level-centric asset editor and reopens it in 'standalone' mode", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( SwitchToWorldCentricEditor, "Switch to World-Centric Editor", "Closes the standalone asset editor and reopens it in 'world-centric' mode, docked within the level editor that it was originally opened in.", EUserInterfaceActionType::Button, FInputChord() );
}

#undef LOCTEXT_NAMESPACE
