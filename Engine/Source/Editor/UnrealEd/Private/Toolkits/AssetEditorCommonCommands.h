// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FUICommandInfo;

/** Asset editor common commands */
class FAssetEditorCommonCommands : public TCommands< FAssetEditorCommonCommands >
{

public:

	FAssetEditorCommonCommands()
		: TCommands< FAssetEditorCommonCommands >( TEXT("AssetEditor"), NSLOCTEXT("Contexts", "AssetEditor", "Asset Editor"), TEXT("EditorViewport"), FAppStyle::GetAppStyleSetName() )
	{
	}	

	virtual void RegisterCommands() override;

	TSharedPtr< FUICommandInfo > SaveAsset;
	TSharedPtr< FUICommandInfo > SaveAssetAs;
	TSharedPtr< FUICommandInfo > ReimportAsset;
	TSharedPtr< FUICommandInfo > SwitchToStandaloneEditor;
	TSharedPtr< FUICommandInfo > SwitchToWorldCentricEditor;
};

