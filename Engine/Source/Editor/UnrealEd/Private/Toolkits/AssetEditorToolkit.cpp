// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/AssetEditorToolkit.h"

#include "AssetEditorModeManager.h"
#include "AssetToolsModule.h"
#include "CollectionManagerModule.h"
#include "CollectionManagerTypes.h"
#include "ContentBrowserModule.h"
#include "Editor.h"
#include "EditorReimportHandler.h"
#include "FileHelpers.h"
#include "IAssetTools.h"
#include "IAssetTypeActions.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "ICollectionSource.h"
#include "IContentBrowserSingleton.h"
#include "IDocumentation.h"
#include "StatusBarSubsystem.h"
#include "ToolMenus.h"
#include "WidgetDrawerConfig.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "GameFramework/Actor.h"
#include "Interfaces/Interface_AsyncCompilation.h"
#include "Logging/LogMacros.h"
#include "Misc/Attribute.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "Settings/EditorStyleSettings.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateIconFinder.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Textures/SlateIcon.h"
#include "Toolkits/AssetEditorCommonCommands.h"
#include "Toolkits/AssetEditorToolkitMenuContext.h"
#include "Toolkits/GlobalEditorCommonCommands.h"
#include "Toolkits/SStandaloneAssetEditorToolkitHost.h"
#include "Toolkits/ToolkitManager.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"
#include "ViewportInteractions/ViewportInteractionsUtilities.h"
#include "ViewportInteractions/ViewportLegacyClickInteraction.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"

#define LOCTEXT_NAMESPACE "AssetEditorToolkit"

DEFINE_LOG_CATEGORY_STATIC(LogAssetEditorToolkit, Log, All);

TWeakPtr< IToolkitHost > FAssetEditorToolkit::PreviousWorldCentricToolkitHostForNewAssetEditor;
TSharedPtr<FExtensibilityManager> FAssetEditorToolkit::SharedMenuExtensibilityManager;
TSharedPtr<FExtensibilityManager> FAssetEditorToolkit::SharedToolBarExtensibilityManager;

const FName FAssetEditorToolkit::DefaultAssetEditorToolBarName("AssetEditor.DefaultToolBar");
const FName FAssetEditorToolkit::ReadOnlyMenuProfileName("AssetEditor.ReadOnlyMenuProfile");

FAssetEditorToolkit::FAssetEditorToolkit()
	: GCEditingObjects(*this)
	, bCheckDirtyOnAssetSave(false)
	, AssetEditorModeManager(nullptr)
	, bIsToolbarFocusable(false)
	, bIsToolbarUsingSmallIcons(false)
	, OpenMethod(EAssetOpenMethod::Edit)
{
	WorkspaceMenuCategory = FWorkspaceItem::NewGroup(LOCTEXT("WorkspaceMenu_BaseAssetEditor", "Asset Editor"));

	if (!FAssetEditorCommonCommands::IsRegistered())
	{
		// Ensure the asset editor common commands are registered in case UnrealEdMisc isn't used, and therefore not initializing the commands.
		FAssetEditorCommonCommands::Register();
	}
}

void FAssetEditorToolkit::InitAssetEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, const FName AppIdentifier, const TSharedRef<FTabManager::FLayout>& StandaloneDefaultLayout, const bool bCreateDefaultStandaloneMenu, const bool bCreateDefaultToolbar, UObject* ObjectToEdit, const bool bInIsToolbarFocusable, const bool bInUseSmallToolbarIcons, const TOptional<EAssetOpenMethod>& InOpenMethod )
{
	TArray< UObject* > ObjectsToEdit;
	ObjectsToEdit.Add( ObjectToEdit );

	InitAssetEditor( Mode, InitToolkitHost, AppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, ObjectsToEdit, bInIsToolbarFocusable, bInUseSmallToolbarIcons );
}

void FAssetEditorToolkit::InitAssetEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, const FName AppIdentifier, const TSharedRef<FTabManager::FLayout>& StandaloneDefaultLayout, const bool bCreateDefaultStandaloneMenu, const bool bCreateDefaultToolbar, const TArray<UObject*>& ObjectsToEdit, const bool bInIsToolbarFocusable, const bool bInUseSmallToolbarIcons, const TOptional<EAssetOpenMethod>& InOpenMethod )
{
	// Must not already be editing an object
	check( ObjectsToEdit.Num() > 0 );
	check( EditingObjects.Num() == 0 );

	bIsToolbarFocusable = bInIsToolbarFocusable;
	bIsToolbarUsingSmallIcons = bInUseSmallToolbarIcons;

	// cache reference to ToolkitManager; also ensure it was initialized.
	FToolkitManager& ToolkitManager = FToolkitManager::Get();

	EditingObjects.Append( ObjectsToEdit );

	// If the open method was manually overriden, use that
	if(InOpenMethod.IsSet())
	{
		OpenMethod = InOpenMethod.GetValue();
	}
	// Otherwise we check the Asset Editor Subsystem to see if there is a method this asset editor is being requested to open in
	else
	{
		TOptional<EAssetOpenMethod> CachedOpenMethod =  GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->GetAssetsBeingOpenedMethod(EditingObjects);

		if(CachedOpenMethod.IsSet())
		{
			OpenMethod = CachedOpenMethod.GetValue();
		}
	}

	// Store "previous" asset editing toolkit host, and clear it out
	PreviousWorldCentricToolkitHost = PreviousWorldCentricToolkitHostForNewAssetEditor;
	PreviousWorldCentricToolkitHostForNewAssetEditor.Reset();

	ToolkitMode = Mode;

	TSharedPtr<SWindow> ParentWindow;

	TSharedPtr<SDockTab> NewMajorTab;

	TSharedPtr< SStandaloneAssetEditorToolkitHost > NewStandaloneHost;
	if( ToolkitMode == EToolkitMode::WorldCentric )		// @todo toolkit major: Do we need to remember this setting on a per-asset editor basis?  Probably.
	{
		// Keep track of the level editor we're attached to (if any)
		ToolkitHost = InitToolkitHost;
	}
	else if( ensure( ToolkitMode == EToolkitMode::Standalone ) )
	{
		// Open a standalone app to edit this asset.
		check( AppIdentifier != NAME_None );

		// Create the label and the link for the toolkit documentation.
		TAttribute<FText> Label = TAttribute<FText>( this, &FAssetEditorToolkit::GetToolkitName );
		TAttribute<FText> LabelSuffix = TAttribute<FText>(this, &FAssetEditorToolkit::GetTabSuffix);
		TAttribute<FText> ToolTipText = TAttribute<FText>( this, &FAssetEditorToolkit::GetToolkitToolTipText );
		FString DocLink = GetDocumentationLink();
		if ( !DocLink.StartsWith( "Shared/" ) )
		{
			DocLink = FString("Shared/") + DocLink;
		}

		// Create a new SlateToolkitHost
		NewMajorTab = SNew(SDockTab)
			.OnExtendContextMenu(SDockTab::FExtendContextMenu::CreateSP(this, &FAssetEditorToolkit::OnExtendTabContextMenu))
			.ContentPadding(0.0f)
			.TabRole(ETabRole::MajorTab)
			.ToolTip(IDocumentation::Get()->CreateToolTip(ToolTipText, nullptr, DocLink, GetToolkitFName().ToString()))
			.IconColor(this, &FAssetEditorToolkit::GetDefaultTabColor)
			.Label(Label)
			.LabelOverflowPolicy(FGlobalTabmanager::Get()->GetShouldUseMiddleEllipsisForDockTabLabel() ? ETextOverflowPolicy::MiddleEllipsis : ETextOverflowPolicy::Ellipsis)
			.LabelSuffix(LabelSuffix);
		const TAttribute<const FSlateBrush*> TabIcon = TAttribute<const FSlateBrush*>::CreateSP(this, &FAssetEditorToolkit::GetDefaultTabIcon);
		NewMajorTab->SetTabIcon(TabIcon);
		{
			static_assert(sizeof(EAssetEditorToolkitTabLocation) == sizeof(int32), "EAssetEditorToolkitTabLocation is the incorrect size");

			const UEditorStyleSettings* StyleSettings = GetDefault<UEditorStyleSettings>();

			FName PlaceholderId(TEXT("StandaloneToolkit"));
			TSharedPtr<FTabManager::FSearchPreference> SearchPreference = nullptr;
			if ( StyleSettings->AssetEditorOpenLocation == EAssetEditorOpenLocation::Default )
			{
				// Work out where we should create this asset editor
				EAssetEditorToolkitTabLocation SavedAssetEditorToolkitTabLocation = EAssetEditorToolkitTabLocation::Standalone;
				GConfig->GetInt(
					TEXT("AssetEditorToolkitTabLocation"),
					*ObjectsToEdit[0]->GetPathName(),
					reinterpret_cast<int32&>( SavedAssetEditorToolkitTabLocation ),
					GEditorPerProjectIni
					);

				PlaceholderId = ( SavedAssetEditorToolkitTabLocation == EAssetEditorToolkitTabLocation::Docked ) ? TEXT("DockedToolkit") : TEXT("StandaloneToolkit");
				SearchPreference = MakeShareable(new FTabManager::FLiveTabSearch());
			}
			else if ( StyleSettings->AssetEditorOpenLocation == EAssetEditorOpenLocation::NewWindow )
			{
				PlaceholderId = TEXT("StandaloneToolkit");
				SearchPreference = MakeShareable(new FTabManager::FRequireClosedTab());
			}
			else if ( StyleSettings->AssetEditorOpenLocation == EAssetEditorOpenLocation::MainWindow )
			{
				PlaceholderId = TEXT("DockedToolkit");
				SearchPreference = MakeShareable(new FTabManager::FLiveTabSearch(TEXT("LevelEditor")));
			}
			else if ( StyleSettings->AssetEditorOpenLocation == EAssetEditorOpenLocation::ContentBrowser )
			{
				PlaceholderId = TEXT("DockedToolkit");
				SearchPreference = MakeShareable(new FTabManager::FLiveTabSearch(TEXT("ContentBrowserTab1")));
			}
			else if ( StyleSettings->AssetEditorOpenLocation == EAssetEditorOpenLocation::LastDockedWindowOrNewWindow )
			{
				PlaceholderId = TEXT("StandaloneToolkit");
				SearchPreference = MakeShareable(new FTabManager::FLastMajorOrNomadTab(NAME_None));
			}
			else if ( StyleSettings->AssetEditorOpenLocation == EAssetEditorOpenLocation::LastDockedWindowOrMainWindow )
			{
				PlaceholderId = TEXT("StandaloneToolkit");
				SearchPreference = MakeShareable(new FTabManager::FLastMajorOrNomadTab(TEXT("LevelEditor")));
			}
			else if ( StyleSettings->AssetEditorOpenLocation == EAssetEditorOpenLocation::LastDockedWindowOrContentBrowser )
			{
				PlaceholderId = TEXT("StandaloneToolkit");
				SearchPreference = MakeShareable(new FTabManager::FLastMajorOrNomadTab(TEXT("ContentBrowserTab1")));
			}
			else
			{
				// Add more cases!
				check(false);
			}

			if (FName OverrideTabId = GetOverrideTabId(); !OverrideTabId.IsNone())
			{
				PlaceholderId = OverrideTabId;
			}

			FGlobalTabmanager::Get()->InsertNewDocumentTab(PlaceholderId, *SearchPreference, NewMajorTab.ToSharedRef());

			// Bring the window to front.  The tab manager will not do this for us to avoid intrusive stealing focus behavior
			// However, here the expectation is that opening an new asset editor is something that should steal focus so the user can see their asset
			TSharedPtr<SWindow> Window = NewMajorTab->GetParentWindow();
			if(Window.IsValid())
			{
				Window->BringToFront();
			}
		}

		const TSharedRef<FTabManager> NewTabManager = FGlobalTabmanager::Get()->NewTabManager( NewMajorTab.ToSharedRef() );		
		NewTabManager->SetOnPersistLayout(FTabManager::FOnPersistLayout::CreateRaw(this, &FAssetEditorToolkit::HandleTabManagerPersistLayout));
		NewTabManager->SetAllowWindowMenuBar(true);
		NewTabManager->SetReadOnly(OpenMethod == EAssetOpenMethod::View);

		this->TabManager = NewTabManager;

		TArray<TWeakObjectPtr<UObject>> ObjectsToEditWeak;
		EVisibility VisibilityWhileCompiling = GetVisibilityWhileAssetCompiling();

		ObjectsToEditWeak.Reserve(ObjectsToEdit.Num());
		for (UObject* Object : ObjectsToEdit)
		{
			ObjectsToEditWeak.Add(Object);
		}
		NewMajorTab->SetContent
		( 
			SAssignNew( NewStandaloneHost, SStandaloneAssetEditorToolkitHost, NewTabManager, AppIdentifier )
			.Visibility_Lambda([ObjectsToEditWeak,VisibilityWhileCompiling]()
				{
					for (const TWeakObjectPtr<UObject> Object : ObjectsToEditWeak)
					{
						if (const IInterface_AsyncCompilation* AsyncAsset = Cast<IInterface_AsyncCompilation>(Object.Get()))
						{
							if (AsyncAsset->IsCompiling())
							{
								return VisibilityWhileCompiling;
							}
						}
					}
					return EVisibility::Visible;
				})
			.OnRequestClose(this, &FAssetEditorToolkit::OnRequestClose, EAssetEditorCloseReason::AssetEditorHostClosed)
			.OnClose(this, &FAssetEditorToolkit::OnClose)
		);

		// Assign our toolkit host before we setup initial content.  (Important: We must cache this pointer here as SetupInitialContent
		// will callback into the toolkit host.)
		ToolkitHost = NewStandaloneHost;

		StandaloneHost = NewStandaloneHost;
	}

	check( ToolkitHost.IsValid() );
	ToolkitManager.RegisterNewToolkit( SharedThis( this ) );

	MapToolkitCommands();

	InitializeReadOnlyMenuProfiles();

	// Give a chance to customize tab manager and other UI before widgets are created
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->NotifyEditorOpeningPreWidgets(ObjectPtrDecay(EditingObjects), this);
	
	// Create menus
	if (ToolkitMode == EToolkitMode::Standalone)
	{
		AddMenuExtender(GetSharedMenuExtensibilityManager()->GetAllExtenders(ToolkitCommands, ObjectPtrDecay(EditingObjects)));

		TSharedRef<FTabManager::FLayout> LayoutToUse = FLayoutSaveRestore::LoadFromConfig(LayoutIni, StandaloneDefaultLayout);

		// Actually create the widget content
		if (NewStandaloneHost)
		{
			NewStandaloneHost->SetupInitialContent(LayoutToUse, NewMajorTab, bCreateDefaultStandaloneMenu);
		}	
	}
	
	// Create toolbars
	AddToolbarExtender(GetSharedToolBarExtensibilityManager()->GetAllExtenders(ToolkitCommands, ObjectPtrDecay(EditingObjects)));

	if (bCreateDefaultToolbar)
	{
		GenerateToolbar();
	}
	else
	{
		Toolbar = SNullWidget::NullWidget;
	}

	if (NewStandaloneHost)
	{
		NewStandaloneHost->SetToolbar(Toolbar);
	}

	// Create our mode manager and set it's toolkit host
	if (!EditorModeManager)
	{
		CreateEditorModeManager();
	}

	if (EditorModeManager)
	{
		EditorModeManager->SetToolkitHost(ToolkitHost.Pin().ToSharedRef());
		
		if (UModeManagerInteractiveToolsContext* ToolsContext = EditorModeManager->GetInteractiveToolsContext())
		{
			ToolsContext->OnBuildViewportInteractions().AddRaw(this, &FAssetEditorToolkit::OnBuildViewportInteractions);
			ToolsContext->PostBuildViewportInteractions().AddRaw(this, &FAssetEditorToolkit::PostBuildViewportInteractions);
		}
	}
	
	PostInitAssetEditor();

	// NOTE: Currently, the AssetEditorSubsystem will keep a hard reference to our object as we're editing it
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->NotifyAssetsOpened( ObjectPtrDecay(EditingObjects), this );
}


FAssetEditorToolkit::~FAssetEditorToolkit()
{
	EditingObjects.Empty();

	// We're no longer editing this object, so let the editor know
	if (GEditor)
	{
		if (UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
		{
			AssetEditorSubsystem->NotifyEditorClosed(this);
		}
	}
	
	if (EditorModeManager)
	{
		if (UModeManagerInteractiveToolsContext* ToolsContext = EditorModeManager->GetInteractiveToolsContext())
		{
			ToolsContext->OnBuildViewportInteractions().RemoveAll(this);
			ToolsContext->PostBuildViewportInteractions().RemoveAll(this);
		}
	}

	EditorModeManager.Reset();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	AssetEditorModeManager = nullptr;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FAssetEditorToolkit::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	// Use the first child category of the local workspace root if there is one, otherwise use the root itself
	const auto& LocalCategories = InTabManager->GetLocalWorkspaceMenuRoot()->GetChildItems();
	AssetEditorTabsCategory = LocalCategories.Num() > 0 ? LocalCategories[0] : InTabManager->GetLocalWorkspaceMenuRoot();
}

void FAssetEditorToolkit::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	InTabManager->ClearLocalWorkspaceMenuCategories();
}

bool FAssetEditorToolkit::IsAssetEditor() const
{
	return true;
}

FText FAssetEditorToolkit::GetToolkitName() const
{
	const UObject* EditingObject = GetEditingObject();

	check (EditingObject != NULL);

	return GetLabelForObject(EditingObject);
}

FText FAssetEditorToolkit::GetTabSuffix() const
{
	bool bDirtyState = false;
	for (int32 x = 0; x < EditingObjects.Num(); ++x)
	{
		if (EditingObjects[x]->GetOutermost()->IsDirty())
		{
			bDirtyState = true;
			break;
		}
	}
	return bDirtyState ? LOCTEXT("TabSuffixAsterix", "*") : FText::GetEmpty();
}

FName FAssetEditorToolkit::GetEditingAssetTypeName() const
{
	if(EditingObjects.IsEmpty())
	{
		return NAME_None;
	}

	if(UClass* EditingClass = EditingObjects[0]->GetClass())
	{
		return EditingClass->GetFName();
	}

	return NAME_None;
}

FText FAssetEditorToolkit::GetToolkitToolTipText() const
{
	const UObject* EditingObject = GetEditingObject();

	check (EditingObject != NULL);

	return GetToolTipTextForObject(EditingObject);
}

FText FAssetEditorToolkit::GetLabelForObject(const UObject* InObject)
{
	FString NameString;
	if(const AActor* ObjectAsActor = Cast<AActor>(InObject))
	{
		NameString = ObjectAsActor->GetActorLabel();
	}
	else
	{
		NameString = InObject->GetName();
	}

	return FText::FromString(NameString);
}

FText FAssetEditorToolkit::GetToolTipTextForObject(const UObject* InObject)
{
	FString ToolTipString;
	if(const AActor* ObjectAsActor = Cast<AActor>(InObject))
	{
		ToolTipString += LOCTEXT("ToolTipActorLabel", "Actor").ToString();
		ToolTipString += TEXT(": ");
		ToolTipString += ObjectAsActor->GetActorLabel();
	}
	else
	{
		ToolTipString += LOCTEXT("ToolTipAssetLabel", "Asset").ToString();
		ToolTipString += TEXT(": ");
		ToolTipString += InObject->GetName();

		FCollectionManagerModule& CollectionManagerModule = FCollectionManagerModule::GetModule();

		TArray<TSharedPtr<ICollectionContainer>> CollectionContainers;
		CollectionManagerModule.Get().GetVisibleCollectionContainers(CollectionContainers);

		for (const TSharedPtr<ICollectionContainer>& CollectionContainer : CollectionContainers)
		{
			const FString CollectionNames = CollectionContainer->GetCollectionsStringForObject(FSoftObjectPath(InObject), ECollectionShareType::CST_All);
			if (!CollectionNames.IsEmpty())
			{
				ToolTipString += TEXT("\n");

				ToolTipString += CollectionContainer->GetCollectionSource()->GetTitle().ToString();
				ToolTipString += TEXT(": ");
				ToolTipString += CollectionNames;
			}
		}
	}

	return FText::FromString(ToolTipString);
}

FEditorModeTools& FAssetEditorToolkit::GetEditorModeManager() const
{
	if (IsWorldCentricAssetEditor() && IsHosted())
	{
		return GetToolkitHost()->GetEditorModeManager();
	}

	check(EditorModeManager.IsValid());
	return *EditorModeManager.Get();
}

const TArray< UObject* >* FAssetEditorToolkit::GetObjectsCurrentlyBeingEdited() const
{
	return &ObjectPtrDecay(EditingObjects);
}

FName FAssetEditorToolkit::GetEditorName() const
{
	return GetToolkitFName();
}

void FAssetEditorToolkit::FocusWindow(UObject* ObjectToFocusOn)
{
	BringToolkitToFront();
}

bool FAssetEditorToolkit::OnRequestClose(EAssetEditorCloseReason)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return OnRequestClose();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool FAssetEditorToolkit::CloseWindow()
{
	// We use AssetEditorHostClosed as the default close reason for legacy cases
	return CloseWindow(EAssetEditorCloseReason::AssetEditorHostClosed);
}

bool FAssetEditorToolkit::CloseWindow(EAssetEditorCloseReason InCloseReason)
{
	if (DoesSharedInstanceExist() && OnRequestClose(InCloseReason))
	{
		// We are closing, unbind OnRequestClose since we're past that point and we want to make sure we don't redo the request close process when closing the host tab
		if (TSharedPtr<SStandaloneAssetEditorToolkitHost> StandaloneHostPtr = StandaloneHost.Pin())
		{
			StandaloneHostPtr->UnbindEditorCloseRequestFromHostTab();
		}

		OnClose();

		// Close this toolkit
		FToolkitManager::Get().CloseToolkit( AsShared() );
	}
	return true;
}

void FAssetEditorToolkit::InvokeTab(const FTabId& TabId)
{
	GetTabManager()->TryInvokeTab(TabId);
}

TSharedPtr<class FTabManager> FAssetEditorToolkit::GetAssociatedTabManager()
{
	return TabManager;
}

double FAssetEditorToolkit::GetLastActivationTime()
{
	double MostRecentTime = 0.0;

	if (TabManager.IsValid())
	{
		TSharedPtr<SDockTab> OwnerTab = TabManager->GetOwnerTab();
		if (OwnerTab.IsValid())
		{
			MostRecentTime = OwnerTab->GetLastActivationTime();
		}
	}

	return MostRecentTime;
}

TSharedPtr< IToolkitHost > FAssetEditorToolkit::GetPreviousWorldCentricToolkitHost()
{
	return PreviousWorldCentricToolkitHost.Pin();
}


void FAssetEditorToolkit::SetPreviousWorldCentricToolkitHostForNewAssetEditor( TSharedRef< IToolkitHost > ToolkitHost )
{
	PreviousWorldCentricToolkitHostForNewAssetEditor = ToolkitHost;
}


UObject* FAssetEditorToolkit::GetEditingObject() const
{
	check( EditingObjects.Num() == 1 );
	return EditingObjects[ 0 ];
}


const TArray< UObject* >& FAssetEditorToolkit::GetEditingObjects() const
{
	check( EditingObjects.Num() > 0 );
	return ObjectPtrDecay(EditingObjects);
}

TArray<TObjectPtr<UObject>>& FAssetEditorToolkit::GetEditingObjectPtrs() 
{
	check(EditingObjects.Num() > 0);
	return EditingObjects;
}

void FAssetEditorToolkit::GetSaveableObjects(TArray<UObject*>& OutObjects) const
{
	for (const TObjectPtr<UObject>& Object : EditingObjects)
	{
		// If we are editing a subobject of asset (e.g., a level script blueprint which is contained in a map asset), still provide the
		// option to work with it but treat save operations/etc... as working on the top level asset itself
		for (UObject* TestObject = Object; TestObject != nullptr; TestObject = TestObject->GetOuter())
		{
			if (TestObject->IsAsset())
			{
				OutObjects.AddUnique(TestObject);
				break;
			}
		}
	}
}


void FAssetEditorToolkit::AddEditingObject(UObject* Object)
{
	// Don't allow adding the same object twice (or notify asset opened twice)
	if(EditingObjects.Contains(Object))
	{
		return;
	}
	
	EditingObjects.Add(Object);
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->NotifyAssetOpened( Object, this );
}


void FAssetEditorToolkit::RemoveEditingObject(UObject* Object)
{
	EditingObjects.Remove(Object);
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->NotifyAssetClosed( Object, this );
}

bool FAssetEditorToolkit::CanSaveAsset_Internal() const
{
	if(GetOpenMethod() != EAssetOpenMethod::Edit)
	{
		return false;
	}

	return CanSaveAsset();
}

bool FAssetEditorToolkit::IsSaveAssetVisible() const
{
	return true;
}

void FAssetEditorToolkit::SaveAsset_Execute()
{
	if (EditingObjects.Num() == 0)
	{
		return;
	}

	TArray<UObject*> ObjectsToSave;
	GetSaveableObjects(ObjectsToSave);

	if (ObjectsToSave.Num() == 0)
	{
		return;
	}

	TArray<UObject*> SavedObjects;
	SavedObjects.Reserve(ObjectsToSave.Num());

	TArray<UPackage*> PackagesToSave;

	for (UObject* Object : ObjectsToSave)
	{
		if ((Object == nullptr) || !Object->IsAsset())
		{
			// Log an invalid object but don't try to save it
			UE_LOGF(LogAssetEditorToolkit, Log, "Invalid object to save: %ls", (Object != nullptr) ? *Object->GetFullName() : TEXT("Null Object"));
		}
		else
		{
			PackagesToSave.Add(Object->GetOutermost());
			SavedObjects.Add(Object);
		}
	}

	FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, bCheckDirtyOnAssetSave, /*bPromptToSave=*/ false);

	OnAssetsSaved(SavedObjects);
}

bool FAssetEditorToolkit::CanSaveAssetAs_Internal() const
{
	if(GetOpenMethod() != EAssetOpenMethod::Edit)
	{
		return false;
	}

	return CanSaveAssetAs();
}

bool FAssetEditorToolkit::IsSaveAssetAsVisible() const
{
	return IsActuallyAnAsset();
}

void FAssetEditorToolkit::SaveAssetAs_Execute()
{
	if (EditingObjects.Num() == 0)
	{
		return;
	}

	TSharedPtr<IToolkitHost> MyToolkitHost = ToolkitHost.Pin();

	if (!MyToolkitHost.IsValid())
	{
		return;
	}

	// get collection of objects to save
	TArray<UObject*> ObjectsToSave;
	GetSaveableObjects(ObjectsToSave);

	if (ObjectsToSave.Num() == 0)
	{
		return;
	}

	// save assets under new name
	TArray<UObject*> SavedObjects;
	FEditorFileUtils::SaveAssetsAs(ObjectsToSave, SavedObjects);

	if (SavedObjects.Num() == 0)
	{
		return;
	}

	// close existing asset editors for resaved assets
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();

	/* @todo editor: Persona does not behave well when closing specific objects
	for (int32 Index = 0; Index < ObjectsToSave.Num(); ++Index)
	{
		if ((SavedObjects[Index] != ObjectsToSave[Index]) && (SavedObjects[Index] != nullptr))
		{
			AssetEditorSubsystem->CloseAllEditorsForAsset(ObjectsToSave[Index]);
		}
	}

	// reopen asset editor
	AssetEditorSubsystem->OpenEditorForAssets(TArrayBuilder<UObject*>().Add(SavedObjects[0]), ToolkitMode, MyToolkitHost.ToSharedRef());
	*/
	// hack
	TArray<UObject*> ObjectsToReopen;
	for (auto Object : EditingObjects)
	{
		if (Object->IsAsset() && !ObjectsToSave.Contains(Object))
		{
			ObjectsToReopen.Add(Object);
		}
	}
	for (auto Object : SavedObjects)
	{
		if (ShouldReopenEditorForSavedAsset(Object))
		{
			ObjectsToReopen.AddUnique(Object);
		}
	}
	for (auto Object : EditingObjects)
	{
		AssetEditorSubsystem->CloseAllEditorsForAsset(Object);
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->NotifyAssetClosed(Object, this);
	}
	AssetEditorSubsystem->OpenEditorForAssets_Advanced(ObjectsToReopen, ToolkitMode, MyToolkitHost.ToSharedRef());
	// end hack

	OnAssetsSavedAs(SavedObjects);
}


const FSlateBrush* FAssetEditorToolkit::GetDefaultTabIcon() const
{
	if (EditingObjects.Num() == 0)
	{
		return nullptr;
	}

	const FSlateBrush* IconBrush = nullptr;

	for (UObject* Object : EditingObjects)
	{
		if (Object)
		{
			UClass* IconClass = Object->GetClass();

			if (IconClass->IsChildOf<UBlueprint>())
			{
				UBlueprint* Blueprint = Cast<UBlueprint>(Object);
				IconClass = Blueprint->GeneratedClass;
			}

			// Find the first object that has a valid brush
			const FSlateBrush* ThisAssetBrush = FSlateIconFinder::FindIconBrushForClass(IconClass);
			if (ThisAssetBrush != nullptr)
			{
				IconBrush = ThisAssetBrush;
				break;
			}
		}
	}

	if (!IconBrush)
	{
		IconBrush = FAppStyle::GetBrush(TEXT("ClassIcon.Default"));;
	}

	return IconBrush;
}

FLinearColor FAssetEditorToolkit::GetDefaultTabColor() const
{
	FLinearColor TabColor = FLinearColor::White;
	if (EditingObjects.Num() == 0 || !GetDefault<UEditorStyleSettings>()->bEnableColorizedEditorTabs)
	{
		return TabColor;
	}

	FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();
	IAssetTools& AssetTools = AssetToolsModule.Get();
	for (auto ObjectIt = EditingObjects.CreateConstIterator(); ObjectIt; ++ObjectIt)
	{
		TWeakPtr<IAssetTypeActions> AssetTypeActions = AssetTools.GetAssetTypeActionsForClass((*ObjectIt)->GetClass());
		if (AssetTypeActions.IsValid())
		{
			const FLinearColor ThisAssetColor = AssetTypeActions.Pin()->GetTypeColor();
			if (ThisAssetColor != FLinearColor::Transparent)
			{
				return ThisAssetColor;
			}
		}
	}

	return TabColor;
}

void FAssetEditorToolkit::CreateEditorModeManager()
{
	EditorModeManager = MakeShared<FEditorModeTools>();
}

void FAssetEditorToolkit::MapToolkitCommands()
{
	ToolkitCommands->MapAction(
		FAssetEditorCommonCommands::Get().SaveAsset,
		FExecuteAction::CreateSP(this, &FAssetEditorToolkit::SaveAsset_Execute),
		FCanExecuteAction::CreateSP(this, &FAssetEditorToolkit::CanSaveAsset_Internal),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &FAssetEditorToolkit::IsSaveAssetVisible));

	ToolkitCommands->MapAction(
		FAssetEditorCommonCommands::Get().SaveAssetAs,
		FExecuteAction::CreateSP(this, &FAssetEditorToolkit::SaveAssetAs_Execute),
		FCanExecuteAction::CreateSP(this, &FAssetEditorToolkit::CanSaveAssetAs_Internal),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &FAssetEditorToolkit::IsSaveAssetAsVisible));

	ToolkitCommands->MapAction(
		FGlobalEditorCommonCommands::Get().FindInContentBrowser,
		FExecuteAction::CreateSP(this, &FAssetEditorToolkit::FindInContentBrowser_Execute),
		FCanExecuteAction::CreateSP(this, &FAssetEditorToolkit::CanFindInContentBrowser),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &FAssetEditorToolkit::IsFindInContentBrowserButtonVisible));

	ToolkitCommands->MapAction(
		FGlobalEditorCommonCommands::Get().OpenDocumentation,
		FExecuteAction::CreateSP(this, &FAssetEditorToolkit::BrowseDocumentation_Execute));

	ToolkitCommands->MapAction(
		FAssetEditorCommonCommands::Get().ReimportAsset,
		FExecuteAction::CreateSP(this, &FAssetEditorToolkit::Reimport_Execute),
		FCanExecuteAction::CreateSP(this, &FAssetEditorToolkit::CanReimport_Internal));

	FGlobalEditorCommonCommands::MapActions(ToolkitCommands);

	if (IsWorldCentricAssetEditor())
	{
		ToolkitCommands->MapAction(
			FAssetEditorCommonCommands::Get().SwitchToStandaloneEditor,
			FExecuteAction::CreateStatic(&FAssetEditorToolkit::SwitchToStandaloneEditor_Execute, TWeakPtr< FAssetEditorToolkit >(AsShared())));
	}
	else
	{
		if (GetPreviousWorldCentricToolkitHost().IsValid())
		{
			ToolkitCommands->MapAction(
				FAssetEditorCommonCommands::Get().SwitchToWorldCentricEditor,
				FExecuteAction::CreateStatic(&FAssetEditorToolkit::SwitchToWorldCentricEditor_Execute, TWeakPtr< FAssetEditorToolkit >(AsShared())));
		}
	}
}

void FAssetEditorToolkit::RestoreFromLayout(const TSharedRef<FTabManager::FLayout>& NewLayout, bool bLoadUserLayout)
{
	TSharedPtr< class SStandaloneAssetEditorToolkitHost > HostWidget = StandaloneHost.Pin();
	if (HostWidget.IsValid())
	{
		// Save the old layout
		FLayoutSaveRestore::SaveToConfig(LayoutIni, TabManager->PersistLayout());

		TSharedRef<FTabManager::FLayout> UserConfiguredNewLayout = NewLayout;
		if (bLoadUserLayout)
		{
			// Load the potentially previously saved new layout
			UserConfiguredNewLayout = FLayoutSaveRestore::LoadFromConfig(LayoutIni, NewLayout);
		}

		for (TSharedPtr<FLayoutExtender> LayoutExtender : LayoutExtenders)
		{
			NewLayout->ProcessExtensions(*LayoutExtender);
		}

		// Apply the new layout
		HostWidget->RestoreFromLayout(UserConfiguredNewLayout);
	}
}

void FAssetEditorToolkit::OnBuildViewportInteractions(UViewportInteractionsBehaviorSource* Source)
{
	UE::Editor::ViewportInteractions::AddDefaultCameraMovementInteractions(Source);
	
	Source->AddInteraction<UViewportLegacyClickInteraction>();
}

FAssetEditorModeManager* FAssetEditorToolkit::GetAssetEditorModeManager() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return AssetEditorModeManager;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FAssetEditorToolkit::SetAssetEditorModeManager(FAssetEditorModeManager* InModeManager)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	AssetEditorModeManager = InModeManager;
	if (AssetEditorModeManager && !AssetEditorModeManager->DoesSharedInstanceExist())
	{
		EditorModeManager = MakeShareable(AssetEditorModeManager);
	}
	else
	{
		EditorModeManager = AssetEditorModeManager->AsShared();
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FReply FAssetEditorToolkit::OnViewportDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	return FReply::Unhandled();
}

void FAssetEditorToolkit::OnExtendTabContextMenu(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.BeginSection("DockingTabAssetOptions", LOCTEXT("AssetTabSection", "Assets"));
	{
		InMenuBuilder.AddMenuEntry(
			LOCTEXT("BrowseAssetInContentBrowser", "Browse to Asset"),
			LOCTEXT("BrowseAssetInContentBrowserTooltip", "Browse to asset in content browser"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.BrowseContent"),
			FUIAction(
				FExecuteAction::CreateSP(this, &FAssetEditorToolkit::OnBrowseAssetInContentBrowser),
				FCanExecuteAction::CreateSP(this, &FAssetEditorToolkit::OnCanBrowseAssetInContentBrowser)
			)
		);
	}
	InMenuBuilder.EndSection();
}

bool FAssetEditorToolkit::OnCanBrowseAssetInContentBrowser() const
{
	for (const TObjectPtr<UObject>& EditingObject : EditingObjects)
	{
		if (EditingObject && EditingObject->IsAsset())
		{
			return true;
		}
	}

	return false;
}

void FAssetEditorToolkit::OnBrowseAssetInContentBrowser()
{
	if (HasEditingObject())
	{
		if (const FContentBrowserModule* ContentBrowserModule = FModuleManager::GetModulePtr<FContentBrowserModule>("ContentBrowser"))
		{
			ContentBrowserModule->Get().SyncBrowserToAssets(EditingObjects);
		}
	}
}

void FAssetEditorToolkit::RemoveEditingAsset(UObject* Asset)
{
	// Just close the editor tab if it's the last element
	if (EditingObjects.Num() == 1 && EditingObjects.Contains(Asset))
	{
		CloseWindow(EAssetEditorCloseReason::AssetUnloadingOrInvalid);
	}
	else
	{
		RemoveEditingObject(Asset);
	}
}

void FAssetEditorToolkit::SwitchToStandaloneEditor_Execute( TWeakPtr< FAssetEditorToolkit > ThisToolkitWeakRef )
{
	// NOTE: We're being very careful here with pointer handling because we need to make sure the toolkit's
	// destructor is called when we call CloseToolkit, as it needs to be fully unregistered before we go
	// and try to open a new asset editor for the same asset

	// First, close the world-centric toolkit 
	TArray< FWeakObjectPtr > ObjectsToEditStandaloneWeak;
	TSharedPtr< IToolkitHost > PreviousWorldCentricToolkitHost;
	{
		TSharedRef< FAssetEditorToolkit > ThisToolkit = ThisToolkitWeakRef.Pin().ToSharedRef();
		check( ThisToolkit->IsWorldCentricAssetEditor() );
		PreviousWorldCentricToolkitHost = ThisToolkit->GetToolkitHost();

		const auto& EditingObjects = *ThisToolkit->GetObjectsCurrentlyBeingEdited();

		for( auto ObjectIter = EditingObjects.CreateConstIterator(); ObjectIter; ++ObjectIter )
		{
			ObjectsToEditStandaloneWeak.Add( *ObjectIter );
		}

		FToolkitManager::Get().CloseToolkit( ThisToolkit );

		// At this point, we should be the only referencer of the toolkit!  It will be fully destroyed when
		// as the code pointer exits this block.
		ensure( ThisToolkit.IsUnique() );
	}

	// Now, reopen the toolkit in "standalone" mode
	TArray< UObject* > ObjectsToEdit;

	for( auto ObjectPtrItr = ObjectsToEditStandaloneWeak.CreateIterator(); ObjectPtrItr; ++ObjectPtrItr )
	{
		const auto WeakObjectPtr = *ObjectPtrItr;
		if( WeakObjectPtr.IsValid() )
		{
			ObjectsToEdit.Add( WeakObjectPtr.Get() );
		}
	}

	if( ObjectsToEdit.Num() > 0 )
	{
		ensure( GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAssets_Advanced( ObjectsToEdit, EToolkitMode::Standalone, PreviousWorldCentricToolkitHost.ToSharedRef() ) );
	}
}


void FAssetEditorToolkit::SwitchToWorldCentricEditor_Execute( TWeakPtr< FAssetEditorToolkit > ThisToolkitWeakRef )
{
	// @todo toolkit minor: Maybe also allow the user to drag and drop the standalone editor's tab into a specific level editor to switch to world-centric mode?
	
	// NOTE: We're being very careful here with pointer handling because we need to make sure the tookit's
	// destructor is called when we call CloseToolkit, as it needs to be fully unregistered before we go
	// and try to open a new asset editor for the same asset

	// First, close the standalone toolkit 
	TArray< FWeakObjectPtr > ObjectToEditWorldCentricWeak;
	TSharedPtr< IToolkitHost > WorldCentricLevelEditor;
	{
		TSharedRef< FAssetEditorToolkit > ThisToolkit = ThisToolkitWeakRef.Pin().ToSharedRef();
		const auto& EditingObjects = *ThisToolkit->GetObjectsCurrentlyBeingEdited();

		for( auto ObjectIter = EditingObjects.CreateConstIterator(); ObjectIter; ++ObjectIter )
		{
			ObjectToEditWorldCentricWeak.Add( *ObjectIter );
		}

		check( !ThisToolkit->IsWorldCentricAssetEditor() );
		WorldCentricLevelEditor = ThisToolkit->GetPreviousWorldCentricToolkitHost();

		FToolkitManager::Get().CloseToolkit( ThisToolkit );

		// At this point, we should be the only referencer of the toolkit!  It will be fully destroyed when
		// as the code pointer exits this block.
		ensure( ThisToolkit.IsUnique() );
	}

	// Now, reopen the toolkit in "world-centric" mode
	TArray< UObject* > ObjectsToEdit;
	for( auto ObjectPtrItr = ObjectToEditWorldCentricWeak.CreateIterator(); ObjectPtrItr; ++ObjectPtrItr )
	{
		const auto WeakObjectPtr = *ObjectPtrItr;
		if( WeakObjectPtr.IsValid() )
		{
			ObjectsToEdit.Add( WeakObjectPtr.Get() );
		}
	}

	if( ObjectsToEdit.Num() > 0 )
	{
		ensure( GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAssets_Advanced( ObjectsToEdit, EToolkitMode::WorldCentric, WorldCentricLevelEditor ) );
	}
}



EVisibility FAssetEditorToolkit::GetVisibilityWhileAssetCompiling() const
{
	// don't tick GUI of asset editor when asset is compiling
	//	this is to prevent deadlocks in broken asset editors
	// @todo : change this default to Visible and instead return Collapsed only in the editors that need this bodge
	return EVisibility::Collapsed;
}

void FAssetEditorToolkit::FindInContentBrowser_Execute()
{
	TArray< UObject* > ObjectsToSyncTo;
	GetSaveableObjects(ObjectsToSyncTo);

	if (ObjectsToSyncTo.Num() > 0)
	{
		GEditor->SyncBrowserToObjects( ObjectsToSyncTo );
	}
}

void FAssetEditorToolkit::BrowseDocumentation_Execute() const
{
	IDocumentation::Get()->Open(GetDocumentationLink(), FDocumentationSourceInfo(TEXT("help_menu_asset")));
}

FString FAssetEditorToolkit::GetDocumentationLink() const
{
	return FString(TEXT("%ROOT%"));
}

bool FAssetEditorToolkit::CanReimport() const
{
	for( auto ObjectIter = EditingObjects.CreateConstIterator(); ObjectIter; ++ObjectIter )
	{
		auto EditingObject = *ObjectIter;
		if ( CanReimport( EditingObject ) )
		{
			return true;
		}
	}
	return false;
}

bool FAssetEditorToolkit::CanReimport_Internal() const
{
	if(GetOpenMethod() != EAssetOpenMethod::Edit)
	{
		return false;
	}

	return CanReimport();
}

bool FAssetEditorToolkit::CanReimport( UObject* EditingObject ) const
{
	// Don't allow user to perform certain actions on objects that aren't actually assets (e.g. Level Script blueprint objects)
	if( EditingObject != NULL && EditingObject->IsAsset() )
	{
		// Apply the same logic as Reimport from the Context Menu, see FAssetFileContextMenu::AreImportedAssetActionsVisible
		FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
		UClass* EditingClass = EditingObject->GetClass();
		auto AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(EditingClass).Pin();
		if(!AssetTypeActions.IsValid() || !AssetTypeActions->IsImportedAsset())
		{
			return false;
		}

		if ( FReimportManager::Instance()->CanReimport( EditingObject ) )
		{
			return true;
		}
	}
	return false;
}


void FAssetEditorToolkit::Reimport_Execute()
{
	if( ensure( EditingObjects.Num() > 0 ) )
	{
		for( auto ObjectIter = EditingObjects.CreateConstIterator(); ObjectIter; ++ObjectIter )
		{
			const auto EditingObject = *ObjectIter;
			Reimport_Execute( EditingObject );
		}
	}
}


void FAssetEditorToolkit::Reimport_Execute( UObject* EditingObject )
{
	// Don't allow user to perform certain actions on objects that aren't actually assets (e.g. Level Script blueprint objects)
	if( EditingObject != NULL && EditingObject->IsAsset() )
	{
		// Reimport the asset
		FReimportManager::Instance()->Reimport(EditingObject, ShouldPromptForNewFilesOnReload(*EditingObject));
	}
}

bool FAssetEditorToolkit::ShouldPromptForNewFilesOnReload(const UObject& EditingObject) const
{
	return true;
}

void FAssetEditorToolkit::FillDefaultFileMenuOpenCommands(FToolMenuSection& InSection)
{
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CreateRecentAssetsMenuForEditor(this, InSection);
}

void FAssetEditorToolkit::FillDefaultFileMenuCommands(FToolMenuSection& InSection)
{
	const FToolMenuInsert InsertPosition(NAME_None, EToolMenuInsertType::First);

	if (UAssetEditorToolkitMenuContext* Context = InSection.FindContext<UAssetEditorToolkitMenuContext>())
	{
		InSection.AddMenuEntry(FAssetEditorCommonCommands::Get().SaveAsset, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "AssetEditor.SaveAsset")).InsertPosition = InsertPosition;
		InSection.AddMenuEntry(FAssetEditorCommonCommands::Get().SaveAssetAs, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "AssetEditor.SaveAssetAs")).InsertPosition = InsertPosition;
	}

	if( IsWorldCentricAssetEditor() )
	{
		// @todo toolkit minor: It would be awesome if the user could just "tear off" the SToolkitDisplay to do SwitchToStandaloneEditor
		//			Would need to probably drop at mouseup location though instead of using saved layout pos.
		InSection.AddMenuEntry( FAssetEditorCommonCommands::Get().SwitchToStandaloneEditor ).InsertPosition = InsertPosition;;
	}
	else
	{
		if( GetPreviousWorldCentricToolkitHost().IsValid() )
		{
			// @todo toolkit checkin: Disabled temporarily until we have world-centric "ready to use"!
			#if 0
			{
				InSection.AddMenuEntry( FAssetEditorCommonCommands::Get().SwitchToWorldCentricEditor ).InsertPosition = InsertPosition;;
			}
			#endif
		}
	}
}


void FAssetEditorToolkit::FillDefaultAssetMenuCommands(FToolMenuSection& InSection)
{
	InSection.AddMenuEntry(FGlobalEditorCommonCommands::Get().FindInContentBrowser, LOCTEXT("FindInContentBrowser", "Find in Content Browser..."));

	// Commands we only want to be accessible when editing an asset should go here 
	if( IsActuallyAnAsset() )
	{
		FName ReimportEntryName = TEXT("Reimport");
		int32 MenuEntryCount = 0;
		// Add a reimport menu entry for each supported editable object
		for( auto ObjectIter = EditingObjects.CreateConstIterator(); ObjectIter; ++ObjectIter )
		{
			const auto EditingObject = *ObjectIter;
			if( EditingObject != NULL && EditingObject->IsAsset() )
			{
				if ( CanReimport( EditingObject ) )
				{
					FFormatNamedArguments LabelArguments;
					LabelArguments.Add(TEXT("Name"), FText::FromString( EditingObject->GetName() ));
					const FText LabelText = FText::Format( LOCTEXT("Reimport_Label", "Reimport {Name}..."), LabelArguments );
					FFormatNamedArguments ToolTipArguments;
					ToolTipArguments.Add(TEXT("Type"), FText::FromString( EditingObject->GetClass()->GetName() ));
					const FText ToolTipText = FText::Format( LOCTEXT("Reimport_ToolTip", "Reimports this {Type}"), ToolTipArguments );
					const FName IconName = TEXT( "AssetEditor.ReimportAsset" );
					FUIAction UIAction;
					UIAction.ExecuteAction.BindRaw( this, &FAssetEditorToolkit::Reimport_Execute, EditingObject.Get() );
					ReimportEntryName.SetNumber(MenuEntryCount++);
					InSection.AddMenuEntry( ReimportEntryName, LabelText, ToolTipText, FSlateIcon(FAppStyle::GetAppStyleSetName(), IconName), UIAction );
				}
			}
		}		
	}
}

void FAssetEditorToolkit::FillDefaultHelpMenuCommands(FToolMenuSection& InSection)
{
	// Only show the documentation menu item if the asset editor has specified a resource.
	FString DocLink = GetDocumentationLink();
	if (DocLink != "%ROOT%") 
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("Editor"), GetBaseToolkitName());

		const FText ToolTip = FText::Format(LOCTEXT("BrowseDocumentationTooltip", "Details on using the {Editor}"), Args);
		InSection.AddMenuEntry(FGlobalEditorCommonCommands::Get().OpenDocumentation, FText::Format(LOCTEXT("AssetEditorDocumentationMenuLabel", "{Editor} Documentation"), Args), ToolTip);
	}
}

FName FAssetEditorToolkit::GetToolMenuAppName() const
{
	if (IsSimpleAssetEditor() && EditingObjects.Num() == 1 && EditingObjects[0])
	{
		return *(EditingObjects[0]->GetClass()->GetFName().ToString() + TEXT("Editor"));
	}

	return GetToolkitFName();
}

FName FAssetEditorToolkit::GetToolMenuName() const
{
	return *(TEXT("AssetEditor.") + GetToolMenuAppName().ToString() + TEXT(".MainMenu"));
}

FName FAssetEditorToolkit::GetToolMenuToolbarName() const
{
	FName ParentName;
	return GetToolMenuToolbarName(ParentName);
}

FName FAssetEditorToolkit::GetToolMenuToolbarName(FName& OutParentName) const
{
	OutParentName = DefaultAssetEditorToolBarName;

	return *(TEXT("AssetEditor.") + GetToolMenuAppName().ToString() + TEXT(".ToolBar"));
}

void FAssetEditorToolkit::RegisterDefaultToolBar()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus->IsMenuRegistered(DefaultAssetEditorToolBarName))
	{
		UToolMenu* ToolbarBuilder = ToolMenus->RegisterMenu(DefaultAssetEditorToolBarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
		FToolMenuSection& Section = ToolbarBuilder->AddSection("Asset");
	}
}

void FAssetEditorToolkit::InitializeReadOnlyMenuProfiles()
{
	// Toolbar Customizations

	// Only allow the "Find in Content Browser" toolbar item by default - which hides "Save" since we are using an allowlist
	ReadOnlyCustomization.ToolbarPermissionList.AddAllowListItem(ReadOnlyMenuProfileName, FGlobalEditorCommonCommands::Get().FindInContentBrowser->GetCommandName());

	// Main Menu Customizations

	// The default menus we provide in the main menu
	TArray<FName> MainMenuSubmenus({"File", "Edit", "Asset", "Window", "Tools", "Help"});

	// Only allow the default menus we provide
	for(const FName& Submenu : MainMenuSubmenus)
	{
		ReadOnlyCustomization.MainMenuPermissionList.AddAllowListItem(ReadOnlyMenuProfileName, Submenu);
		ReadOnlyCustomization.MainMenuSubmenuPermissionLists.Add(Submenu, FNamePermissionList());
	}

	// Hide the save commands in the "File" menu in read only mode
	FNamePermissionList& FileMenuPermissionList = ReadOnlyCustomization.MainMenuSubmenuPermissionLists.FindOrAdd("File");

	FileMenuPermissionList.AddDenyListItem(ReadOnlyMenuProfileName, FAssetEditorCommonCommands::Get().SaveAsset->GetCommandName());
	FileMenuPermissionList.AddDenyListItem(ReadOnlyMenuProfileName, FAssetEditorCommonCommands::Get().SaveAssetAs->GetCommandName());

	// Hide the re-import command in the "asset" menu in read only mode
	// There is a reimport command per editing object, so we make sure to hide them all. See FAssetEditorToolkit::FillDefaultAssetMenuCommands
	FNamePermissionList& AssetMenuPermissionList = ReadOnlyCustomization.MainMenuSubmenuPermissionLists.FindOrAdd("Asset");
	
	FName ReimportEntryName = TEXT("Reimport");
	int32 MenuEntryCount = 0;

	for( auto ObjectIter = EditingObjects.CreateConstIterator(); ObjectIter; ++ObjectIter )
	{
		ReimportEntryName.SetNumber(MenuEntryCount++);
		AssetMenuPermissionList.AddDenyListItem(ReadOnlyMenuProfileName, ReimportEntryName);
	}
	
	// Give specific asset editors a chance to customize the default behavior (e.g show specific menus they want to allow in read only mode)
	SetupReadOnlyMenuProfiles(ReadOnlyCustomization);

	// Create the menu profile and apply the permission lists
	FToolMenuProfile* MainMenuProfile = UToolMenus::Get()->AddRuntimeMenuProfile(GetToolMenuName(), ReadOnlyMenuProfileName);
	FToolMenuProfile* ToolbarProfile = UToolMenus::Get()->AddRuntimeMenuProfile(GetToolMenuToolbarName(), ReadOnlyMenuProfileName);
	FToolMenuProfile* CommonActionsToolbarProfile = UToolMenus::Get()->AddRuntimeMenuProfile("AssetEditorToolbar.CommonActions", ReadOnlyMenuProfileName);

	MainMenuProfile->MenuPermissions = ReadOnlyCustomization.MainMenuPermissionList;
	ToolbarProfile->MenuPermissions = ReadOnlyCustomization.ToolbarPermissionList;
	CommonActionsToolbarProfile->MenuPermissions = ReadOnlyCustomization.ToolbarPermissionList;

	for(const FName& Submenu : MainMenuSubmenus)
	{
		const FName SubmenuName = *(GetToolMenuName().ToString() + TEXT(".") + Submenu.ToString());
		
		FToolMenuProfile* SubmenuProfile = UToolMenus::Get()->AddRuntimeMenuProfile(SubmenuName, ReadOnlyMenuProfileName);
		SubmenuProfile->MenuPermissions = ReadOnlyCustomization.MainMenuSubmenuPermissionLists[Submenu];
	}

}

void FAssetEditorToolkit::InitToolMenuContext(FToolMenuContext& MenuContext)
{
	UAssetEditorToolkitMenuContext* ToolkitMenuContext = MenuContext.FindContext<UAssetEditorToolkitMenuContext>();
	
	if(!ToolkitMenuContext || !ToolkitMenuContext->Toolkit.IsValid())
	{
		return;
	}

	// If we are in read only mode, set the read only menu profile as active
	if(ToolkitMenuContext->Toolkit.Pin()->GetOpenMethod() == EAssetOpenMethod::View)
	{
		UToolMenuProfileContext* ProfileContext = NewObject<UToolMenuProfileContext>();
		ProfileContext->ActiveProfiles.Add(ReadOnlyMenuProfileName);
		MenuContext.AddObject(ProfileContext);
	}
}

UToolMenu* FAssetEditorToolkit::GenerateCommonActionsToolbar(FToolMenuContext& MenuContext)
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	FName ToolBarName = "AssetEditorToolbar.CommonActions";

	UToolMenu* FoundMenu = ToolMenus->FindMenu(ToolBarName);

	if (!FoundMenu || !FoundMenu->IsRegistered())
	{
		FoundMenu = ToolMenus->RegisterMenu(ToolBarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
		FoundMenu->StyleName = "AssetEditorToolbar";

		FToolMenuSection& Section = FoundMenu->AddSection("CommonActions");

		Section.AddDynamicEntry("CommonActionsDynamic", FNewToolMenuSectionDelegate::CreateLambda([this](FToolMenuSection& InSection)
		{
			UAssetEditorToolkitMenuContext* AssetEditorToolkitMenuContext = InSection.FindContext<UAssetEditorToolkitMenuContext>();

			if (AssetEditorToolkitMenuContext)
			{
				if(TSharedPtr<FAssetEditorToolkit> AssetEditorToolkit = AssetEditorToolkitMenuContext->Toolkit.Pin())
				{
					InSection.AddEntry(FToolMenuEntry::InitToolBarButton(FAssetEditorCommonCommands::Get().SaveAsset));
				}
			}
		}));

		Section.AddEntry(FToolMenuEntry::InitToolBarButton(FGlobalEditorCommonCommands::Get().FindInContentBrowser, LOCTEXT("FindInContentBrowserButton", "Browse")));
		Section.AddSeparator(NAME_None);
	}

	return ToolMenus->GenerateMenu(ToolBarName, MenuContext);
}

UToolMenu* FAssetEditorToolkit::GenerateReadOnlyToolbar(FToolMenuContext& MenuContext)
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	FName ToolBarName = "AssetEditorToolbar.ReadOnly";

	UToolMenu* FoundMenu = ToolMenus->FindMenu(ToolBarName);
	
	if (!FoundMenu || !FoundMenu->IsRegistered())
	{
		FoundMenu = ToolMenus->RegisterMenu(ToolBarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
		FoundMenu->StyleName = "AssetEditorToolbar";
		FToolMenuSection& Section = FoundMenu->AddSection("ReadOnly");

		Section.AddDynamicEntry("ReadOnlyDynamic", FNewToolMenuSectionDelegate::CreateLambda([this](FToolMenuSection& InSection)
		{
			// A custom widget to show the read only status of the asset editor
			TSharedRef<SWidget> ReadOnlyIndicatorWidget =
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.Padding(20.0f, 0.0f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("AssetEditor.ReadOnlyBorder"))
				.Padding(0.0f, 0.0f)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(8.0f, 0.0f, 0.0f, 0.0f)
					[
						SNew(SBox)
						.HeightOverride(16.0f)
						.WidthOverride(16.0f)
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("AssetEditor.ReadOnlyOpenable"))
							.ColorAndOpacity(FStyleColors::AccentBlack)
						]
						
					]
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(4.0f, 0.0f, 8.0f, 0.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ReadOnlyText", "Read Only"))
						.ColorAndOpacity(FStyleColors::AccentBlack)
					]
				]	
			];
		
			InSection.AddSeparator(NAME_None);
			InSection.AddEntry(FToolMenuEntry::InitWidget("ReadOnlyIndicatorWidget", ReadOnlyIndicatorWidget, FText::GetEmpty()));
		}));
	}

	return ToolMenus->GenerateMenu(ToolBarName, MenuContext);;
}

void FAssetEditorToolkit::GenerateToolbar()
{
	TSharedPtr<FExtender> Extender = FExtender::Combine(ToolbarExtenders);

	RegisterDefaultToolBar();

	FName ParentToolbarName;
	const FName ToolBarName = GetToolMenuToolbarName(ParentToolbarName);
	UToolMenus* ToolMenus = UToolMenus::Get();
	UToolMenu* FoundMenu = ToolMenus->FindMenu(ToolBarName);
	if (!FoundMenu || !FoundMenu->IsRegistered())
	{
		FoundMenu = ToolMenus->RegisterMenu(ToolBarName, ParentToolbarName, EMultiBoxType::SlimHorizontalToolBar);
	}

	FToolMenuContext MenuContext(GetToolkitCommands(), Extender);

	UAssetEditorToolkitMenuContext* ToolkitMenuContext = NewObject<UAssetEditorToolkitMenuContext>(FoundMenu);
	ToolkitMenuContext->Toolkit = AsShared();
	MenuContext.AddObject(ToolkitMenuContext);

	InitToolMenuContext(MenuContext);

	UToolMenu* GeneratedToolbar = ToolMenus->GenerateMenu(ToolBarName, MenuContext);
	GeneratedToolbar->bToolBarIsFocusable = bIsToolbarFocusable;
	GeneratedToolbar->bToolBarForceSmallIcons = bIsToolbarUsingSmallIcons;
	TSharedRef< class SWidget > ToolBarWidget = ToolMenus->GenerateWidget(GeneratedToolbar);

	UToolMenu* CommonActionsToolbar = GenerateCommonActionsToolbar(MenuContext);
	TSharedRef< class SWidget > CommonActionsToolbarWidget = ToolMenus->GenerateWidget(CommonActionsToolbar);

	UToolMenu* ReadOnlyToolbar = GenerateReadOnlyToolbar(MenuContext);
	TSharedRef< class SWidget > ReadOnlyWidget = ToolMenus->GenerateWidget(ReadOnlyToolbar);

	TSharedRef<SWidget> MiscWidgets = SNullWidget::NullWidget;

	if(ToolbarWidgets.Num() > 0)
	{
		TSharedRef<SHorizontalBox> MiscWidgetsHBox = SNew(SHorizontalBox);

		for (int32 WidgetIdx = 0; WidgetIdx < ToolbarWidgets.Num(); ++WidgetIdx)
		{
			MiscWidgetsHBox->AddSlot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					ToolbarWidgets[WidgetIdx]
				];
		}

		MiscWidgets = MiscWidgetsHBox;
	}
	
	Toolbar = 
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			CommonActionsToolbarWidget
		]
		+SHorizontalBox::Slot()
		[
			SNew(SBorder)
			.VAlign(VAlign_Center)
			.BorderImage(FAppStyle::Get().GetBrush("AssetEditorToolbar.Background"))
			.Padding(FMargin(0.0f))
			[
				ToolBarWidget
			]
		]
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.AutoWidth()
		[
			SNew(SBorder)
			.VAlign(VAlign_Center)
			.BorderImage(FAppStyle::Get().GetBrush("AssetEditorToolbar.Background"))
			.Padding(MiscWidgets == SNullWidget::NullWidget ? FMargin(0.0f) : FMargin(4.f, 0.f, 0.f, 0.f))
			[
				MiscWidgets
			]
		]
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.AutoWidth()
		[
			SNew(SBorder)
			.VAlign(VAlign_Center)
			.Visibility_Lambda([this]()
			{
				return GetOpenMethod() == EAssetOpenMethod::View ? EVisibility::Visible : EVisibility::Collapsed;
			})
			.BorderImage(FAppStyle::Get().GetBrush("AssetEditorToolbar.Background"))
			.Padding(FMargin(0.0f))
			[
				ReadOnlyWidget
			]
		];

	if (ToolbarWidgetContent.IsValid())
	{
		ToolbarWidgetContent->SetContent(Toolbar.ToSharedRef());
	}
}

void FAssetEditorToolkit::RegenerateMenusAndToolbars()
{
	RemoveAllToolbarWidgets();

	TSharedPtr<SStandaloneAssetEditorToolkitHost> HostWidget = StandaloneHost.Pin();
	if (HostWidget)
	{
		HostWidget->GenerateMenus(false);
	}

	if (Toolbar != SNullWidget::NullWidget)
	{
		GenerateToolbar();
	}

	PostRegenerateMenusAndToolbars();

	if (HostWidget)
	{
		HostWidget->SetToolbar(Toolbar);
	}
}

void FAssetEditorToolkit::RegisterDrawer(FWidgetDrawerConfig&& Drawer, int32 SlotIndex)
{
	TSharedPtr< class SStandaloneAssetEditorToolkitHost > HostWidget = StandaloneHost.Pin();
	if (HostWidget.IsValid())
	{
		HostWidget->RegisterDrawer(MoveTemp(Drawer), SlotIndex);
	}
}

void FAssetEditorToolkit::RestoreFromLayout(const TSharedRef<FTabManager::FLayout>& NewLayout)
{
	constexpr bool bLoadUserLayout = true;
	RestoreFromLayout(NewLayout, bLoadUserLayout);
}

bool FAssetEditorToolkit::IsActuallyAnAsset() const
{
	// Don't allow user to perform certain actions on objects that aren't actually assets (e.g. Level Script blueprint objects)
	bool bIsActuallyAnAsset = false;
	for( auto ObjectIter = GetObjectsCurrentlyBeingEdited()->CreateConstIterator(); !bIsActuallyAnAsset && ObjectIter; ++ObjectIter )
	{
		const auto ObjectBeingEdited = *ObjectIter;
		bIsActuallyAnAsset |= ObjectBeingEdited != NULL && ObjectBeingEdited->IsAsset();
	}
	return bIsActuallyAnAsset;
}

UE::Editor::Toolbars::ECreateStatusBarOptions FAssetEditorToolkit::GetStatusBarCreationOptions() const
{
	return UE::Editor::Toolbars::ECreateStatusBarOptions::Default;
}

void FAssetEditorToolkit::AddMenuExtender(TSharedPtr<FExtender> Extender)
{
	if (TSharedPtr<SStandaloneAssetEditorToolkitHost> StandaloneHostPtr = StandaloneHost.Pin())
	{
		StandaloneHostPtr->GetMenuExtenders().AddUnique(Extender);
	}
}

void FAssetEditorToolkit::RemoveMenuExtender(TSharedPtr<FExtender> Extender)
{
	if (TSharedPtr<SStandaloneAssetEditorToolkitHost> StandaloneHostPtr = StandaloneHost.Pin())
	{
		StandaloneHostPtr->GetMenuExtenders().Remove(Extender);
	}
}

void FAssetEditorToolkit::AddToolbarExtender(TSharedPtr<FExtender> Extender)
{
	ToolbarExtenders.AddUnique(Extender);
}

void FAssetEditorToolkit::RemoveToolbarExtender(TSharedPtr<FExtender> Extender)
{
	ToolbarExtenders.Remove(Extender);
}

TSharedPtr<FExtensibilityManager> FAssetEditorToolkit::GetSharedMenuExtensibilityManager()
{
	if (!SharedMenuExtensibilityManager.IsValid())
	{
		SharedMenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
	}
	return SharedMenuExtensibilityManager;
}

TSharedPtr<FExtensibilityManager> FAssetEditorToolkit::GetSharedToolBarExtensibilityManager()
{
	if (!SharedToolBarExtensibilityManager.IsValid())
	{
		SharedToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);
	}
	return SharedToolBarExtensibilityManager;
}

void FAssetEditorToolkit::SetMenuOverlay( TSharedRef<SWidget> Widget )
{
	if (TSharedPtr<SStandaloneAssetEditorToolkitHost> StandaloneHostPtr = StandaloneHost.Pin())
	{
		StandaloneHostPtr->SetMenuOverlay(Widget);
	}
}

void FAssetEditorToolkit::AddToolbarWidget(TSharedRef<SWidget> Widget)
{
	ToolbarWidgets.AddUnique(Widget);
}

void FAssetEditorToolkit::RemoveAllToolbarWidgets()
{
	ToolbarWidgets.Empty();
}


void FAssetEditorToolkit::FGCEditingObjects::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(OwnerToolkit.EditingObjects);
	
	// Remove null objects as a safe guard against assets being forcibly GC'd
	OwnerToolkit.EditingObjects.RemoveAllSwap([](UObject* Obj) { return Obj == nullptr; });
}

FString FAssetEditorToolkit::FGCEditingObjects::GetReferencerName() const
{
	return TEXT("FAssetEditorToolkit::FGCEditorObjects");
}

TSharedPtr<FExtender> FExtensibilityManager::GetAllExtenders()
{
	return FExtender::Combine(Extenders);
}
	
TSharedPtr<FExtender> FExtensibilityManager::GetAllExtenders(const TSharedRef<FUICommandList>& CommandList, const TArray<UObject*>& ContextSensitiveObjects)
{
	auto OutExtenders = Extenders;
	for (int32 i = 0; i < ExtenderDelegates.Num(); ++i)
	{
		if (ExtenderDelegates[i].IsBound())
		{
			OutExtenders.Add(ExtenderDelegates[i].Execute(CommandList, ContextSensitiveObjects));
		}
	}
	return FExtender::Combine(OutExtenders);
}

TArray<UObject*> UAssetEditorToolkitMenuContext::GetEditingObjects() const
{
	TArray<UObject*> Result;
	if (TSharedPtr<FAssetEditorToolkit> Pinned = Toolkit.Pin())
	{
		for (UObject* Object : Pinned->GetEditingObjects())
		{
			if (Object)
			{
				Result.Add(Object);
			}
		}
	}

	return Result;
}
	
#undef LOCTEXT_NAMESPACE
