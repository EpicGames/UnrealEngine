// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/BaseToolkit.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "Toolkits/ToolkitManager.h"
#include "PropertyEditorModule.h"
#include "Modules/ModuleManager.h"
#include "IDetailsView.h"
#include "InteractiveToolManager.h"
#include "Editor.h"
#include "InteractiveTool.h"
#include "Tools/UEdMode.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "EdMode.h"
#include "EditorModeManager.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Toolkits/AssetEditorModeUILayer.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"

#define LOCTEXT_NAMESPACE "BaseToolkit"

FBaseToolkit::FBaseToolkit()
	: ToolkitMode( EToolkitMode::Standalone ),
	  ToolkitCommands( new FUICommandList() )
{
	WorkspaceMenuCategory = FWorkspaceItem::NewGroup(LOCTEXT("WorkspaceMenu_BaseToolkit", "Toolkit"));
}


FBaseToolkit::~FBaseToolkit()
{
}



bool FBaseToolkit::IsWorldCentricAssetEditor() const
{
	return ToolkitMode == EToolkitMode::WorldCentric;
}



bool FBaseToolkit::IsHosted() const
{
	return ToolkitHost.IsValid();
}


const TSharedRef< IToolkitHost > FBaseToolkit::GetToolkitHost() const
{
	return ToolkitHost.Pin().ToSharedRef();
}

FName FBaseToolkit::GetToolkitContextFName() const
{
	return GetToolkitFName();
}


bool FBaseToolkit::ProcessCommandBindings( const FKeyEvent& InKeyEvent ) const
{
	if( ToolkitCommands->ProcessCommandBindings( InKeyEvent ) )
	{
		return true;
	}
	
	return false;
}

FString FBaseToolkit::GetTabPrefix() const
{
	if( IsWorldCentricAssetEditor() )
	{
		return GetWorldCentricTabPrefix();
	}
	else
	{
		return TEXT( "" );
	}
}


FLinearColor FBaseToolkit::GetTabColorScale() const
{
	return IsWorldCentricAssetEditor() ? GetWorldCentricTabColorScale() : FLinearColor( 0, 0, 0, 0 );
}

void FBaseToolkit::CreateEditorModeManager()
{
}

void FBaseToolkit::BringToolkitToFront()
{
	if( ensure( ToolkitHost.IsValid() ) )
	{
		// Bring the host window to front
		ToolkitHost.Pin()->BringToFront();
		// Tell the toolkit its been brought to the fore - give it a chance to update anything it needs to
		ToolkitBroughtToFront();
	}
}

TSharedPtr<class SWidget> FBaseToolkit::GetInlineContent() const
{
	return TSharedPtr<class SWidget>();
}


bool FBaseToolkit::IsBlueprintEditor() const
{
	return false;
}

void FModeToolkit::Init(const TSharedPtr< class IToolkitHost >& InitToolkitHost)
{
	Init(InitToolkitHost, TWeakObjectPtr<UEdMode>());
}

void FModeToolkit::Init(const TSharedPtr< class IToolkitHost >& InitToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode)
{
	check( InitToolkitHost.IsValid() );

	ToolkitMode = EToolkitMode::Type::Standalone;
	ToolkitHost = InitToolkitHost;
	OwningEditorMode = InOwningMode;
	PrimaryTabInfo = FMinorTabConfig();
	ToolbarInfo = FMinorTabConfig();

	if (OwningEditorMode.IsValid())
	{
		UInteractiveToolManager* EditorToolManager = OwningEditorMode->GetToolManager(EToolsContextScope::Editor);
		if (EditorToolManager)
		{
			EditorToolManager->OnToolStarted.AddSP(this, &FModeToolkit::OnToolStarted);
			EditorToolManager->OnToolEnded.AddSP(this, &FModeToolkit::OnToolEnded);
		}

		UInteractiveToolManager* ModeToolManager = OwningEditorMode->GetToolManager(EToolsContextScope::EdMode);
		if (ModeToolManager)
		{
			ModeToolManager->OnToolStarted.AddSP(this, &FModeToolkit::OnToolStarted);
			ModeToolManager->OnToolEnded.AddSP(this, &FModeToolkit::OnToolEnded);
		}
	}


	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	{
		FDetailsViewArgs ModeDetailsViewArgs;
		ModeDetailsViewArgs.bAllowSearch = false;
		ModeDetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		ModeDetailsViewArgs.bHideSelectionTip = true;
		ModeDetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;
		ModeDetailsViewArgs.bShowOptions = false;
		ModeDetailsViewArgs.bAllowMultipleTopLevelObjects = true;

		CustomizeModeDetailsViewArgs(ModeDetailsViewArgs);		// allow subclass to customize arguments

		ModeDetailsView = PropertyEditorModule.CreateDetailView(ModeDetailsViewArgs);
	}

	{
		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bAllowMultipleTopLevelObjects = true;

		CustomizeDetailsViewArgs(DetailsViewArgs);		// allow subclass to customize arguments

		DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	}

	GetEditorModeManager().OnEditorModeIDChanged().AddSP(this, &FModeToolkit::OnModeIDChanged);

	if (HasToolkitBuilder())	
	{
		ToolkitBuilder->VerticalToolbarElement->GenerateWidget();
	}
}


FModeToolkit::~FModeToolkit()
{
	if (IsHosted())
	{
		GetEditorModeManager().OnEditorModeIDChanged().RemoveAll(this);
		if (OwningEditorMode.IsValid())
		{
			UInteractiveToolManager* EditorToolManager = OwningEditorMode->GetToolManager(EToolsContextScope::Editor);
			if (EditorToolManager)
			{
				EditorToolManager->OnToolStarted.RemoveAll(this);
				EditorToolManager->OnToolEnded.RemoveAll(this);
			}

			UInteractiveToolManager* ModeToolManager = OwningEditorMode->GetToolManager(EToolsContextScope::EdMode);
			if (ModeToolManager)
			{
				ModeToolManager->OnToolStarted.RemoveAll(this);
				ModeToolManager->OnToolEnded.RemoveAll(this);
			}
		}
	}
	if (ModeToolBarContainer)
	{
		ModeToolBarContainer->SetContent(SNullWidget::NullWidget);
	}
	if (ModeToolHeader)
	{
		ModeToolHeader->SetContent(SNullWidget::NullWidget);
	}
	if (InlineContentHolder)
	{
		InlineContentHolder->SetContent(SNullWidget::NullWidget);
	}
	OwningEditorMode.Reset();
}

void FModeToolkit::SetModeUILayer(const TSharedPtr<FAssetEditorModeUILayer> InLayer)
{
	ModeUILayer = InLayer;

	//TODO: Maybe Mode Toolbar Commands should be separate from the Toolkit Commands? Or the ModeUILayer should be
	// responsible for a generic set of "Mode Commands"
	ModeUILayer.Pin()->GetModeCommands()->Append(GetToolkitCommands());
	ModeUILayer.Pin()->ToolkitHostReadyForUI().BindSP(this, &FModeToolkit::InvokeUI);
	ModeUILayer.Pin()->ToolkitHostShutdownUI().BindSP(this, &FModeToolkit::ShutdownUI);

	ModeUILayer.Pin()->RegisterSecondaryModeToolbarExtension().BindSP(this, &FModeToolkit::ExtendSecondaryModeToolbar);
}

void FModeToolkit::RegisterTabSpawners(const TSharedRef<FTabManager>& TabManager)
{
	RequestModeUITabs();
}

void FModeToolkit::UnregisterTabSpawners(const TSharedRef<FTabManager>& TabManager)
{
}

FName FModeToolkit::GetToolkitFName() const
{
	return FName("EditorModeToolkit");
}

FText FModeToolkit::GetBaseToolkitName() const
{
	return NSLOCTEXT("EditorModeToolkit", "DisplayName", "EditorMode Tool");
}

FString FModeToolkit::GetWorldCentricTabPrefix() const
{
	return FString();
}

bool FModeToolkit::IsAssetEditor() const
{
	return false;
}

const TArray< UObject* >* FModeToolkit::GetObjectsCurrentlyBeingEdited() const
{
	return NULL;
}

FLinearColor FModeToolkit::GetWorldCentricTabColorScale() const
{
	return FLinearColor();
}

void FModeToolkit::OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	// Update properties panel
	if (!OwningEditorMode.IsValid())
	{
		return;
	}

	UInteractiveTool* CurTool = OwningEditorMode->GetToolManager(EToolsContextScope::Editor)->GetActiveTool(EToolSide::Left);
	if (CurTool == nullptr)   // try Mode-level ToolManager
	{
		CurTool = OwningEditorMode->GetToolManager(EToolsContextScope::EdMode)->GetActiveTool(EToolSide::Left);
	}

	if (CurTool)
	{
		DetailsView->SetObjects(CurTool->GetToolProperties());
	}
}

void FModeToolkit::OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	DetailsView->SetObject(nullptr);
}

class FEdMode* FModeToolkit::GetEditorMode() const
{
	return nullptr; 
}

FText FModeToolkit::GetEditorModeDisplayName() const
{
	if (const FEditorModeInfo* ModeInfo = GetEditorModeInfo())
	{
		return ModeInfo->Name;
	}

	return FText::GetEmpty();
}

FSlateIcon FModeToolkit::GetEditorModeIcon() const
{
	if (const FEditorModeInfo* ModeInfo = GetEditorModeInfo())
	{
		return ModeInfo->IconBrush;
	}

	return FSlateIcon();
}

FEditorModeTools& FModeToolkit::GetEditorModeManager() const
{
	check(IsHosted());
	return GetToolkitHost()->GetEditorModeManager();
}

TWeakObjectPtr<UEdMode> FModeToolkit::GetScriptableEditorMode() const
{
	return OwningEditorMode;
}

TSharedPtr<SWidget> FModeToolkit::GetInlineContent() const
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			ModeDetailsView.ToSharedRef()
		]
		+ SVerticalBox::Slot()
		[
			DetailsView.ToSharedRef()
		];
}

void FModeToolkit::BuildToolPalette(FName PaletteName, class FToolBarBuilder& ToolbarBuilder)
{
	if (!OwningEditorMode.IsValid())
	{
		return;
	}

	TMap<FName, TArray<TSharedPtr<FUICommandInfo>>> CommandLists = OwningEditorMode->GetModeCommands();
	TArray<TSharedPtr<FUICommandInfo>>* CurrentCommandListPtr = CommandLists.Find(PaletteName);
	if (CurrentCommandListPtr)
	{
		TArray<TSharedPtr<FUICommandInfo>> CurrentCommandList = *CurrentCommandListPtr;
		for (TSharedPtr<FUICommandInfo> Command : CurrentCommandList)
		{
			ToolbarBuilder.AddToolBarButton(Command);
		}

	}
}

FName FModeToolkit::GetCurrentPalette() const
{
	return CurrentPaletteName;
}

void FModeToolkit::SetCurrentPalette(FName InPalette)
{
	CurrentPaletteName = InPalette;
	OnToolPaletteChanged(CurrentPaletteName);
	OnPaletteChangedDelegate.Broadcast(InPalette);
}

void FModeToolkit::SetModeSettingsObject(UObject* InSettingsObject)
{
	ModeDetailsView->SetObject(InSettingsObject);
}

void FModeToolkit::InvokeUI()
{
	if (ModeUILayer.IsValid())
	{
		TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin();
		TSharedPtr<SDockTab> CreatedTab = ModeUILayerPtr->GetTabManager()->TryInvokeTab(UAssetEditorUISubsystem::TopLeftTabID);
		UpdatePrimaryModePanel();
		TSharedPtr<FUICommandList> CommandList;
		if (!HasIntegratedToolPalettes())
		{
			if (GetScriptableEditorMode().IsValid())
			{
				UEdMode* ScriptableMode = GetScriptableEditorMode().Get();
				CommandList = GetToolkitCommands();

				// Also build the toolkit here 
				TArray<FName> PaletteNames;
				GetToolPaletteNames(PaletteNames);
				for (const FName& Palette : PaletteNames)
				{
					TSharedRef<SWidget> PaletteWidget = CreatePaletteWidget(CommandList, ScriptableMode->GetModeInfo().ToolbarCustomizationName, Palette);
					ActiveToolBarRows.Emplace(ScriptableMode->GetID(), Palette, GetToolPaletteDisplayName(Palette), PaletteWidget);
				}
			}
			if (!HasToolkitBuilder())
			{
				const TSharedPtr<SDockTab> CreatedToolbarTab = ModeUILayerPtr->GetTabManager()->TryInvokeTab(UAssetEditorUISubsystem::VerticalToolbarID);
				ModeToolbarTab = CreatedToolbarTab;				
			}


		}
	}
}



void FModeToolkit::RebuildModeToolPalette()
{
	if (ModeUILayer.IsValid() && HasIntegratedToolPalettes() == false)
	{
		if ( TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin() )
		{
			TSharedPtr<FUICommandList> CommandList;
			if (GetScriptableEditorMode().IsValid())
			{
				UEdMode* ScriptableMode = GetScriptableEditorMode().Get();
				CommandList = GetToolkitCommands();
				ActiveToolBarRows.Reset();

				TArray<FName> PaletteNames;
				GetToolPaletteNames(PaletteNames);
				for (const FName& Palette : PaletteNames)
				{
					TSharedRef<SWidget> PaletteWidget = CreatePaletteWidget(CommandList, ScriptableMode->GetModeInfo().ToolbarCustomizationName, Palette);
					ActiveToolBarRows.Emplace(ScriptableMode->GetID(), Palette, GetToolPaletteDisplayName(Palette), PaletteWidget);
				}

				RebuildModeToolBar();
			}
		}
	}

}

bool FModeToolkit::HasToolkitBuilder() const
{
	return bUsesToolkitBuilder && ToolkitBuilder != nullptr;
}

TSharedRef<SDockTab> FModeToolkit::CreatePrimaryModePanel(const FSpawnTabArgs& Args)
{
	TSharedPtr<SWidget> TabContent;
	if (!HasToolkitBuilder())
	{
		TabContent = SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		.Padding(0.0f)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Left)
			[
				SAssignNew(ModeToolBarContainer, SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Padding(FMargin(4, 0, 0, 0))
			]

			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.Padding(0.0, 8.0, 0.0, 0.0)
				.AutoHeight()
				[
					SAssignNew(ModeToolHeader, SBorder)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				]

				+ SVerticalBox::Slot()
				.FillHeight(1)
				[
					SAssignNew(InlineContentHolder, SBorder)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					.Visibility(this, &FModeToolkit::GetInlineContentHolderVisibility)
				]
		]
		];
	}
	// else if ToolkitBuilder is defined, make the Toolkit
	else
	{
		TabContent = SAssignNew(InlineContentHolder, SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(0.0f)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Left)
			[
				SAssignNew(ModeToolBarContainer, SBorder)
				.Padding(FMargin(4, 0, 0, 0))
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			]
		];
	}

	const TSharedPtr<SDockTab> CreatedTab = SNew(SDockTab);
	if (TabContent)
	{
		CreatedTab->SetContent(TabContent.ToSharedRef());
	}
	PrimaryTab = CreatedTab;	
	UpdatePrimaryModePanel();
	return CreatedTab.ToSharedRef();

}



void FModeToolkit::UpdatePrimaryModePanel()
{
	if (GetEditorMode() || GetScriptableEditorMode().IsValid())
	{
		const FText TabName = GetEditorModeDisplayName();
		const FSlateBrush* TabIcon = GetEditorModeIcon().GetSmallIcon();
		if (PrimaryTab.IsValid())
		{
			TSharedPtr<SDockTab> TabPtr  = PrimaryTab.Pin(); 
			TabPtr->SetTabIcon(TabIcon);
			TabPtr->SetLabel(TabName);

			if (HasToolkitBuilder())
			{
				TabPtr->SetParentDockTabStackTabWellHidden(HasToolkitBuilder());
				const TSharedPtr<SWidget> Content = GetInlineContent() ;
				
				if ( Content && InlineContentHolder.IsValid() )
				{
					InlineContentHolder->SetContent( Content.ToSharedRef() );
				}
				return;
			}
		}

		if (HasIntegratedToolPalettes())
		{
			TSharedRef<SSegmentedControl<FName>> PaletteTabBox = SNew(SSegmentedControl<FName>)
				.UniformPadding(FMargin(8.f, 3.f))
				.Value_Lambda([this]() { return GetCurrentPalette(); } ) 
				.OnValueChanged_Lambda([this](const FName& Palette) { SetCurrentPalette(Palette); } );

			// Only show if there is more than one child in the switcher
			PaletteTabBox->SetVisibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda([PaletteTabBox]() -> EVisibility
				{
					return PaletteTabBox->NumSlots() > 1 ? EVisibility::Visible : EVisibility::Collapsed;
				})));

			// Also build the toolkit here 
			TArray<FName> PaletteNames;
			GetToolPaletteNames(PaletteNames);

			TSharedPtr<FUICommandList> CommandList;
			CommandList = GetToolkitCommands();

			TSharedRef< SWidgetSwitcher > PaletteSwitcher = SNew(SWidgetSwitcher)
				.WidgetIndex_Lambda(
				[this, PaletteNames]() -> int32
				{
					int32 FoundIndex;
					if (PaletteNames.Find(GetCurrentPalette(), FoundIndex))
					{
						return FoundIndex;
					}
					return 0;
				});

			
			for (auto Palette : PaletteNames)
			{
				FName ToolbarCustomizationName = GetEditorMode() ?  GetEditorMode()->GetModeInfo().ToolbarCustomizationName : GetScriptableEditorMode()->GetModeInfo().ToolbarCustomizationName;
				TSharedRef<SWidget> PaletteWidget = CreatePaletteWidget(CommandList, ToolbarCustomizationName, Palette);

				const bool bRebuildChildren = false;
				PaletteTabBox->AddSlot(Palette, false)
				.Text(GetToolPaletteDisplayName(Palette));

				PaletteSwitcher->AddSlot()
				[
					PaletteWidget
				];
			}

			PaletteTabBox->RebuildChildren();

			if (ModeToolHeader)
			{
				ModeToolHeader->SetContent(
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.Padding(0.f, 0.f, 0.f, 8.f)
					.HAlign(HAlign_Center)
					.AutoHeight()
					[
						PaletteTabBox
					]

				+ SVerticalBox::Slot()
					.AutoHeight()
					[
						PaletteSwitcher
					]
				);
			}
		}
		else
		{
			if (ModeToolHeader)
			{
				ModeToolHeader->SetContent(SNullWidget::NullWidget);
			}
			if (ModeToolBarContainer)
			{
				ModeToolBarContainer->SetContent(SNullWidget::NullWidget);
			}
		}

		if (InlineContentHolder.IsValid())
		{
			if (TSharedPtr<SWidget> InlineContent = GetInlineContent())
			{
				InlineContentHolder->SetContent(
					SNew(SScrollBox)
					+ SScrollBox::Slot()
					[
						InlineContent.ToSharedRef()
					]);
			}
		}

	}
	
}

EVisibility FModeToolkit::GetInlineContentHolderVisibility() const
{
	if (InlineContentHolder)
	{
		return InlineContentHolder->GetContent() == SNullWidget::NullWidget ? EVisibility::Collapsed : EVisibility::Visible;
	}
	return EVisibility::Collapsed;
}

EVisibility FModeToolkit::GetNoToolSelectedTextVisibility() const
{
	if (InlineContentHolder)
	{
		return InlineContentHolder->GetContent() == SNullWidget::NullWidget ? EVisibility::Visible : EVisibility::Collapsed;
	}
	return EVisibility::Collapsed;
}

TSharedRef<SDockTab> FModeToolkit::MakeModeToolbarTab(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> ToolbarTabRef =
		SNew(SDockTab)
		.Label(NSLOCTEXT("EditorModes", "EditorModesToolbarTitle", "Mode Toolbar"))
		.ContentPadding(0.0f)
		[
			SAssignNew(ModeToolbarBox, SVerticalBox)
		];

	ModeToolbarTab = ToolbarTabRef;
	SpawnOrUpdateModeToolbar();
	return ToolbarTabRef;

}

void FModeToolkit::RequestModeUITabs()
{
	if (ModeUILayer.IsValid())
	{
		TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin();
		PrimaryTabInfo.OnSpawnTab = FOnSpawnTab::CreateSP(SharedThis(this), &FModeToolkit::CreatePrimaryModePanel);
		PrimaryTabInfo.TabLabel = LOCTEXT("ModesToolboxTab", "Mode Toolbox");
		PrimaryTabInfo.TabTooltip = LOCTEXT("ModesToolboxTabTooltipText", "Open the  Modes tab, which contains the active editor mode's settings.");
		ModeUILayerPtr->SetModePanelInfo(UAssetEditorUISubsystem::TopLeftTabID, PrimaryTabInfo);
		if (!HasIntegratedToolPalettes() && !HasToolkitBuilder())
		{

			ToolbarInfo.OnSpawnTab = FOnSpawnTab::CreateSP(SharedThis(this), &FModeToolkit::MakeModeToolbarTab);
			ToolbarInfo.TabLabel = LOCTEXT("ModesToolbarTab", "Mode Toolbar");
			ToolbarInfo.TabTooltip = LOCTEXT("LevelEditorModesToolbarTabTooltipText", "Opens a toolbar for the active editor mode");
			ModeUILayerPtr->SetModePanelInfo(UAssetEditorUISubsystem::VerticalToolbarID, ToolbarInfo);
		}
	}
}

void FModeToolkit::OnModeIDChanged(const FEditorModeID& InID, bool bIsEntering)
{
	if (const FEditorModeInfo* ModeInfo = GetEditorModeInfo())
	{
		if (ModeInfo->ID != NAME_None && ModeInfo->ID == InID && bIsEntering)
		{
			FToolkitManager::Get().RegisterNewToolkit(SharedThis(this));
		}
	}
}

const FEditorModeInfo* FModeToolkit::GetEditorModeInfo() const
{
	if (const FEdMode* EdMode = GetEditorMode())
	{
		return &EdMode->GetModeInfo();
	}
	else if (OwningEditorMode.IsValid())
	{
		return &OwningEditorMode->GetModeInfo();
	}

	return nullptr;
}

bool FModeToolkit::ShouldShowModeToolbar() const
{
	return ActiveToolBarRows.Num() > 0;
}

TSharedRef<SWidget> FModeToolkit::CreatePaletteWidget(TSharedPtr<FUICommandList> InCommandList, FName InToolbarCustomizationName, FName InPaletteName)
{
	FUniformToolBarBuilder ModeToolbarBuilder(InCommandList, FMultiBoxCustomization(InToolbarCustomizationName));
	ModeToolbarBuilder.SetStyle(&FAppStyle::Get(), "PaletteToolBar");

	BuildToolPalette(InPaletteName, ModeToolbarBuilder);

	return ModeToolbarBuilder.MakeWidget();
}

void FModeToolkit::SpawnOrUpdateModeToolbar()
{
	if (ShouldShowModeToolbar())
	{
		if (ModeToolbarTab.IsValid())
		{
			RebuildModeToolBar();
		}
		else if (ModeUILayer.IsValid())
		{
			ModeUILayer.Pin()->GetTabManager()->TryInvokeTab(UAssetEditorUISubsystem::VerticalToolbarID);
		}
	}
}

void FModeToolkit::RebuildModeToolBar()
{
	TSharedPtr<SDockTab> ToolbarTabPtr = ModeToolbarTab.Pin();
	if (ToolbarTabPtr  && HasToolkitBuilder())
	{
		ToolbarTabPtr->SetParentDockTabStackTabWellHidden(true);
	}

	// If the tab or box is not valid the toolbar has not been opened or has been closed by the user
	TSharedPtr<SVerticalBox> ModeToolbarBoxPinned = ModeToolbarBox.Pin();
	if (ModeToolbarTab.IsValid() && ModeToolbarBoxPinned)
	{
		ModeToolbarBoxPinned->ClearChildren();
		bool bExclusivePalettes = true;
		TSharedRef<SVerticalBox> ToolBoxVBox = SNew(SVerticalBox);

		TSharedRef< SUniformWrapPanel> PaletteTabBox = SNew(SUniformWrapPanel)
			.SlotPadding(FMargin(1.f, 2.f))
			.HAlign(HAlign_Left);
		TSharedRef< SWidgetSwitcher > PaletteSwitcher = SNew(SWidgetSwitcher);

		int32 PaletteCount = ActiveToolBarRows.Num();
		if (PaletteCount > 0)
		{
			for (int32 RowIdx = 0; RowIdx < PaletteCount; ++RowIdx)
			{
				const FEdModeToolbarRow& Row = ActiveToolBarRows[RowIdx];
				if (ensure(Row.ToolbarWidget.IsValid()))
				{
					TSharedRef<SWidget> PaletteWidget = Row.ToolbarWidget.ToSharedRef();

					bExclusivePalettes = HasExclusiveToolPalettes();

					if (!bExclusivePalettes)
					{
						ToolBoxVBox->AddSlot()
							.AutoHeight()
							.Padding(FMargin(2.0, 2.0))
							[
								SNew(SExpandableArea)
								.AreaTitle(Row.DisplayName)
								.AreaTitleFont(FAppStyle::Get().GetFontStyle("NormalFont"))
								.BorderImage(FAppStyle::Get().GetBrush("PaletteToolbar.ExpandableAreaHeader"))
								.BodyBorderImage(FAppStyle::Get().GetBrush("PaletteToolbar.ExpandableAreaBody"))
								.HeaderPadding(FMargin(4.f))
								.Padding(FMargin(4.0, 0.0))
								.BodyContent()
								[
									PaletteWidget
								]
							];
					}
					else
					{
						// Don't show Palette Tabs if there is only one
						if (PaletteCount > 1)
						{
							PaletteTabBox->AddSlot()
								[
									SNew(SCheckBox)
									.Style(FAppStyle::Get(), "ToolPalette.DockingTab")
									.OnCheckStateChanged_Lambda([PaletteSwitcher, Row, this](const ECheckBoxState) {
										PaletteSwitcher->SetActiveWidget(Row.ToolbarWidget.ToSharedRef());
										SetCurrentPalette(Row.PaletteName);
									}
								)
									.IsChecked_Lambda([PaletteSwitcher, PaletteWidget]() -> ECheckBoxState { return PaletteSwitcher->GetActiveWidget() == PaletteWidget ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
										[
											SNew(STextBlock)
											.Text(Row.DisplayName)
										]
								];
						}

						PaletteSwitcher->AddSlot()
							[
								PaletteWidget
							];
					}
				}
			}

			ModeToolbarBoxPinned->AddSlot()
				.AutoHeight()
				[
					SNew(SOverlay)
					+ SOverlay::Slot()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("ToolPalette.DockingWell"))
				]

			+ SOverlay::Slot()
				[
					PaletteTabBox
				]
				];

			ModeToolbarBoxPinned->AddSlot()
				.AutoHeight()
				.Padding(1.f)
				[
					SNew(SBox)
					.HeightOverride(PaletteSwitcher->GetNumWidgets() > 0 ? 45.f : 0.f)
				[
					PaletteSwitcher
				]
				];

			ModeToolbarBoxPinned->AddSlot()
				[
					SNew(SScrollBox)

					+ SScrollBox::Slot()
				[
					ToolBoxVBox
				]
				];

			ModeToolbarPaletteSwitcher = PaletteSwitcher;
		}
	}
}

#undef LOCTEXT_NAMESPACE
