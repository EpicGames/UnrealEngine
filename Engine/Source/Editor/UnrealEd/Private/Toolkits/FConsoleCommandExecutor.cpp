// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/FConsoleCommandExecutor.h"
#include "ConsoleSettings.h"
#include "EngineGlobals.h"
#include "Editor.h"
#include "Toolkits/GlobalEditorCommonCommands.h"
#include "Engine/LocalPlayer.h"
#include "GameFramework/GameStateBase.h"

#define LOCTEXT_NAMESPACE "SOutputLog"

FName FConsoleCommandExecutor::StaticName()
{
	static const FName CmdExecName = TEXT("Cmd");
	return CmdExecName;
}

FName FConsoleCommandExecutor::GetName() const
{
	return StaticName();
}

FText FConsoleCommandExecutor::GetDisplayName() const
{
	return LOCTEXT("ConsoleCommandExecutorDisplayName", "Cmd");
}

FText FConsoleCommandExecutor::GetDescription() const
{
	return LOCTEXT("ConsoleCommandExecutorDescription", "Execute Unreal Console Commands");
}

FText FConsoleCommandExecutor::GetHintText() const
{
	return LOCTEXT("ConsoleCommandExecutorHintText", "Enter Console Command");
}

void FConsoleCommandExecutor::GetSuggestedCompletions(const TCHAR* Input, TArray<FConsoleSuggestion>& Out)
{
	auto OnConsoleVariable = [&Out](const TCHAR *Name, IConsoleObject* CVar)
	{
		if (CVar->IsEnabled())
		{
			Out.Add(FConsoleSuggestion(Name, CVar->GetDetailedHelp().ToString()));
		}
	};

	IConsoleManager& ConsoleManager = IConsoleManager::Get();
	ConsoleManager.ForEachConsoleObjectThatContains(FConsoleObjectVisitor::CreateLambda(OnConsoleVariable), Input);
	for (const FString& CommandName : GetDefault<UConsoleSettings>()->GetFilteredManualAutoCompleteCommands(Input))
	{
		FString HelpString;
		// Try to find a console object for this entry in order to retrieve a help string if possible :
		const TCHAR* CommandNamePtr = *CommandName;
		if (IConsoleObject* CObj = ConsoleManager.FindConsoleObject(*FParse::Token(CommandNamePtr, /*UseEscape = */false), /*bTrackFrequentCalls = */false); CObj && CObj->IsEnabled())
		{
			HelpString = CObj->GetDetailedHelp().ToString();
		}
		Out.Add(FConsoleSuggestion(CommandName, HelpString));
	}
}

void FConsoleCommandExecutor::GetExecHistory(TArray<FString>& Out)
{
	IConsoleManager::Get().GetConsoleHistory(TEXT(""), Out);
}

bool FConsoleCommandExecutor::Exec(const TCHAR* Input)
{
	IConsoleManager::Get().AddConsoleHistoryEntry(TEXT(""), Input);

	int32 Len = FCString::Strlen(Input);
	TArray<TCHAR> Buffer;
	Buffer.AddZeroed(Len+1);

	bool bHandled = false;
	const TCHAR* ParseCursor = Input;
	while (FParse::Line(&ParseCursor, Buffer.GetData(), Buffer.Num()))
	{
		bHandled = ExecInternal(Buffer.GetData()) || bHandled;
	}

	// return true if we successfully executed any of the commands 
	return bHandled;
}

bool FConsoleCommandExecutor::ExecInternal(const TCHAR* Input) const
{
	bool bWasHandled = false;
	UWorld* World = nullptr;
	UWorld* OldWorld = nullptr;

	// The play world needs to handle these commands if it exists
	if (GIsEditor && GEditor->PlayWorld && !GIsPlayInEditorWorld)
	{
		World = GEditor->PlayWorld;
		OldWorld = SetPlayInEditorWorld(GEditor->PlayWorld);
	}

	ULocalPlayer* Player = GEngine->GetDebugLocalPlayer();
	if (Player)
	{
		UWorld* PlayerWorld = Player->GetWorld();
		if (!World)
		{
			World = PlayerWorld;
		}
		bWasHandled = Player->Exec(PlayerWorld, Input, *GLog);
	}

	if (!World)
	{
		World = GEditor->GetEditorWorldContext().World();
	}
	if (World)
	{
		if (!bWasHandled)
		{
			AGameModeBase* const GameMode = World->GetAuthGameMode();
			AGameStateBase* const GameState = World->GetGameState();
			if (GameMode && GameMode->ProcessConsoleExec(Input, *GLog, nullptr))
			{
				bWasHandled = true;
			}
			else if (GameState && GameState->ProcessConsoleExec(Input, *GLog, nullptr))
			{
				bWasHandled = true;
			}
		}

		if (!bWasHandled && !Player)
		{
			if (GIsEditor)
			{
				bWasHandled = GEditor->Exec(World, Input, *GLog);
			}
			else
			{
				bWasHandled = GEngine->Exec(World, Input, *GLog);
			}
		}
	}

	// Restore the old world of there was one
	if (OldWorld)
	{
		RestoreEditorWorld(OldWorld);
	}

	return bWasHandled;
}

bool FConsoleCommandExecutor::AllowHotKeyClose() const
{
	return true;
}

bool FConsoleCommandExecutor::AllowMultiLine() const
{
	return true;
}

#undef LOCTEXT_NAMESPACE