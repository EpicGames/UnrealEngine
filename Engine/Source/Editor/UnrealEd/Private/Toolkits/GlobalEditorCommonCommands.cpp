// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/GlobalEditorCommonCommands.h"

#include "FindInBlueprints.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Interfaces/IMainFrameModule.h"
#include "ISettingsModule.h"
#include "Layout/WidgetPath.h"
#include "LevelEditor.h"
#include "LevelEditorActions.h"
#include "Modules/ModuleManager.h"
#include "OutputLogModule.h"
#include "StatusBarSubsystem.h"
#include "Styling/AppStyle.h"
#include "Toolkits/SGlobalOpenAssetDialog.h"
#include "Toolkits/SGlobalTabSwitchingDialog.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWindow.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "GlobalEditorCommonCommands"

//////////////////////////////////////////////////////////////////////////
// FGlobalEditorCommonCommands

UE_DEFINE_TCOMMANDS(FGlobalEditorCommonCommands)

FGlobalEditorCommonCommands::FGlobalEditorCommonCommands()
	: TCommands<FGlobalEditorCommonCommands>(TEXT("SystemWideCommands"), NSLOCTEXT("Contexts", "SystemWideCommands", "System-wide"), NAME_None, FAppStyle::GetAppStyleSetName())
{
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner("GlobalAssetPicker", FOnSpawnTab::CreateStatic(&FGlobalEditorCommonCommands::SpawnAssetPicker))
		.SetDisplayName(LOCTEXT("AssetPickerTabTitle", "Open Asset"))
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "SystemWideCommands.SummonOpenAssetDialog"))
		.SetMenuType(ETabSpawnerMenuType::Hidden);
}


FGlobalEditorCommonCommands::~FGlobalEditorCommonCommands()
{
	FGlobalTabmanager::Get()->UnregisterTabSpawner("GlobalAssetPicker");
}

void FGlobalEditorCommonCommands::RegisterCommands()
{
	UI_COMMAND(SummonControlTabNavigation, "Tab Navigation", "Summons a list of open assets and tabs, and navigates forwards in it.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::Tab));
	UI_COMMAND(SummonControlTabNavigationAlternate, "Tab Navigation", "Summons a list of open assets and tabs, and navigates forward in its.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Command, EKeys::Tab));

	UI_COMMAND(SummonControlTabNavigationBackwards, "Tab Navigation Backwards", "Summons a list of open assets and tabs, and navigates backwards in it.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::Tab));
	UI_COMMAND(SummonControlTabNavigationBackwardsAlternate, "Tab Navigation Backwards", "Summons a list of open assets and tabs, and navigates backwards in it.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Command | EModifierKey::Shift, EKeys::Tab));

	UI_COMMAND(SummonOpenAssetDialog, "Open Asset...", "Summons an asset picker", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::P));
	UI_COMMAND(SummonOpenAssetDialogAlternate, "Open Asset...", "Summons an asset picker", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt | EModifierKey::Shift, EKeys::O));
	UI_COMMAND(FindInContentBrowser, "Browse to Asset(s)", "Browses and selects the associated asset(s) in the most recently used Content Browser (summoning one if necessary)", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::B));

	UI_COMMAND(OpenConsoleCommandBox, "Open Console Command Box", "Opens an edit box where you can type in a console command", EUserInterfaceActionType::Button, FInputChord(EKeys::Tilde));
	UI_COMMAND(SelectNextConsoleExecutor, "Iterate Console Executor", "Iterates through active Console Executors (Python, Cmd, etc.)", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::Tilde));
	
	UI_COMMAND(OpenOutputLogDrawer, "Open Output Log Drawer", "Opens the output log drawer from the active asset editor status bar", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::Tilde));

	UI_COMMAND(OpenDocumentation, "Open Documentation...", "Opens documentation for this tool", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::F1));

	UI_COMMAND(OpenLevel, "Open Level...", "Loads an existing level", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::O));

#if PLATFORM_MAC
	// On mac command and ctrl are automatically swapped. Command + Space is spotlight search so we use ctrl+space on mac to avoid the conflict
	UI_COMMAND(OpenContentBrowserDrawer, "Open Content Browser Drawer", "Opens the content browser drawer from the status bar and focuses the search field", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Command, EKeys::SpaceBar));
#else
	UI_COMMAND(OpenContentBrowserDrawer, "Open Content Browser Drawer", "Opens the content browser drawer from the status bar and focuses the search field", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::SpaceBar));
#endif

	UI_COMMAND(OpenFindInAllBlueprints, "Open Find in all Blueprints", "Opens the Find in all Blueprints dialogue", EUserInterfaceActionType::Button, FInputChord());

#if PLATFORM_MAC
	EModifierKey::Type ModifierKey = EModifierKey::Command;
#else
	EModifierKey::Type ModifierKey = EModifierKey::Control;
#endif

	UI_COMMAND(OpenEditorPreferences, "Editor Preferences...", "Configure the behavior and features of the Unreal Editor.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt | ModifierKey, EKeys::E));
	UI_COMMAND(OpenKeyboardShortcuts, "Keyboard Shortcuts...", "Configure the keyboard shortcuts of the Unreal Editor.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt | ModifierKey, EKeys::K));
	UI_COMMAND(OpenProjectSettings, "Project Settings...", "Change the settings of the currently loaded project.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt | ModifierKey, EKeys::P));
}

void FGlobalEditorCommonCommands::MapActions(TSharedRef<FUICommandList>& ToolkitCommands)
{
	Register();
	
	FGlobalEditorCommonCommands& GlobalEditorCommands = Get();

	ToolkitCommands->MapAction(
		GlobalEditorCommands.SummonControlTabNavigation,
		FExecuteAction::CreateStatic(&FGlobalEditorCommonCommands::OnPressedCtrlTab, GlobalEditorCommands.SummonControlTabNavigation));

	ToolkitCommands->MapAction(
		GlobalEditorCommands.SummonControlTabNavigationAlternate,
		FExecuteAction::CreateStatic(&FGlobalEditorCommonCommands::OnPressedCtrlTab, GlobalEditorCommands.SummonControlTabNavigationAlternate));

	ToolkitCommands->MapAction(
		GlobalEditorCommands.SummonControlTabNavigationBackwards,
		FExecuteAction::CreateStatic(&FGlobalEditorCommonCommands::OnPressedCtrlTab, GlobalEditorCommands.SummonControlTabNavigationBackwards));

	ToolkitCommands->MapAction(
		GlobalEditorCommands.SummonControlTabNavigationBackwardsAlternate,
		FExecuteAction::CreateStatic(&FGlobalEditorCommonCommands::OnPressedCtrlTab, GlobalEditorCommands.SummonControlTabNavigationBackwardsAlternate));

	ToolkitCommands->MapAction(
		GlobalEditorCommands.SummonOpenAssetDialog,
		FExecuteAction::CreateStatic(&FGlobalEditorCommonCommands::OnSummonedAssetPicker));

	ToolkitCommands->MapAction(
		GlobalEditorCommands.SummonOpenAssetDialogAlternate,
		FExecuteAction::CreateStatic(&FGlobalEditorCommonCommands::OnSummonedAssetPicker));

	ToolkitCommands->MapAction(
		GlobalEditorCommands.OpenConsoleCommandBox,
		FExecuteAction::CreateStatic(&FGlobalEditorCommonCommands::OnSummonedConsoleCommandBox));

	ToolkitCommands->MapAction(
		GlobalEditorCommands.OpenContentBrowserDrawer,
		FExecuteAction::CreateStatic(&FGlobalEditorCommonCommands::OnOpenContentBrowserDrawer));

	ToolkitCommands->MapAction(
		GlobalEditorCommands.OpenOutputLogDrawer,
		FExecuteAction::CreateStatic(&FGlobalEditorCommonCommands::OnOpenOutputLogDrawer));

	ToolkitCommands->MapAction(
		GlobalEditorCommands.OpenLevel,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::OpenLevel),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::OpenLevel_CanExecute));

	ToolkitCommands->MapAction(
		GlobalEditorCommands.OpenFindInAllBlueprints,
		FExecuteAction::CreateStatic(&FGlobalEditorCommonCommands::OnOpenFindInAllBlueprints));
	
	// Get editor default category and section
	FName CategoryName = NAME_None; // "All settings"
	FName SectionName = NAME_None; // "All settings"
	if (IMainFrameModule* MainFrameModule = FModuleManager::GetModulePtr<IMainFrameModule>("MainFrame"))
	{
		MainFrameModule->GetEditorSettingsDefaultSelectionOverride(CategoryName, SectionName);
	}

	ToolkitCommands->MapAction(
		GlobalEditorCommands.OpenEditorPreferences,
		FExecuteAction::CreateStatic(&FGlobalEditorCommonCommands::OpenSettings, FName("Editor"), CategoryName, SectionName));

	ToolkitCommands->MapAction(
		GlobalEditorCommands.OpenKeyboardShortcuts,
		FExecuteAction::CreateStatic(&FGlobalEditorCommonCommands::OpenSettings, FName("InputBinding"), FName("Shortcuts"), FName("InputBindings")));

	CategoryName = NAME_None; // "All settings"
	SectionName = NAME_None; // "All settings"
	ToolkitCommands->MapAction(
		GlobalEditorCommands.OpenProjectSettings,
		FExecuteAction::CreateStatic(&FGlobalEditorCommonCommands::OpenSettings, FName("Project"), CategoryName, SectionName));
}

void FGlobalEditorCommonCommands::OnPressedCtrlTab(TSharedPtr<FUICommandInfo> TriggeringCommand)
{
	if (!SGlobalTabSwitchingDialog::IsAlreadyOpen())
	{
		const FVector2D TabListSize(700.0f, 486.0f);

		// Create the contents of the popup
		TSharedRef<SWidget> ActualWidget = SNew(SGlobalTabSwitchingDialog, TabListSize, *TriggeringCommand->GetFirstValidChord());

		OpenPopupMenu(ActualWidget, TabListSize);
	}
}

TSharedRef<SDockTab> FGlobalEditorCommonCommands::SpawnAssetPicker(const FSpawnTabArgs& InArgs)
{
	const FVector2D AssetPickerSize(600.0f, 586.0f);
	// Create the contents of the popup
	TSharedRef<SWidget> ActualWidget = SNew(SGlobalOpenAssetDialog, AssetPickerSize);

	/**
	 * The Global Asset Picker has been changed to open as a tab. This is because it would close on selecting some menu and sub menu
	 * options due to it being opened as a menu before, which would lead to the selected menu option doing nothing. Furthermore,
	 * There was also weird behavior when you had another asset picker in a menu open and tried to open the global asset picker
	 * where the other picker would close but leave any parent menus hanging forever with no way to close them
	 */

	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab);

	DockTab->SetContent(ActualWidget);

	return DockTab;

}

void FGlobalEditorCommonCommands::OnSummonedAssetPicker()
{
	if (TSharedPtr<SDockTab> AssetPickerTab = FGlobalTabmanager::Get()->FindExistingLiveTab(FTabId("GlobalAssetPicker")))
	{
		AssetPickerTab->RequestCloseTab();
	}
	else
	{
		FGlobalTabmanager::Get()->TryInvokeTab(FTabId("GlobalAssetPicker"));
	}
	
}

TSharedPtr<IMenu> FGlobalEditorCommonCommands::OpenPopupMenu(TSharedRef<SWidget> WindowContents, const FVector2D& PopupDesiredSize)
{
	// Determine where the pop-up should open
	TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().GetActiveTopLevelRegularWindow();
	FVector2D WindowPosition = FSlateApplication::Get().GetCursorPos();
	if (!ParentWindow.IsValid())
	{
		TSharedPtr<SDockTab> LevelEditorTab = FModuleManager::Get().GetModuleChecked<FLevelEditorModule>("LevelEditor").GetLevelEditorTab();
		ParentWindow = LevelEditorTab->GetParentWindow();
		check(ParentWindow.IsValid());
	}

	if (ParentWindow.IsValid())
	{
		FSlateRect ParentMonitorRect = ParentWindow->GetFullScreenInfo();
		const FVector2D MonitorCenter((ParentMonitorRect.Right + ParentMonitorRect.Left) * 0.5f, (ParentMonitorRect.Top + ParentMonitorRect.Bottom) * 0.5f);
		WindowPosition = MonitorCenter - PopupDesiredSize * 0.5f;

		// Open the pop-up
		FPopupTransitionEffect TransitionEffect(FPopupTransitionEffect::None);
		return FSlateApplication::Get().PushMenu(ParentWindow.ToSharedRef(), FWidgetPath(), WindowContents, WindowPosition, TransitionEffect, /*bFocusImmediately=*/ true);
	}

	return TSharedPtr<IMenu>();
}

void FGlobalEditorCommonCommands::OpenSettings(FName InContainerName, FName InCategoryName, FName InSectionName)
{
	FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer(InContainerName, InCategoryName, InSectionName);
}

static void CloseDebugConsole()
{
	FOutputLogModule& OutputLogModule = FModuleManager::LoadModuleChecked< FOutputLogModule >(TEXT("OutputLog"));
	OutputLogModule.CloseDebugConsole();
}

void FGlobalEditorCommonCommands::OnSummonedConsoleCommandBox()
{
	TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().GetActiveTopLevelWindow();
	if (!ParentWindow.IsValid())
	{
		if (TSharedPtr<SDockTab> ActiveTab = FGlobalTabmanager::Get()->GetActiveTab())
		{
			if (TSharedPtr<FTabManager> ActiveTabManager = ActiveTab->GetTabManagerPtr())
			{
				if (TSharedPtr<SDockTab> ActiveMajorTab = FGlobalTabmanager::Get()->GetMajorTabForTabManager(ActiveTabManager.ToSharedRef()))
				{
					ParentWindow = ActiveMajorTab->GetParentWindow();
				}
			}
		}
	}

	if (ParentWindow.IsValid() && ParentWindow->GetType() == EWindowType::Normal)
	{
		TSharedRef<SWindow> WindowRef = ParentWindow.ToSharedRef();
		FOutputLogModule& OutputLogModule = FModuleManager::LoadModuleChecked<FOutputLogModule>(TEXT("OutputLog"));

		if (!GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->ToggleDebugConsole(ParentWindow.ToSharedRef()))
		{
			// A status bar was not found, pop open a floating window instead
			FDebugConsoleDelegates Delegates;
			Delegates.OnConsoleCommandExecuted = FSimpleDelegate::CreateStatic(&CloseDebugConsole);
			Delegates.OnCloseConsole = FSimpleDelegate::CreateStatic(&CloseDebugConsole);

			OutputLogModule.ToggleDebugConsoleForWindow(WindowRef, EDebugConsoleStyle::Compact, Delegates);
		}
	}
}

void FGlobalEditorCommonCommands::OnOpenContentBrowserDrawer()
{
	GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->ToggleContentBrowserDrawer();
}

void FGlobalEditorCommonCommands::OnOpenOutputLogDrawer()
{
	GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->OpenOutputLogDrawer();
}

void FGlobalEditorCommonCommands::OnOpenFindInAllBlueprints()
{
	if (const TSharedPtr<SFindInBlueprints> Dialogue = FFindInBlueprintSearchManager::Get().GetGlobalFindResults())
	{
		Dialogue->FocusForUse(false);
	}
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
