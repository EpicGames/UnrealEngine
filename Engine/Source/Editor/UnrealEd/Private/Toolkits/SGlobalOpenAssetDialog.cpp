// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/SGlobalOpenAssetDialog.h"
#include "AssetRegistry/AssetData.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SBoxPanel.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Docking/SDockTab.h"

#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"

#include "Subsystems/AssetEditorSubsystem.h"
#include "Toolkits/GlobalEditorCommonCommands.h"
#include "ToolMenus.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "SGlobalOpenAssetDialog"

namespace GlobalOpenAssetDialogUtils
{
	static const FName ContextMenuName("GlobalOpenAssetDialog.ContextMenu");
}

//////////////////////////////////////////////////////////////////////////
// SGlobalOpenAssetDialog

void SGlobalOpenAssetDialog::Construct(const FArguments& InArgs, FVector2D InSize)
{
	IContentBrowserSingleton& ContentBrowser = GetContentBrowser();

	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.OnAssetDoubleClicked = FOnAssetSelected::CreateSP(this, &SGlobalOpenAssetDialog::OnAssetSelectedFromPicker);
	AssetPickerConfig.OnAssetEnterPressed = FOnAssetEnterPressed::CreateSP(this, &SGlobalOpenAssetDialog::OnPressedEnterOnAssetsInPicker);
	AssetPickerConfig.OnGetAssetContextMenu = FOnGetAssetContextMenu::CreateSP(this, &SGlobalOpenAssetDialog::OnGetAssetContextMenu);
	AssetPickerConfig.GetCurrentSelectionDelegates.Add(&GetCurrentSelectionDelegate);
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
	AssetPickerConfig.bAllowNullSelection = false;
	AssetPickerConfig.bShowBottomToolbar = true;
	AssetPickerConfig.bAutohideSearchBar = false;
	AssetPickerConfig.bCanShowClasses = false;
	AssetPickerConfig.bAddFilterUI = true;
	AssetPickerConfig.SaveSettingsName = TEXT("GlobalAssetPicker");

	if (!UToolMenus::Get()->IsMenuRegistered(GlobalOpenAssetDialogUtils::ContextMenuName))
	{
		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(GlobalOpenAssetDialogUtils::ContextMenuName);
		FToolMenuSection& AssetSection = Menu->AddSection("Asset", LOCTEXT("AssetSectionLabel", "Asset"));
		AssetSection.AddMenuEntry(FGlobalEditorCommonCommands::Get().FindInContentBrowser);
	}

	Commands = MakeShared<FUICommandList>();
	Commands->MapAction(FGlobalEditorCommonCommands::Get().FindInContentBrowser, FUIAction(
		FExecuteAction::CreateSP(this, &SGlobalOpenAssetDialog::FindInContentBrowser),
		FCanExecuteAction::CreateSP(this, &SGlobalOpenAssetDialog::AreAnyAssetsSelected)
	));

	ChildSlot
	[
		SNew(SBox)
		.WidthOverride(static_cast<float>(InSize.X))
		.HeightOverride(static_cast<float>(InSize.Y))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				ContentBrowser.CreateAssetPicker(AssetPickerConfig)
			]
		]
	];
}


FReply SGlobalOpenAssetDialog::OnPreviewKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		RequestCloseAssetPicker();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SGlobalOpenAssetDialog::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (Commands->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

void SGlobalOpenAssetDialog::OnAssetSelectedFromPicker(const FAssetData& AssetData)
{
	if (UObject* ObjectToEdit = AssetData.GetAsset())
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ObjectToEdit);
	}

	RequestCloseAssetPicker();
}

void SGlobalOpenAssetDialog::OnPressedEnterOnAssetsInPicker(const TArray<FAssetData>& SelectedAssets)
{
	for (auto AssetIt = SelectedAssets.CreateConstIterator(); AssetIt; ++AssetIt)
	{
		if (UObject* ObjectToEdit = AssetIt->GetAsset())
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ObjectToEdit);
		}
	}

	RequestCloseAssetPicker();
}

void SGlobalOpenAssetDialog::RequestCloseAssetPicker()
{
	if (TSharedPtr<SDockTab> AssetPickerTab = FGlobalTabmanager::Get()->FindExistingLiveTab(FTabId("GlobalAssetPicker")))
	{
		AssetPickerTab->RequestCloseTab();
	}
}

TSharedPtr<SWidget> SGlobalOpenAssetDialog::OnGetAssetContextMenu(const TArray<FAssetData>& SelectedAssets) const
{
	if (SelectedAssets.IsEmpty())
	{
		return nullptr;
	}

	return UToolMenus::Get()->GenerateWidget(GlobalOpenAssetDialogUtils::ContextMenuName, FToolMenuContext(Commands));
}

IContentBrowserSingleton& SGlobalOpenAssetDialog::GetContentBrowser() const
{
	return FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser")).Get();
}

void SGlobalOpenAssetDialog::FindInContentBrowser()
{
	GetContentBrowser().SyncBrowserToAssets(GetCurrentSelectionDelegate.Execute());
	RequestCloseAssetPicker();
}

bool SGlobalOpenAssetDialog::AreAnyAssetsSelected() const
{
	return !GetCurrentSelectionDelegate.Execute().IsEmpty();
}


//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
