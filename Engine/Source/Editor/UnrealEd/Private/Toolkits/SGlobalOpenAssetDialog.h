// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ContentBrowserDelegates.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

//////////////////////////////////////////////////////////////////////////
// SGlobalOpenAssetDialog

class SGlobalOpenAssetDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SGlobalOpenAssetDialog){}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, FVector2D InSize);

	// SWidget interface
	virtual FReply OnPreviewKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	// End of SWidget interface

protected:
	void OnAssetSelectedFromPicker(const struct FAssetData& AssetData);
	void OnPressedEnterOnAssetsInPicker(const TArray<struct FAssetData>& SelectedAssets);
	void RequestCloseAssetPicker();

	TSharedPtr<SWidget> OnGetAssetContextMenu(const TArray<FAssetData>& SelectedAssets) const;
	class IContentBrowserSingleton& GetContentBrowser() const;

	void FindInContentBrowser();
	bool AreAnyAssetsSelected() const;
	
	FGetCurrentSelectionDelegate GetCurrentSelectionDelegate;
	TSharedPtr<FUICommandList> Commands;
};
