// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/SGlobalTabSwitchingDialog.h"

#include "Editor.h"
#include "EngineGlobals.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Interfaces/IMainFrameModule.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#if PLATFORM_MAC
#include "Mac/MacApplication.h"
#endif

#define LOCTEXT_NAMESPACE "SGlobalTabSwitchingDialog"

//////////////////////////////////////////////////////////////////////////
// FTabSwitchingListItemBase

class FTabSwitchingListItemBase
{
public:
	FTabSwitchingListItemBase()
		: LastAccessTime(0.0)
	{
	}

	virtual ~FTabSwitchingListItemBase() {}

	virtual TSharedRef<SWidget> CreateWidget(TSharedPtr<class FAssetThumbnailPool> AssetThumbnailPool)
	{
		return SNullWidget::NullWidget;
	}

	virtual FText GetTypeString() const
	{
		return FText::GetEmpty();
	}

	virtual FText GetPathString() const
	{
		return FText::GetEmpty();
	}

	virtual void ActivateTab() { }

	virtual void ShowInContentBrowser()
	{
	}

	virtual TSharedPtr<FTabManager> GetAssociatedTabManager()
	{
		return TSharedPtr<FTabManager>();
	}
public:
	double LastAccessTime;
};

//////////////////////////////////////////////////////////////////////////
// FTabSwitchingListItem_Asset

class FTabSwitchingListItem_Asset : public FTabSwitchingListItemBase
{
public:
	FTabSwitchingListItem_Asset(UObject* InAsset)
		: MyAsset(InAsset)
	{
		if (IAssetEditorInstance* EditorInstance = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(MyAsset, /*bFocusIfOpen=*/ false))
		{
			LastAccessTime = EditorInstance->GetLastActivationTime();
		}
	}

	virtual TSharedRef<SWidget> CreateWidget(TSharedPtr<class FAssetThumbnailPool> AssetThumbnailPool) override
	{
		// Create a label for the asset name
		const bool bDirtyState = MyAsset->GetOutermost()->IsDirty();
		FFormatNamedArguments Args;
		Args.Add(TEXT("AssetName"), FText::AsCultureInvariant(MyAsset->GetName()));
		Args.Add(TEXT("DirtyState"), bDirtyState ? LOCTEXT("AssetModified", " [Modified]") : FText::GetEmpty());
		FText AssetText = FText::Format(LOCTEXT("AssetEntryLabel", "{AssetName}{DirtyState}"), Args);

		// Create a thumbnail to represent the asset type
		constexpr int32 ThumbnailSize = 32;

		FAssetData AssetData(MyAsset);
		Thumbnail = MakeShareable(new FAssetThumbnail(MyAsset, ThumbnailSize, ThumbnailSize, AssetThumbnailPool));

		FAssetThumbnailConfig ThumbnailConfig;

		return SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(5.f)
			[
				SNew(SBox)
				.WidthOverride(static_cast<float>(ThumbnailSize))
				.HeightOverride(static_cast<float>(ThumbnailSize))
				[
					Thumbnail->MakeThumbnailWidget(ThumbnailConfig)
				]
			]
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			.Padding(5.f)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "ControlTabMenu.AssetNameStyle")
				.Text(AssetText)
			];
	}

	virtual void ShowInContentBrowser() override
	{
		TArray<UObject*> ObjectsToSync;
		ObjectsToSync.Add(MyAsset);
		GEditor->SyncBrowserToObjects(ObjectsToSync);
	}

	virtual FText GetTypeString() const override
	{
		return MyAsset->GetClass()->GetDisplayNameText();
	}

	virtual FText GetPathString() const override
	{
		return FText::AsCultureInvariant(MyAsset->GetOutermost()->GetName());
	}

	virtual void ActivateTab() override
	{
		if (const TSharedPtr<FTabManager> AssetEditorTabManager = GetAssociatedTabManager())
		{
			if (const TSharedPtr<SDockTab> DockTab = AssetEditorTabManager->GetOwnerTab())
			{
				// Don't use DrawAttention since it steals focus
				DockTab->ActivateInParent(ETabActivationCause::SetDirectly);
			}
		}
	}

	virtual TSharedPtr<FTabManager> GetAssociatedTabManager() override
	{
		IAssetEditorInstance* Instance = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(MyAsset, /*bFocusIfOpen=*/ false);
		if (Instance)
		{
			return Instance->GetAssociatedTabManager();
		}

		return nullptr;
	}

protected:
	UObject* MyAsset;
	TSharedPtr<class FAssetThumbnail> Thumbnail;
};

//////////////////////////////////////////////////////////////////////////
// FTabSwitchingListItem_World

class FTabSwitchingListItem_World : public FTabSwitchingListItem_Asset
{
public:
	static TSharedPtr<FTabSwitchingListItem_World> MakeWorldItem()
	{
		UWorld* MyWorld = nullptr;
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			if (Context.WorldType == EWorldType::PIE)
			{
				MyWorld = Context.World();
				break;
			}
			else if (Context.WorldType == EWorldType::Editor)
			{
				MyWorld = Context.World();
			}
		}

		check(MyWorld != nullptr);
		return MakeShareable(new FTabSwitchingListItem_World(MyWorld));
	}

	virtual void ActivateTab() override
	{
		if (const TSharedPtr<FTabManager> LevelEditorTabManager = GetAssociatedTabManager())
		{
			if (const TSharedPtr<SDockTab> DockTab = LevelEditorTabManager->GetOwnerTab())
			{
				// Don't use DrawAttention since it steals focus
				DockTab->ActivateInParent(ETabActivationCause::SetDirectly);
			}
		}
	}

	virtual TSharedPtr<FTabManager> GetAssociatedTabManager() override
	{
		return FModuleManager::Get().GetModuleChecked<FLevelEditorModule>("LevelEditor").GetLevelEditorTabManager();
	}

protected:
	FTabSwitchingListItem_World(UWorld* InWorld)
		: FTabSwitchingListItem_Asset(InWorld)
	{
		const TSharedPtr<SDockTab> LevelEditorTabPtr = FModuleManager::Get().GetModuleChecked<FLevelEditorModule>("LevelEditor").GetLevelEditorTab();
		LastAccessTime = LevelEditorTabPtr->GetLastActivationTime();
	}
};

class FTabSwitchingListItem_DockTab : public FTabSwitchingListItemBase
{
public:
	FTabSwitchingListItem_DockTab(const TSharedPtr<SDockTab>& InTab)
		: DockTab(InTab)
	{
	}

	virtual TSharedRef<SWidget> CreateWidget(TSharedPtr<class FAssetThumbnailPool> AssetThumbnailPool) override
	{
		if (!DockTab.IsValid())
		{
			return SNullWidget::NullWidget;
		}

		constexpr int32 ThumbnailSize = 32;

		return SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(5.f)
			[
				SNew(SBox)
				.WidthOverride(static_cast<float>(ThumbnailSize))
				.HeightOverride(static_cast<float>(ThumbnailSize))
				[
					SNew(SImage)
					.Image(DockTab->GetTabIcon())
				]
			]
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			.Padding(5.f)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "ControlTabMenu.AssetNameStyle")
				.Text(DockTab->GetTabLabel())
			];
	}

	virtual void ShowInContentBrowser() override
	{
		// Do nothing here
	}

	virtual FText GetTypeString() const override
	{
		return DockTab.IsValid() ? DockTab->GetTabLabel() : FText::GetEmpty();
	}

	virtual FText GetPathString() const override
	{
		return FText::GetEmpty();
	}

	virtual void ActivateTab() override
	{
		if (DockTab.IsValid())
		{
			// Don't use DrawAttention since it steals focus
			DockTab->ActivateInParent(ETabActivationCause::SetDirectly);
		}
	}

	virtual TSharedPtr<FTabManager> GetAssociatedTabManager() override
	{
		return DockTab.IsValid() ? DockTab->GetTabManagerPtr() : nullptr;
	}

protected:
	TSharedPtr<SDockTab> DockTab;
};

//////////////////////////////////////////////////////////////////////////
// FTabSwitchingListItem_Asset

bool SGlobalTabSwitchingDialog::bIsAlreadyOpen = false;

SGlobalTabSwitchingDialog::~SGlobalTabSwitchingDialog()
{
	bIsAlreadyOpen = false;

#if PLATFORM_MAC
	MacApplication->SetIsRightClickEmulationEnabled(true);
#endif
}

SGlobalTabSwitchingDialog::FTabListItemPtr SGlobalTabSwitchingDialog::GetMainTabListSelectedItem() const
{
	TArray<FTabListItemPtr> SelectedItems = MainTabsListWidget->GetSelectedItems();
	if (SelectedItems.Num() > 0)
	{
		return SelectedItems[0];
	}
	else
	{
		return FTabListItemPtr();
	}
}

FReply SGlobalTabSwitchingDialog::OnBrowseToSelectedAsset()
{
	FTabListItemPtr SelectedItem = GetMainTabListSelectedItem();
	if (SelectedItem.IsValid())
	{
		SelectedItem->ShowInContentBrowser();

		constexpr bool ActivateTab = false;
		DismissDialog(ActivateTab);
	}
	return FReply::Handled();
}

void SGlobalTabSwitchingDialog::OnMainTabListSelectionChanged(FTabListItemPtr InItem, ESelectInfo::Type SelectInfo)
{
	TSharedRef<SWidget> NewTopContents = SNullWidget::NullWidget;
	TSharedRef<SWidget> NewBottomContents = SNullWidget::NullWidget;
	TSharedRef<SWidget> NewToolTabsContent = SNullWidget::NullWidget;

	TArray<FTabListItemPtr> SelectedItems = MainTabsListWidget->GetSelectedItems();

	if (SelectedItems.Num() > 0)
	{
		const FTabListItemPtr SelectedItem = SelectedItems[0];

		const FText ItemPathString = SelectedItem->GetPathString();
		NewTopContents = SelectedItem->CreateWidget(UThumbnailManager::Get().GetSharedThumbnailPool());

		NewBottomContents =
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "HoverOnlyHyperlinkButton")
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.Visibility(ItemPathString.IsEmpty() ? EVisibility::Hidden : EVisibility::Visible)
				.OnClicked(this, &SGlobalTabSwitchingDialog::OnBrowseToSelectedAsset)
				.ToolTipText(LOCTEXT("BrowseButtonToolTipText", "Browse to Asset in Content Browser"))
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("Icons.Search"))
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), "ControlTabMenu.AssetPathStyle")
						.Text(ItemPathString)
					]
				]
			]
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNullWidget::NullWidget
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "ControlTabMenu.AssetTypeStyle")
				.Text(SelectedItem->GetTypeString())
			];
	}

	NewTabItemToActivateDisplayBox->SetContent(NewTopContents);
	NewTabItemToActivatePathBox->SetContent(NewBottomContents);
}

void SGlobalTabSwitchingDialog::OnMainTabListItemClicked(FTabListItemPtr InItem)
{
	DismissDialog(InItem.IsValid());
}

void SGlobalTabSwitchingDialog::CycleSelection(bool bForwards)
{
	// This is done here each time in case someone clicks off of the selected item (and to prime the pump at startup),
	// otherwise the code below wouldn't cycle back into an item if nothing was selected
	if ((MainTabsListWidget->GetNumItemsSelected() == 0) && (MainTabsListDataSource.Num() > 0))
	{
		MainTabsListWidget->SetSelection(MainTabsListDataSource[0]);
	}

	// Move to the next/previous item
	FTabListItemPtr OldSelectedItem = GetMainTabListSelectedItem();
	if (OldSelectedItem.IsValid())
	{
		int32 OldSelectionIndex;
		if (MainTabsListDataSource.Find(OldSelectedItem, /*out*/ OldSelectionIndex))
		{
			const int32 NewSelectionIndex = (OldSelectionIndex + MainTabsListDataSource.Num() + (bForwards ? 1 : -1)) % MainTabsListDataSource.Num();
			if (NewSelectionIndex != OldSelectionIndex)
			{
				FTabListItemPtr NewSelectedItem = MainTabsListDataSource[NewSelectionIndex];
				MainTabsListWidget->SetSelection(NewSelectedItem);
				MainTabsListWidget->RequestScrollIntoView(NewSelectedItem);

				// Activate tab without giving focus, this will allow input events to still flow through this dialog widget
				NewSelectedItem->ActivateTab();
			}
		}
	}
}

void SGlobalTabSwitchingDialog::DismissDialog(bool bInActivateTab)
{
	if (bInActivateTab)
	{
		const FTabListItemPtr SelectedItem = GetMainTabListSelectedItem();
		if (SelectedItem.IsValid())
		{
			SelectedItem->ActivateTab();
		}
	}

	FSlateApplication::Get().DismissAllMenus();
}

void SGlobalTabSwitchingDialog::Construct(const FArguments& InArgs, FVector2D InSize, FInputChord InTriggerChord)
{
	check(!bIsAlreadyOpen);
	bIsAlreadyOpen = true;

#if PLATFORM_MAC
	// On Mac we emulate right click with ctrl+left click. This needs to be disabled for tab navigator, so that users can click on its widgets while they keep the ctrl key pressed
	MacApplication->SetIsRightClickEmulationEnabled(false);
#endif

	TriggerChord = InTriggerChord;

	const UEditorPerProjectUserSettings* EditorUserSettings = GetDefault<UEditorPerProjectUserSettings>();

	ensureMsgf(EditorUserSettings != nullptr, TEXT("Editor Per Project User Settings cannot be retrieved"));

	if (!EditorUserSettings || EditorUserSettings->TabSwitchingBehavior == EGlobalTabSwitchingBehavior::LastAccessedAsset)
	{
		// Populate the list with open asset editors
		TArray<UObject*> OpenAssetList = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->GetAllEditedAssets();
		for (UObject* OpenAsset : OpenAssetList)
		{
			if (OpenAsset->GetOuter() != GetTransientPackage())
			{
				MainTabsListDataSource.Add(MakeShareable(new FTabSwitchingListItem_Asset(OpenAsset)));
			}
		}

		MainTabsListDataSource.Add(FTabSwitchingListItem_World::MakeWorldItem());

		// Sort the list by access time
		MainTabsListDataSource.Sort([](const FTabListItemPtr& A, const FTabListItemPtr& B) { return (A->LastAccessTime > B->LastAccessTime); });
	}
	else
	{
		const TSharedPtr<SWindow> Window = FSlateApplication::Get().GetActiveTopLevelWindow();

		if (ensureMsgf(Window.IsValid(), TEXT("Active top level window cannot be invalid")))
		{
			const TSharedRef<FGlobalTabmanager> TabManager = FGlobalTabmanager::Get();
			const FGlobalTabmanager::FWindowMajorTabsInfo TabsInfo = TabManager->GetDockableWindowMajorTabsInfo(Window.ToSharedRef());

			if (TabsInfo.ActiveTab.IsValid() && !TabsInfo.DockTabs.IsEmpty())
			{
				// In this mode, active tab is the first to be added
				const int32 IndexActiveTab = TabsInfo.DockTabs.Find(TabsInfo.ActiveTab);

				const FText HomeScreenTabName = IMainFrameModule::GetHomeScreenTabLabel();

				// Add all right side tabs
				for (int32 Index = IndexActiveTab; Index < TabsInfo.DockTabs.Num(); Index++)
				{
					// HomeScreen tab should not appear in the switching tab dialog
					if (TabsInfo.DockTabs[Index]->GetTabLabel().EqualToCaseIgnored(HomeScreenTabName))
					{
						continue;
					}
					MainTabsListDataSource.Add(MakeShared<FTabSwitchingListItem_DockTab>(TabsInfo.DockTabs[Index]));
				}

				// Add all left side tabs
				for (int32 Index = 0; Index < IndexActiveTab; Index++)
				{
					// HomeScreen tab should not appear in the switching tab dialog
					if (TabsInfo.DockTabs[Index]->GetTabLabel().EqualToCaseIgnored(HomeScreenTabName))
					{
						continue;
					}
					MainTabsListDataSource.Add(MakeShared<FTabSwitchingListItem_DockTab>(TabsInfo.DockTabs[Index]));
				}
			}
		}
	}

	// Create the widgets
	NewTabItemToActivateDisplayBox =
		SNew(SBox)
		.Padding(5.f)
		.HeightOverride(40.0f)
		.VAlign(VAlign_Top);

	NewTabItemToActivatePathBox =
		SNew(SBox)
		.Padding(5.f)
		.HeightOverride(40.0f)
		.VAlign(VAlign_Center);

	MainTabsListWidget = SNew(STabListWidget)
		.ListItemsSource(&MainTabsListDataSource)
		.ConsumeMouseWheel(EConsumeMouseWheel::WhenScrollingPossible)
		.OnGenerateRow(this, &SGlobalTabSwitchingDialog::OnGenerateTabSwitchListItemWidget)
		.OnSelectionChanged(this, &SGlobalTabSwitchingDialog::OnMainTabListSelectionChanged)
		.OnMouseButtonClick(this, &SGlobalTabSwitchingDialog::OnMainTabListItemClicked)
		.SelectionMode(ESelectionMode::Single);

	TSharedRef<SWidget> DocumentTabList = SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5.f)
		[
			SNew(STextBlock)
			.TextStyle(FAppStyle::Get(), "ControlTabMenu.HeadingStyle")
			.Text(LOCTEXT("OpenAssetsHeading", "Active Files"))
		]
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(5.f)
		[
			MainTabsListWidget.ToSharedRef()
		];

	// Show or collapse children, this will still allow input events to flow through this dialog widget
	const EVisibility ChildVisibility = (!EditorUserSettings || EditorUserSettings->bShowTabSwitchingDialog)
		? EVisibility::Visible
		: EVisibility::Collapsed;

	ChildSlot
	[
		SNew(SBorder)
		.Padding(5.f)
		.Visibility(ChildVisibility)
		.BorderImage(FAppStyle::Get().GetBrush("ControlTabMenu.Background"))
		.ForegroundColor(FCoreStyle::Get().GetSlateColor("DefaultForeground"))
		[
			SNew(SBox)
			.WidthOverride(static_cast<float>(InSize.X))
			.HeightOverride(static_cast<float>(InSize.Y))
			.Padding(5.f)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.Padding(5.f)
				.AutoHeight()
				[
					SNew(SBox)
					.HeightOverride(50.f)
					[
						NewTabItemToActivateDisplayBox.ToSharedRef()
					]
				]
				+SVerticalBox::Slot()
				.Padding(5.f)
				.FillHeight(1.0f)
				[
					DocumentTabList
				]
				+SVerticalBox::Slot()
				.Padding(5.f)
				.AutoHeight()
				[
					NewTabItemToActivatePathBox.ToSharedRef()
				]
			]
		]
	];

	// Pick the second most recent or least recent file based on whether Shift was held down when we were summoned
	if (MainTabsListDataSource.Num() > 0)
	{
		CycleSelection(/*bForwards=*/ !FSlateApplication::Get().GetModifierKeys().IsShiftDown());
	}
}

FReply SGlobalTabSwitchingDialog::OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	// Check to see if the trigger modifier key was released, which should close the dialog
	const bool bCloseViaControl = TriggerChord.NeedsControl() && ((InKeyEvent.GetKey() == EKeys::LeftControl) || (InKeyEvent.GetKey() == EKeys::RightControl));
	const bool bCloseViaCommand = TriggerChord.NeedsCommand() && ((InKeyEvent.GetKey() == EKeys::LeftCommand) || (InKeyEvent.GetKey() == EKeys::RightCommand));
	const bool bCloseViaAlt = TriggerChord.NeedsAlt() && ((InKeyEvent.GetKey() == EKeys::LeftAlt) || (InKeyEvent.GetKey() == EKeys::RightAlt));
	
	if (bCloseViaControl || bCloseViaCommand || bCloseViaAlt)
	{
		constexpr bool bActivateTab = true;
		DismissDialog(bActivateTab);
		return FReply::Handled();
	}

	// Reset processing delay in case user quickly re-presses keys
	if (TriggerChord.Key == InKeyEvent.GetKey())
	{
		LastProcessedTime = 0.0;
	}

	return FReply::Unhandled();
}

FReply SGlobalTabSwitchingDialog::OnMouseWheel(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	if (MainTabsListWidget)
	{
		// This dialog only appears when CTRL+Tab is pressed and CTRL is held
		// But listview only consumes mouse wheel event if CTRL is not pressed...
		// So fake it until you make it
		FPointerEvent ModifiedEvent(
			InMouseEvent.GetUserIndex(),
			InMouseEvent.GetPointerIndex(),
			InMouseEvent.GetScreenSpacePosition(),
			InMouseEvent.GetLastScreenSpacePosition(),
			InMouseEvent.GetPressedButtons(),
			FKey(),
			InMouseEvent.GetWheelDelta(),
			FModifierKeysState()
		);

		FReply Reply = MainTabsListWidget->OnMouseWheel(InGeometry, ModifiedEvent);

		if (Reply.IsEventHandled())
		{
			return Reply;
		}
	}

	return SCompoundWidget::OnMouseWheel(InGeometry, InMouseEvent);
}

FReply SGlobalTabSwitchingDialog::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (TriggerChord.Key == InKeyEvent.GetKey())
	{
		// Don't switch tab too quickly when holding keys down
		constexpr double ProcessingDelay = 0.3;
		const double CurrentTime = FPlatformTime::Seconds();
		if (CurrentTime - LastProcessedTime > ProcessingDelay)
		{
			const bool bCycleForward = !InKeyEvent.IsShiftDown();
			CycleSelection(bCycleForward);
			LastProcessedTime = CurrentTime;
		}
	}

	return FReply::Unhandled();
}

FReply SGlobalTabSwitchingDialog::OnPreviewKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		constexpr bool bActivateTab = false;
		DismissDialog(bActivateTab);
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

TSharedRef<ITableRow> SGlobalTabSwitchingDialog::OnGenerateTabSwitchListItemWidget(FTabListItemPtr InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<FTabListItemPtr>, OwnerTable)[InItem->CreateWidget(UThumbnailManager::Get().GetSharedThumbnailPool())];
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
