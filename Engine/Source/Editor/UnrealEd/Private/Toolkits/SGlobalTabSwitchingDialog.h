// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetThumbnail.h"
#include "Framework/Commands/InputChord.h"
#include "InputCoreTypes.h"
#include "Input/Events.h"
#include "Input/Reply.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

class FTabSwitchingListItemBase;
template <typename ItemType> class SListView;

//////////////////////////////////////////////////////////////////////////
// SGlobalTabSwitchingDialog

class SGlobalTabSwitchingDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SGlobalTabSwitchingDialog){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FVector2D InSize, FInputChord InTriggerChord);

	virtual ~SGlobalTabSwitchingDialog() override;

	//~ Begin SWidget
	virtual FReply OnPreviewKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual FReply OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual FReply OnMouseWheel(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override;
	//~ End SWidget

	// Is an instance already open?
	static bool IsAlreadyOpen()
	{
		return bIsAlreadyOpen;
	}

private:
	typedef TSharedPtr<class FTabSwitchingListItemBase> FTabListItemPtr;
	typedef SListView<FTabListItemPtr> STabListWidget;

	TSharedRef<ITableRow> OnGenerateTabSwitchListItemWidget(FTabListItemPtr InItem, const TSharedRef<STableViewBase>& OwnerTable);

	void CycleSelection(bool bForwards);
	void OnMainTabListSelectionChanged(FTabListItemPtr InItem, ESelectInfo::Type SelectInfo);
	void OnMainTabListItemClicked(FTabListItemPtr InItem);
	void DismissDialog(bool bInActivateTab);
	FReply OnBrowseToSelectedAsset();

	FTabListItemPtr GetMainTabListSelectedItem() const;

	/** The chord that triggered the dialog (so we can handle the correct Tab/`/etc... key repeat, and dismiss on the correct control/command modifier release */
	FInputChord TriggerChord;

	// The array of 'document' items
	TArray<FTabListItemPtr> MainTabsListDataSource;

	// The widget representing the list of 'document' items
	TSharedPtr<STabListWidget> MainTabsListWidget;

	// The container widget for the indication of the asset that will be activated when the dialog closes
	TSharedPtr<SBox> NewTabItemToActivateDisplayBox;

	// The container widget for the indication of the path to the asset that will be activated when the dialog closes
	TSharedPtr<SBox> NewTabItemToActivatePathBox;

	/** Used to delay same key repeat */
	double LastProcessedTime = 0.f;

	static bool bIsAlreadyOpen;
};
