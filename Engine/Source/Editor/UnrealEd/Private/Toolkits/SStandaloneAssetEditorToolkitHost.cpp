// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/SStandaloneAssetEditorToolkitHost.h"
#include "Toolkits/AssetEditorToolkitMenuContext.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Layout/SBorder.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/AppStyle.h"
#include "Toolkits/ToolkitManager.h"
#include "Interfaces/IMainFrameModule.h"
#include "Widgets/Docking/SDockTab.h"
#include "UObject/Package.h"
#include "StatusBarSubsystem.h"
#include "ToolMenus.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "WidgetDrawerConfig.h"
#include "Elements/Framework/TypedElementCommonActions.h"

#define LOCTEXT_NAMESPACE "StandaloneAssetEditorToolkit"

static int32 StatusBarIdGenerator = 0;

void SStandaloneAssetEditorToolkitHost::Construct( const SStandaloneAssetEditorToolkitHost::FArguments& InArgs, const TSharedPtr<FTabManager>& InTabManager, const FName InitAppName )
{
	ToolbarSlot = nullptr;

	EditorCloseRequest = InArgs._OnRequestClose;
	EditorClosing = InArgs._OnClose;
	AppName = InitAppName;

	// Asset editors have non-unique names. For example every material editor is just "MaterialEditor" so the base app name plus a unique number to generate uniqueness. This number is not used across sessions and should never be saved.
	StatusBarName = FName(AppName, ++StatusBarIdGenerator);

	MyTabManager = InTabManager;

	CommonActions = NewObject<UTypedElementCommonActions>();
	CommonActions->AddToRoot();
}

void SStandaloneAssetEditorToolkitHost::SetupInitialContent( const TSharedRef<FTabManager::FLayout>& DefaultLayout, const TSharedPtr<SDockTab>& InHostTab, const bool bCreateDefaultStandaloneMenu )
{
	// @todo toolkit major: Expose common asset editing features here! (or require the asset editor's content to do this itself!)
	//		- Add a "toolkit menu"
	//				- Toolkits can access this and add menu items as needed
	//				- In world-centric, main frame menu becomes extendable
	//						- e.g., "Blueprint", "Debug" menus added
	//				- In standalone, toolkits get their own menu
	//						- Also, the core menu is just added as the first pull-down in the standalone menu
	//				- Multiple toolkits can be active and add their own menu items!
	//				- In world-centric, the core toolkit menu is available from the drop down
	//						- No longer need drop down next to toolkit display?  Not sure... Probably still want this
	//		- Add a "toolkit toolbar"
	//				- In world-centric, draws next to the level editor tool bar (or on top of)
	//						- Could either extend existing tool bar or add additional tool bars
	//						- May need to change arrangement to allow for wider tool bars (maybe displace grid settings too)
	//				- In standalone, just draws under the toolkit's menu

	const FName AssetEditorMenuName = GetMenuName();
	if (!UToolMenus::Get()->IsMenuRegistered(AssetEditorMenuName))
	{
		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(AssetEditorMenuName, "MainFrame.MainMenu");

		if (bCreateDefaultStandaloneMenu)
		{
			CreateDefaultStandaloneMenuBar(Menu);
		}
	}

	DefaultMenuWidget = SNullWidget::NullWidget;

	HostTabPtr = InHostTab;

	MenuOverlayWidgetContent = SNew(SBox);

	RestoreFromLayout(DefaultLayout);
	GenerateMenus(bCreateDefaultStandaloneMenu);

	if (InHostTab)
	{
		InHostTab->SetRightContent(
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(8.0f, 0.0f, 8.0f, 0.0f)
			[
				MenuOverlayWidgetContent.ToSharedRef()
			]
		);
	}

}

void SStandaloneAssetEditorToolkitHost::CreateDefaultStandaloneMenuBar(UToolMenu* MenuBar)
{
	struct Local
	{
		static void ExtendFileMenu(UToolMenu* InMenuBar)
		{
			const FName MenuName = *(InMenuBar->GetMenuName().ToString() + TEXT(".") + TEXT("File"));
			UToolMenu* Menu = UToolMenus::Get()->ExtendMenu(MenuName);

			FToolMenuSection& FileAssetSection = Menu->FindOrAddSection("FileAsset");

			FileAssetSection.Label = LOCTEXT("FileAssetSectionHeading", "Open");
			
			FileAssetSection.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				if (UAssetEditorToolkitMenuContext* Context = InSection.FindContext<UAssetEditorToolkitMenuContext>())
				{
					Context->Toolkit.Pin()->FillDefaultFileMenuOpenCommands(InSection);
				}
			}));
			
			FToolMenuSection& Section = Menu->FindOrAddSection("FileLoadAndSave");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				if (UAssetEditorToolkitMenuContext* Context = InSection.FindContext<UAssetEditorToolkitMenuContext>())
				{
					Context->Toolkit.Pin()->FillDefaultFileMenuCommands(InSection);
				}
			}));
		}

		static void FillAssetMenu(UToolMenu* InMenu)
		{
			FToolMenuSection& Section = InMenu->AddSection("AssetEditorActions", LOCTEXT("ActionsHeading", "Actions"));
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				if (UAssetEditorToolkitMenuContext* Context = InSection.FindContext<UAssetEditorToolkitMenuContext>())
				{
					Context->Toolkit.Pin()->FillDefaultAssetMenuCommands(InSection);
				}
			}));
		}

		static void ExtendHelpMenu(UToolMenu* InMenuBar)
		{
			const FName MenuName = *(InMenuBar->GetMenuName().ToString() + TEXT(".") + TEXT("Help"));
			UToolMenu* Menu = UToolMenus::Get()->ExtendMenu(MenuName);
			Menu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
			{
				TSharedPtr<FAssetEditorToolkit> Toolkit = InMenu->FindContext<UAssetEditorToolkitMenuContext>()->Toolkit.Pin();
				FFormatNamedArguments Args;
				Args.Add(TEXT("Editor"), Toolkit->GetBaseToolkitName());
				FToolMenuSection& Section = InMenu->AddSection("HelpResources", FText::Format(NSLOCTEXT("MainHelpMenu", "AssetEditorHelpResources", "{Editor} Resources"), Args));
				Section.InsertPosition = FToolMenuInsert("Learn", EToolMenuInsertType::First);
				Toolkit->FillDefaultHelpMenuCommands(Section);
			}));
		}
	};

	// Add asset-specific menu items to the top of the "File" menu
	Local::ExtendFileMenu(MenuBar);

	// Add the "Asset" menu, if we're editing an asset
	MenuBar->FindOrAddSection(NAME_None).AddDynamicEntry("DynamicAssetEntry", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		UAssetEditorToolkitMenuContext* Context = InSection.FindContext<UAssetEditorToolkitMenuContext>();
		if (Context && Context->Toolkit.IsValid() && Context->Toolkit.Pin()->IsActuallyAnAsset())
		{
			InSection.AddSubMenu(
				"Asset",
				LOCTEXT("AssetMenuLabel", "Asset"),		// @todo toolkit major: Either use "Asset", "File", or the asset type name e.g. "Blueprint" (Also update custom pull-down menus)
				LOCTEXT("AssetMenuLabel_ToolTip", "Opens a menu with commands for managing this asset"),
				FNewToolMenuDelegate::CreateStatic(&Local::FillAssetMenu)
				).InsertPosition = FToolMenuInsert("Edit", EToolMenuInsertType::After);
		}
	}));

	// Add asset-specific menu items to the "Help" menu
	Local::ExtendHelpMenu(MenuBar);
}


void SStandaloneAssetEditorToolkitHost::RestoreFromLayout( const TSharedRef<FTabManager::FLayout>& NewLayout )
{
	BindEditorCloseRequestToHostTab();

	TSharedPtr<SDockTab> HostTab = HostTabPtr.Pin();
	if (!HostTab.IsValid())
	{
		return;
	}
	HostTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateSP(this, &SStandaloneAssetEditorToolkitHost::OnTabClosed));
	HostTab->SetOnTabRelocated(FSimpleDelegate::CreateSP(this, &SStandaloneAssetEditorToolkitHost::OnTabRelocated));

	// Let the manager handle some logic on the Areas before destroying
	MyTabManager->CloseAllAreas();

	this->ChildSlot[SNullWidget::NullWidget];

	TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow( HostTab.ToSharedRef() );
	TSharedPtr<SWidget> RestoredUI = MyTabManager->RestoreFrom( NewLayout, ParentWindow );

	checkf(RestoredUI.IsValid(), TEXT("The layout must have a primary dock area"));

	UE::Editor::Toolbars::ECreateStatusBarOptions StatusBarCreationOptions = UE::Editor::Toolbars::ECreateStatusBarOptions::Default;
	if (HostedAssetEditorToolkit.IsValid())
	{
		StatusBarCreationOptions = HostedAssetEditorToolkit->GetStatusBarCreationOptions();
	}

	TSharedRef<SWidget> AssetEditorContent = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(FMargin(0.0f, 0.0f, 0.0f, 2.0f))
		.AutoHeight()
		.Expose(ToolbarSlot)
		+ SVerticalBox::Slot()
		.Padding(4.f, 2.f, 4.f, 2.f)
		.FillHeight(1.0f)
		[
			RestoredUI.ToSharedRef()
		];


	/**
	 * The ordering is important here, otherwise the status bar won't have a summon tab in PanelDrawer widget.
	 * The layout and panel drawer area must be restored before creating the status bar.
	 */
	if (TSharedPtr<SWidget> PanelDrawer = MyTabManager->RestorePanelDrawer(AssetEditorContent, ParentWindow.ToSharedRef()))
	{
		AssetEditorContent = PanelDrawer.ToSharedRef();
	}

	if (UStatusBarSubsystem* StatusBarSubsystem = GEditor ? GEditor->GetEditorSubsystem<UStatusBarSubsystem>() : nullptr)
	{
		StatusBarWidget = StatusBarSubsystem->MakeStatusBarWidget(StatusBarName, HostTab.ToSharedRef(), StatusBarCreationOptions);
	}

	this->ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			AssetEditorContent
		]
		+ SVerticalBox::Slot()
		.Padding(0.0f, 2.0f, 0.0f, 0.0f)
		.AutoHeight()
		[
			StatusBarWidget.IsValid() ? StatusBarWidget.ToSharedRef() : SNullWidget::NullWidget
		]
	];
}

FName SStandaloneAssetEditorToolkitHost::GetMenuName() const
{
	FName MenuAppName;
	if (HostedAssetEditorToolkit.IsValid())
	{
		MenuAppName = HostedAssetEditorToolkit->GetToolMenuAppName();
	}
	else
	{
		MenuAppName = AppName;
	}

	return *(FString(TEXT("AssetEditor.")) + MenuAppName.ToString() + TEXT(".MainMenu"));
}

TSharedRef<SWidget> SStandaloneAssetEditorToolkitHost::CreateMenuBar(FToolMenuContext& ToolMenuContext) const
{
	TSharedPtr<SWidget> ToolbarMenu;

	const FName AssetEditorMenuName = GetMenuName();
	if (HostedAssetEditorToolkit.IsValid())
	{
		// This might return nullptr if the implementation doesn't provide a custom toolbar menu.
		ToolbarMenu = HostedAssetEditorToolkit->CreateMenuBar(MyTabManager, AssetEditorMenuName, ToolMenuContext);
	}

	if (!ToolbarMenu)
	{
		IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>( "MainFrame" );
		ToolbarMenu = MainFrameModule.MakeMainMenu(MyTabManager, AssetEditorMenuName, ToolMenuContext);
	}
	
	return ToolbarMenu.ToSharedRef();
}

void SStandaloneAssetEditorToolkitHost::GenerateMenus(bool bForceCreateMenu)
{
	if( bForceCreateMenu || DefaultMenuWidget != SNullWidget::NullWidget )
	{
		UAssetEditorToolkitMenuContext* ContextObject = NewObject<UAssetEditorToolkitMenuContext>();
		ContextObject->Toolkit = HostedAssetEditorToolkit;
		FToolMenuContext ToolMenuContext(HostedAssetEditorToolkit->GetToolkitCommands(), FExtender::Combine(MenuExtenders), ContextObject);
		HostedAssetEditorToolkit->InitToolMenuContext(ToolMenuContext);
		DefaultMenuWidget = CreateMenuBar(ToolMenuContext);
	}
}

void SStandaloneAssetEditorToolkitHost::SetMenuOverlay( TSharedRef<SWidget> NewOverlay )
{
	MenuOverlayWidgetContent->SetContent(NewOverlay);
}

void SStandaloneAssetEditorToolkitHost::SetToolbar(TSharedPtr<SWidget> Toolbar)
{
	if (Toolbar)
	{
		(*ToolbarSlot)
		[
			Toolbar.ToSharedRef()
		];
	}
	else
	{
		(*ToolbarSlot)
		[
			SNullWidget::NullWidget
		];
	}
}

void SStandaloneAssetEditorToolkitHost::RegisterDrawer(FWidgetDrawerConfig&& Drawer, int32 SlotIndex)
{
	if (StatusBarWidget.IsValid())
	{
		GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->RegisterDrawer(StatusBarName, MoveTemp(Drawer), SlotIndex);
	}
}

FEditorModeTools& SStandaloneAssetEditorToolkitHost::GetEditorModeManager() const
{
	check(HostedAssetEditorToolkit.IsValid());

	return HostedAssetEditorToolkit->GetEditorModeManager();
}

void SStandaloneAssetEditorToolkitHost::BindEditorCloseRequestToHostTab()
{
	if (TSharedPtr<SDockTab> HostTab = HostTabPtr.Pin())
	{
		HostTab->SetCanCloseTab(EditorCloseRequest);
	}
}

void SStandaloneAssetEditorToolkitHost::UnbindEditorCloseRequestFromHostTab()
{
	if (TSharedPtr<SDockTab> HostTab = HostTabPtr.Pin())
	{
		HostTab->SetCanCloseTab(SDockTab::FCanCloseTab());
	}
}

SStandaloneAssetEditorToolkitHost::~SStandaloneAssetEditorToolkitHost()
{
	ShutdownToolkitHost();
	
	CommonActions->RemoveFromRoot();
}


TSharedRef< SWidget > SStandaloneAssetEditorToolkitHost::GetParentWidget()
{
	return AsShared();
}


void SStandaloneAssetEditorToolkitHost::BringToFront()
{
	// If our host window is not active, force it to front to ensure the tab will be visible
	// The tab manager won't activate a tab on an inactive window in all cases
	const TSharedPtr<SDockTab> HostTab = HostTabPtr.Pin();
	if (HostTab.IsValid())
	{
		TSharedPtr<SWindow> ParentWindow = HostTab->GetParentWindow();
		if (ParentWindow.IsValid() && !ParentWindow->IsActive())
		{
			ParentWindow->BringToFront();
		}
	}

	FGlobalTabmanager::Get()->DrawAttentionToTabManager( this->MyTabManager.ToSharedRef() );
}

void SStandaloneAssetEditorToolkitHost::OnToolkitHostingStarted( const TSharedRef< class IToolkit >& Toolkit )
{
	// Keep track of the toolkit we're hosting
	HostedToolkits.Add(Toolkit);

	// The tab manager needs to know how to spawn tabs from this toolkit
	Toolkit->RegisterTabSpawners(MyTabManager.ToSharedRef());

	if (!HostedAssetEditorToolkit.IsValid())
	{
		HostedAssetEditorToolkit = StaticCastSharedRef<FAssetEditorToolkit>(Toolkit);
	}
	else
	{
		HostedAssetEditorToolkit->OnToolkitHostingStarted(Toolkit);
	}
}

void SStandaloneAssetEditorToolkitHost::ShutdownToolkitHost()
{
	const TSharedPtr<SDockTab> HostTab = HostTabPtr.Pin();
	if (HostTab.IsValid())
	{
		HostTab->RequestCloseTab();
	}

	// Let the toolkit manager know that we're going away now
	FToolkitManager::Get().OnToolkitHostDestroyed(this);
	HostedToolkits.Reset();
	HostedAssetEditorToolkit.Reset();
}

void SStandaloneAssetEditorToolkitHost::OnToolkitHostingFinished( const TSharedRef< class IToolkit >& Toolkit )
{
	// The tab manager should forget how to spawn tabs from this toolkit
	Toolkit->UnregisterTabSpawners(MyTabManager.ToSharedRef());

	HostedToolkits.Remove(Toolkit);

	// Standalone Asset Editors close by shutting down their major tab.
	if (Toolkit == HostedAssetEditorToolkit)
	{
		ShutdownToolkitHost();
	}
	else if (HostedAssetEditorToolkit.IsValid())
	{
		HostedAssetEditorToolkit->OnToolkitHostingFinished(Toolkit);
	}
}


UWorld* SStandaloneAssetEditorToolkitHost::GetWorld() const 
{
	// Currently, standalone asset editors never have a world
	UE_LOGF(LogInit, Warning, "IToolkitHost::GetWorld() doesn't make sense in SStandaloneAssetEditorToolkitHost currently");
	return NULL;
}


UTypedElementCommonActions* SStandaloneAssetEditorToolkitHost::GetCommonActions() const
{
	return CommonActions.Get();
}

void SStandaloneAssetEditorToolkitHost::AddViewportOverlayWidget(TSharedRef<SWidget> InOverlaidWidget, int32 ZOrder, TSharedPtr<IAssetViewport> InViewport)
{
	if (HostedAssetEditorToolkit.IsValid())
	{
		HostedAssetEditorToolkit->AddViewportOverlayWidget(InOverlaidWidget, ZOrder);
	}
}

void SStandaloneAssetEditorToolkitHost::RemoveViewportOverlayWidget(TSharedRef<SWidget> InOverlaidWidget, TSharedPtr<IAssetViewport> InViewport)
{
	if (HostedAssetEditorToolkit.IsValid())
	{
		HostedAssetEditorToolkit->RemoveViewportOverlayWidget(InOverlaidWidget);
	}
}


FReply SStandaloneAssetEditorToolkitHost::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	// Check to see if any of the actions for the toolkits can be processed by the current event
	// If we are in debug mode do not process commands
	if (FSlateApplication::Get().IsNormalExecution())
	{
		for (TSharedPtr<IToolkit>& HostedToolkit : HostedToolkits)
		{
			if (HostedToolkit->ProcessCommandBindings(InKeyEvent))
			{
				return FReply::Handled();
			}
		}
	}

	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}


void SStandaloneAssetEditorToolkitHost::OnTabClosed(TSharedRef<SDockTab> TabClosed)
{
	check(TabClosed == HostTabPtr.Pin());

	EditorClosing.ExecuteIfBound();

	MyTabManager->SetMenuMultiBox(nullptr, nullptr);
	
	SaveTabLocation();
}

void SStandaloneAssetEditorToolkitHost::OnTabRelocated()
{
	SaveTabLocation();
}

void SStandaloneAssetEditorToolkitHost::SaveTabLocation() const
{
	if (!HostedAssetEditorToolkit.IsValid())
	{
		return;
	}

	const TSharedPtr<SDockTab> HostTab = HostTabPtr.Pin();
	if (!HostTab.IsValid())
	{
		return;
	}

	const TArray<UObject*>* const ObjectsBeingEdited = HostedAssetEditorToolkit->GetObjectsCurrentlyBeingEdited();
	if (!ObjectsBeingEdited)
	{
		return;
	}

	bool bConfigChanged = false;
	const bool IsDockedAssetEditor = HostTab->HasSiblingTab(FName("DockedToolkit"), false/*TreatIndexNoneAsWildcard*/);
	const EAssetEditorToolkitTabLocation AssetEditorToolkitTabLocation = (IsDockedAssetEditor) ? EAssetEditorToolkitTabLocation::Docked : EAssetEditorToolkitTabLocation::Standalone;
	for (const UObject* ObjectBeingEdited : *ObjectsBeingEdited)
	{
		// Only record assets that have a valid saved package
		UPackage* const Package = ObjectBeingEdited->GetOutermost();
		if (Package && Package->GetFileSize())
		{
			GConfig->SetInt(
				TEXT("AssetEditorToolkitTabLocation"),
				*ObjectBeingEdited->GetPathName(),
				static_cast<int32>(AssetEditorToolkitTabLocation),
				GEditorPerProjectIni
			);

			bConfigChanged = true;
		}
	}

	if (!bConfigChanged)
	{
		return;
	}

	// Only write this section to disk to avoid loosing last tab position on crash
	if (const FConfigFile* ConfigFile = GConfig->FindConfigFile(GEditorPerProjectIni))
	{
		const FConfigSection* Section = ConfigFile->FindSection(TEXT("AssetEditorToolkitTabLocation"));
		if (Section && ConfigFile->Branch)
		{
			FString IniPath = ConfigFile->Branch->IniPath;
			FConfigSection SectionCopy = *Section;

			// Avoid editor freeze since config flush happens on main thread...
			Async(EAsyncExecution::ThreadPool, [IniPath, SectionCopy = MoveTemp(SectionCopy)]()
			{
				FConfigFile DiskFile;
				DiskFile.Read(IniPath);
				FConfigSection* MutableSection = const_cast<FConfigSection*>(DiskFile.FindOrAddConfigSection(TEXT("AssetEditorToolkitTabLocation")));
				*MutableSection = SectionCopy;
				DiskFile.Write(IniPath);
			});
		}
	}
}

#undef LOCTEXT_NAMESPACE
