// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/Docking/TabManager.h"
#include "Toolkits/IToolkit.h"
#include "Toolkits/IToolkitHost.h"
#include "Toolkits/AssetEditorToolkit.h"

class SBorder;
class SBox;

/**
 * Base class for standalone asset editing host tabs
 */
class SStandaloneAssetEditorToolkitHost :
	public IToolkitHost, public SCompoundWidget
{

public:
	SLATE_BEGIN_ARGS( SStandaloneAssetEditorToolkitHost ){}
		SLATE_EVENT(FRequestAssetEditorClose, OnRequestClose)
		SLATE_EVENT(FAssetEditorClosing, OnClose)
	SLATE_END_ARGS()

	/** SCompoundWidget interface */

	/**
	 * Constructs this widget
	 *
	 * @param	InArgs          Declaration from which to construct the widget
	 * @param   InTabManager    TabManager to use
	 * @param	InitAppName     The app identifier for this standalone toolkit host
	 */
	void Construct( const FArguments& InArgs, const TSharedPtr<FTabManager>& InTabManager, const FName InitAppName );
	
	/**
	 * Fills in initial content by loading layout or using the defaults provided.  Must be called after
	 * the widget is constructed.
	 *
	 * @param   DefaultLayout                   The default layout to use if one couldn't be loaded
	 * @param   InHostTab                       MajorTab hosting this standalong editor
	 * @param   bCreateDefaultStandaloneMenu    True if the asset editor should automatically generate a default "asset" menu, or false if you're going to do this yourself in your derived asset editor's implementation
	 */
	void SetupInitialContent( const TSharedRef<FTabManager::FLayout>& DefaultLayout, const TSharedPtr<SDockTab>& InHostTab, const bool bCreateDefaultStandaloneMenu );

	/** Destructor */
	virtual ~SStandaloneAssetEditorToolkitHost();

	/** IToolkitHost interface */
	virtual TSharedRef< class SWidget > GetParentWidget() override;
	virtual void BringToFront() override;
	virtual TSharedPtr< class FTabManager > GetTabManager() const override { return MyTabManager; }
	virtual void OnToolkitHostingStarted( const TSharedRef< class IToolkit >& Toolkit ) override;
	virtual void OnToolkitHostingFinished( const TSharedRef< class IToolkit >& Toolkit ) override;
	virtual UWorld* GetWorld() const override;
	virtual UTypedElementCommonActions* GetCommonActions() const override;
	FName GetStatusBarName() const override { return StatusBarName; } 
	virtual void AddViewportOverlayWidget(TSharedRef<SWidget> InOverlaidWidget, int32 ZOrder = INDEX_NONE, TSharedPtr<IAssetViewport> InViewport = nullptr) override;
	virtual void RemoveViewportOverlayWidget(TSharedRef<SWidget> InOverlaidWidget, TSharedPtr<IAssetViewport> InViewport = nullptr) override;
	virtual FOnActiveViewportChanged& OnActiveViewportChanged() override { return OnActiveViewportChangedDelegate; }
	void CreateDefaultStandaloneMenuBar(UToolMenu* MenuBar);

	/** SWidget overrides */
	virtual bool SupportsKeyboardFocus() const override
	{
		return true;
	}
	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;

	/** Fills in the content by loading the associated layout or using the defaults provided.  Must be called after
	  * the widget is constructed.*/
	virtual void RestoreFromLayout( const TSharedRef<FTabManager::FLayout>& NewLayout );

	/** Generates the ui for all menus and toolbars, potentially forcing the menu to be created even if it shouldn't */
	void GenerateMenus(bool bForceCreateMenu);

	/** Gets all extenders that this toolkit host uses */
	TArray< TSharedPtr<FExtender> >& GetMenuExtenders() {return MenuExtenders;}

	/**
	 * Set a widget to use in the menu bar overlay
	 *
	 * @param NewOverlay  The widget to use as the overlay, will appear on the right side of the menu bar.
	 */
	void SetMenuOverlay( TSharedRef<SWidget> NewOverlay );

	/**
	 * Sets the contents of the toolbar in this asset editor
	 */
	void SetToolbar(TSharedPtr<SWidget> Toolbar);

	/**
	 * Registers a drawer for the asset editor status bar
	 */
	void RegisterDrawer(struct FWidgetDrawerConfig&& Drawer, int32 SlotIndex = INDEX_NONE);

	virtual FEditorModeTools& GetEditorModeManager() const override;

	/**
	 * Binds EditorCloseRequest to the HostTab so that it triggers when the tab is trying to close.
	 */
	void BindEditorCloseRequestToHostTab();

	/**
	 * Unbinds EditorCloseRequest from the HostTab so that it doesn't trigger when the tab is trying to close.
	 */
	void UnbindEditorCloseRequestFromHostTab();

private:
	void OnTabClosed(TSharedRef<SDockTab> TabClosed);
	void OnTabRelocated();

	/** Saves this tab location into config to reopen properly */
	void SaveTabLocation() const;

	void ShutdownToolkitHost();

	/** Create the main menu bar, either generated by the toolkit host or by the MainFrame module. */
	TSharedRef<SWidget> CreateMenuBar(struct FToolMenuContext& ToolMenuContext) const;

	FName GetMenuName() const;

	/** Manages internal tab layout */
	TSharedPtr<FTabManager> MyTabManager;

	/** The widget that will house the overlay widgets (if any) */
	TSharedPtr<SBox> MenuOverlayWidgetContent;

	/** The default menu widget */
	TSharedPtr< SWidget > DefaultMenuWidget;

	/** The DockTab in which we reside. */
	TWeakPtr<SDockTab> HostTabPtr;

	/** Name ID for this app */
	FName AppName;

	/** Name for the status bar. Must be unique. This name is not preserved across sessions and should never be saved. */
	FName StatusBarName;

	/** List of all of the toolkits we're currently hosting */
	TArray< TSharedPtr< class IToolkit > > HostedToolkits;

	/** The 'owning' asset editor toolkit we're hosting */
	TSharedPtr< class FAssetEditorToolkit > HostedAssetEditorToolkit;

	/** Delegate to be called to determine if we are allowed to close this toolkit host */
	FRequestAssetEditorClose EditorCloseRequest;

	/** Delegate to be called when this toolkit host is closing */
	FAssetEditorClosing EditorClosing;

	/** The menu extenders to populate the main toolkit host menu with */
	TArray< TSharedPtr<FExtender> > MenuExtenders;

	/** A delegate which is called any time the LevelEditor's active viewport changes. */
	FOnActiveViewportChanged OnActiveViewportChangedDelegate;

	/** Reference to bottom status bar, containing dockable tabs */
	TSharedPtr<SWidget> StatusBarWidget;

	SVerticalBox::FSlot* ToolbarSlot;

	TObjectPtr<UTypedElementCommonActions> CommonActions;
};


