// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/SimpleAssetEditor.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "Widgets/Docking/SDockTab.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "EditorClassUtils.h"
#include "Editor.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"

#define LOCTEXT_NAMESPACE "GenericEditor"

const FName FSimpleAssetEditor::ToolkitFName( TEXT( "GenericAssetEditor" ) );
const FName FSimpleAssetEditor::PropertiesTabId( TEXT( "GenericEditor_Properties" ) );

void FSimpleAssetEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_GenericAssetEditor", "Asset Editor"));

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner( PropertiesTabId, FOnSpawnTab::CreateSP(this, &FSimpleAssetEditor::SpawnPropertiesTab) )
		.SetDisplayName( LOCTEXT("PropertiesTab", "Details") )
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"))
		.SetReadOnlyBehavior(ETabReadOnlyBehavior::Custom);
}

void FSimpleAssetEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner( PropertiesTabId );
}

const FName FSimpleAssetEditor::SimpleEditorAppIdentifier( TEXT( "GenericEditorApp" ) );

FSimpleAssetEditor::~FSimpleAssetEditor()
{
	GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetPostImport.RemoveAll(this);
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);

	DetailsView.Reset();
}


void FSimpleAssetEditor::InitEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, const TArray<UObject*>& ObjectsToEdit, FGetDetailsViewObjects GetDetailsViewObjects )
{
	EditingObjects = ObjectsToEdit;
	GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetPostImport.AddSP(this, &FSimpleAssetEditor::HandleAssetPostImport);
	FCoreUObjectDelegates::OnObjectsReplaced.AddSP(this, &FSimpleAssetEditor::OnObjectsReplaced);

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>( "PropertyEditor" );
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsView = PropertyEditorModule.CreateDetailView( DetailsViewArgs );

	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout( "Standalone_SimpleAssetEditor_Layout_v4" )
	->AddArea
	(
		FTabManager::NewPrimaryArea() ->SetOrientation(Orient_Vertical)
		->Split
		(
			FTabManager::NewSplitter()
			->Split
			(
				FTabManager::NewStack()
				->AddTab( PropertiesTabId, ETabState::OpenedTab )
			)
		)
	);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor( Mode, InitToolkitHost, FSimpleAssetEditor::SimpleEditorAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, ObjectsToEdit );

	RegenerateMenusAndToolbars();
	// @todo toolkit world centric editing
	// Setup our tool's layout
	/*if( IsWorldCentricAssetEditor() && !PropertiesTab.IsValid() )
	{
		const FString TabInitializationPayload(TEXT(""));		// NOTE: Payload not currently used for asset properties
		SpawnToolkitTab(GetToolbarTabId(), FString(), EToolkitTabSpot::ToolBar);
		PropertiesTab = SpawnToolkitTab( PropertiesTabId, TabInitializationPayload, EToolkitTabSpot::Details );
	}*/

	// Get the list of objects to edit the details of
	const TArray<UObject*> ObjectsToEditInDetailsView = ( GetDetailsViewObjects.IsBound() ) ? GetDetailsViewObjects.Execute( ObjectsToEdit ) : ObjectsToEdit;

	// Ensure all objects are transactable for undo/redo in the details panel
	for( UObject* ObjectToEditInDetailsView : ObjectsToEditInDetailsView )
	{
		ObjectToEditInDetailsView->SetFlags(RF_Transactional);
	}

	if( DetailsView.IsValid() )
	{
		DetailsView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateSP(this, &FSimpleAssetEditor::IsPropertyEditingEnabled));

		// Make sure details window is pointing to our object
		DetailsView->SetObjects( ObjectsToEditInDetailsView );
	}
}

FName FSimpleAssetEditor::GetToolkitFName() const
{
	return ToolkitFName;
}

FText FSimpleAssetEditor::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Generic Asset Editor");
}

FText FSimpleAssetEditor::GetToolkitName() const
{
	const auto& EditingObjs = GetEditingObjects();

	check( EditingObjs.Num() > 0 );

	FFormatNamedArguments Args;
	Args.Add( TEXT("ToolkitName"), GetBaseToolkitName() );

	if( EditingObjs.Num() == 1 )
	{
		const UObject* EditingObject = EditingObjs[ 0 ];
		return FText::FromString(EditingObject->GetName());
	}
	else
	{
		UClass* SharedBaseClass = nullptr;
		for( int32 x = 0; x < EditingObjs.Num(); ++x )
		{
			UObject* Obj = EditingObjs[ x ];
			check( Obj );

			UClass* ObjClass = Cast<UClass>(Obj);
			if (ObjClass == nullptr)
			{
				ObjClass = Obj->GetClass();
			}
			check( ObjClass );

			// Initialize with the class of the first object we encounter.
			if( SharedBaseClass == nullptr )
			{
				SharedBaseClass = ObjClass;
			}

			// If we've encountered an object that's not a subclass of the current best baseclass,
			// climb up a step in the class hierarchy.
			while( !ObjClass->IsChildOf( SharedBaseClass ) )
			{
				SharedBaseClass = SharedBaseClass->GetSuperClass();
			}
		}

		check(SharedBaseClass);

		Args.Add( TEXT("NumberOfObjects"), EditingObjs.Num() );
		Args.Add( TEXT("ClassName"), FText::FromString( SharedBaseClass->GetName() ) );
		return FText::Format( LOCTEXT("ToolkitTitle_EditingMultiple", "{NumberOfObjects} {ClassName} - {ToolkitName}"), Args );
	}
}

FText FSimpleAssetEditor::GetToolkitToolTipText() const
{
	const auto& EditingObjs = GetEditingObjects();

	check( EditingObjs.Num() > 0 );

	FFormatNamedArguments Args;
	Args.Add( TEXT("ToolkitName"), GetBaseToolkitName() );

	if( EditingObjs.Num() == 1 )
	{
		const UObject* EditingObject = EditingObjs[ 0 ];
		return FAssetEditorToolkit::GetToolTipTextForObject(EditingObject);
	}
	else
	{
		UClass* SharedBaseClass = NULL;
		for( int32 x = 0; x < EditingObjs.Num(); ++x )
		{
			UObject* Obj = EditingObjs[ x ];
			check( Obj );

			UClass* ObjClass = Cast<UClass>(Obj);
			if (ObjClass == nullptr)
			{
				ObjClass = Obj->GetClass();
			}
			check( ObjClass );

			// Initialize with the class of the first object we encounter.
			if( SharedBaseClass == nullptr )
			{
				SharedBaseClass = ObjClass;
			}

			// If we've encountered an object that's not a subclass of the current best baseclass,
			// climb up a step in the class hierarchy.
			while( !ObjClass->IsChildOf( SharedBaseClass ) )
			{
				SharedBaseClass = SharedBaseClass->GetSuperClass();
			}
		}

		check(SharedBaseClass);

		Args.Add( TEXT("NumberOfObjects"), EditingObjs.Num() );
		Args.Add( TEXT("ClassName"), FText::FromString( SharedBaseClass->GetName() ) );
		return FText::Format( LOCTEXT("ToolkitTitle_EditingMultipleToolTip", "{NumberOfObjects} {ClassName} - {ToolkitName}"), Args );
	}
}

FLinearColor FSimpleAssetEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor( 0.5f, 0.0f, 0.0f, 0.5f );
}

void FSimpleAssetEditor::SetPropertyVisibilityDelegate(FIsPropertyVisible InVisibilityDelegate)
{
	DetailsView->SetIsPropertyVisibleDelegate(InVisibilityDelegate);
	DetailsView->ForceRefresh();
}

void FSimpleAssetEditor::SetPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled InPropertyEditingDelegate)
{
	IsPropertyEditingEnabledDelegate = InPropertyEditingDelegate;
	DetailsView->ForceRefresh();
}

TSharedRef<SDockTab> FSimpleAssetEditor::SpawnPropertiesTab( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId() == PropertiesTabId );

	return SNew(SDockTab)
		.Label( LOCTEXT("GenericDetailsTitle", "Details") )
		.TabColorScale( GetTabColorScale() )
		.OnCanCloseTab_Lambda([]() { return false; })
		[
			DetailsView.ToSharedRef()
		];
}

void FSimpleAssetEditor::HandleAssetPostImport(UFactory* InFactory, UObject* InObject)
{
	if (EditingObjects.Contains(InObject))
	{
		// The details panel likely needs to be refreshed if an asset was imported again
		DetailsView->SetObjects(EditingObjects);
	}
}

void FSimpleAssetEditor::OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap)
{
	bool bChangedAny = false;

	// Refresh our details view if one of the objects replaced was in the map. This gets called before the reinstance GC fixup, so we might as well fixup EditingObjects now too
	for (int32 i = 0; i < EditingObjects.Num(); i++)
	{
		UObject* SourceObject = EditingObjects[i];
		UObject* ReplacedObject = ReplacementMap.FindRef(SourceObject);

		if (ReplacedObject && ReplacedObject != SourceObject)
		{
			EditingObjects[i] = ReplacedObject;
			bChangedAny = true;
		}
	}

	if (bChangedAny)
	{
		DetailsView->SetObjects(EditingObjects);
	}
}

FString FSimpleAssetEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Generic Asset ").ToString();
}

TSharedRef<FSimpleAssetEditor> FSimpleAssetEditor::CreateEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UObject* ObjectToEdit, FGetDetailsViewObjects GetDetailsViewObjects )
{
	TSharedRef< FSimpleAssetEditor > NewEditor( new FSimpleAssetEditor() );

	TArray<UObject*> ObjectsToEdit;
	ObjectsToEdit.Add( ObjectToEdit );
	NewEditor->InitEditor( Mode, InitToolkitHost, ObjectsToEdit, GetDetailsViewObjects );

	return NewEditor;
}

TSharedRef<FSimpleAssetEditor> FSimpleAssetEditor::CreateEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, const TArray<UObject*>& ObjectsToEdit, FGetDetailsViewObjects GetDetailsViewObjects )
{
	TSharedRef< FSimpleAssetEditor > NewEditor( new FSimpleAssetEditor() );
	NewEditor->InitEditor( Mode, InitToolkitHost, ObjectsToEdit, GetDetailsViewObjects );
	return NewEditor;
}

FReply FSimpleAssetEditor::OnFindParentClassInContentBrowserClicked(TObjectPtr<UObject> SyncToClass) const
{
	if (SyncToClass)
	{
		TArray<UObject*> ObjectList { SyncToClass };
		GEditor->SyncBrowserToObjects(ObjectList);
	}

	return FReply::Handled();
}

FReply FSimpleAssetEditor::OnEditParentClassClicked(TObjectPtr<UObject> EditClass) const
{
	if (EditClass)
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(EditClass);
	}

	return FReply::Handled();
}

bool FSimpleAssetEditor::IsPropertyEditingEnabled() const
{
	bool bIsPropertyEditingEnabled = true;

	// First check any user provided delegate
	if (IsPropertyEditingEnabledDelegate.IsBound())
	{
		bIsPropertyEditingEnabled &= IsPropertyEditingEnabledDelegate.Execute();
	}

	// Also make sure we are opened in Edit mode
	bIsPropertyEditingEnabled &= (GetOpenMethod() == EAssetOpenMethod::Edit);

	return bIsPropertyEditingEnabled;
}

void FSimpleAssetEditor::PostRegenerateMenusAndToolbars()
{
	// Find the common denominator class of the assets we're editing
	TArray<UClass*> ClassList;
	for (UObject* Obj : EditingObjects)
	{
		check(Obj);
		ClassList.Add(Obj->GetClass());
	}

	UClass* CommonDenominatorClass = UClass::FindCommonBase(ClassList);
	const bool bNotAllSame = (EditingObjects.Num() > 0) && (EditingObjects[0]->GetClass() != CommonDenominatorClass);

	// Provide a hyperlink to view that native class
	if (CommonDenominatorClass)
	{
		// If the common denominator is a blueprint generated class, link to the BP instead of an inaccessible _c class
		if (CommonDenominatorClass->ClassGeneratedBy)
		{
			// build and attach the menu overlay
			TSharedRef<SHorizontalBox> MenuOverlayBox = SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					.ShadowOffset(FVector2D::UnitVector)
					.Text(LOCTEXT("BlueprintEditor_ParentClass", "Parent class: "))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SSpacer)
					.Size(FVector2D(2.0f, 1.0f))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.ShadowOffset(FVector2D::UnitVector)
					.Text(FText::FromName(CommonDenominatorClass->ClassGeneratedBy->GetFName()))
					.TextStyle(FAppStyle::Get(), "Common.InheritedFromBlueprintTextStyle")
					.ToolTipText(LOCTEXT("ParentClassToolTip", "The class that the current Blueprint is based on. The parent provides the base definition, which the current Blueprint extends."))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.VAlign(VAlign_Center)
					.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
					.OnClicked(this, &FSimpleAssetEditor::OnFindParentClassInContentBrowserClicked, CommonDenominatorClass->ClassGeneratedBy)
					.ToolTipText(LOCTEXT("FindParentInCBToolTip", "Find parent in Content Browser"))
					.ContentPadding(4.0f)
					.ForegroundColor(FSlateColor::UseForeground())
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("Icons.Search"))
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.VAlign(VAlign_Center)
					.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
					.OnClicked(this, &FSimpleAssetEditor::OnEditParentClassClicked, CommonDenominatorClass->ClassGeneratedBy)
					.ToolTipText(LOCTEXT("EditParentClassToolTip", "Open parent in editor"))
					.ContentPadding(4.0f)
					.ForegroundColor(FSlateColor::UseForeground())
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("Icons.Edit"))
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SSpacer)
					.Size(FVector2D(8.0f, 1.0f))
				]
				;
			SetMenuOverlay(MenuOverlayBox);
		}
		else
		{
			// build and attach the menu overlay
			TSharedRef<SHorizontalBox> MenuOverlayBox = SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					.ShadowOffset(FVector2D::UnitVector)
					.Text(bNotAllSame ? LOCTEXT("SimpleAssetEditor_AssetType_Varied", "Common Asset Type: ") : LOCTEXT("SimpleAssetEditor_AssetType", "Asset Type: "))
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 8.0f, 0.0f)
				[
					FEditorClassUtils::GetSourceLink(CommonDenominatorClass)
				];
		
			SetMenuOverlay(MenuOverlayBox);
		}
	}
}

FName FSimpleAssetEditor::GetEditingAssetTypeName() const
{
	// We want the global recent assets menu for simple asset editors so we report our editing asset type as none
	return NAME_None;
}

#undef LOCTEXT_NAMESPACE
