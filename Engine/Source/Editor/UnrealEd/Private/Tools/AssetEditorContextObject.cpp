// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/AssetEditorContextObject.h"

#include "EditorModeManager.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Framework/TypedElementCommonActions.h"
#include "Toolkits/IToolkitHost.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetEditorContextObject)

const UTypedElementSelectionSet* UAssetEditorContextObject::GetSelectionSet() const
{
	if (GetToolkitHost())
	{
		return GetToolkitHost()->GetEditorModeManager().GetEditorSelectionSet();
	}

	return nullptr;
}

UTypedElementSelectionSet* UAssetEditorContextObject::GetMutableSelectionSet()
{
	if (GetToolkitHost())
	{
		return GetToolkitHost()->GetEditorModeManager().GetEditorSelectionSet();
	}

	return nullptr;
}

UTypedElementCommonActions* UAssetEditorContextObject::GetCommonActions()
{
	if (GetToolkitHost())
	{
		return GetToolkitHost()->GetCommonActions();
	}

	return nullptr;
}

UWorld* UAssetEditorContextObject::GetEditingWorld() const
{
	return GetWorld();
}

const IToolkitHost* UAssetEditorContextObject::GetToolkitHost() const
{
	return ToolkitHost;
}

IToolkitHost* UAssetEditorContextObject::GetMutableToolkitHost()
{
	return ToolkitHost;
}

void UAssetEditorContextObject::SetToolkitHost(IToolkitHost* InToolkitHost)
{
	ToolkitHost = InToolkitHost;
}
