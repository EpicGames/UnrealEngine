// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/AssetEditorContextInterface.h"
#include "UObject/Object.h"

#include "AssetEditorContextObject.generated.h"

UCLASS(Transient)
class UAssetEditorContextObject : public UObject, public IAssetEditorContextInterface
{
	GENERATED_BODY()

public:
	// Begin UAssetEditorContextObject interface
	const UTypedElementSelectionSet* GetSelectionSet() const override;
	UTypedElementSelectionSet* GetMutableSelectionSet() override;
	UTypedElementCommonActions* GetCommonActions() override;
	UWorld* GetEditingWorld() const override;
	const IToolkitHost* GetToolkitHost() const override;
	IToolkitHost* GetMutableToolkitHost() override;
	// End UAssetEditorContextObject interface

	/**
	 * Set the toolkit host associated with our asset editor.
	 */
	void SetToolkitHost(IToolkitHost* InToolkitHost);

private:
	/** The toolkit host associated with our asset editor. */
	IToolkitHost* ToolkitHost = nullptr;
};