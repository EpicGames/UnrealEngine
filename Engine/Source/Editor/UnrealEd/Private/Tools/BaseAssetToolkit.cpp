// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/BaseAssetToolkit.h"
#include "Framework/Docking/TabManager.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Widgets/Docking/SDockTab.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "EditorViewportTabContent.h"
#include "SAssetEditorViewport.h"
#include "SEditorViewport.h"
#include "EditorViewportClient.h"
#include "PreviewScene.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Modules/ModuleManager.h"
#include "Tools/UAssetEditor.h"
#include "AssetEditorModeManager.h"


#define LOCTEXT_NAMESPACE "BaseAssetToolkit"

const FName FBaseAssetToolkit::ViewportTabID(TEXT("BaseAssetToolkit_Viewport"));
const FName FBaseAssetToolkit::DetailsTabID(TEXT("BaseAssetToolkit_Details"));

FBaseAssetToolkit::FBaseAssetToolkit(UAssetEditor* InOwningAssetEditor)
	: FAssetEditorToolkit()
{
	OwningAssetEditor = InOwningAssetEditor;

	FString LayoutString = TEXT("Standalone_Test_Layout_v2_") + LayoutAppendix;
	StandaloneDefaultLayout = FTabManager::NewLayout(FName(LayoutString))
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.7f)
					->AddTab(ViewportTabID, ETabState::OpenedTab)
					->SetHideTabWell(true)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.3f)
					->AddTab(DetailsTabID, ETabState::OpenedTab)
					->SetHideTabWell(true)
				)
			)
		);
}

FBaseAssetToolkit::~FBaseAssetToolkit()
{
	OwningAssetEditor->OnToolkitClosed();
}

void FBaseAssetToolkit::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	if(!AssetEditorTabsCategory.IsValid())
	{
		// Use the first child category of the local workspace root if there is one, otherwise use the root itself
		const auto& LocalCategories = InTabManager->GetLocalWorkspaceMenuRoot()->GetChildItems();
		AssetEditorTabsCategory = LocalCategories.Num() > 0 ? LocalCategories[0] : InTabManager->GetLocalWorkspaceMenuRoot();
	}

	InTabManager->RegisterTabSpawner(ViewportTabID, FOnSpawnTab::CreateSP(this, &FBaseAssetToolkit::SpawnTab_Viewport))
		.SetDisplayName(LOCTEXT("ViewportTab", "Viewport"))
		.SetGroup(AssetEditorTabsCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"));

	InTabManager->RegisterTabSpawner(DetailsTabID, FOnSpawnTab::CreateSP(this, &FBaseAssetToolkit::SpawnTab_Details))
		.SetDisplayName(LOCTEXT("Details", "Details"))
		.SetGroup(AssetEditorTabsCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

}

void FBaseAssetToolkit::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(ViewportTabID);
	InTabManager->UnregisterTabSpawner(DetailsTabID);
}

const TSharedRef<FTabManager::FLayout> FBaseAssetToolkit::GetDefaultLayout() const
{
	return StandaloneDefaultLayout.ToSharedRef();
}

TSharedRef<SDockTab> FBaseAssetToolkit::SpawnTab_Viewport(const FSpawnTabArgs& Args)
{
	TSharedRef< SDockTab > DockableTab =
		SNew(SDockTab);

	const FString LayoutId = FString("BaseAssetViewport");
	ViewportTabContent->Initialize(ViewportDelegate, DockableTab, LayoutId);
	return DockableTab;
}

TSharedRef<SDockTab>  FBaseAssetToolkit::SpawnTab_Details(const FSpawnTabArgs& Args)
{
	TSharedPtr<SDockTab> DetailsTab = SNew(SDockTab)
		.Label(LOCTEXT("BaseDetailsTitle", "Details"))
		[
			DetailsView.ToSharedRef()
		];

	return DetailsTab.ToSharedRef();
}

TSharedRef<IDetailsView> FBaseAssetToolkit::CreateDetailsView()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	return PropertyEditorModule.CreateDetailView(DetailsViewArgs);
}

void FBaseAssetToolkit::RegisterToolbar()
{
}

AssetEditorViewportFactoryFunction FBaseAssetToolkit::GetViewportDelegate()
{
	// Set up functor for viewport tab
	AssetEditorViewportFactoryFunction TempViewportDelegate = [this](const FAssetEditorViewportConstructionArgs& InArgs)
	{
		return SNew(SAssetEditorViewport, InArgs)
			.EditorViewportClient(ViewportClient);
	};

	return TempViewportDelegate;
}

TSharedPtr<FEditorViewportClient> FBaseAssetToolkit::CreateEditorViewportClient() const
{
	FPreviewScene* PreviewScene = new FPreviewScene(FPreviewScene::ConstructionValues());
	StaticCastSharedPtr<FAssetEditorModeManager>(EditorModeManager)->SetPreviewScene(PreviewScene);

	return MakeShared<FEditorViewportClient>(EditorModeManager.Get(), PreviewScene);
}

void FBaseAssetToolkit::CreateWidgets()
{
	RegisterToolbar();
	CreateEditorModeManager();
	ViewportClient = CreateEditorViewportClient();
	ViewportDelegate = GetViewportDelegate();
	ViewportTabContent = MakeShareable(new FEditorViewportTabContent());
	LayoutExtender = MakeShared<FLayoutExtender>();

	DetailsView = CreateDetailsView();
	check(DetailsView.IsValid());
}

void FBaseAssetToolkit::SetEditingObject(class UObject* InObject)
{
	DetailsView->SetObject(InObject);
}

void FBaseAssetToolkit::CreateEditorModeManager()
{
	EditorModeManager = MakeShared<FAssetEditorModeManager>();
}

#undef LOCTEXT_NAMESPACE