// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/DefaultEdMode.h"

#include "EditorModeManager.h"
#include "EditorModes.h"
#include "Textures/SlateIcon.h"
#include "LevelEditorViewport.h"
#include "Elements/Framework/TypedElementList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DefaultEdMode)

class FLevelEditorSelectModeWidgetHelper : public FLegacyEdModeWidgetHelper
{
public:
	virtual bool ShouldDrawWidget() const override
	{
		if (GCurrentLevelEditingViewportClient)
		{
			FTypedElementListConstRef ElementsToManipulate = GCurrentLevelEditingViewportClient->GetElementsToManipulate();
			return ElementsToManipulate->Num() > 0;
		}
		return false;
	}
	
	virtual bool RequiresLegacyViewportInteractions() const override { return false; }
};

UEdModeDefault::UEdModeDefault()
{
	Info = FEditorModeInfo(
		FBuiltinEditorModes::EM_Default,
		NSLOCTEXT("DefaultMode", "DisplayName", "Selection"),
		FSlateIcon("EditorStyle", "LevelEditor.SelectMode", "LevelEditor.SelectMode.Small"),
		true, 0);
}

bool UEdModeDefault::UsesPropertyWidgets() const
{
	return true;
}

bool UEdModeDefault::UsesToolkits() const
{
	return false;
}

TSharedRef<FLegacyEdModeWidgetHelper> UEdModeDefault::CreateWidgetHelper()
{
	return MakeShared<FLevelEditorSelectModeWidgetHelper>();
}

namespace FAssetEdModes
{
	const FEditorModeID EM_AssetDefault(TEXT("EM_AssetDefault"));
}

class FAssetEdModeWidgetHelper : public FLegacyEdModeWidgetHelper
{
public:	
	virtual bool RequiresLegacyViewportInteractions() const override { return false; }
};

UAssetEdModeDefault::UAssetEdModeDefault()
{
	Info = FEditorModeInfo(
		FAssetEdModes::EM_AssetDefault,
		NSLOCTEXT("AssetDefaultMode", "DisplayName", "AssetSelection"),
		FSlateIcon("EditorStyle", "LevelEditor.SelectMode", "LevelEditor.SelectMode.Small"),
		false, 0);
}

TSharedRef<FLegacyEdModeWidgetHelper> UAssetEdModeDefault::CreateWidgetHelper()
{
	return MakeShared<FAssetEdModeWidgetHelper>();
}
