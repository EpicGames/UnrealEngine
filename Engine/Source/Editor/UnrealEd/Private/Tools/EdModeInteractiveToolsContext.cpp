// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/EdModeInteractiveToolsContext.h"

#include "BaseGizmos/GizmoViewContext.h"
#include "ContextObjectStore.h"
#include "Editor.h"
#include "EditorViewportClient.h"
#include "EditorModeManager.h"
#include "Framework/Application/SlateApplication.h"
#include "LevelEditor.h"
#include "LevelEditorViewport.h"
#include "IAssetViewport.h"
#include "Math/Rotator.h"
#include "Misc/AssertionMacros.h"
#include "SLevelViewport.h"

#include "Modules/ModuleManager.h"
#include "ShowFlags.h"				// for EngineShowFlags
#include "Engine/Engine.h"
#include "Engine/Selection.h"
#include "Engine/World.h"
#include "Misc/ITransaction.h"
#include "ScopedTransaction.h"
#include "Materials/Material.h"
#include "Engine/StaticMesh.h"
#include "Components/StaticMeshComponent.h"

#include "ToolContextInterfaces.h"
#include "InteractiveToolObjects.h"
#include "EditorInteractiveGizmoManager.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "EditorModeManager.h"
#include "EdMode.h"
#include "EditorDragTools.h"
#include "ToolMenuOwner.h"
#include "ToolMenus.h"

#include "BaseGizmos/GizmoRenderingUtil.h"
#include "UnrealClient.h"
#include "EditorDragTools/DragToolInteraction.h"
#include "UObject/ObjectSaveContext.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"

//#define ENABLE_DEBUG_PRINTING

#include "SnappingUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EdModeInteractiveToolsContext)

DEFINE_LOG_CATEGORY_STATIC(LogInteractiveToolsContext, Log, All);

#define LOCTEXT_NAMESPACE "EdModeInteractiveToolsContext"

namespace UE::Editor::Private
{
	static TAutoConsoleVariable<int32> CVarUnboundedCursorDrag(
		TEXT("Editor.UnboundedCursorDrag"),
		0,
		TEXT("Whether to allow (ITF) mouse drag operations to continue beyond the viewport bounds.\n")
		TEXT("0: Do not allow mouse drag operations to continue beyond the viewport bounds (default)\n")
		TEXT("1: Allow mouse drag operations to continue beyond the viewport bounds"),
		ECVF_Default);
		
	static TAutoConsoleVariable<bool> CVarIgnoreModeLegacyGizmoRequirements(
		TEXT("ViewportInteractions.IgnoreModeRequirements"),
		false,
		TEXT("Whether to force on viewport interactions, ignoring legacy requests from editor modes.")
	);
	
	TAutoConsoleVariable<int32> CVarCursorOverrideFallback(
		TEXT("Editor.ITF.CursorOverrideFallback"),
		0,
		TEXT("How to handle the cursor when an ITF behavior has mouse capture but there is no cursor override.\n")
		TEXT("0 - Fall back to the legacy cursor determination path.\n")
		TEXT("1 - Show an error \"Slashed Circle\"\n")
	);
	
	TAutoConsoleVariable<int32> CVarEnableITFSoftwareCursor(
		TEXT("Editor.ITF.SoftwareCursorOverride"),
		2,
		TEXT("0 - No software cursor will be used\n")
		TEXT("1 - Software cursor will be used when requested\n")
		TEXT("2 - Software cursor will be used when requested, defaulting to On when set to \"Grab Hand\"\n"));
}

class FEdModeToolsContextQueriesImpl : public IToolsContextQueriesAPI
{
public:
	UEditorInteractiveToolsContext* ToolsContext;
	FEditorModeTools* EditorModeManager;

	FViewCameraState CachedViewState;
	FEditorViewportClient* CachedViewportClient;

	FEdModeToolsContextQueriesImpl(UEditorInteractiveToolsContext* Context, FEditorModeTools* InEditorModeManager)
	{
		ToolsContext = Context;
		EditorModeManager = InEditorModeManager;
	}

	void CacheCurrentViewState(FEditorViewportClient* ViewportClient)
	{
		CachedViewportClient = ViewportClient;
		FViewportCameraTransform ViewTransform = ViewportClient->GetViewTransform();
		CachedViewState.bIsOrthographic = ViewportClient->IsOrtho();
		CachedViewState.Position = ViewTransform.GetLocation();
		CachedViewState.HorizontalFOVDegrees = ViewportClient->ViewFOV;
		CachedViewState.AspectRatio = ViewportClient->AspectRatio;

		// ViewTransform rotation is only initialized for perspective!
		if (CachedViewState.bIsOrthographic == false)
		{
			// if using Orbit camera, the rotation in the ViewTransform is not the current camera rotation, it
			// is set to a different rotation based on the Orbit. So we have to convert back to camera rotation.
			FRotator ViewRotation = (ViewportClient->bUsingOrbitCamera) ? 
				ViewTransform.ComputeOrbitMatrix().InverseFast().Rotator()  :   ViewTransform.GetRotation();

			CachedViewState.Orientation = ViewRotation.Quaternion();
		}
		else
		{
			// These rotations are based on hardcoded values in EditorViewportClient.cpp, see switches in FEditorViewportClient::CalcSceneView and FEditorViewportClient::Draw
			switch (ViewportClient->ViewportType)
			{
			case LVT_OrthoXY:
				CachedViewState.Orientation = FQuat(FRotator(-90.0f, -90.0f, 0.0f));
				break;
			case LVT_OrthoNegativeXY:
				CachedViewState.Orientation = FQuat(FRotator(90.0f, 90.0f, 0.0f));
				break;
			case LVT_OrthoXZ:
				CachedViewState.Orientation = FQuat(FRotator(0.0f, -90.0f, 0.0f));
				break;
			case LVT_OrthoNegativeXZ:
				CachedViewState.Orientation = FQuat(FRotator(0.0f, 90.0f, 0.0f));
				break;
			case LVT_OrthoYZ:
				CachedViewState.Orientation = FQuat(FRotator(0.0f, 0.0f, 0.0f));
				break;
			case LVT_OrthoNegativeYZ:
				CachedViewState.Orientation = FQuat(FRotator(0.0f, 180.0f, 0.0f));
				break;
			default:
				CachedViewState.Orientation = FQuat::Identity;
			}

			CachedViewState.OrthoWorldCoordinateWidth = ViewportClient->GetOrthoUnitsPerPixel(ViewportClient->Viewport) * static_cast<float>(ViewportClient->Viewport->GetSizeXY().X);
		}

		CachedViewState.bIsVR = false;
	}

	virtual UWorld* GetCurrentEditingWorld() const override
	{
		return EditorModeManager->GetWorld();
	}

	virtual void GetCurrentSelectionState(FToolBuilderState& StateOut) const override
	{
		StateOut.ToolManager = ToolsContext->ToolManager;
		StateOut.TargetManager = ToolsContext->TargetManager;
		StateOut.GizmoManager = ToolsContext->GizmoManager;
		StateOut.World = EditorModeManager->GetWorld();
		EditorModeManager->GetSelectedActors()->GetSelectedObjects(StateOut.SelectedActors);
		EditorModeManager->GetSelectedComponents()->GetSelectedObjects(StateOut.SelectedComponents);

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		StateOut.TypedElementSelectionSet = EditorModeManager->GetEditorSelectionSet();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	virtual void GetCurrentViewState(FViewCameraState& StateOut) const override
	{
		StateOut = CachedViewState;
	}

	virtual EToolContextCoordinateSystem GetCurrentCoordinateSystem() const override
	{
		ECoordSystem CoordSys = EditorModeManager->GetCoordSystem();
		return (CoordSys == COORD_World) ? EToolContextCoordinateSystem::World : EToolContextCoordinateSystem::Local;
	}

	virtual EToolContextTransformGizmoMode GetCurrentTransformGizmoMode() const override
	{
		if (ToolsContext->GetForceCombinedGizmoModeEnabled() == false)
		{
			UE::Widget::EWidgetMode WidgetMode = EditorModeManager->GetWidgetMode();
			switch (WidgetMode)
			{
			case UE::Widget::EWidgetMode::WM_None:
				return EToolContextTransformGizmoMode::NoGizmo;
			case UE::Widget::EWidgetMode::WM_Translate:
				return EToolContextTransformGizmoMode::Translation;
			case UE::Widget::EWidgetMode::WM_Rotate:
				return EToolContextTransformGizmoMode::Rotation;
			case UE::Widget::EWidgetMode::WM_Scale:
				return EToolContextTransformGizmoMode::Scale;
			}
		}
		
		return EToolContextTransformGizmoMode::Combined;
	}

	virtual FToolContextSnappingConfiguration GetCurrentSnappingSettings() const override
	{
		const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();

		FToolContextSnappingConfiguration Config;
		Config.bIsSnappingActive = ViewportSettings->bSnappingActive;
		Config.bEnablePositionGridSnapping = (ViewportSettings->GridEnabled != 0);
		float EditorGridSize = GEditor->GetGridSize();
		Config.PositionGridDimensions = FVector(EditorGridSize, EditorGridSize, EditorGridSize);
		Config.bEnableRotationGridSnapping = (ViewportSettings->RotGridEnabled != 0);
		Config.RotationGridAngles = GEditor->GetRotGridSize();
		Config.bEnableScaleGridSnapping = (ViewportSettings->SnapScaleEnabled != 0);
		Config.ScaleGridSize = GEditor->GetScaleGridSize();
		Config.bEnableAbsoluteWorldSnapping = ToolsContext->GetAbsoluteWorldSnappingEnabled();

		Config.ObjectTransform.bEnable = (ViewportSettings->bEnableActorSnap != 0);
		Config.ObjectTransform.SnapDistance = FSnappingUtils::GetActorSnapDistance();
		
		return Config;
	}

	virtual UMaterialInterface* GetStandardMaterial(EStandardToolContextMaterials MaterialType) const
	{
		if (MaterialType == EStandardToolContextMaterials::VertexColorMaterial)
		{
			return ToolsContext->StandardVertexColorMaterial;
		}
		check(false);
		return nullptr;
	}

	virtual FViewport* GetHoveredViewport() const override
	{
		if (FEditorViewportClient* HoveredClient = EditorModeManager->GetHoveredViewportClient())
		{
			return HoveredClient->Viewport;
		}

		return nullptr;
	}

	virtual FViewport* GetFocusedViewport() const override
	{
		if (FEditorViewportClient* FocusedClient = EditorModeManager->GetFocusedViewportClient())
		{
			return FocusedClient->Viewport;
		}

		return nullptr;
	}
};

class FEdModeToolsContextTransactionImpl : public IToolsContextTransactionsAPI
{
public:
	FEdModeToolsContextTransactionImpl(UEditorInteractiveToolsContext* Context, FEditorModeTools* InEditorModeManager)
		: ToolsContext(Context)
		, EditorModeManager(InEditorModeManager)
	{
		check(EditorModeManager);
		check(ToolsContext);
	}


	virtual void DisplayMessage(const FText& Message, EToolMessageLevel Level) override
	{
		if (Level == EToolMessageLevel::UserNotification || Level == EToolMessageLevel::UserMessage)
		{
			UE_LOGF(LogInteractiveToolsContext, Display, "%ls", *Message.ToString());
			ToolsContext->PostToolNotificationMessage(Message);
		}
		else if (Level == EToolMessageLevel::UserWarning || Level == EToolMessageLevel::UserError)
		{
			UE_LOGF(LogInteractiveToolsContext, Warning, "%ls", *Message.ToString());
			ToolsContext->PostToolWarningMessage(Message);
		}
		else
		{
			UE_LOGF(LogInteractiveToolsContext, Warning, "%ls", *Message.ToString());
		}
	}


	virtual void PostInvalidation() override
	{
		ToolsContext->PostInvalidation();
	}

	virtual void BeginUndoTransaction(const FText& Description) override
	{
		Transaction = MakeUnique<FScopedTransaction>(Description);
	}

	virtual void EndUndoTransaction() override
	{
		Transaction.Reset();
	}
	
	virtual void CancelUndoTransaction() override
	{
		if (Transaction)
		{
			if (GUndo)
			{
				GUndo->Apply();
			}
			
			Transaction->Cancel();
			Transaction.Reset();
		}
	}

	virtual void AppendChange(UObject* TargetObject, TUniquePtr<FToolCommandChange> Change, const FText& Description) override
	{
		FScopedTransaction ScopedTransaction(Description);
		//if (ensure(GUndo != nullptr))		// ideally we would ensure here, but currently this can be hit on world teardown, resolution TBD
		if (GUndo != nullptr)
		{
			GUndo->StoreUndo(TargetObject, MoveTemp(Change));
		}
	}


	virtual bool RequestSelectionChange(const FSelectedObjectsChangeList& SelectionChange) override
	{
		checkf(SelectionChange.Components.Num() == 0, TEXT("FEdModeToolsContextTransactionImpl::RequestSelectionChange - Component selection not supported yet"));

		if (SelectionChange.ModificationType == ESelectedObjectsModificationType::Clear)
		{
			GEditor->SelectNone(true, true, false);
			return true;
		}

		if (SelectionChange.ModificationType == ESelectedObjectsModificationType::Replace )
		{
			GEditor->SelectNone(false, true, false);
		}

		bool bAdd = (SelectionChange.ModificationType != ESelectedObjectsModificationType::Remove);
		int NumActors = SelectionChange.Actors.Num();
		for (int k = 0; k < NumActors; ++k)
		{
			// Calling GEditor->NoteSelectionChange(true) will not send out change notifications on the TypedElementSelectionSet.
			// The selection change will work but any Editor stuff listening for changes will not be notified.
			// This may be a bug, needs further investigation.
			// In the meantime, just send out the notification on the last SelectActor() call
			bool bNotify = (k == NumActors-1);
			GEditor->SelectActor(SelectionChange.Actors[k], bAdd, bNotify, true, false);
		}

		GEditor->NoteSelectionChange(true);

		return true;
	}

protected:
	UEditorInteractiveToolsContext* ToolsContext;
	FEditorModeTools* EditorModeManager;
	
	TUniquePtr<FScopedTransaction> Transaction;
};




UEditorInteractiveToolsContext::UEditorInteractiveToolsContext()
{
	QueriesAPI = nullptr;
	TransactionAPI = nullptr;
}



void UEditorInteractiveToolsContext::Initialize(IToolsContextQueriesAPI* QueriesAPIIn, IToolsContextTransactionsAPI* TransactionsAPIIn)
{
	UInteractiveToolsContext::Initialize(QueriesAPIIn, TransactionsAPIIn);

	InvalidationTimestamp = 0;

	// This gets set up in UInteractiveToolsContext::Initialize;
	GizmoViewContext = ToolManager->GetContextObjectStore()->FindContext<UGizmoViewContext>();

	UToolsContextCursorAPI* ToolsContextCursorAPI = ToolManager->GetContextObjectStore()->FindContext<UToolsContextCursorAPI>();
	if (!ToolsContextCursorAPI)
	{
		ToolsContextCursorAPI = NewObject< UToolsContextCursorAPI >();
		ToolManager->GetContextObjectStore()->AddContextObject(ToolsContextCursorAPI);
	}
}

void UEditorInteractiveToolsContext::Shutdown()
{
	bIsActive = false;

	// auto-accept any in-progress tools
	DeactivateAllActiveTools(EToolShutdownType::Accept);

	UInteractiveToolsContext::Shutdown();
}


void UEditorInteractiveToolsContext::InitializeContextWithEditorModeManager(FEditorModeTools* InEditorModeManager, UInputRouter* UseInputRouter)
{
	check(InEditorModeManager);
	EditorModeManager = InEditorModeManager;

	this->TransactionAPI = new FEdModeToolsContextTransactionImpl(this, InEditorModeManager);
	this->QueriesAPI = new FEdModeToolsContextQueriesImpl(this, InEditorModeManager);

	SetCreateGizmoManagerFunc([this](const FContextInitInfo& ContextInfo)
	{
		UEditorInteractiveGizmoManager* NewGizmoManager = NewObject<UEditorInteractiveGizmoManager>(ContextInfo.ToolsContext);
		NewGizmoManager->InitializeWithEditorModeManager(ContextInfo.QueriesAPI, ContextInfo.TransactionsAPI, ContextInfo.InputRouter, EditorModeManager);
		NewGizmoManager->RegisterDefaultGizmos();
		return NewGizmoManager;
	});

	if (UseInputRouter != nullptr)
	{
		SetCreateInputRouterFunc([this, UseInputRouter](const FContextInitInfo& ContextInfo)
		{
			return UseInputRouter;
		});
		SetShutdownInputRouterFunc([this](UInputRouter*) {});
	}

	Initialize(QueriesAPI, TransactionAPI);

	if (UseInputRouter != nullptr)
	{
		// enable auto invalidation in Editor, because invalidating for all hover and capture events is unpleasant
		this->InputRouter->bAutoInvalidateOnHover = true;
		this->InputRouter->bAutoInvalidateOnCapture = true;
	}

	// set up standard materials
	StandardVertexColorMaterial = GEngine->VertexColorMaterial;
}


void UEditorInteractiveToolsContext::ShutdownContext()
{
	Shutdown();

	OnToolNotificationMessage.Clear();
	OnToolWarningMessage.Clear();

	if (QueriesAPI != nullptr)
	{
		delete QueriesAPI;
		QueriesAPI = nullptr;
	}

	if (TransactionAPI != nullptr)
	{
		delete TransactionAPI;
		TransactionAPI = nullptr;
	}
}


void UEditorInteractiveToolsContext::TerminateActiveToolsOnPIEStart()
{
	if (bDeactivateOnPIEStart)
	{
		DeactivateAllActiveTools(EToolShutdownType::Accept);	
	}
}
void UEditorInteractiveToolsContext::TerminateActiveToolsOnSaveWorld()
{
	if (bDeactivateOnSaveWorld)
	{
		DeactivateAllActiveTools(EToolShutdownType::Accept);
	}
}
void UEditorInteractiveToolsContext::TerminateActiveToolsOnWorldTearDown()
{
	DeactivateAllActiveTools(EToolShutdownType::Cancel);
}
void UEditorInteractiveToolsContext::TerminateActiveToolsOnLevelChange()
{
	if(bDeactivateOnLevelChange)
	{
		DeactivateAllActiveTools(EToolShutdownType::Cancel);
	}
}

void UEditorInteractiveToolsContext::PostInvalidation()
{
	InvalidationTimestamp++;
}

UWorld* UEditorInteractiveToolsContext::GetWorld() const
{
	if (bIsActive && EditorModeManager)
	{
		return EditorModeManager->GetWorld();
	}

	return nullptr;
}

void UEditorInteractiveToolsContext::PropagateInvalidations(FEditorViewportClient* ViewportClient)
{
	// Invalidate this viewport if its timestamp is not current.
	// During active tracking (e.g. gizmo drag), only invalidate the focused viewport
	const bool bSkipInvalidation = EditorModeManager && EditorModeManager->IsTracking() && ViewportClient != EditorModeManager->GetFocusedViewportClient();
	if (!bSkipInvalidation && ViewportClient)
	{
		int32& ViewportInvalidation = InvalidationMap.FindOrAdd(ViewportClient);
		if(ViewportInvalidation < InvalidationTimestamp)
		{
			ViewportClient->Invalidate(false, false);
			ViewportInvalidation = InvalidationTimestamp;
		}
	}
}

void UEditorInteractiveToolsContext::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
{
	if ( PendingToolShutdownType )
	{
		UInteractiveToolsContext::EndTool(EToolSide::Mouse, *PendingToolShutdownType);
		PendingToolShutdownType.Reset();
	}
	if ( PendingToolToStart )
	{
		if (UInteractiveToolsContext::StartTool(EToolSide::Mouse, *PendingToolToStart))
		{
			SetEditorStateForTool();
		}
		PendingToolToStart.Reset();
	}
	
	// This Tick() is called for every ViewportClient, however we only want to Tick the ToolManager and GizmoManager
	// once, for the 'Active'/Focused Viewport, so early-out here
	if (EditorModeManager && EditorModeManager->GetFocusedViewportClient() != ViewportClient)
	{
		return;
	}

	// Cache current camera state from this Viewport in the ContextQueries, which we will use for things like snapping/etc that
	// is computed by the Tool and Gizmo Tick()s
	// (This is not necessarily correct for Hover, because we might be Hovering over a different Viewport than the Active one...)
	if (ViewportClient)
	{
		((FEdModeToolsContextQueriesImpl*)this->QueriesAPI)->CacheCurrentViewState(ViewportClient);
	}

	// tick our stuff
	ToolManager->Tick(DeltaTime);
	GizmoManager->Tick(DeltaTime);
	OnTick.Broadcast(DeltaTime);
}



class FEdModeTempRenderContext : public IToolsContextRenderAPI
{
public:
	FPrimitiveDrawInterface* PDI;
	const FSceneView* SceneView;
	FViewCameraState ViewCameraState;
	EViewInteractionState ViewInteractionState;

	FEdModeTempRenderContext(const FSceneView* View, FViewport* Viewport, FEditorViewportClient* ViewportClient, FPrimitiveDrawInterface* DrawInterface, EViewInteractionState ViewInteractionState)
		:PDI(DrawInterface), SceneView(View), ViewInteractionState(ViewInteractionState)
	{
		CacheCurrentViewState(Viewport, ViewportClient);
	}

	virtual FPrimitiveDrawInterface* GetPrimitiveDrawInterface() override
	{
		return PDI;
	}

	virtual const FSceneView* GetSceneView() override
	{
		return SceneView;
	}

	virtual FViewCameraState GetCameraState() override
	{
		return ViewCameraState;
	}

	virtual EViewInteractionState GetViewInteractionState() override
	{
		return ViewInteractionState;
	}

	void CacheCurrentViewState(FViewport* Viewport, FEditorViewportClient* ViewportClient)
	{
		if (!ViewportClient)
		{
			return;
		}

		FViewportCameraTransform ViewTransform = ViewportClient->GetViewTransform();
		ViewCameraState.bIsOrthographic = ViewportClient->IsOrtho();
		ViewCameraState.Position = ViewTransform.GetLocation();
		ViewCameraState.HorizontalFOVDegrees = ViewportClient->ViewFOV;
		ViewCameraState.AspectRatio = ViewportClient->AspectRatio;
		ViewCameraState.DPIScale = ViewportClient->GetDPIScale();

		// ViewTransform rotation is only initialized for perspective!
		if (ViewCameraState.bIsOrthographic == false)
		{
			// if using Orbit camera, the rotation in the ViewTransform is not the current camera rotation, it
			// is set to a different rotation based on the Orbit. So we have to convert back to camera rotation.
			FRotator ViewRotation = (ViewportClient->bUsingOrbitCamera) ?
				ViewTransform.ComputeOrbitMatrix().InverseFast().Rotator() : ViewTransform.GetRotation();

			ViewCameraState.Orientation = ViewRotation.Quaternion();
		}
		else
		{
			// These rotations are based on hardcoded values in EditorViewportClient.cpp, see switches in FEditorViewportClient::CalcSceneView and FEditorViewportClient::Draw
			switch (ViewportClient->ViewportType)
			{
			case LVT_OrthoXY:
				ViewCameraState.Orientation = FQuat(FRotator(-90.0f, -90.0f, 0.0f));
				break;
			case LVT_OrthoNegativeXY:
				ViewCameraState.Orientation = FQuat(FRotator(90.0f, 90.0f, 0.0f));
				break;
			case LVT_OrthoXZ:
				ViewCameraState.Orientation = FQuat(FRotator(0.0f, -90.0f, 0.0f));
				break;
			case LVT_OrthoNegativeXZ:
				ViewCameraState.Orientation = FQuat(FRotator(0.0f, 90.0f, 0.0f));
				break;
			case LVT_OrthoYZ:
				ViewCameraState.Orientation = FQuat(FRotator(0.0f, 0.0f, 0.0f));
				break;
			case LVT_OrthoNegativeYZ:
				ViewCameraState.Orientation = FQuat(FRotator(0.0f, 180.0f, 0.0f));
				break;
			default:
				ViewCameraState.Orientation = FQuat::Identity;
			}

			ViewCameraState.OrthoWorldCoordinateWidth = ViewportClient->GetOrthoUnitsPerPixel(ViewportClient->Viewport) * static_cast<float>(ViewportClient->Viewport->GetSizeXY().X);
		}

		ViewCameraState.bIsVR = false;
	}

};


void UEditorInteractiveToolsContext::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	// skip HitProxy rendering passes if desired
	if (PDI->IsHitTesting() && bEnableRenderingDuringHitProxyPass == false)
	{
		return;
	}

	// THIS IS NOT SAFE!! However it appears that (1) it is only possible to get certain info from the EditorViewportClient,
	// but (2) there is no way to know if a FViewportClient is an FEditorViewportClient. Currently this ::Render() function
	// is only intended to be called by FEdMode/UEdMode::Render(), and their ::Render() calls are only called by the
	// FEditorViewportClient, which passes it's own Viewport down. So, this cast should be valid (for now)
	FEditorViewportClient* ViewportClient = static_cast<FEditorViewportClient*>(Viewport->GetClient());

	// Update the currently-hovered scene view information, which GizmoArrowComponent and friends will
	// use to recalculate their size/visibility/etc.
	if (GizmoViewContext && ViewportClient == EditorModeManager->GetHoveredViewportClient())
	{
		GizmoViewContext->ResetFromSceneView(*View);
		GizmoViewContext->SetDPIScale(ViewportClient->GetDPIScale());
	}

	// Render Tool and Gizmos
	const EViewInteractionState InteractionState = GetViewInteractionStateFromViewportClient(ViewportClient);

	FEdModeTempRenderContext RenderContext(View, Viewport, ViewportClient, PDI, InteractionState);
	ToolManager->Render(&RenderContext);
	GizmoManager->Render(&RenderContext);
	OnRender.Broadcast(&RenderContext);
}

void UEditorInteractiveToolsContext::DrawHUD(FViewportClient* ViewportClient,FViewport* Viewport,const FSceneView* View, FCanvas* Canvas)
{
	FEditorViewportClient* EditorViewportClient = static_cast<FEditorViewportClient*>(Viewport->GetClient());
	const EViewInteractionState InteractionState = GetViewInteractionStateFromViewportClient(EditorViewportClient);

	FEdModeTempRenderContext RenderContext(View, Viewport, EditorViewportClient, nullptr /*PDI*/, InteractionState);
	ToolManager->DrawHUD(Canvas, &RenderContext);
	GizmoManager->DrawHUD(Canvas, &RenderContext);
	OnDrawHUD.Broadcast(Canvas, &RenderContext);
}


bool UEditorInteractiveToolsContext::ProcessEditDelete()
{
	if (ToolManager->HasAnyActiveTool() == false)
	{
		return false;
	}

	bool bSkipDelete = false;

	USelection* SelectedActors = GEditor->GetSelectedActors();
	for (int i = 0; i < SelectedActors->Num() && bSkipDelete == false; ++i)
	{
		UObject* SelectedActor = SelectedActors->GetSelectedObject(i);

		// If any of the selected actors are AInternalToolFrameworkActor, we do not want to allow them to be deleted,
		// as generally this will cause problems for the Tool.
		if ( Cast<AInternalToolFrameworkActor>(SelectedActor) != nullptr)
		{
			bSkipDelete = true;
		}

		// If any Components of selected Actors implement UToolFrameworkComponent, we disable delete (for now).
		// (Currently Sculpt and a few other Modeling Tools attach their preview mesh components to the selected Actor)
		AActor* Actor = Cast<AActor>(SelectedActor);
		if (Actor != nullptr)
		{
			const TSet<UActorComponent*>& Components = Actor->GetComponents();
			for (const UActorComponent* Component : Components)
			{
				if ( Component->Implements<UToolFrameworkComponent>() )
				{
					bSkipDelete = true;
				}
			}
		}

	}

	return bSkipDelete;
}



FRay UEditorInteractiveToolsContext::GetRayFromMousePos(FEditorViewportClient* ViewportClient, FViewport* Viewport, int MouseX, int MouseY)
{
	constexpr bool bClampMouseToBounds = true;
	return GetRayFromMousePos(ViewportClient, Viewport, MouseX, MouseY, bClampMouseToBounds);
}

FRay UEditorInteractiveToolsContext::GetRayFromMousePos(FEditorViewportClient* ViewportClient, FViewport* Viewport, int MouseX, int MouseY, const bool bInClampMouseToBounds)
{
	if (!ViewportClient)
	{
		return FRay();
	}

	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		ViewportClient->Viewport,
		ViewportClient->GetScene(),
		ViewportClient->EngineShowFlags).SetRealtimeUpdate(ViewportClient->IsRealtime()));		// why SetRealtimeUpdate here??
	// this View is deleted by the FSceneViewFamilyContext destructor
	FSceneView* View = ViewportClient->CalcSceneView(&ViewFamily);
	FViewportCursorLocation MouseViewportRay(View, ViewportClient, MouseX, MouseY, bInClampMouseToBounds);

	FVector RayOrigin = MouseViewportRay.GetOrigin();
	FVector RayDirection = MouseViewportRay.GetDirection();

	// in Ortho views, the RayOrigin appears to be completely arbitrary, in some views it is on the view plane,
	// others it moves back/forth with the OrthoZoom. Translate by a large amount here in hopes of getting
	// ray origin "outside" the scene (which is a disaster for numerical precision !! ... )
	if (ViewportClient->IsOrtho())
	{
		RayOrigin -= UE_FLOAT_HUGE_DISTANCE * RayDirection;
	}

	return FRay(RayOrigin, RayDirection, true);
}


bool UEditorInteractiveToolsContext::CanStartTool(const FString ToolTypeIdentifier) const
{
	return UInteractiveToolsContext::CanStartTool(EToolSide::Mouse, ToolTypeIdentifier);
}

bool UEditorInteractiveToolsContext::HasActiveTool() const
{
	return UInteractiveToolsContext::HasActiveTool(EToolSide::Mouse);
}

FString UEditorInteractiveToolsContext::GetActiveToolName() const
{
	return UInteractiveToolsContext::GetActiveToolName(EToolSide::Mouse);
}

bool UEditorInteractiveToolsContext::ActiveToolHasAccept() const
{
	return  UInteractiveToolsContext::ActiveToolHasAccept(EToolSide::Mouse);
}

bool UEditorInteractiveToolsContext::CanAcceptActiveTool() const
{
	return UInteractiveToolsContext::CanAcceptActiveTool(EToolSide::Mouse);
}

bool UEditorInteractiveToolsContext::CanCancelActiveTool() const
{
	return UInteractiveToolsContext::CanCancelActiveTool(EToolSide::Mouse);
}

bool UEditorInteractiveToolsContext::CanCompleteActiveTool() const
{
	return UInteractiveToolsContext::CanCompleteActiveTool(EToolSide::Mouse);
}

void UEditorInteractiveToolsContext::StartTool(const FString ToolTypeIdentifier)
{
	FString LocalIdentifier(ToolTypeIdentifier);
	PendingToolToStart = LocalIdentifier;
	PostInvalidation();
}

void UEditorInteractiveToolsContext::EndTool(EToolShutdownType ShutdownType)
{
	PendingToolShutdownType = ShutdownType;
	PostInvalidation();
}

void UEditorInteractiveToolsContext::Activate()
{
	bIsActive = true;
}

void UEditorInteractiveToolsContext::Deactivate()
{
	bIsActive = false;
}


void UEditorInteractiveToolsContext::DeactivateActiveTool(EToolSide WhichSide, EToolShutdownType ShutdownType)
{
	UInteractiveToolsContext::DeactivateActiveTool(WhichSide, ShutdownType);
	RestoreEditorState();
}

void UEditorInteractiveToolsContext::DeactivateAllActiveTools(EToolShutdownType ShutdownType)
{
	UInteractiveToolsContext::DeactivateAllActiveTools(ShutdownType);
	RestoreEditorState();
}

void UEditorInteractiveToolsContext::SetEditorStateForTool()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor();
	if (LevelEditor.IsValid())
	{
		TArray<TSharedPtr<SLevelViewport>> Viewports = LevelEditor->GetViewports();
		for (const TSharedPtr<SLevelViewport>& ViewportWindow : Viewports)
		{
			if (ViewportWindow.IsValid())
			{
				FEditorViewportClient& Viewport = ViewportWindow->GetAssetViewportClient();
				Viewport.EnableOverrideEngineShowFlags([](FEngineShowFlags& Flags)
				{
					Flags.SetTemporalAA(false);
					Flags.SetMotionBlur(false);
					// disable this as depending on fixed exposure settings the entire scene may turn black
					//Flags.SetEyeAdaptation(false);
				});
			}
		}
	}
}

void UEditorInteractiveToolsContext::RestoreEditorState()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<ILevelEditor> LevelEditor  = LevelEditorModule.GetFirstLevelEditor();
	if (LevelEditor.IsValid())
	{
		TArray<TSharedPtr<SLevelViewport>> Viewports = LevelEditor->GetViewports();
		for (const TSharedPtr<SLevelViewport>& ViewportWindow : Viewports)
		{
			if (ViewportWindow.IsValid())
			{
				FEditorViewportClient& ViewportClient = ViewportWindow->GetAssetViewportClient();
				ViewportClient.DisableOverrideEngineShowFlags();

				// Rebuild the cached hit proxy. The tool may have disabled some viewport items while active (e.g. transform gizmo) and
				// so the hit proxy cache may be out of date. In the past this has led to, for example, the gizmo being visible but not
				// clickable when returning from a tool (UE-116888).
				// TODO: Figure out why the hit proxy is not being rebuilt elsewhere
				ViewportClient.RequestInvalidateHitProxy(ViewportClient.Viewport);
			}
		}
	}
}

void UEditorInteractiveToolsContext::OnToolEnded(UInteractiveToolManager* InToolManager, UInteractiveTool* InEndedTool)
{
	RestoreEditorState();
}

void UEditorInteractiveToolsContext::OnToolPostBuild(UInteractiveToolManager* InToolManager, EToolSide InSide, UInteractiveTool* InBuiltTool, UInteractiveToolBuilder* InToolBuilder, const FToolBuilderState& ToolState)
{
	// todo: Add any shared tool targets for the mode toolkit
}

EViewInteractionState UEditorInteractiveToolsContext::GetViewInteractionStateFromViewportClient(const FEditorViewportClient* InViewportClient) const
{
	if (!ensure(InViewportClient))
	{
		return EViewInteractionState::None;
	}

	const FEditorViewportClient* Focused = EditorModeManager->GetFocusedViewportClient();
	const FEditorViewportClient* Hovered = EditorModeManager->GetHoveredViewportClient();
	EViewInteractionState InteractionState = EViewInteractionState::None;
	if (InViewportClient == Focused )
	{
		InteractionState |= EViewInteractionState::Focused;
	}
	if (InViewportClient == Hovered )
	{
		InteractionState |= EViewInteractionState::Hovered;
	}

	return InteractionState;
}

void UEditorInteractiveToolsContext::SetEnableRenderingDuringHitProxyPass(bool bEnabled)
{
	bEnableRenderingDuringHitProxyPass = bEnabled;
}

void UEditorInteractiveToolsContext::SetForceCombinedGizmoMode(bool bEnabled)
{
	bForceCombinedGizmoMode = bEnabled;
}

void UEditorInteractiveToolsContext::SetAbsoluteWorldSnappingEnabled(bool bEnabled)
{
	bEnableAbsoluteWorldSnapping = bEnabled;
}

void UEditorInteractiveToolsContext::SetDeactivateToolsOnPIEStart(bool bDeactivateTools)
{
	bDeactivateOnPIEStart = bDeactivateTools;
}

void UEditorInteractiveToolsContext::SetDeactivateToolsOnSaveWorld(bool bDeactivateTools)
{
	bDeactivateOnSaveWorld = bDeactivateTools;
}

void UEditorInteractiveToolsContext::SetDeactivateToolsOnLevelChange(bool bDeactivateTools)
{
	bDeactivateOnLevelChange = bDeactivateTools;
}

void UModeManagerInteractiveToolsContext::PropagateInvalidations(FEditorViewportClient* ViewportClient)
{
	UEditorInteractiveToolsContext::PropagateInvalidations(ViewportClient);
	for (const TObjectPtr<UEdModeInteractiveToolsContext>& EdModeContext : EdModeToolsContexts)
	{
		EdModeContext->PropagateInvalidations(ViewportClient);
	}
}

void UModeManagerInteractiveToolsContext::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
{
	UEditorInteractiveToolsContext::Tick(ViewportClient, DeltaTime);
	for (const TObjectPtr<UEdModeInteractiveToolsContext>& EdModeContext : EdModeToolsContexts)
	{
		EdModeContext->Tick(ViewportClient, DeltaTime);
	}

	if (bViewportInteractionsDirty)
	{
		bViewportInteractionsDirty = false;
		
		if (const FEditorModeTools* ModeTools = GetParentEditorModeManager())
		{
			bModesSupportViewportInteractions = !ModeTools->RequiresLegacyViewportInteractions() || UE::Editor::Private::CVarIgnoreModeLegacyGizmoRequirements.GetValueOnGameThread();
			if (UEditorInteractiveGizmoManager* EditorGizmoManager = Cast<UEditorInteractiveGizmoManager>(GizmoManager))
			{
				EditorGizmoManager->SetAllowEditorGizmos(bModesSupportViewportInteractions);
			}
		}
		
		if (UsesViewportInteractions())
		{
			RegisterViewportInteractions();	
		}
		else
		{
			UnregisterViewportInteractions();
		}
	}

	if (ViewportInteractionsBehaviorSource)
	{
		ViewportInteractionsBehaviorSource->Tick(DeltaTime);
	}
}

void UModeManagerInteractiveToolsContext::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	UEditorInteractiveToolsContext::Render(View, Viewport, PDI);
	for (const TObjectPtr<UEdModeInteractiveToolsContext>& EdModeContext : EdModeToolsContexts)
	{
		EdModeContext->Render(View, Viewport, PDI);
	}

	if (ViewportInteractionsBehaviorSource)
	{
		FEditorViewportClient* EditorViewportClient = static_cast<FEditorViewportClient*>(Viewport->GetClient());
		const EViewInteractionState InteractionState = GetViewInteractionStateFromViewportClient(EditorViewportClient);

		FEdModeTempRenderContext RenderContext(View, Viewport, EditorViewportClient, PDI, InteractionState);
		ViewportInteractionsBehaviorSource->RenderTools(&RenderContext);
	}
}

void UModeManagerInteractiveToolsContext::DrawHUD(FViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas)
{
	UEditorInteractiveToolsContext::DrawHUD(ViewportClient, Viewport, View, Canvas);
	for (const TObjectPtr<UEdModeInteractiveToolsContext>& EdModeContext : EdModeToolsContexts)
	{
		EdModeContext->DrawHUD(ViewportClient, Viewport, View, Canvas);
	}

	if (ViewportInteractionsBehaviorSource)
	{
		FEditorViewportClient* EditorViewportClient = static_cast<FEditorViewportClient*>(Viewport->GetClient());
		const EViewInteractionState InteractionState = GetViewInteractionStateFromViewportClient(EditorViewportClient);

		FEdModeTempRenderContext RenderContext(View, Viewport, EditorViewportClient, nullptr /* canvas-based, no PDI */, InteractionState);
		ViewportInteractionsBehaviorSource->DrawTools(Canvas, &RenderContext);
	}
}


bool UModeManagerInteractiveToolsContext::ProcessEditDelete()
{
	bool bHandled = UEditorInteractiveToolsContext::ProcessEditDelete();
	for (const TObjectPtr<UEdModeInteractiveToolsContext>& EdModeContext : EdModeToolsContexts)
	{
		bHandled |= EdModeContext->ProcessEditDelete();
	}
	return bHandled;
}

void UModeManagerInteractiveToolsContext::DeactivateAllActiveTools(EToolShutdownType ShutdownType)
{
	for (UEdModeInteractiveToolsContext* EdModeContext : EdModeToolsContexts)
	{
		EdModeContext->DeactivateAllActiveTools(ShutdownType);
	}

	Super::DeactivateAllActiveTools(ShutdownType);
}

void UModeManagerInteractiveToolsContext::UpdateStateWithoutRoutingInputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event)
{
	// Currently, the only internal state we keep is the state of various mouse keys being down. Note
	// that we don't want to save bPressed or bReleased for them as those are one-time events issued
	// from InputKey, and shouldn't show up on mouse moves or on other keys being pressed/released.
	if ((Event == IE_Pressed || Event == IE_Released)
		&& Key.IsMouseButton())
	{
		if (Key.IsMouseButton())
		{
			if (Key == EKeys::LeftMouseButton)
			{
				CurrentMouseState.Mouse.Left.bDown = (Event == IE_Pressed);
			}
			else if (Key == EKeys::MiddleMouseButton)
			{
				CurrentMouseState.Mouse.Middle.bDown = (Event == IE_Pressed);
			}
			else if (Key == EKeys::RightMouseButton)
			{
				CurrentMouseState.Mouse.Right.bDown = (Event == IE_Pressed);
			}
		}
	}
}

bool UModeManagerInteractiveToolsContext::InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event)
{
#ifdef ENABLE_DEBUG_PRINTING
	if (Event == IE_Pressed) { UE_LOGF(LogInteractiveToolsContext, Warning, "PRESSED EVENT"); }
	else if (Event == IE_Released) { UE_LOGF(LogInteractiveToolsContext, Warning, "RELEASED EVENT"); }
	else if (Event == IE_Repeat) { UE_LOGF(LogInteractiveToolsContext, Warning, "REPEAT EVENT"); }
	else if (Event == IE_Axis) { UE_LOGF(LogInteractiveToolsContext, Warning, "AXIS EVENT"); }
	else if (Event == IE_DoubleClick) { UE_LOGF(LogInteractiveToolsContext, Warning, "DOUBLECLICK EVENT"); }
#endif

	// Update the current state, then route result
	UpdateStateWithoutRoutingInputKey(ViewportClient, Viewport, Key, Event);

	if (Event == IE_Pressed || Event == IE_Released || Event == IE_DoubleClick)
	{
		if (Key.IsMouseButton())
		{
			UpdateMousePosition(ViewportClient, Viewport, Viewport->GetMouseX(), Viewport->GetMouseY());
		
			bool bIsLeftMouse = (Key == EKeys::LeftMouseButton);
			bool bIsMiddleMouse = (Key == EKeys::MiddleMouseButton);
			bool bIsRightMouse = (Key == EKeys::RightMouseButton);
			if (bIsLeftMouse || bIsMiddleMouse || bIsRightMouse)
			{
				// Currently, we don't capture mouse clicks that start with Alt being down because we want
				// Alt camera manipulation to have priority over tools. So, we let those inputs pass on up
				// to wherever they get handled.
				// Someday these kinds of prioritizations will be handled by having camera manipulation be
				// in a common input router so that behavior priorities can determine the ordering.
				if (BypassAltModifier())
				{
					if (ViewportClient && ViewportClient->IsAltPressed() && InputRouter->HasActiveMouseCapture() == false)
					{
						return false;
					}
				}

				FInputDeviceState InputState = CurrentMouseState;
				InputState.InputDevice = EInputDevices::Mouse;

				FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
				InputState.SetModifierKeyStates(
					ModifierKeys.IsShiftDown(), ModifierKeys.IsAltDown(),
					ModifierKeys.IsControlDown(), ModifierKeys.IsCommandDown());

				// DoubleClick replaces a Press event. Include DoubleClick in Pressed/Down events so
				// that it can still be processed by single click behaviors. For explicit double click
				// handling, clients can listen for the DoubleClick event with a higher capture priority.
				const bool bIsPressed = (Event == IE_Pressed || Event == IE_DoubleClick);
				const bool bIsReleased = (Event == IE_Released);
				const bool bIsDoubleClick = (Event == IE_DoubleClick);
				if (bIsLeftMouse)
				{
					InputState.Mouse.Left.SetStates(bIsPressed, bIsPressed, bIsReleased, bIsDoubleClick);
				}
				else if (bIsMiddleMouse)
				{
					InputState.Mouse.Middle.SetStates(bIsPressed, bIsPressed, bIsReleased, bIsDoubleClick);
				}
				else
				{
					InputState.Mouse.Right.SetStates(bIsPressed, bIsPressed, bIsReleased, bIsDoubleClick);
				}
				if (InputRouter->PostInputEvent(InputState))
				{
					return true;
				}
			}
			else if (Key == EKeys::MouseScrollUp || Key == EKeys::MouseScrollDown)
			{
				// Note that we get two events for each scroll- an IE_Pressed, and IE_Released.
				// We pass both of these in, though only the first one will have WheelDelta set.
				// If a behavior captures the mouse wheel interaction, the second event will give
				// it the opportunity to immediately release capture. Not passing in the second
				// event would probably be ok too- capture would get released on next hover event.

				FInputDeviceState InputState = CurrentMouseState;
				InputState.InputDevice = EInputDevices::Mouse;

				FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
				InputState.SetModifierKeyStates(
					ModifierKeys.IsShiftDown(), ModifierKeys.IsAltDown(),
					ModifierKeys.IsControlDown(), ModifierKeys.IsCommandDown());

				InputState.Mouse.WheelDelta = (Event != IE_Pressed) ? 0.f
					: (Key == EKeys::MouseScrollUp) ? 1.f 
					: -1.f;

				return InputRouter->PostInputEvent(InputState);
			}
		}
		else if (Key.IsGamepadKey())
		{
			// not supported yet
		}
		else if (Key.IsTouch())
		{
			// not supported yet
		}
		else if (Key.IsAnalog())
		{
			// not supported yet
		}
		else    // is this definitely a keyboard key?
		{
			FInputDeviceState InputState;
			InputState.InputDevice = EInputDevices::Keyboard;

			FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
			InputState.SetModifierKeyStates(
				ModifierKeys.IsShiftDown(), ModifierKeys.IsAltDown(),
				ModifierKeys.IsControlDown(), ModifierKeys.IsCommandDown());

			InputState.Keyboard.ActiveKey.Button = Key;
			bool bPressed = (Event == IE_Pressed);
			InputState.Keyboard.ActiveKey.SetStates(bPressed, bPressed, !bPressed);
			InputState.OnGetButtonStateFallback.BindUObject(this, &UModeManagerInteractiveToolsContext::OnGetButtonStateFallback);
			return InputRouter->PostInputEvent(InputState);
		}
	}

	return false;
}

bool UModeManagerInteractiveToolsContext::MouseEnter(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y)
{
#ifdef ENABLE_DEBUG_PRINTING
	UE_LOG(LogInteractiveToolsContext, Warning, TEXT("MOUSE ENTER"));
#endif

	if (!bIsTrackingMouse)
	{
		UpdateMousePosition(ViewportClient, Viewport, x, y);
	}

	return false;
}


bool UModeManagerInteractiveToolsContext::MouseMove(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y)
{
#ifdef ENABLE_DEBUG_PRINTING
	UE_LOGF(LogInteractiveToolsContext, Warning, "HOVER %p", ViewportClient);
#endif

	UpdateMousePosition(ViewportClient, Viewport, x, y);

	FInputDeviceState InputState = CurrentMouseState;
	InputState.InputDevice = EInputDevices::Mouse;

	FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
	InputState.SetModifierKeyStates(
		ModifierKeys.IsShiftDown(), ModifierKeys.IsAltDown(),
		ModifierKeys.IsControlDown(), ModifierKeys.IsCommandDown());

	if (InputRouter->HasActiveMouseCapture())
	{
		// TODO: This should no longer be necessary: test and remove.
		// This state occurs if InputBehavior did not release capture on mouse release.
		// UMultiClickSequenceInputBehavior does this, eg for multi-click draw-polygon sequences.
		// It's not ideal though and maybe would be better done via multiple captures + hover...?
		InputRouter->PostInputEvent(InputState);
	}
	else
	{
		InputRouter->PostHoverInputEvent(InputState);
	}

	return false;
}

bool UModeManagerInteractiveToolsContext::LostFocus(FEditorViewportClient* InViewportClient, FViewport* Viewport) const
{
	InputRouter->ForceTerminateAll();

	return false;
}

bool UModeManagerInteractiveToolsContext::MouseLeave(FEditorViewportClient* ViewportClient, FViewport* Viewport)
{
#ifdef ENABLE_DEBUG_PRINTING
	UE_LOGF(LogInteractiveToolsContext, Warning, "MOUSE LEAVE");
#endif

	return false;
}



bool UModeManagerInteractiveToolsContext::StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	bIsTrackingMouse = InputRouter->HasActiveMouseCapture();
	return bIsTrackingMouse;
}

bool UModeManagerInteractiveToolsContext::CapturedMouseMove(FEditorViewportClient* InViewportClient, FViewport* InViewport, int32 InMouseX, int32 InMouseY)
{
	const bool bClampMouseToBoundsWhenCaptured = UE::Editor::Private::CVarUnboundedCursorDrag.GetValueOnGameThread() == 0;
	const bool bHasActiveMouseCapture = InputRouter->HasActiveMouseCapture();

	FVector2D OldPosition = CurrentMouseState.Mouse.Position2D;
	CurrentMouseState.Mouse.Position2D = FVector2D(InMouseX, InMouseY);
	CurrentMouseState.Mouse.WorldRay = GetRayFromMousePos(InViewportClient, InViewport, InMouseX, InMouseY, bHasActiveMouseCapture && bClampMouseToBoundsWhenCaptured);

	// Unbound cursor
	{
		FIntPoint UnboundedMousePos;
		InViewport->GetUnboundedMousePos(UnboundedMousePos);
		const FVector2D UnboundedPosition = UnboundedMousePos;

		CurrentMouseState.Mouse.UnboundedMouseDelta2D = UnboundedPosition - CurrentMouseState.Mouse.UnboundedPosition2D;
		CurrentMouseState.Mouse.UnboundedPosition2D = UnboundedPosition;
		CurrentMouseState.Mouse.UnboundedWorldRay = GetRayFromMousePos(InViewportClient, InViewport, UnboundedPosition.X, UnboundedPosition.Y, false);
	}

	if (bHasActiveMouseCapture)
	{
#ifdef ENABLE_DEBUG_PRINTING
		UE_LOGF(LogInteractiveToolsContext, Warning, "CAPTURED MOUSE MOVE");
#endif

		FInputDeviceState InputState = CurrentMouseState;
		InputState.InputDevice = EInputDevices::Mouse;
		
		FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
		InputState.SetModifierKeyStates(
			ModifierKeys.IsShiftDown(), ModifierKeys.IsAltDown(),
			ModifierKeys.IsControlDown(), ModifierKeys.IsCommandDown());

		InputState.Mouse.Delta2D = CurrentMouseState.Mouse.Position2D - OldPosition;
		InputRouter->PostInputEvent(InputState);
		return true;
	}

	return false;
}

bool UModeManagerInteractiveToolsContext::EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	if (bIsTrackingMouse)
	{
		// If the input router captured the mouse input, we need to invalidate the viewport client here, since the mouse delta tracker's end tracking will not be called.
		constexpr bool bForceChildViewportRedraw = true;
		constexpr bool bInvalidateHitProxies = true;
		if (InViewportClient)
		{
			InViewportClient->Invalidate(bForceChildViewportRedraw, bInvalidateHitProxies);
		}
		bIsTrackingMouse = false;
		return true;
	}

	return false;
}

void UModeManagerInteractiveToolsContext::UpdateMousePosition(FEditorViewportClient* InViewportClient, FViewport* InViewport, int32 X, int32 Y)
{
	FVector2D NewPosition(X, Y);
	CurrentMouseState.Mouse.Position2D = NewPosition;
	CurrentMouseState.Mouse.WorldRay = GetRayFromMousePos(InViewportClient, InViewport, X, Y);

	if (InViewport)
	{
		FIntPoint UnboundedMousePos;
		InViewport->GetUnboundedMousePos(UnboundedMousePos);

		CurrentMouseState.Mouse.UnboundedPosition2D = UnboundedMousePos;
		CurrentMouseState.Mouse.UnboundedWorldRay = GetRayFromMousePos(InViewportClient, InViewport, UnboundedMousePos.X, UnboundedMousePos.Y, false);
	}
}

void UModeManagerInteractiveToolsContext::OnApplicationFocusChanged(bool bInIsFocused)
{
	if (!bInIsFocused)
	{
		// Any pressed mouse or modifier keys need to be reset as they are no longer reliable.	
		FInputDeviceState ResetState = FInputDeviceState();
		ResetState.OnGetButtonStateFallback = CurrentMouseState.OnGetButtonStateFallback;
		CurrentMouseState = ResetState;
	}
}

bool UModeManagerInteractiveToolsContext::GetCursor(EMouseCursor::Type& OutCursor) const
{	
	UToolsContextCursorAPI* ToolsContextCursorAPI = ToolManager->GetContextObjectStore()->FindContext<UToolsContextCursorAPI>();

	if (ToolsContextCursorAPI && ToolsContextCursorAPI->IsCursorOverridden())
	{
		OutCursor = ToolsContextCursorAPI->GetCurrentCursorOverride();
		return true;
	}

	if (bIsTrackingMouse && UE::Editor::Private::CVarCursorOverrideFallback.GetValueOnAnyThread() == 1)
	{
		OutCursor = EMouseCursor::SlashedCircle;				
		return true;
	}

	return false;
}

TSharedPtr<SWidget> UModeManagerInteractiveToolsContext::GetCursorWidget() const
{
	const UToolsContextCursorAPI* ToolsContextCursorAPI = ToolManager->GetContextObjectStore()->FindContext<UToolsContextCursorAPI>();
	if (ToolsContextCursorAPI
		&& ToolsContextCursorAPI->IsCursorOverridden()
		&& ToolsContextCursorAPI->GetCurrentCursorOverride() == EMouseCursor::Custom)
	{
		return ToolsContextCursorAPI->GetCurrentCursorOverrideWidget();
	}

	return nullptr;
}

bool UModeManagerInteractiveToolsContext::GetCursorVisibilityOverride(bool& bOutHardwareCursorVisible, bool& bOutSoftwareCursorVisible) const
{
	const int32 ITFSoftwareCursor = UE::Editor::Private::CVarEnableITFSoftwareCursor.GetValueOnAnyThread();
	if (ITFSoftwareCursor > 0 && InputRouter->HasActiveMouseCapture())
	{
		if (const UToolsContextCursorAPI* CursorAPI = ToolManager->GetContextObjectStore()->FindContext<UToolsContextCursorAPI>())
		{
			if (CursorAPI->IsCursorOverridden())
			{
				const TOptional<bool> SoftwareCursorOverride = CursorAPI->GetUseSoftwareCursorOverride();
				if (SoftwareCursorOverride.IsSet())
				{
					bOutSoftwareCursorVisible = SoftwareCursorOverride.GetValue();
					bOutHardwareCursorVisible = !bOutSoftwareCursorVisible;
					return true;
				}
				
				if (ITFSoftwareCursor == 2 && CursorAPI->GetCurrentCursorOverride() == EMouseCursor::GrabHandClosed)
				{
					// The logic here is twofold:
					// 1 - A "grab hand" is usually manipulating something visually in the viewport
					// 2 - We want the connection between the cursor and the thing its manipulating to be rock-solid.
					// The hardware cursor is disconnected from the frame rate & rendering latency of the viewport.
					// This causes dragged objects to visually lag behind the cursor. The hardware cursor rendering has less latency than the full scene. 
					// Using the software cursor helps lend a sense of solidity.
					// There is no discrepancy between the position of the cursor and the position of the object
					// as both are rendered within the same system with the same latency.
					// Thus, defaulting the GrabHand to using the software cursor should result in a net-benefit to interaction solidity.
					bOutHardwareCursorVisible = false;
					bOutSoftwareCursorVisible = true;
					return true;
				}
				
				bOutHardwareCursorVisible = true;
				bOutSoftwareCursorVisible = false;
				return true;
			}
		}
	}
	return false;
}

FRay UModeManagerInteractiveToolsContext::GetLastWorldRay() const
{
	return CurrentMouseState.Mouse.WorldRay;
}

FEditorViewportClient* UModeManagerInteractiveToolsContext::GetHoveredViewportClient() const
{
	FEditorViewportClient* HoveredViewportClient = nullptr;

	if (const FEditorModeTools* const ModeManager = GetParentEditorModeManager())
	{
		HoveredViewportClient = ModeManager->GetHoveredViewportClient();
	}

	if (!HoveredViewportClient)
	{
		if (ToolManager)
		{
			if (const IToolsContextQueriesAPI* const ContextQueriesAPI = ToolManager->GetContextQueriesAPI())
			{
				if (const FViewport* const Viewport = ContextQueriesAPI->GetHoveredViewport())
				{
					HoveredViewportClient = static_cast<FEditorViewportClient*>(Viewport->GetClient());
				}
			}
		}
	}

	return HoveredViewportClient;
}

FEditorViewportClient* UModeManagerInteractiveToolsContext::GetFocusedViewportClient() const
{
	FEditorViewportClient* FocusedViewportClient = nullptr;

	if (const FEditorModeTools* const ModeManager = GetParentEditorModeManager())
	{
		FocusedViewportClient = ModeManager->GetFocusedViewportClient();
	}

	if (!FocusedViewportClient)
	{
		if (ToolManager)
		{
			if (const IToolsContextQueriesAPI* const ContextQueriesAPI = ToolManager->GetContextQueriesAPI())
			{
				if (const FViewport* const Viewport = ContextQueriesAPI->GetFocusedViewport())
				{
					FocusedViewportClient = static_cast<FEditorViewportClient*>(Viewport->GetClient());
				}
			}
		}
	}

	return FocusedViewportClient;
}

void UModeManagerInteractiveToolsContext::Initialize(IToolsContextQueriesAPI* QueriesAPIIn, IToolsContextTransactionsAPI* TransactionsAPIIn)
{
	Super::Initialize(QueriesAPIIn, TransactionsAPIIn);

	BeginPIEDelegateHandle = FEditorDelegates::BeginPIE.AddLambda([this](bool bSimulating)
	{
		TerminateActiveToolsOnPIEStart();
	});
	PreSaveWorldDelegateHandle = FEditorDelegates::PreSaveWorldWithContext.AddLambda([this](UWorld* World, FObjectPreSaveContext ObjectSaveContext)
	{
		TerminateActiveToolsOnSaveWorld();
	});

	FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	WorldTearDownDelegateHandle = LevelEditor.OnMapChanged().AddLambda([this](UWorld* World, EMapChangeType ChangeType)
	{
		if (ChangeType == EMapChangeType::TearDownWorld)
		{
			TerminateActiveToolsOnWorldTearDown();
		}
	});

	// Tools frequently spawn temporary objects in the level for visualization, gizmos, etc, so having a level be
	// removed from the world puts the tools at risk of letting those objects be garbage collected and cause crashes
	FWorldDelegates::PreLevelRemovedFromWorld.AddWeakLambda(this, [this](ULevel*, UWorld*) {
		TerminateActiveToolsOnLevelChange();
	});

	ToolManager->OnToolEnded.AddUObject(this, &UModeManagerInteractiveToolsContext::OnToolEnded);
	ToolManager->OnToolPostBuild.AddUObject(this, &UModeManagerInteractiveToolsContext::OnToolPostBuild);

	// if viewport clients change we will discard our overrides as we aren't sure what happened
	ViewportClientListChangedHandle = GEditor->OnViewportClientListChanged().AddLambda([this]()
	{
		RestoreEditorState();
	});

	GetParentEditorModeManager()->OnEditorModeIDChanged().AddUObject(this, &UModeManagerInteractiveToolsContext::OnEditorModeChanged);

	// Delegates used to handle InputTools.EnableITFTools CVAR changes
	UE::Editor::ViewportInteractions::OnEditorViewportInteractionsActivated().AddWeakLambda(this, [this]() { bViewportInteractionsDirty = true; });
	UE::Editor::ViewportInteractions::OnEditorViewportInteractionsDeactivated().AddWeakLambda(this, [this]() { bViewportInteractionsDirty = true; });
	UEditorInteractiveGizmoManager::OnUsesNewTRSGizmosChangedDelegate().AddWeakLambda(this, [this](const bool) { bViewportInteractionsDirty = true; });
	
	CurrentMouseState.OnGetButtonStateFallback.BindUObject(this, &UModeManagerInteractiveToolsContext::OnGetButtonStateFallback);
	
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().OnApplicationActivationStateChanged().AddUObject(this, &UModeManagerInteractiveToolsContext::OnApplicationFocusChanged);
	}
}

void UModeManagerInteractiveToolsContext::Shutdown()
{
	UnregisterViewportInteractions();

	GetParentEditorModeManager()->OnEditorModeIDChanged().RemoveAll(this);
	UEditorInteractiveGizmoManager::OnUsesNewTRSGizmosChangedDelegate().RemoveAll(this);
	UE::Editor::ViewportInteractions::OnEditorViewportInteractionsActivated().RemoveAll(this);
	UE::Editor::ViewportInteractions::OnEditorViewportInteractionsDeactivated().RemoveAll(this);

	if (ToolManager)
	{
		ToolManager->OnToolPostBuild.RemoveAll(this);
		ToolManager->OnToolEnded.RemoveAll(this);
	}

	if (FLevelEditorModule* LevelEditor = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor"))
	{
		LevelEditor->OnMapChanged().Remove(WorldTearDownDelegateHandle);
		FEditorDelegates::BeginPIE.Remove(BeginPIEDelegateHandle);
		FEditorDelegates::PreSaveWorldWithContext.Remove(PreSaveWorldDelegateHandle);
		GEditor->OnViewportClientListChanged().Remove(ViewportClientListChangedHandle);
	}

	FWorldDelegates::PreLevelRemovedFromWorld.RemoveAll(this);

	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().OnApplicationActivationStateChanged().RemoveAll(this);
	}

	Super::Shutdown();
}

FDeviceButtonState UModeManagerInteractiveToolsContext::OnGetButtonStateFallback(const FKey& Key) const
{
	if (FEditorViewportClient* Client = GetFocusedViewportClient())
	{
		if (Client->Viewport->KeyState(Key))
		{
			FDeviceButtonState State(Key);
			State.bDown = true;
			return State;
		}
	}
	return FDeviceButtonState();
}

UEdModeInteractiveToolsContext* UModeManagerInteractiveToolsContext::CreateNewChildEdModeToolsContext()
{
	UEdModeInteractiveToolsContext* NewModeToolsContext = 
		NewObject<UEdModeInteractiveToolsContext>(GetTransientPackage(), UEdModeInteractiveToolsContext::StaticClass(), NAME_None, RF_Transient);
	NewModeToolsContext->InitializeContextFromModeManagerContext(this);

	return NewModeToolsContext;
}

bool UModeManagerInteractiveToolsContext::OnChildEdModeActivated(UEdModeInteractiveToolsContext* ChildToolsContext)
{
	if (!ensureMsgf(EdModeToolsContexts.Find(ChildToolsContext), TEXT("Child ToolsContext was already found!")))
	{
		return false;
	}

	EdModeToolsContexts.Add(ChildToolsContext);
	ChildToolsContext->Activate();
	return true;
}


bool UModeManagerInteractiveToolsContext::OnChildEdModeDeactivated(UEdModeInteractiveToolsContext* ChildToolsContext)
{
	for (int32 k = 0; k < EdModeToolsContexts.Num(); ++k)
	{
		if (EdModeToolsContexts[k] == ChildToolsContext)
		{
			ChildToolsContext->DeactivateAllActiveTools(EToolShutdownType::Cancel);
			ChildToolsContext->Deactivate();
			EdModeToolsContexts.RemoveAt(k);
			return true;
		}
	}
	ensureMsgf(false, TEXT("Child ToolsContext was not found! It may have already been removed"));
	return false;
}

void UModeManagerInteractiveToolsContext::SetBypassAltModifier(bool bInBypassAltModifier)
{
	bBypassAltModifier = bInBypassAltModifier;
}

void UModeManagerInteractiveToolsContext::SetViewportInteractionsEnabled(bool bInEnabled)
{
	if (bUsesViewportInteractions != bInEnabled)
	{
		bViewportInteractionsDirty = true;
		bUsesViewportInteractions = bInEnabled;
		if (!bInEnabled)
		{
			UnregisterViewportInteractions();
		}
	}
}

bool UModeManagerInteractiveToolsContext::UsesViewportInteractions() const
{
	return bUsesViewportInteractions && bModesSupportViewportInteractions && UE::Editor::ViewportInteractions::UseEditorViewportInteractions();
}

UViewportInteractionsBehaviorSource* UModeManagerInteractiveToolsContext::GetViewportInteractionsBehaviorSource()
{
	if (!ViewportInteractionsBehaviorSource && UsesViewportInteractions())
	{
		RegisterViewportInteractions();
	}

	return ViewportInteractionsBehaviorSource.Get();
}

void UModeManagerInteractiveToolsContext::RegisterViewportInteractions()
{
	if (!UsesViewportInteractions())
	{
		return;
	}
	
	if (ViewportInteractionsBehaviorSource)
	{
		ViewportInteractionsBehaviorSource->Reset();
	}
	else
	{
		SetBypassAltModifier(false);
		
		ViewportInteractionsBehaviorSource = NewObject<UViewportInteractionsBehaviorSource>();
		ViewportInteractionsBehaviorSource->Initialize(this);
	}
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	OnRegisterViewportInteractions().Broadcast();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	TArray<FName> UnsupportedInteractions;
	OnGetUnsupportedInteractions().Broadcast(UnsupportedInteractions);
	ViewportInteractionsBehaviorSource->SetUnsupportedViewportInteractions(UnsupportedInteractions);
	OnBuildViewportInteractions().Broadcast(ViewportInteractionsBehaviorSource);
	PostBuildViewportInteractions().Broadcast(ViewportInteractionsBehaviorSource);

	ViewportInteractionsBehaviorSource->RegisterBehaviorSources();
}

void UModeManagerInteractiveToolsContext::UnregisterViewportInteractions()
{
	if (ViewportInteractionsBehaviorSource)
	{
		// Revert to previous state, with Alt Modifier not being processed
		SetBypassAltModifier(true);
		ViewportInteractionsBehaviorSource->DeregisterBehaviorSources();
		ViewportInteractionsBehaviorSource = nullptr;
		
		UToolMenus::UnregisterOwner(this);
	}
}

void UModeManagerInteractiveToolsContext::OnEditorModeChanged(const FEditorModeID& InModeID, bool bInIsEnteringMode)
{
	bViewportInteractionsDirty = true;
}

void UEdModeInteractiveToolsContext::InitializeContextFromModeManagerContext(UModeManagerInteractiveToolsContext* ModeManagerToolsContext)
{
	check(ModeManagerToolsContext != nullptr);
	check(ModeManagerToolsContext->InputRouter != nullptr);
	FEditorModeTools* ModeManager = ModeManagerToolsContext->GetParentEditorModeManager();
	check(ModeManager != nullptr);

	ParentModeManagerToolsContext = ModeManagerToolsContext;


	SetCreateContextStoreFunc([this, ModeManagerToolsContext](const FContextInitInfo& ContextInfo)
	{
		UContextObjectStore* NewContextStore = NewObject<UContextObjectStore>(ModeManagerToolsContext->ContextObjectStore);
		return NewContextStore;
	});

	InitializeContextWithEditorModeManager(ModeManager, ModeManagerToolsContext->InputRouter);
}


FRay UEdModeInteractiveToolsContext::GetLastWorldRay() const
{
	return ParentModeManagerToolsContext->GetLastWorldRay();
}

#undef LOCTEXT_NAMESPACE
