// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/EditorComponentSourceFactory.h"

#include "Engine/StaticMesh.h"
#include "Components/PrimitiveComponent.h"
#include "Components/StaticMeshComponent.h"
#include "StaticMeshAttributes.h"
#include "ComponentReregisterContext.h"
#include "PhysicsEngine/BodySetup.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "TargetInterfaces/MaterialProvider.h"

static void DisplayCriticalWarningMessage(const FString& Message)
{
	FNotificationInfo Info(FText::FromString(Message));
	Info.ExpireDuration = 5.0f;
	FSlateNotificationManager::Get().AddNotification(Info);

	UE_LOGF(LogTemp, Warning, "%ls", *Message);
}


FStaticMeshComponentTarget::FStaticMeshComponentTarget(UPrimitiveComponent* Component, EMeshLODIdentifier EditingLODIn)
	: FPrimitiveComponentTarget(Cast<UStaticMeshComponent>(Component))
{
	EditingLOD = EMeshLODIdentifier::LOD0;

	UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component);
	if (ensure(StaticMeshComponent != nullptr))
	{
		UStaticMesh* StaticMeshAsset = StaticMeshComponent->GetStaticMesh();
		if (ensure(StaticMeshAsset != nullptr))
		{
			if (EditingLODIn == EMeshLODIdentifier::MaxQuality)
			{
				EditingLOD = StaticMeshAsset->IsHiResMeshDescriptionValid() ? EMeshLODIdentifier::HiResSource : EMeshLODIdentifier::LOD0;
			}
			else if (EditingLODIn == EMeshLODIdentifier::HiResSource)
			{
				EditingLOD = StaticMeshAsset->IsHiResMeshDescriptionValid() ? EMeshLODIdentifier::HiResSource : EMeshLODIdentifier::LOD0;
				if (EditingLOD != EMeshLODIdentifier::HiResSource)
				{
					DisplayCriticalWarningMessage(FString(TEXT("HiRes Source selected but not available - Falling Back to LOD0")));
				}
			}
			else
			{
				int32 WantLOD = (int)EditingLODIn;
				int32 MaxExistingLOD = StaticMeshAsset->GetNumSourceModels() - 1;
				if (WantLOD > MaxExistingLOD)
				{
					DisplayCriticalWarningMessage(FString::Printf(TEXT("LOD%d Requested but not available - Falling Back to LOD%d"), WantLOD, MaxExistingLOD));
					EditingLOD = (EMeshLODIdentifier)MaxExistingLOD;
				}
			}
		}
	}
}




bool FStaticMeshComponentTarget::IsValid() const
{
	if (!FPrimitiveComponentTarget::IsValid())
	{
		return false;
	}
	UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component);
	if (StaticMeshComponent == nullptr)
	{
		return false;
	}
	UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
	if (StaticMesh == nullptr)
	{
		return false;
	}

	if (EditingLOD == EMeshLODIdentifier::HiResSource)
	{
		if (StaticMesh->IsHiResMeshDescriptionValid() == false)
		{
			return false;
		}
	}
	else if ( (int32)EditingLOD >= StaticMesh->GetNumSourceModels() )
	{
		return false;
	}

	return true;
}

FMeshDescription* FStaticMeshComponentTarget::GetMesh() 
{
	if (ensure(IsValid()))
	{
		UStaticMesh* StaticMesh = Cast<UStaticMeshComponent>(Component)->GetStaticMesh();
		return (EditingLOD == EMeshLODIdentifier::HiResSource) ?
			StaticMesh->GetHiResMeshDescription() : StaticMesh->GetMeshDescription((int32)EditingLOD);
	}
	return nullptr;
}


void FStaticMeshComponentTarget::GetMaterialSet(FComponentMaterialSet& MaterialSetOut, bool bAssetMaterials) const
{
	if (ensure(IsValid()) == false) return;

	if (bAssetMaterials)
	{
		UStaticMesh* StaticMesh = Cast<UStaticMeshComponent>(Component)->GetStaticMesh();
		int32 NumMaterials = Component->GetNumMaterials();
		MaterialSetOut.Materials.SetNum(NumMaterials);
		for (int32 k = 0; k < NumMaterials; ++k)
		{
			MaterialSetOut.Materials[k] = StaticMesh->GetMaterial(k);
		}
	}
	else
	{
		FPrimitiveComponentTarget::GetMaterialSet(MaterialSetOut, false);
	}
}


void FStaticMeshComponentTarget::CommitMaterialSetUpdate(const FComponentMaterialSet& MaterialSet, bool bApplyToAsset)
{
	if (ensure(IsValid()) == false) return;

	if (bApplyToAsset)
	{
		UStaticMesh* StaticMesh = Cast<UStaticMeshComponent>(Component)->GetStaticMesh();

		if (StaticMesh->GetPathName().StartsWith(TEXT("/Engine/")))
		{
			UE_LOGF(LogTemp, Warning, "CANNOT MODIFY BUILT-IN ENGINE ASSET %ls", *StaticMesh->GetPathName());
			return;
		}

		// flush any pending rendering commands, which might touch this component while we are rebuilding its mesh
		FlushRenderingCommands();

		// unregister the component while we update it's static mesh
		TUniquePtr<FComponentReregisterContext> ComponentReregisterContext = MakeUnique<FComponentReregisterContext>(Component);

		// make sure transactional flag is on
		StaticMesh->SetFlags(RF_Transactional);

		StaticMesh->Modify();

		int NewNumMaterials = MaterialSet.Materials.Num();
		if (NewNumMaterials != StaticMesh->GetStaticMaterials().Num())
		{
			StaticMesh->GetStaticMaterials().SetNum(NewNumMaterials);
		}
		for (int k = 0; k < NewNumMaterials; ++k)
		{
			if (StaticMesh->GetMaterial(k) != MaterialSet.Materials[k])
			{
				StaticMesh->SetMaterial(k, MaterialSet.Materials[k]);
			}
		}

		StaticMesh->PostEditChange();
	}
	else
	{
		int32 NumMaterialsNeeded = Component->GetNumMaterials();
		int32 NumMaterialsGiven = MaterialSet.Materials.Num();

		// We wrote the below code to support a mismatch in the number of materials.
		// However, it is not yet clear whether this might be desirable, and we don't
		// want to inadvertantly hide bugs in the meantime. So, we keep this ensure here
		// for now, and we can remove it if we decide that we want the ability.
		ensure(NumMaterialsNeeded == NumMaterialsGiven);

		check(NumMaterialsGiven > 0);

		for (int32 i = 0; i < NumMaterialsNeeded; ++i)
		{
			int32 MaterialToUseIndex = FMath::Min(i, NumMaterialsGiven - 1);
			Component->SetMaterial(i, MaterialSet.Materials[MaterialToUseIndex]);
		}
	}
	
}


bool FStaticMeshComponentTarget::HasSameSourceData(const FPrimitiveComponentTarget& OtherTarget) const
{
	if (ensure(IsValid()))
	{
		const UStaticMesh* StaticMesh = Cast<UStaticMeshComponent>(Component)->GetStaticMesh();
		const UStaticMesh* OtherStaticMesh = Cast<UStaticMeshComponent>(OtherTarget.Component)->GetStaticMesh();
		return StaticMesh && StaticMesh == OtherStaticMesh;
	}
	else
	{
		return false;
	}
}

void FStaticMeshComponentTarget::CommitMesh( const FCommitter& Committer )
{
	if (ensure(IsValid()) == false) return;

	//bool bSaved = Component->Modify();
	//check(bSaved);
	UStaticMesh* StaticMesh = Cast<UStaticMeshComponent>(Component)->GetStaticMesh();

	if (StaticMesh->GetPathName().StartsWith(TEXT("/Engine/")))
	{
		DisplayCriticalWarningMessage(FString::Printf(TEXT("CANNOT MODIFY BUILT-IN ENGINE ASSET %s"), *StaticMesh->GetPathName()));
		return;
	}

	// flush any pending rendering commands, which might touch this component while we are rebuilding it's mesh
	FlushRenderingCommands();

	// unregister the component while we update it's static mesh
	TUniquePtr<FComponentReregisterContext> ComponentReregisterContext = MakeUnique<FComponentReregisterContext>(Component);

	// make sure transactional flag is on for this asset
	StaticMesh->SetFlags(RF_Transactional);

	verify(StaticMesh->Modify());
	if (EditingLOD == EMeshLODIdentifier::HiResSource)
	{
		verify(StaticMesh->ModifyHiResMeshDescription());
	}
	else
	{
		verify(StaticMesh->ModifyMeshDescription((int32)EditingLOD));
	}

	FCommitParams CommitParams;
	CommitParams.MeshDescription = GetMesh();

	Committer(CommitParams);

	if (EditingLOD == EMeshLODIdentifier::HiResSource)
	{
		StaticMesh->CommitHiResMeshDescription();
	}
	else
	{
		StaticMesh->CommitMeshDescription( (int32)EditingLOD );
	}

	StaticMesh->PostEditChange();

	// this rebuilds physics, but it doesn't undo!
	Component->RecreatePhysicsState();

	//Component->PostEditChange();
}

bool FStaticMeshComponentTargetFactory::CanBuild(UActorComponent* Component)
{
	UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component);
	if (StaticMeshComponent)
	{
		UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
		if (StaticMesh)
		{
			return (StaticMesh->GetNumSourceModels() > 0);
		}
	}
	return false;
}

TUniquePtr<FPrimitiveComponentTarget> FStaticMeshComponentTargetFactory::Build(UPrimitiveComponent* Component)
{
	UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component);
	if (StaticMeshComponent != nullptr 
		&& StaticMeshComponent->GetStaticMesh() != nullptr 
		&& StaticMeshComponent->GetStaticMesh()->GetNumSourceModels() > 0)
	{
		return MakeUnique<FStaticMeshComponentTarget>(Component, CurrentEditingLOD);
	}
	return {};
}
