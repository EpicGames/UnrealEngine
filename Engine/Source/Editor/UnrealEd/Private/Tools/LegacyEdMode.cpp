// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/LegacyEdMode.h"

#include "EdMode.h"
#include "Containers/Array.h"
#include "GameFramework/Actor.h"
#include "Engine/Texture2D.h"
#include "Math/Box.h"
#include "Math/Vector.h"
#include "Math/Rotator.h"
#include "EditorModeManager.h"
#include "EditorModeTools.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LegacyEdMode)

ULegacyEdModeWrapper::ULegacyEdModeWrapper()
{
}

bool ULegacyEdModeWrapper::CreateLegacyMode(FEditorModeID ModeID, FEditorModeTools& ModeManager)
{
	LegacyEditorMode = FEditorModeRegistry::Get().CreateMode(ModeID, ModeManager);
	Owner = &ModeManager;

	return LegacyEditorMode.IsValid();
}

void ULegacyEdModeWrapper::Initialize()
{
	Info = LegacyEditorMode->GetModeInfo();
}

bool ULegacyEdModeWrapper::IsSelectionAllowed(AActor* InActor, bool bInSelected) const
{
	return LegacyEditorMode->IsSelectionAllowed(InActor, bInSelected);
}

bool ULegacyEdModeWrapper::Select(AActor* InActor, bool bInSelected)
{
	return LegacyEditorMode->Select(InActor, bInSelected);
}

bool ULegacyEdModeWrapper::ProcessEditDuplicate()
{
	return LegacyEditorMode->ProcessEditDuplicate();
}

bool ULegacyEdModeWrapper::ProcessEditDelete()
{
	return LegacyEditorMode->ProcessEditDelete();
}

bool ULegacyEdModeWrapper::ProcessEditCut()
{
	return LegacyEditorMode->ProcessEditCut();
}

bool ULegacyEdModeWrapper::ProcessEditCopy()
{
	return LegacyEditorMode->ProcessEditCopy();
}

bool ULegacyEdModeWrapper::ProcessEditPaste()
{
	return LegacyEditorMode->ProcessEditPaste();
}

EEditAction::Type ULegacyEdModeWrapper::GetActionEditDuplicate()
{
	return LegacyEditorMode->GetActionEditDuplicate();
}

EEditAction::Type ULegacyEdModeWrapper::GetActionDragDuplicate()
{
	return LegacyEditorMode->GetActionDragDuplicate();
}

EEditAction::Type ULegacyEdModeWrapper::GetActionEditDelete()
{
	return LegacyEditorMode->GetActionEditDelete();
}

EEditAction::Type ULegacyEdModeWrapper::GetActionEditCut()
{
	return LegacyEditorMode->GetActionEditCut();
}

EEditAction::Type ULegacyEdModeWrapper::GetActionEditCopy()
{
	return LegacyEditorMode->GetActionEditCopy();
}

EEditAction::Type ULegacyEdModeWrapper::GetActionEditPaste()
{
	return LegacyEditorMode->GetActionEditPaste();
}

bool ULegacyEdModeWrapper::IsSnapRotationEnabled()
{
	return LegacyEditorMode->IsSnapRotationEnabled();
}

bool ULegacyEdModeWrapper::SnapRotatorToGridOverride(FRotator& Rotation)
{
	return LegacyEditorMode->SnapRotatorToGridOverride(Rotation);
}

void ULegacyEdModeWrapper::ActorsDuplicatedNotify(TArray<AActor*>& PreDuplicateSelection, TArray<AActor*>& PostDuplicateSelection, bool bOffsetLocations)
{
	LegacyEditorMode->ActorsDuplicatedNotify(PreDuplicateSelection, PostDuplicateSelection, bOffsetLocations);
}

void ULegacyEdModeWrapper::ActorMoveNotify()
{
	LegacyEditorMode->ActorMoveNotify();
}

void ULegacyEdModeWrapper::ActorSelectionChangeNotify()
{
	LegacyEditorMode->ActorSelectionChangeNotify();
}

void ULegacyEdModeWrapper::ActorPropChangeNotify()
{
	LegacyEditorMode->ActorPropChangeNotify();
}

void ULegacyEdModeWrapper::UpdateInternalData()
{
	LegacyEditorMode->UpdateInternalData();
}

void ULegacyEdModeWrapper::MapChangeNotify()
{
	LegacyEditorMode->MapChangeNotify();
}

void ULegacyEdModeWrapper::SelectNone()
{
	LegacyEditorMode->SelectNone();
}

bool ULegacyEdModeWrapper::GetPivotForOrbit(FVector& OutPivot) const
{
	return LegacyEditorMode->GetPivotForOrbit(OutPivot);
}

void ULegacyEdModeWrapper::PostUndo()
{
	LegacyEditorMode->PostUndo();
}

bool ULegacyEdModeWrapper::DisallowMouseDeltaTracking() const
{
	return LegacyEditorMode->DisallowMouseDeltaTracking();
}

bool ULegacyEdModeWrapper::RequiresLegacyViewportInteractions() const
{
	return LegacyEditorMode->RequiresLegacyViewportInteractions();
}

bool ULegacyEdModeWrapper::GetCursor(EMouseCursor::Type& OutCursor) const
{
	return LegacyEditorMode->GetCursor(OutCursor);
}

bool ULegacyEdModeWrapper::GetOverrideCursorVisibility(bool& bWantsOverride, bool& bHardwareCursorVisible, bool bSoftwareCursorVisible) const
{
	return LegacyEditorMode->GetOverrideCursorVisibility(bWantsOverride, bHardwareCursorVisible, bSoftwareCursorVisible);
}

bool ULegacyEdModeWrapper::CanAutoSave() const
{
	return LegacyEditorMode->CanAutoSave();
}

bool ULegacyEdModeWrapper::IsCompatibleWith(FEditorModeID OtherModeID) const
{
	return LegacyEditorMode->IsCompatibleWith(OtherModeID);
}

bool ULegacyEdModeWrapper::ComputeBoundingBoxForViewportFocus(AActor* Actor, UPrimitiveComponent* PrimitiveComponent, FBox& InOutBox) const
{
	return LegacyEditorMode->ComputeBoundingBoxForViewportFocus(Actor, PrimitiveComponent, InOutBox);
}

bool ULegacyEdModeWrapper::AllowsViewportDragTool() const
{
	return LegacyEditorMode->AllowsViewportDragTool();
}

bool ULegacyEdModeWrapper::UsesToolkits() const
{
	return LegacyEditorMode->UsesToolkits();
}

bool ULegacyEdModeWrapper::ProcessCapturedMouseMoves(FEditorViewportClient* InViewportClient, FViewport* InViewport, const TArrayView<FIntPoint>& CapturedMouseMoves)
{
	return LegacyEditorMode->ProcessCapturedMouseMoves(InViewportClient, InViewport, CapturedMouseMoves);
}

bool ULegacyEdModeWrapper::InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event)
{
	return LegacyEditorMode->InputKey(ViewportClient, Viewport, Key, Event);
}

bool ULegacyEdModeWrapper::InputAxis(FEditorViewportClient* InViewportClient, FViewport* Viewport, int32 ControllerId, FKey Key, float Delta, float DeltaTime)
{
	return LegacyEditorMode->InputAxis(InViewportClient, Viewport, ControllerId, Key, Delta, DeltaTime);
}

bool ULegacyEdModeWrapper::InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale)
{
	return LegacyEditorMode->InputDelta(InViewportClient, InViewport, InDrag, InRot, InScale);
}

bool ULegacyEdModeWrapper::BeginTransform(const FGizmoState& InState)
{
	return LegacyEditorMode->BeginTransform(InState);
}

bool ULegacyEdModeWrapper::EndTransform(const FGizmoState& InState)
{
	return LegacyEditorMode->EndTransform(InState);
}

bool ULegacyEdModeWrapper::StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	return LegacyEditorMode->StartTracking(InViewportClient, InViewport);
}

bool ULegacyEdModeWrapper::EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	return LegacyEditorMode->EndTracking(InViewportClient, InViewport);
}

bool ULegacyEdModeWrapper::HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click)
{
	return LegacyEditorMode->HandleClick(InViewportClient, HitProxy, Click);
}

void ULegacyEdModeWrapper::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
{
	LegacyEditorMode->Tick(ViewportClient, DeltaTime);
}

bool ULegacyEdModeWrapper::CapturedMouseMove(FEditorViewportClient* InViewportClient, FViewport* InViewport, int32 InMouseX, int32 InMouseY)
{
	return LegacyEditorMode->CapturedMouseMove(InViewportClient, InViewport, InMouseX, InMouseY);
}

bool ULegacyEdModeWrapper::MouseEnter(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y)
{
	return LegacyEditorMode->MouseEnter(ViewportClient, Viewport, x, y);
}

bool ULegacyEdModeWrapper::MouseLeave(FEditorViewportClient* ViewportClient, FViewport* Viewport)
{
	return LegacyEditorMode->MouseLeave(ViewportClient, Viewport);
}

bool ULegacyEdModeWrapper::MouseMove(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y)
{
	return LegacyEditorMode->MouseMove(ViewportClient, Viewport, x, y);
}

bool ULegacyEdModeWrapper::ReceivedFocus(FEditorViewportClient* ViewportClient, FViewport* Viewport)
{
	return LegacyEditorMode->ReceivedFocus(ViewportClient, Viewport);
}

bool ULegacyEdModeWrapper::LostFocus(FEditorViewportClient* ViewportClient, FViewport* Viewport)
{
	return LegacyEditorMode->LostFocus(ViewportClient, Viewport);
}

void ULegacyEdModeWrapper::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	LegacyEditorMode->Render(View, Viewport, PDI);
}

void ULegacyEdModeWrapper::DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas)
{
	LegacyEditorMode->DrawHUD(ViewportClient, Viewport, View, Canvas);
}

bool ULegacyEdModeWrapper::PreConvertMouseMovement(FEditorViewportClient* InViewportClient)
{
	return LegacyEditorMode->PreConvertMouseMovement(InViewportClient);
}

bool ULegacyEdModeWrapper::PostConvertMouseMovement(FEditorViewportClient* InViewportClient)
{
	return LegacyEditorMode->PostConvertMouseMovement(InViewportClient);
}

bool ULegacyEdModeWrapper::ShouldDrawBrushWireframe(AActor* InActor) const
{
	return LegacyEditorMode->ShouldDrawBrushWireframe(InActor);
}

void ULegacyEdModeWrapper::Enter()
{
	CreateInteractiveToolsContexts();
	LegacyEditorMode->Enter();
	Toolkit = LegacyEditorMode->GetToolkit();
}

void ULegacyEdModeWrapper::Exit()
{
	Toolkit.Reset();

	LegacyEditorMode->Exit();
	DestroyInteractiveToolsContexts();
}

FEdMode* ULegacyEdModeWrapper::AsLegacyMode()
{
	return LegacyEditorMode.Get();
}

UTexture2D* ULegacyEdModeWrapper::GetVertexTexture()
{
	return LegacyEditorMode->GetVertexTexture();
}

bool ULegacyEdModeWrapper::AllowWidgetMove()
{
	return LegacyEditorMode->AllowWidgetMove();
}

bool ULegacyEdModeWrapper::CanCycleWidgetMode() const
{
	return LegacyEditorMode->CanCycleWidgetMode();
}

bool ULegacyEdModeWrapper::ShowModeWidgets() const
{
	return LegacyEditorMode->ShowModeWidgets();
}

EAxisList::Type ULegacyEdModeWrapper::GetWidgetAxisToDraw(UE::Widget::EWidgetMode InWidgetMode) const
{
	return LegacyEditorMode->GetWidgetAxisToDraw(InWidgetMode);
}

FVector ULegacyEdModeWrapper::GetWidgetLocation() const
{
	return LegacyEditorMode->GetWidgetLocation();
}

bool ULegacyEdModeWrapper::ShouldDrawWidget() const
{
	return LegacyEditorMode->ShouldDrawWidget();
}

bool ULegacyEdModeWrapper::UsesTransformWidget() const
{
	return LegacyEditorMode->UsesTransformWidget();
}

bool ULegacyEdModeWrapper::UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const
{
	return LegacyEditorMode->UsesTransformWidget(CheckMode);
}

FVector ULegacyEdModeWrapper::GetWidgetNormalFromCurrentAxis(void* InData)
{
	return LegacyEditorMode->GetWidgetNormalFromCurrentAxis(InData);
}

bool ULegacyEdModeWrapper::BoxSelect(FBox& InBox, bool InSelect)
{
	return LegacyEditorMode->BoxSelect(InBox, InSelect);
}

bool ULegacyEdModeWrapper::FrustumSelect(const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, bool InSelect)
{
	return LegacyEditorMode->FrustumSelect(InFrustum, InViewportClient, InSelect);
}

void ULegacyEdModeWrapper::SetCurrentWidgetAxis(EAxisList::Type InAxis)
{
	LegacyEditorMode->SetCurrentWidgetAxis(InAxis);
}

EAxisList::Type ULegacyEdModeWrapper::GetCurrentWidgetAxis() const
{
	return LegacyEditorMode->GetCurrentWidgetAxis();
}

bool ULegacyEdModeWrapper::UsesPropertyWidgets() const
{
	return LegacyEditorMode->UsesPropertyWidgets();
}

bool ULegacyEdModeWrapper::GetCustomDrawingCoordinateSystem(FMatrix& InMatrix, void* InData)
{
	return LegacyEditorMode->GetCustomDrawingCoordinateSystem(InMatrix, InData);
}

bool ULegacyEdModeWrapper::GetCustomInputCoordinateSystem(FMatrix& InMatrix, void* InData)
{
	return LegacyEditorMode->GetCustomInputCoordinateSystem(InMatrix, InData);
}

void ULegacyEdModeWrapper::SetCurrentTool(EModeTools InID)
{
	LegacyEditorMode->SetCurrentTool(InID);
}

void ULegacyEdModeWrapper::SetCurrentTool(FModeTool* InModeTool)
{
	LegacyEditorMode->SetCurrentTool(InModeTool);
}

FModeTool* ULegacyEdModeWrapper::FindTool(EModeTools InID)
{
	return LegacyEditorMode->FindTool(InID);
}

const TArray<FModeTool*>& ULegacyEdModeWrapper::GetTools() const
{
	return LegacyEditorMode->GetTools();
}

FModeTool* ULegacyEdModeWrapper::GetCurrentTool()
{
	return LegacyEditorMode->GetCurrentTool();
}

const FModeTool* ULegacyEdModeWrapper::GetCurrentTool() const
{
	return LegacyEditorMode->GetCurrentTool();
}


void ULegacyEdModeWrapper::Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	LegacyEditorMode->Draw(View, PDI);
}
