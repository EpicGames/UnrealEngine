// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/UEdMode.h"
#include "Tools/Modes.h"
#include "Engine/EngineBaseTypes.h"
#include "Templates/SharedPointer.h"
#include "Tools/LegacyEdModeInterfaces.h"
#include "GizmoEdModeInterface.h"

#include "LegacyEdMode.generated.h"

class AActor;
class FEditorModeTools;
class FEdMode;
class UTexture2D;
class FEditorViewportClient;
class FViewport;
class UPrimitiveComponent;

UCLASS(MinimalAPI)
class ULegacyEdModeWrapper final : public UEdMode, public ILegacyEdModeWidgetInterface, public ILegacyEdModeToolInterface, public ILegacyEdModeSelectInterface, public ILegacyEdModeDrawHelperInterface, public ILegacyEdModeViewportInterface, public IGizmoEdModeInterface
{
	GENERATED_BODY()

public:
	ULegacyEdModeWrapper();

	UNREALED_API bool CreateLegacyMode(FEditorModeID ModeID, FEditorModeTools& ModeManager);

	// Begin UEdMode overrides
	UNREALED_API virtual void Initialize() override;
	UNREALED_API virtual bool IsSelectionAllowed(AActor* InActor, bool bInSelected) const override;
	UNREALED_API virtual bool Select(AActor* InActor, bool bInSelected) override;
	UNREALED_API virtual bool ProcessEditDuplicate() override;
	UNREALED_API virtual bool ProcessEditDelete() override;
	UNREALED_API virtual bool ProcessEditCut() override;
	UNREALED_API virtual bool ProcessEditCopy() override;
	UNREALED_API virtual bool ProcessEditPaste() override;
	UNREALED_API virtual EEditAction::Type GetActionEditDuplicate() override;
	UNREALED_API virtual EEditAction::Type GetActionEditDelete() override;
	UNREALED_API virtual EEditAction::Type GetActionEditCut() override;
	UNREALED_API virtual EEditAction::Type GetActionEditCopy() override;
	UNREALED_API virtual EEditAction::Type GetActionEditPaste() override;
	UNREALED_API virtual EEditAction::Type GetActionDragDuplicate() override;
	UNREALED_API virtual bool IsSnapRotationEnabled() override;
	UNREALED_API virtual bool SnapRotatorToGridOverride(FRotator& Rotation) override;
	UNREALED_API virtual void ActorsDuplicatedNotify(TArray<AActor*>& PreDuplicateSelection, TArray<AActor*>& PostDuplicateSelection, bool bOffsetLocations) override;
	UNREALED_API virtual void ActorMoveNotify() override;
	UNREALED_API virtual void ActorSelectionChangeNotify() override;
	UNREALED_API virtual void ActorPropChangeNotify() override;
	UNREALED_API virtual void UpdateInternalData() override;
	UNREALED_API virtual void MapChangeNotify() override;
	UNREALED_API virtual void SelectNone() override;
	UNREALED_API virtual bool GetPivotForOrbit(FVector& OutPivot) const override;
	UNREALED_API virtual void PostUndo() override;
	UNREALED_API virtual bool DisallowMouseDeltaTracking() const override;
	UNREALED_API virtual bool RequiresLegacyViewportInteractions() const override;
	UNREALED_API virtual bool GetCursor(EMouseCursor::Type& OutCursor) const override;
	UNREALED_API virtual bool GetOverrideCursorVisibility(bool& bWantsOverride, bool& bHardwareCursorVisible, bool bSoftwareCursorVisible) const;
	UNREALED_API virtual bool CanAutoSave() const override;
	UNREALED_API virtual bool IsCompatibleWith(FEditorModeID OtherModeID) const override;
	UNREALED_API virtual bool ComputeBoundingBoxForViewportFocus(AActor* Actor, UPrimitiveComponent* PrimitiveComponent, FBox& InOutBox) const override;
	UNREALED_API virtual bool AllowsViewportDragTool() const override;
	UNREALED_API virtual bool UsesToolkits() const override;

	UNREALED_API virtual bool ProcessCapturedMouseMoves(FEditorViewportClient* InViewportClient, FViewport* InViewport, const TArrayView<FIntPoint>& CapturedMouseMoves) override;
	UNREALED_API virtual bool InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event) override;
	UNREALED_API virtual bool InputAxis(FEditorViewportClient* InViewportClient, FViewport* Viewport, int32 ControllerId, FKey Key, float Delta, float DeltaTime) override;
	UNREALED_API virtual bool InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale) override;
	UNREALED_API virtual bool StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	UNREALED_API virtual bool EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	UNREALED_API virtual bool HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click) override;
	UNREALED_API virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;
	UNREALED_API virtual bool CapturedMouseMove(FEditorViewportClient* InViewportClient, FViewport* InViewport, int32 InMouseX, int32 InMouseY) override;
	UNREALED_API virtual bool MouseEnter(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y) override;
	UNREALED_API virtual bool MouseLeave(FEditorViewportClient* ViewportClient, FViewport* Viewport) override;
	UNREALED_API virtual bool MouseMove(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y) override;
	UNREALED_API virtual bool ReceivedFocus(FEditorViewportClient* ViewportClient, FViewport* Viewport) override;
	UNREALED_API virtual bool LostFocus(FEditorViewportClient* ViewportClient, FViewport* Viewport) override;
	UNREALED_API virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	UNREALED_API virtual void DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas) override;
	UNREALED_API virtual bool PreConvertMouseMovement(FEditorViewportClient* InViewportClient) override;
	UNREALED_API virtual bool PostConvertMouseMovement(FEditorViewportClient* InViewportClient) override;

	UNREALED_API virtual bool ShouldDrawBrushWireframe(AActor* InActor) const override;
	UNREALED_API virtual void Enter() override;
	UNREALED_API virtual void Exit() override;

	UNREALED_API virtual FEdMode* AsLegacyMode() override;
	UNREALED_API virtual UTexture2D* GetVertexTexture() override;
	// End UEdMode overrides

	// ILegacyEdModeWidgetInterface
	UNREALED_API virtual bool AllowWidgetMove() override;
	UNREALED_API virtual bool CanCycleWidgetMode() const override;
	UNREALED_API virtual bool ShowModeWidgets() const override;
	UNREALED_API virtual EAxisList::Type GetWidgetAxisToDraw(UE::Widget::EWidgetMode InWidgetMode) const override;
	UNREALED_API virtual FVector GetWidgetLocation() const override;
	UNREALED_API virtual bool ShouldDrawWidget() const override;
	UNREALED_API virtual bool UsesTransformWidget() const override;
	UNREALED_API virtual bool UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const override;
	UNREALED_API virtual FVector GetWidgetNormalFromCurrentAxis(void* InData) override;
	UNREALED_API virtual bool BoxSelect(FBox& InBox, bool InSelect = true) override;
	UNREALED_API virtual bool FrustumSelect(const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, bool InSelect = true) override;
	UNREALED_API virtual void SetCurrentWidgetAxis(EAxisList::Type InAxis) override;
	UNREALED_API virtual EAxisList::Type GetCurrentWidgetAxis() const override;
	UNREALED_API virtual bool UsesPropertyWidgets() const override;
	UNREALED_API virtual bool GetCustomDrawingCoordinateSystem(FMatrix& InMatrix, void* InData) override;
	UNREALED_API virtual bool GetCustomInputCoordinateSystem(FMatrix& InMatrix, void* InData) override;
	// End ILegacyEdModeWidgetInterface

	// ILegacyEdModeToolInterface overrides
	UNREALED_API virtual void SetCurrentTool(EModeTools InID) override;
	UNREALED_API virtual void SetCurrentTool(FModeTool* InModeTool) override;
	UNREALED_API virtual FModeTool* FindTool(EModeTools InID) override;
	UNREALED_API virtual const TArray<FModeTool*>& GetTools() const override;
	UNREALED_API virtual FModeTool* GetCurrentTool() override;
	UNREALED_API virtual const FModeTool* GetCurrentTool() const override;
	// End ILegacyEdModeToolInterface overrides

	// Start FCommonDrawHelper overrides
	UNREALED_API virtual void Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	// End FCommonDrawHelper overrides

	// Start IGizmoEdModeInterface overrides
	UNREALED_API virtual bool BeginTransform(const FGizmoState& InState) override;
	UNREALED_API virtual bool EndTransform(const FGizmoState& InState) override;
	// End IGizmoEdModeInterface overrides

private:
	TSharedPtr<FEdMode> LegacyEditorMode;
};
