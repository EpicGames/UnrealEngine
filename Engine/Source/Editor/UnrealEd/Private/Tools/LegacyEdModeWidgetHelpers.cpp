// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/LegacyEdModeWidgetHelpers.h"
#include "HitProxies.h"
#include "Components/SceneComponent.h"
#include "GameFramework/Actor.h"
#include "EditorModeManager.h"
#include "SceneView.h"
#include "Selection.h"
#include "EditorViewportClient.h"
#include "EngineUtils.h"
#include "CanvasTypes.h"
#include "CanvasItem.h"
#include "Settings/LevelEditorViewportSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LegacyEdModeWidgetHelpers)

const FName FLegacyEdModeWidgetHelper::MD_MakeEditWidget(TEXT("MakeEditWidget"));
const FName FLegacyEdModeWidgetHelper::MD_ValidateWidgetUsing(TEXT("ValidateWidgetUsing"));
const FName FLegacyEdModeWidgetHelper::MD_WidgetLocalSpace(TEXT("LocalSpaceEditWidget"));

/** Hit proxy used for editable properties */
struct HPropertyWidgetProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	/** Name of property this is the widget for */
	FString	PropertyName;

	/** If the property is an array property, the index into that array that this widget is for */
	int32	PropertyIndex;

	/** This property is a transform */
	bool	bPropertyIsTransform;

	/** This property is in local space */
	bool	bLocalSpace;

	HPropertyWidgetProxy(FString InPropertyName, int32 InPropertyIndex, bool bInPropertyIsTransform, bool bInLocalSpace = true)
		: HHitProxy(HPP_Foreground)
		, PropertyName(InPropertyName)
		, PropertyIndex(InPropertyIndex)
		, bPropertyIsTransform(bInPropertyIsTransform)
		, bLocalSpace(bInLocalSpace)
	{}

	/** Show cursor as cross when over this handle */
	virtual EMouseCursor::Type GetMouseCursor() override
	{
		return EMouseCursor::Crosshairs;
	}
};

IMPLEMENT_HIT_PROXY(HPropertyWidgetProxy, HHitProxy);


namespace
{
	/**
	 * Returns a reference to the named property value data in the given container.
	 */
	template<typename T>
	T* GetPropertyValuePtrByName(const UStruct* InStruct, void* InContainer, FString PropertyName, int32 ArrayIndex, FProperty*& OutProperty)
	{
		T* ValuePtr = NULL;

		// Extract the vector ptr recursively using the property name
		int32 DelimPos = PropertyName.Find(TEXT("."));
		if (DelimPos != INDEX_NONE)
		{
			// Parse the property name and (optional) array index
			int32 SubArrayIndex = 0;
			FString NameToken = PropertyName.Left(DelimPos);
			int32 ArrayPos = NameToken.Find(TEXT("["));
			if (ArrayPos != INDEX_NONE)
			{
				FString IndexToken = NameToken.RightChop(ArrayPos + 1).LeftChop(1);
				SubArrayIndex = FCString::Atoi(*IndexToken);

				NameToken = PropertyName.Left(ArrayPos);
			}

			// Obtain the property info from the given structure definition
			FProperty* CurrentProp = FindFProperty<FProperty>(InStruct, FName(*NameToken));

			// Check first to see if this is a simple structure (i.e. not an array of structures)
			FStructProperty* StructProp = CastField<FStructProperty>(CurrentProp);
			if (StructProp != NULL)
			{
				// Recursively call back into this function with the structure property and container value
				ValuePtr = GetPropertyValuePtrByName<T>(StructProp->Struct, StructProp->ContainerPtrToValuePtr<void>(InContainer), PropertyName.RightChop(DelimPos + 1), ArrayIndex, OutProperty);
			}
			else
			{
				// Check to see if this is an array
				FArrayProperty* ArrayProp = CastField<FArrayProperty>(CurrentProp);
				if (ArrayProp != NULL)
				{
					// It is an array, now check to see if this is an array of structures
					StructProp = CastField<FStructProperty>(ArrayProp->Inner);
					if (StructProp != NULL)
					{
						FScriptArrayHelper_InContainer ArrayHelper(ArrayProp, InContainer);
						if (ArrayHelper.IsValidIndex(SubArrayIndex))
						{
							// Recursively call back into this function with the array element and container value
							ValuePtr = GetPropertyValuePtrByName<T>(StructProp->Struct, ArrayHelper.GetRawPtr(SubArrayIndex), PropertyName.RightChop(DelimPos + 1), ArrayIndex, OutProperty);
						}
					}
				}
			}
		}
		else
		{
			FProperty* Prop = FindFProperty<FProperty>(InStruct, FName(*PropertyName));
			if (Prop != NULL)
			{
				if (FArrayProperty* ArrayProp = CastField<FArrayProperty>(Prop))
				{
					check(ArrayIndex != INDEX_NONE);

					// Property is an array property, so make sure we have a valid index specified
					FScriptArrayHelper_InContainer ArrayHelper(ArrayProp, InContainer);
					if (ArrayHelper.IsValidIndex(ArrayIndex))
					{
						ValuePtr = (T*)ArrayHelper.GetRawPtr(ArrayIndex);
					}
				}
				else
				{
					// Property is a vector property, so access directly
					ValuePtr = Prop->ContainerPtrToValuePtr<T>(InContainer);
				}

				OutProperty = Prop;
			}
		}

		return ValuePtr;
	}

	/**
	 * Returns the value of the property with the given name in the given Actor instance.
	 */
	template<typename T>
	T GetPropertyValueByName(UObject* Object, FString PropertyName, int32 PropertyIndex)
	{
		T Value = {};
		FProperty* DummyProperty = NULL;
		if (T* ValuePtr = GetPropertyValuePtrByName<T>(Object->GetClass(), Object, PropertyName, PropertyIndex, DummyProperty))
		{
			Value = *ValuePtr;
		}
		return Value;
	}

	/**
	 * Sets the property with the given name in the given Actor instance to the given value.
	 */
	template<typename T>
	void SetPropertyValueByName(UObject* Object, FString PropertyName, int32 PropertyIndex, const T& InValue, FProperty*& OutProperty)
	{
		if (T* ValuePtr = GetPropertyValuePtrByName<T>(Object->GetClass(), Object, PropertyName, PropertyIndex, OutProperty))
		{
			*ValuePtr = InValue;
		}
	}
}

static bool IsTransformProperty(FProperty* InProp)
{
	FStructProperty* StructProp = CastField<FStructProperty>(InProp);
	return (StructProp != NULL && StructProp->Struct->GetFName() == NAME_Transform);
}

static bool IsLocalSpace(const FProperty* InProp)
{
	return !InProp->HasMetaData(FEdMode::MD_WidgetLocalSpace)
		|| InProp->GetMetaData(FEdMode::MD_WidgetLocalSpace).IsEmpty()
		|| InProp->GetBoolMetaData(FEdMode::MD_WidgetLocalSpace);
}

struct FPropertyWidgetInfoChainElement
{
	FProperty* Property;
	int32 Index;

	FPropertyWidgetInfoChainElement(FProperty* InProperty = nullptr, int32 InIndex = INDEX_NONE)
		: Property(InProperty), Index(InIndex)
	{}

	static bool ShouldCreateWidgetSomwhereInBranch(FProperty* InProp)
	{
		FStructProperty* StructProperty = CastField<FStructProperty>(InProp);
		if (!StructProperty)
		{
			FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InProp);
			if (ArrayProperty)
			{
				StructProperty = CastField<FStructProperty>(ArrayProperty->Inner);
			}
		}

		if (StructProperty)
		{
			if (FEdMode::CanCreateWidgetForStructure(StructProperty->Struct) && InProp->HasMetaData(FEdMode::MD_MakeEditWidget))
			{
				return true;
			}

			for (TFieldIterator<FProperty> PropertyIt(StructProperty->Struct, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
			{
				if (ShouldCreateWidgetSomwhereInBranch(*PropertyIt))
				{
					return true;
				}
			}
		}

		return false;
	}

	static FLegacyEdModeWidgetHelper::FPropertyWidgetInfo CreateWidgetInfo(const TArray<FPropertyWidgetInfoChainElement>& Chain, bool bIsTransform, bool bIsLocalSpace, FProperty* CurrentProp, int32 Index = INDEX_NONE)
	{
		check(CurrentProp);
		FEdMode::FPropertyWidgetInfo WidgetInfo;
		WidgetInfo.PropertyValidationName = FName(*CurrentProp->GetMetaData(FEdMode::MD_ValidateWidgetUsing));
		WidgetInfo.bIsTransform = bIsTransform;
		WidgetInfo.PropertyIndex = Index;
		WidgetInfo.bLocalSpace = bIsLocalSpace;

		const FString SimplePostFix(TEXT("."));
		for (int32 ChainIndex = 0; ChainIndex < Chain.Num(); ++ChainIndex)
		{
			const FPropertyWidgetInfoChainElement& Element = Chain[ChainIndex];
			check(Element.Property);
			const FString Postfix = (Element.Index != INDEX_NONE) ? FString::Printf(TEXT("[%d]."), Element.Index) : SimplePostFix;
			const FString PropertyName = Element.Property->GetName() + Postfix;
			const FString& DisplayName = Element.Property->GetMetaData(TEXT("DisplayName"));

			WidgetInfo.PropertyName += PropertyName;
			WidgetInfo.DisplayName += (!DisplayName.IsEmpty()) ? (DisplayName + Postfix) : PropertyName;
		}

		{
			const FString PropertyName = CurrentProp->GetName();
			const FString& DisplayName = CurrentProp->GetMetaData(TEXT("DisplayName"));

			WidgetInfo.PropertyName += PropertyName;
			WidgetInfo.DisplayName += (!DisplayName.IsEmpty()) ? DisplayName : PropertyName;
		}
		return WidgetInfo;
	}

	static void RecursiveGet(const FLegacyEdModeWidgetHelper& EdMode, const UStruct* InStruct, const void* InContainer, TArray<FEdMode::FPropertyWidgetInfo>& OutInfos, TArray<FPropertyWidgetInfoChainElement>& Chain)
	{
		for (TFieldIterator<FProperty> PropertyIt(InStruct, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
		{
			FProperty* CurrentProp = *PropertyIt;
			check(CurrentProp);

			if (EdMode.ShouldCreateWidgetForProperty(CurrentProp))
			{
				if (FArrayProperty* ArrayProp = CastField<FArrayProperty>(CurrentProp))
				{
					check(InContainer);
					FScriptArrayHelper_InContainer ArrayHelper(ArrayProp, InContainer);
					// See how many widgets we need to make for the array property
					const uint32 ArrayDim = ArrayHelper.Num();
					for (uint32 Index = 0; Index < ArrayDim; ++Index)
					{
						OutInfos.Add(FPropertyWidgetInfoChainElement::CreateWidgetInfo(Chain, IsTransformProperty(ArrayProp->Inner), IsLocalSpace(ArrayProp->Inner), CurrentProp, Index));
					}
				}
				else
				{
					OutInfos.Add(FPropertyWidgetInfoChainElement::CreateWidgetInfo(Chain, IsTransformProperty(CurrentProp), IsLocalSpace(CurrentProp), CurrentProp));
				}
			}
			else if (FStructProperty* StructProp = CastField<FStructProperty>(CurrentProp))
			{
				// Recursively traverse into structures, looking for additional vector properties to expose
				Chain.Push(FPropertyWidgetInfoChainElement(StructProp));
				RecursiveGet(EdMode
					, StructProp->Struct
					, StructProp->ContainerPtrToValuePtr<void>(InContainer)
					, OutInfos
					, Chain);
				Chain.Pop(EAllowShrinking::No);
			}
			else if (FArrayProperty* ArrayProp = CastField<FArrayProperty>(CurrentProp))
			{
				// Recursively traverse into arrays of structures, looking for additional vector properties to expose
				FStructProperty* InnerStructProp = CastField<FStructProperty>(ArrayProp->Inner);
				if (InnerStructProp)
				{
					FScriptArrayHelper_InContainer ArrayHelper(ArrayProp, InContainer);

					// If the array is not empty the do additional check to tell if iteration is necessary
					if (ArrayHelper.Num() && ShouldCreateWidgetSomwhereInBranch(InnerStructProp))
					{
						for (int32 ArrayIndex = 0; ArrayIndex < ArrayHelper.Num(); ++ArrayIndex)
						{
							Chain.Push(FPropertyWidgetInfoChainElement(ArrayProp, ArrayIndex));
							RecursiveGet(EdMode
								, InnerStructProp->Struct
								, ArrayHelper.GetRawPtr(ArrayIndex)
								, OutInfos
								, Chain);
							Chain.Pop(EAllowShrinking::No);
						}
					}
				}
			}
		}
	}
};

void FLegacyEdModeWidgetHelper::FPropertyWidgetInfo::GetTransformAndColor(UObject* BestSelectedItem, bool bIsSelected, FTransform& OutLocalTransform, FString& OutValidationMessage, FColor& OutDrawColor) const
{
	// Determine the desired position
	if (bIsTransform)
	{
		OutLocalTransform = GetPropertyValueByName<FTransform>(BestSelectedItem, PropertyName, PropertyIndex);
	}
	else
	{
		OutLocalTransform = FTransform(GetPropertyValueByName<FVector>(BestSelectedItem, PropertyName, PropertyIndex));
	}

	// Determine the desired color
	OutDrawColor = bIsSelected ? FColor::White : FColor(128, 128, 255);
	if (PropertyValidationName != NAME_None)
	{
		if (UFunction* ValidateFunc = BestSelectedItem->FindFunction(PropertyValidationName))
		{
			BestSelectedItem->ProcessEvent(ValidateFunc, &OutValidationMessage);

			// if we have a negative result, the widget color is red.
			OutDrawColor = OutValidationMessage.IsEmpty() ? OutDrawColor : FColor::Red;
		}
	}
}

FLegacyEdModeWidgetHelper::FLegacyEdModeWidgetHelper()
	: EditedPropertyIndex(INDEX_NONE)
	, bEditedPropertyIsTransform(false)
	, bLocalSpace(true)
	, CurrentWidgetAxis(EAxisList::None)
	, Owner(nullptr)
{
}

bool FLegacyEdModeWidgetHelper::AllowWidgetMove()
{
	return true;
}

bool FLegacyEdModeWidgetHelper::CanCycleWidgetMode() const
{
	return true;
}

bool FLegacyEdModeWidgetHelper::ShowModeWidgets() const
{
	return true;
}

EAxisList::Type FLegacyEdModeWidgetHelper::GetWidgetAxisToDraw(UE::Widget::EWidgetMode InWidgetMode) const
{
	return EAxisList::All;
}

FVector FLegacyEdModeWidgetHelper::GetWidgetLocation() const
{
	if (UsesPropertyWidgets())
	{
		FTransform DisplayWidgetToWorld;
		UObject* BestSelectedItem = GetItemToTryDisplayingWidgetsFor(/*out*/ DisplayWidgetToWorld);
		if (BestSelectedItem)
		{
			if (!EditedPropertyName.IsEmpty())
			{
				FVector PropertyPosition = FVector::ZeroVector;

				if (bEditedPropertyIsTransform)
				{
					FTransform PropertyTransform = GetPropertyValueByName<FTransform>(BestSelectedItem, EditedPropertyName, EditedPropertyIndex);
					PropertyPosition = PropertyTransform.GetLocation();
				}
				else
				{
					PropertyPosition = GetPropertyValueByName<FVector>(BestSelectedItem, EditedPropertyName, EditedPropertyIndex);
				}

				if (!bLocalSpace)
				{
					return PropertyPosition;
				}
				
				const FVector WorldPos = DisplayWidgetToWorld.TransformPosition(PropertyPosition);
				return WorldPos;
			}
		}
	}

	return Owner ? Owner->PivotLocation : FVector();
}

bool FLegacyEdModeWidgetHelper::ShouldDrawWidget() const
{
	if (!Owner)
	{
		return false;
	}

	bool bDrawWidget = false;
	bool bHadSelectableComponents = false;
	if (Owner->GetSelectedComponents() && Owner->GetSelectedComponents()->Num() > 0)
	{
		// when components are selected, only show the widget when one or more are scene components
		for (FSelectedEditableComponentIterator It(*Owner->GetSelectedComponents()); It; ++It)
		{
			bHadSelectableComponents = true;
			if (It->IsA<USceneComponent>())
			{
				bDrawWidget = true;
				break;
			}
		}
	}

	if (!bHadSelectableComponents)
	{
		// when actors are selected, only show the widget when all selected actors have scene components
		bDrawWidget = Owner->SelectionHasSceneComponent();
	}

	return bDrawWidget;
}

bool FLegacyEdModeWidgetHelper::UsesTransformWidget() const
{
	return true;
}

bool FLegacyEdModeWidgetHelper::UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const
{
	if (UsesPropertyWidgets())
	{
		FTransform DisplayWidgetToWorld;
		UObject* BestSelectedItem = GetItemToTryDisplayingWidgetsFor(/*out*/ DisplayWidgetToWorld);

		if (BestSelectedItem != nullptr)
		{
			// If editing a vector (not a transform)
			if (!EditedPropertyName.IsEmpty() && !bEditedPropertyIsTransform)
			{
				return (CheckMode == UE::Widget::WM_Translate);
			}
		}
	}

	return true;
}

FVector FLegacyEdModeWidgetHelper::GetWidgetNormalFromCurrentAxis(void* InData)
{
	// Figure out the proper coordinate system.

	FMatrix Matrix = FMatrix::Identity;
	if (Owner && (Owner->GetCoordSystem() == COORD_Local))
	{
		GetCustomDrawingCoordinateSystem(Matrix, InData);
	}

	// Get a base normal from the current axis.

	FVector BaseNormal(1, 0, 0);		// Default to X axis
	switch (CurrentWidgetAxis)
	{
	case EAxisList::Y:	BaseNormal = FVector(0, 1, 0);	break;
	case EAxisList::Z:	BaseNormal = FVector(0, 0, 1);	break;
	case EAxisList::XY:	BaseNormal = FVector(1, 1, 0);	break;
	case EAxisList::XZ:	BaseNormal = FVector(1, 0, 1);	break;
	case EAxisList::YZ:	BaseNormal = FVector(0, 1, 1);	break;
	case EAxisList::XYZ:	BaseNormal = FVector(1, 1, 1);	break;
	}

	// Transform the base normal into the proper coordinate space.
	return Matrix.TransformPosition(BaseNormal);
}

void FLegacyEdModeWidgetHelper::SetCurrentWidgetAxis(EAxisList::Type InAxis)
{
	CurrentWidgetAxis = InAxis;
}

EAxisList::Type FLegacyEdModeWidgetHelper::GetCurrentWidgetAxis() const
{
	return CurrentWidgetAxis;
}

bool FLegacyEdModeWidgetHelper::UsesPropertyWidgets() const
{
	if (ParentModeInterface)
	{
		return ParentModeInterface->UsesPropertyWidgets();
	}

	return false;
}

bool FLegacyEdModeWidgetHelper::GetCustomDrawingCoordinateSystem(FMatrix& InMatrix, void* InData)
{
	if (UsesPropertyWidgets())
	{
		FTransform DisplayWidgetToWorld;
		UObject* BestSelectedItem = GetItemToTryDisplayingWidgetsFor(/*out*/ DisplayWidgetToWorld);
		if (BestSelectedItem)
		{
			if (EditedPropertyName != TEXT(""))
			{
				if (bEditedPropertyIsTransform)
				{
					FTransform LocalTM = GetPropertyValueByName<FTransform>(BestSelectedItem, EditedPropertyName, EditedPropertyIndex);
					InMatrix = FRotationMatrix::Make((LocalTM * DisplayWidgetToWorld).GetRotation());
					return true;
				}
				else
				{
					InMatrix = FRotationMatrix::Make(DisplayWidgetToWorld.GetRotation());
					return true;
				}
			}
		}
	}

	return false;
}

bool FLegacyEdModeWidgetHelper::GetCustomInputCoordinateSystem(FMatrix& InMatrix, void* InData)
{
	return false;
}

void FLegacyEdModeWidgetHelper::ActorSelectionChangeNotify()
{
	EditedPropertyName = TEXT("");
	EditedPropertyIndex = INDEX_NONE;
	bEditedPropertyIsTransform = false;
	bLocalSpace = true;
}

bool FLegacyEdModeWidgetHelper::AllowsViewportDragTool() const
{
	return true;
}

bool FLegacyEdModeWidgetHelper::RequiresLegacyViewportInteractions() const
{
	return true;
}

bool FLegacyEdModeWidgetHelper::InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale)
{
	if (UsesPropertyWidgets())
	{
		FTransform DisplayWidgetToWorld;
		UObject* BestSelectedItem = GetItemToTryDisplayingWidgetsFor(/*out*/ DisplayWidgetToWorld);

		if ((BestSelectedItem != nullptr) && (InViewportClient->GetCurrentWidgetAxis() != EAxisList::None))
		{
			GEditor->NoteActorMovement();

			if (!EditedPropertyName.IsEmpty())
			{
				FTransform PropertyTransform = FTransform::Identity;
				if (bEditedPropertyIsTransform)
				{
					PropertyTransform = GetPropertyValueByName<FTransform>(BestSelectedItem, EditedPropertyName, EditedPropertyIndex);

				}
				else
				{
					FVector PropertyTranslation = GetPropertyValueByName<FVector>(BestSelectedItem, EditedPropertyName, EditedPropertyIndex);
					PropertyTransform = FTransform(PropertyTranslation);
				}

				// Convert local transform into world space
				if (bLocalSpace)
				{
					PropertyTransform = PropertyTransform * DisplayWidgetToWorld;
				}

				// Apply delta in world space
				PropertyTransform.SetTranslation(PropertyTransform.GetTranslation() + InDrag);
				PropertyTransform.SetRotation(InRot.Quaternion() * PropertyTransform.GetRotation());
				// Apply delta scale
				PropertyTransform.SetScale3D(PropertyTransform.GetScale3D() + InScale);

				// Convert world transform back into local space
				if (bLocalSpace)
				{
					PropertyTransform = PropertyTransform.GetRelativeTransform(DisplayWidgetToWorld);
				}
				BestSelectedItem->PreEditChange(nullptr);
				// Property that we actually change
				FProperty* SetProperty = nullptr;
				if (bEditedPropertyIsTransform)
				{
					SetPropertyValueByName<FTransform>(BestSelectedItem, EditedPropertyName, EditedPropertyIndex, PropertyTransform, SetProperty);
				}
				else
				{
					SetPropertyValueByName<FVector>(BestSelectedItem, EditedPropertyName, EditedPropertyIndex, PropertyTransform.GetLocation(), SetProperty);
				}
				FPropertyChangedEvent PropertyChangeEvent(SetProperty);
				BestSelectedItem->PostEditChangeProperty(PropertyChangeEvent);

				return true;
			}
		}
	}

	return false;
}

bool FLegacyEdModeWidgetHelper::HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click)
{
	if (UsesPropertyWidgets() && (HitProxy != nullptr))
	{
		if (HitProxy->IsA(HPropertyWidgetProxy::StaticGetType()))
		{
			HPropertyWidgetProxy* PropertyProxy = (HPropertyWidgetProxy*)HitProxy;
			EditedPropertyName = PropertyProxy->PropertyName;
			EditedPropertyIndex = PropertyProxy->PropertyIndex;
			bEditedPropertyIsTransform = PropertyProxy->bPropertyIsTransform;
			bLocalSpace = PropertyProxy->bLocalSpace;
			return true;
		}
		// Left clicking on an actor, stop editing a property
		else if (HitProxy->IsA(HActor::StaticGetType()))
		{
			EditedPropertyName = FString();
			EditedPropertyIndex = INDEX_NONE;
			bEditedPropertyIsTransform = false;
			bLocalSpace = true;
		}
	}

	return false;
}

void FLegacyEdModeWidgetHelper::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	if (UsesPropertyWidgets())
	{
		const bool bHitTesting = PDI->IsHitTesting();

		FTransform DisplayWidgetToWorld;
		UObject* BestSelectedItem = GetItemToTryDisplayingWidgetsFor(/*out*/ DisplayWidgetToWorld);

		if (BestSelectedItem != nullptr)
		{
			UClass* Class = BestSelectedItem->GetClass();
			TArray<FPropertyWidgetInfo> WidgetInfos;
			GetPropertyWidgetInfos(Class, BestSelectedItem, WidgetInfos);

			const bool bAllowEditWidgetAxisDisplay = GetDefault<ULevelEditorViewportSettings>()->bAllowEditWidgetAxisDisplay;

			FEditorScriptExecutionGuard ScriptGuard;
			for (const FPropertyWidgetInfo& WidgetInfo : WidgetInfos)
			{
				const bool bSelected = (WidgetInfo.PropertyName == EditedPropertyName) && (WidgetInfo.PropertyIndex == EditedPropertyIndex);

				FTransform LocalWidgetTransform;
				FString ValidationMessage;
				FColor WidgetColor;
				WidgetInfo.GetTransformAndColor(BestSelectedItem, bSelected, /*out*/ LocalWidgetTransform, /*out*/ ValidationMessage, /*out*/ WidgetColor);
				
				FTransform WorldWidgetTransform = LocalWidgetTransform;
				if (WidgetInfo.bLocalSpace)
				{
					WorldWidgetTransform = LocalWidgetTransform * DisplayWidgetToWorld;
				}
				const FMatrix WidgetTM(WorldWidgetTransform.ToMatrixWithScale());

				const double WidgetSize = 0.035;
				const double ZoomFactor = FMath::Min(View->ViewMatrices.GetViewToClip().M[0][0], View->ViewMatrices.GetViewToClip().M[1][1]);
				const float WidgetRadius = static_cast<float>(View->Project(WorldWidgetTransform.GetTranslation()).W * (WidgetSize / ZoomFactor));

				if (bHitTesting) PDI->SetHitProxy(new HPropertyWidgetProxy(WidgetInfo.PropertyName, WidgetInfo.PropertyIndex, WidgetInfo.bIsTransform, WidgetInfo.bLocalSpace));
				DrawWireDiamond(PDI, WidgetTM, WidgetRadius, WidgetColor, SDPG_Foreground);
				if (bAllowEditWidgetAxisDisplay && WidgetInfo.bIsTransform)
				{
					constexpr float AxisScale = 1.5f;
					DrawCoordinateSystem(PDI, WorldWidgetTransform.GetTranslation(), WorldWidgetTransform.GetRotation().Rotator(), AxisScale * WidgetRadius, SDPG_Foreground);
				}
				if (bHitTesting) PDI->SetHitProxy(NULL);
			}
		}
	}
}

void FLegacyEdModeWidgetHelper::DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas)
{
	if (UsesPropertyWidgets())
	{
		FTransform DisplayWidgetToWorld;
		UObject* BestSelectedItem = GetItemToTryDisplayingWidgetsFor(/*out*/ DisplayWidgetToWorld);
		if (BestSelectedItem != nullptr)
		{
			FEditorScriptExecutionGuard ScriptGuard;

			FIntPoint SizeXY = Viewport->GetSizeXY();
			int32 ScaledX = FMath::TruncToInt32(static_cast<float>(SizeXY.X) / Canvas->GetDPIScale());
			int32 ScaledY = FMath::TruncToInt32(static_cast<float>(SizeXY.Y) / Canvas->GetDPIScale());

			FIntPoint SizeXYWithDPIScale{ ScaledX, ScaledY };

			const int32 HalfX = SizeXYWithDPIScale.X / 2;
			const int32 HalfY = SizeXYWithDPIScale.Y / 2;

			UClass* Class = BestSelectedItem->GetClass();
			TArray<FPropertyWidgetInfo> WidgetInfos;
			GetPropertyWidgetInfos(Class, BestSelectedItem, WidgetInfos);
			for (const FPropertyWidgetInfo& WidgetInfo : WidgetInfos)
			{
				FTransform LocalWidgetTransform;
				FString ValidationMessage;
				FColor IgnoredWidgetColor;
				WidgetInfo.GetTransformAndColor(BestSelectedItem, /*bSelected=*/ false, /*out*/ LocalWidgetTransform, /*out*/ ValidationMessage, /*out*/ IgnoredWidgetColor);

				FTransform WorldWidgetTransform = LocalWidgetTransform;
				if (WidgetInfo.bLocalSpace)
				{
					WorldWidgetTransform = LocalWidgetTransform * DisplayWidgetToWorld;
				}
				
				const FPlane Proj = View->Project(WorldWidgetTransform.GetTranslation());
				if (Proj.W > 0.f)
				{
					// do some string fixing
					const uint32 VectorIndex = WidgetInfo.PropertyIndex;
					const FString WidgetDisplayName = WidgetInfo.DisplayName + ((VectorIndex != INDEX_NONE) ? FString::Printf(TEXT("[%d]"), VectorIndex) : TEXT(""));
					const FString DisplayString = ValidationMessage.IsEmpty() ? WidgetDisplayName : ValidationMessage;

					const int32 XPos = static_cast<int32>(HalfX + (HalfX * Proj.X));
					const int32 YPos = static_cast<int32>(HalfY + (HalfY * (Proj.Y * -1.f)));
					FCanvasTextItem TextItem(FVector2D(XPos + 5, YPos), FText::FromString(DisplayString), GEngine->GetSmallFont(), FLinearColor::White);
					TextItem.EnableShadow(FLinearColor::Black);
					Canvas->DrawItem(TextItem);
				}
			}
		}
	}
}

bool FLegacyEdModeWidgetHelper::CanCreateWidgetForStructure(const UStruct* InPropStruct)
{
	return InPropStruct && (InPropStruct->GetFName() == NAME_Vector || InPropStruct->GetFName() == NAME_Transform);
}

bool FLegacyEdModeWidgetHelper::CanCreateWidgetForProperty(FProperty* InProp)
{
	FStructProperty* TestProperty = CastField<FStructProperty>(InProp);
	if (!TestProperty)
	{
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InProp);
		if (ArrayProperty)
		{
			TestProperty = CastField<FStructProperty>(ArrayProperty->Inner);
		}
	}
	return (TestProperty != NULL) && CanCreateWidgetForStructure(TestProperty->Struct);
}

bool FLegacyEdModeWidgetHelper::ShouldCreateWidgetForProperty(FProperty* InProp)
{
	return CanCreateWidgetForProperty(InProp) && InProp->HasMetaData(MD_MakeEditWidget);
}

AActor* FLegacyEdModeWidgetHelper::GetFirstSelectedActorInstance() const
{
	return Owner ? Owner->GetSelectedActors()->GetTop<AActor>() : nullptr;
}

void FLegacyEdModeWidgetHelper::GetPropertyWidgetInfos(const UStruct* InStruct, const void* InContainer, TArray<FPropertyWidgetInfo>& OutInfos) const
{
	TArray<FPropertyWidgetInfoChainElement> Chain;
	FPropertyWidgetInfoChainElement::RecursiveGet(*this, InStruct, InContainer, OutInfos, Chain);
}

UObject* FLegacyEdModeWidgetHelper::GetItemToTryDisplayingWidgetsFor(FTransform& OutLocalToWorld) const
{
	// Determine what is selected, preferring a component over an actor
	USceneComponent* SelectedComponent = Owner ? Owner->GetSelectedComponents()->GetTop<USceneComponent>() : nullptr;
	UObject* BestSelectedItem = SelectedComponent;

	if (SelectedComponent == nullptr)
	{
		AActor* SelectedActor = GetFirstSelectedActorInstance();
		if (SelectedActor != nullptr)
		{
			if (USceneComponent* RootComponent = SelectedActor->GetRootComponent())
			{
				BestSelectedItem = SelectedActor;
				OutLocalToWorld = RootComponent->GetComponentToWorld();
			}
		}
	}
	else
	{
		OutLocalToWorld = SelectedComponent->GetComponentToWorld();
	}

	return BestSelectedItem;
}

void UBaseLegacyWidgetEdMode::Initialize()
{
	WidgetHelper = CreateWidgetHelper();
	WidgetHelper->Owner = this->Owner;
	WidgetHelper->ParentModeInterface = this;

	Super::Initialize();
}

TSharedRef<FLegacyEdModeWidgetHelper> UBaseLegacyWidgetEdMode::CreateWidgetHelper()
{
	return MakeShared<FLegacyEdModeWidgetHelper>();
}

bool UBaseLegacyWidgetEdMode::InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale)
{
	if (WidgetHelper->InputDelta(InViewportClient, InViewport, InDrag, InRot, InScale))
	{
		return true;
	}

	return ILegacyEdModeViewportInterface::InputDelta(InViewportClient, InViewport, InDrag, InRot, InScale);
}

bool UBaseLegacyWidgetEdMode::HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click)
{
	if (WidgetHelper->HandleClick(InViewportClient, HitProxy, Click))
	{
		return true;
	}

	return ILegacyEdModeViewportInterface::HandleClick(InViewportClient, HitProxy, Click);
}

void UBaseLegacyWidgetEdMode::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	WidgetHelper->Render(View, Viewport, PDI);

	ILegacyEdModeWidgetInterface::Render(View, Viewport, PDI);
}

void UBaseLegacyWidgetEdMode::DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas)
{
	WidgetHelper->DrawHUD(ViewportClient, Viewport, View, Canvas);

	ILegacyEdModeWidgetInterface::DrawHUD(ViewportClient, Viewport, View, Canvas);
}

bool UBaseLegacyWidgetEdMode::AllowWidgetMove()
{
	return WidgetHelper->AllowWidgetMove();
}

bool UBaseLegacyWidgetEdMode::CanCycleWidgetMode() const
{
	return WidgetHelper->CanCycleWidgetMode();
}

bool UBaseLegacyWidgetEdMode::ShowModeWidgets() const
{
	return WidgetHelper->ShowModeWidgets();
}

EAxisList::Type UBaseLegacyWidgetEdMode::GetWidgetAxisToDraw(UE::Widget::EWidgetMode InWidgetMode) const
{
	return WidgetHelper->GetWidgetAxisToDraw(InWidgetMode);
}

FVector UBaseLegacyWidgetEdMode::GetWidgetLocation() const
{
	return WidgetHelper->GetWidgetLocation();
}

bool UBaseLegacyWidgetEdMode::ShouldDrawWidget() const
{
	return WidgetHelper->ShouldDrawWidget();
}

bool UBaseLegacyWidgetEdMode::UsesTransformWidget() const
{
	return WidgetHelper->UsesTransformWidget();
}

bool UBaseLegacyWidgetEdMode::UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const
{
	return WidgetHelper->UsesTransformWidget(CheckMode);
}

FVector UBaseLegacyWidgetEdMode::GetWidgetNormalFromCurrentAxis(void* InData)
{
	return WidgetHelper->GetWidgetNormalFromCurrentAxis(InData);
}

void UBaseLegacyWidgetEdMode::SetCurrentWidgetAxis(EAxisList::Type InAxis)
{
	WidgetHelper->SetCurrentWidgetAxis(InAxis);
}

EAxisList::Type UBaseLegacyWidgetEdMode::GetCurrentWidgetAxis() const
{
	return WidgetHelper->GetCurrentWidgetAxis();
}

bool UBaseLegacyWidgetEdMode::UsesPropertyWidgets() const
{
	return false;
}

bool UBaseLegacyWidgetEdMode::GetCustomDrawingCoordinateSystem(FMatrix& InMatrix, void* InData)
{
	return WidgetHelper->GetCustomDrawingCoordinateSystem(InMatrix, InData);
}

bool UBaseLegacyWidgetEdMode::GetCustomInputCoordinateSystem(FMatrix& InMatrix, void* InData)
{
	return WidgetHelper->GetCustomInputCoordinateSystem(InMatrix, InData);
}

void UBaseLegacyWidgetEdMode::ActorSelectionChangeNotify()
{
	WidgetHelper->ActorSelectionChangeNotify();
}

bool UBaseLegacyWidgetEdMode::AllowsViewportDragTool() const
{
	return WidgetHelper->AllowsViewportDragTool();
}

bool UBaseLegacyWidgetEdMode::RequiresLegacyViewportInteractions() const
{
	return WidgetHelper->RequiresLegacyViewportInteractions();
}

