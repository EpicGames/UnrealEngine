// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/StandardToolModeCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "GenericPlatform/GenericApplication.h"
#include "HAL/Platform.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

#define LOCTEXT_NAMESPACE "StandardToolModeCommands"

UE_DEFINE_TCOMMANDS(FStandardToolModeCommands)

FStandardToolModeCommands::FStandardToolModeCommands() :
	TCommands<FStandardToolModeCommands>(
		"StandardToolCommands", // Context name for fast lookup
		NSLOCTEXT("Contexts", "StandardToolCommands", "Standard Tool Commands"), // Localized context name for displaying
		NAME_None, // Parent
		FAppStyle::GetAppStyleSetName() // Icon Style Set
		)
{}


void FStandardToolModeCommands::RegisterCommands()
{
	TSharedPtr<FUICommandInfo> IncreaseBrushSize;
	UI_COMMAND(IncreaseBrushSize, "Increase Brush Size", "Increases the size of the brush", EUserInterfaceActionType::Button, FInputChord(EKeys::RightBracket));
	Commands.Add(EStandardToolModeCommands::IncreaseBrushSize, IncreaseBrushSize);

	TSharedPtr<FUICommandInfo> DecreaseBrushSize;
	UI_COMMAND(DecreaseBrushSize, "Decrease Brush Size", "Decreases the size of the brush", EUserInterfaceActionType::Button, FInputChord(EKeys::LeftBracket));
	Commands.Add(EStandardToolModeCommands::DecreaseBrushSize, DecreaseBrushSize);

	TSharedPtr<FUICommandInfo> ToggleWireframe;
	UI_COMMAND(ToggleWireframe, "Toggle Wireframe", "Toggle Wireframe overlay on/off", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::W));
	Commands.Add(EStandardToolModeCommands::ToggleWireframe, ToggleWireframe);

}


TSharedPtr<FUICommandInfo> FStandardToolModeCommands::FindStandardCommand(EStandardToolModeCommands Command) const
{
	const TSharedPtr<FUICommandInfo>* Found = Commands.Find(Command);
	checkf(Found != nullptr, TEXT("FStandardToolModeCommands::FindStandardCommand: standard command %d was not found!"), EnumToUnderlyingType(Command));
	return (Found != nullptr) ? *Found : nullptr;
}


#undef LOCTEXT_NAMESPACE
