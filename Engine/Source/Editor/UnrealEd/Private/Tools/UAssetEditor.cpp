// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/UAssetEditor.h"

#include "Framework/Docking/TabManager.h"
#include "Tools/BaseAssetToolkit.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UAssetEditor)

#define LOCTEXT_NAMESPACE "BaseAssetEditor"

//////////////////////////////////
// UAssetEditor

UAssetEditor::UAssetEditor()
{
}

FName UAssetEditor::GetEditorName() const
{
	FName EditorName = ToolkitInstance->GetEditorName();
	if (EditorName == NAME_None)
	{
		EditorName = GetClass()->GetFName();
	}

	return EditorName;
}

void UAssetEditor::FocusWindow(UObject* ObjectToFocusOn /*= nullptr*/)
{
}

TSharedPtr<class FTabManager> UAssetEditor::GetAssociatedTabManager()
{
	return ToolkitInstance->GetAssociatedTabManager();
}

void UAssetEditor::Initialize()
{
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	AssetEditorSubsystem->RegisterUAssetEditor(this);
	TSharedPtr<FBaseAssetToolkit> Toolkit = CreateToolkit();
	ToolkitInstance = Toolkit.Get();
	check(ToolkitInstance != nullptr);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;

	Toolkit->CreateWidgets();
	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = Toolkit->GetDefaultLayout();
	TArray<UObject*> ObjectsToEdit;
	GetObjectsToEdit(ObjectsToEdit);
	check(ObjectsToEdit.Num() > 0 && ObjectsToEdit[0] != nullptr);
	Toolkit->InitAssetEditor(EToolkitMode::Standalone, TSharedPtr<IToolkitHost>(), GetEditorName(), StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, ObjectsToEdit, false);
	Toolkit->SetEditingObject(ObjectsToEdit[0]);
}

void UAssetEditor::GetObjectsToEdit(TArray<UObject*>& InObjectsToEdit)
{
}

TSharedPtr<FBaseAssetToolkit> UAssetEditor::CreateToolkit()
{
	return MakeShareable(new FBaseAssetToolkit(this));
}

void UAssetEditor::OnToolkitClosed()
{
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	if (AssetEditorSubsystem)
	{
		AssetEditorSubsystem->UnregisterUAssetEditor(this);
	}
}

#undef LOCTEXT_NAMESPACE
