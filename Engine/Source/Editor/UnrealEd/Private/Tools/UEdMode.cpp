// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/UEdMode.h"

#include "EditorModeManager.h"
#include "EditorModes.h"
#include "Toolkits/BaseToolkit.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "Toolkits/ToolkitManager.h"
#include "InteractiveToolManager.h"
#include "InteractiveToolQueryInterfaces.h"
#include "GameFramework/Actor.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Interfaces/TypedElementObjectInterface.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "EditorViewportClient.h"
#include "EditorViewportCommands.h"

//////////////////////////////////
// UEdMode

#include UE_INLINE_GENERATED_CPP_BY_NAME(UEdMode)

UEdMode::UEdMode()
	: Owner(nullptr)
{
	EditorToolsContext = nullptr;
	ModeToolsContext = nullptr;
	ToolCommandList = MakeShareable(new FUICommandList);
}

void UEdMode::Initialize()
{
}

void UEdMode::SelectNone()
{
}

bool UEdMode::ProcessEditDelete()
{
	return false;
}

void UEdMode::Enter()
{
	// Update components for selected actors, in case the mode we just exited
	// was hijacking selection events selection and not updating components.
	Owner->GetEditorSelectionSet()->ForEachSelectedObject<AActor>([](AActor* ActorPtr)
		{
			ActorPtr->MarkComponentsRenderStateDirty();
			return true;
		});

	CreateInteractiveToolsContexts();

	ModeToolsContext->ToolManager->OnToolStarted.AddUObject(this, &UEdMode::OnToolStarted);
	ModeToolsContext->ToolManager->OnToolEnded.AddUObject(this, &UEdMode::OnToolEnded);

	// Create the settings object so that the toolkit has access to the object we are going to use at creation time
	if (SettingsClass.IsValid())
	{
		UClass* LoadedSettingsObject = SettingsClass.LoadSynchronous();
		SettingsObject = NewObject<UObject>(this, LoadedSettingsObject);
	}

	// Now that the context is ready, make the toolkit
	CreateToolkit();
	if (Toolkit.IsValid())
	{
		Toolkit->Init(Owner->GetToolkitHost(), this);
	}

	BindCommands();

	if (SettingsObject)
	{
		SettingsObject->LoadConfig();

		if (Toolkit.IsValid())
		{
			Toolkit->SetModeSettingsObject(SettingsObject);
		}
	}

	Owner->OnEditorModeIDChanged().AddUObject(this, &UEdMode::OnModeActivated);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FEditorDelegates::EditorModeIDEnter.Broadcast(GetID());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UEdMode::RegisterTool(TSharedPtr<FUICommandInfo> UICommand, FString ToolIdentifier, UInteractiveToolBuilder* Builder, EToolsContextScope ToolScope)
{
	if (!Toolkit.IsValid())
	{
		return;
	}

	if (ToolScope == EToolsContextScope::Default)
	{
		ToolScope = GetDefaultToolScope();
	}

	UEditorInteractiveToolsContext* UseToolsContext = GetInteractiveToolsContext(ToolScope);
	if (ensure(UseToolsContext != nullptr) == false)
	{
		return;
	}

	const TSharedRef<FUICommandList>& CommandList = Toolkit->GetToolkitCommands();
	UseToolsContext->ToolManager->RegisterToolType(ToolIdentifier, Builder);
	CommandList->MapAction(UICommand,
		FExecuteAction::CreateUObject(UseToolsContext, &UEdModeInteractiveToolsContext::StartTool, ToolIdentifier),
		FCanExecuteAction::CreateWeakLambda(UseToolsContext, [this, ToolIdentifier, UseToolsContext]() {
			return ShouldToolStartBeAllowed(ToolIdentifier) &&
				UseToolsContext->ToolManager->CanActivateTool(EToolSide::Mouse, ToolIdentifier);
		}),
		FIsActionChecked::CreateUObject(UseToolsContext, &UEdModeInteractiveToolsContext::IsToolActive, EToolSide::Mouse, ToolIdentifier),
		EUIActionRepeatMode::RepeatDisabled);

	if (ToolScope == EToolsContextScope::Editor)
	{
		RegisteredEditorTools.Emplace(UICommand, ToolIdentifier);
	}
}

bool UEdMode::ShouldToolStartBeAllowed(const FString& ToolIdentifier) const
{
	// Disallow starting tools when playing in editor or simulating.
	return !GEditor->PlayWorld && !GIsPlayInEditorWorld;
}

void UEdMode::Exit()
{
	if (SettingsObject)
	{
		SettingsObject->SaveConfig();
	}

	if (UObjectInitialized())
	{
		Owner->OnEditorModeIDChanged().RemoveAll(this);

		// Shutdown the Mode-scope ToolsContext, and notify the EditorToolsContext so it can release the reference it holds.
		// Do this before shutting down the Toolkit as if a Mode-scope Tool is still active it will need to clean up
		DestroyInteractiveToolsContexts();
	}

	if (Toolkit.IsValid())
	{
		const TSharedRef<FUICommandList>& CommandList = Toolkit->GetToolkitCommands();
		for (auto& RegisteredTool : RegisteredEditorTools)
		{
			CommandList->UnmapAction(RegisteredTool.Key);
			EditorToolsContext->ToolManager->UnregisterToolType(RegisteredTool.Value);
		}

		Toolkit.Reset();
	}
	RegisteredEditorTools.SetNum(0);

	// disconnect from the Mode Manager's shared ToolsContext
	EditorToolsContext = nullptr;
	ModeToolsContext = nullptr;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FEditorDelegates::EditorModeIDExit.Broadcast(GetID());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UEdMode::UsesToolkits() const
{
	return true;
}

UWorld* UEdMode::GetWorld() const
{
	return EditorToolsContext.IsValid() ? EditorToolsContext->GetWorld() : nullptr;
}

FEditorModeTools* UEdMode::GetModeManager() const
{
	return EditorToolsContext.IsValid() ? EditorToolsContext->GetParentEditorModeManager() : nullptr;
}

AActor* UEdMode::GetFirstSelectedActorInstance() const
{
	if (FEditorModeTools* ModeManager = GetModeManager())
	{
		return ModeManager->GetEditorSelectionSet()->GetTopSelectedObject<AActor>();
	}
	return nullptr;
}

UInteractiveToolManager* UEdMode::GetToolManager(EToolsContextScope ToolScope) const
{
	if (ToolScope == EToolsContextScope::Default)
	{
		ToolScope = GetDefaultToolScope();
	}

	if (ToolScope == EToolsContextScope::Editor)
	{
		return (EditorToolsContext.IsValid()) ? EditorToolsContext->ToolManager : nullptr;
	}
	else if (ToolScope == EToolsContextScope::EdMode)
	{
		return ModeToolsContext ? ModeToolsContext->ToolManager : nullptr;
	}
	ensure(false);
	return nullptr;
}

UEditorInteractiveToolsContext* UEdMode::GetInteractiveToolsContext(EToolsContextScope ToolScope) const
{
	if (ToolScope == EToolsContextScope::Default)
	{
		ToolScope = GetDefaultToolScope();
	}

	if (ToolScope == EToolsContextScope::Editor)
	{
		return EditorToolsContext.IsValid() ? EditorToolsContext.Get() : nullptr;
	}
	else if (ToolScope == EToolsContextScope::EdMode)
	{
		return ModeToolsContext;
	}
	ensure(false);
	return nullptr;
}

void UEdMode::CreateToolkit()
{
	if (!UsesToolkits())
	{
		return;
	}

	check(!Toolkit.IsValid())
	Toolkit = MakeShareable(new FModeToolkit);
}

void UEdMode::OnModeActivated(const FEditorModeID& InID, bool bIsActive)
{
	if (InID == GetID())
	{
		if (bIsActive)
		{
			EditorToolsContext->OnChildEdModeActivated(ModeToolsContext);
			EditorToolsContext->ToolManager->OnToolStarted.AddUObject(this, &UEdMode::OnToolStarted);
			EditorToolsContext->ToolManager->OnToolEnded.AddUObject(this, &UEdMode::OnToolEnded);
		}
		else
		{
			EditorToolsContext->ToolManager->OnToolStarted.RemoveAll(this);
			EditorToolsContext->ToolManager->OnToolEnded.RemoveAll(this);
			EditorToolsContext->OnChildEdModeDeactivated(ModeToolsContext);
		}
	}
}

void UEdMode::CreateInteractiveToolsContexts()
{
	// Editor-scope ToolsContext is provided by the Mode Manager
	EditorToolsContext = Owner->GetInteractiveToolsContext();
	check(EditorToolsContext.IsValid());

	// Mode-scope ToolsContext is created derived from the Editor-Scope ToolsContext so that the UInputRouter can be shared
	ModeToolsContext = EditorToolsContext->CreateNewChildEdModeToolsContext();
	check(ModeToolsContext);
}

void UEdMode::DestroyInteractiveToolsContexts()
{
	if (ModeToolsContext)
	{
		ModeToolsContext->ShutdownContext();
	}
}

bool UEdMode::IsSnapRotationEnabled()
{
	return GetDefault<ULevelEditorViewportSettings>()->RotGridEnabled;
}

void UEdMode::BindCommands()
{
	if (Toolkit && GetModeManager())
	{
		const TSharedRef<FUICommandList>& CommandList = Toolkit->GetToolkitCommands();
		const FEditorViewportCommands& ViewportCommands = FEditorViewportCommands::Get();
		CommandList->MapAction(
			ViewportCommands.FocusViewportToSelection,
			FExecuteAction::CreateLambda([this]()
				{
					if (ensure(GetModeManager() && GetModeManager()->GetFocusedViewportClient() && GetModeManager()->GetFocusedViewportClient()->IsLevelEditorClient()))
					{
						FEditorViewportClient* ViewportClient = GetModeManager()->GetFocusedViewportClient();
						FBox FocusBox = ComputeCustomViewportFocus();
						if (FocusBox.IsValid)
						{
							// This method has custom logic for smoothly moving linked ortho viewports, so we prefer it for level editor viewports
							GEditor->MoveViewportCamerasToBox(FocusBox, true);
							// Note that to support non-level-editor viewports we could do GetModeManager()->GetFocusedViewportClient()->FocusViewportOnBox(FocusBox)
							// However we instead currently expect that these viewports support the focus api by overriding ComputeBoundingBoxForViewportFocus()
						}
					}
				}),
			FCanExecuteAction::CreateLambda([this]() 
				{
					// For level-editor viewports, implement the focus api here to decouple it from the standard level-editor focus logic (which depends on the level editor selection)
					// Note we do not execute for non-level-editor viewports (e.g., asset editors), which must implement their own focus logic
					return GetModeManager() && GetModeManager()->GetFocusedViewportClient() 
						&& GetModeManager()->GetFocusedViewportClient()->IsLevelEditorClient() 
						&& HasCustomViewportFocus();
				}
			)
		);
	}

}

FBox UEdMode::GetFocusBoxFromActiveToolFocusAPI() const
{
	UInteractiveToolManager* ToolManager = GetToolManager();
	if (ToolManager && ToolManager->HasAnyActiveTool())
	{
		UInteractiveTool* Tool = ToolManager->GetActiveTool(EToolSide::Mouse);
		IInteractiveToolCameraFocusAPI* FocusAPI = Cast<IInteractiveToolCameraFocusAPI>(Tool);
		if (FocusAPI && FocusAPI->SupportsWorldSpaceFocusBox())
		{
			return FocusAPI->GetWorldSpaceFocusBox();
		}
	}
	return FBox();
}

bool UEdMode::HasCustomViewportFocus() const
{
	// Support ITF Focus API
	FBox FocusBox = GetFocusBoxFromActiveToolFocusAPI();
	return (bool)FocusBox.IsValid;
}

FBox UEdMode::ComputeCustomViewportFocus() const
{
	// Support ITF Focus API
	return GetFocusBoxFromActiveToolFocusAPI();
}


