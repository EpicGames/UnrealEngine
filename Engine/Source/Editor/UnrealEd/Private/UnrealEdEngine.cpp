// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/UnrealEdEngine.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/App.h"
#include "Model.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectIterator.h"
#include "Framework/Application/SlateApplication.h"
#include "Components/PrimitiveComponent.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "Materials/Material.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "ISourceControlModule.h"
#include "SourceControlHelpers.h"
#include "SourceControlOperations.h"
#include "Settings/EditorExperimentalSettings.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Preferences/UnrealEdKeyBindings.h"
#include "Preferences/UnrealEdOptions.h"
#include "GameFramework/Volume.h"
#include "Components/ArrowComponent.h"
#include "Components/BillboardComponent.h"
#include "Components/BrushComponent.h"
#include "Engine/GameViewportClient.h"
#include "Engine/Selection.h"
#include "Editor.h"
#include "LevelEditorViewport.h"
#include "EditorCommandLineUtils.h"
#include "EditorModeRegistry.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "Elements/Framework/EditorElements.h"
#include "UnrealEdMisc.h"
#include "UnrealEdGlobals.h"

#include "Animation/AnimCompress.h"

#include "EditorSupportDelegates.h"
#include "EditorLevelUtils.h"
#include "EdMode.h"
#include "PropertyEditorModule.h"
#include "LevelEditor.h"
#include "InstancedStaticMeshDelegates.h"
#include "Interfaces/IMainFrameModule.h"
#include "Settings/BlueprintEditorSettingsCustomization.h"
#include "Settings/BlueprintEditorProjectSettingsCustomization.h"
#include "Settings/EditorLoadingSavingSettingsCustomization.h"
#include "Settings/GameMapsSettingsCustomization.h"
#include "Settings/LevelEditorPlaySettingsCustomization.h"
#include "Settings/ProjectPackagingSettingsCustomization.h"
#include "Settings/LevelEditorPlayNetworkEmulationSettings.h"
#include "StatsViewerModule.h"
#include "SnappingUtils.h"
#include "PackageAutoSaver.h"
#include "PerformanceMonitor.h"
#include "BSPOps.h"
#include "SourceCodeNavigation.h"
#include "AutoReimport/AutoReimportManager.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "AutoReimport/AssetSourceFilenameCache.h"
#include "UObject/UObjectThreadContext.h"
#include "EngineUtils.h"
#include "EngineAnalytics.h"
#include "CookerSettings.h"
#include "Logging/MessageLog.h"
#include "Misc/MessageDialog.h"
#include "Logging/MessageLog.h"
#include "Subsystems/EditorActorSubsystem.h"
#include "ProfilingDebugging/StallDetector.h"
#include "Settings/EditorStyleSettings.h"
#include "ISettingsModule.h"
#include "Settings/EditorStyleSettingsCustomization.h"
#include "GameMapsSettings.h"
#include "HAL/PlatformApplicationMisc.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ObjectTools.h"
#include "Cooker/ExternalCookOnTheFlyServer.h"
#include "ISettingsSection.h"
#include "DirectoryWatcherModule.h"
#include "IDirectoryWatcher.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UnrealEdEngine)

DEFINE_LOG_CATEGORY_STATIC(LogUnrealEdEngine, Log, All);

void UUnrealEdEngine::Init(IEngineLoop* InEngineLoop)
{
	Super::Init(InEngineLoop);

	RegisterEditorElements();

	RebuildTemplateMapData();

	// Build databases used by source code navigation
	FSourceCodeNavigation::Initialize();

	PackageAutoSaver.Reset(new FPackageAutoSaver);
	PackageAutoSaver->LoadRestoreFile();	

#if !UE_BUILD_DEBUG
	if( !GEditorSettingsIni.IsEmpty() )
	{
		// We need the game agnostic ini for this code
		PerformanceMonitor = new FPerformanceMonitor;
	}
#endif

	// Register for the package dirty state updated callback to catch packages that have been modified and need to be checked out.
	UPackage::PackageDirtyStateChangedEvent.AddUObject(this, &UUnrealEdEngine::OnPackageDirtyStateUpdated);

	// Watch for new content mount paths
	FPackageName::OnContentPathMounted().AddUObject(this, &UUnrealEdEngine::OnContentPathMounted);
	FPackageName::OnContentPathDismounted().AddUObject(this, &UUnrealEdEngine::OnContentPathDismounted);

	// Register to the PostGarbageCollect delegate, as we want to use this to trigger the RefreshAllBroweser delegate from 
	// here rather then from Core
	FCoreUObjectDelegates::GetPostGarbageCollect().AddUObject(this, &UUnrealEdEngine::OnPostGarbageCollect);

	// register to color picker changed event and trigger RedrawAllViewports when that happens */
	FCoreDelegates::ColorPickerChanged.AddUObject(this, &UUnrealEdEngine::OnColorPickerChanged);

	// register windows message pre and post handler
	FEditorSupportDelegates::PreWindowsMessage.AddUObject(this, &UUnrealEdEngine::OnPreWindowsMessage);
	FEditorSupportDelegates::PostWindowsMessage.AddUObject(this, &UUnrealEdEngine::OnPostWindowsMessage);

	FHierarchicalInstancedStaticMeshDelegates::OnTreeBuilt.AddUObject(this, &UUnrealEdEngine::OnHISMTreeBuilt);
	USelection::SelectionElementSelectionPtrChanged.AddUObject(this, &UUnrealEdEngine::OnEditorElementSelectionPtrChanged);

	// Initialize the snap manager
	FSnappingUtils::InitEditorSnappingTools();

	// Register for notification of volume changes
	AVolume::GetOnVolumeShapeChangedDelegate().AddStatic(&FBSPOps::HandleVolumeShapeChanged);

	// Iterate over all always fully loaded packages and load them.
	if (!IsRunningCommandlet())
	{
		for( int32 PackageNameIndex=0; PackageNameIndex<PackagesToBeFullyLoadedAtStartup.Num(); PackageNameIndex++ )
		{
			const FString& PackageName = PackagesToBeFullyLoadedAtStartup[PackageNameIndex];
			// Load package if it's found in the package file cache.
			if( FPackageName::DoesPackageExist( PackageName ) )
			{
				LoadPackage( NULL, *PackageName, LOAD_None );
			}
		}
	}

	// Populate the data structures related to the sprite category visibility feature for use elsewhere in the editor later
	TArray<FSpriteCategoryInfo> SortedSpriteInfo;
	UUnrealEdEngine::MakeSortedSpriteInfo(SortedSpriteInfo);

	// Iterate over the sorted list, constructing a mapping of unlocalized categories to the index the localized category
	// resides in. This is an optimization to prevent having to localize values repeatedly.
	for( int32 InfoIndex = 0; InfoIndex < SortedSpriteInfo.Num(); ++InfoIndex )
	{
		const FSpriteCategoryInfo& SpriteInfo = SortedSpriteInfo[InfoIndex];
		SpriteIDToIndexMap.Add( SpriteInfo.Category, InfoIndex );
	}

	if (FPaths::IsProjectFilePathSet() && GIsEditor && !FApp::IsUnattended())
	{

		UE_SCOPED_ENGINE_ACTIVITY(TEXT("Initializing AutoReimportManager"));
		AutoReimportManager = NewObject<UAutoReimportManager>();
		AutoReimportManager->Initialize();
	}

	// register details panel customizations
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		PropertyModule.RegisterCustomClassLayout("BlueprintEditorSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FBlueprintEditorSettingsCustomization::MakeInstance));
		PropertyModule.RegisterCustomClassLayout("BlueprintEditorProjectSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FBlueprintEditorProjectSettingsCustomization::MakeInstance));
		PropertyModule.RegisterCustomClassLayout("EditorLoadingSavingSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FEditorLoadingSavingSettingsCustomization::MakeInstance));
		PropertyModule.RegisterCustomClassLayout("GameMapsSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FGameMapsSettingsCustomization::MakeInstance));
		PropertyModule.RegisterCustomClassLayout("LevelEditorPlaySettings", FOnGetDetailCustomizationInstance::CreateStatic(&FLevelEditorPlaySettingsCustomization::MakeInstance));
		PropertyModule.RegisterCustomClassLayout("ProjectPackagingSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FProjectPackagingSettingsCustomization::MakeInstance));

		PropertyModule.RegisterCustomPropertyTypeLayout("LevelEditorPlayNetworkEmulationSettings", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FLevelEditorPlayNetworkEmulationSettingsDetail::MakeInstance));


		UEditorStyleSettings* Settings = GetMutableDefault<UEditorStyleSettings>();
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"); 
	

		if (SettingsModule != nullptr)
		{
			ISettingsSectionPtr StyleSettingsPtr = SettingsModule->RegisterSettings("Editor", "General", "Appearance",
				NSLOCTEXT("UnrealEd", "Appearance_UserSettingsName", "Appearance"),
				NSLOCTEXT("UnrealEd", "Appearance_UserSettingsDescription", "Customize the look of the editor."),
				Settings
			);

			StyleSettingsPtr->OnImport().BindUObject(Settings, &UEditorStyleSettings::OnImportBegin);
			StyleSettingsPtr->OnExport().BindUObject(Settings, &UEditorStyleSettings::OnExportBegin);
		} 


		FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyEditorModule.RegisterCustomClassLayout("EditorStyleSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FEditorStyleSettingsCustomization::MakeInstance));
		PropertyEditorModule.RegisterCustomPropertyTypeLayout("StyleColorList", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FStyleColorListCustomization::MakeInstance));

	}

	// Set the UE_EditorUIPid variable; must be set before constructing the UCookOnTheFlyServer
	if (!IsRunningCommandlet())
	{
		FString ParentPid = FPlatformMisc::GetEnvironmentVariable(GEditorUIPidVariable);
		if (ParentPid.IsEmpty())
		{
			FPlatformMisc::SetEnvironmentVar(GEditorUIPidVariable,
				*LexToString(FPlatformProcess::GetCurrentProcessId()));
		}
	}

	if (!IsRunningCommandlet())
	{
		UEditorExperimentalSettings const* ExperimentalSettings = GetDefault<UEditorExperimentalSettings>();
		UCookerSettings const* CookerSettings = GetDefault<UCookerSettings>();
		ECookInitializationFlags BaseCookingFlags = ECookInitializationFlags::AutoTick | ECookInitializationFlags::AsyncSave;
		BaseCookingFlags |= CookerSettings->bEnableBuildDDCInBackground ? ECookInitializationFlags::BuildDDCInBackground : ECookInitializationFlags::None;

		bool bLegacyIterative = false;
		GConfig->GetBool(TEXT("CookSettings"), TEXT("LegacyIterative"), bLegacyIterative, GEditorIni);

		if (CookerSettings->bIterativeCookingForLaunchOn)
		{
			// Despite the name, the LegacyIterative cook flag does not cause CookSettings.LegacyIterative, aka
			// bLegacyBuildDependencies. It triggers the same code as the "-iterate" commandline, which causes incremental
			// cooks when LegacyIterative=false, and causes the cook to cook iteratively using the legacy method when
			// LegacyIterative=true.
			BaseCookingFlags |= ECookInitializationFlags::LegacyIterative;
			if (bLegacyIterative)
			{
				BaseCookingFlags |= CookerSettings->bIgnoreIniSettingsOutOfDateForIteration
					? ECookInitializationFlags::IgnoreIniSettingsOutOfDate : ECookInitializationFlags::None;
				BaseCookingFlags |= CookerSettings->bIgnoreScriptPackagesOutOfDateForIteration
					? ECookInitializationFlags::IgnoreScriptPackagesOutOfDate : ECookInitializationFlags::None;
			}
		}
		
		if (CookerSettings->bEnableCookOnTheSide)
		{
			if ( ExperimentalSettings->bSharedCookedBuilds )
			{
				BaseCookingFlags |= ECookInitializationFlags::LegacyIterativeSharedBuild | ECookInitializationFlags::IgnoreIniSettingsOutOfDate;
			}

			CookServer = NewObject<UCookOnTheFlyServer>();
			CookServer->Initialize(ECookMode::CookOnTheFlyFromTheEditor, BaseCookingFlags);

			UCookOnTheFlyServer::FCookOnTheFlyStartupOptions CookOnTheFlyStartupOptions;
			CookOnTheFlyStartupOptions.Port = UCookOnTheFlyServer::FCookOnTheFlyStartupOptions::DefaultPort;
			CookOnTheFlyStartupOptions.bZenStore = GetDefault<UProjectPackagingSettings>()->bUseZenStore;
			CookServer->StartCookOnTheFly(CookOnTheFlyStartupOptions);
		}
		else if (!ExperimentalSettings->bDisableCookInEditor)
		{
			FMessageLog EditorLog("PackagingResults");
			EditorLog.Warning(NSLOCTEXT("UnrealEd", "DisableCookingInEditor=true is required",
				"bDisableCookInEditor=false is deprecated in 5.7; cooks launched from editor must be configured to call an external commandlet. "
				"See comment in EditorExperimentalSettings.h. "
				"Remove your config setting for EditorPerProjectUserSettings:[/Script/UnrealEd.EditorExperimentalSettings]:bDisableCookInEditor. "
				"Support will be removed entirely in a future release and the config setting will no longer be used."));

			CookServer = NewObject<UCookOnTheFlyServer>();
			CookServer->Initialize(ECookMode::CookByTheBookFromTheEditor, BaseCookingFlags);
		}

#if WITH_COTF
		ExternalCookOnTheFlyServer = new FExternalCookOnTheFlyServer();
#endif
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("nomcp")))
	{
		// If our editor has nomcp, pass it through to any subprocesses.
		FCommandLine::AddToSubprocessCommandLine(TEXT(" -nomcp"), ECommandLineArgumentFlags::AllContexts);
	}

	bPivotMovedIndependently = false;

#if STALL_DETECTOR
	// Start tracking stalls when we open the Main Frame
	IMainFrameModule::Get().OnMainFrameCreationFinished().AddWeakLambda(this, [this](TSharedPtr<SWindow>, bool)
		{
			bRequiresStallDetectorShutdown = true;
			UE::FStallDetector::Startup();
		});

	// Stop tracking stalls when we close the Main Frame, the closest event around
	OnEditorClose().AddLambda([this]()
		{
			// We conditionally shutdown here because for now there is no UnreadEdEngine specific shutdown 
			bRequiresStallDetectorShutdown = false;
			if (UE::FStallDetector::IsRunning())
			{
				UE::FStallDetector::Shutdown();
			}
		});
#endif
}

bool CanCookForPlatformInThisProcess( const FString& PlatformName )
{
	////////////////////////////////////////
	// hack remove this hack when we properly support changing the mobileHDR setting 
	// check if our mobile hdr setting in memory is different from the one which is saved in the config file
	
	
	FConfigFile PlatformEngineIni;
	GConfig->LoadLocalIniFile(PlatformEngineIni, TEXT("Engine"), true, *PlatformName );

	FString IniValueString;
	bool ConfigSetting = false;
	if ( PlatformEngineIni.GetString( TEXT("/Script/Engine.RendererSettings"), TEXT("r.MobileHDR"), IniValueString ) == false )
	{
		// must always match the RSetting setting because we don't have a config setting
		return true; 
	}
	ConfigSetting = IniValueString.ToBool();

	const bool CurrentRSetting = IsMobileHDR();
	if ( CurrentRSetting != ConfigSetting )
	{
		UE_LOG(LogUnrealEdEngine, Warning, TEXT("Unable to use cook in editor because r.MobileHDR from Engine ini doesn't match console value r.MobileHDR"));
		return false;
	}
	////////////////////////////////////////
	return true;
}


bool UUnrealEdEngine::CanCookByTheBookInEditor(const FString& PlatformName) const 
{ 	
	if ( !CookServer )
	{
		return false;
	}

	if ( !CanCookForPlatformInThisProcess(PlatformName) )
	{
		CookServer->ClearAllCookedData();
		return false;
	}

	return CookServer->GetCookMode() == ECookMode::CookByTheBookFromTheEditor; 
}

bool UUnrealEdEngine::CanCookOnTheFlyInEditor(const FString& PlatformName) const
{
	if ( !CookServer )
	{
		return false;
	}

	if ( !CanCookForPlatformInThisProcess(PlatformName) )
	{
		CookServer->ClearAllCookedData();
		return false;
	}

	return CookServer->GetCookMode() == ECookMode::CookOnTheFlyFromTheEditor;
}

void UUnrealEdEngine::StartCookByTheBookInEditor( const TArray<ITargetPlatform*> &TargetPlatforms, const TArray<FString> &CookMaps, const TArray<FString> &CookDirectories, const TArray<FString> &CookCultures, const TArray<FString> &IniMapSections )
{
	UCookOnTheFlyServer::FCookByTheBookStartupOptions StartupOptions;
	StartupOptions.CookMaps = CookMaps;
	StartupOptions.TargetPlatforms = TargetPlatforms;
	StartupOptions.CookDirectories = CookDirectories;
	StartupOptions.CookCultures = CookCultures;
	StartupOptions.IniMapSections = IniMapSections;
	if (GetDefault<UProjectPackagingSettings>()->GetUseZenStoreEffective())
	{
		StartupOptions.CookOptions |= ECookByTheBookOptions::ZenStore;
	}

	CookServer->StartCookByTheBook( StartupOptions );
}

bool UUnrealEdEngine::IsCookByTheBookInEditorFinished() const 
{ 
	return CookServer ? !CookServer->IsCookByTheBookRunning() : true;
}

void UUnrealEdEngine::CancelCookByTheBookInEditor()
{
	CookServer->QueueCancelCookByTheBook();
}

void UUnrealEdEngine::MakeSortedSpriteInfo(TArray<FSpriteCategoryInfo>& OutSortedSpriteInfo)
{
	struct Local
	{
		static void AddSortedSpriteInfo(TArray<FSpriteCategoryInfo>& InOutSortedSpriteInfo, const FSpriteCategoryInfo& InSpriteInfo )
		{
			const FSpriteCategoryInfo* ExistingSpriteInfo = InOutSortedSpriteInfo.FindByPredicate([&InSpriteInfo](const FSpriteCategoryInfo& SpriteInfo){ return InSpriteInfo.Category == SpriteInfo.Category; });
			if (ExistingSpriteInfo != NULL)
			{
				//Already present
				checkSlow(ExistingSpriteInfo->DisplayName.EqualTo(InSpriteInfo.DisplayName)); //Catch mismatches between DisplayNames
			}
			else
			{
				// Add the category to the correct position in the array to keep it sorted
				const int32 CatIndex = InOutSortedSpriteInfo.IndexOfByPredicate([&InSpriteInfo](const FSpriteCategoryInfo& SpriteInfo){ return InSpriteInfo.DisplayName.CompareTo( SpriteInfo.DisplayName ) < 0; });
				if (CatIndex != INDEX_NONE)
				{
					InOutSortedSpriteInfo.Insert( InSpriteInfo, CatIndex );
				}
				else
				{
					InOutSortedSpriteInfo.Add( InSpriteInfo );
				}
			}
		}
	};

	// Iterate over all classes searching for those which derive from AActor and are neither deprecated nor abstract.
	// It would be nice to only check placeable classes here, but we cannot do that as some non-placeable classes
	// still end up in the editor (with sprites) procedurally, such as prefab instances and landscape actors.
	for ( UClass* Class : TObjectRange<UClass>() )
	{
		if ( Class->IsChildOf( AActor::StaticClass() )
		&& !( Class->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated) ) )
		{
			// Check if the class default actor has billboard components or arrow components that should be treated as
			// sprites, and if so, add their categories to the array
			const AActor* CurDefaultClassActor = Class->GetDefaultObject<AActor>();
			if ( CurDefaultClassActor )
			{
				for ( UActorComponent* Comp : CurDefaultClassActor->GetComponents() )
				{
					const UBillboardComponent* CurSpriteComponent = Cast<UBillboardComponent>( Comp );
					const UArrowComponent* CurArrowComponent = (CurSpriteComponent ? nullptr : Cast<UArrowComponent>( Comp ));
					if ( CurSpriteComponent )
					{
						Local::AddSortedSpriteInfo( OutSortedSpriteInfo, CurSpriteComponent->SpriteInfo );
					}
					else if ( CurArrowComponent && CurArrowComponent->bTreatAsASprite )
					{
						Local::AddSortedSpriteInfo( OutSortedSpriteInfo, CurArrowComponent->SpriteInfo );
					}
				}
			}
		}
	}

	// It wont find sounds, but we want it to be there
	{
		FSpriteCategoryInfo SpriteInfo;
		SpriteInfo.Category = TEXT("Sounds");
		SpriteInfo.DisplayName = NSLOCTEXT( "SpriteCategory", "Sounds", "Sounds" );
		Local::AddSortedSpriteInfo( OutSortedSpriteInfo, SpriteInfo );
	}
}


void UUnrealEdEngine::PreExit()
{
	FAssetSourceFilenameCache::Get().Shutdown();

	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule != nullptr)
	{
		SettingsModule->UnregisterSettings("Editor", "General", "Appearance");
	}


	Super::PreExit();
}

// Disable deprecated property warnings which are accessed when destructed here
PRAGMA_DISABLE_DEPRECATION_WARNINGS
UUnrealEdEngine::~UUnrealEdEngine()
{
	if (this == GUnrealEd)
	{
		GUnrealEd = NULL; 
	}

#if STALL_DETECTOR
	// Handle cases where OnEditorClose() was not called such as UE_LOG fatal errors or any call to RequestEngineExit()
	// Without this stall detector thread will continue to run until it crashes trying to access cleaned up critical section
	if (bRequiresStallDetectorShutdown && UE::FStallDetector::IsRunning())
	{
		bRequiresStallDetectorShutdown = false;
		UE::FStallDetector::Shutdown();
	}
#endif // STALL_DETECTOR
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UUnrealEdEngine::FinishDestroy()
{
	if( CookServer)
	{
		// Casting to UObject because UCookOnTheFlyServer is an incomplete type (forward declared only)
		FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll((const UObject*)CookServer.Get());
		FCoreUObjectDelegates::OnObjectModified.RemoveAll((const UObject*)CookServer.Get());
	}

	if (ExternalCookOnTheFlyServer)
	{
		delete ExternalCookOnTheFlyServer;
		ExternalCookOnTheFlyServer = nullptr;
	}

	if(PackageAutoSaver.Get())
	{
		// We've finished shutting down, so disable the auto-save restore
		PackageAutoSaver->UpdateRestoreFile(false);
		PackageAutoSaver.Reset();
	}

	if( PerformanceMonitor )
	{
		delete PerformanceMonitor;
	}

	FPackageName::OnContentPathMounted().RemoveAll(this);
	FPackageName::OnContentPathDismounted().RemoveAll(this);
	UPackage::PackageDirtyStateChangedEvent.RemoveAll(this);
	FCoreUObjectDelegates::GetPostGarbageCollect().RemoveAll(this);
	FCoreDelegates::ColorPickerChanged.RemoveAll(this);
	Super::FinishDestroy();
}


void UUnrealEdEngine::Tick(float DeltaSeconds, bool bIdleMode)
{
	Super::Tick( DeltaSeconds, bIdleMode );

	// Increment the "seconds since last autosave" counter, then try to autosave.
	if (!GSlowTaskOccurred)
	{
		// Don't increment autosave count while in game/pie/automation testing
		const bool PauseAutosave = (PlayWorld != nullptr) || GIsAutomationTesting;
		if (!PauseAutosave && PackageAutoSaver.Get())
		{
			PackageAutoSaver->UpdateAutoSaveCount(DeltaSeconds);
		}
	}
	if (!GIsSlowTask)
	{
		GSlowTaskOccurred = false;
	}

	// Display any load errors that happened while starting up the editor.
	static bool bFirstTick = true;
	if (bFirstTick)
	{
		FEditorDelegates::DisplayLoadErrors.Broadcast();
	}
	bFirstTick = false;

	if(PackageAutoSaver.Get())
	{
		PackageAutoSaver->AttemptAutoSave();
	}

	// Try and notify the user about modified packages needing checkout and write permission warnings
	AttemptModifiedPackageNotification();

	// Update lightmass
	UpdateBuildLighting();
}


void UUnrealEdEngine::OnPackageDirtyStateUpdated( UPackage* Pkg)
{
	// The passed in object should never be NULL
	check(Pkg);

	UPackage* Package = Pkg->GetOutermost();
	const FString PackageName = Package->GetName();

	if( Package->IsDirty() )
	{
		// Find out if we have already asked the user to modify this package
		const uint8* PromptState = PackageToNotifyState.Find( Package );
		const bool bAlreadyAsked = PromptState != NULL;

		// During an autosave, packages are saved in the autosave directory which switches off their dirty flags.
		// To preserve the pre-autosave state, any saved package is then remarked as dirty because it wasn't saved in the normal location where it would be picked up by source control.
		// Any callback that happens during an autosave is bogus since a package wasn't marked dirty due to a user modification.
		const bool bIsAutoSaving = PackageAutoSaver.Get() && PackageAutoSaver->IsAutoSaving();

		if( !bIsAutoSaving && 
			!UE::GetIsEditorLoadingPackage() && // Don't ask if the package was modified as a result of a load
			!GIsCookerLoadingPackage && // don't ask if the package was modified as a result of a cooker load
			!(Package->GetPackageFlags() & PKG_CompiledIn)) // don't ask if the package is a script package (changes are saved elsewhere via config files)
		{
			PackagesDirtiedThisTick.Add(Package);

			const UEditorLoadingSavingSettings* Settings = GetDefault<UEditorLoadingSavingSettings>();
			if (!bAlreadyAsked && // Don't ask if we already asked once!
				Settings->bPromptForCheckoutOnAssetModification)
			{
				PackageToNotifyState.Add(Package, NS_Updating);
			}
		}
	}
	else
	{
		// This package was saved, the user should be prompted again if they checked in the package
		PackagesDirtiedThisTick.Remove(Package);
		PackageToNotifyState.Remove( Package );
	}
}

void UUnrealEdEngine::AttemptModifiedPackageNotification()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UUnrealEdEngine::AttemptModifiedPackageNotification);

	bool bIsCooking = CookServer && CookServer->IsCookingInEditor() && CookServer->IsCookByTheBookRunning();

	if (bShowPackageNotification && !bIsCooking)
	{
		ShowPackageNotification();
	}

	bool bShowWritePermissionWarning = false;
	FMessageLog EditorLog("EditorErrors");
	if (PackagesDirtiedThisTick.Num() > 0 && !bIsCooking)
	{
		FString PackageName;
		TArray<FString> Files;
		TArray<TWeakObjectPtr<UPackage>> Packages;
		for (const TWeakObjectPtr<UPackage>& Package : PackagesDirtiedThisTick)
		{
			if (Package.IsValid())
			{
				Packages.Add(Package);
				Files.Add(SourceControlHelpers::PackageFilename(Package.Get()));

				// if we do not have write permission under the mount point for this package log an error in the message log to link to.
				PackageName = Package->GetName();
				if (!HasMountWritePermissionForPackage(PackageName))
				{
					bShowWritePermissionWarning = true;
					EditorLog.Warning(FText::Format(NSLOCTEXT("UnrealEd", "WritePermissionFailureLog", "Insufficient writing permission to save {0}"), FText::FromString(PackageName)));
				}
			}
		}

		const bool bIsCheckoutPromptEnabled = GetDefault<UEditorLoadingSavingSettings>()->bPromptForCheckoutOnAssetModification;
		if (bIsCheckoutPromptEnabled && Files.Num() > 0)
		{
			// Force source control state to be updated
			ISourceControlModule::Get().GetProvider().Execute(ISourceControlOperation::Create<FUpdateStatus>(), SourceControlHelpers::AbsoluteFilenames(Files), EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateUObject(this, &UUnrealEdEngine::OnSourceControlStateUpdated, Packages));
		}
	}
	
	if (bShowWritePermissionWarning)
	{
		// Display a toast notification when a writing permission failure occurs.
		FText WarningText = NSLOCTEXT("UnrealEd", "WritePermissionFailureNotification", "Insufficient permission to save content.");
		if (!WritePermissionWarningNotificationWeakPtr.IsValid())
		{
			FNotificationInfo WarningNotification(WarningText);
			WarningNotification.bFireAndForget = true;
			WarningNotification.Hyperlink = FSimpleDelegate::CreateLambda([]()
			{
				FMessageLog("EditorErrors").Open();
			});
			WarningNotification.HyperlinkText = NSLOCTEXT("UnrealEd", "WritePermissionFailureHyperlink", "Open Message Log");
			WarningNotification.ExpireDuration = 6.0f;
			WarningNotification.bUseThrobber = false;

			// For adding notifications.
			WritePermissionWarningNotificationWeakPtr = FSlateNotificationManager::Get().AddNotification(WarningNotification);
		}
		else
		{
			WritePermissionWarningNotificationWeakPtr.Pin()->SetText(WarningText);
			WritePermissionWarningNotificationWeakPtr.Pin()->ExpireAndFadeout();
		}
	}

	PackagesDirtiedThisTick.Empty();
}

void UUnrealEdEngine::OnSourceControlStateUpdated(const FSourceControlOperationRef& SourceControlOp, ECommandResult::Type ResultType, TArray<TWeakObjectPtr<UPackage>> Packages)
{
	if (ResultType == ECommandResult::Succeeded)
	{
		// Get the source control state of the package
		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

		const UEditorLoadingSavingSettings* Settings = GetDefault<UEditorLoadingSavingSettings>();
		for (const TWeakObjectPtr<UPackage>& PackagePtr : Packages)
		{
			if (PackagePtr.IsValid())
			{
				UPackage* Package = PackagePtr.Get();

				FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(Package, EStateCacheUsage::Use);
				if (SourceControlState.IsValid())
				{
					if (SourceControlState->CanCheckout())
					{
						// If this package is checked out or modified in another branch
						if (SourceControlState->IsCheckedOutOrModifiedInOtherBranch())
						{
							PackageToNotifyState.Add(PackagePtr, NS_PendingWarning);
							bShowPackageNotification = true;
						}
						else
						{
							if (!Settings->GetAutomaticallyCheckoutOnAssetModification())
							{
								PackageToNotifyState.Add(PackagePtr, NS_PendingPrompt);
								bShowPackageNotification = true;
							}
						}
					}
					else if (!SourceControlState->IsCurrent() || SourceControlState->IsCheckedOutOther())
					{
						PackageToNotifyState.Add(PackagePtr, NS_PendingWarning);
						bShowPackageNotification = true;
					}
				}
			}
		}
	}
}


void UUnrealEdEngine::OnPackagesCheckedOut(const FSourceControlOperationRef& SourceControlOp, ECommandResult::Type ResultType, TArray<TWeakObjectPtr<UPackage>> Packages)
{
	if (ResultType == ECommandResult::Succeeded)
	{
		FNotificationInfo Notification(NSLOCTEXT("SourceControl", "AutoCheckOutNotification", "Packages automatically checked out."));
		Notification.bFireAndForget = true;
		Notification.ExpireDuration = 4.0f;
		Notification.bUseThrobber = true;

		FSlateNotificationManager::Get().AddNotification(Notification);

		for (const TWeakObjectPtr<UPackage>& Package : Packages)
		{
			PackageToNotifyState.Add(Package, NS_DialogPrompted);
		}
	}
	else
	{
		FNotificationInfo ErrorNotification(NSLOCTEXT("SourceControl", "AutoCheckOutFailedNotification", "Unable to automatically check out packages."));
		ErrorNotification.bFireAndForget = true;
		ErrorNotification.ExpireDuration = 4.0f;
		ErrorNotification.bUseThrobber = true;

		FSlateNotificationManager::Get().AddNotification(ErrorNotification);

		for (const TWeakObjectPtr<UPackage>& Package : Packages)
		{
			PackageToNotifyState.Add(Package, NS_PendingPrompt);
		}
	}
}


void UUnrealEdEngine::OnPostGarbageCollect()
{
	// Refresh Editor browsers after GC in case objects where removed.  Note that if the user is currently
	// playing in a PIE level, we don't want to interrupt performance by refreshing the Generic Browser window.
	if( GIsEditor && !GIsPlayInEditorWorld )
	{
		FEditorDelegates::RefreshAllBrowsers.Broadcast();
	}

	// Clean up any GCed packages in the PackageToNotifyState
	for( TMap<TWeakObjectPtr<UPackage>,uint8>::TIterator It(PackageToNotifyState); It; ++It )
	{
		if( It.Key().IsValid() == false )
		{
			It.RemoveCurrent();
		}
	}

	for (FEditorViewportClient* ViewportClient : GetAllViewportClients())
	{
		if (ViewportClient && ViewportClient->Viewport)
		{
			ViewportClient->Viewport->InvalidateHitProxy();
		}
	}
}


void UUnrealEdEngine::OnColorPickerChanged()
{
	FEditorSupportDelegates::RedrawAllViewports.Broadcast();

	FEditorSupportDelegates::PreWindowsMessage.RemoveAll(this);
	FEditorSupportDelegates::PostWindowsMessage.RemoveAll(this);
}


UWorld* SavedGWorld = NULL;


void UUnrealEdEngine::OnPreWindowsMessage(FViewport* Viewport, uint32 Message)
{
	// Make sure the proper GWorld is set before handling the windows message
	if( GameViewport && !bIsSimulatingInEditor && GameViewport->Viewport == Viewport && !GIsPlayInEditorWorld )
	{
		// remember the current GWorld that will be restored in the PostWindowsMessage callback
		SavedGWorld = GWorld;
		SetPlayInEditorWorld( PlayWorld );
	}
	else
	{
		SavedGWorld = NULL;
	}
}


void UUnrealEdEngine::OnPostWindowsMessage(FViewport* Viewport, uint32 Message)
{
	if( SavedGWorld )
	{
		RestoreEditorWorld( SavedGWorld );
	}		
}

bool UUnrealEdEngine::IsAutosaving(const EPackageAutoSaveType AutoSaveType) const
{
	if (PackageAutoSaver)
	{
		return PackageAutoSaver->IsAutoSaving(AutoSaveType);
	}
	
	return false;
}

void UUnrealEdEngine::ShowActorProperties()
{
	// See if we have any unlocked property windows available.  If not, create a new one.
	if( FSlateApplication::IsInitialized() )	
	{
		IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));

		bool bHasUnlockedViews = false;
	
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>( TEXT("PropertyEditor") );
		bHasUnlockedViews = PropertyEditorModule.HasUnlockedDetailViews();
	
		// If the slate main frame is shown, summon a new property viewer in the Level editor module
		if(MainFrameModule.IsWindowInitialized())
		{
			FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>( TEXT("LevelEditor") );
			LevelEditorModule.SummonSelectionDetails();
		}

		if( !bHasUnlockedViews )
		{
			UpdateFloatingPropertyWindows();
		}
	}
}

bool UUnrealEdEngine::GetMapBuildCancelled() const
{
	return FUnrealEdMisc::Get().GetMapBuildCancelled();
}


void UUnrealEdEngine::SetMapBuildCancelled( bool InCancelled )
{
	FUnrealEdMisc::Get().SetMapBuildCancelled( InCancelled );
}

// namespace to match the original in the old loc system
#define LOCTEXT_NAMESPACE "UnrealEd"


FText FClassPickerDefaults::GetName() const
{
	FText Result = LOCTEXT("NullClass", "(null class)");

	if (UClass* ItemClass = LoadClass<UObject>(NULL, *ClassName, NULL, LOAD_None, NULL))
	{
		Result = ItemClass->GetDisplayNameText();
	}

	return Result;
}


FText FClassPickerDefaults::GetDescription() const
{
	FText Result = LOCTEXT("NullClass", "(null class)");

	if (UClass* ItemClass = LoadClass<UObject>(NULL, *ClassName, NULL, LOAD_None, NULL))
	{
		Result = ItemClass->GetToolTipText(/*bShortTooltip=*/ true);
	}

	return Result;
}

#undef LOCTEXT_NAMESPACE


UUnrealEdKeyBindings::UUnrealEdKeyBindings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UUnrealEdOptions::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		EditorKeyBindings = NewObject<UUnrealEdKeyBindings>(this, FName("EditorKeyBindingsInst"));
	}
}


UUnrealEdOptions* UUnrealEdEngine::GetUnrealEdOptions()
{
	if(EditorOptionsInst == NULL)
	{
		EditorOptionsInst = NewObject<UUnrealEdOptions>();
	}

	return EditorOptionsInst;
}

 
void UUnrealEdEngine::CloseEditor()
{
	check(GEngine);

	// if PIE is still happening, stop it before doing anything
	if (PlayWorld)
	{
		EndPlayMap();
	}

	// End any play on console/pc games still happening
	EndPlayOnLocalPc();

	// Can't use FPlatformMisc::RequestExit as it uses PostQuitMessage which is not what we want here.
	RequestEngineExit(TEXT("UUnrealEdEngine::CloseEditor()"));
}


bool UUnrealEdEngine::AllowSelectTranslucent() const
{
	return GetDefault<UEditorPerProjectUserSettings>()->bAllowSelectTranslucent;
}


bool UUnrealEdEngine::OnlyLoadEditorVisibleLevelsInPIE() const
{
	return GetDefault<ULevelEditorPlaySettings>()->bOnlyLoadVisibleLevelsInPIE;
}

bool UUnrealEdEngine::PreferToStreamLevelsInPIE() const
{
	return GetDefault<ULevelEditorPlaySettings>()->bPreferToStreamLevelsInPIE;
}

const TArray<FTemplateMapInfo>& UUnrealEdEngine::GetTemplateMapInfos() const
{
	return GetTemplateMapInfosDelegate.IsBound() ? GetTemplateMapInfosDelegate.Execute() : GetProjectDefaultMapTemplates();
}

const TArray<FTemplateMapInfo>& UUnrealEdEngine::GetProjectDefaultMapTemplates() const
{
	return TemplateMapInfoCache;
}

void UUnrealEdEngine::OnHISMTreeBuilt(UHierarchicalInstancedStaticMeshComponent* Component, bool bWasAsyncBuild)
{
	if (bWasAsyncBuild)
	{
		// Async tree builds require that hit-proxies be updated when the build has finished (as it generates new hit-proxies).
		// Sadly we don't know which viewport needs updating, so we just have to update all of them.
		for (FEditorViewportClient* ViewportClient : GetAllViewportClients())
		{
			if (ViewportClient && ViewportClient->Viewport)
			{
				ViewportClient->RequestInvalidateHitProxy(ViewportClient->Viewport);
			}
		}
	}
}

void UUnrealEdEngine::RedrawLevelEditingViewports(bool bInvalidateHitProxies)
{
	// Redraw Slate based viewports
	if( FModuleManager::Get().IsModuleLoaded("LevelEditor") )
	{
		FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>( "LevelEditor" );
		LevelEditor.BroadcastRedrawViewports( bInvalidateHitProxies );
	}

}


void UUnrealEdEngine::TakeHighResScreenShots()
{
	// Tell all viewports to take a screenshot
	if( FModuleManager::Get().IsModuleLoaded("LevelEditor") )
	{
		FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>( "LevelEditor" );
		LevelEditor.BroadcastTakeHighResScreenShots( );
	}
}


void UUnrealEdEngine::RebuildTemplateMapData()
{
	const TArray<FTemplateMapInfoOverride> TemplateMapOverrides = GetDefault<UGameMapsSettings>()->EditorTemplateMapOverrides;

	if (TemplateMapOverrides.Num() > 0)
	{
		TemplateMapInfoCache.Reset(TemplateMapOverrides.Num());

		for (const FTemplateMapInfoOverride& Override : TemplateMapOverrides)
		{
			FTemplateMapInfo NewInfo;
			NewInfo.Thumbnail = Override.Thumbnail;
			NewInfo.Map = Override.Map;
			NewInfo.DisplayName = Override.DisplayName;
			TemplateMapInfoCache.Add(MoveTemp(NewInfo));
		}
	}
	else
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		TemplateMapInfoCache = TemplateMapInfos;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

void UUnrealEdEngine::AppendTemplateMaps(const TArray<FTemplateMapInfo>& InTemplateMapInfos)
{
	bool bTemplateWasAdded = false;
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	for (const FTemplateMapInfo& TemplateMapInfo : InTemplateMapInfos)
	{
		if (!TemplateMapInfos.ContainsByPredicate(
			[&TemplateMapInfo](const FTemplateMapInfo& InTemplate)
		{
			return InTemplate.Map == TemplateMapInfo.Map
				&& InTemplate.DisplayName.EqualTo(InTemplate.DisplayName);
		}))
		{
			TemplateMapInfos.Emplace(TemplateMapInfo);
			bTemplateWasAdded = true;
		}
		else
		{
			UE_LOGF(LogUnrealEdEngine, Warning, "Attempted to register an already registered template map ('%ls'). Skipping registration.", *TemplateMapInfo.Map.ToString());
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	if (bTemplateWasAdded)
	{
		// A new template was added, so refresh
		RebuildTemplateMapData();
	}
}

void UUnrealEdEngine::SetCurrentClass( UClass* InClass )
{
	USelection* SelectionSet = GetSelectedObjects();
	SelectionSet->DeselectAll( UClass::StaticClass() );

	if(InClass != NULL)
	{
		SelectionSet->Select( InClass );
	}
}


void UUnrealEdEngine::GetPackageList( TArray<UPackage*>* InPackages, UClass* InClass )
{
	InPackages->Empty();

	for( FThreadSafeObjectIterator It; It ; ++It )
	{
		if( It->GetOuter() && It->GetOuter() != GetTransientPackage() )
		{
			UObject* TopParent = NULL;

			if( InClass == NULL || It->IsA( InClass ) )
				TopParent = It->GetOutermost();

			if( Cast<UPackage>(TopParent) )
				InPackages->AddUnique( (UPackage*)TopParent );
		}
	}
}


bool UUnrealEdEngine::CanSavePackage( UPackage* PackageToSave )
{
	return HasMountWritePermissionForPackage(PackageToSave->GetName());
}


UThumbnailManager* UUnrealEdEngine::GetThumbnailManager()
{
	return &(UThumbnailManager::Get());
}


void UUnrealEdEngine::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar << MaterialCopyPasteBuffer;
	Ar << AnimationCompressionAlgorithms;
}

void UUnrealEdEngine::DuplicateSelectedActors(UWorld* InWorld)
{
	bool bHandled = false;
	bHandled |= GLevelEditorModeTools().ProcessEditDuplicate();

	// if not specially handled by the current editing mode,
	if (!bHandled)
	{
		UEditorActorSubsystem* EditorActorSubsystem = GEditor->GetEditorSubsystem<UEditorActorSubsystem>();

		if (EditorActorSubsystem)
		{
			EditorActorSubsystem->DuplicateSelectedActors(InWorld);
		}
	}
	// DuplicateSelectedActors also calls RedrawLevelEditingViewports
	else
	{
		RedrawLevelEditingViewports();
	}
}

void UUnrealEdEngine::MakeSelectedActorsLevelCurrent()
{
	ULevel* LevelToMakeCurrent = NULL;

	// Look to the selected actors for the level to make current.
	// If actors from multiple levels are selected, do nothing.
	for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
	{
		AActor* Actor = static_cast<AActor*>( *It );
		checkSlow( Actor->IsA(AActor::StaticClass()) );

		ULevel* ActorLevel = Actor->GetLevel();

		if ( !LevelToMakeCurrent )
		{
			// First assignment.
			LevelToMakeCurrent = ActorLevel;
		}
		else if ( LevelToMakeCurrent != ActorLevel )
		{
			// Actors from multiple levels are selected -- abort.
			LevelToMakeCurrent = NULL;
			break;
		}
	}

	if (LevelToMakeCurrent)
	{
		if (!LevelToMakeCurrent->IsCurrentLevel())
		{
			// Change the current level to something different
			EditorLevelUtils::MakeLevelCurrent(LevelToMakeCurrent);
		}
		// Set the Level tab's selected level.
		TArray<class ULevel*> SelectedLevelsList(&LevelToMakeCurrent, 1);
		LevelToMakeCurrent->GetWorld()->SetSelectedLevels(SelectedLevelsList);
	}
}


int32 UUnrealEdEngine::GetSpriteCategoryIndex( const FName& InSpriteCategory )
{
	// Find the sprite category in the unlocalized to index map, if possible
	const int32* CategoryIndexPtr = SpriteIDToIndexMap.Find( InSpriteCategory );
	
	const int32 CategoryIndex = CategoryIndexPtr ? *CategoryIndexPtr : INDEX_NONE;

	return CategoryIndex;
}


void UUnrealEdEngine::ShowLightingStaticMeshInfoWindow()
{
	// first invoke the stats viewer tab
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>( TEXT("LevelEditor") );
	TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();
	LevelEditorTabManager->TryInvokeTab(FName("LevelEditorStatsViewer"));

	// then switch pages
	FStatsViewerModule& StatsViewerModule = FModuleManager::Get().LoadModuleChecked<FStatsViewerModule>(TEXT("StatsViewer"));
	StatsViewerModule.GetPage(EStatsPage::StaticMeshLightingInfo)->Show();
}


void UUnrealEdEngine::OpenSceneStatsWindow()
{
	// first invoke the stats viewer tab
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>( TEXT("LevelEditor") );
	TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();
	LevelEditorTabManager->TryInvokeTab(FName("LevelEditorStatsViewer"));

	// then switch pages
	FStatsViewerModule& StatsViewerModule = FModuleManager::Get().LoadModuleChecked<FStatsViewerModule>(TEXT("StatsViewer"));
	StatsViewerModule.GetPage(EStatsPage::PrimitiveStats)->Show();
}


void UUnrealEdEngine::OpenTextureStatsWindow()
{
	// first invoke the stats viewer tab
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>( TEXT("LevelEditor") );
	TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();
	LevelEditorTabManager->TryInvokeTab(FName("LevelEditorStatsViewer"));

	// then switch pages
	FStatsViewerModule& StatsViewerModule = FModuleManager::Get().LoadModuleChecked<FStatsViewerModule>(TEXT("StatsViewer"));
	StatsViewerModule.GetPage(EStatsPage::TextureStats)->Show();
}


void UUnrealEdEngine::GetSortedVolumeClasses( TArray< UClass* >* VolumeClasses )
{
	// Add all of the volume classes to the passed in array and then sort it
	for( UClass* Class : TObjectRange<UClass>() )
	{
		if (Class->IsChildOf(AVolume::StaticClass()) && !Class->HasAnyClassFlags(CLASS_Deprecated | CLASS_Abstract | CLASS_NotPlaceable) && Class->ClassGeneratedBy == nullptr)
		{
			VolumeClasses->AddUnique( Class );
		}
	}

	VolumeClasses->Sort();
}


void UUnrealEdOptions::GenerateCommandMap()
{
	CommandMap.Empty();
	for(int32 CmdIdx=0; CmdIdx<EditorCommands.Num(); CmdIdx++)
	{
		FEditorCommand &Cmd = EditorCommands[CmdIdx];

		CommandMap.Add(Cmd.CommandName, CmdIdx);
	}
}


FString UUnrealEdOptions::GetExecCommand(FKey Key, bool bAltDown, bool bCtrlDown, bool bShiftDown, FName EditorSet)
{
	TArray<FEditorKeyBinding> &KeyBindings = EditorKeyBindings->KeyBindings;
	FString Result;

	for(int32 BindingIdx=0; BindingIdx<KeyBindings.Num(); BindingIdx++)
	{
		FEditorKeyBinding &Binding = KeyBindings[BindingIdx];
		int32* CommandIdx = CommandMap.Find(Binding.CommandName);

		if(CommandIdx && EditorCommands.IsValidIndex(*CommandIdx))
		{
			FEditorCommand &Cmd = EditorCommands[*CommandIdx];

			if(Cmd.Parent == EditorSet)
			{
				// See if this key binding matches the key combination passed in.
				if(bAltDown == Binding.bAltDown && bCtrlDown == Binding.bCtrlDown && bShiftDown == Binding.bShiftDown && Key == Binding.Key)
				{
					int32* EditorCommandIdx = CommandMap.Find(Binding.CommandName);

					if(EditorCommandIdx && EditorCommands.IsValidIndex(*EditorCommandIdx))
					{
						FEditorCommand &EditorCommand = EditorCommands[*EditorCommandIdx];
						Result = EditorCommand.ExecCommand;
					}
					break;
				}
			}
		}
	}

	return Result;
}


/**
 * Does the update for volume actor visibility
 *
 * @param ActorsToUpdate	The list of actors to update
 * @param ViewClient		The viewport client to apply visibility changes to
 */
static void InternalUpdateVolumeActorVisibility( TArray<AActor*>& ActorsToUpdate, const FLevelEditorViewportClient& ViewClient, TArray<AActor*>& OutActorsThatChanged )
{
	for( int32 ActorIdx = 0; ActorIdx < ActorsToUpdate.Num(); ++ActorIdx )
	{
		AVolume* VolumeToUpdate = Cast<AVolume>(ActorsToUpdate[ActorIdx]);

		if ( VolumeToUpdate )
		{
			const bool bIsVisible = ViewClient.IsVolumeVisibleInViewport( *VolumeToUpdate );

			uint64 OriginalViews = VolumeToUpdate->HiddenEditorViews;
			if( bIsVisible )
			{
				// If the actor should be visible, unset the bit for the actor in this viewport
				VolumeToUpdate->HiddenEditorViews &= ~((uint64)1<<ViewClient.ViewIndex);	
			}
			else
			{
				if( VolumeToUpdate->IsSelected() )
				{
					// We are hiding the actor, make sure its not selected anymore
					GEditor->SelectActor( VolumeToUpdate, false, true  );
				}

				// If the actor should be hidden, set the bit for the actor in this viewport
				VolumeToUpdate->HiddenEditorViews |= ((uint64)1<<ViewClient.ViewIndex);	
			}

			if( OriginalViews != VolumeToUpdate->HiddenEditorViews )
			{
				// At least one actor has visibility changes
				OutActorsThatChanged.AddUnique( VolumeToUpdate );
			}
		}
	}
}


void UUnrealEdEngine::UpdateVolumeActorVisibility( UClass* InVolumeActorClass, FLevelEditorViewportClient* InViewport )
{
	TSubclassOf<AActor> VolumeClassToCheck = InVolumeActorClass ? InVolumeActorClass : AVolume::StaticClass();
	
	// Build a list of actors that need to be updated.  Only take actors of the passed in volume class.  
	UWorld* World = InViewport ? InViewport->GetWorld() : GWorld;
	TArray< AActor *> ActorsToUpdate;
	for( TActorIterator<AActor> It( World, VolumeClassToCheck ); It; ++It)
	{
		ActorsToUpdate.Add(*It);
	}

	if( ActorsToUpdate.Num() > 0 )
	{
		TArray< AActor* > ActorsThatChanged;
		if( !InViewport )
		{
			// Update the visibility state of each actor for each viewport
			for( FLevelEditorViewportClient* ViewClient : GetLevelViewportClients() )
			{
				// Only update the editor frame clients as those are the only viewports right now that show volumes.
				InternalUpdateVolumeActorVisibility( ActorsToUpdate, *ViewClient, ActorsThatChanged );
				if( ActorsThatChanged.Num() )
				{
					// If actor visibility changed in the viewport, it needs to be redrawn
					ViewClient->Invalidate();
				}
			}
		}
		else
		{
			// Only update the editor frame clients as those are the only viewports right now that show volumes.
			InternalUpdateVolumeActorVisibility( ActorsToUpdate, *InViewport, ActorsThatChanged );
			if( ActorsThatChanged.Num() )
			{	
				// If actor visibility changed in the viewport, it needs to be redrawn
				InViewport->Invalidate();
			}
		}

		// Push all changes in the actors to the scene proxy so the render thread correctly updates visibility
		for( int32 ActorIdx = 0; ActorIdx < ActorsThatChanged.Num(); ++ActorIdx )
		{
			AActor* ActorToUpdate = ActorsThatChanged[ ActorIdx ];

			// Find all registered primitive components and update the scene proxy with the actors updated visibility map
			TInlineComponentArray<UPrimitiveComponent*> PrimitiveComponents;
			ActorToUpdate->GetComponents(PrimitiveComponents);

			for( int32 ComponentIdx = 0; ComponentIdx < PrimitiveComponents.Num(); ++ComponentIdx )
			{
				UPrimitiveComponent* PrimitiveComponent = PrimitiveComponents[ComponentIdx];
				if (PrimitiveComponent->IsRegistered())
				{
					// Push visibility to the render thread
					PrimitiveComponent->PushEditorVisibilityToProxy( ActorToUpdate->HiddenEditorViews );
				}
			}
		}
	}
}


void UUnrealEdEngine::FixAnyInvertedBrushes(UWorld* World)
{
	// Get list of brushes with inverted polys
	TArray<ABrush*> Brushes;
	for (TActorIterator<ABrush> It(World); It; ++It)
	{
		ABrush* Brush = *It;
		if (Brush->GetBrushComponent() && Brush->GetBrushComponent()->HasInvertedPolys())
		{
			Brushes.Add(Brush);
		}
	}

	if (Brushes.Num() > 0)
	{
		for (ABrush* Brush : Brushes)
		{
			UE_LOGF(LogUnrealEdEngine, Warning, "Brush '%ls' appears to be inside out - fixing.", *Brush->GetName());

			// Invert the polys of the brush
			for (FPoly& Poly : Brush->GetBrushComponent()->Brush->Polys->Element)
			{
				Poly.Reverse();
				Poly.CalcNormal();
			}

			if (Brush->IsStaticBrush())
			{
				// Static brushes require a full BSP rebuild
				ABrush::SetNeedRebuild(Brush->GetLevel());
			}
			else
			{
				// Dynamic brushes can be fixed up here
				FBSPOps::csgPrepMovingBrush(Brush);
				Brush->GetBrushComponent()->BuildSimpleBrushCollision();
			}

			Brush->MarkPackageDirty();
		}
	}
}


void UUnrealEdEngine::RegisterComponentVisualizer(FName ComponentClassName, TSharedPtr<FComponentVisualizer> Visualizer)
{
	if( !ComponentClassName.IsNone() )
	{
		ComponentVisualizerMap.Add(ComponentClassName, Visualizer);		
	}
}

void UUnrealEdEngine::UnregisterComponentVisualizer(FName ComponentClassName)
{
	TSharedPtr<FComponentVisualizer> Visualizer = FindComponentVisualizer(ComponentClassName);
	VisualizersForSelection.RemoveAll([&Visualizer](const FComponentVisualizerForSelection& VisualizerForSelection) { return VisualizerForSelection.ComponentVisualizer.Visualizer == Visualizer; });

	ComponentVisualizerMap.Remove(ComponentClassName);
}

TSharedPtr<FComponentVisualizer> UUnrealEdEngine::FindComponentVisualizer(FName ComponentClassName) const
{
	TSharedPtr<FComponentVisualizer> Visualizer = NULL;

	const TSharedPtr<FComponentVisualizer>* VisualizerPtr = ComponentVisualizerMap.Find(ComponentClassName);
	if(VisualizerPtr != NULL)
	{
		Visualizer = *VisualizerPtr;
	}

	return Visualizer;
}

/** Find a component visualizer for the given component class (checking parent classes too) */
TSharedPtr<class FComponentVisualizer> UUnrealEdEngine::FindComponentVisualizer(UClass* ComponentClass) const
{
	TSharedPtr<FComponentVisualizer> Visualizer;
	while (!Visualizer.IsValid() && (ComponentClass != nullptr) && (ComponentClass != UActorComponent::StaticClass()))
	{
		Visualizer = FindComponentVisualizer(ComponentClass->GetFName());
		ComponentClass = ComponentClass->GetSuperClass();
	}

	return Visualizer;
}



void UUnrealEdEngine::DrawComponentVisualizers(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	for(FComponentVisualizerForSelection& VisualizerForSelection : VisualizersForSelection)
	{
		if (!VisualizerForSelection.IsEnabledDelegate.IsSet() || VisualizerForSelection.IsEnabledDelegate.GetValue()())
		{
			VisualizerForSelection.ComponentVisualizer.Visualizer->DrawVisualization(VisualizerForSelection.ComponentVisualizer.ComponentPropertyPath.GetComponent(), View, PDI);
		}
	}
}


void UUnrealEdEngine::DrawComponentVisualizersHUD(const FViewport* Viewport, const FSceneView* View, FCanvas* Canvas)
{
	for(FComponentVisualizerForSelection& VisualizerForSelection : VisualizersForSelection)
	{
		if (!VisualizerForSelection.IsEnabledDelegate.IsSet() || VisualizerForSelection.IsEnabledDelegate.GetValue()())
		{
			VisualizerForSelection.ComponentVisualizer.Visualizer->DrawVisualizationHUD(VisualizerForSelection.ComponentVisualizer.ComponentPropertyPath.GetComponent(), Viewport, View, Canvas);
		}
	}
}

bool UUnrealEdEngine::HasMountWritePermissionForPackage(const FString& PackageName)
{
	FName MountPoint = FPackageName::GetPackageMountPoint(PackageName, false);
	if (const bool* bWritePermission = MountPointCheckedForWritePermission.Find(MountPoint))
	{
		return *bWritePermission;
	}
	return VerifyMountPointWritePermission(MountPoint);
}

bool UUnrealEdEngine::VerifyMountPointWritePermission(FName MountPoint)
{
	// If mount is unknown assume write permissions are fine	
	if (MountPoint.IsNone())
	{
		return true;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(VerifyMountPointWritePermission);

	// Get a unique temp filename under the mount point
	FString WriteCheckPath = FPackageName::LongPackageNameToFilename(MountPoint.ToString()) + TEXT("WritePermissions.") + FGuid::NewGuid().ToString() + TEXT(".temp");

	// Check if we could successfully write to a file
	bool bHasWritePermission = FFileHelper::SaveStringToFile(TEXT("Write"), *WriteCheckPath);
	if (bHasWritePermission)
	{
		// We can successfully write to the folder containing the package.
		// Delete the temp file.
		IFileManager::Get().Delete(*WriteCheckPath);
	}

	// Add the result to the mount point checks
	MountPointCheckedForWritePermission.Add(MountPoint, bHasWritePermission);
	return bHasWritePermission;
}

void UUnrealEdEngine::OnContentPathMounted(const FString& AssetPath, const FString& /*FileSystemPath*/)
{
	// Will be verified lazily if needed. Just invalidate the entry if one already existed.
	MountPointCheckedForWritePermission.Remove(*AssetPath);
}

void UUnrealEdEngine::OnContentPathDismounted(const FString& AssetPath, const FString& /*FileSystemPath*/)
{
	MountPointCheckedForWritePermission.Remove(*AssetPath);
}

