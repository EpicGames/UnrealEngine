// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnrealEd.cpp: UnrealEd package file
=============================================================================*/

#include "UnrealEdGlobals.h"
#include "Stats/Stats.h"
#include "Async/TaskGraphInterfaces.h"
#include "EditorModeTools.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "HAL/PlatformSplash.h"
#include "Misc/CommandLine.h"
#include "Misc/App.h"
#include "EngineGlobals.h"
#include "Modules/ModuleInterface.h"
#include "EditorModeManager.h"
#include "EditorDirectories.h"
#include "Misc/OutputDeviceConsole.h"
#include "UnrealEngine.h"
#include "UnrealEdMisc.h"
#include "Selection.h"
#include "EditorModes.h"
#include "Framework/Application/SlateApplication.h"
#include "DesktopPlatformModule.h"

#include "DebugToolExec.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/IMainFrameModule.h"
#include "EngineAnalytics.h"
#include "AnalyticsEventAttribute.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "GameFramework/InputSettings.h"

#include "GameProjectGenerationModule.h"

#include "ActorFolders/TypedElements/ActorFolderTypedElementInterfaces.h"
#include "ActorFolders/TypedElements/ActorFolderTypedElementSupport.h"
#include "EditorActorFolders.h"
#include "Elements/Framework/TypedElementRegistry.h"

#include "IVREditorModule.h"

UUnrealEdEngine* GUnrealEd = nullptr;

DEFINE_LOG_CATEGORY_STATIC(LogUnrealEd, Log, All);

/**
 * Provides access to the FEditorModeTools for the level editor
 */
namespace Internal
{
	static TSharedPtr<FEditorModeTools> EditorModeToolsSingleton;
	class FLevelEditorModeTools : public FEditorModeTools
	{
	public:
		FLevelEditorModeTools()
		{
			DefaultModeIDs = { FBuiltinEditorModes::EM_Default };
			USelection::SelectNoneEvent.AddRaw(this, &FLevelEditorModeTools::OnEditorSelectNone);
			USelection::SelectionChangedEvent.AddRaw(this, &FLevelEditorModeTools::OnEditorSelectionChanged);
			USelection::SelectObjectEvent.AddRaw(this, &FLevelEditorModeTools::OnEditorSelectionChanged);
		}
		virtual ~FLevelEditorModeTools()
		{
			if (UObjectInitialized())
			{
				USelection::SelectObjectEvent.RemoveAll(this);
				USelection::SelectionChangedEvent.RemoveAll(this);
				USelection::SelectNoneEvent.RemoveAll(this);
			}
		}
	private:
		void OnEditorSelectionChanged(UObject* NewSelection)
		{
			if (NewSelection == GetSelectedActors())
			{
				// when actors are selected check if there is at least one component selected and cache that off
				// Editor modes use this primarily to determine of transform gizmos should be drawn.  
				// Performing this check each frame with lots of actors is expensive so only do this when selection changes
				bSelectionHasSceneComponent = false;
				for (FSelectionIterator It(*GetSelectedActors()); It; ++It)
				{
					AActor* Actor = Cast<AActor>(*It);
					if (Actor != nullptr && Actor->FindComponentByClass<USceneComponent>() != nullptr)
					{
						bSelectionHasSceneComponent = true;
						break;
					}
				}

			}
			else
			{
				// If selecting an actor, move the pivot location.
				AActor* Actor = Cast<AActor>(NewSelection);
				if (Actor != nullptr)
				{
					if (Actor->IsSelected())
					{
						SetPivotLocation(Actor->GetActorLocation(), false);

						// If this actor wasn't part of the original selection set during pie/sie, clear it now
						if (GEditor->ActorsThatWereSelected.Num() > 0)
						{
							AActor* EditorActor = EditorUtilities::GetEditorWorldCounterpartActor(Actor);
							if (!EditorActor || !GEditor->ActorsThatWereSelected.Contains(EditorActor))
							{
								GEditor->ActorsThatWereSelected.Empty();
							}
						}
					}
					else if (GEditor->ActorsThatWereSelected.Num() > 0)
					{
						// Clear the selection set
						GEditor->ActorsThatWereSelected.Empty();
					}
				}
			}

			for (const auto& Pair : FEditorModeRegistry::Get().GetFactoryMap())
			{
				Pair.Value->OnSelectionChanged(*this, NewSelection);
			}
		}
		void OnEditorSelectNone()
		{
			GEditor->SelectNone(false, true);
			GEditor->ActorsThatWereSelected.Empty();
		}
	};
};

// @todo: Can remove after permanent switch to new style
namespace UE::Editor::ContentBrowser
{
	namespace Private
	{
		static FAutoConsoleVariable CVarEnableContentBrowserNewStyle(
			TEXT("ContentBrowser.EnableNewStyle"),
			true,
			TEXT("Whether or not to enable the Content Browser restyle (~5.6).")
		);

		void SetEnableNewStyleFromCmdLine()
		{
			static constexpr const TCHAR* CVarName = TEXT("ContentBrowser.EnableNewStyle");

			const FString OriginalCmdLine = FCommandLine::GetOriginal();
			if (const int32 FoundIdx = OriginalCmdLine.Find(CVarName);
				FoundIdx != INDEX_NONE)
			{
				// If present, enable
				bool bEnableNewStyle = true;

				// Check for explicit disable =0 or \s0
				for (int32 CharIdx = FoundIdx + FCString::Strlen(CVarName); CharIdx < OriginalCmdLine.Len(); ++CharIdx)
				{
					if (OriginalCmdLine[CharIdx] == TEXT(' ')
						|| OriginalCmdLine[CharIdx] == TEXT('='))
					{
						continue;
					}

					if (FChar::IsDigit(OriginalCmdLine[CharIdx]))
					{
						const int32 Value = FChar::ConvertCharDigitToInt(OriginalCmdLine[CharIdx]);
						// Explicitly disable
						if (Value == 0)
						{
							bEnableNewStyle = false;
							break;
						}
					}

					break;
				}

				CVarEnableContentBrowserNewStyle->Set(bEnableNewStyle);
			}
		}

		static FDelayedAutoRegisterHelper EnableContentBrowserNewStyleCVarRegistration(
			EDelayedRegisterRunPhase::FileSystemReady,
			[]()
			{
				SetEnableNewStyleFromCmdLine();
			});
	}
}

FEditorModeTools& GLevelEditorModeTools()
{
	checkf(!IsRunningCommandlet(), TEXT("The global mode manager should not be created or accessed in a commandlet environment. Check that your mode or module is not accessing the global mode tools or that scriptable features of modes have been moved to subsystems."));
	if (!ensureMsgf(Internal::EditorModeToolsSingleton.IsValid(), TEXT("The level editor is not started up yet. If you need to access the global mode manager early in the startup phase, please use FLevelEditorModule::OnLevelEditorCreated to gate the access.")))
	{
		Internal::EditorModeToolsSingleton = MakeShared<Internal::FLevelEditorModeTools>();
	}
	return *Internal::EditorModeToolsSingleton.Get();
}

FLevelEditorViewportClient* GCurrentLevelEditingViewportClient = NULL;
/** Tracks the last level editing viewport client that received a key press. */
FLevelEditorViewportClient* GLastKeyLevelEditingViewportClient = NULL;

/**
 * Returns the path to the engine's editor resources directory (e.g. "/../../Engine/Content/Editor/")
 */
const FString GetEditorResourcesDir()
{
	return FPaths::Combine( FPlatformProcess::BaseDir(), *FPaths::EngineContentDir(), TEXT("Editor/") );
}

void CheckAndMaybeGoToVRModeInternal(const bool bIsImmersive)
{
	// Go straight to VR mode if we were asked to
	{
		if (!bIsImmersive && FParse::Param(FCommandLine::Get(), TEXT("VREditor")))
		{
			IVREditorModule& VREditorModule = IVREditorModule::Get();
			VREditorModule.EnableVREditor(true);
		}
		else if (FParse::Param(FCommandLine::Get(), TEXT("ForceVREditor")))
		{
			GEngine->DeferredCommands.Add(TEXT("VREd.ForceVRMode"));
		}
	}
}

int32 EditorInit( IEngineLoop& EngineLoop )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(EditorInit);

	// Create debug exec.	
	GDebugToolExec = new FDebugToolExec;

	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("Editor Initialized"), STAT_EditorStartup, STATGROUP_LoadTime);
	
	FScopedSlowTask SlowTask(100, NSLOCTEXT("EngineLoop", "EngineLoop_Loading", "Loading..."));
	
	SlowTask.EnterProgressFrame(50);

	int32 ErrorLevel = EngineLoop.Init();
	if( ErrorLevel != 0 )
	{
		FPlatformSplash::Hide();
		return 0;
	}

	// Let the analytics know that the editor has started
	if ( FEngineAnalytics::IsAvailable() )
	{
		TArray<FAnalyticsEventAttribute> EventAttributes;
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("GameName"), FApp::GetProjectName()));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("CommandLine"), FCommandLine::Get()));

		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.ProgramStarted"), EventAttributes);
	}

	SlowTask.EnterProgressFrame(40);

	// Set up the actor folders singleton
	FActorFolders::Get();
	
	// Register the typed elements interfaces for actor folders
	{
		using namespace UE::Editor::ActorFolders;
		if (UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance())
		{
			Registry->RegisterElementType<FActorFolderElementData, true>(NAME_ActorFolder);
			Registry->RegisterElementInterface<ITypedElementSelectionInterface>(NAME_ActorFolder, NewObject<UActorFolderElementSelectionInterface>());
			Registry->RegisterElementInterface<ITedsTypedElementBridgeInterface>(NAME_ActorFolder, NewObject<UActorFolderTypedElementBridgeInterface>());
			Registry->RegisterElementInterface<ITypedElementWorldInterface>(NAME_ActorFolder, NewObject<UActorFolderElementWorldInterface>());
		}
	}
	
	if (!Internal::EditorModeToolsSingleton.IsValid())
	{
		Internal::EditorModeToolsSingleton = MakeShared<Internal::FLevelEditorModeTools>();
	}

	// Initialize the misc editor
	FUnrealEdMisc::Get().OnInit();
	FCoreDelegates::OnExit.AddLambda([]()
	{
		// Shutdown the global static mode manager
		if (Internal::EditorModeToolsSingleton.IsValid())
		{
			GLevelEditorModeTools().SetDefaultMode(FBuiltinEditorModes::EM_Default);
			Internal::EditorModeToolsSingleton.Reset();
		}
	});
	
	SlowTask.EnterProgressFrame(10);

	// Prime our array of default directories for loading and saving content files to
	FEditorDirectories::Get().LoadLastDirectories();

	// Cache the available targets for the current project, so we can display the appropriate options in the package project menu
	FDesktopPlatformModule::Get()->GetTargetsForCurrentProject();

	// =================== CORE EDITOR INIT FINISHED ===================

	// Hide the splash screen now that everything is ready to go
	FPlatformSplash::Hide();

	// Are we in immersive mode?
	const bool bIsImmersive = FPaths::IsProjectFilePathSet() && FParse::Param( FCommandLine::Get(), TEXT( "immersive" ) );
	const bool bIsPlayInEditorRequested = FPaths::IsProjectFilePathSet() && FParse::Param(FCommandLine::Get(), TEXT("pie"));

	// Do final set up on the editor frame and show it
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(EditorInit::MainFrame);

		// Startup Slate main frame and other editor windows if possible
		{
			const bool bStartImmersive = bIsImmersive;
			const bool bStartPIE = bIsImmersive || bIsPlayInEditorRequested;

			IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
			if (!MainFrameModule.IsWindowInitialized())
			{
				if (FSlateApplication::IsInitialized())
				{
					MainFrameModule.CreateDefaultMainFrame(bStartImmersive, bStartPIE);
				}
				else
				{
					RequestEngineExit(TEXT("Slate Application terminated or not initialized for MainFrame"));
					return 1;
				}
			}
		}
	}

	// Go straight to VR mode if we were asked to
	CheckAndMaybeGoToVRModeInternal(bIsImmersive);

	// Check for automated build/submit option
	const bool bDoAutomatedMapBuild = FParse::Param( FCommandLine::Get(), TEXT("AutomatedMapBuild") );

	// Prompt to update the game project file to the current version, if necessary
	if ( FPaths::IsProjectFilePathSet() )
	{
		FGameProjectGenerationModule::Get().CheckForOutOfDateGameProjectFile();
		FGameProjectGenerationModule::Get().CheckAndWarnProjectFilenameValid();
	}

	// =================== EDITOR STARTUP FINISHED ===================
	
	// Stat tracking
	{
		const float StartupTime = (float)( FPlatformTime::Seconds() - GStartTime );

		if( FEngineAnalytics::IsAvailable() )
		{
			FEngineAnalytics::GetProvider().RecordEvent(
				TEXT( "Editor.Performance.Startup" ),
				TEXT( "Duration" ), FString::Printf( TEXT( "%.3f" ), StartupTime ) );
		}
	}

	FModuleManager::LoadModuleChecked<IModuleInterface>(TEXT("HierarchicalLODOutliner"));

	//we have to remove invalid keys * after * all the plugins and modules have been loaded.  Doing this in the editor should be caught during a config save
	if (UInputSettings* InputSettings = UInputSettings::GetInputSettings())
	{
		InputSettings->RemoveInvalidKeys();
	}

	// This will be ultimately returned from main(), so no error should be 0.
	return 0;
}

int32 EditorReinit()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(EditorReinit);

	// Are we in immersive mode?
	const bool bIsImmersive = FPaths::IsProjectFilePathSet() && FParse::Param(FCommandLine::Get(), TEXT("immersive"));
	// Do final set up on the editor frame and show it
	{
		const bool bStartImmersive = bIsImmersive;
		const bool bStartPIE = bIsImmersive;

		// Startup Slate main frame and other editor windows
		IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
		MainFrameModule.RecreateDefaultMainFrame(bStartImmersive, bStartPIE);
	}
	// Go straight to VR mode if we were asked to
	CheckAndMaybeGoToVRModeInternal(bIsImmersive);
	// No error should be 0
	return 0;
}

void EditorExit()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(EditorExit);
	LLM_SCOPE(ELLMTag::EngineMisc);

	// Save out any config settings for the editor so they don't get lost
	GEditor->SaveConfig();
	GLevelEditorModeTools().SaveConfig();


	// Save out default file directories
	FEditorDirectories::Get().SaveLastDirectories();

	// Allow the game thread to finish processing any latent tasks.
	// Some editor functions may queue tasks that need to be run before the editor is finished.
	FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);

	// Cleanup the misc editor
	FUnrealEdMisc::Get().OnExit();

	if( GLogConsole )
	{
		GLogConsole->Show( false );
	}

	delete GDebugToolExec;
	GDebugToolExec = NULL;
}

IMPLEMENT_MODULE( FDefaultModuleImpl, UnrealEd );
