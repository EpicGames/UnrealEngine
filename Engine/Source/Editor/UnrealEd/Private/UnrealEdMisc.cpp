// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnrealEdMisc.h"

#include "Engine/Blueprint.h"
#include "GameFramework/Pawn.h"
#include "Model.h"
#include "TickableEditorObject.h"
#include "Components/PrimitiveComponent.h"
#include "Containers/VersePath.h"
#include "Misc/MessageDialog.h"
#include "HAL/FileManager.h"
#include "Misc/CommandLine.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/AutomationTest.h"
#include "Misc/CoreDelegates.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "Features/IModularFeatures.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/InputBindingManager.h"
#include "Framework/Docking/TabManager.h"
#include "Toolkits/FConsoleCommandExecutor.h"
#include "TexAlignTools.h"
#include "IAssetTools.h"
#include "ISourceControlModule.h"
#include "Editor/UnrealEdEngine.h"
#include "Settings/EditorExperimentalSettings.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "GameMapsSettings.h"
#include "GeneralProjectSettings.h"
#include "Lightmass/LightmappedSurfaceCollection.h"
#include "HAL/PlatformSplash.h"
#include "Internationalization/Culture.h"
#include "Misc/ConfigCacheIni.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/UObjectIterator.h"
#include "EngineUtils.h"
#include "EditorViewportClient.h"
#include "EditorModeRegistry.h"
#include "EditorModeManager.h"
#include "FileHelpers.h"
#include "Dialogs/Dialogs.h"
#include "UnrealEdGlobals.h"
#include "EditorSupportDelegates.h"
#include "Kismet2/DebuggerCommands.h"
#include "Toolkits/AssetEditorCommonCommands.h"
#include "SoundCueGraphEditorCommands.h"
#include "CurveEditorCommands.h"
#include "EditorBuildUtils.h"
#include "Logging/TokenizedMessage.h"
#include "Logging/MessageLog.h"
#include "MessageLogInitializationOptions.h"
#include "MessageLogModule.h"
#include "Kismet2/KismetDebugUtilities.h"
#include "FbxLibs.h"
#include "Kismet2/CompilerResultsLog.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/AssetDataToken.h"
#include "EngineAnalytics.h"
#include "AnalyticsEventAttribute.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "ISettingsEditorModule.h"
#include "EngineGlobals.h"
#include "LevelEditor.h"
#include "Misc/UObjectToken.h"
#include "BusyCursor.h"
#include "ComponentAssetBroker.h"
#include "PackageTools.h"
#include "GameProjectGenerationModule.h"
#include "MaterialEditorActions.h"
#include "Misc/EngineBuildSettings.h"
#include "ShaderCompiler.h"
#include "NavigationBuildingNotification.h"
#include "Misc/HotReloadInterface.h"
#include "PerformanceMonitor.h"
#include "Engine/WorldComposition.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/WorldPartitionActorDescUtils.h"
#include "Interfaces/IProjectManager.h"
#include "FeaturePackContentSource.h"
#include "ProjectDescriptor.h"
#include "TemplateProjectDefs.h"
#include "GameProjectUtils.h"
#include "Async/AsyncResult.h"
#include "Application/IPortalApplicationWindow.h"
#include "IPortalServiceLocator.h"
#include "ILauncherPlatform.h"
#include "LauncherPlatformModule.h"
#include "UserActivityTracking.h"
#include "Widgets/Docking/SDockTab.h"
#include "IVREditorModule.h"
#include "ILauncherPlatform.h"
#include "LauncherPlatformModule.h"
#include "ILauncherServicesModule.h"
#include "HAL/PlatformTime.h"
#include "DeveloperToolSettingsDelegates.h"
#include "Cooker/CookConfigAccessTracker.h"
#include "Cooker/PackageBuildDependencyTracker.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "EditorAxisDisplayInfo.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "Factories/FbxAnimSequenceImportData.h"
#include "Factories/FbxAssetImportData.h"
#include "Factories/FbxSkeletalMeshImportData.h"
#include "Factories/FbxStaticMeshImportData.h"
#include "OutputLogModule.h"
#include "AI/NavigationSystemBase.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Async/TaskGraphInterfaces.h"
#include "RenderTimer.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#define USE_UNIT_TESTS 0

#define LOCTEXT_NAMESPACE "UnrealEd"

DEFINE_LOG_CATEGORY_STATIC(LogUnrealEdMisc, Log, All);

namespace
{
	static const FName LevelEditorName("LevelEditor");
	static const FName AssetRegistryName("AssetRegistry");
}

/**
 * Manages the stats needed by the analytics heartbeat
 * This is very similar to FStatUnitData, however it's not tied to a single viewport, 
 * nor does it rely on the stats being active to be updated
 */
class FPerformanceAnalyticsStats
{
public:
	FPerformanceAnalyticsStats()
		: AverageFrameTime(SampleSize)
		, AverageGameThreadTime(SampleSize)
		, AverageRenderThreadTime(SampleSize)
		, AverageGPUFrameTime(SampleSize)
	{
	}

	/** Get the average number of milliseconds in total over the frames that have been sampled */
	float GetAverageFrameTime() const
	{
		return AverageFrameTime.GetAverage();
	}

	/** Get the average number of milliseconds the gamethread was used over the frames that have been sampled */
	float GetAverageGameThreadTime() const
	{
		return AverageGameThreadTime.GetAverage();
	}

	/** Get the average number of milliseconds the renderthread was used over the frames that have been sampled */
	float GetAverageRenderThreadTime() const
	{
		return AverageRenderThreadTime.GetAverage();
	}

	/** Get the average number of milliseconds the GPU was busy over the frames that have been sampled */
	float GetAverageGPUFrameTime() const
	{
		return AverageGPUFrameTime.GetAverage();
	}

	/** Have we taken enough samples to get a reliable average? */
	bool IsReliable() const
	{
		return AverageFrameTime.IsReliable();
	}

	/** Update the samples based on what happened last frame */
	void Update()
	{
		const double CurrentTime = FApp::GetCurrentTime();
		const double DeltaTime = CurrentTime - FApp::GetLastTime();

		// Number of milliseconds in total last frame
		const double RawFrameTime = DeltaTime * 1000.0;
		AverageFrameTime.Tick(CurrentTime, static_cast<float>(RawFrameTime));

		// Number of milliseconds the gamethread was used last frame
		const double RawGameThreadTime = FPlatformTime::ToMilliseconds(GGameThreadTime);
		AverageGameThreadTime.Tick(CurrentTime, static_cast<float>(RawGameThreadTime));

		// Number of milliseconds the renderthread was used last frame
		const double RawRenderThreadTime = FPlatformTime::ToMilliseconds(GRenderThreadTime);
		AverageRenderThreadTime.Tick(CurrentTime, static_cast<float>(RawRenderThreadTime));

		// Number of milliseconds the GPU was busy last frame
		const uint32 GPUCycles = RHIGetGPUFrameCycles();
		const double RawGPUFrameTime = FPlatformTime::ToMilliseconds(GPUCycles);
		AverageGPUFrameTime.Tick(CurrentTime, static_cast<float>(RawGPUFrameTime));
	}

private:
	/** Samples for the total frame time */
	FMovingAverage AverageFrameTime;

	/** Samples for the gamethread time */
	FMovingAverage AverageGameThreadTime;

	/** Samples for the renderthread time */
	FMovingAverage AverageRenderThreadTime;

	/** Samples for the GPU busy time */
	FMovingAverage AverageGPUFrameTime;

	/** Number of samples to average over */
	static const int32 SampleSize = 10;
};

namespace PerformanceSurveyDefs
{
	const static int32 NumFrameRateSamples = 10;
	const static FTimespan FrameRateSampleInterval(0, 0, 1);	// 1 second intervals
}

namespace UnrealEdMiscDefs
{
	const static int32 HeartbeatIntervalSeconds = 60;
}

FUnrealEdMisc::FUnrealEdMisc() :
	bCancelBuild( false ),
	bInitialized( false ),
	bDeletePreferences( false ),
	bIsAssetAnalyticsPending( false ),
	PerformanceAnalyticsStats(new FPerformanceAnalyticsStats()),
	NavigationBuildingNotificationHandler(NULL)
{
	//This is an early entry-point into the UnrealEd module to perform some editor-specific configuration
#if UE_WITH_PACKAGE_ACCESS_TRACKING || UE_WITH_CONFIG_TRACKING
	const bool bBuildDependencyTrackingNeeded = GIsEditor &&
		(IsRunningCookCommandlet() || !GetDefault<UEditorExperimentalSettings>()->bDisableCookInEditor)
		&& UE::Cook::IsDependencyTrackingEnabled();
	if (!bBuildDependencyTrackingNeeded)
	{
#if UE_WITH_PACKAGE_ACCESS_TRACKING
		FPackageBuildDependencyTracker::Get().Disable();
#endif
#if UE_WITH_CONFIG_TRACKING
		UE::ConfigAccessTracking::FCookConfigAccessTracker::Get().Disable();
#endif
	}
#endif

	// Register the EditorAxisDisplayInfo modular feature
	if (!IModularFeatures::Get().IsModularFeatureAvailable(IAxisDisplayInfo::GetModularFeatureName()))
	{
		EditorAxisDisplayInfo = MakeUnique<FEditorAxisDisplayInfo>();
		IModularFeatures::Get().RegisterModularFeature(IAxisDisplayInfo::GetModularFeatureName(), EditorAxisDisplayInfo.Get());

		const bool bUsingLUFCoordinateSysem = AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward;

		if (bUsingLUFCoordinateSysem)
		{
			// Adjust the default SceneThumbnailInfo to look down the forward axis
			USceneThumbnailInfo::StaticClass()->GetDefaultObject<USceneThumbnailInfo>()->OrbitYaw += 90.f;
			
			// Adjust the default FbxAssetImportData's properties related to character orientation to look down the forward axis
			auto AdjustDefaultProperties = [](UFbxAssetImportData* FbxAssetImportData)
				{
					if (FbxAssetImportData)
					{
						FbxAssetImportData->bUsingLUFCoordinateSysem = true;
						FbxAssetImportData->bConvertScene = true;
						FbxAssetImportData->bForceFrontXAxis = true;
					}
				};

			AdjustDefaultProperties(UFbxStaticMeshImportData::StaticClass()->GetDefaultObject<UFbxStaticMeshImportData>());
			AdjustDefaultProperties(UFbxSkeletalMeshImportData::StaticClass()->GetDefaultObject<UFbxSkeletalMeshImportData>());
			AdjustDefaultProperties(UFbxAnimSequenceImportData::StaticClass()->GetDefaultObject<UFbxAnimSequenceImportData>());
		}
	}
}

FUnrealEdMisc::~FUnrealEdMisc()
{
}

FUnrealEdMisc& FUnrealEdMisc::Get()
{
	static FUnrealEdMisc UnrealEdMisc;
	return UnrealEdMisc;
}

void FUnrealEdMisc::OnInit()
{
	if ( bInitialized )
	{
		return;
	}
	bInitialized = true;

	FScopedSlowTask SlowTask(100);
	SlowTask.EnterProgressFrame(10);

	// Register the command executor
	CmdExec = MakeUnique<FConsoleCommandExecutor>();
	IModularFeatures::Get().RegisterModularFeature(IConsoleCommandExecutor::ModularFeatureName(), CmdExec.Get());

	// Register all callback notifications
	FEditorDelegates::SelectedProps.AddRaw(this, &FUnrealEdMisc::CB_SelectedProps);
	FEditorDelegates::DisplayLoadErrors.AddRaw(this, &FUnrealEdMisc::CB_DisplayLoadErrors);
	FEditorDelegates::MapChange.AddRaw(this, &FUnrealEdMisc::CB_MapChange);
	FEditorDelegates::RefreshEditor.AddRaw(this, &FUnrealEdMisc::CB_RefreshEditor);
	FEditorDelegates::PreSaveWorldWithContext.AddRaw(this, &FUnrealEdMisc::PreSaveWorld);
	FEditorSupportDelegates::RedrawAllViewports.AddRaw(this, &FUnrealEdMisc::CB_RedrawAllViewports);
	GEngine->OnLevelActorAdded().AddRaw( this, &FUnrealEdMisc::CB_LevelActorsAdded );

	FCoreUObjectDelegates::OnObjectPreSave.AddRaw(this, &FUnrealEdMisc::OnObjectSaved);

#if USE_UNIT_TESTS
	FAutomationTestFramework::Get().PreTestingEvent.AddRaw(this, &FUnrealEdMisc::CB_PreAutomationTesting);
	FAutomationTestFramework::Get().PostTestingEvent.AddRaw(this, &FUnrealEdMisc::CB_PostAutomationTesting);
#endif // USE_UNIT_TESTS

	/** Delegate that gets called when a script exception occurs */
	FBlueprintCoreDelegates::OnScriptException.AddStatic(&FKismetDebugUtilities::OnScriptException);
	FBlueprintContextTracker::OnExitScriptContext.AddStatic(&FKismetDebugUtilities::EndOfScriptExecution);
	
	FEditorDelegates::ChangeEditorMode.AddRaw(this, &FUnrealEdMisc::OnEditorChangeMode);
	FCoreDelegates::PreModal.AddRaw(this, &FUnrealEdMisc::OnEditorPreModal);
	FCoreDelegates::PostModal.AddRaw(this, &FUnrealEdMisc::OnEditorPostModal);
	FCoreDelegates::OnEnsureFailed.AddRaw(this, &FUnrealEdMisc::OnEnsureFailed);

	// Register the play world commands
	FPlayWorldCommands::Register();
	FPlayWorldCommands::BindGlobalPlayWorldCommands();

	// Register common asset editor commands
	FAssetEditorCommonCommands::Register();

	// Register Material Editor commands
	FMaterialEditorCommands::Register();

	// Register navigation commands for all viewports
	FViewportNavigationCommands::Register();

	// Register curve editor commands.
	FCurveEditorCommands::Register();

	// Have the User Activity Tracker reject non-editor activities for this run
	FUserActivityTracking::SetContextFilter(EUserActivityContext::Editor);
	OnActiveTabChangedDelegateHandle = FGlobalTabmanager::Get()->OnActiveTabChanged_Subscribe(FOnActiveTabChanged::FDelegate::CreateRaw(this, &FUnrealEdMisc::OnActiveTabChanged));
	OnTabForegroundedDelegateHandle = FGlobalTabmanager::Get()->OnTabForegrounded_Subscribe(FOnActiveTabChanged::FDelegate::CreateRaw(this, &FUnrealEdMisc::OnTabForegrounded));
	FUserActivityTracking::SetActivity(FUserActivity(TEXT("EditorInit"), EUserActivityContext::Editor));

	// Are we in immersive mode?
	const TCHAR* ParsedCmdLine = FCommandLine::Get();
	const bool bIsImmersive = FParse::Param( ParsedCmdLine, TEXT( "immersive" ) );

	SlowTask.EnterProgressFrame(10);

	ISourceControlModule::Get().GetProvider().Init();

	// Init the editor tools.
	GTexAlignTools.Init();

	EKeys::SetConsoleForGamepadLabels(GetDefault<UEditorExperimentalSettings>()->ConsoleForGamepadLabels);

	// =================== CORE EDITOR INIT FINISHED ===================

	// Offer to restore the auto-save packages before the startup map gets loaded (in case we want to restore the startup map)
	const bool bHasPackagesToRestore = GUnrealEd->GetPackageAutoSaver().HasPackagesToRestore();
	if(bHasPackagesToRestore)
	{
		// Hide the splash screen while we show the restore UI
		FPlatformSplash::Hide();
		GUnrealEd->GetPackageAutoSaver().OfferToRestorePackages();
		FPlatformSplash::Show();
	}

	const double InitialEditorStartupTime = (FPlatformTime::Seconds() - GStartTime);
	UE_LOGF(LogUnrealEdMisc, Log, "Loading editor; pre map load, took %.3f", InitialEditorStartupTime);

	FEditorDelegates::OnEditorBoot.Broadcast(InitialEditorStartupTime);
	
	// Check for automated build/submit option
	const bool bDoAutomatedMapBuild = FParse::Param( ParsedCmdLine, TEXT("AutomatedMapBuild") );

	// Load startup map (conditionally)
	SlowTask.EnterProgressFrame(60);
	{
		bool bMapLoaded = false;

		// Insert any feature packs if required. We need to do this before we try and load a map since any pack may contain a map
		FFeaturePackContentSource::ImportPendingPacks();

		FString ParsedMapName;
		if ( FParse::Token(ParsedCmdLine, ParsedMapName, false) && 
			 // If it's not a parameter
			 ParsedMapName.StartsWith( TEXT("-") ) == false )
		{
			FString InitialMapLongPackageName = FindMapFileFromPartialName(ParsedMapName);
			
			// If the specified package exists
			if (!InitialMapLongPackageName.IsEmpty())
			{
				// Never show loading progress when loading a map at startup.  Loading status will instead
				// be reflected in the splash screen status
				const bool bShowProgress = false;
				const bool bLoadAsTemplate = false;

				// Load the map
				FEditorFileUtils::LoadMap(InitialMapLongPackageName, bLoadAsTemplate, bShowProgress);
				bMapLoaded = true;
			}
		}

		if( !bDoAutomatedMapBuild )
		{
			if (!bMapLoaded && GEditor)
			{
				FCanLoadMap CanLoadDefaultStartupMap;
				FEditorDelegates::OnEditorLoadDefaultStartupMap.Broadcast(CanLoadDefaultStartupMap);

				if (CanLoadDefaultStartupMap.Get())
				{
					if (GetDefault<UEditorLoadingSavingSettings>()->LoadLevelAtStartup != ELoadLevelAtStartup::None)
					{
						FEditorFileUtils::LoadDefaultMapAtStartup();
						BeginPerformanceSurvey();
					}
				}
			}
		}
	}


	// Process global shader results before we try to render anything
	// CreateDefaultMainFrame below will access global shaders
	if (GShaderCompilingManager)
	{
		GShaderCompilingManager->ProcessAsyncResults(false, true);
	}


	// =================== MAP LOADING FINISHED ===================


	// Don't show map check if we're starting up in immersive mode
	if( !bIsImmersive )
	{
		FMessageLog("MapCheck").Open( EMessageSeverity::Warning );
	}

	if ( bDoAutomatedMapBuild )
	{
		// If the user is doing an automated build, configure the settings for the build appropriately
		FEditorBuildUtils::FEditorAutomatedBuildSettings AutomatedBuildSettings;

		// Assume the user doesn't want to add files not in source control, they can specify that they
		// want to via commandline option
		AutomatedBuildSettings.bAutoAddNewFiles = false;
		AutomatedBuildSettings.bCheckInPackages = false;

		// Shut down the editor upon completion of the automated build
		AutomatedBuildSettings.bShutdownEditorOnCompletion = true;

		// Assume that save, SCC, and new map errors all result in failure and don't submit anything if any of those occur. If the user
		// wants, they can explicitly ignore each warning type via commandline option
		AutomatedBuildSettings.BuildErrorBehavior = FEditorBuildUtils::ABB_ProceedOnError;
		AutomatedBuildSettings.FailedToSaveBehavior = FEditorBuildUtils::ABB_FailOnError;
		AutomatedBuildSettings.NewMapBehavior = FEditorBuildUtils::ABB_FailOnError;
		AutomatedBuildSettings.UnableToCheckoutFilesBehavior = FEditorBuildUtils::ABB_FailOnError;

		// Attempt to parse the changelist description from the commandline
		FString ParsedString;
		if ( FParse::Value( ParsedCmdLine, TEXT("CLDesc="), ParsedString ) )
		{
			AutomatedBuildSettings.ChangeDescription = ParsedString;
		}

		// See if the user has specified any additional commandline options and set the build setting appropriately if so
		bool ParsedBool;
		if ( FParse::Bool( ParsedCmdLine, TEXT("IgnoreBuildErrors="), ParsedBool ) )
		{
			AutomatedBuildSettings.BuildErrorBehavior = ParsedBool ? FEditorBuildUtils::ABB_ProceedOnError : FEditorBuildUtils::ABB_FailOnError;
		}
		if ( FParse::Bool( ParsedCmdLine, TEXT("UseSCC="), ParsedBool ) )
		{
			AutomatedBuildSettings.bUseSCC = ParsedBool;
		}
		if ( FParse::Bool( ParsedCmdLine, TEXT("IgnoreSCCErrors="), ParsedBool ) )
		{
			AutomatedBuildSettings.UnableToCheckoutFilesBehavior = ParsedBool ? FEditorBuildUtils::ABB_ProceedOnError : FEditorBuildUtils::ABB_FailOnError;
		}
		if ( FParse::Bool( ParsedCmdLine, TEXT("IgnoreMapSaveErrors="), ParsedBool ) )
		{
			AutomatedBuildSettings.FailedToSaveBehavior = ParsedBool ? FEditorBuildUtils::ABB_ProceedOnError : FEditorBuildUtils::ABB_FailOnError;
		}
		if ( FParse::Bool( ParsedCmdLine, TEXT("AddFilesNotInDepot="), ParsedBool ) )
		{
			AutomatedBuildSettings.bAutoAddNewFiles = ParsedBool;
		}	

		// Kick off the automated build
		FText ErrorText;
		FEditorBuildUtils::EditorAutomatedBuildAndSubmit( AutomatedBuildSettings, ErrorText );
	}

	SlowTask.EnterProgressFrame(10);

	//Fbx dll is currently compile with a different windows platform sdk so we should use this memory bypass later when unreal ed will be using windows 10 platform sdk
	//LoadFBxLibraries();

	// Register message log UIs
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	{
		FMessageLogInitializationOptions InitOptions;
		InitOptions.bShowPages = true;
		MessageLogModule.RegisterLogListing("EditorErrors", LOCTEXT("EditorErrors", "Editor Errors"), InitOptions);
	}

	{
		FMessageLogInitializationOptions InitOptions;
		InitOptions.bDiscardDuplicates = true;
		MessageLogModule.RegisterLogListing("LoadErrors", LOCTEXT("LoadErrors", "Load Errors"), InitOptions);
	}

	{
		FMessageLogInitializationOptions InitOptions;
		InitOptions.bShowPages = true;
		MessageLogModule.RegisterLogListing("LightingResults", LOCTEXT("LightingResults", "Lighting Results"), InitOptions);
	}

	{
		FMessageLogInitializationOptions InitOptions;
		InitOptions.bShowPages = true;
		InitOptions.bShowFilters = true;
		MessageLogModule.RegisterLogListing("PackagingResults", LOCTEXT("PackagingResults", "Packaging Results"), InitOptions);
	}

	{
		FMessageLogInitializationOptions InitOptions;
		InitOptions.bShowFilters = true;
		MessageLogModule.RegisterLogListing("MapCheck", LOCTEXT("MapCheck", "Map Check"), InitOptions);
	}

	{
		FMessageLogInitializationOptions InitOptions;
		InitOptions.bShowFilters = true;
		MessageLogModule.RegisterLogListing("AssetCheck", LOCTEXT("AssetCheckLog", "Asset Check"), InitOptions);
	}

	{
		FMessageLogInitializationOptions InitOptions;
		InitOptions.bShowFilters = true;
		MessageLogModule.RegisterLogListing("SlateStyleLog", LOCTEXT("SlateStyleLog", "Slate Style Log"), InitOptions );
	}

	{
		FMessageLogInitializationOptions InitOptions;
		InitOptions.bShowFilters = true;
		MessageLogModule.RegisterLogListing("HLODResults", LOCTEXT("HLODResults", "HLOD Results"), InitOptions);
	}

	FCompilerResultsLog::Register();
	{
		FMessageLogInitializationOptions InitOptions;
		InitOptions.bShowPages = true;
		InitOptions.bShowFilters = true;
		MessageLogModule.RegisterLogListing("PIE", LOCTEXT("PlayInEditor", "Play In Editor"), InitOptions);
	}

	{
		FMessageLogInitializationOptions InitOptions;
		InitOptions.bShowFilters = true;
		MessageLogModule.RegisterLogListing("EnsureFailed", LOCTEXT("EnsureFailed", "Ensure Failed"), InitOptions);
	}

	// install message log delegates
	FMessageLog::OnMessageSelectionChanged().BindRaw(this, &FUnrealEdMisc::OnMessageSelectionChanged);
	FMessageLog::OnMultiSelectActorBegin().BindRaw(this, &FUnrealEdMisc::OnMultiSelectActorFromMessageTokenBegin);
	FMessageLog::OnMultiSelectActorEnd().BindRaw(this, &FUnrealEdMisc::OnMultiSelectActorFromMessageTokenEnd);
	FUObjectToken::DefaultOnMessageTokenActivated().BindRaw(this, &FUnrealEdMisc::OnMessageTokenActivated);
	FUObjectToken::DefaultOnGetObjectDisplayName().BindRaw(this, &FUnrealEdMisc::OnGetDisplayName);
	FAssetNameToken::OnGotoAsset().BindRaw(this, &FUnrealEdMisc::OnGotoAsset);
	FActorToken::DefaultOnMessageTokenActivated().BindRaw(this, &FUnrealEdMisc::OnActorTokenActivated);
	FAssetDataToken::DefaultOnMessageTokenActivated().BindRaw(this, &FUnrealEdMisc::OnAssetDataTokenActivated);
	FAssetDataToken::DefaultOnGetAssetDisplayName().BindRaw(this, &FUnrealEdMisc::OnGetAssetDataDisplayName);

	// Register to receive notification of new key bindings
	OnUserDefinedChordChangedDelegateHandle = FInputBindingManager::Get().RegisterUserDefinedChordChanged(FOnUserDefinedChordChanged::FDelegate::CreateRaw( this, &FUnrealEdMisc::OnUserDefinedChordChanged ));

	SlowTask.EnterProgressFrame(10);

	// Send Project Analytics
	InitEngineAnalytics();
	
	// Setup a timer for a heartbeat event to track if users are actually using the editor or it is idle.
	FTimerDelegate Delegate;
	Delegate.BindRaw( this, &FUnrealEdMisc::EditorAnalyticsHeartbeat );

	GEditor->GetTimerManager()->SetTimer( EditorAnalyticsHeartbeatTimerHandle, Delegate, (float)UnrealEdMiscDefs::HeartbeatIntervalSeconds, true );

	// Give the settings editor a way to restart the editor when it needs to
	ISettingsEditorModule& SettingsEditorModule = FModuleManager::GetModuleChecked<ISettingsEditorModule>("SettingsEditor");
	SettingsEditorModule.SetRestartApplicationCallback(FSimpleDelegate::CreateLambda([this]() { RestartEditor(/*bWarn =*/false); }));

	// add handler to notify about navmesh building process
	NavigationBuildingNotificationHandler = MakeShareable(new FNavigationBuildingNotificationImpl());

	// Handles "Enable World Composition" option in WorldSettings
	UWorldComposition::EnableWorldCompositionEvent.BindRaw(this, &FUnrealEdMisc::EnableWorldComposition);

	const double TotalEditorStartupTime = (FPlatformTime::Seconds() - GStartTime);
	UE_LOGF(LogUnrealEdMisc, Log, "Total Editor Startup Time, took %.3f", TotalEditorStartupTime);

	TRACE_BOOKMARK(TEXT("Editor Startup"));

	FEditorDelegates::OnEditorInitialized.Broadcast(TotalEditorStartupTime);
	
	GShaderCompilingManager->PrintStats();
}

FString FUnrealEdMisc::FindMapFileFromPartialName(const FString& PartialMapName)
{
	TArray<FString> CommandLineMapCache;
	GConfig->GetArray(TEXT("EditorMapCache"), TEXT("CommandLineMapCache"), CommandLineMapCache, GEditorPerProjectIni);
	bool CommandLineMapCacheDirty = false;

	FString FoundMapFile;

	for (int32 Index = 0; Index < CommandLineMapCache.Num(); Index++)
	{
		bool bRemoveEntry = false;
		FString CurrentCacheEntry = CommandLineMapCache[Index];

		TArray<FString> Tokens;
		CurrentCacheEntry.ParseIntoArray(Tokens, TEXT(","));
		if (Tokens.Num() == 2)
		{
			FString ShortMapName = Tokens[0];
			FString RelativeFileName = Tokens[1];
		
			if (ShortMapName == PartialMapName)
			{
				// Verify both that the file exists and that it's actually
				// mounted. This ensures that we don't use a stale cache
				// entry to an unmounted file in case a previously opened
				// map was copied to a different location on disk and is
				// now being mounted from there, but the original file still
				// exists.
				FString PackageName;
				if (FPaths::FileExists(RelativeFileName) &&
					FPackageName::TryConvertFilenameToLongPackageName(RelativeFileName, PackageName))
				{
					FoundMapFile = RelativeFileName;
					break;
				}
				else
				{
					bRemoveEntry = true;
				}
			}
		}
		else
		{
			bRemoveEntry = true;
		}

		if (bRemoveEntry)
		{
			CommandLineMapCacheDirty = true;
			CommandLineMapCache.RemoveAt(Index);
			Index--;
		}
	}

	if (FoundMapFile.IsEmpty())
	{
		// If the specified package exists
		if (FPackageName::SearchForPackageOnDisk(PartialMapName, NULL, &FoundMapFile) &&
			// and it's a valid map file
			FPaths::GetExtension(FoundMapFile, /*bIncludeDot=*/true) == FPackageName::GetMapPackageExtension())
		{
			CommandLineMapCache.Add(PartialMapName + TEXT(",") + FoundMapFile);
			CommandLineMapCacheDirty = true;
		}
	}

	if (CommandLineMapCacheDirty)
	{
		GConfig->SetArray(TEXT("EditorMapCache"), TEXT("CommandLineMapCache"), CommandLineMapCache, GEditorPerProjectIni);
		GConfig->Flush(false, GEditorPerProjectIni);
	}

	return FoundMapFile;
}

void FUnrealEdMisc::InitEngineAnalytics()
{
	if ( FEngineAnalytics::IsAvailable() )
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FUnrealEdMisc::InitEngineAnalytics);

		IAnalyticsProvider& EngineAnalytics = FEngineAnalytics::GetProvider();

		// Send analytics about sample projects
		if( FPaths::IsProjectFilePathSet() )
		{
			const FString& LoadedProjectFilePath = FPaths::GetProjectFilePath();
			FProjectStatus ProjectStatus;

			if (IProjectManager::Get().QueryStatusForProject(LoadedProjectFilePath, ProjectStatus))
			{
				if ( ProjectStatus.bSignedSampleProject )
				{
					EngineAnalytics.RecordEvent(TEXT( "Rocket.Usage.SampleProjectLoaded" ), TEXT("FileName"), FPaths::GetCleanFilename(LoadedProjectFilePath));
				}
			}

			// Gather Project Code/Module Stats
			TArray< FAnalyticsEventAttribute > ProjectAttributes;
			ProjectAttributes.Add( FAnalyticsEventAttribute( FString( "Name" ), *GetDefault<UGeneralProjectSettings>()->ProjectName ));
			ProjectAttributes.Add( FAnalyticsEventAttribute( FString( "Id" ), *GetDefault<UGeneralProjectSettings>()->ProjectID.ToString() ));

			FGameProjectGenerationModule& GameProjectModule = FModuleManager::LoadModuleChecked<FGameProjectGenerationModule>(TEXT("GameProjectGeneration"));
			
			bool bShouldIncludeSourceFileCountAndSize = true;
			GConfig->GetBool(TEXT("EngineAnalytics"), TEXT("IncludeSourceFileCountAndSize"), bShouldIncludeSourceFileCountAndSize, GEditorIni);
			if (bShouldIncludeSourceFileCountAndSize)
			{
				int32 SourceFileCount = 0;
				int64 SourceFileDirectorySize = 0;
				GameProjectModule.Get().GetProjectSourceDirectoryInfo(SourceFileCount, SourceFileDirectorySize);

				ProjectAttributes.Add( FAnalyticsEventAttribute( FString( "SourceFileCount" ), SourceFileCount ));
				ProjectAttributes.Add(FAnalyticsEventAttribute(FString("SourceFileDirectorySize"), SourceFileDirectorySize));
			}
			ProjectAttributes.Add( FAnalyticsEventAttribute( FString( "ModuleCount" ), FModuleManager::Get().GetModuleCount() ));

			// UObject class count
			int32 UObjectClasses = 0;
			int32 UBlueprintClasses = 0;
			for( TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt )
			{
				if( !ClassIt->ClassGeneratedBy )
				{
					UObjectClasses++;
				}
				else
				{
					UBlueprintClasses++;
				}
			}
			ProjectAttributes.Add( FAnalyticsEventAttribute( FString( "ObjectClasses" ), UObjectClasses ));
			ProjectAttributes.Add( FAnalyticsEventAttribute( FString( "BlueprintClasses" ), UBlueprintClasses ));
			ProjectAttributes.Emplace( TEXT("Enterprise"), IProjectManager::Get().IsEnterpriseProject() );

			// Send project analytics
			EngineAnalytics.RecordEvent( FString( "Editor.Usage.Project" ), ProjectAttributes );
			// Trigger pending asset survey
			bIsAssetAnalyticsPending = true;
		}

		// Record known modules' compilation methods
		IHotReloadInterface* HotReload = IHotReloadInterface::GetPtr();
		if(HotReload != nullptr)
		{
			TArray<FModuleStatus> Modules;
			FModuleManager::Get().QueryModules(Modules);
			for (auto& Module : Modules)
			{
				// Record only game modules as these are the only ones that should be hot-reloaded
				if (Module.bIsGameModule)
				{
					TArray< FAnalyticsEventAttribute > ModuleAttributes;
					ModuleAttributes.Add(FAnalyticsEventAttribute(FString("ModuleName"), Module.Name));
					ModuleAttributes.Add(FAnalyticsEventAttribute(FString("CompilationMethod"), HotReload->GetModuleCompileMethod(*Module.Name)));
					EngineAnalytics.RecordEvent(FString("Editor.Usage.Modules"), ModuleAttributes);
				}
			}
		}
	}
}

/*
* @EventName Editor.Usage.Heartbeat
*
* @Trigger Every minute of non-idle time in the editor
*
* @Type Dynamic
*
* @EventParam Idle (bool) Whether the user is idle
* @EventParam AverageFrameTime (float) Average frame time
* @EventParam AverageGameThreadTime (float) Average game thread time
* @EventParam AverageRenderThreadTime (float) Average render thread time
* @EventParam AverageGPUFrameTime (float) Average GPU frame time
* @EventParam IsVanilla (bool) Whether the editor is vanilla launcher install with no marketplace plugins
* @EventParam IntervalSec (int32) The time since the last heartbeat
* @EventParam IsDebugger (bool) Whether the debugger is currently present
* @EventParam WasDebuggerPresent (bool) Whether the debugger was present previously
* @EventParam IsInVRMode (bool) If the current heartbeat occurred while VR mode was active
* @EventParam bIsEnterprise (bool) If the editor has an enterprise project loaded
* @EventParam Is5MinIdle (bool) Whether the user is idle, using the 5 minute methodology
* @EventParam Is30MinIdle (bool) Whether the user is idle, using the 30 minute methodology
* @EventParam IsInPIE (bool) Whether the user is/was in PIE during this heartbeat
* @EventParam RealIntervalSec (bool) The real time since the last heartbeat
*
* @Source Editor
*
* @Owner Matt.Kuhlenschmidt
*
*/
void FUnrealEdMisc::EditorAnalyticsHeartbeat()
{
	// Don't attempt to send the heartbeat if analytics isn't available
	if(!FEngineAnalytics::IsAvailable())
	{
		return;
	}

	// Analytics has had some very rare instances where a user spams heartbeats at an incredibly high frequency.
	// So while it's technically impossible to trigger this event more than once per HeartbeatIntervalSeconds,
	// put an additional guard band around this function in absolute wall-clock time to ensure it doesn't fire too quickly for some strange reason.
	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	const double HeartbeatInvervalSec = (double)(UnrealEdMiscDefs::HeartbeatIntervalSeconds);
	const double Now = FPlatformTime::Seconds();

	static double LastTimeInPIE = 0;
	if (FPlayWorldCommandCallbacks::IsInPIE())
	{
		LastTimeInPIE = Now;
	}

	// Initialize to ensure the first time this is called we will DEF execute the heartbeat.
	static double LastHeartbeatTime = Now - HeartbeatInvervalSec;
	// allow a little bit of slop in the timing in case the event is only firing slightly too soon, even though that is technically impossible.
	if (Now <= LastHeartbeatTime + HeartbeatInvervalSec*0.9)
	{
		// This can happen too often with editor hitches and long-running tasks, which cause the timer manager to flush many events at once.
		// UE-62403 is tracking a deeeper fix.
		//UE_LOGF(LogUnrealEdMisc, Warning, "Heartbeat event firing too frequently (%.3f sec). This should never happen. Something is wrong with the timer delegate!", (float)(Now - LastHeartbeatTime) );
		return;
	}

	bool bIsDebuggerPresent = FPlatformMisc::IsDebuggerPresent();
	static bool bWasDebuggerPresent = false;
	if (!bWasDebuggerPresent)
	{
		bWasDebuggerPresent = bIsDebuggerPresent;
	}
	const bool bInVRMode = IVREditorModule::Get().GetVRModeBase() != nullptr;
	const bool bIsEnterprise = IProjectManager::Get().IsEnterpriseProject();

	const double LastInteractionTime = FSlateApplication::Get().GetLastUserInteractionTime();
	
	// Did the user interact since the last heartbeat
	const bool bIdle = LastInteractionTime < LastHeartbeatTime;
	
	extern ENGINE_API float GAverageFPS;

	TArray< FAnalyticsEventAttribute > Attributes;
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Idle"), bIdle));
	if(PerformanceAnalyticsStats->IsReliable())
	{
		Attributes.Add(FAnalyticsEventAttribute(TEXT("AverageFPS"), GAverageFPS));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("AverageFrameTime"), PerformanceAnalyticsStats->GetAverageFrameTime()));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("AverageGameThreadTime"), PerformanceAnalyticsStats->GetAverageGameThreadTime()));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("AverageRenderThreadTime"), PerformanceAnalyticsStats->GetAverageRenderThreadTime()));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("AverageGPUFrameTime"), PerformanceAnalyticsStats->GetAverageGPUFrameTime()));
	}
	Attributes.Add(FAnalyticsEventAttribute(TEXT("IsVanilla"), (GEngine && GEngine->IsVanillaProduct())));
	Attributes.Add(FAnalyticsEventAttribute(TEXT("IntervalSec"), UnrealEdMiscDefs::HeartbeatIntervalSeconds));
	Attributes.Add(FAnalyticsEventAttribute(TEXT("IsDebugger"), bIsDebuggerPresent));
	Attributes.Add(FAnalyticsEventAttribute(TEXT("WasDebuggerPresent"), bWasDebuggerPresent));
	Attributes.Add(FAnalyticsEventAttribute(TEXT("IsInVRMode"), bInVRMode));
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Enterprise"), bIsEnterprise));

	// In the 5 Min and 30 Min idle case we treat longer periods of time as none idle post user interaction
	const bool b5MinIdle = LastInteractionTime + (5.0f * 60.0f) < LastHeartbeatTime;
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Is5MinIdle"), b5MinIdle));
	const bool b30MinIdle = LastInteractionTime + (30.0f * 60.0f) < LastHeartbeatTime;
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Is30MinIdle"), b30MinIdle));

	// InPIE bool, added a 5 second grace period to include PIE shutdown issues in the "InPIE" window
	const bool bInPIE = LastTimeInPIE + 5.0f > LastHeartbeatTime;
	Attributes.Add(FAnalyticsEventAttribute(TEXT("IsInPIE"), bInPIE));

	// Interval seconds actually track the entirety of the time since last heartbeat.
	Attributes.Add(FAnalyticsEventAttribute(TEXT("RealIntervalSec"), Now - LastHeartbeatTime));

	FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.Heartbeat"), Attributes);
	
	LastHeartbeatTime = Now;
}

void FUnrealEdMisc::TickAssetAnalytics()
{
	if( bIsAssetAnalyticsPending && FEngineAnalytics::IsAvailable())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FUnrealEdMisc::TickAssetAnalytics);

		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryName);

		if( !AssetRegistryModule.Get().IsLoadingAssets())
		{
			// kill the pending flag
			bIsAssetAnalyticsPending = false;
			// Gather Asset stats
			TArray<FAssetData> AssetData;
			AssetRegistryModule.Get().GetAllAssets(AssetData);

			TArray< FAnalyticsEventAttribute > AssetAttributes;
			int32 NumMapFiles = 0;
			TSet< FName > PackageNames;
			TMap< FTopLevelAssetPath, int32 > ClassInstanceCounts;

			for( auto AssetIter = AssetData.CreateConstIterator(); AssetIter; ++AssetIter )
			{
				PackageNames.Add( AssetIter->PackageName );
				if( AssetIter->AssetClassPath == UWorld::StaticClass()->GetClassPathName()  )
				{
					NumMapFiles++;
				}

				if (!AssetIter->AssetClassPath.IsNull())
				{
					int32* ExistingClassCount = ClassInstanceCounts.Find( AssetIter->AssetClassPath);
					if( ExistingClassCount )
					{
						++(*ExistingClassCount);
					}
					else
					{
						ClassInstanceCounts.Add( AssetIter->AssetClassPath, 1 );
					}
				}
			}
			const UGeneralProjectSettings& ProjectSettings = *GetDefault<UGeneralProjectSettings>();
			AssetAttributes.Add( FAnalyticsEventAttribute( FString( "ProjectId" ), *ProjectSettings.ProjectID.ToString() ));
			AssetAttributes.Add( FAnalyticsEventAttribute( FString( "AssetPackageCount" ), PackageNames.Num() ));
			AssetAttributes.Add( FAnalyticsEventAttribute( FString( "Maps" ), NumMapFiles ));
			AssetAttributes.Emplace( TEXT("Enterprise"), IProjectManager::Get().IsEnterpriseProject() );

			// Send project analytics
			FEngineAnalytics::GetProvider().RecordEvent( FString( "Editor.Usage.AssetCounts" ), AssetAttributes );

			TArray< FAnalyticsEventAttribute > AssetInstances;
			AssetInstances.Add( FAnalyticsEventAttribute( FString( "ProjectId" ), *ProjectSettings.ProjectID.ToString() ));
			for( auto ClassIter = ClassInstanceCounts.CreateIterator(); ClassIter; ++ClassIter )
			{
				AssetInstances.Add( FAnalyticsEventAttribute( ClassIter.Key().ToString(), ClassIter.Value() ) );
			}
			// Send class instance analytics
			FEngineAnalytics::GetProvider().RecordEvent( FString( "Editor.Usage.AssetClasses" ), AssetInstances );
		}
	}
}

bool FUnrealEdMisc::EnableWorldComposition(UWorld* InWorld, bool bEnable)
{
	if (InWorld == nullptr || InWorld->WorldType != EWorldType::Editor)
	{
		return false;
	}
			
	if (!bEnable)
	{
		if (InWorld->WorldComposition != nullptr)
		{
			InWorld->FlushLevelStreaming();
			InWorld->WorldComposition->MarkAsGarbage();
			InWorld->WorldComposition = nullptr;
			UWorldComposition::WorldCompositionChangedEvent.Broadcast(InWorld);
		}

		return false;
	}
	
	if (InWorld->WorldComposition == nullptr)
	{
		FString RootPackageName = InWorld->GetOutermost()->GetName();

		// Map should be saved to disk
		if (!FPackageName::DoesPackageExist(RootPackageName))
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("EnableWorldCompositionNotSaved_Message", "Please save your level to disk before enabling World Composition"));
			return false;
		}
			
		// All existing sub-levels on this map should be removed
		int32 NumExistingSublevels = InWorld->GetStreamingLevels().Num();
		if (NumExistingSublevels > 0)
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("EnableWorldCompositionExistingSublevels_Message", "World Composition cannot be enabled because there are already sub-levels manually added to the persistent level. World Composition uses auto-discovery so you must first remove any manually added sub-levels from the Levels window"));
			return false;
		}
			
		auto WorldComposition = NewObject<UWorldComposition>(InWorld);
		// All map files found in the same and folder and all sub-folders will be added ass sub-levels to this map
		// Make sure user understands this
		WorldComposition->Rescan();
		int32 NumFoundSublevels = WorldComposition->GetTilesList().Num();
		if (NumFoundSublevels)
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("NumSubLevels"), NumFoundSublevels);
			Arguments.Add(TEXT("FolderLocation"), FText::FromString(FPackageName::GetLongPackagePath(RootPackageName)));
			const FText Message = FText::Format(LOCTEXT("EnableWorldCompositionPrompt_Message", "World Composition auto-discovers sub-levels by scanning the folder the level is saved in, and all sub-folders. {NumSubLevels} level files were found in {FolderLocation} and will be added as sub-levels. Do you want to continue?"), Arguments);
			
			auto AppResult = FMessageDialog::Open(EAppMsgType::OkCancel, Message);
			if (AppResult != EAppReturnType::Ok)
			{
				WorldComposition->MarkAsGarbage();
				return false;
			}
		}

		InWorld->WorldComposition = WorldComposition;
		UWorldComposition::WorldCompositionChangedEvent.Broadcast(InWorld);
	}
	
	return true;
}

FString FUnrealEdMisc::GetProjectEditorBinaryPath()
{
#if PLATFORM_WINDOWS
	return FPlatformProcess::ExecutablePath();
#elif PLATFORM_MAC
	@autoreleasepool
	{
		return UTF8_TO_TCHAR([[[NSBundle mainBundle]executablePath] fileSystemRepresentation] );
	}
#elif PLATFORM_LINUX
	const TCHAR* PlatformConfig = FPlatformMisc::GetUBTPlatform();
	FString ExeFileName = FPaths::EngineDir() / TEXT("Binaries") / PlatformConfig / FString(FPlatformProcess::ExecutableName());
	return ExeFileName;
#else
#error "Unknown platform"
#endif
}

bool FUnrealEdMisc::SpawnEditorInstance(const FString& ProjectName)
{
	// Use the same command line parameters that were used for this editor instance.
	const FString Cmd = FString::Printf(TEXT("%s %s"), *ProjectName, *PendingCommandLine.Get(FCommandLine::Get()));

	const FString ExeFilename = GetProjectEditorBinaryPath();
	FProcHandle Handle = FPlatformProcess::CreateProc(*ExeFilename, *Cmd, true, false, false, NULL, 0, NULL, NULL);
	const bool bSuccess = Handle.IsValid();
	if (bSuccess)
	{
		FPlatformProcess::CloseProc(Handle);
	}
	return bSuccess;
}

void FUnrealEdMisc::OnExit()
{
	if ( !bInitialized )
	{
		return;
	}
	bInitialized = false;
	
	if (bIsSurveyingPerformance)
	{
		CancelPerformanceSurvey();
	}

	if (NavigationBuildingNotificationHandler.IsValid())
	{
		NavigationBuildingNotificationHandler = NULL;
	}

	// Report session maximum window and tab counts to engine analytics, if available
	if (FEngineAnalytics::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> TabsAttribs;
		TabsAttribs.Add(FAnalyticsEventAttribute(FString("MaxTabs"), FGlobalTabmanager::Get()->GetMaximumTabCount()));
		TabsAttribs.Add(FAnalyticsEventAttribute(FString("MaxTopLevelWindows"), FGlobalTabmanager::Get()->GetMaximumWindowCount()));

		const UGeneralProjectSettings& ProjectSettings = *GetDefault<UGeneralProjectSettings>();
		TabsAttribs.Add(FAnalyticsEventAttribute(FString("ProjectId"), ProjectSettings.ProjectID.ToString()));

		FEngineAnalytics::GetProvider().RecordEvent(FString("Editor.Usage.WindowCounts"), TabsAttribs);
		
		// Report asset updates (to reflect forward progress made by the user)
		TArray<FAnalyticsEventAttribute> AssetUpdateCountAttribs;
		for (auto& UpdatedAssetPair : NumUpdatesByAssetName)
		{
			AssetUpdateCountAttribs.Add(FAnalyticsEventAttribute(UpdatedAssetPair.Key.ToString(), UpdatedAssetPair.Value));
		}
		AssetUpdateCountAttribs.Emplace( TEXT("Enterprise"), IProjectManager::Get().IsEnterpriseProject() );

		FEngineAnalytics::GetProvider().RecordEvent(FString("Editor.Usage.AssetsSaved"), AssetUpdateCountAttribs);

		FSlateApplication::Get().GetPlatformApplication()->SendAnalytics(&FEngineAnalytics::GetProvider());
		FEditorViewportStats::SendUsageData();
	}

	FInputBindingManager::Get().UnregisterUserDefinedChordChanged(OnUserDefinedChordChangedDelegateHandle);
	FMessageLog::OnMessageSelectionChanged().Unbind();
	FMessageLog::OnMultiSelectActorBegin().Unbind();
	FMessageLog::OnMultiSelectActorEnd().Unbind();
	FUObjectToken::DefaultOnMessageTokenActivated().Unbind();
	FUObjectToken::DefaultOnGetObjectDisplayName().Unbind();
	FAssetNameToken::OnGotoAsset().Unbind();
	FActorToken::DefaultOnMessageTokenActivated().Unbind();

	// Unregister message log UIs
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	MessageLogModule.UnregisterLogListing("EditorErrors");
	MessageLogModule.UnregisterLogListing("LoadErrors");
	MessageLogModule.UnregisterLogListing("LightingResults");
	MessageLogModule.UnregisterLogListing("PackagingResults");
	MessageLogModule.UnregisterLogListing("MapCheck");
	MessageLogModule.UnregisterLogListing("HLODResults");
	FCompilerResultsLog::Unregister();
	MessageLogModule.UnregisterLogListing("PIE");

	// Unregister all events
	FGlobalTabmanager::Get()->OnActiveTabChanged_Unsubscribe(OnActiveTabChangedDelegateHandle);
	FGlobalTabmanager::Get()->OnTabForegrounded_Unsubscribe(OnTabForegroundedDelegateHandle);
	FUserActivityTracking::SetActivity(FUserActivity(TEXT("EditorExit"), EUserActivityContext::Editor));
	FEditorDelegates::SelectedProps.RemoveAll(this);
	FEditorDelegates::DisplayLoadErrors.RemoveAll(this);
	FEditorDelegates::MapChange.RemoveAll(this);
	FEditorDelegates::RefreshEditor.RemoveAll(this);
	FEditorDelegates::PreSaveWorldWithContext.RemoveAll(this);
	FEditorSupportDelegates::RedrawAllViewports.RemoveAll(this);
	GEngine->OnLevelActorAdded().RemoveAll(this);

#if USE_UNIT_TESTS
	FAutomationTestFramework::Get().PreTestingEvent.RemoveAll(this);
	FAutomationTestFramework::Get().PostTestingEvent.RemoveAll(this);
#endif // USE_UNIT_TESTS

	// FCoreDelegates::OnBreakpointTriggered.RemoveAll(this);
	// FCoreDelegates::OnScriptFatalError.RemoveAll(this);

	FEditorDelegates::ChangeEditorMode.RemoveAll(this);
	FCoreDelegates::PreModal.RemoveAll(this);
	FCoreDelegates::PostModal.RemoveAll(this);
	FCoreDelegates::OnEnsureFailed.RemoveAll(this);

	FComponentAssetBrokerage::PRIVATE_ShutdownBrokerage();

	ISourceControlModule::Get().GetProvider().Close();

	UWorldComposition::EnableWorldCompositionEvent.Unbind();
	
	//Fbx dll is currently compile with a different windows platform sdk so we should use this memory bypass later when unreal ed will be using windows 10 platform sdk
	//UnloadFBxLibraries();

	const TMap<FString, FString>& IniRestoreFiles = GetConfigRestoreFilenames();

	for (auto Iter = IniRestoreFiles.CreateConstIterator(); Iter; ++Iter)
	{
		// Key = Config Filename, Value = Backup Filename
		if (FPaths::FileExists(Iter.Value()))
		{
			IFileManager::Get().Copy(*Iter.Key(), *Iter.Value());
		}
	}
	

	// The new process needs to be spawned as late as possible so two editor processes aren't running concurrently for very long.
	// It definitely needs to happen after the preferences file is restored from an import on the line above
	const FString& PendingProjName = FUnrealEdMisc::Get().GetPendingProjectName();
	if( PendingProjName.Len() > 0 )
	{
#if WITH_EDITOR
			// Prevent the Zen subprocess data path from being inherited by a post-close process spawn
			// such as when the editor is restarting itself.
			FPlatformMisc::SetEnvironmentVar(TEXT("UE-ZenSubprocessDataPath"), nullptr);
#endif // WITH_EDITOR
		bool bSuccess = false;
		if (FEditorDelegates::OnRestartRequested.IsBound())
		{
			bSuccess = FEditorDelegates::OnRestartRequested.Execute(PendingProjName);
		}
		else
		{
			bSuccess = SpawnEditorInstance(PendingProjName);
		}
		
		if (!bSuccess)
		{
			// We were not able to spawn the new project exe.
			// Its likely that the exe doesn't exist.
			// Skip shutting down the editor if this happens
			UE_LOGF(LogUnrealEdMisc, Warning, "Could not restart the editor");

			// Clear the pending project to ensure the editor can still be shut down normally
			ClearPendingProjectName();

			return;
		}
	}

	// Unregister the EditorAxisDisplayInfo modular feature
	if (EditorAxisDisplayInfo)
	{
		IModularFeatures::Get().UnregisterModularFeature(IAxisDisplayInfo::GetModularFeatureName(), EditorAxisDisplayInfo.Get());
		EditorAxisDisplayInfo.Reset();
	}

	// Unregister the command executor
	IModularFeatures::Get().UnregisterModularFeature(IConsoleCommandExecutor::ModularFeatureName(), CmdExec.Get());
	CmdExec.Reset();

	//Release static class to be sure its not release in a random way. This class use a static multicastdelegate which can be delete before and crash the editor on exit
	GTexAlignTools.Release();
}

void FUnrealEdMisc::ShutdownAfterError()
{
	ISourceControlModule::Get().GetProvider().Close();
}

void FUnrealEdMisc::CB_SelectedProps()
{
	// Display the actor properties dialog if any actors are selected at all
	if ( GUnrealEd->GetSelectedActorCount() > 0 )
	{
		GUnrealEd->ShowActorProperties();
	}
}

void FUnrealEdMisc::CB_DisplayLoadErrors()
{
	if( !GIsDemoMode )
	{
		// Don't display load errors when starting up in immersive mode
		// @todo immersive: Really only matters on first level load and PIE startup, maybe only disallow at startup?
		const bool bIsImmersive = FParse::Param( FCommandLine::Get(), TEXT( "immersive" ) );
		if( !bIsImmersive && !GIsAutomationTesting)
		{
			FMessageLog("LoadErrors").Open();
		}
	}
}

void FUnrealEdMisc::CB_RefreshEditor()
{
	FEditorDelegates::RefreshAllBrowsers.Broadcast();
}

void FUnrealEdMisc::PreSaveWorld(UWorld* World, FObjectPreSaveContext ObjectSaveContext)
{
	LogAssetUpdate(World, ObjectSaveContext);
	const bool bAutosaveOrPIE = ObjectSaveContext.IsFromAutoSave();
	if (bAutosaveOrPIE || !World || World != GEditor->GetEditorWorldContext().World() || !FEngineAnalytics::IsAvailable())
	{
		return;
	}

	int32 NumAdditiveBrushes = 0;
	int32 NumSubtractiveBrushes = 0;
	for (TActorIterator<ABrush> BrushIt(World); BrushIt; ++BrushIt)
	{
		ABrush* Brush = *BrushIt;
		if (Brush != NULL)
		{
			if (Brush->BrushType == EBrushType::Brush_Add) NumAdditiveBrushes++;
			else if (Brush->BrushType == EBrushType::Brush_Subtract) NumSubtractiveBrushes++;
		}
	}

	TArray< FAnalyticsEventAttribute > BrushAttributes;
	BrushAttributes.Add( FAnalyticsEventAttribute( FString( "Additive" ), NumAdditiveBrushes ));
	BrushAttributes.Add( FAnalyticsEventAttribute( FString( "Subtractive" ), NumSubtractiveBrushes ));
	const UGeneralProjectSettings& ProjectSettings = *GetDefault<UGeneralProjectSettings>();
	BrushAttributes.Add( FAnalyticsEventAttribute( FString( "ProjectId" ), ProjectSettings.ProjectID.ToString()) );

	FEngineAnalytics::GetProvider().RecordEvent( FString( "Editor.Usage.Brushes" ), BrushAttributes );
}

void FUnrealEdMisc::CB_MapChange( uint32 InFlags )
{
	UWorld* World = GWorld;

	// Make sure the world package is never marked dirty here
	const bool bOldDirtyState = World->GetCurrentLevel()->GetOutermost()->IsDirty();


	// Clear property coloration settings.
	const FString EmptyString(TEXT(""));
	GEditor->SetPropertyColorationTarget( World, EmptyString, NULL, NULL, NULL );

	if (InFlags != MapChangeEventFlags::NewMap)
	{
		// Rebuild the collision hash if this map change was rebuilt
		// Minor things like brush subtraction will set it to "0".
		if (InFlags != MapChangeEventFlags::Default)
		{
			if (World->IsInitialized())
			{
				// Call CleanupWorld/InitWorld to detach components and some other stuff
				World->ReInitWorld();

				// ReInitWorld (just above) cleanup and initialize the world but InitWorld does not fully initialize the navigation system for the world.
				// Do it here for map rebuilds ('Build All Levels').
				if (InFlags & MapChangeEventFlags::MapRebuild)
				{
					FNavigationSystem::AddNavigationSystemToWorld(*World, FNavigationSystemRunMode::EditorMode);
				}
			}
		}

		GEditor->EditorUpdateComponents();
	}

	GLevelEditorModeTools().MapChangeNotify();

	/*if ((InFlags&MapChangeEventFlags::MapRebuild) != 0)
	{
		GUnrealEd->UpdateFloatingPropertyWindows();
	}*/

	CB_RefreshEditor();

	// Only reset the auto save timer if we've created or loaded a new map
	if( InFlags & MapChangeEventFlags::NewMap )
	{
		GUnrealEd->GetPackageAutoSaver().ResetAutoSaveTimer();
	}

	if (!bOldDirtyState)
	{
		World->GetCurrentLevel()->GetOutermost()->SetDirtyFlag( bOldDirtyState );
	}
}

void FUnrealEdMisc::CB_RedrawAllViewports()
{
	GUnrealEd->RedrawAllViewports();
}

void FUnrealEdMisc::CB_LevelActorsAdded(AActor* InActor)
{
	if (!UE::GetIsEditorLoadingPackage() &&
		!GIsCookerLoadingPackage &&
		FEngineAnalytics::IsAvailable() &&
		InActor &&
		InActor->GetWorld() == GUnrealEd->GetEditorWorldContext().World() &&
		InActor->IsA(APawn::StaticClass()))
	{
		const UGeneralProjectSettings& ProjectSettings = *GetDefault<UGeneralProjectSettings>();
		FEngineAnalytics::GetProvider().RecordEvent(FString("Editor.Usage.PawnPlacement"), FString( "ProjectId" ), ProjectSettings.ProjectID.ToString());
	}
}

void FUnrealEdMisc::CB_PreAutomationTesting()
{
	// Shut down SCC if it's enabled, as unit tests shouldn't be allowed to make any modifications to source control
	if ( ISourceControlModule::Get().IsEnabled() )
	{
		ISourceControlModule::Get().GetProvider().Close();
	}
}

void FUnrealEdMisc::CB_PostAutomationTesting()
{
	// Re-enable source control
	ISourceControlModule::Get().GetProvider().Init();
}

void FUnrealEdMisc::OnEditorChangeMode(FEditorModeID NewEditorMode)
{
	GLevelEditorModeTools().ActivateMode( NewEditorMode, true );
}

void FUnrealEdMisc::OnEditorPreModal()
{
	if( FSlateApplication::IsInitialized() )
	{
		FSlateApplication::Get().ExternalModalStart();
	}
}

void FUnrealEdMisc::OnEditorPostModal()
{
	if( FSlateApplication::IsInitialized() )
	{
		FSlateApplication::Get().ExternalModalStop();
	}
}

void FUnrealEdMisc::OnEnsureFailed(const ANSICHAR* Expr, const ANSICHAR* File, int32 Line, const TCHAR* Msg, const TCHAR* CombinedErrorMsg)
{
	if (CombinedErrorMsg == nullptr)
	{
		return;
	}

	const UEditorPerProjectUserSettings* const EditorSettings = GetDefault<UEditorPerProjectUserSettings>();
	if (EditorSettings == nullptr || !EditorSettings->bShowEnsuresInMessageLog)
	{
		return;
	}

	FString NotifyMsg;
	if (Msg != nullptr && FCString::Strlen(Msg) > 0)
	{
		NotifyMsg = Msg;
	}
	else
	{
		NotifyMsg = Expr;
	}

	// Always delay, in case we're not in the gamethread or we're in a callstack that doesn't allow slate windows to be created
	FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(FSimpleDelegateGraphTask::FDelegate::CreateLambda([ErrorMsg = FString(CombinedErrorMsg), NotifyMsg = MoveTemp(NotifyMsg)]()
		{
			if (IsEngineExitRequested())
			{
				return;
			}

			FMessageLog EnsureFailedLogger = FMessageLog("EnsureFailed");
			EnsureFailedLogger.Error(FTextStringHelper::CreateFromBuffer(*ErrorMsg));
			EnsureFailedLogger.Notify(FText::Format(NSLOCTEXT("Core", "EnsureFailed", "Ensure condition failed: {0}"), FTextStringHelper::CreateFromBuffer(*NotifyMsg)));
		}), TStatId(), nullptr, ENamedThreads::GameThread);
}

void FUnrealEdMisc::OnActiveTabChanged(TSharedPtr<SDockTab> PreviouslyActive, TSharedPtr<SDockTab> NewlyActivated)
{
	OnUserActivityTabChanged(NewlyActivated);
}

void FUnrealEdMisc::OnTabForegrounded(TSharedPtr<SDockTab> ForegroundTab, TSharedPtr<SDockTab> BackgroundTab)
{
	OnUserActivityTabChanged(ForegroundTab);
}

void FUnrealEdMisc::OnUserActivityTabChanged(TSharedPtr<SDockTab> InTab)
{
	if (InTab.IsValid())
	{
		FString Activity = FString::Printf(TEXT("Layout=\"%s\" Label=\"%s\" Content=%s"), *InTab->GetLayoutIdentifier().ToString(), *InTab->GetTabLabel().ToString(), *InTab->GetContent()->GetTypeAsString());

		FUserActivityTracking::SetActivity(FUserActivity(Activity, EUserActivityContext::Editor));
	}
}

void FUnrealEdMisc::OnDeferCommand( const FString& DeferredCommand )
{
	GUnrealEd->DeferredCommands.Add( DeferredCommand );
}


void FUnrealEdMisc::OnMessageTokenActivated(const TSharedRef<IMessageToken>& Token)
{
	if(Token->GetType() != EMessageToken::Object )
	{
		return;
	}

	const TSharedRef<FUObjectToken> UObjectToken = StaticCastSharedRef<FUObjectToken>(Token);
	UObject* Object = nullptr;

	// Due to blueprint reconstruction, we can't directly use the Object as it will get trashed during the blueprint reconstruction and the message token will no longer point to the right UObject.
	// Instead we will retrieve the object from the name which should always be good.
	if (UObjectToken->GetObject().IsValid())
	{
		if (!UObjectToken->ToText().ToString().Equals(UObjectToken->GetObject().Get()->GetName()))
		{
			Object = FindObject<UObject>(nullptr, *UObjectToken->GetOriginalObjectPathName());
		}
		else
		{
			Object = const_cast<UObject*>(UObjectToken->GetObject().Get());
		}
	}
	else
	{
		// We have no object (probably because is now stale), try finding the original object linked to this message token to see if it still exist
		Object = FindObject<UObject>(nullptr, *UObjectToken->GetOriginalObjectPathName());
	}

	if(Object != nullptr)
	{
		ULightmappedSurfaceCollection* SurfaceCollection = Cast<ULightmappedSurfaceCollection>(Object);
		if (SurfaceCollection)
		{
			// Ignore surface collection for actor multi-select
			if (bActorMultiSelect)
			{
				return;
			}

			// Deselect all selected object...
			GEditor->SelectNone( true, true );

			// Select the surfaces in this mapping
			TArray<AActor*> SelectedActors;
			for (int32 SurfaceIdx = 0; SurfaceIdx < SurfaceCollection->Surfaces.Num(); SurfaceIdx++)
			{
				int32 SurfaceIndex = SurfaceCollection->Surfaces[SurfaceIdx];
				FBspSurf& Surf = SurfaceCollection->SourceModel->Surfs[SurfaceIndex];
				SurfaceCollection->SourceModel->ModifySurf(SurfaceIndex, 0);
				Surf.PolyFlags |= PF_Selected;
				if (Surf.Actor)
				{
					SelectedActors.AddUnique(Surf.Actor);
				}
			}

			// Add the brushes to the selected actors list...
			if (SelectedActors.Num() > 0)
			{
				GEditor->MoveViewportCamerasToActor(SelectedActors, false);
			}

			GEditor->NoteSelectionChange();
		}
		else
		{
			AActor* Actor = Cast<AActor>(Object);
			if( !Actor )
			{
				Actor = Object->GetTypedOuter<AActor>();
			}

			if (Actor && Actor->GetLevel() != nullptr)
			{
				SelectActorFromMessageToken(Actor);
			}
			else
			{
				// Ignore blueprint objects for actor multi-select
				if (bActorMultiSelect)
				{
					return;
				}

				TArray<UObject*> ObjectArray;

				if (Object->IsInBlueprint())
				{
					// Determine if we are the root of our blueprint
					UBlueprint* Blueprint = UBlueprint::GetBlueprintFromClass(Object->GetClass());

					if (Blueprint != nullptr)
					{
						ObjectArray.Add(Blueprint);
					}
					else // we are a sub object, so we need to find the root of our current blueprint(not the outermost as blueprint can contain other blueprint)
					{
						UObject* ParentObject = Object->GetOuter();

						while (Blueprint == nullptr && ParentObject != nullptr)
						{
							Blueprint = UBlueprint::GetBlueprintFromClass(ParentObject->GetClass());
							ParentObject = ParentObject->GetOuter();
						}

						if (Blueprint != nullptr)
						{
							ObjectArray.Add(Blueprint);
						}
					}
				}
				else
				{
					ObjectArray.Add(Object);
				}

				GEditor->SyncBrowserToObjects(ObjectArray);
			}
		}
	}
}

/** Delegate used to get a display name for a message log FAssetData token  */
FText FUnrealEdMisc::OnGetAssetDataDisplayName(const FAssetData& InObject, const bool bFullPath)
{
	static FName NAME_ActorLabel("ActorLabel");
	FString DisplayName;
	TStringBuilder<FName::StringBufferSize> Buffer;
	if (InObject.GetTagValue(NAME_ActorLabel, DisplayName) || 
		InObject.GetTagValue(FPrimaryAssetId::PrimaryAssetDisplayNameTag, DisplayName))
	{
		Buffer << DisplayName << TEXT(" (");
		InObject.AppendObjectPath(Buffer);
		Buffer << TEXT(")");
		return FText::FromStringView(Buffer.ToView());
	}
	
	// Verse paths only exist for top level assets, so show the same value regardless of bFullPath;
	// we will fall through if we have a subobject.
	if (IAssetTools::Get().ShowingContentVersePath())
	{
		UE::Core::FVersePath VersePath = InObject.GetVersePath();
		if (VersePath.IsValid())
		{
			return FText::FromString(MoveTemp(VersePath).ToString());
		}
	}
	
	if (bFullPath)
	{
		InObject.AppendObjectPath(Buffer);
		return FText::FromStringView(Buffer.ToView());
	}
	return FText::FromName(InObject.PackageName);
}

/** Delegate used on message log FAssetData token activation */
void FUnrealEdMisc::OnAssetDataTokenActivated(const TSharedRef<class IMessageToken>& InToken)
{
	if (InToken->GetType() != EMessageToken::AssetData)
	{
		return;
	}
	
	const TSharedRef<FAssetDataToken> Token = StaticCastSharedRef<FAssetDataToken>(InToken);
	const FAssetData& AssetData = Token->GetAssetData();
	// If this is a standalone asset, jump to it in the content browser
	if (AssetData.GetOptionalOuterPathName().IsNone())
	{
		// Actor multi-select only cares about actors, not assets
		if (bActorMultiSelect)
		{
			return;
		}

		GEditor->SyncBrowserToObjects({ AssetData });
	}
	// If this is a loaded actor, resolve the pointer and select it
	else if (AActor* Actor = Cast<AActor>(AssetData.GetSoftObjectPath().ResolveObject()))
	{
		SelectActorFromMessageToken(Actor);
	}
	else
	{
		FName PackageName = FName(FPackageName::ObjectPathToPackageName(WriteToString<FName::StringBufferSize>(AssetData.GetOptionalOuterPathName()).ToView()));
		UWorld* EditorWorld = GEditor->GetEditorWorldContext().World();
		// If this is an unloaded actor in the currently loaded level, select it as an unloaded actor
		if (EditorWorld && EditorWorld->GetPackage()->GetFName() == PackageName)
		{
			TUniquePtr<FWorldPartitionActorDesc> Desc = FWorldPartitionActorDescUtils::GetActorDescriptorFromAssetData(AssetData);
			if (Desc.IsValid())
			{
				if (bActorMultiSelect)
				{
					// Selection is deferred to OnMultiSelectActorEnd when multi-select is active
					UnloadedActorsSelected.AddUnique(Desc->GetGuid());
				}
				else
				{
					GEditor->BroadcastSelectUnloadedActors({ Desc->GetGuid() });
				}
			}
		}
		// If this is an actor in an unloaded level, jump to the level in the content browser
		else 
		{
			// Actor multi-select doesn't care about browsing to unloaded levels
			if (bActorMultiSelect)
			{
				return;
			}

			FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry");
			IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
			TArray<FAssetData> OutAssets;
			AssetRegistry.GetAssetsByPackageName(PackageName, OutAssets);
			GEditor->SyncBrowserToObjects(OutAssets);
		}
	}
}

void FUnrealEdMisc::OnActorTokenActivated(const TSharedRef<class IMessageToken>& Token)
{
	if (Token->GetType() == EMessageToken::Actor)
	{
		const TSharedRef<FActorToken> ActorToken = StaticCastSharedRef<FActorToken>(Token);
		if (AActor* Actor = FindObject<AActor>(nullptr, *ActorToken->GetActorPath()))
		{
			SelectActorFromMessageToken(Actor);
		}
		else
		{
			if (bActorMultiSelect)
			{
				// Selection is deferred to OnMultiSelectActorEnd when multi-select is active
				UnloadedActorsSelected.AddUnique(ActorToken->GetActorGuid());
			}
			else
			{
				GEditor->BroadcastSelectUnloadedActors({ ActorToken->GetActorGuid() });
			}
		}
	}
}

void FUnrealEdMisc::SelectActorFromMessageToken(AActor* InActor)
{
	if (bActorMultiSelect)
	{
		// Selection is deferred to OnMultiSelectActorEnd when multi-select is active
		ActorsSelected.AddUnique(InActor);
		return;
	}

	// Select the actor
	GEditor->SelectNone(false, true);
	GEditor->SelectActor(InActor, /*InSelected=*/true, /*bNotify=*/false, /*bSelectEvenIfHidden=*/true);
	GEditor->NoteSelectionChange();
	GEditor->MoveViewportCamerasToActor(*InActor, false);

	// Update the property windows and create one if necessary
	GUnrealEd->ShowActorProperties();
	GUnrealEd->UpdateFloatingPropertyWindows();
}

UNREALED_API void FUnrealEdMisc::OnMultiSelectActorFromMessageTokenBegin()
{
	ensure(bActorMultiSelect == false);
	bActorMultiSelect = true;
}

UNREALED_API void FUnrealEdMisc::OnMultiSelectActorFromMessageTokenEnd()
{
	ensure(bActorMultiSelect == true);
	bActorMultiSelect = false;

	// Clear selection
	GEditor->SelectNone(false, true);

	// Notify about selection of unloaded actors
	if (UnloadedActorsSelected.Num())
	{
		GEditor->BroadcastSelectUnloadedActors(UnloadedActorsSelected);
		UnloadedActorsSelected.Empty();
	}

	// Select the actors
	if (ActorsSelected.Num())
	{
		for (AActor* Actor : ActorsSelected)
		{
			GEditor->SelectActor(Actor, /*InSelected=*/true, /*bNotify=*/false, /*bSelectEvenIfHidden=*/true);
		}
		GEditor->NoteSelectionChange();
		GEditor->MoveViewportCamerasToActor(ActorsSelected, false);
		ActorsSelected.Empty();
	}

	// Update the property windows and create one if necessary
	GUnrealEd->ShowActorProperties();
	GUnrealEd->UpdateFloatingPropertyWindows();
}

FText FUnrealEdMisc::OnGetDisplayName(const UObject* InObject, const bool bFullPath)
{
	FText Name = LOCTEXT("DisplayNone", "<None>");

	if (InObject != nullptr)
	{
		// Is this an object held by an actor?
		const AActor* Actor = nullptr;
		const UActorComponent* Component = Cast<UActorComponent>(InObject);
 
		if (Component != nullptr)
		{
			Actor = Cast<AActor>(Component->GetOuter());
		}
 
		if (Actor != nullptr)
		{
			Name = FText::FromString( bFullPath ? Actor->GetPathName() : Actor->GetName() );
		}
		else if (InObject != nullptr)
		{
			if (bFullPath)
			{
				UE::Core::FVersePath VersePath;

				if (IAssetTools::Get().ShowingContentVersePath())
				{
					VersePath = InObject->GetVersePath();
				}

				if (VersePath.IsValid())
				{
					Name = FText::FromString(MoveTemp(VersePath).ToString());
				}
				else
				{
					Name = FText::FromString(InObject->GetPathName());
				}
			}
			else
			{
				Name = FText::FromString(InObject->GetName());
			}
		}
	}

	return Name;
}

void FUnrealEdMisc::OnMessageSelectionChanged(TArray< TSharedRef<FTokenizedMessage> >& Selection)
{
	// Clear existing selections
	GEditor->SelectNone(false, true);

	bool bActorsSelected = false;
	TArray<UObject*> ObjectArray;	

	const int32 NumSelected = Selection.Num();
	if (NumSelected > 0)
	{
		const FScopedBusyCursor BusyCursor;
		for( int32 LineIndex = 0; LineIndex < NumSelected; ++LineIndex )
		{
			TSharedPtr<FTokenizedMessage> Line = Selection[ LineIndex ];

			// Find objects reference by this message
			const TArray< TSharedRef<IMessageToken> >& MessageTokens = Line->GetMessageTokens();
			for(auto TokenIt = MessageTokens.CreateConstIterator(); TokenIt; ++TokenIt)
			{
				const TSharedRef<IMessageToken> Token = *TokenIt;
				if( Token->GetType() == EMessageToken::Object )
				{
					const TSharedRef<FUObjectToken> UObjectToken = StaticCastSharedRef<FUObjectToken>(Token);
					if( UObjectToken->GetObject().IsValid() )
					{
						// Check referenced object type
						UObject* Object = UObjectToken->GetObject().Get();
						UPrimitiveComponent* Component = Cast<UPrimitiveComponent>(Object);
						AActor* Actor =  Cast<AActor>(Object);
						if( Component != NULL )
						{
							check( !Actor);
							if( Component->GetOwner() )
							{
								Actor = Component->GetOwner();
							}							
						}

						if( Actor != NULL )
						{
							// Actor found, move to it if it's first and only in the list
							if( !bActorsSelected )
							{
								GEditor->SelectNone(false, true);	
								bActorsSelected = true;
								if( Selection.Num() == 1)
								{
									GEditor->MoveViewportCamerasToActor(*Actor, false);
								}
							}

							GEditor->SelectActor(Actor, /*InSelected=*/true, /*bNotify=*/false, /*bSelectEvenIfHidden=*/true); 
						}
						else
						{
							// Add object to list of objects to sync content browser to
							ObjectArray.Add(Object);
						}
					}
				}
			}
		}

		if( bActorsSelected )
		{
			GEditor->NoteSelectionChange();

			// Update the property windows and create one if necessary
			GUnrealEd->ShowActorProperties();
			GUnrealEd->UpdateFloatingPropertyWindows();
		}

		if (ObjectArray.Num() > 0)
		{
			GEditor->SyncBrowserToObjects(ObjectArray);
		}
	}

	// Now, special handle the BSP mappings...
	if (NumSelected > 0)
	{
		const FScopedBusyCursor BusyCursor;
		TArray<ULightmappedSurfaceCollection*> SelectedSurfaceCollections;

		for( int32 LineIndex = 0; LineIndex < NumSelected; ++LineIndex )
		{
			TSharedPtr<FTokenizedMessage> Line = Selection[ LineIndex ];

			// Find objects reference by this message
			const TArray< TSharedRef<IMessageToken> >& MessageTokens = Line->GetMessageTokens();
			for(auto TokenIt = MessageTokens.CreateConstIterator(); TokenIt; ++TokenIt)
			{
				const TSharedRef<IMessageToken> Token = *TokenIt;
				if( Token->GetType() == EMessageToken::Object )
				{
					const TSharedRef<FUObjectToken> UObjectToken = StaticCastSharedRef<FUObjectToken>(Token);
					if( UObjectToken->GetObject().IsValid() )
					{
						// Check referenced object type
						UObject* Object = UObjectToken->GetObject().Get();
						if (Object != NULL)
						{
							ULightmappedSurfaceCollection* SelectedSurfaceCollection = Cast<ULightmappedSurfaceCollection>(Object);
							if (SelectedSurfaceCollection)
							{
								SelectedSurfaceCollections.Add(SelectedSurfaceCollection);
							}
						}
					}
				}
			}
		}

		// If any surface collections are selected, select them in the editor
		if (SelectedSurfaceCollections.Num() > 0)
		{
			TArray<AActor*> SelectedActors;
			for (int32 CollectionIdx = 0; CollectionIdx < SelectedSurfaceCollections.Num(); CollectionIdx++)
			{
				ULightmappedSurfaceCollection* SurfaceCollection = SelectedSurfaceCollections[CollectionIdx];
				if (SurfaceCollection != NULL)
				{
					// Select the surfaces in this mapping
					for (int32 SurfaceIdx = 0; SurfaceIdx < SurfaceCollection->Surfaces.Num(); SurfaceIdx++)
					{
						int32 SurfaceIndex = SurfaceCollection->Surfaces[SurfaceIdx];
						FBspSurf& Surf = SurfaceCollection->SourceModel->Surfs[SurfaceIndex];
						SurfaceCollection->SourceModel->ModifySurf(SurfaceIndex, 0);
						Surf.PolyFlags |= PF_Selected;
						if (Surf.Actor != NULL)
						{
							SelectedActors.AddUnique(Surf.Actor);
						}
					}
				}
			}

			// Add the brushes to the selected actors list...
			if (SelectedActors.Num() > 0)
			{
				GEditor->MoveViewportCamerasToActor(SelectedActors, false);
			}

			GEditor->NoteSelectionChange();
		}
	}
}

void FUnrealEdMisc::OnGotoAsset(const FString& InAssetPath) const
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryName);
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	FAssetData AssetData = AssetRegistry.GetAssetByObjectPath( FSoftObjectPath(InAssetPath) );
	if ( AssetData.IsValid() )
	{
		TArray<FAssetData> AssetDataToSync;

		// if its a package, sync the browser to the assets inside the package
		if(AssetData.GetClass() == UPackage::StaticClass() && !AssetData.HasAnyPackageFlags(PKG_ContainsMapData | PKG_ContainsNoAsset))
		{
			TArray<FAssetData> AssetsInPackage;
			if (AssetRegistry.GetAssetsByPackageName(AssetData.PackageName, AssetsInPackage))
			{
				for (const FAssetData& SubAssetData : AssetsInPackage)
				{
					if (SubAssetData.IsValid() && SubAssetData.IsUAsset())
					{
						AssetDataToSync.Add(SubAssetData);
					}
				}
			}
		}
		
		if(AssetDataToSync.Num() == 0)
		{
			AssetDataToSync.Add(AssetData);
		}
		
		GEditor->SyncBrowserToObjects(AssetDataToSync);
	}	
}

void FUnrealEdMisc::OnObjectSaved(UObject* SavedObject, FObjectPreSaveContext SaveContext)
{
	// Ensure the saved object is a non-UWorld asset (UWorlds are handled separately)
	if (!SavedObject->IsA<UWorld>() && SavedObject->IsAsset())
	{
		LogAssetUpdate(SavedObject, SaveContext);
	}
}

void FUnrealEdMisc::LogAssetUpdate(UObject* UpdatedAsset, FObjectPreSaveContext SaveContext)
{
	UPackage* AssetPackage = UpdatedAsset->GetOutermost();
	const bool bIsPIESave = AssetPackage->RootPackageHasAnyFlags(PKG_PlayInEditor);
	const bool bIsAutosave = GUnrealEd->GetPackageAutoSaver().IsAutoSaving();

	if (!bIsPIESave && !bIsAutosave && !GIsAutomationTesting && !SaveContext.IsProceduralSave())
	{
		uint32& NumUpdates = NumUpdatesByAssetName.FindOrAdd(UpdatedAsset->GetClass()->GetFName());
		NumUpdates++;
	}
}

void FUnrealEdMisc::AllowSavingLayoutOnClose(bool bIsEnabled)
{
	FGlobalTabmanager::Get()->SetCanSavePersistentLayouts(bIsEnabled);
}

bool FUnrealEdMisc::IsSavingLayoutOnClosedAllowed()
{
	return FGlobalTabmanager::Get()->CanSavePersistentLayouts();
}

void FUnrealEdMisc::SwitchProject(const FString& GameOrProjectFileName, bool bWarn, const TOptional<FString>& NewCommandLine)
{
	if (GUnrealEd->WarnIfLightingBuildIsCurrentlyRunning())
	{
		return;
	}

	const bool bIsProjectFileName = FPaths::GetExtension(GameOrProjectFileName) == FProjectDescriptor::GetExtension();

	bool bSwitch = true;

	if(bWarn)
	{
		// Get the project name to switch to
		FString ProjectDisplayName = GameOrProjectFileName;
		if ( bIsProjectFileName )
		{
			// In rocket the display name is just the base filename of the project
			ProjectDisplayName = FPaths::GetBaseFilename(GameOrProjectFileName);
		}

		// Warn the user that this will restart the editor.  Make sure they want to continue
		const FText Title = LOCTEXT( "SwitchProject", "Switch Project" ); 
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("CurrentProjectName"), FText::FromString( ProjectDisplayName ));
		const FText Message = FText::Format( LOCTEXT( "SwitchProjectWarning", "The editor will restart to switch to the {CurrentProjectName} project.  You will be prompted to save any changes before the editor restarts.  Continue switching projects?" ), Arguments ); 

		// Present the user with a warning that changing projects has to restart the editor
		FSuppressableWarningDialog::FSetupInfo Info( Message, Title, "Warning_SwitchProject", GEditorSettingsIni );
		Info.ConfirmText = LOCTEXT( "Yes", "Yes" );
		Info.CancelText = LOCTEXT( "No", "No" );

		FSuppressableWarningDialog SwitchProjectDlg( Info );
		if( SwitchProjectDlg.ShowModal() == FSuppressableWarningDialog::Cancel )
		{
			bSwitch = false;
		}
	}

	// If the user wants to continue with the restart set the pending project to swtich to and close the editor
	if( bSwitch )
	{
		FString PendingProjName;
		if ( bIsProjectFileName )
		{
			// Put quotes around the file since it may contain spaces.
			PendingProjName = FString::Printf(TEXT("\"%s\""), *GameOrProjectFileName);
		}
		else
		{
			PendingProjName = GameOrProjectFileName;
		}

		SetPendingProjectName( PendingProjName );
		PendingCommandLine = NewCommandLine;

		// Close the editor.  This will prompt the user to save changes.  If they hit cancel, we abort the project switch
		GEngine->DeferredCommands.Add( TEXT("CLOSE_SLATE_MAINFRAME"));
	}
	else
	{
		ClearPendingProjectName();
	}
}

void FUnrealEdMisc::RestartEditor(bool bWarn, const TOptional<FString>& NewCommandLine)
{
	if (GUnrealEd->WarnIfLightingBuildIsCurrentlyRunning())
	{
		return;
	}

	if( FPaths::IsProjectFilePathSet() )
	{
		SwitchProject(FPaths::GetProjectFilePath(), bWarn, NewCommandLine);
	}
	else if(FApp::HasProjectName())
	{
		SwitchProject(FApp::GetProjectName(), bWarn, NewCommandLine);
	}
	else
	{
		SwitchProject(TEXT(""), bWarn, NewCommandLine);
	}
}

void FUnrealEdMisc::BeginPerformanceSurvey()
{
	// Don't attempt to run the survey if analytics isn't available
	if(!FEngineAnalytics::IsAvailable())
	{
		return;
	}

	// Tell the level editor we want to be notified when selection changes
	FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>( LevelEditorName );
	OnMapChangedDelegateHandle = LevelEditor.OnMapChanged().AddRaw( this, &FUnrealEdMisc::OnMapChanged );

	// Initialize survey variables
	bIsSurveyingPerformance = true;
	LastFrameRateTime = FDateTime::UtcNow();
	FrameRateSamples.Empty();
}

void FUnrealEdMisc::TickPerformanceAnalytics()
{
	// Don't run if we've not yet loaded a project
	if( !FApp::HasProjectName() )
	{
		return;
	}

	// Before beginning the survey wait for the asset registry to load and make sure Slate is ready
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryName);
	if (AssetRegistryModule.Get().IsLoadingAssets() || !FSlateApplication::IsInitialized())
	{
		return;
	}

	// Don't run the survey if Slate isn't running normally
	FSlateApplication& SlateApp = FSlateApplication::Get();
	if (!SlateApp.IsNormalExecution())
	{
		return;
	}

	// Don't run the test if we are throttling (due to minimized or not in foreground) as this will 
	// greatly affect the framerate
	if( GEditor->ShouldThrottleCPUUsage() )
	{
		return;
	}

	// Update the stats needed by the analytics heartbeat
	PerformanceAnalyticsStats->Update();

	// Also check to see if we need to run the performance survey
	if (!bIsSurveyingPerformance)
	{
		return;
	}

	// Sample the frame rate until we have enough samples to take the average
	if (FrameRateSamples.Num() < PerformanceSurveyDefs::NumFrameRateSamples)
	{
		FDateTime Now = FDateTime::UtcNow();
		if (Now - LastFrameRateTime > PerformanceSurveyDefs::FrameRateSampleInterval)
		{
			FrameRateSamples.Add(SlateApp.GetAverageDeltaTimeForResponsiveness());
			LastFrameRateTime = Now;
		}
	}
	else
	{
		// We have enough samples - take the average and record with analytics
		float FrameTime = 0.0f;
		for (auto Iter = FrameRateSamples.CreateConstIterator(); Iter; ++Iter)
		{
			FrameTime += *Iter;
		}
		float AveFrameRate = PerformanceSurveyDefs::NumFrameRateSamples / FrameTime;

		if( FEngineAnalytics::IsAvailable() )
		{
			FString AveFrameRateString = FString::Printf( TEXT( "%.1f" ), AveFrameRate);
			IAnalyticsProvider& EngineAnalytics = FEngineAnalytics::GetProvider();

			TArray< FAnalyticsEventAttribute > EventAttributes;
			EventAttributes.Emplace( TEXT( "MeanFrameRate" ), AveFrameRateString );
			EventAttributes.Emplace( TEXT( "Enterprise" ), IProjectManager::Get().IsEnterpriseProject() );

			EngineAnalytics.RecordEvent(TEXT( "Editor.Performance.FrameRate" ), EventAttributes);
		}

		CancelPerformanceSurvey();
	}
}

void FUnrealEdMisc::CancelPerformanceSurvey()
{
	bIsSurveyingPerformance = false;
	FrameRateSamples.Empty();

	FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>( LevelEditorName );
	LevelEditor.OnMapChanged().Remove( OnMapChangedDelegateHandle );
}

void FUnrealEdMisc::OnMapChanged( UWorld* World, EMapChangeType MapChangeType )
{
	if (bIsSurveyingPerformance)
	{
		CancelPerformanceSurvey();
	}
}

bool FUnrealEdMisc::GetURL( const TCHAR* InKey, FString& OutURL, const bool bCheckRocket/* = false*/ ) const
{
	check( InKey );
	check( GConfig );
	OutURL.Empty();

	bool bFound = false;

	const FString MainUrlSection = TEXT("UnrealEd.URLs");
	const FString OverrideUrlSection = TEXT("UnrealEd.URLOverrides");
	const FString TestUrlSection = TEXT("UnrealEd.TestURLs");

	if(  !FEngineBuildSettings::IsInternalBuild() && !FEngineBuildSettings::IsPerforceBuild() )
	{
		// For external builds try to find in the overrides first. 
		bFound = GConfig->GetString(*OverrideUrlSection, InKey, OutURL, GEditorIni);
	}

	if( !bFound )
	{
		bFound = GConfig->GetString(*MainUrlSection, InKey, OutURL, GEditorIni);
	}

	return bFound;
}

void FUnrealEdMisc::ReplaceDocumentationURLWildcards(FString& Url, const FCultureRef& Culture, const FString& PageId)
{
	static FString Version = FString::FromInt(FEngineVersion::Current().GetMajor()) + TEXT(".") + FString::FromInt(FEngineVersion::Current().GetMinor());
	Url.ReplaceInline(TEXT("{VERSION}"), *Version);
	Url.ReplaceInline(TEXT("{I18N}"), *(Culture->GetName()));
	Url.ReplaceInline(TEXT("{PAGEID}"), *PageId);
}

FString FUnrealEdMisc::GetExecutableForCommandlets() const
{
	ILauncherServicesModule& LauncherServicesModule = FModuleManager::LoadModuleChecked<ILauncherServicesModule>(TEXT("LauncherServices"));
	return LauncherServicesModule.GetExecutableForCommandlets();
}

void FUnrealEdMisc::OpenMarketplace(const FString& CustomLocation)
{
	TArray<FAnalyticsEventAttribute> EventAttributes;

	FString Location = CustomLocation.IsEmpty() ? TEXT("/ue/marketplace") : CustomLocation;

	EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Location"), Location));
	
	auto Service = GEditor->GetServiceLocator()->GetServiceRef<IPortalApplicationWindow>();
	if(Service->IsAvailable())
	{
		TAsyncResult<bool> Result = Service->NavigateTo(Location);
		if(FEngineAnalytics::IsAvailable())
		{
			EventAttributes.Add(FAnalyticsEventAttribute(TEXT("OpenSucceeded"), TEXT("TRUE")));
		}
	}
	else
	{
		ILauncherPlatform* LauncherPlatform = FLauncherPlatformModule::Get();

		if(LauncherPlatform != nullptr)
		{
			FOpenLauncherOptions OpenOptions(Location);
			if(LauncherPlatform->OpenLauncher(OpenOptions))
			{
				EventAttributes.Add(FAnalyticsEventAttribute(TEXT("OpenSucceeded"), TEXT("TRUE")));
			}
			else
			{
				EventAttributes.Add(FAnalyticsEventAttribute(TEXT("OpenSucceeded"), TEXT("FALSE")));

				if(EAppReturnType::Yes == FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("InstallMarketplacePrompt", "The Marketplace requires the Epic Games Launcher, which does not seem to be installed on your computer. Would you like to install it now?")))
				{
					FOpenLauncherOptions InstallOptions(true, Location);
					if(!LauncherPlatform->OpenLauncher(InstallOptions))
					{
						EventAttributes.Add(FAnalyticsEventAttribute(TEXT("InstallSucceeded"), TEXT("FALSE")));
						FMessageDialog::Open(EAppMsgType::Ok, FText::FromString(TEXT("Sorry, there was a problem installing the Launcher.\nPlease try to install it manually!")));
					}
					else
					{
						EventAttributes.Add(FAnalyticsEventAttribute(TEXT("InstallSucceeded"), TEXT("TRUE")));
					}
				}
			}

			EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Source"), TEXT("EditorToolbar")));

		}
	}


	if(FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.OpenMarketplace"), EventAttributes);
	}
}

void FUnrealEdMisc::OnUserDefinedChordChanged(const FUICommandInfo& CommandInfo)
{
	if( FEngineAnalytics::IsAvailable() )
	{
		FString ChordName = FString::Printf(TEXT("%s.%s"), *CommandInfo.GetBindingContext().ToString(), *CommandInfo.GetCommandName().ToString());

		//@todo This shouldn't be using a localized value; GetInputText() [10/11/2013 justin.sargent]
		TArray< FAnalyticsEventAttribute > ChordAttribs;
		ChordAttribs.Add(FAnalyticsEventAttribute(TEXT("Context"), ChordName));
		ChordAttribs.Add(FAnalyticsEventAttribute(TEXT("Shortcut"), CommandInfo.GetInputText().ToString()));
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.KeyboardShortcut"), ChordAttribs);
	}
}

void FUnrealEdMisc::MountTemplateSharedPaths()
{
	FString TemplateFilename = FPaths::GetPath(FPaths::GetProjectFilePath());
	UTemplateProjectDefs* TemplateInfo = GameProjectUtils::LoadTemplateDefs(TemplateFilename);
	
	if( TemplateInfo != nullptr )
	{	
		EFeaturePackDetailLevel EditDetail = TemplateInfo->EditDetailLevelPreference;

		// Extract the mount names and insert mount points for each of the shared packs
		TArray<FString> AddedMountSources;
		for (int32 iShared = 0; iShared < TemplateInfo->SharedContentPacks.Num() ; iShared++)
		{
			EFeaturePackDetailLevel EachEditDetail = (EFeaturePackDetailLevel)EditDetail;
			FString DetailString;
			UEnum::GetValueAsString(TEXT("/Script/AddContentDialog.EFeaturePackDetailLevel"), EachEditDetail, DetailString);

			FFeaturePackLevelSet EachPack = TemplateInfo->SharedContentPacks[iShared];
			if((EachPack.DetailLevels.Num() == 1) && ( EachEditDetail !=  EachPack.DetailLevels[0] ))
			{
				// If theres only only detail level override the requirement with that
				EachEditDetail = EachPack.DetailLevels[0];
				// Get the name of the level we are falling back to so we can tell the user
				FString FallbackDetailString;
				UEnum::GetValueAsString(TEXT("/Script/AddContentDialog.EFeaturePackDetailLevel"), EachEditDetail, FallbackDetailString);
				UE_LOGF(LogUnrealEdMisc, Verbose, "Only 1 detail level defined for %ls in %ls. Cannot edit detail level %ls. Will fallback to %ls", *EachPack.MountName, *TemplateFilename, *DetailString,*FallbackDetailString);
				// Then correct the string too !
				DetailString = FallbackDetailString;

			}
			else if (EachPack.DetailLevels.Num() == 0)
			{
				// If no levels are supplied we cant really use this pack !
				UE_LOGF(LogUnrealEdMisc, Warning, "No detail levels defined for %ls in %ls.", *EachPack.MountName, *TemplateFilename );
				continue;
			}
			for (int32 iDetail = 0; iDetail < EachPack.DetailLevels.Num(); iDetail++)
			{
				if (EachPack.DetailLevels[iDetail] == EachEditDetail)
				{					
					FString ShareMountName = EachPack.MountName;
					if (AddedMountSources.Find(ShareMountName) == INDEX_NONE)
					{
						FString ResourcePath = FPaths::Combine(TEXT("Templates"), TEXT("TemplateResources"), *DetailString, *ShareMountName, TEXT("Content"));
						FString FullPath = FPaths::Combine(*FPaths::RootDir(), *ResourcePath);

						if (FPaths::DirectoryExists(FullPath))
						{
							FString MountName = FString::Printf(TEXT("/Game/%s/"), *ShareMountName);	
							FPackageName::RegisterMountPoint(*MountName, *FullPath);
							AddedMountSources.Add(ShareMountName);
						}
						else
						{
							UE_LOGF(LogUnrealEdMisc, Warning, "Cannot find path %ls to mount for %ls resource in %ls.", *FullPath, *EachPack.MountName, *TemplateFilename);
						}
					}
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE

