// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// From Core:
#include "CoreTypes.h"
#include "Misc/Exec.h"
#include "Misc/AssertionMacros.h"
#include "HAL/PlatformMisc.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "CoreFwd.h"
#include "Containers/ContainersFwd.h"
#include "UObject/UObjectHierarchyFwd.h"
#include "HAL/PlatformCrt.h"
#include "Containers/Array.h"
#include "HAL/UnrealMemory.h"
#include "Templates/IsPointer.h"
#include "HAL/PlatformMemory.h"
#include "GenericPlatform/GenericPlatformMemory.h"
#include "HAL/MemoryBase.h"
#include "Misc/OutputDevice.h"
#include "Logging/LogVerbosity.h"
#include "Misc/VarArgs.h"
#include "HAL/PlatformAtomics.h"
#include "GenericPlatform/GenericPlatformAtomics.h"
#include "Templates/UnrealTypeTraits.h"
#include "Templates/AndOrNot.h"
#include "Templates/IsArithmetic.h"
#include "Templates/IsPODType.h"
#include "Templates/IsTriviallyCopyConstructible.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/EnableIf.h"
#include "Templates/RemoveReference.h"
#include "Templates/TypeCompatibleBytes.h"
#include "Templates/IntegralConstant.h"
#include "Templates/IsClass.h"
#include "Traits/IsContiguousContainer.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "HAL/PlatformMath.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "Templates/MemoryOps.h"
#include "Templates/IsTriviallyCopyAssignable.h"
#include "Math/NumericLimits.h"
#include "Serialization/Archive.h"
#include "Templates/IsEnumClass.h"
#include "HAL/PlatformProperties.h"
#include "GenericPlatform/GenericPlatformProperties.h"
#include "Misc/Compression.h"
#include "Misc/EngineVersionBase.h"
#include "Internationalization/TextNamespaceFwd.h"
#include "Templates/Less.h"
#include "Templates/Sorting.h"
#include "Containers/UnrealString.h"
#include "Misc/CString.h"
#include "Misc/Char.h"
#include "HAL/PlatformString.h"
#include "GenericPlatform/GenericPlatformStricmp.h"
#include "GenericPlatform/GenericPlatformString.h"
#include "Misc/Crc.h"
#include "Math/UnrealMathUtility.h"
#include "Containers/Map.h"
#include "Misc/StructBuilder.h"
#include "Templates/AlignmentTemplates.h"
#include "Templates/Function.h"
#include "Templates/Decay.h"
#include "Templates/Invoke.h"
#include "Templates/PointerIsConvertibleFromTo.h"
#include "Containers/Set.h"
#include "Templates/TypeHash.h"
#include "Containers/SparseArray.h"
#include "Containers/ScriptArray.h"
#include "Containers/BitArray.h"
#include "Algo/Reverse.h"
#include "Math/Color.h"
#include "Misc/Parse.h"
#include "Math/IntPoint.h"
#include "Math/Vector2D.h"
#include "Logging/LogMacros.h"
#include "Logging/LogCategory.h"
#include "UObject/NameTypes.h"
#include "HAL/CriticalSection.h"
#include "Misc/Timespan.h"
#include "Containers/StringConv.h"
#include "UObject/UnrealNames.h"
#include "Math/Vector.h"
#include "Misc/ByteSwap.h"
#include "Internationalization/Text.h"
#include "Containers/EnumAsByte.h"
#include "Templates/SharedPointer.h"
#include "CoreGlobals.h"
#include "HAL/PlatformTLS.h"
#include "GenericPlatform/GenericPlatformTLS.h"
#include "Internationalization/CulturePointer.h"
#include "Internationalization/TextLocalizationManager.h"
#include "Delegates/Delegate.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Delegates/MulticastDelegateBase.h"
#include "Delegates/IDelegateInstance.h"
#include "Delegates/DelegateSettings.h"
#include "Delegates/DelegateBase.h"
#include "Delegates/IntegerSequence.h"
#include "Templates/Tuple.h"
#include "UObject/ScriptDelegates.h"
#include "Misc/Optional.h"
#include "Templates/UniquePtr.h"
#include "Templates/IsArray.h"
#include "Templates/RemoveExtent.h"
#include "Internationalization/Internationalization.h"
#include "Templates/UniqueObj.h"
#include "Math/IntVector.h"
#include "Math/Rotator.h"
#include "Math/VectorRegister.h"
#include "CoreMinimal.h"
#include "Math/UnrealMath.h"
#include "Templates/IsFloatingPoint.h"
#include "Templates/IsIntegral.h"
#include "Templates/IsSigned.h"
#include "Templates/Greater.h"
#include "Math/ColorList.h"
#include "Math/IntRect.h"
#include "Math/Edge.h"
#include "Math/CapsuleShape.h"
#include "Math/RangeSet.h"
#include "Math/Range.h"
#include "Misc/DateTime.h"
#include "Math/RangeBound.h"
#include "Math/Box2D.h"
#include "Math/BoxSphereBounds.h"
#include "Math/Sphere.h"
#include "Math/Box.h"
#include "Math/OrientedBox.h"
#include "Math/Interval.h"
#include "Math/RotationAboutPointMatrix.h"
#include "Math/Matrix.h"
#include "Math/Vector4.h"
#include "Math/Plane.h"
#include "UObject/ObjectVersion.h"
#include "Math/Axis.h"
#include "Math/RotationTranslationMatrix.h"
#include "Math/ScaleRotationTranslationMatrix.h"
#include "Math/RotationMatrix.h"
#include "Math/PerspectiveMatrix.h"
#include "Math/OrthoMatrix.h"
#include "Math/TranslationMatrix.h"
#include "Math/QuatRotationTranslationMatrix.h"
#include "Math/Quat.h"
#include "Math/InverseRotationMatrix.h"
#include "Math/ScaleMatrix.h"
#include "Math/MirrorMatrix.h"
#include "Math/ClipProjectionMatrix.h"
#include "Math/InterpCurve.h"
#include "Math/TwoVectors.h"
#include "Math/InterpCurvePoint.h"
#include "Math/CurveEdInterface.h"
#include "Math/Float16Color.h"
#include "Math/Float16.h"
#include "Math/Float32.h"
#include "Math/Vector2DHalf.h"
#include "Math/Transform.h"
#include "Math/ConvexHull2d.h"
#include "Misc/EnumClassFlags.h"
#include "HAL/PlatformTime.h"
#include "GenericPlatform/GenericPlatformTime.h"
#include "HAL/ThreadSingleton.h"
#include "HAL/TlsAutoCleanup.h"
#include "Stats/Stats.h"
#include "HAL/ThreadSafeCounter.h"
#include "Containers/LockFreeList.h"
#include "Misc/NoopCounter.h"
#include "Containers/ChunkedArray.h"
#include "Containers/IndirectArray.h"
#include "Misc/OutputDeviceRedirector.h"
#include "ProfilingDebugging/ResourceSize.h"
#include "Misc/Guid.h"
#include "Math/RandomStream.h"
#include "HAL/Event.h"
#include "Misc/Attribute.h"
#include "Misc/CoreMisc.h"
#include "Modules/ModuleInterface.h"
#include "HAL/PlatformProcess.h"
#include "GenericPlatform/GenericPlatformProcess.h"
#include "Containers/LockFreeFixedSizeAllocator.h"
#include "Misc/MemStack.h"
#include "Containers/List.h"
#include "GenericPlatform/GenericApplication.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"
#include "GenericPlatform/GenericWindowDefinition.h"
#include "GenericPlatform/GenericWindow.h"
#include "GenericPlatform/ICursor.h"
#include "HAL/IConsoleManager.h"
#include "Async/AsyncWork.h"
#include "Misc/IQueuedWork.h"
#include "Misc/QueuedThreadPool.h"
#include "GenericPlatform/GenericPlatformAffinity.h"
#include "Math/TransformCalculus.h"
#include "Math/TransformCalculus2D.h"
#include "Templates/RefCounting.h"
#include "Serialization/BitReader.h"
#include "Misc/NetworkGuid.h"
#include "Serialization/BitWriter.h"
#include "Async/TaskGraphInterfaces.h"
#include "Serialization/BufferReader.h"
#include "Misc/SecureHash.h"
#include "Containers/StaticArray.h"
#include "Misc/Paths.h"
#include "Misc/CoreStats.h"
#include "Templates/ScopedCallback.h"
#include "Serialization/CustomVersion.h"
#include "Misc/OutputDeviceError.h"
#include "Misc/ObjectThumbnail.h"
#include "Internationalization/InternationalizationMetadata.h"
#include "Misc/EngineVersion.h"
#include "Internationalization/GatherableTextData.h"
#include "Serialization/ArchiveProxy.h"
#include "UObject/DebugSerializationFlags.h"
#include "Containers/ResourceArray.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "Modules/ModuleManager.h"
#include "Modules/Boilerplate/ModuleBoilerplate.h"
#include "Misc/ScopeLock.h"
#include "Misc/BufferedOutputDevice.h"
#include "Async/Future.h"
#include "Math/SHMath.h"
#include "Misc/ScopedEvent.h"
#include "Features/IModularFeature.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/FileManager.h"
#include "Misc/SlowTask.h"
#include "Logging/TokenizedMessage.h"
#include "Misc/FeedbackContext.h"
#include "Misc/SlowTaskStack.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CommandLine.h"
#include "HAL/ThreadSafeBool.h"
#include "Features/IModularFeatures.h"
#include "Misc/App.h"
#include "Containers/ArrayView.h"
#include "Misc/MessageDialog.h"
#include "Containers/Ticker.h"
#include "Misc/FileHelper.h"
#include "ProfilingDebugging/ProfilingHelpers.h"
#include "Misc/IFilter.h"
#include "Misc/FilterCollection.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryArchive.h"
#include "Logging/MessageLog.h"
#include "UObject/PropertyPortFlags.h"
#include "HAL/Runnable.h"
#include "Misc/ScopedSlowTask.h"
#include "Internationalization/InternationalizationManifest.h"
#include "Internationalization/InternationalizationArchive.h"

// From Json:
#include "Serialization/JsonTypes.h"
#include "Serialization/JsonWriter.h"
#include "Policies/PrettyJsonPrintPolicy.h"
#include "Policies/JsonPrintPolicy.h"

// From CoreUObject:
#include "UObject/ObjectMacros.h"
#include "UObject/Script.h"
#include "UObject/Object.h"
#include "UObject/UObjectBaseUtility.h"
#include "UObject/UObjectBase.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectArray.h"
#include "UObject/UObjectMarks.h"
#include "UObject/Class.h"
#include "UObject/GarbageCollection.h"
#include "UObject/GeneratedCppIncludes.h"
#include "Serialization/ArchiveUObject.h"
#include "UObject/CoreNative.h"
#include "UObject/WeakObjectPtr.h"
#include "Templates/Casts.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/Interface.h"
#include "Templates/SubclassOf.h"
#include "UObject/LazyObjectPtr.h"
#include "Misc/WorldCompositionUtility.h"
#include "UObject/CoreNetTypes.h"
#include "UObject/ScriptInterface.h"
#include "UObject/UnrealType.h"
#include "UObject/PropertyTag.h"
#include "Serialization/SerializedPropertyScope.h"
#include "UObject/CoreNet.h"
#include "UObject/GCObject.h"
#include "UObject/ScriptMacros.h"
#include "UObject/Stack.h"
#include "UObject/Package.h"
#include "Misc/PackageName.h"
#include "UObject/ObjectResource.h"
#include "UObject/Linker.h"
#include "UObject/PackageFileSummary.h"
#include "UObject/LinkerLoad.h"
#include "UObject/ObjectRedirector.h"
#include "Serialization/BulkData.h"
#include "UObject/UObjectAnnotation.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "Misc/AssetRegistryInterface.h"
#include "Misc/NotifyHook.h"
#include "UObject/StructOnScope.h"

// From InputCore:
#include "InputCoreTypes.h"

// From SlateCore:
#include "Types/SlateEnums.h"
#include "Layout/Visibility.h"
#include "Styling/SlateColor.h"
#include "Styling/WidgetStyle.h"
#include "Layout/SlateRect.h"
#include "Rendering/SlateLayoutTransform.h"
#include "Layout/Geometry.h"
#include "Layout/PaintGeometry.h"
#include "Rendering/SlateRenderTransform.h"
#include "Input/CursorReply.h"
#include "Input/ReplyBase.h"
#include "Input/Reply.h"
#include "Input/Events.h"
#include "Input/DragAndDrop.h"
#include "Input/NavigationReply.h"
#include "Input/PopupMethodReply.h"
#include "Types/ISlateMetaData.h"
#include "Widgets/SWidget.h"
#include "Layout/ArrangedWidget.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "Layout/LayoutGeometry.h"
#include "Layout/Margin.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SNullWidget.h"
#include "SlotBase.h"
#include "Layout/Children.h"
#include "Widgets/SPanel.h"
#include "Widgets/SOverlay.h"
#include "Fonts/CompositeFont.h"
#include "Fonts/SlateFontInfo.h"
#include "Styling/SlateBrush.h"
#include "Sound/SlateSound.h"
#include "Styling/SlateWidgetStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/ISlateStyle.h"
#include "Styling/StyleDefaults.h"
#include "Brushes/SlateNoResource.h"
#include "Styling/CoreStyle.h"
#include "Widgets/SCompoundWidget.h"
#include "Types/SlateStructs.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Styling/SlateWidgetStyleContainerBase.h"
#include "Styling/SlateWidgetStyleContainerInterface.h"
#include "Widgets/SBoxPanel.h"
#include "Animation/CurveSequence.h"
#include "Animation/CurveHandle.h"
#include "Rendering/RenderingCommon.h"
#include "Widgets/SWindow.h"
#include "Textures/SlateIcon.h"
#include "Fonts/ShapedTextFwd.h"
#include "Widgets/IToolTip.h"
#include "SlateGlobals.h"
#include "Textures/SlateShaderResource.h"
#include "Textures/SlateTextureData.h"
#include "Rendering/DrawElements.h"
#include "Types/PaintArgs.h"
#include "Rendering/ShaderResourceManager.h"
#include "Widgets/SLeafWidget.h"
#include "Brushes/SlateDynamicImageBrush.h"
#include "Application/ThrottleManager.h"
#include "Layout/ArrangedChildren.h"
#include "Application/SlateWindowHelper.h"
#include "Rendering/SlateRenderer.h"
#include "Application/SlateApplicationBase.h"
#include "Layout/WidgetPath.h"
#include "Types/SlateConstants.h"
#include "Fonts/FontProviderInterface.h"

// From Slate:
#include "SlateFwd.h"
#include "Framework/SlateDelegates.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Widgets/Layout/SBorder.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Application/IMenu.h"
#include "Framework/Commands/InputBindingManager.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Text/TextLayout.h"
#include "Framework/Text/TextRunRenderer.h"
#include "Framework/Text/TextLineHighlight.h"
#include "Framework/Text/IRun.h"
#include "Framework/Text/ShapedTextCacheFwd.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Application/MenuStack.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Docking/WorkspaceItem.h"
#include "Framework/Views/ITypedTableView.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Input/SButton.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Widgets/SToolTip.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Framework/Layout/InertialScrollManager.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Images/SImage.h"
#include "Framework/Layout/Overscroll.h"
#include "Widgets/Views/STableViewBase.h"
#include "Framework/Layout/IScrollableWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Framework/Views/TableViewTypeTraits.h"
#include "Widgets/Views/SListView.h"
#include "Framework/Docking/LayoutService.h"
#include "Widgets/Input/IVirtualKeyboardEntry.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Input/SEditableText.h"

// From RHI:
#include "RHIDefinitions.h"
#include "RHI.h"
#include "RHIStaticStates.h"

// From RenderCore:
#include "RenderCommandFence.h"
#include "RenderResource.h"
#include "RenderingThread.h"
#include "UniformBuffer.h"
#include "PackedNormal.h"
#include "RenderUtils.h"
#include "ShaderParameters.h"
#include "ShaderCore.h"
#include "Shader.h"
#include "VertexFactory.h"

// From Messaging:
#include "IMessageContext.h"

// From AssetRegistry:
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/ARFilter.h"

// From Internationalization:
#include "LocTextHelper.h"

// From Engine:
#include "EngineLogs.h"
#include "Engine/EngineTypes.h"
#include "Engine/NetSerialization.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "Engine/EngineBaseTypes.h"
#include "Interfaces/Interface_AssetUserData.h"
#include "PixelFormat.h"
#include "Engine/MaterialMerging.h"
#include "Engine/Level.h"
#include "GameFramework/Actor.h"
#include "Components/ActorComponent.h"
#include "Components/SceneComponent.h"
#include "EngineDefines.h"
#include "CollisionQueryParams.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintCore.h"
#include "AI/Navigation/NavigationTypes.h"
#include "Engine/LatentActionManager.h"
#include "Engine/World.h"
#include "WorldCollision.h"
#include "GameFramework/Pawn.h"
#include "AI/Navigation/NavAgentInterface.h"
#include "Engine/PendingNetGame.h"
#include "Engine/GameInstance.h"
#include "Components.h"
#include "SceneTypes.h"
#include "Interfaces/Interface_CollisionDataProvider.h"
#include "Engine/BlendableInterface.h"
#include "HitProxies.h"
#include "RawIndexBuffer.h"
#include "Engine/StaticMesh.h"
#include "Engine/Brush.h"
#include "Engine/Engine.h"
#include "LocalVertexFactory.h"
#include "Model.h"
#include "Engine/Scene.h"
#include "TimerManager.h"
#include "UnrealClient.h"
#include "ShowFlags.h"
#include "Engine/GameViewportClient.h"
#include "Engine/ScriptViewportClient.h"
#include "Engine/ViewportSplitScreen.h"
#include "Engine/TitleSafeZone.h"
#include "Engine/GameViewportDelegates.h"
#include "Engine/DebugDisplayProperty.h"
#include "ConvexVolume.h"
#include "SceneView.h"
#include "SceneInterface.h"
#include "FinalPostProcessSettings.h"
#include "BlendableManager.h"
#include "DebugViewModeHelpers.h"
#include "PrimitiveUniformShaderParameters.h"
#include "MaterialShared.h"
#include "StaticParameterSet.h"
#include "Materials/MaterialInterface.h"
#include "Engine/Texture.h"
#include "Engine/TextureDefines.h"
#include "Curves/RichCurve.h"
#include "Curves/KeyHandle.h"
#include "Curves/IndexedCurve.h"
#include "TextureResource.h"
#include "BatchedElements.h"
#include "StaticBoundShaderState.h"
#include "PhysxUserData.h"
#include "Engine/TextureStreamingTypes.h"
#include "Engine/Texture2D.h"
#include "PhysicsEngine/BodyInstance.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "Components/PrimitiveComponent.h"
#include "AI/Navigation/NavRelevantInterface.h"
#include "Tickable.h"
#include "Engine/TextureLightProfile.h"
#include "MeshBatch.h"
#include "SceneUtils.h"
#include "SceneManagement.h"
#include "ReferenceSkeleton.h"
#include "BoneIndices.h"
#include "Animation/AnimTypes.h"
#include "Animation/AnimLinkableElement.h"
#include "Animation/PreviewAssetAttachComponent.h"
#include "BoneContainer.h"
#include "GPUSkinPublicDefs.h"
#include "Engine/SkeletalMesh.h"
#include "EngineGlobals.h"
#include "Curves/CurveBase.h"
#include "Curves/CurveOwnerInterface.h"
#include "Camera/CameraTypes.h"
#include "Curves/CurveFloat.h"
#include "EditorFramework/AssetImportData.h"
#include "Animation/Skeleton.h"
#include "Animation/SmartName.h"
#include "Audio.h"
#include "Sound/SoundAttenuation.h"
#include "IAudioExtensionPlugin.h"
#include "Components/MeshComponent.h"
#include "Animation/AnimationAsset.h"
#include "AnimInterpFilter.h"
#include "Animation/AnimCurveTypes.h"
#include "Engine/Selection.h"
#include "GameFramework/Volume.h"
#include "EdGraph/EdGraph.h"
#include "BlueprintUtilities.h"
#include "EdGraph/EdGraphSchema.h"
#include "Commandlets/Commandlet.h"
#include "GameFramework/Info.h"
#include "GameFramework/WorldSettings.h"
#include "GameFramework/DamageType.h"
#include "Sound/AudioVolume.h"
#include "Materials/Material.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialFunction.h"
#include "Animation/AnimSequenceBase.h"
#include "Camera/CameraComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Components/SkinnedMeshComponent.h"
#include "Animation/AnimSequence.h"
#include "Components/SkeletalMeshComponent.h"
#include "ClothSimData.h"
#include "SingleAnimationPlayData.h"
#include "EngineUtils.h"
#include "Engine/Polys.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Animation/SkeletalMeshActor.h"
#include "Engine/LevelStreaming.h"
#include "LatentActions.h"
#include "PreviewScene.h"
#include "BodySetupEnums.h"
#include "Sound/SoundConcurrency.h"
#include "AlphaBlend.h"
#include "Engine/Font.h"
#include "Engine/FontImportOptions.h"
#include "Animation/AnimBlueprint.h"
#include "EngineModule.h"
#include "AI/NavigationSystemBase.h"
#include "Math/GenericOctreePublic.h"
#include "CanvasTypes.h"
#include "Sound/SoundBase.h"
#include "Components/LightComponent.h"
#include "Components/LightComponentBase.h"

// From BlueprintGraph:
#include "BlueprintNodeSignature.h"
#include "K2Node.h"
#include "K2Node_EditablePinBase.h"
#include "EdGraphSchema_K2.h"

// From UnrealEd:
#include "Editor/UnrealEdTypes.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Viewports.h"
#include "Editor/EditorEngine.h"
#include "Settings/LevelEditorPlaySettings.h"
#include "Editor.h"
#include "Factories/Factory.h"
#include "TickableEditorObject.h"
#include "UnrealWidgetFwd.h"
#include "EditorComponents.h"
#include "Toolkits/IToolkit.h"
#include "AssetThumbnail.h"
#include "EditorViewportClient.h"
#include "Factories/FbxAssetImportData.h"
#include "EditorUndoClient.h"
#include "Toolkits/IToolkitHost.h"
#include "Factories/FbxMeshImportData.h"
#include "ComponentVisualizer.h"
#include "Factories/ImportSettings.h"
#include "Editor/UnrealEdEngine.h"
#include "IPackageAutoSaver.h"
#include "ComponentVisualizerManager.h"

#include "Factories/FbxAnimSequenceImportData.h"
#include "Factories/FbxSceneImportFactory.h"
#include "EditorModeManager.h"
#include "UnrealEdGlobals.h"
#include "ScopedTransaction.h"
#include "Factories/FbxStaticMeshImportData.h"
#include "FileHelpers.h"
#include "Factories/FbxImportUI.h"
#include "ThumbnailRendering/ThumbnailRenderer.h"
#include "FbxImporter.h"
#include "Toolkits/BaseToolkit.h"
#include "EditorModes.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "LevelEditorViewport.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "EditorModeRegistry.h"
#include "ObjectTools.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "EdMode.h"
#include "PackageTools.h"
#include "Commandlets/GatherTextCommandletBase.h"
#include "Dialogs/Dialogs.h"

