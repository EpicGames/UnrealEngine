// Copyright Epic Games, Inc. All Rights Reserved.


#include "CoreMinimal.h"
#include "EngineDefines.h"
#include "Misc/MessageDialog.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"
#include "UObject/StrongObjectPtr.h"
#include "InputCoreTypes.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "GameFramework/Actor.h"
#include "RawIndexBuffer.h"
#include "Model.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "Builders/CubeBuilder.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Settings/LevelEditorMiscSettings.h"
#include "Engine/Brush.h"
#include "Engine/GameViewportClient.h"
#include "AssetRegistry/AssetData.h"
#include "Editor/EditorEngine.h"
#include "ISourceControlModule.h"
#include "Editor/UnrealEdEngine.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "EditorFramework/AssetImportData.h"
#include "Animation/SkeletalMeshActor.h"
#include "Components/CapsuleComponent.h"
#include "Components/SphereComponent.h"
#include "Components/BoxComponent.h"
#include "Components/PointLightComponent.h"
#include "Engine/StaticMeshActor.h"
#include "Components/BrushComponent.h"
#include "PhysicsEngine/RadialForceComponent.h"
#include "Engine/Polys.h"
#include "Engine/Selection.h"
#include "Editor.h"
#include "LevelEditorViewport.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "EditorDirectories.h"
#include "FileHelpers.h"
#include "UnrealEdGlobals.h"
#include "UObject/UObjectIterator.h"
#include "StaticMeshResources.h"
#include "EditorSupportDelegates.h"
#include "BusyCursor.h"
#include "ScopedTransaction.h"
#include "LevelUtils.h"
#include "ObjectTools.h"
#include "PackageTools.h"
#include "Interfaces/IMainFrameModule.h"
#include "EditorLevelUtils.h"
#include "EditorBuildUtils.h"
#include "ScriptDisassembler.h"
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "FbxExporter.h"
#include "DesktopPlatformModule.h"
#include "Elements/Actor/ActorElementData.h"
#include "Elements/Framework/TypedElementList.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Framework/TypedElementCommonActions.h"
#include "Elements/Interfaces/TypedElementWorldInterface.h"
#include "SnappingUtils.h"
#include "AssetSelection.h"
#include "HighResScreenshot.h"
#include "ActorEditorUtils.h"
#include "Editor/ActorPositioning.h"
#include "LandscapeInfo.h"
#include "LandscapeInfoMap.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Logging/LogScopedCategoryAndVerbosityOverride.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FeedbackContext.h"
#include "EngineUtils.h"
#include "AutoReimport/AssetSourceFilenameCache.h"
#if PLATFORM_WINDOWS
	#include "Windows/WindowsHWrapper.h"
#endif
#include "ActorGroupingUtils.h"
#include "EdMode.h"
#include "ILevelEditor.h"
#include "Subsystems/BrushEditingSubsystem.h"
#include "Subsystems/EditorActorSubsystem.h"
#include "Elements/Framework/TypedElementCommonActions.h"
#include "HAL/PlatformApplicationMisc.h"

DEFINE_LOG_CATEGORY_STATIC(LogUnrealEdSrv, Log, All);

#define LOCTEXT_NAMESPACE "UnrealEdSrv"

namespace UnrealEdSrvLocals
{
	UTypedElementSelectionSet* GetLevelEditorSelectionSet()
	{
		if (GCurrentLevelEditingViewportClient)
		{
			if (TSharedPtr<ILevelEditor> LevelEditor = GCurrentLevelEditingViewportClient->ParentLevelEditor.Pin())
			{
				return LevelEditor->GetMutableElementSelectionSet();
			}
		}
		return nullptr;
	}
	UTypedElementCommonActions* GetLevelEditorCommonActions()
	{
		if (GCurrentLevelEditingViewportClient)
		{
			if (TSharedPtr<ILevelEditor> LevelEditor = GCurrentLevelEditingViewportClient->ParentLevelEditor.Pin())
			{
				return LevelEditor->GetCommonActions();
			}
		}
		return nullptr;
	}
}

/**
 * Dumps a set of selected objects to debugf.
 */
static void PrivateDumpSelection(USelection* Selection)
{
	for ( FSelectionIterator Itor(*Selection) ; Itor ; ++Itor )
	{
		UObject *CurObject = *Itor;
		if ( CurObject )
		{
			UE_LOGF(LogUnrealEdSrv, Log, "    %ls", *CurObject->GetClass()->GetName() );
		}
		else
		{
			UE_LOGF(LogUnrealEdSrv, Log, "    NULL object");
		}
	}
}

class SModalWindowTest : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SModalWindowTest ){}
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs )
	{
		this->ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew( SVerticalBox )
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew( SHorizontalBox )
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Center)
					[
						SNew( STextBlock )
						.Text( LOCTEXT("ModelTestWindowLabel", "This is a modal window test") )
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Center)
					[
						SNew( SButton )
						.Text( LOCTEXT("NewModalTestWindowButtonLabel", "New Modal Window") )
						.OnClicked( this, &SModalWindowTest::OnNewModalWindowClicked )
					]
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Right)
				[
					SNew( SHorizontalBox )
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew( SButton )
						.Text( NSLOCTEXT("UnrealEd", "OK", "OK") )
						.OnClicked( this, &SModalWindowTest::OnOKClicked )
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew( SButton )
						.Text( NSLOCTEXT("UnrealEd", "Cancel", "Cancel") )
						.OnClicked( this, &SModalWindowTest::OnCancelClicked )
					]
				]
			]
		];
	}
	
	SModalWindowTest()
		: bUserResponse( false )
	{
	}

	void SetWindow( TSharedPtr<SWindow> InWindow )
	{
		MyWindow = InWindow;
	}

	bool GetResponse() const { return bUserResponse; }

private:
	
	FReply OnOKClicked()
	{
		bUserResponse = true;
		MyWindow->RequestDestroyWindow();
		return FReply::Handled();
	}

	FReply OnCancelClicked()
	{
		bUserResponse = false;
		MyWindow->RequestDestroyWindow();
		return FReply::Handled();
	}

	FReply OnNewModalWindowClicked()
	{
		TSharedRef<SModalWindowTest> ModalWindowContent = SNew(SModalWindowTest);
		TSharedRef<SWindow> ModalWindow = SNew(SWindow)
			.Title( LOCTEXT("TestModalWindowTitle", "Modal Window") )
			.ClientSize(FVector2D(250,100))
			[
				ModalWindowContent
			];

		ModalWindowContent->SetWindow( ModalWindow );

		FSlateApplication::Get().AddModalWindow( ModalWindow, AsShared() );

		UE_LOGF(LogUnrealEdSrv, Log, "Modal Window Returned");

		return FReply::Handled();
	}

	FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
	{
		if( MouseEvent.GetEffectingButton() == EKeys::RightMouseButton )
		{
			struct Local
			{
				static void FillSubMenuEntries( FMenuBuilder& MenuBuilder )
				{
					MenuBuilder.AddMenuEntry( LOCTEXT("TestItem2", "Test Item 2"), LOCTEXT("TestToolTip", "TestToolTip"), FSlateIcon(), FUIAction() );

					MenuBuilder.AddMenuEntry( LOCTEXT("TestItem3", "Test Item 3"), LOCTEXT("TestToolTip", "TestToolTip"), FSlateIcon(), FUIAction() );

					MenuBuilder.AddSubMenu( LOCTEXT("SubMenu", "Sub Menu"), LOCTEXT("OpensASubmenu", "Opens a submenu"), FNewMenuDelegate::CreateStatic( &Local::FillSubMenuEntries ) );

					MenuBuilder.AddSubMenu( LOCTEXT("SubMenu2", "Sub Menu2"), LOCTEXT("OpensASubmenu", "Opens a submenu"), FNewMenuDelegate::CreateStatic( &Local::FillSubMenuEntries ) );
				}
			};

			FMenuBuilder NewMenu( true, NULL );
			NewMenu.BeginSection("TestMenuModalWindow", LOCTEXT("MenuInAModalWindow", "Menu in a modal window") );
			{
				NewMenu.AddMenuEntry( LOCTEXT("TestItem1", "Test Item 1"), FText::GetEmpty(), FSlateIcon(), FUIAction() );
				NewMenu.AddSubMenu( LOCTEXT("SubMenu", "Sub Menu"), LOCTEXT("OpenASubmenu", "Opens a sub menu"), FNewMenuDelegate::CreateStatic( &Local::FillSubMenuEntries ) );
			}
			NewMenu.EndSection();

			FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
			FSlateApplication::Get().PushMenu(SharedThis(this), WidgetPath, NewMenu.MakeWidget(), MouseEvent.GetScreenSpacePosition(), FPopupTransitionEffect(FPopupTransitionEffect::None));

			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	TSharedPtr<SWindow> MyWindow;
	bool bUserResponse;
};

UPackage* UUnrealEdEngine::GeneratePackageThumbnailsIfRequired( const TCHAR* Str, FOutputDevice& Ar, TArray<FString>& GeneratedThumbNamesList )
{
	UPackage* Pkg = NULL;
	if( FParse::Command( &Str, TEXT( "SavePackage" ) ) )
	{
		FString TempFname;
		if( FParse::Value( Str, TEXT( "FILE=" ), TempFname ) && ParseObject<UPackage>( Str, TEXT( "Package=" ), Pkg, NULL ) )
		{
			if (Pkg == nullptr)
			{
				return nullptr;
			}

			// Update any thumbnails for objects in this package that were modified or generate
			// new thumbnails for objects that don't have any

			bool bSilent = false;
			FParse::Bool( Str, TEXT( "SILENT=" ), bSilent );

			// Make a list of packages to query (in our case, just the package we're saving)
			TArray< UPackage* > Packages;
			Packages.Add( Pkg );

			// Allocate a new thumbnail map if we need one
			if( !Pkg->HasThumbnailMap() )
			{
				Pkg->SetThumbnailMap(MakeUnique<FThumbnailMap>());
			}

			// OK, now query all of the browsable objects in the package we're about to save
			TArray< UObject* > BrowsableObjectsInPackage;

			// Load the asset tools module to get access to thumbnail tools
			FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
				
			// NOTE: The package should really be fully loaded before we try to generate thumbnails
			UPackageTools::GetObjectsInPackages(
				&Packages,														// Packages to search
				BrowsableObjectsInPackage );									// Out: Objects

			// Check to see if any of the objects need thumbnails generated
			TSet< UObject* > ObjectsMissingThumbnails;
			TSet< UObject* > ObjectsWithThumbnails;
			for( int32 CurObjectIndex = 0; CurObjectIndex < BrowsableObjectsInPackage.Num(); ++CurObjectIndex )
			{
				UObject* CurObject = BrowsableObjectsInPackage[ CurObjectIndex ];
				check( CurObject != NULL );

				bool bUsesGenericThumbnail = AssetToolsModule.Get().AssetUsesGenericThumbnail(FAssetData(CurObject));

				// Archetypes always use a shared thumbnail
				if( CurObject->HasAllFlags( RF_ArchetypeObject ) )
				{
					bUsesGenericThumbnail = true;
				}

				bool bPrintThumbnailDiagnostics = false;
				GConfig->GetBool(TEXT("Thumbnails"), TEXT("Debug"), bPrintThumbnailDiagnostics, GEditorPerProjectIni);

				const FObjectThumbnail* ExistingThumbnail = ThumbnailTools::FindCachedThumbnail( CurObject->GetFullName() );
				if (bPrintThumbnailDiagnostics)
				{
					UE_LOGF(LogUnrealEdSrv, Log, "Saving Thumb for %ls", *CurObject->GetFullName());
					UE_LOGF(LogUnrealEdSrv, Log, "   Thumb existed = %d", (ExistingThumbnail!=NULL) ? 1: 0);
					UE_LOGF(LogUnrealEdSrv, Log, "   Shared Thumb = %d", (bUsesGenericThumbnail) ? 1: 0);
				}
				//if it's not generatable, let's make sure it doesn't have a custom thumbnail before saving
				if (!ExistingThumbnail && bUsesGenericThumbnail)
				{
					//let it load the custom icons from disk
					// @todo CB: Batch up requests for multiple thumbnails!
					TArray< FName > ObjectFullNames;
					FName ObjectFullNameFName( *CurObject->GetFullName() );
					ObjectFullNames.Add( ObjectFullNameFName );

					// Load thumbnails
					FThumbnailMap& LoadedThumbnails = Pkg->AccessThumbnailMap();
					if( ThumbnailTools::ConditionallyLoadThumbnailsForObjects( ObjectFullNames, LoadedThumbnails ) )
					{
						//store off the names of the thumbnails that were loaded as part of a save so we can delete them after the save
						GeneratedThumbNamesList.Add(ObjectFullNameFName.ToString());

						if (bPrintThumbnailDiagnostics)
						{
							UE_LOGF(LogUnrealEdSrv, Log, "   Unloaded thumb loaded successfully");
						}

						ExistingThumbnail = LoadedThumbnails.Find( ObjectFullNameFName );
						if (bPrintThumbnailDiagnostics)
						{
							UE_LOGF(LogUnrealEdSrv, Log, "   Newly loaded thumb exists = %d", (ExistingThumbnail!=NULL) ? 1: 0);
							if (ExistingThumbnail)
							{
								UE_LOGF(LogUnrealEdSrv, Log, "   Thumb created after proper version = %d", (ExistingThumbnail->IsCreatedAfterCustomThumbsEnabled()) ? 1: 0);
							}
						}

						if (ExistingThumbnail && !ExistingThumbnail->IsCreatedAfterCustomThumbsEnabled())
						{
							if (bPrintThumbnailDiagnostics)
							{
								UE_LOGF(LogUnrealEdSrv, Log,  "   WIPING OUT THUMBNAIL!!!!");
							}

							//Casting away const to save memory behind the scenes
							FObjectThumbnail* ThumbToClear = (FObjectThumbnail*)ExistingThumbnail;
							ThumbToClear->SetImageSize(0, 0);
							ThumbToClear->AccessImageData().Empty();
						}
					}
					else
					{
						if (bPrintThumbnailDiagnostics)
						{
							UE_LOGF(LogUnrealEdSrv, Log, "   Unloaded thumb does not exist");
						}
					}
				}

				if ( bUsesGenericThumbnail )
				{
					// This is a generic thumbnail object, but it may have a custom thumbnail.
					if( ExistingThumbnail != NULL && !ExistingThumbnail->IsEmpty() )
					{
						ObjectsWithThumbnails.Add( CurObject );
					}
				}
				else
				{
					// This is not a generic thumbnail object, so if it is dirty or missing we will render it.
					if( ExistingThumbnail != NULL && !ExistingThumbnail->IsEmpty() && !ExistingThumbnail->IsDirty() )
					{
						ObjectsWithThumbnails.Add( CurObject );
					}
					else
					{
						ObjectsMissingThumbnails.Add( CurObject );
					}
				}
			}


			if( BrowsableObjectsInPackage.Num() > 0 )
			{
				// Missing some thumbnails, so go ahead and try to generate them now

				// Start a busy cursor
				const FScopedBusyCursor BusyCursor;

				if( !bSilent )
				{
					const bool bWantProgressMeter = true;
					GWarn->BeginSlowTask( NSLOCTEXT("UnrealEd", "SavingPackage_GeneratingThumbnails", "Generating thumbnails..." ), bWantProgressMeter );
				}

				Ar.Logf( TEXT( "OBJ SavePackage: Generating thumbnails for [%i] asset(s) in package [%s] ([%i] browsable assets)..." ), ObjectsMissingThumbnails.Num(), *Pkg->GetName(), BrowsableObjectsInPackage.Num() );

				for( int32 CurObjectIndex = 0; CurObjectIndex < BrowsableObjectsInPackage.Num(); ++CurObjectIndex )
				{
					UObject* CurObject = BrowsableObjectsInPackage[ CurObjectIndex ];
					check( CurObject != NULL );

					if( !bSilent )
					{
						GWarn->UpdateProgress( CurObjectIndex, BrowsableObjectsInPackage.Num() );
					}


					bool bNeedEmptyThumbnail = false;
					if( ObjectsMissingThumbnails.Contains( CurObject ) && !GIsAutomationTesting )
					{
						// Generate a thumbnail!
						FObjectThumbnail* GeneratedThumbnail = ThumbnailTools::GenerateThumbnailForObjectToSaveToDisk( CurObject );
						if( GeneratedThumbnail != NULL )
						{
							Ar.Logf( TEXT( "OBJ SavePackage:     Rendered thumbnail for [%s]" ), *CurObject->GetFullName() );
						}
						else
						{
							// Couldn't generate a thumb; perhaps this object doesn't support thumbnails?
							bNeedEmptyThumbnail = true;
						}
					}
					else if( !ObjectsWithThumbnails.Contains( CurObject ) )
					{
						// Even though this object uses a shared thumbnail, we'll add a "dummy thumbnail" to
						// the package (zero dimension) for all browsable assets so that the Content Browser
						// can quickly verify that existence of assets on the fly.
						bNeedEmptyThumbnail = true;
					}


					// Create an empty thumbnail if we need to.  All browsable assets need at least a placeholder
					// thumbnail so the Content Browser can check for non-existent assets in the background
					if( bNeedEmptyThumbnail )
					{
						UPackage* MyOutermostPackage = CurObject->GetOutermost();
						ThumbnailTools::CacheEmptyThumbnail( CurObject->GetFullName(), MyOutermostPackage );
					}
				}

				Ar.Logf( TEXT( "OBJ SavePackage: Finished generating thumbnails for package [%s]" ), *Pkg->GetName() );

				if( !bSilent )
				{
					GWarn->UpdateProgress( 1, 1 );
					GWarn->EndSlowTask();
				}
			}
		}
	}
	return Pkg;
}

bool UUnrealEdEngine::HandleDumpModelGUIDCommand( const TCHAR* Str, FOutputDevice& Ar )
{
	for (TObjectIterator<UModel> It; It; ++It)
	{
		UE_LOGF(LogUnrealEdSrv, Log, "%ls Guid = '%ls'", *It->GetFullName(), *It->LightingGuid.ToString());
	}
return true;
}

bool UUnrealEdEngine::HandleModalTestCommand( const TCHAR* Str, FOutputDevice& Ar )
{
	TSharedRef<SModalWindowTest> MessageBox = SNew(SModalWindowTest);
	TSharedRef<SWindow> ModalWindow = SNew(SWindow)
		.Title( LOCTEXT("WindowTitle", "Modal Window") )
		.ClientSize(FVector2D(250,100))
		[
			MessageBox
		];

	MessageBox->SetWindow( ModalWindow );

	GEditor->EditorAddModalWindow( ModalWindow );

	UE_LOGF(LogUnrealEdSrv, Log,  "User response was: %ls", MessageBox->GetResponse() ? TEXT("OK") : TEXT("Cancel") );
	return true;
}

bool UUnrealEdEngine::HandleDisallowExportCommand(const TCHAR* Str, FOutputDevice& Ar)
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	TArray<FAssetData> SelectedAssets;
	ContentBrowserModule.Get().GetSelectedAssets(SelectedAssets);

	for (const FAssetData& AssetData : SelectedAssets)
	{
		UObject* Object = AssetData.GetAsset();
		if (Object)
		{
			UPackage* Package = Object->GetOutermost();
			Package->SetPackageFlags(EPackageFlags::PKG_DisallowExport);
			Package->MarkPackageDirty();
			UE_LOGF(LogUnrealEdSrv, Log, "Marked '%ls' as not exportable", *Object->GetName());
		}
	}

	return true;
}

bool UUnrealEdEngine::HandleDumpBPClassesCommand( const TCHAR* Str, FOutputDevice& Ar )
{
	UE_LOGF(LogUnrealEdSrv, Log, "--- Listing all blueprint generated classes ---");
	for( TObjectIterator<UClass> it; it; ++it )
	{
		const UClass* CurrentClass = *it;
		if( CurrentClass && CurrentClass->ClassGeneratedBy )
		{
			UE_LOGF(LogUnrealEdSrv, Log, "  %ls (%ls)", *CurrentClass->GetName(), *CurrentClass->GetOutermost()->GetName());
		}
	}
	return true;
}

bool UUnrealEdEngine::HandleFindOutdateInstancesCommand( const TCHAR* Str, FOutputDevice& Ar )
{
	UE_LOGF(LogUnrealEdSrv, Log, "--- Finding all actor instances with outdated classes ---");
	int32 NumFound = 0;
	for( TObjectIterator<UObject> it; it; ++it )
	{
		const UObject* CurrentObj = *it;
		if( CurrentObj->GetClass()->HasAnyClassFlags(CLASS_NewerVersionExists) )
		{
			UE_LOGF(LogUnrealEdSrv, Log, "  %ls (%ls)", *CurrentObj->GetName(), *CurrentObj->GetClass()->GetName());
			NumFound++;
		}
	}
	UE_LOGF(LogUnrealEdSrv, Log, "Found %d instance(s).", NumFound);
	return true;
}

bool UUnrealEdEngine::HandleDumpSelectionCommand( const TCHAR* Str, FOutputDevice& Ar )
{
	UE_LOGF(LogUnrealEdSrv, Log, "Selected Actors:");
	PrivateDumpSelection( GetSelectedActors() );
	UE_LOGF(LogUnrealEdSrv, Log, "Selected Non-Actors:");
	PrivateDumpSelection( GetSelectedObjects() );
	return true;
}


bool UUnrealEdEngine::HandleBuildLightingCommand( const TCHAR* Str, FOutputDevice& Ar, UWorld* InWorld )
{
	return FEditorBuildUtils::EditorBuild(InWorld, FBuildOptions::BuildLighting);	
}

bool UUnrealEdEngine::HandleBuildPathsCommand( const TCHAR* Str, FOutputDevice& Ar, UWorld* InWorld )
{
	return FEditorBuildUtils::EditorBuild(InWorld, FBuildOptions::BuildAIPaths);
}

bool UUnrealEdEngine::HandleRecreateLandscapeCollisionCommand(const TCHAR* Str, FOutputDevice& Ar, UWorld* InWorld)
{
	if (!PlayWorld && InWorld && InWorld->GetWorldSettings())
	{
		for (auto It = ULandscapeInfoMap::GetLandscapeInfoMap(InWorld).Map.CreateIterator(); It; ++It)
		{
			ULandscapeInfo* Info = It.Value();
			Info->RecreateCollisionComponents();
		}
	}
	return true;
}

bool UUnrealEdEngine::HandleRemoveLandscapeXYOffsetsCommand(const TCHAR* Str, FOutputDevice& Ar, UWorld* InWorld)
{
	if (!PlayWorld && InWorld && InWorld->GetWorldSettings())
	{
		for (auto It = ULandscapeInfoMap::GetLandscapeInfoMap(InWorld).Map.CreateIterator(); It; ++It)
		{
			ULandscapeInfo* Info = It.Value();
			Info->RemoveXYOffsets();
		}
	}
	return true;
}

bool UUnrealEdEngine::HandleDisasmScriptCommand(const TCHAR* Str, FOutputDevice& Ar)
{
	FString ClassName;

	if (FParse::Token(Str, ClassName, false))
	{
		FKismetBytecodeDisassembler::DisassembleAllFunctionsInClasses(Ar, ClassName);
	}

	return true;
}

#if UE_ALLOW_EXEC_COMMANDS
bool UUnrealEdEngine::Exec( UWorld* InWorld, const TCHAR* Stream, FOutputDevice& Ar )
{
	const TCHAR* Str = Stream;
	// disallow set commands in the editor as that modifies the default object, affecting object serialization
	if (FParse::Command(&Str, TEXT("SET")) || FParse::Command(&Str, TEXT("SETNOPEC")))
	{
		Ar.Logf(TEXT("Set commands not allowed in the editor"));
		return true;
	}

	//for thumbnail reclamation post save
	UPackage* Pkg = NULL;
	//thumbs that are loaded expressly for the sake of saving.  To be deleted again post-save
	TArray<FString> ThumbNamesToUnload;
	
	// Peek for the SavePackage command and generate thumbnails for the package if we need to
	// NOTE: The actual package saving happens in the UEditorEngine::Exec_Obj, but we do the 
	//		 thumbnail generation here in UnrealEd
	if( FParse::Command(&Str,TEXT("OBJ")) && !IsRunningCommandlet() )
	{
		Pkg = GeneratePackageThumbnailsIfRequired( Str, Ar, ThumbNamesToUnload );
	}

	// If we don't have a viewport specified to catch the stat commands, use to the active viewport.  If there is a game viewport ignore this as we do not want 
	if (GStatProcessingViewportClient == NULL && (GameViewport == NULL || GameViewport->IsSimulateInEditorViewport() ) )
	{
		GStatProcessingViewportClient = GLastKeyLevelEditingViewportClient ? GLastKeyLevelEditingViewportClient : GCurrentLevelEditingViewportClient;
	}

	bool bExecSucceeded = UEditorEngine::Exec( InWorld, Stream, Ar );

	GStatProcessingViewportClient = NULL;

	//if we loaded thumbs for saving, purge them back from the package
	//append loaded thumbs onto the existing thumbs list
	if (Pkg)
	{
		for (int32 ThumbRemoveIndex = 0; ThumbRemoveIndex < ThumbNamesToUnload.Num(); ++ThumbRemoveIndex)
		{
			ThumbnailTools::CacheThumbnail(ThumbNamesToUnload[ThumbRemoveIndex], NULL, Pkg);
		}
	}

	if(bExecSucceeded)
	{
		return true;
	}

	if( FParse::Command(&Str, TEXT("DUMPMODELGUIDS")) )
	{
		HandleDumpModelGUIDCommand( Str, Ar );
	}

	if( FParse::Command(&Str, TEXT("ModalTest") ) )
	{
		HandleModalTestCommand( Str, Ar );
		return true;
	}

	if (FParse::Command(&Str, TEXT("DisallowExport")))
	{
		HandleDisallowExportCommand(Str, Ar);
		return true;
	}

	if( FParse::Command(&Str, TEXT("DumpBPClasses")) )
	{
		HandleDumpBPClassesCommand( Str, Ar );
	}

	if( FParse::Command(&Str, TEXT("FindOutdatedInstances")) )
	{
		HandleFindOutdateInstancesCommand( Str, Ar );
	}

	if( FParse::Command(&Str, TEXT("DUMPSELECTION")) )
	{
		HandleDumpSelectionCommand( Str, Ar );
	}

	//----------------------------------------------------------------------------------
	// EDIT
	//
	if( FParse::Command(&Str,TEXT("EDIT")) )
	{
		return Exec_Edit( InWorld, Str, Ar );
	}
	//------------------------------------------------------------------------------------
	// ACTOR: Actor-related functions
	//
	else if (FParse::Command(&Str,TEXT("ACTOR")))
	{
		return Exec_Actor( InWorld, Str, Ar );
	}
	//------------------------------------------------------------------------------------
	// ELEMENT: Element-related functions
	//
	else if (FParse::Command(&Str,TEXT("ELEMENT")))
	{
		return Exec_Element( InWorld, Str, Ar );
	}
	//------------------------------------------------------------------------------------
	// MODE management (Global EDITOR mode):
	//
	else if( FParse::Command(&Str,TEXT("MODE")) )
	{
		return Exec_Mode( Str, Ar );
	}
	//----------------------------------------------------------------------------------
	// PIVOT
	//
	else if( FParse::Command(&Str,TEXT("PIVOT")) )
	{
		return Exec_Pivot( Str, Ar );
	}
	else if (FParse::Command(&Str,TEXT("BUILDLIGHTING")))
	{
		HandleBuildLightingCommand( Str, Ar, InWorld );
	}
	// BUILD PATHS
	else if (FParse::Command(&Str,TEXT("BUILDPATHS")))
	{
		HandleBuildPathsCommand( Str, Ar, InWorld );
	}
#if WITH_EDITOR
	else if (FParse::Command(&Str, TEXT("RecreateLandscapeCollision")))
	{
		// InWorld above is the PIE world if PIE is active, but this is specifically an editor command
		UWorld* World = GetEditorWorldContext().World();
		return HandleRecreateLandscapeCollisionCommand(Str, Ar, World);
	}
	else if (FParse::Command(&Str, TEXT("RemoveLandscapeXYOffsets")))
	{
		// InWorld above is the PIE world if PIE is active, but this is specifically an editor command
		UWorld* World = GetEditorWorldContext().World();
		return HandleRemoveLandscapeXYOffsetsCommand(Str, Ar, World);
	}
#endif // WITH_EDITOR
	else if( FParse::Command(&Str, TEXT("DISASMSCRIPT")) )
	{
		return HandleDisasmScriptCommand( Str, Ar );
	}
#if WITH_EDITOR
	else if (FParse::Command(&Str, TEXT("cook")))
	{
		if (CookServer)
		{
			return CookServer->Exec(InWorld, Str, Ar);
		}
	}
#endif
	else if ( FParse::Command(&Str, TEXT("GROUPS")) )
	{
		return Exec_Group( Str, Ar );
	}
	// #ttp 322815 - GDC, temp exec command for scaling the level
	else if ( FParse::Command(&Str,TEXT("SCALELEVEL")) )
	{
		// e.g. ScaleLevel Scale=1,2,3 Snap=4	// Non-uniform scaling
		// e.g. ScaleLevel Scale=2 Snap=4		// Uniform scaling

		// We can only scale radii if the level is given uniform scaling
		bool bScale = false;
		bool bScaleRadii = false;

		FVector::FReal Scale = 1.0f;
		FString ScaleStr;
		FVector ScaleVec( Scale );
		if(FParse::Value( Str, TEXT("Scale="), ScaleStr, false) && GetFVECTOR( *ScaleStr, ScaleVec ))
		{
			// Update uniform incase the user used uniform scale with a vector parm
			Scale = ScaleVec.X;
			bScaleRadii = (Scale == ScaleVec.Y && Scale == ScaleVec.Z ? true : false);
			bScale = true;
		}
		else if(FParse::Value( Str, TEXT("Scale="), Scale ))
		{
			// Copy the uniform scale to our vector param
			ScaleVec = FVector( Scale );
			bScaleRadii = true;
			bScale = true;
		}

		// Can we scale the level?
		if(bScale)
		{
			// See if a snap value was specified for the grid
			float NewGridSize;
			const bool bSnap = FParse::Value( Str, TEXT("Snap="), NewGridSize);

			const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "ScalingLevel", "Scaling Level") );

			// If it was, force the grid size to be this value temporarily
			const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
			TArray<float>& PosGridSizes = const_cast<TArray<float>&>(GetCurrentPositionGridArray());
			float& CurGridSize = PosGridSizes[ViewportSettings->CurrentPosGridSize];
			const float OldGridSize = CurGridSize;
			if ( bSnap )
			{
				CurGridSize = NewGridSize;
			}

			// "iterates through each actor in the current level"
			bool bBuildBSPs = false;
			for( TActorIterator<AActor> It(InWorld); It; ++It )
			{
				AActor* Actor = *It;

				// "It should skip all static meshes.  The reason for this is that they will scale the static meshes via the static mesh editor with the new BuildScale setting."
				if( Actor )
				{
					/*if (AStaticMeshActor* StaticMesh = Cast< AStaticMeshActor >( Actor ))
					{
						// Skip static meshes?
					}
					else*/if (ABrush* Brush = Cast< ABrush >( Actor ))
					{
						// "For volumes and brushes scale each vertex by the specified amount."
						if ( !FActorEditorUtils::IsABuilderBrush(Brush) && Brush->Brush )
						{
							const FVector OldLocation = Brush->GetActorLocation();
							const FVector NewLocation = OldLocation * ScaleVec;
							Brush->Modify(false);
							Brush->SetActorLocation( NewLocation );
							
							Brush->Brush->Modify(false);
							for( int32 poly = 0 ; poly < Brush->Brush->Polys->Element.Num() ; poly++ )
							{
								FPoly* Poly = &(Brush->Brush->Polys->Element[poly]);

								Poly->TextureU /= (FVector3f)ScaleVec;
								Poly->TextureV /= (FVector3f)ScaleVec;
								Poly->Base = ((Poly->Base - (FVector3f)Brush->GetPivotOffset()) * (FVector3f)ScaleVec) + (FVector3f)Brush->GetPivotOffset();

								for( int32 vtx = 0 ; vtx < Poly->Vertices.Num() ; vtx++ )
								{
									Poly->Vertices[vtx] = ((Poly->Vertices[vtx] - (FVector3f)Brush->GetPivotOffset()) * (FVector3f)ScaleVec) + (FVector3f)Brush->GetPivotOffset();

									// "Then snap the vertices new positions by the specified Snap amount"
									if ( bSnap )
									{
										FVector VPos = (FVector)Poly->Vertices[vtx];	// LWC_TODO: Perf pessimization
										FSnappingUtils::SnapPointToGrid( VPos, FVector(0, 0, 0) );
										Poly->Vertices[vtx] = (FVector3f)VPos;
									}
								}

								Poly->CalcNormal();
							}
							
							Brush->Brush->BuildBound();
							Brush->MarkPackageDirty();
							bBuildBSPs = true;
						}
					}
					else
					{
						// "Do not scale any child components."
						if( Actor->GetAttachParentActor() == NULL )
						{
							// "Only the root component"
							if (USceneComponent *RootComponent = Actor->GetRootComponent())
							{
								RootComponent->Modify();

								// "scales root component by the specified amount."
								const FVector OldLocation = RootComponent->GetComponentLocation();
								const FVector NewLocation = OldLocation * ScaleVec;
								RootComponent->SetWorldLocation( NewLocation );

								// Scale up the triggers
								if (UBoxComponent* BoxComponent = Cast< UBoxComponent >( RootComponent ))
								{
									const FVector OldExtent = BoxComponent->GetUnscaledBoxExtent();
									const FVector NewExtent = OldExtent * ScaleVec;
									BoxComponent->SetBoxExtent( NewExtent );
								}

								if ( bScaleRadii )
								{
									if (USphereComponent* SphereComponent = Cast< USphereComponent >( RootComponent ))
									{
										const float OldRadius = SphereComponent->GetUnscaledSphereRadius();
										const float NewRadius = OldRadius * Scale;
										SphereComponent->SetSphereRadius( NewRadius );
									}
									else if (UCapsuleComponent* CapsuleComponent = Cast< UCapsuleComponent >( RootComponent ))
									{
										float OldRadius, OldHalfHeight;
										CapsuleComponent->GetUnscaledCapsuleSize( OldRadius, OldHalfHeight );
										const float NewRadius = OldRadius * Scale;
										const float NewHalfHeight = OldHalfHeight * Scale;
										CapsuleComponent->SetCapsuleSize( NewRadius, NewHalfHeight );
									}
									else if (UPointLightComponent* PointLightComponent = Cast< UPointLightComponent >( RootComponent ))
									{
										PointLightComponent->AttenuationRadius *= Scale;
										PointLightComponent->SourceRadius *= Scale;
										PointLightComponent->SourceLength *= Scale;
									}
									else if (URadialForceComponent* RadialForceComponent = Cast< URadialForceComponent >( RootComponent ))
									{
										RadialForceComponent->Radius *= Scale;
									}
									/* Other components that have radii
									UPathFollowingComponent
									USmartNavLinkComponent
									UPawnSensingComponent
									USphereReflectionCaptureComponent
									UAIPerceptionComponent
									*/
								}
							}
						}
					}
				}
			}

			// Restore snap
			if ( bSnap )
			{
				CurGridSize = OldGridSize;
			}

			// Kick off a rebuild if any of the bsps have changed
			if ( bBuildBSPs )
			{
				GUnrealEd->Exec( InWorld, TEXT("MAP REBUILD ALLVISIBLE") );
			}
		}

		return true;
	}
	else if( FParse::Command(&Str, TEXT("ScaleMeshes") ) )
	{
		bool bScale = false;
		bool bScaleVec = false;

		// Was just a scale specified
		float Scale=1.0f;
		FVector BoxVec(Scale);
		if(FParse::Value(Str, TEXT("Scale="), Scale))
		{
			bScale = true;
		}
		else
		{
			// or was a bounding box specified instead
			FString BoxStr;	
			if((FParse::Value( Str, TEXT("BBOX="), BoxStr, false) || FParse::Value( Str, TEXT("FFD="), BoxStr, false)) && GetFVECTOR( *BoxStr, BoxVec ))
			{
				bScaleVec = true;
			}
		}

		if ( bScale || bScaleVec )
		{
			USelection* SelectedObjects = GetSelectedObjects();
			TArray<UStaticMesh*> SelectedMeshes;
			SelectedObjects->GetSelectedObjects(SelectedMeshes);

			if( SelectedMeshes.Num() )
			{
				GWarn->BeginSlowTask(NSLOCTEXT("UnrealEd", "ScalingStaticMeshes", "Scaling Static Meshes"), true, true);

				for (int32 MeshIndex = 0; MeshIndex < SelectedMeshes.Num(); ++MeshIndex)
				{
					UStaticMesh* Mesh = SelectedMeshes[MeshIndex];

					if (Mesh && Mesh->GetNumSourceModels() > 0)
					{
						Mesh->Modify();

						GWarn->StatusUpdate(MeshIndex + 1, SelectedMeshes.Num(), FText::Format(NSLOCTEXT("UnrealEd", "ScalingStaticMeshes_Value", "Static Mesh: {0}"), FText::FromString(Mesh->GetName())));

						FStaticMeshSourceModel& Model = Mesh->GetSourceModel(0);

						FVector ScaleVec(Scale, Scale, Scale);	// bScale
						if ( bScaleVec )
						{
							FBoxSphereBounds Bounds = Mesh->GetBounds();
							ScaleVec = BoxVec / (Bounds.BoxExtent * 2.0f);	// x2 as artists wanted length not radius	
						}
						Model.BuildSettings.BuildScale3D *= ScaleVec;	// Scale by the current modification
						
						UE_LOGF(LogUnrealEdSrv, Log, "Rescaling mesh '%ls' with scale: %ls", *Mesh->GetName(), *Model.BuildSettings.BuildScale3D.ToString() );
						
						Mesh->Build();
					}
				}
				GWarn->EndSlowTask();
			}
		}
	}
	else if( FParse::Command(&Str, TEXT("ClearSourceFiles") ) )
	{
		struct Local
		{
			static bool RemoveSourcePath( const FAssetImportInfo& ImportInfo, const FAssetData& AssetData, const TArray<FString>* SearchTerms )
			{
				FAssetImportInfo AssetImportInfo;

				bool bModified = false;
				for (const auto& File : ImportInfo.SourceFiles)
				{
					const bool bRemoveFile = File.RelativeFilename.IsEmpty() || !SearchTerms ||
						SearchTerms->ContainsByPredicate([&](const FString& SearchTerm){ return File.RelativeFilename.Contains(SearchTerm); });

					if( bRemoveFile )
					{
						UE_LOGF(LogUnrealEdSrv, Log, "Removing Path: %ls", *File.RelativeFilename);
						bModified = true;
					}
					else
					{
						AssetImportInfo.Insert(File);
					}
				}

				if (bModified)
				{
					if (UObject* Asset = AssetData.GetAsset())
					{
						UAssetImportData* ImportData = nullptr;

						// Root out the asset import data property
						for (FObjectProperty* Property : TFieldRange<FObjectProperty>(Asset->GetClass()))
						{
							ImportData = Cast<UAssetImportData>(Property->GetObjectPropertyValue(Property->ContainerPtrToValuePtr<UObject*>(Asset)));
							if (ImportData)
							{
								Asset->Modify();
								ImportData->SourceData = AssetImportInfo;
								return true;
							}
						}
					}
				}

				return false;
			}

			static void RemoveSourcePaths( const TArray<FAssetData>& AllAssets, const TArray<FString>* SearchTerms )
			{
				FScopedSlowTask SlowTask(AllAssets.Num(), NSLOCTEXT("UnrealEd", "ClearingSourceFiles", "Clearing Source Files"));
				SlowTask.MakeDialog(true);

				for (const FAssetData& Asset : AllAssets)
				{
					SlowTask.EnterProgressFrame();

					// Optimization - check the asset has import information before loading it
					TOptional<FAssetImportInfo> ImportInfo = FAssetSourceFilenameCache::ExtractAssetImportInfo(Asset);
					if (ImportInfo.IsSet() && ImportInfo->SourceFiles.Num())
					{
						RemoveSourcePath(ImportInfo.GetValue(), Asset, SearchTerms);
					}
				}
			}
		};

		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

		FString Path;
		FParse::Value(Str, TEXT("Path="), Path, false);

		TArray<FAssetData> AllAssets;
		if (!Path.IsEmpty())
		{
			AssetRegistryModule.Get().GetAssetsByPath(*Path, AllAssets, true);
		}
		else
		{
			AssetRegistryModule.Get().GetAllAssets(AllAssets);
		}

		FString SearchTermStr;
		if (FParse::Value(Str, TEXT("Find="), SearchTermStr, false))
		{
			// Searching for particular paths to remove
			TArray<FString> SearchTerms;
			SearchTermStr.ParseIntoArray( SearchTerms, TEXT(","), true );

			TArray<UObject*> ModifiedObjects;
			if( SearchTerms.Num() )
			{
				Local::RemoveSourcePaths(AllAssets, &SearchTerms);
			}
		}
		else
		{
			// Remove every source path on any asset
			Local::RemoveSourcePaths(AllAssets, nullptr);
		}
	}
	else if (FParse::Command(&Str, TEXT("RenameAssets")))
	{
		FString SearchTermStr;
		if ( FParse::Value(Str, TEXT("Find="), SearchTermStr) )
		{
			FString ReplaceStr;
			FParse::Value(Str, TEXT("Replace="), ReplaceStr );

			FString AutoCheckOutStr;
			FParse::Value(Str, TEXT("AutoCheckOut="), AutoCheckOutStr);
			AutoCheckOutStr = AutoCheckOutStr.ToLower();
			bool bAutoCheckOut = (AutoCheckOutStr == "yes" || AutoCheckOutStr == "true" || AutoCheckOutStr == "1");

			GWarn->BeginSlowTask(NSLOCTEXT("UnrealEd", "RenamingAssets", "Renaming Assets"), true, true);

			FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
			IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

			TArray<FAssetData> AllAssets;
			AssetRegistryModule.Get().GetAllAssets( AllAssets );

			TArray<FAssetRenameData> AssetsToRename;
			for( const FAssetData& Asset : AllAssets )
			{
				bool bRenamedPath = false;
				bool bRenamedAsset = false;
				FString NewAssetName = Asset.AssetName.ToString();
				FString NewPathName = Asset.PackagePath.ToString();
				if( NewAssetName.Contains( SearchTermStr ) )
				{
					FString TempPathName = NewAssetName.Replace(*SearchTermStr, *ReplaceStr);
					if (!TempPathName.IsEmpty())
					{
						NewAssetName = TempPathName;
						bRenamedAsset = true;
					}
				}
				
				if( NewPathName.Contains( SearchTermStr ) )
				{
					FString TempPathName = NewPathName.Replace( *SearchTermStr, *ReplaceStr );
					FPaths::RemoveDuplicateSlashes(TempPathName);

					if( !TempPathName.IsEmpty() )
					{
						NewPathName = TempPathName;
						bRenamedPath = true;
					}
				}

				if( bRenamedAsset || bRenamedPath )
				{
					FAssetRenameData RenameData(Asset.GetAsset(), NewPathName, NewAssetName);
					AssetsToRename.Add(RenameData);
				}
			}

			if( AssetsToRename.Num() > 0 )
			{
				AssetTools.RenameAssetsWithDialog( AssetsToRename, bAutoCheckOut );
			}

			GWarn->EndSlowTask();
		}
	}
	else if( FParse::Command(&Str, TEXT("HighResShot") ) )
	{
		// this is HighResShot from the Editor NOT in PIE
		// Editor PIE HighResShot is in GameViewportClient
		if (GetHighResScreenshotConfig().ParseConsoleCommand(Str, Ar))
		{
			TakeHighResScreenShots();
		}

		return true;
	}
	else if( FParse::Command(&Str, TEXT("EditorShot")) || FParse::Command(&Str, TEXT("EditorScreenShot")) )
	{
		struct Local
		{
			static void TakeScreenShotOfWidget( TSharedRef<SWidget> InWidget )
			{
				TArray<FColor> OutImageData;
				FIntVector OutImageSize;
				if (FSlateApplication::Get().TakeScreenshot(InWidget, OutImageData, OutImageSize))
				{
					FString FileName;
					const FString BaseFileName = GetDefault<ULevelEditorMiscSettings>()->EditorScreenshotSaveDirectory.Path / TEXT("EditorScreenshot");
					FFileHelper::GenerateNextBitmapFilename(BaseFileName, TEXT("bmp"), FileName);
					FFileHelper::CreateBitmap(*FileName, OutImageSize.X, OutImageSize.Y, OutImageData.GetData());
				}
			}
		};

		if( FSlateApplication::IsInitialized() )
		{
			if( FParse::Command(&Str, TEXT("All") ))
			{
				TArray< TSharedRef<SWindow> > OpenWindows;
				FSlateApplication::Get().GetAllVisibleWindowsOrdered(OpenWindows);
				for( int32 WindowId = 0; WindowId < OpenWindows.Num(); ++WindowId )
				{
					Local::TakeScreenShotOfWidget(OpenWindows[WindowId]);
				}
			}
			else
			{
				FString WindowNameStr;
				if ( FParse::Value(Str, TEXT("Name="), WindowNameStr) )
				{
					TArray< TSharedRef<SWindow> > OpenWindows;
					FSlateApplication::Get().GetAllVisibleWindowsOrdered(OpenWindows);
					for( int32 WindowId = 0; WindowId < OpenWindows.Num(); ++WindowId )
					{
						FString CurrentWindowName = OpenWindows[WindowId]->GetTitle().ToString();

						//Strip off the * from the end if it exists
						if( CurrentWindowName.EndsWith(TEXT("*"), ESearchCase::CaseSensitive) )
						{
							CurrentWindowName.LeftChopInline(1, EAllowShrinking::No);
						}

						if( CurrentWindowName == WindowNameStr )
						{
							Local::TakeScreenShotOfWidget(OpenWindows[WindowId]);
						}
					}
				}
				else
				{
					TSharedPtr<SWindow> ActiveWindow = FSlateApplication::Get().GetActiveTopLevelWindow();
					if( ActiveWindow.IsValid() )
					{
						Local::TakeScreenShotOfWidget(ActiveWindow.ToSharedRef());
					}
				}
			}
		}
		return true;
	}
	return false;
}
#endif // UE_ALLOW_EXEC_COMMANDS

bool UUnrealEdEngine::AnyWorldsAreDirty( UWorld* InWorld ) const
{
	// Get the set of all reference worlds.
	TArray<UWorld*> WorldsArray;
	EditorLevelUtils::GetWorlds( InWorld, WorldsArray, true );

	if ( WorldsArray.Num() > 0 )
	{
		FString FinalFilename;
		for ( int32 WorldIndex = 0 ; WorldIndex < WorldsArray.Num() ; ++WorldIndex )
		{
			UWorld* World = WorldsArray[ WorldIndex ];
			UPackage* Package = Cast<UPackage>( World->GetOuter() );
			check( Package );

			// The world needs saving if...
			if ( Package->IsDirty() )
			{
				return true;
			}
		}
	}

	return false;
}


bool UUnrealEdEngine::AnyContentPackagesAreDirty() const
{
	const UPackage* TransientPackage = GetTransientPackage();

	// Check all packages for dirty, non-map, non-transient packages
	for ( TObjectIterator<UPackage> PackageIter; PackageIter; ++PackageIter )
	{
		UPackage* CurPackage = *PackageIter;

		// The package needs saving if it's not the transient package
		if ( CurPackage && ( CurPackage != TransientPackage ) && CurPackage->IsDirty() )
		{
			return true;
		}
	}

	return false;
}


bool UUnrealEdEngine::IsTemplateMap( const FString& MapName ) const
{
	for (const FTemplateMapInfo& It : GetTemplateMapInfos())
	{
		if (It.Map.GetLongPackageName() == MapName)
		{
			return true;
		}
	}

	return false;
}


bool UUnrealEdEngine::IsUserInteracting()
{
	// Check to see if the user is in the middle of a drag operation.
	bool bUserIsInteracting = false;
	for (const FEditorViewportClient* VC : GetAllViewportClients())
	{
		// Check for tracking and capture.  If a viewport has mouse capture, it could be locking the mouse to the viewport, which means if we prompt with a dialog
		// while the mouse is locked to a viewport, we wont be able to interact with the dialog.  
		if (VC->IsTracking() || (VC->Viewport && VC->Viewport->HasMouseCapture()))
		{
			bUserIsInteracting = true;
			break;
		}
	}
	return bUserIsInteracting;
}

void UUnrealEdEngine::ShowPackageNotification()
{
	if( !FApp::IsUnattended() )
	{
		// Defer prompting for checkout if we cant prompt because of the following:
		// The user is interacting with something,
		// We are performing a slow task
		// We have a play world
		// The user disabled prompting on package modification
		// A window has capture on the mouse
		bool bCanPrompt = !IsUserInteracting() && !GIsSlowTask && !PlayWorld && GetDefault<UEditorLoadingSavingSettings>()->bPromptForCheckoutOnAssetModification && (FSlateApplication::Get().GetMouseCaptureWindow() == NULL);

		if( bCanPrompt )
		{
			bShowPackageNotification = false;
			bool bNeedWarningDialog = false;
			for (const auto& Entry : PackageToNotifyState)
			{
				if (Entry.Value == NS_PendingWarning)
				{
					bNeedWarningDialog = true;
					break;
				}
			}

			// The user is not interacting with anything, prompt to checkout packages that have been modified
			
			struct Local
			{
				static void OpenCheckOutDialog()
				{
					GUnrealEd->PromptToCheckoutModifiedPackages(true);
				}
			};

			if (bNeedWarningDialog)
			{
				Local::OpenCheckOutDialog();
			}
			else
			{
				int32 NumPackagesToCheckOut = GetNumDirtyPackagesThatNeedCheckout();

				FFormatNamedArguments Args;
				Args.Add(TEXT("NumFiles"), NumPackagesToCheckOut);

				FText ErrorText = FText::Format(NSLOCTEXT("SourceControl", "CheckOutNotification", "{NumFiles} files need check-out!"), Args);

				if (!CheckOutNotificationWeakPtr.IsValid())
				{
					FNotificationInfo ErrorNotification(ErrorText);
					ErrorNotification.bFireAndForget = true;;
					ErrorNotification.Hyperlink = FSimpleDelegate::CreateStatic(&Local::OpenCheckOutDialog);
					ErrorNotification.HyperlinkText = NSLOCTEXT("SourceControl", "CheckOutHyperlinkText", "Check-Out");
					ErrorNotification.ExpireDuration = 10.0f; // Need this message to last a little longer than normal since the user will probably want to click the hyperlink to check out files
					ErrorNotification.bUseThrobber = true;

					// For adding notifications.
					CheckOutNotificationWeakPtr = FSlateNotificationManager::Get().AddNotification(ErrorNotification);
				}
				else
				{
					CheckOutNotificationWeakPtr.Pin()->SetText(ErrorText);
					CheckOutNotificationWeakPtr.Pin()->ExpireAndFadeout();
				}
			}
		}
	}
}

void UUnrealEdEngine::PromptToCheckoutModifiedPackages( bool bPromptAll )
{
	TArray<UPackage*> PackagesToCheckout;
	if( bPromptAll )
	{
		for( TMap<TWeakObjectPtr<UPackage>,uint8>::TIterator It(PackageToNotifyState); It; ++It )
		{
			if( It.Key().IsValid() )
			{
				PackagesToCheckout.Add( It.Key().Get() );
			}
		}
	}
	else
	{
		for( TMap<TWeakObjectPtr<UPackage>,uint8>::TIterator It(PackageToNotifyState); It; ++It )
		{
			if( It.Key().IsValid() && (It.Value() == NS_PendingWarning || It.Value() == NS_PendingPrompt) )
			{
				PackagesToCheckout.Add( It.Key().Get() );
				It.Value() = NS_DialogPrompted;
			}
		}
	}

	const bool bCheckDirty = true;
	const bool bPromptingAfterModify = true;
	FEditorFileUtils::PromptToCheckoutPackages( bCheckDirty, PackagesToCheckout, NULL, NULL, bPromptingAfterModify );
}

int32 UUnrealEdEngine::InternalGetNumDirtyPackagesThatNeedCheckout(bool bCheckIfAny) const
{
	int32 PackageCount = 0;
		
	if (ISourceControlModule::Get().IsEnabled())
	{
		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
		for (TMap<TWeakObjectPtr<UPackage>, uint8>::TConstIterator It(PackageToNotifyState); It; ++It)
		{
			const UPackage* Package = It.Key().Get();
			if (Package != NULL)
			{
				FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(Package, EStateCacheUsage::Use);
				if (SourceControlState.IsValid() && (SourceControlState->CanCheckout() || !SourceControlState->IsCurrent() || SourceControlState->IsCheckedOutOther()))
				{
					++PackageCount;
					if (bCheckIfAny)
					{
						break;
					}
				}
			}
		}
	}

	return PackageCount;
}

int32 UUnrealEdEngine::GetNumDirtyPackagesThatNeedCheckout() const
{
	return InternalGetNumDirtyPackagesThatNeedCheckout(false);
}

bool UUnrealEdEngine::DoDirtyPackagesNeedCheckout() const
{
	return InternalGetNumDirtyPackagesThatNeedCheckout(true) > 0;
}

bool UUnrealEdEngine::Exec_Edit( UWorld* InWorld, const TCHAR* Str, FOutputDevice& Ar )
{
	using namespace UnrealEdSrvLocals;

	const FText TransactionName = NSLOCTEXT("UnrealEd", "Cut", "Cut");

	const bool bComponentsSelected = GetSelectedComponentCount() > 0;

	if( FParse::Command(&Str,TEXT("CUT")) )
	{
		if (GLevelEditorModeTools().ProcessEditCut())
		{
			return true;
		}
		
		// There are a few different cases that can happen here:
		// 1. If TypedElementCommonActionsUtils::IsElementCopyAndPasteEnabled() is enabled, actors and components go through the common actions
		//	  If not, they go through the legacy code path
		// 2. If TEDS.Feature.Folders.TypedElements is enabled, folders go through common actions
		//    If not, folders rely on delegates like OnEditCutActorsBegin to get copied by the Scene Outliner widget

		// Since different copy paths for folders/actors/components can end up overwriting each other depending on what is enabled if we directly 
		// modify the clipboard, we create a temporary buffer instead to append to
		FString CopyBuffer;
		const FScopedTransaction Transaction(TransactionName);
	
		// Keep track of whether the actor cut begin delegate was fired along the various code paths so we can also fire the cut end delegate
		bool bActorCutBeginDelegateFired = false;
		
		// When typed elements copy paste is enabled for actors, we don't need to go through the legacy cut path for actors or components
		if (TypedElementCommonActionsUtils::IsElementCopyAndPasteEnabled())
		{
			// Do nothing
		}
		else
		{
			if (bComponentsSelected)
			{
				edactCopySelected(InWorld, &CopyBuffer);
				const bool bCheckRef = GetDefault<ULevelEditorMiscSettings>()->bCheckReferencesOnDelete;
				edactDeleteSelected(InWorld, true, bCheckRef, bCheckRef);
			}
			else
			{
				FEditorDelegates::OnEditCutActorsBegin.Broadcast();
				bActorCutBeginDelegateFired = true;
				const bool bCheckRef = GetDefault<ULevelEditorMiscSettings>()->bCheckReferencesOnDelete;
				CopySelectedActorsToClipboard(InWorld, true, false /*bIsMove*/, bCheckRef, &CopyBuffer);
			}
		}
		
		// Unlike copy, we first go through the legacy path for actors and then the common actions for folders (if enabled). This because if we 
		// cut (copy+delete) the folder first and then cut the actor it does not have a parent folder anymore to copy onto the clipboard, which leads
		// to incorrect data on paste
		if (UTypedElementCommonActions* CommonActions = GetLevelEditorCommonActions())
		{
			FString CommonActionsCopyBuffer;
			// End drags to avoid deleting something out from under one.
			FSlateApplication::Get().CancelDragDrop();
					
			const bool bCheckRef = GetDefault<ULevelEditorMiscSettings>()->bCheckReferencesOnDelete;
			FTypedElementDeletionOptions Options;
			Options
				.SetWarnAboutReferences(bCheckRef)
				.SetWarnAboutSoftReferences(bCheckRef);

			UTypedElementSelectionSet* SelectionSet = GetLevelEditorSelectionSet();
			CommonActions->CutSelectedElements(SelectionSet, InWorld, Options, &CommonActionsCopyBuffer);
			CopyBuffer.Append(CommonActionsCopyBuffer);
		}
		
		// Append all the copy contents so far into one buffer and then copy to clipboard
		if (!CopyBuffer.IsEmpty())
		{
			FPlatformApplicationMisc::ClipboardCopy(*CopyBuffer);
		}
		
		// This delegate was fired after the contents were copied onto the clipboard in the past, so we fire it at the end
		if (bActorCutBeginDelegateFired)
		{
			FEditorDelegates::OnEditCutActorsEnd.Broadcast();
		}
		
	}
	else if( FParse::Command(&Str,TEXT("COPY")) )
	{
		if (GLevelEditorModeTools().ProcessEditCopy())
		{
			return true;
		}
		
		// There are a few different cases that can happen here:
		// 1. If TypedElementCommonActionsUtils::IsElementCopyAndPasteEnabled() is enabled, actors and components go through the common actions
		//	  If not, they go through the legacy code path
		// 2. If TEDS.Feature.Folders.TypedElements is enabled, folders go through common actions
		//    If not, folders rely on delegates like OnEditCutActorsBegin to get copied by the Scene Outliner widget

		// Since different copy paths for folders/actors/components can end up overwriting each other depending on what is enabled if we directly 
		// modify the clipboard, we create a temporary buffer instead to append to
		FString CopyBuffer;
		if (UTypedElementCommonActions* CommonActions = GetLevelEditorCommonActions())
		{
			UTypedElementSelectionSet* SelectionSet = GetLevelEditorSelectionSet();
			CommonActions->CopySelectedElementsToString(SelectionSet, CopyBuffer);
		}
		
		// Keep track of whether the actor copy begin delegate was fired along the various code paths so we can also fire the copy end delegate
		bool bActorCopyBeginDelegateFired = false;
		
		// When typed elements copy paste is enabled for actors, we don't need to go through the legacy cut path for actors or components
		if (TypedElementCommonActionsUtils::IsElementCopyAndPasteEnabled())
		{
			// do nothing
		}
		else
		{
			FString LegacyCopyBuffer;
			if (bComponentsSelected)
			{
				edactCopySelected(InWorld, &LegacyCopyBuffer);
			}
			else
			{
				FEditorDelegates::OnEditCopyActorsBegin.Broadcast();
				bActorCopyBeginDelegateFired = true;
				CopySelectedActorsToClipboard(InWorld, false, true, true, &LegacyCopyBuffer);
			}
			
			// The old paste logic for actors and folders combined relies on the actor copy being first in the clipboard for some reason
			// So we append the common actions copy buffer to the end of the legacy copy buffer and then do a swap
			LegacyCopyBuffer.Append(CopyBuffer);
			Swap(CopyBuffer, LegacyCopyBuffer);
		}
		
		// Append all the copy contents so far into one buffer and then copy to clipboard
		if (!CopyBuffer.IsEmpty())
		{
			FPlatformApplicationMisc::ClipboardCopy(*CopyBuffer);
		}
		
		// This delegate was fired after the contents were copied onto the clipboard in the past, so we fire it at the end
		if (bActorCopyBeginDelegateFired)
		{
			FEditorDelegates::OnEditCopyActorsEnd.Broadcast();
		}
	}
	else if( FParse::Command(&Str,TEXT("PASTE")) )
	{
		if (GLevelEditorModeTools().ProcessEditPaste())
		{
			return true;
		}
		
		bool bPastedViaCommonActions = false;
		bool bClipboardCopyFailed = false;
		
		// Single transaction for all paste code paths, since similar to copy/cut the actual contents of the paste depend on the various
		// legacy code paths and enabled cvars for actors/components/folders to determine whether common actions or legacy code paths are used. 
		// But we want a single transaction for all of those to ensure that an undo reverts all of them at once
		const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "Paste", "Paste"));
		
		if (UTypedElementCommonActions* CommonActions = GetLevelEditorCommonActions())
		{
			FString ClipboardPaste;
			FPlatformApplicationMisc::ClipboardPaste(ClipboardPaste);
			if (!ClipboardPaste.IsEmpty())
			{
				UTypedElementSelectionSet* SelectionSet = GetLevelEditorSelectionSet();
						
				bool bPasteAtLocation = false;
				FVector PasteLocation = FVector::ZeroVector;
				FString TempStr;
				if (FParse::Value(Str, TEXT("TO="), TempStr))
				{
					if (!FCString::Strcmp(*TempStr, TEXT("HERE")))
					{
						bPasteAtLocation = true;
						const FSnappedPositioningData PositioningData = FSnappedPositioningData(GCurrentLevelEditingViewportClient, ClickLocation, ClickPlane)
							.AlignToSurfaceRotation(false);
						FVector SaveClickLocation = FActorPositioning::GetSnappedSurfaceAlignedTransform(PositioningData).GetLocation();
								
						PasteLocation = SaveClickLocation;
					}
					else
					{
						if (!FCString::Strcmp(*TempStr, TEXT("ORIGIN")))
						{
							bPasteAtLocation = false;
						}
					}
				}
						
				FTypedElementPasteOptions Options;
				Options.SelectionSetToModify = SelectionSet;
				Options.bPasteAtLocation = bPasteAtLocation;
				Options.PasteLocation = PasteLocation;
				TArray<FScriptTypedElementHandle> PastedElements = CommonActions->PasteElementsFromString(SelectionSet, InWorld, Options, ClipboardPaste);
				bPastedViaCommonActions = !PastedElements.IsEmpty();
			}
			else
			{
				// Grabbing clipboard contents can fail on some platforms if another program has the lock.
				// Our HAL doesn't expose that though.  If the clipboard is empty then fail to go the legacy path (which internally will
				// retry the clipboard - this MAY succeed and we don't want to go down that path if it does).
				bClipboardCopyFailed = true;
			}
		}

		if (bClipboardCopyFailed || (bPastedViaCommonActions && TypedElementCommonActionsUtils::IsElementCopyAndPasteEnabled()))
		{
			// do nothing
		}
		else
		{ 
			if (bComponentsSelected)
			{
				edactPasteSelected(InWorld, false, false, true);
			}
			else
			{
				// How should this paste be handled
				EPasteTo PasteTo = PT_OriginalLocation;
				FText TransDescription = NSLOCTEXT("UnrealEd", "Paste", "Paste");

				FString TempStr;
				if (FParse::Value(Str, TEXT("TO="), TempStr))
				{
					if (!FCString::Strcmp(*TempStr, TEXT("HERE")))
					{
						PasteTo = PT_Here;
						TransDescription = NSLOCTEXT("UnrealEd", "PasteHere", "Paste Here");
					}
					else
					{
						if (!FCString::Strcmp(*TempStr, TEXT("ORIGIN")))
						{
							PasteTo = PT_WorldOrigin;
							TransDescription = NSLOCTEXT("UnrealEd", "PasteToWorldOrigin", "Paste To World Origin");
						}
					}
				}

				FEditorDelegates::OnEditPasteActorsBegin.Broadcast();
				PasteSelectedActorsFromClipboard(InWorld, TransDescription, PasteTo);
				FEditorDelegates::OnEditPasteActorsEnd.Broadcast();
			}
		}
	}

	return false;
}

bool UUnrealEdEngine::Exec_Pivot( const TCHAR* Str, FOutputDevice& Ar )
{
	using namespace UnrealEdSrvLocals;

	if( FParse::Command(&Str,TEXT("HERE")) )
	{
		NoteActorMovement();
		SetPivot( ClickLocation, false, false );
		FinishAllSnaps();
		SetPivotMovedIndependently(true);
		RedrawLevelEditingViewports();
	}
	else if( FParse::Command(&Str,TEXT("SNAPPED")) )
	{
		NoteActorMovement();
		SetPivot( ClickLocation, true, false );
		FinishAllSnaps();
		SetPivotMovedIndependently(true);
		RedrawLevelEditingViewports();
	}
	else if( FParse::Command(&Str,TEXT("CENTERSELECTION")) )
	{
		NoteActorMovement();

		// Figure out the center location of all selections

		int32 Count = 0;
		FVector Center(0,0,0);

		if (UTypedElementSelectionSet* ElementSelectionSet = GetLevelEditorSelectionSet())
		{
			ElementSelectionSet->ForEachSelectedElement<ITypedElementWorldInterface>(
				[&Center, &Count](const TTypedElement<ITypedElementWorldInterface>& InElement)
			{
				// Special case legacy ABrush handling
				if (ABrush* Brush = Cast<ABrush>(ActorElementDataUtil::GetActorFromHandle(InElement)))
				{
					// Treat brushes as a special case; calculate an effective position from the center point of the vertices.
					// This way, "Center on Selection" has a special meaning for brushes.
					TSet<FVector> UniqueVertices;
					FVector VertexCenter = FVector::ZeroVector;

					if (Brush->Brush && Brush->Brush->Polys)
					{
						for (const auto& Poly : Brush->Brush->Polys->Element)
						{
							for (const auto& Vertex : Poly.Vertices)
							{
								UniqueVertices.Add((FVector)Vertex);
							}
						}

						for (const auto& Vertex : UniqueVertices)
						{
							VertexCenter += Vertex;
						}

						if (UniqueVertices.Num() > 0)
						{
							VertexCenter /= UniqueVertices.Num();
						}
					}

					Center += Brush->GetTransform().TransformPosition(VertexCenter);
					++Count;
					return true;
				}
				// If we got here, it's not a brush

				FTransform WorldTransform;
				if (InElement.GetWorldTransform(WorldTransform))
				{
					Center += WorldTransform.GetLocation();
					++Count;
				}

				return true;
			});
		}
		if( Count > 0 )
		{
			FVector CenterLocation = Center / Count;
			UnsnappedClickLocation = CenterLocation;
			ClickLocation = CenterLocation;
			ClickPlane = FPlane(0.f,0.f,0.f,0.f);

			SetPivot( ClickLocation, false, false );
			FinishAllSnaps();
			SetPivotMovedIndependently(true);
		}

		RedrawLevelEditingViewports();
	}

	return false;
}

/**
* Gathers up a list of selection FPolys from selected static meshes.
*
* @return	A TArray containing FPolys representing the triangles in the selected static meshes (note that these
*           triangles are transformed into world space before being added to the array.
*/

TArray<FPoly*> GetSelectedPolygons()
{
	// Build a list of polygons from all selected static meshes

	TArray<FPoly*> SelectedPolys;

	for( FSelectionIterator It( GEditor->GetSelectedActorIterator() ) ; It ; ++It )
	{
		AActor* Actor = static_cast<AActor*>( *It );
		checkSlow( Actor->IsA(AActor::StaticClass()) );
		FTransform ActorToWorld = Actor->ActorToWorld();
		
		for (UActorComponent* Component : Actor->GetComponents())
		{
			// If its a static mesh component, with a static mesh
			UStaticMeshComponent* SMComp = Cast<UStaticMeshComponent>(Component);
			if (SMComp && SMComp->IsRegistered() && SMComp->GetStaticMesh())
			{
				UStaticMesh* StaticMesh = SMComp->GetStaticMesh();
				if ( StaticMesh )
				{
					int32 NumLods = StaticMesh->GetNumLODs();
					if ( NumLods )
					{
						const FStaticMeshLODResources& MeshLodZero = StaticMesh->GetLODForExport(0);
						int32 NumTriangles = MeshLodZero.GetNumTriangles();
						int32 NumVertices = MeshLodZero.GetNumVertices();
			
						const FPositionVertexBuffer& PositionVertexBuffer = MeshLodZero.VertexBuffers.PositionVertexBuffer;
						FIndexArrayView Indices = MeshLodZero.IndexBuffer.GetArrayView();

						if (Indices.Num() < NumTriangles * 3)
						{
							continue;
						}

						for ( int32 TriangleIndex = 0; TriangleIndex < NumTriangles; TriangleIndex++ )
						{
							const uint32 Idx0 = Indices[(TriangleIndex*3)+0];
							const uint32 Idx1 = Indices[(TriangleIndex*3)+1];
							const uint32 Idx2 = Indices[(TriangleIndex*3)+2];

							TUniquePtr<FPoly> Polygon = MakeUnique<FPoly>();

							// Add the poly
							Polygon->Init();
							Polygon->PolyFlags = PF_DefaultFlags;

							new(Polygon->Vertices) FVector3f(ActorToWorld.TransformPosition( (FVector)PositionVertexBuffer.VertexPosition(Idx2) ));
							new(Polygon->Vertices) FVector3f(ActorToWorld.TransformPosition( (FVector)PositionVertexBuffer.VertexPosition(Idx1) ));
							new(Polygon->Vertices) FVector3f(ActorToWorld.TransformPosition( (FVector)PositionVertexBuffer.VertexPosition(Idx0) ));

							Polygon->CalcNormal(1);
							Polygon->Fix();
							if( Polygon->Vertices.Num() > 2 )
							{
								if( !Polygon->Finalize( NULL, 1 ) )
								{
									SelectedPolys.Add( Polygon.Release() );
								}
							}

							// And add a flipped version of it to account for negative scaling
							Polygon = MakeUnique<FPoly>();
							Polygon->Init();
							Polygon->PolyFlags = PF_DefaultFlags;

							new(Polygon->Vertices) FVector3f(ActorToWorld.TransformPosition( (FVector)PositionVertexBuffer.VertexPosition(Idx2) ));
							new(Polygon->Vertices) FVector3f(ActorToWorld.TransformPosition( (FVector)PositionVertexBuffer.VertexPosition(Idx0) ));
							new(Polygon->Vertices) FVector3f(ActorToWorld.TransformPosition( (FVector)PositionVertexBuffer.VertexPosition(Idx1) ));
							Polygon->CalcNormal(1);
							Polygon->Fix();
							if( Polygon->Vertices.Num() > 2 )
							{
								if( !Polygon->Finalize( NULL, 1 ) )
								{
									SelectedPolys.Add( Polygon.Release() );
								}
							}
						}
					}
				}
			}
		}
	}

	return SelectedPolys;
}

/**
* Creates an axis aligned bounding box based on the bounds of SelectedPolys.  This bounding box
* is then copied into the builder brush.  This function is a set up function that the blocking volume
* creation execs will call before doing anything fancy.
*
* @param	InWorld				The world in which the builder brush needs to be created
* @param	SelectedPolys		The list of selected FPolys to create the bounding box from.
* @param	bSnapVertsToGrid	Should the brush verts snap to grid
*/

void CreateBoundingBoxBuilderBrush( UWorld* InWorld, const TArray<FPoly*> SelectedPolys, bool bSnapVertsToGrid )
{
	int x;
	FPoly* Poly;
	FBox BBox(ForceInit);
	FVector Vertex;

	for( x = 0 ; x < SelectedPolys.Num() ; ++x )
	{
		Poly = SelectedPolys[x];

		for( int v = 0 ; v < Poly->Vertices.Num() ; ++v )
		{
			if( bSnapVertsToGrid )
			{
				Vertex = (FVector)Poly->Vertices[v].GridSnap(GEditor->GetGridSize());
			}
			else
			{
				Vertex = (FVector)Poly->Vertices[v];
			}

			BBox += Vertex;
		}
	}

	// Change the builder brush to match the bounding box so that it exactly envelops the selected meshes
	{
		const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "BrushSet", "Brush Set"));

		UCubeBuilder* CubeBuilder = NewObject<UCubeBuilder>(GetTransientPackage(), NAME_None, RF_Transactional);
		FVector Extent = BBox.GetExtent();
		CubeBuilder->X = Extent.X * 2;
		CubeBuilder->Y = Extent.Y * 2;
		CubeBuilder->Z = Extent.Z * 2;
		CubeBuilder->Build(InWorld);

		ABrush* DefaultBrush = InWorld->GetDefaultBrush();
		check(DefaultBrush != nullptr);
		DefaultBrush->SetActorLocation(BBox.GetCenter(), false);
		DefaultBrush->ReregisterAllComponents();
	}
}

/**
* Take a plane and creates a gigantic triangle polygon that lies along it.  The blocking
* volume creation routines call this when they are cutting geometry and need to create
* capping polygons.
*
* This polygon is so huge that it doesn't matter where the vertices actually land.
*
* @param	InPlane		The plane to lay the polygon on
* @return	An FPoly representing the giant triangle we created (NULL if there was a problem)
*/

FPoly* CreateHugeTrianglePolygonOnPlane( const FPlane* InPlane )
{
	// Using the plane normal, get 2 good axis vectors

	FVector A, B;
	InPlane->GetSafeNormal().FindBestAxisVectors( A, B );

	// Create 4 vertices from the plane origin and the 2 axis generated above

	FPoly* Triangle = new FPoly();

	FVector Center = FVector( InPlane->X, InPlane->Y, InPlane->Z ) * InPlane->W;
	FVector V0 = Center + (A * UE_OLD_WORLD_MAX);	// LWC_TODO: WORLD_MAX misuse?
	FVector V1 = Center + (B * UE_OLD_WORLD_MAX);
	FVector V2 = Center - (((A + B) / 2.0f) * UE_OLD_WORLD_MAX);

	// Create a triangle that lays on InPlane

	Triangle->Init();
	Triangle->PolyFlags = PF_DefaultFlags;

	new(Triangle->Vertices) FVector3f( V0 );
	new(Triangle->Vertices) FVector3f( V2 );
	new(Triangle->Vertices) FVector3f( V1 );

	Triangle->CalcNormal(1);
	Triangle->Fix();
	if( Triangle->Finalize( NULL, 1 ) )
	{
		delete Triangle;
		Triangle = NULL;
	}

	return Triangle;
}

bool UUnrealEdEngine::Exec_Actor( UWorld* InWorld, const TCHAR* Str, FOutputDevice& Ar )
{
	using namespace UnrealEdSrvLocals;

	// Keep a pointer to the beginning of the string to use for message displaying purposes
	const TCHAR* const FullStr = Str;

	// Determine whether or not components are selected (used to properly label transaction names)
	const bool bComponentsSelected = GetSelectedComponentCount() > 0;

	if( FParse::Command(&Str,TEXT("ADD")) )
	{
		UClass* Class;
		if( ParseObject<UClass>( Str, TEXT("CLASS="), Class, nullptr ) )
		{
			int32 bSnap = 1;
			FParse::Value(Str,TEXT("SNAP="),bSnap);
			
			AActor* Default = Class->GetDefaultObject<AActor>();
			const FTransform ActorTransform = FActorPositioning::GetCurrentViewportPlacementTransform(*Default, !!bSnap);

			AddActor( InWorld->GetCurrentLevel(), Class, ActorTransform );
			RedrawLevelEditingViewports();
			return true;
		}
	}
	else if( FParse::Command(&Str,TEXT("CREATE_BV_BOUNDINGBOX")) )
	{
		const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "CreateBoundingBoxBlockingVolume", "Create Bounding Box Blocking Volume") );
		InWorld->GetDefaultBrush()->Modify(false);

		bool bSnapToGrid=0;
		FParse::Bool( Str, TEXT("SNAPTOGRID="), bSnapToGrid );

		// Create a bounding box for the selected static mesh triangles and set the builder brush to match it

		TArray<FPoly*> SelectedPolys = GetSelectedPolygons();
		CreateBoundingBoxBuilderBrush( InWorld, SelectedPolys, bSnapToGrid );

		// Create the blocking volume

		GUnrealEd->Exec( InWorld, TEXT("BRUSH ADDVOLUME CLASS=BlockingVolume") );

		// Clean up memory

		for( int x = 0 ; x < SelectedPolys.Num() ; ++x )
		{
			delete SelectedPolys[x];
		}

		SelectedPolys.Empty();

		// Finish up

		RedrawLevelEditingViewports();
		return true;
	}
	else if( FParse::Command(&Str,TEXT("CREATE_BV_CONVEXVOLUME")) )
	{
		const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "CreateConvexBlockingVolume", "Create Convex Blocking Volume") );
		InWorld->GetDefaultBrush()->Modify(false);

		bool bSnapToGrid=0;
		FParse::Bool( Str, TEXT("SNAPTOGRID="), bSnapToGrid );

		// The rejection tolerance.  When figuring out which planes to cut the blocking volume cube with
		// the code will reject any planes that are less than "NormalTolerance" different in their normals.
		//
		// This cuts down on the number of planes that will be used for generating the cutting planes and,
		// as a side effect, eliminates duplicates.

		float NormalTolerance = 0.25f;
		FParse::Value( Str, TEXT("NORMALTOLERANCE="), NormalTolerance );

		FVector3f NormalLimits( 1.0f, 1.0f, 1.0f );
		FParse::Value( Str, TEXT("NLIMITX="), NormalLimits.X );
		FParse::Value( Str, TEXT("NLIMITY="), NormalLimits.Y );
		FParse::Value( Str, TEXT("NLIMITZ="), NormalLimits.Z );

		// Create a bounding box for the selected static mesh triangles and set the builder brush to match it

		TArray<FPoly*> SelectedPolys = GetSelectedPolygons();
		CreateBoundingBoxBuilderBrush( InWorld, SelectedPolys, bSnapToGrid );

		// Get a list of the polygons that make up the builder brush

		FPoly* poly;
		TArray<FPoly>* BuilderBrushPolys = new TArray<FPoly>( InWorld->GetDefaultBrush()->Brush->Polys->Element );

		// Create a list of valid splitting planes

		TArray<FPlane*> SplitterPlanes;

		for( int p = 0 ; p < SelectedPolys.Num() ; ++p )
		{
			// Get a splitting plane from the first poly in our selection

			poly = SelectedPolys[p];
			FPlane* SplittingPlane = new FPlane( (FVector)poly->Vertices[0], (FVector)poly->Normal );

			// Make sure this poly doesn't clip any other polys in the selection.  If it does, we can't use it for generating the convex volume.

			bool bUseThisSplitter = true;

			for( int pp = 0 ; pp < SelectedPolys.Num() && bUseThisSplitter ; ++pp )
			{
				FPoly* ppoly = SelectedPolys[pp];

				if( p != pp && !(poly->Normal - ppoly->Normal).IsNearlyZero() )
				{
					int res = ppoly->SplitWithPlaneFast( *SplittingPlane, NULL, NULL );

					if( res == SP_Split || res == SP_Front )
					{
						// Whoops, this plane clips polygons (and/or sits between static meshes) in the selection so it can't be used
						bUseThisSplitter = false;
					}
				}
			}

			// If this polygons plane doesn't clip the selection in any way, we can carve the builder brush with it. Save it.

			if( bUseThisSplitter )
			{
				// Move the plane into the same coordinate space as the builder brush

				*SplittingPlane = SplittingPlane->TransformBy(InWorld->GetDefaultBrush()->ActorToWorld().ToMatrixWithScale().InverseFast());

				// Before keeping this plane, make sure there aren't any existing planes that have a normal within the rejection tolerance.

				bool bAddPlaneToList = true;

				for( int x = 0 ; x < SplitterPlanes.Num() ; ++x )
				{
					FPlane* plane = SplitterPlanes[x];

					if( plane->GetSafeNormal().Equals( SplittingPlane->GetSafeNormal(), NormalTolerance ) )
					{
						bAddPlaneToList = false;
						break;
					}
				}

				// As a final test, make sure that this planes normal falls within the normal limits that were defined

				if( FMath::Abs( SplittingPlane->GetSafeNormal().X ) > NormalLimits.X )
				{
					bAddPlaneToList = false;
				}
				if( FMath::Abs( SplittingPlane->GetSafeNormal().Y ) > NormalLimits.Y )
				{
					bAddPlaneToList = false;
				}
				if( FMath::Abs( SplittingPlane->GetSafeNormal().Z ) > NormalLimits.Z )
				{
					bAddPlaneToList = false;
				}

				// If this plane passed every test - it's a keeper!

				if( bAddPlaneToList )
				{
					SplitterPlanes.Add( SplittingPlane );
				}
				else
				{
					delete SplittingPlane;
				}
			}
		}

		// The builder brush is a bounding box at this point that fully surrounds the selected static meshes.
		// Now we will carve away at it using the splitting planes we collected earlier.  When this process
		// is complete, we will have a convex volume inside of the builder brush that can then be used to add
		// a blocking volume.

		TArray<FPoly> NewBuilderBrushPolys;

		for( int sp = 0 ; sp < SplitterPlanes.Num() ; ++sp )
		{
			FPlane* plane = SplitterPlanes[sp];

			// Carve the builder brush with each splitting plane we collected.  We place the results into
			// NewBuilderBrushPolys since we don't want to overwrite the original array just yet.

			bool bNeedCapPoly = false;

			for( int bp = 0 ; bp < BuilderBrushPolys->Num() ; ++bp )
			{
				poly = &(*BuilderBrushPolys)[bp];

				FPoly Front, Back;
				int res = poly->SplitWithPlane( FVector3f( plane->X, plane->Y, plane->Z ) * plane->W, (FVector3f)plane->GetSafeNormal(), &Front, &Back, true );
				switch( res )
				{
					// Ignore these results.  We don't want them.
					case SP_Coplanar:
					case SP_Front:
						break;

					// In the case of a split, keep the polygon on the back side of the plane.
					case SP_Split:
					{
						NewBuilderBrushPolys.Add( Back );
						bNeedCapPoly = true;
					}
					break;

					// By default, just keep the polygon that we had.
					default:
					{
						NewBuilderBrushPolys.Add( (*BuilderBrushPolys)[bp] );
					}
					break;
				}
			}

			// NewBuilderBrushPolys contains the newly clipped polygons so copy those into
			// the real array of polygons.

			BuilderBrushPolys = new TArray<FPoly>( NewBuilderBrushPolys );
			NewBuilderBrushPolys.Empty();

			// If any splitting occured, we need to generate a cap polygon to cover the hole.

			if( bNeedCapPoly )
			{
				// Create a large triangle polygon that covers the newly formed hole in the builder brush.

				FPoly* CappingPoly = CreateHugeTrianglePolygonOnPlane( plane );

				if( CappingPoly )
				{
					// Now we do the clipping the other way around.  We are going to use the polygons in the builder brush to
					// create planes which will clip the huge triangle polygon we just created.  When this process is over,
					// we will be left with a new polygon that covers the newly formed hole in the builder brush.

					for( int bp = 0 ; bp < BuilderBrushPolys->Num() ; ++bp )
					{
						poly = &((*BuilderBrushPolys)[bp]);
						plane = new FPlane((FVector)poly->Vertices[0], (FVector)poly->Vertices[1], (FVector)poly->Vertices[2] );

						FPoly Front, Back;
						int res = CappingPoly->SplitWithPlane( FVector3f( plane->X, plane->Y, plane->Z ) * plane->W, (FVector3f)plane->GetSafeNormal(), &Front, &Back, true );
						switch( res )
						{
							case SP_Split:
							{
								*CappingPoly = Back;
							}
							break;
						}
					}

					// Add that new polygon into the builder brush polys as a capping polygon.

					BuilderBrushPolys->Add( *CappingPoly );
				}
			}
		}

		// Create a new builder brush from the freshly clipped polygons.

		InWorld->GetDefaultBrush()->Brush->Polys->Element.Empty();

		for( int x = 0 ; x < BuilderBrushPolys->Num() ; ++x )
		{
			InWorld->GetDefaultBrush()->Brush->Polys->Element.Add((*BuilderBrushPolys)[x]);
		}

		InWorld->GetDefaultBrush()->ReregisterAllComponents();

		// Create the blocking volume

		GUnrealEd->Exec( InWorld, TEXT("BRUSH ADDVOLUME CLASS=BlockingVolume") );

		// Clean up memory

		for( int x = 0 ; x < SelectedPolys.Num() ; ++x )
		{
			delete SelectedPolys[x];
		}

		SelectedPolys.Empty();

		for( int x = 0 ; x < SplitterPlanes.Num() ; ++x )
		{
			delete SplitterPlanes[x];
		}

		SplitterPlanes.Empty();

		delete BuilderBrushPolys;

		// Finish up

		RedrawLevelEditingViewports();
		return true;
	}
	else if( FParse::Command(&Str,TEXT("MIRROR")) )
	{
		FVector MirrorScale( 1, 1, 1 );
		GetFVECTOR( Str, MirrorScale );
		// We can't have zeroes in the vector
		if( !MirrorScale.X )		MirrorScale.X = 1;
		if( !MirrorScale.Y )		MirrorScale.Y = 1;
		if( !MirrorScale.Z )		MirrorScale.Z = 1;

		if (GCurrentLevelEditingViewportClient)
		{
			const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "MirroringActors", "Mirroring Actors"));
			GCurrentLevelEditingViewportClient->MirrorSelectedActors(MirrorScale);
			RebuildAlteredBSP(); // Update the Bsp of any levels containing a modified brush
		}

		return true;
	}
	else if( FParse::Command(&Str,TEXT("DELTAMOVE")) )
	{
		const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "DeltaMoveActors", "Move Actors by Delta"));
		FVector DeltaMove = FVector::ZeroVector;
		GetFVECTOR( Str, DeltaMove );

		if (GCurrentLevelEditingViewportClient)
		{
			if (TSharedPtr<ILevelEditor> LevelEditor = GCurrentLevelEditingViewportClient->ParentLevelEditor.Pin())
			{
				FEditorModeTools& Tools = LevelEditor->GetEditorModeManager();
				Tools.SetPivotLocation(Tools.PivotLocation + DeltaMove, false);
			}

			GCurrentLevelEditingViewportClient->ApplyDeltaToActors(DeltaMove, FRotator::ZeroRotator, FVector::ZeroVector);
			RedrawLevelEditingViewports();
		}

		return true;
	}
	else if( FParse::Command(&Str,TEXT("HIDE")) )
	{
		if( FParse::Command(&Str,TEXT("SELECTED")) ) // ACTOR HIDE SELECTED
		{
			if ( FParse::Command(&Str,TEXT("STARTUP")) ) // ACTOR HIDE SELECTED STARTUP
			{
				if (FParse::Command(&Str, TEXT("HIERARCHY")))
				{
					const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "HideSelectedHierarchyAtStartup", "Hide Selected Hierarchy at Editor Startup") );
					edactHideSelectedHierarchyStartup( InWorld );
					return true;
				}
				else
				{
					const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "HideSelectedAtStartup", "Hide Selected at Editor Startup") );
					edactHideSelectedStartup( InWorld );
					return true;
				}
			}
			else if (FParse::Command(&Str, TEXT("HIERARCHY")))
			{
				const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "HideSelectedHierarchy", "Hide Selected Hierarchy") );
				edactHideSelectedHierarchy( InWorld );
				SelectNone( true, true );
				return true;
			}
			else
			{
				const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "HideSelected", "Hide Selected") );
				edactHideSelected( InWorld );
				SelectNone( true, true );
				return true;
			}
		}
		else if( FParse::Command(&Str,TEXT("UNSELECTED")) ) // ACTOR HIDE UNSELECTEED
		{
			const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "HideUnselected", "Hide Unselected") );
			edactHideUnselected( InWorld );
			SelectNone( true, true );
			return true;
		}
	}
	else if( FParse::Command(&Str,TEXT("UNHIDE")) ) 
	{
		if ( FParse::Command(&Str,TEXT("ALL")) ) // ACTOR UNHIDE ALL
		{
			if ( FParse::Command(&Str,TEXT("STARTUP")) ) // ACTOR UNHIDE ALL STARTUP
			{
				const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "ShowAllAtStartup", "Show All at Editor Startup") );
				edactUnHideAllStartup( InWorld );
				return true;
			}
			else
			{
				const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "UnHideAll", "UnHide All") );
				edactUnHideAll( InWorld );
				return true;
			}
		}
		else if( FParse::Command(&Str,TEXT("SELECTED")) )	// ACTOR UNHIDE SELECTED
		{
			if ( FParse::Command(&Str,TEXT("STARTUP")) ) // ACTOR UNHIDE SELECTED STARTUP
			{
				if (FParse::Command(&Str, TEXT("HIERARCHY")))
				{
					const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "ShowSelectedHierarchyAtStartup", "Show Selected Hierarchy at Editor Startup") );
					edactUnHideSelectedHierarchyStartup( InWorld );
					return true;
				}
				else
				{
					const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "ShowSelectedAtStartup", "Show Selected at Editor Startup") );
					edactUnHideSelectedStartup( InWorld );
					return true;
				}
			}
			else if (FParse::Command(&Str, TEXT("HIERARCHY")))
			{
				const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "UnhideHierarchySelected", "Unhide Selected Hierarchy") );
				edactUnhideSelectedHierarchy( InWorld );
				return true;
			}
			else
			{
				const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "UnhideSelected", "Unhide Selected") );
				edactUnhideSelected( InWorld );
				return true;
			}
		}
	}
	else if ( FParse::Command(&Str, TEXT("TOGGLE SELECTED VISIBILITY")) )
	{
		if (FParse::Command(&Str, TEXT("HIERARCHY")))
		{
			const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "ToggleSelectedHierarchyVisibility", "Toggle Selected Hierarchy Visibility") );
			edactToggleSelectedHierarchyVisibility( InWorld );
			return true;
		}
		else
		{
			const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "ToggleSelectedVisibility", "Toggle Selected Visibility") );
			edactToggleSelectedVisibility( InWorld );
			return true;
		}
	}
	else if( FParse::Command(&Str, TEXT("APPLYTRANSFORM")) )
	{
		CommandIsDeprecated( TEXT("ACTOR APPLYTRANSFORM"), Ar );
	}
	else if( FParse::Command(&Str, TEXT("REPLACE")) )
	{
		UClass* Class;
		if( FParse::Command(&Str, TEXT("BRUSH")) ) // ACTOR REPLACE BRUSH
		{
			const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "ReplaceSelectedBrushActors", "Replace Selected Brush Actors") );
			edactReplaceSelectedBrush( InWorld );
			return true;
		}
		else if( ParseObject<UClass>( Str, TEXT("CLASS="), Class, nullptr ) ) // ACTOR REPLACE CLASS=<class>
		{
			const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "ReplaceSelectedNonBrushActors", "Replace Selected Non-Brush Actors") );
			edactReplaceSelectedNonBrushWithClass( Class );
			return true;
		}
	}
	//@todo locked levels - handle the rest of these....is this required, or can we assume that actors in locked levels can't be selected
	else if( FParse::Command(&Str,TEXT("SELECT")) )
	{
		if( FParse::Command(&Str,TEXT("NONE")) ) // ACTOR SELECT NONE
		{
			return Exec( InWorld, TEXT("SELECT NONE") );
		}
		else if( FParse::Command(&Str,TEXT("ALL")) ) // ACTOR SELECT ALL
		{
			if(FParse::Command(&Str, TEXT("FROMOBJ"))) // ACTOR SELECT ALL FROMOBJ
			{		
				bool bHasStaticMeshes = false;
				TArray<UClass*> ClassesToSelect;

				for(FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
				{
					AActor* Actor = static_cast<AActor*>(*It);
					checkSlow(Actor->IsA(AActor::StaticClass()));

					if( Actor->IsA(AStaticMeshActor::StaticClass()) )
					{
						bHasStaticMeshes = true;
					}
					else
					{
						ClassesToSelect.AddUnique(Actor->GetClass());
					}
				}

				const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "SelectAll", "Select All") );
				if(bHasStaticMeshes)
				{
					edactSelectMatchingStaticMesh(false);
				}

				if(ClassesToSelect.Num() > 0)
				{
					for(int Index = 0; Index < ClassesToSelect.Num(); ++Index)
					{
						edactSelectOfClass( InWorld, ClassesToSelect[Index]);
					}
				}

				return true;
			}
			else if( FParse::Command(&Str, TEXT("CHILDREN")) ) // ACTOR SELECT ALL CHILDREN
			{
				UEditorActorSubsystem* EditorActorSubsystem = GEditor->GetEditorSubsystem<UEditorActorSubsystem>();

				if (EditorActorSubsystem)
				{
					EditorActorSubsystem->SelectAllChildren(false);
				}
				
				return true;
			}
			else if( FParse::Command(&Str, TEXT("DESCENDANTS")) ) // ACTOR SELECT ALL DESCENDANTS
			{
				UEditorActorSubsystem* EditorActorSubsystem = GEditor->GetEditorSubsystem<UEditorActorSubsystem>();

				if (EditorActorSubsystem)
				{
					EditorActorSubsystem->SelectAllChildren(true);
				}
				return true;
			}
			else
			{
				UEditorActorSubsystem* EditorActorSubsystem = GEditor->GetEditorSubsystem<UEditorActorSubsystem>();

				if (EditorActorSubsystem)
				{
					EditorActorSubsystem->SelectAll(InWorld);
				}
				return true;
			}
		}
		else if( FParse::Command(&Str,TEXT("INSIDE") ) ) // ACTOR SELECT INSIDE
		{
			CommandIsDeprecated( TEXT("ACTOR SELECT INSIDE"), Ar );
		}
		else if( FParse::Command(&Str,TEXT("INVERT") ) ) // ACTOR SELECT INVERT
		{
			UEditorActorSubsystem* EditorActorSubsystem = GEditor->GetEditorSubsystem<UEditorActorSubsystem>();

			if (EditorActorSubsystem)
			{
				EditorActorSubsystem->InvertSelection(InWorld);
			}
			return true;
		}
		else if( FParse::Command(&Str,TEXT("OFCLASS")) ) // ACTOR SELECT OFCLASS CLASS=<class>
		{
			UClass* Class;
			if( ParseObject<UClass>(Str,TEXT("CLASS="),Class,nullptr) )
			{
				const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "SelectOfClass", "Select Of Class") );
				edactSelectOfClass( InWorld, Class );
			}
			else
			{
				UE_SUPPRESS(LogExec, Warning, Ar.Log(TEXT("Missing class") ));
			}
			return true;
		}
		else if( FParse::Command(&Str,TEXT("OFSUBCLASS")) ) // ACTOR SELECT OFSUBCLASS CLASS=<class>
		{
			UClass* Class;
			if( ParseObject<UClass>(Str,TEXT("CLASS="),Class,nullptr) )
			{
				const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "SelectSubclassOfClass", "Select Subclass Of Class") );
				edactSelectSubclassOf( InWorld, Class );
			}
			else
			{
				UE_SUPPRESS(LogExec, Warning, Ar.Log(TEXT("Missing class") ));
			}
			return true;
		}
		else if( FParse::Command(&Str,TEXT("BASED")) ) // ACTOR SELECT BASED
		{
			// @TODO no longer meaningful
			return true;
		}
		else if( FParse::Command(&Str,TEXT("DELETED")) ) // ACTOR SELECT DELETED
		{
			const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "SelectDeleted", "Select Deleted") );
			edactSelectDeleted( InWorld );
			return true;
		}
		else if( FParse::Command(&Str,TEXT("MATCHINGSTATICMESH")) ) // ACTOR SELECT MATCHINGSTATICMESH
		{
			const bool bAllClasses = FParse::Command( &Str, TEXT("ALLCLASSES") );
			const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "SelectMatchingStaticMesh", "Select Matching Static Mesh") );
			edactSelectMatchingStaticMesh( bAllClasses );
			return true;
		}
		else if( FParse::Command(&Str,TEXT("MATCHINGSKELETALMESH")) ) // ACTOR SELECT MATCHINGSKELETALMESH
		{
			const bool bAllClasses = FParse::Command( &Str, TEXT("ALLCLASSES") );
			const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "SelectMatchingSkeletalMesh", "Select Matching Skeletal Mesh") );
			edactSelectMatchingSkeletalMesh( bAllClasses );
			return true;
		}
		else if( FParse::Command(&Str,TEXT("MATCHINGMATERIAL")) )
		{
			const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "SelectAllWithMatchingMaterial", "Select All With Matching Material") );
			edactSelectMatchingMaterial();
			return true;
		}
		else if( FParse::Command(&Str,TEXT("MATCHINGEMITTER")) )
		{
			const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "SelectMatchingEmitter", "Select Matching Emitters") );
			edactSelectMatchingEmitter();
			return true;
		}
		else if (FParse::Command(&Str, TEXT("RELEVANTLIGHTS")))	// ACTOR SELECT RELEVANTLIGHTS
		{
			UE_LOGF(LogUnrealEdSrv, Log, "Select relevant lights!");
			edactSelectRelevantLights( InWorld );
		}
		else
		{
			// Get actor name.
			FName ActorName(NAME_None);
			if ( FParse::Value( Str, TEXT("NAME="), ActorName ) )
			{
				AActor* Actor = FindObject<AActor>( InWorld->GetCurrentLevel(), *ActorName.ToString() );
				const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "SelectToggleSingleActor", "Select Toggle Single Actor") );
				SelectActor( Actor, !(Actor && Actor->IsSelected()), false, true );
			}
			return true;
		}
	}
	else if( FParse::Command(&Str,TEXT("DELETE")) )		// ACTOR SELECT DELETE
	{
		if (UTypedElementCommonActions* CommonActions = GetLevelEditorCommonActions())
		{
			UTypedElementSelectionSet* SelectionSet = GetLevelEditorSelectionSet();
				
			// End drags to avoid deleting something out from under one.
			FSlateApplication::Get().CancelDragDrop();

			const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "DeleteElements", "Delete Elements"));
			if (!ensure(SelectionSet) || SelectionSet->GetNumSelectedElements() == 0)
			{
				// HACK: Not all modes will select elements, so allow them a shot at deletion if we don't think anything else is selected
				// TODO: Move this logic into FLevelEditorActionCallbacks and have it call into the mode directly
				if (!GLevelEditorModeTools().ProcessEditDelete())
				{
					// HACK: Call these directly for an empty selection so that folder deletion in the outliner still works
					// TODO: Move this logic into FLevelEditorActionCallbacks and have it call into the outliner directly
					FEditorDelegates::OnDeleteActorsBegin.Broadcast();
					FEditorDelegates::OnDeleteActorsEnd.Broadcast();
				}
			}
			else
			{
				FEditorDelegates::OnDeleteActorsBegin.Broadcast();
				const bool bCheckRef = GetDefault<ULevelEditorMiscSettings>()->bCheckReferencesOnDelete;
				FTypedElementDeletionOptions Options;
				Options
					.SetWarnAboutReferences(bCheckRef)
					.SetWarnAboutSoftReferences(bCheckRef);
				CommonActions->DeleteSelectedElements(SelectionSet, InWorld, Options);
				FEditorDelegates::OnDeleteActorsEnd.Broadcast();
			}
		}
		return true;
	}
	else if( FParse::Command(&Str,TEXT("UPDATE")) )		// ACTOR SELECT UPDATE
	{
		bool bLockedLevel = false;
		for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
		{
			AActor* Actor = static_cast<AActor*>( *It );
			checkSlow( Actor->IsA(AActor::StaticClass()) );

			if ( !Actor->IsTemplate() && FLevelUtils::IsLevelLocked(Actor) )
			{
				bLockedLevel = true;
			}
			else
			{
				Actor->PreEditChange(NULL);
				Actor->PostEditChange();
			}
		}

		if ( bLockedLevel )
		{
			FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Error_OperationDisallowedOnLockedLevelUpdateActor", "Update Actor: The requested operation could not be completed because the level is locked.") );
		}
		return true;
	}
	else if( FParse::Command(&Str,TEXT("SET")) )
	{
		// @todo DB: deprecate the ACTOR SET exec.
		RedrawLevelEditingViewports();
		return true;
	}
	else if( FParse::Command(&Str,TEXT("BAKEPREPIVOT")) )
	{
		UTypedElementSelectionSet* ElementSelectionSet = GetLevelEditorSelectionSet();
		if (ensure(ElementSelectionSet))
		{
			FScopedLevelDirtied				LevelDirtyCallback;
			FScopedActorPropertiesChange	ActorPropertiesChangeCallback;

			// Bakes the current pivot position into all selected elements

			FEditorModeTools& EditorModeTools = GLevelEditorModeTools();

			ElementSelectionSet->ForEachSelectedElement<ITypedElementWorldInterface>(
				[&EditorModeTools](const TTypedElement<ITypedElementWorldInterface>& InElement)
			{
				FTransform WorldTransform;
				if (InElement.GetWorldTransform(WorldTransform))
				{
					FVector Delta(EditorModeTools.PivotLocation - WorldTransform.GetLocation());
					InElement.SetPivotOffset(WorldTransform.InverseTransformVector(Delta));
				}
				return true; // continue
			});
			SetPivotMovedIndependently(false);

			GUnrealEd->NoteSelectionChange();
		}
	} 
	else if( FParse::Command(&Str,TEXT("UNBAKEPREPIVOT")) )
	{
		UTypedElementSelectionSet* ElementSelectionSet = GetLevelEditorSelectionSet();
		if (ensure(ElementSelectionSet))
		{
			FScopedLevelDirtied		LevelDirtyCallback;
			FScopedActorPropertiesChange	ActorPropertiesChangeCallback;

			// Resets the pivot offset of the selected elements to zero while leaving them in the same world location.

			ElementSelectionSet->ForEachSelectedElement<ITypedElementWorldInterface>(
			[](const TTypedElement<ITypedElementWorldInterface>& InElement)
			{
				InElement.SetPivotOffset(FVector::ZeroVector);
				return true; // continue
			});
			SetPivotMovedIndependently(false);

			GUnrealEd->NoteSelectionChange();
		}
	}
	else if( FParse::Command(&Str,TEXT("RESET")) )
	{
		FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "ResetActors", "Reset Actors") );

		bool bLocation=false;
		bool bPivot=false;
		bool bRotation=false;
		bool bScale=false;
		if( FParse::Command(&Str,TEXT("LOCATION")) )
		{
			bLocation=true;
			ResetPivot();
		}
		else if( FParse::Command(&Str, TEXT("PIVOT")) )
		{
			bPivot=true;
			ResetPivot();
		}
		else if( FParse::Command(&Str,TEXT("ROTATION")) )
		{
			bRotation=true;
		}
		else if( FParse::Command(&Str,TEXT("SCALE")) )
		{
			bScale=true;
		}
		else if( FParse::Command(&Str,TEXT("ALL")) )
		{
			bLocation=bRotation=bScale=true;
			ResetPivot();
		}

		// Fires ULevel::LevelDirtiedEvent when falling out of scope.
		FScopedLevelDirtied		LevelDirtyCallback;

		bool bHadLockedLevels = false;
		bool bModifiedActors = false;
		for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
		{
			AActor* Actor = static_cast<AActor*>( *It );
			checkSlow( Actor->IsA(AActor::StaticClass()) );

			if ( !Actor->IsTemplate() && FLevelUtils::IsLevelLocked(Actor) )
			{
				bHadLockedLevels = true;
			}
			else
			{
				bModifiedActors = true;

				Actor->PreEditChange(NULL);
				Actor->Modify();

				if( bLocation ) 
				{
					Actor->SetActorLocation(FVector::ZeroVector, false);
				}
				if( bPivot )
				{
					Actor->SetPivotOffset(FVector::ZeroVector);
				}

				if( bScale && Actor->GetRootComponent() != NULL ) 
				{
					Actor->GetRootComponent()->SetRelativeScale3D( FVector(1.f) );
				}

				Actor->MarkPackageDirty();
				LevelDirtyCallback.Request();
			}
		}

		if ( bHadLockedLevels )
		{
			FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Error_OperationDisallowedOnLockedLevelResetActor", "Reset Actor: The requested operation could not be completed because the level is locked.") );
		}

		if ( bModifiedActors )
		{
			RedrawLevelEditingViewports();
		}
		else
		{
			Transaction.Cancel();
		}
		return true;
	}
	else if( FParse::Command(&Str,TEXT("DUPLICATE")) )
	{
		if (UTypedElementCommonActions* CommonActions = GetLevelEditorCommonActions())
		{
			UTypedElementSelectionSet* SelectionSet = GetLevelEditorSelectionSet();

			const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "DuplicateElements", "Duplicate Elements"));
			if (!ensure(SelectionSet) || SelectionSet->GetNumSelectedElements() == 0)
			{
				// HACK: Not all modes will select elements, so allow them a shot at duplication if we don't think anything else is selected
				// TODO: Move this logic into FLevelEditorActionCallbacks and have it call into the mode directly
				if (!GLevelEditorModeTools().ProcessEditDuplicate())
				{
					// HACK: Call these directly for an empty selection so that folder duplication in the outliner still works
					// TODO: Move this logic into FLevelEditorActionCallbacks and have it call into the outliner directly
					FEditorDelegates::OnDuplicateActorsBegin.Broadcast();
					FEditorDelegates::OnDuplicateActorsEnd.Broadcast();
				}
			}
			else
			{
				const FVector DuplicateOffset = GEditor->GetGridLocationOffset(/*bUniformOffset*/false);
				const TArray<FTypedElementHandle> DuplicatedElements = CommonActions->DuplicateSelectedElements(SelectionSet, InWorld, DuplicateOffset);
				if (DuplicatedElements.Num() > 0)
				{
					SelectionSet->SetSelection(DuplicatedElements, FTypedElementSelectionOptions());
					SelectionSet->NotifyPendingChanges();

					// notify the global mode tools, the selection set should be identical to the new actors at this point
					TArray<AActor*> SelectedActors = SelectionSet->GetSelectedObjects<AActor>();
					GLevelEditorModeTools().ActorsDuplicatedNotify(SelectedActors, SelectedActors, DuplicateOffset != FVector::ZeroVector);
				}
			}
		}
		return true;
	}
	else if( FParse::Command(&Str, TEXT("ALIGN")) )
	{
		if( FParse::Command(&Str,TEXT("ORIGIN")) )
		{
			const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "Undo_SnapBrushOrigin", "Snap Brush Origin") );
			edactAlignOrigin();
			RedrawLevelEditingViewports();
			return true;
		}
		else // "VERTS" (default)
		{
			const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "Undo_SnapBrushVertices", "Snap Brush Vertices") );
			edactAlignVertices();
			RedrawLevelEditingViewports();
			RebuildAlteredBSP(); // Update the Bsp of any levels containing a modified brush
			return true;
		}
	}
	else if( FParse::Command(&Str,TEXT("TOGGLE")) )
	{
		if( FParse::Command(&Str,TEXT("LOCKMOVEMENT")) )			// ACTOR TOGGLE LOCKMOVEMENT
		{
			ToggleSelectedItemMovementLock();
		}

		RedrawLevelEditingViewports();
		return true;
	}
	else if( FParse::Command(&Str,TEXT("LEVELCURRENT")) )
	{
		MakeSelectedActorsLevelCurrent();
		return true;
	}
	else if( FParse::Command(&Str,TEXT("MOVETOCURRENT")) )
	{
		UEditorLevelUtils::MoveSelectedActorsToLevel( InWorld->GetCurrentLevel() );
		return true;
	}
	else if(FParse::Command(&Str, TEXT("DESELECT")))
	{
		const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "DeselectActors", "Deselect Actor(s)") );
		GEditor->GetSelectedActors()->Modify();

		//deselects everything in UnrealEd
		GUnrealEd->SelectNone(true, true);
		
		return true;
	}
	else if(FParse::Command(&Str, TEXT("EXPORT")))
	{
		if(FParse::Command(&Str, TEXT("FBX")))
		{
			TArray<FString> SaveFilenames;
			IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
			bool bSave = false;
			if ( DesktopPlatform )
			{
				void* ParentWindowWindowHandle = NULL;

				IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
				const TSharedPtr<SWindow>& MainFrameParentWindow = MainFrameModule.GetParentWindow();
				if ( MainFrameParentWindow.IsValid() && MainFrameParentWindow->GetNativeWindow().IsValid() )
				{
					ParentWindowWindowHandle = MainFrameParentWindow->GetNativeWindow()->GetOSWindowHandle();
				}

				bSave = DesktopPlatform->SaveFileDialog(
					ParentWindowWindowHandle,
					NSLOCTEXT("UnrealEd", "StaticMeshEditor_ExportToPromptTitle", "Export to...").ToString(), 
					*FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_EXPORT),
					TEXT(""), 
					TEXT("FBX document|*.fbx"),
					EFileDialogFlags::None,
					SaveFilenames
					);
			}

			// Show dialog and execute the export if the user did not cancel out
			if( bSave )
			{
				// Get the filename from dialog
				FString FileName = SaveFilenames[0];
				FEditorDirectories::Get().SetLastDirectory(ELastDirectory::GENERIC_EXPORT, FPaths::GetPath(FileName)); // Save path as default for next time.

				INodeNameAdapter NodeNameAdapter;
				UnFbx::FFbxExporter* Exporter = UnFbx::FFbxExporter::GetInstance();
				//Show the fbx export dialog options
				bool ExportCancel = false;
				bool ExportAll = false;
				Exporter->FillExportOptions(false, true, FileName, ExportCancel, ExportAll);
				if (!ExportCancel)
				{
					Exporter->CreateDocument();
					for (FSelectionIterator It(GetSelectedActorIterator()); It; ++It)
					{
						AActor* Actor = static_cast<AActor*>(*It);
						if (Actor->IsA(AActor::StaticClass()))
						{
							if (Actor->IsA(AStaticMeshActor::StaticClass()))
							{
								Exporter->ExportStaticMesh(Actor, CastChecked<AStaticMeshActor>(Actor)->GetStaticMeshComponent(), NodeNameAdapter);
							}
							else if (Actor->IsA(ASkeletalMeshActor::StaticClass()))
							{
								Exporter->ExportSkeletalMesh(Actor, CastChecked<ASkeletalMeshActor>(Actor)->GetSkeletalMeshComponent(), NodeNameAdapter);
							}
							else if (Actor->IsA(ABrush::StaticClass()))
							{
								Exporter->ExportBrush(CastChecked<ABrush>(Actor), NULL, true, NodeNameAdapter);
							}
						}
					}
					Exporter->WriteToFile(*FileName);
				}
			}
			return true;
		}

	}
	else if ( FParse::Command(&Str, TEXT("SNAP"))) // ACTOR SNAP
	{
		FSnappingUtils::EnableActorSnap( !FSnappingUtils::IsSnapToActorEnabled() );
		return true;
	}

	return false;
}

bool UUnrealEdEngine::Exec_Element( UWorld* InWorld, const TCHAR* Str, FOutputDevice& Ar )
{
	// Keep a pointer to the beginning of the string to use for message displaying purposes
	const TCHAR* const FullStr = Str;

	if (FParse::Command(&Str, TEXT("MIRROR")))
	{
		FVector MirrorScale(1, 1, 1);
		GetFVECTOR(Str, MirrorScale);
		// We can't have zeroes in the vector
		if (!MirrorScale.X)		MirrorScale.X = 1;
		if (!MirrorScale.Y)		MirrorScale.Y = 1;
		if (!MirrorScale.Z)		MirrorScale.Z = 1;

		if (GCurrentLevelEditingViewportClient)
		{
			const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "MirroringElements", "Mirroring Elements"));
			GCurrentLevelEditingViewportClient->MirrorSelectedElements(MirrorScale);
			RebuildAlteredBSP(); // Update the Bsp of any levels containing a modified brush
		}

		return true;
	}
	
	if (FParse::Command(&Str, TEXT("DELTAMOVE")))
	{
		const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "DeltaMovElements", "Move Elements by Delta"));
		FVector DeltaMove = FVector::ZeroVector;
		GetFVECTOR(Str, DeltaMove);

		if (GCurrentLevelEditingViewportClient)
		{
			if (TSharedPtr<ILevelEditor> LevelEditor = GCurrentLevelEditingViewportClient->ParentLevelEditor.Pin())
			{
				FEditorModeTools& Tools = LevelEditor->GetEditorModeManager();
				Tools.SetPivotLocation(Tools.PivotLocation + DeltaMove, false);
			}

			GCurrentLevelEditingViewportClient->ApplyDeltaToSelectedElements(FTransform(FRotator::ZeroRotator, DeltaMove, FVector::ZeroVector));
			RedrawLevelEditingViewports();
		}

		return true;
	}

	return false;
}

bool UUnrealEdEngine::Exec_Mode( const TCHAR* Str, FOutputDevice& Ar )
{
	int32 DWord1;

	if( FParse::Command(&Str, TEXT("WIDGETCOORDSYSTEMCYCLE")) )
	{
		const bool bGetRawValue = true;
		int32 Wk = GLevelEditorModeTools().GetCoordSystem(bGetRawValue);
		Wk++;

		if( Wk == COORD_Max )
		{
			Wk -= COORD_Max;
		}

		GLevelEditorModeTools().SetCoordSystem((ECoordSystem)Wk);
		FEditorSupportDelegates::RedrawAllViewports.Broadcast();
		FEditorSupportDelegates::UpdateUI.Broadcast();
	}

	if( FParse::Command(&Str, TEXT("WIDGETMODECYCLE")) )
	{
		GLevelEditorModeTools().CycleWidgetMode();
	}

	if( FParse::Value(Str, TEXT("GRID="), DWord1) )
	{
		FinishAllSnaps();

		ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>();
		ViewportSettings->GridEnabled = DWord1;
		ViewportSettings->PostEditChange();

		FEditorDelegates::OnGridSnappingChanged.Broadcast(ViewportSettings->GridEnabled, GetGridSize());
		FEditorSupportDelegates::UpdateUI.Broadcast();
	}

	if( FParse::Value(Str, TEXT("ROTGRID="), DWord1) )
	{
		FinishAllSnaps();

		ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>();
		ViewportSettings->RotGridEnabled = DWord1;
		ViewportSettings->PostEditChange();

		FEditorSupportDelegates::UpdateUI.Broadcast();
	}

	if( FParse::Value(Str, TEXT("SCALEGRID="), DWord1) )
	{
		FinishAllSnaps();

		ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>();
		ViewportSettings->SnapScaleEnabled = DWord1;
		ViewportSettings->PostEditChange();

		FEditorSupportDelegates::UpdateUI.Broadcast();
	}

	if( FParse::Value(Str, TEXT("SNAPVERTEX="), DWord1) )
	{
		FinishAllSnaps();

		ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>();
		ViewportSettings->bSnapVertices = !!DWord1;
		ViewportSettings->PostEditChange();

		FEditorSupportDelegates::UpdateUI.Broadcast();
	}

	if( FParse::Value(Str, TEXT("SHOWBRUSHMARKERPOLYS="), DWord1) )
	{
		FinishAllSnaps();
		GetMutableDefault<ULevelEditorViewportSettings>()->bShowBrushMarkerPolys = DWord1;
	}

	if( FParse::Value(Str, TEXT("SELECTIONLOCK="), DWord1) )
	{
		FinishAllSnaps();
		// If -1 is passed in, treat it as a toggle.  Otherwise, use the value as a literal assignment.
		if( DWord1 == -1 )
		{
			GEdSelectionLock=(GEdSelectionLock == 0) ? 1 : 0;
		}
		else
		{
			GEdSelectionLock=!!DWord1;
		}
	}

	if( FParse::Value(Str,TEXT("USESIZINGBOX="), DWord1) )
	{
		FinishAllSnaps();
		// If -1 is passed in, treat it as a toggle.  Otherwise, use the value as a literal assignment.
		if( DWord1 == -1 )
			UseSizingBox=(UseSizingBox == 0) ? 1 : 0;
		else
			UseSizingBox=DWord1;
	}
	
	if(GCurrentLevelEditingViewportClient)
	{
		float NewCameraSpeed = 1.0f;
		if ( FParse::Value( Str, TEXT("SPEED="), NewCameraSpeed ) )
		{
			GetMutableDefault<ULevelEditorViewportSettings>()->CameraSpeedSettings.SetCurrentSpeed(NewCameraSpeed); 
		
			GetMutableDefault<ULevelEditorViewportSettings>()->CameraSpeed_DEPRECATED = NewCameraSpeed;
		}
	}

	FParse::Value( Str, TEXT("SNAPDIST="), GetMutableDefault<ULevelEditorViewportSettings>()->SnapDistance );
	
	//
	// Major modes:
	//
	FEditorModeID EditorMode = FBuiltinEditorModes::EM_None;

	FString CommandToken = FParse::Token(Str, false);
	FEdMode* FoundMode = GLevelEditorModeTools().GetActiveMode(FName(*CommandToken));

	if (FoundMode != NULL)
	{
		EditorMode = FName( *CommandToken );
	}

	if( EditorMode != FBuiltinEditorModes::EM_None )
	{
		FEditorDelegates::ChangeEditorMode.Broadcast(EditorMode);
	}

	// Reset the roll on all viewport cameras
	for(FLevelEditorViewportClient* ViewportClient : GetLevelViewportClients())
	{
		if(ViewportClient->IsPerspective())
		{
			ViewportClient->RemoveCameraRoll();
		}
	}

	FEditorSupportDelegates::RedrawAllViewports.Broadcast();

	return true;
}

bool UUnrealEdEngine::Exec_Group( const TCHAR* Str, FOutputDevice& Ar )
{
	if(UActorGroupingUtils::IsGroupingActive())
	{
		if( FParse::Command(&Str,TEXT("REGROUP")) )
		{
			UActorGroupingUtils::Get()->GroupSelected();
			return true;
		}
		else if ( FParse::Command(&Str,TEXT("UNGROUP")) )
		{
			UActorGroupingUtils::Get()->UngroupSelected();
			return true;
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
