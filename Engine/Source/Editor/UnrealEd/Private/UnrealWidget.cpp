// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnrealWidget.h"

#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "DynamicMeshBuilder.h"
#include "EdMode.h"
#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "GenericPlatform/ICursor.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Misc/AxisDisplayInfo.h"
#include "SceneView.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "SnappingUtils.h"
#include "TransformGizmoEditorSettings.h"

IMPLEMENT_HIT_PROXY(HWidgetAxis, HHitProxy);

constexpr float FWidget::AXIS_LENGTH;
constexpr float FWidget::TRANSLATE_ROTATE_AXIS_CIRCLE_RADIUS;
constexpr float FWidget::TWOD_AXIS_CIRCLE_RADIUS;
constexpr float FWidget::INNER_AXIS_CIRCLE_RADIUS;
constexpr float FWidget::OUTER_AXIS_CIRCLE_RADIUS;
constexpr float FWidget::ROTATION_TEXT_RADIUS;
constexpr int32 FWidget::AXIS_CIRCLE_SIDES;
constexpr float FWidget::AXIS_LENGTH_SCALE_OFFSET;

FWidget::FWidget()
{
	EditorModeTools      = nullptr;
	TotalDeltaRotation   = 0;
	CurrentDeltaRotation = 0;

	AxisColorX       = AxisDisplayInfo::GetAxisColor(EAxisList::Forward);
	AxisColorY       = AxisDisplayInfo::GetAxisColor(EAxisList::Left);
	AxisColorZ       = AxisDisplayInfo::GetAxisColor(EAxisList::Up);
	ScreenAxisColor  = FLinearColor(0.76f, 0.72f, 0.14f);
	PlaneColorXY     = FColor::Yellow;
	ArcBallColor     = FColor(128, 128, 128, 6);
	ScreenSpaceColor = FColor(196, 196, 196);
	CurrentColor     = FColor::Yellow;

	UMaterial* AxisMaterialBase = GEngine->ArrowMaterial;

	AxisMaterialX = UMaterialInstanceDynamic::Create(AxisMaterialBase, nullptr);
	AxisMaterialX->SetVectorParameterValue("GizmoColor", AxisColorX);

	AxisMaterialY = UMaterialInstanceDynamic::Create(AxisMaterialBase, nullptr);
	AxisMaterialY->SetVectorParameterValue("GizmoColor", AxisColorY);

	AxisMaterialZ = UMaterialInstanceDynamic::Create(AxisMaterialBase, nullptr);
	AxisMaterialZ->SetVectorParameterValue("GizmoColor", AxisColorZ);

	CurrentAxisMaterial = UMaterialInstanceDynamic::Create(AxisMaterialBase, nullptr);
	CurrentAxisMaterial->SetVectorParameterValue("GizmoColor", CurrentColor);

	OpaquePlaneMaterialXY = UMaterialInstanceDynamic::Create(AxisMaterialBase, nullptr);
	OpaquePlaneMaterialXY->SetVectorParameterValue("GizmoColor", FLinearColor::White);

	TransparentPlaneMaterialXY = (UMaterial*)StaticLoadObject(
	    UMaterial::StaticClass(), nullptr,
	    TEXT("/Engine/EditorMaterials/WidgetVertexColorMaterial.WidgetVertexColorMaterial"), nullptr, LOAD_None, nullptr);

	GridMaterial = (UMaterial*)StaticLoadObject(
	    UMaterial::StaticClass(), nullptr,
	    TEXT("/Engine/EditorMaterials/WidgetGridVertexColorMaterial_Ma.WidgetGridVertexColorMaterial_Ma"), nullptr,
	    LOAD_None, nullptr);
	if (!GridMaterial)
	{
		GridMaterial = TransparentPlaneMaterialXY;
	}

	CurrentAxis = EAxisList::None;

	CustomCoordSystem      = FMatrix::Identity;
	CustomCoordSystemSpace = COORD_World;

	bAbsoluteTranslationInitialOffsetCached = false;
	InitialTranslationOffset                = FVector::ZeroVector;
	InitialTranslationPosition              = FVector(0, 0, 0);

	bDragging               = false;
	bSnapEnabled            = false;
	bDefaultVisibility      = true;
	bIsOrthoDrawingFullRing = false;

	Origin       = FVector2D::ZeroVector;
	XAxisDir     = FVector2D::ZeroVector;
	YAxisDir     = FVector2D::ZeroVector;
	ZAxisDir     = FVector2D::ZeroVector;
	DragStartPos = FVector2D::ZeroVector;
	LastDragPos  = FVector2D::ZeroVector;
}

extern ENGINE_API void StringSize(UFont* Font, int32& XL, int32& YL, const TCHAR* Text, FCanvas* Canvas);

void FWidget::SetUsesEditorModeTools(FEditorModeTools* InEditorModeTools)
{
	EditorModeTools = InEditorModeTools;
}

void FWidget::ConvertMouseToAxis_Translate(FVector2D DragDir, FVector& InOutDelta, FVector& OutDrag) const
{
	// Get drag delta in widget axis space
	OutDrag = FVector((CurrentAxis & EAxisList::X) ? FVector2D::DotProduct(XAxisDir, DragDir) : 0.0f,
	                  (CurrentAxis & EAxisList::Y) ? FVector2D::DotProduct(YAxisDir, DragDir) : 0.0f,
	                  (CurrentAxis & EAxisList::Z) ? FVector2D::DotProduct(ZAxisDir, DragDir) : 0.0f);

	// Snap to grid in widget axis space
	const FVector GridSize = FVector(GEditor->GetGridSize());
	FSnappingUtils::SnapPointToGrid(OutDrag, GridSize);

	// Convert to effective screen space delta, and replace input delta, adjusted for inverted screen space Y axis
	const FVector2D EffectiveDelta = OutDrag.X * XAxisDir + OutDrag.Y * YAxisDir + OutDrag.Z * ZAxisDir;
	InOutDelta                     = FVector(EffectiveDelta.X, -EffectiveDelta.Y, 0.0f);

	if (AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward)
	{
		OutDrag.Y = -OutDrag.Y;
	}

	// Transform drag delta into world space
	OutDrag = CustomCoordSystem.TransformPosition(OutDrag);
}

//CVAR for the arcball size, so animators can adjust it
static TAutoConsoleVariable<float> CVarArcballLimit(
	TEXT("r.Editor.ArcballDragLimit"),
	2.0,
	TEXT("For how long the arcball rotates until it switches to a screens space rotate, default of 1.0 equals the size of the arcball"),
	ECVF_RenderThreadSafe
);
//function used to find quat between two angles, and works much better, faster and less degenerate, then trying to use cross and dot product
static FQuat FindQuatBetweenNormals(const FVector& A, const FVector& B)
{
	const FQuat::FReal Dot = FVector::DotProduct(A, B);
	FQuat::FReal W = 1 + Dot;
	FQuat Result;

	if (W < SMALL_NUMBER)
	{
		// A and B point in opposite directions
		W = 2 - W;
		Result = FQuat(-A.Y * B.Z + A.Z * B.Y, -A.Z * B.X + A.X * B.Z, -A.X * B.Y + A.Y * B.X, W).GetNormalized();

		const FVector Normal = FMath::Abs(A.X) > FMath::Abs(A.Y) ? FVector::YAxisVector : FVector::XAxisVector;
		const FVector BiNormal = FVector::CrossProduct(A, Normal);
		const FVector TauNormal = FVector::CrossProduct(A, BiNormal);
		Result = Result * FQuat(TauNormal, PI);
	}
	else
	{
		//Axis = FVector::CrossProduct(A, B);
		Result = FQuat(A.Y * B.Z - A.Z * B.Y, A.Z * B.X - A.X * B.Z, A.X * B.Y - A.Y * B.X, W);
	}

	Result.Normalize();
	return Result;
};

void FWidget::ConvertMouseToAxis_Rotate(FVector2D TangentDir, FVector2D DragDir, FSceneView* InView,
	FEditorViewportClient* InViewportClient, FVector& InOutDelta,
	FRotator& OutRotation)
{
	if (CurrentAxis == EAxisList::X)
	{
		FRotator Rotation;
		FVector2D EffectiveDelta;
		// Get screen direction representing positive rotation
		const FVector2D AxisDir = bIsOrthoDrawingFullRing ? TangentDir : XAxisDir;

		// Get rotation in widget local space
		Rotation = FRotator(0, 0, FVector2D::DotProduct(AxisDir, DragDir));
		FSnappingUtils::SnapRotatorToGrid(Rotation);

		// Record delta rotation (used by the widget to render the accumulated delta)
		CurrentDeltaRotation = static_cast<float>(Rotation.Roll);

		// Use to calculate the new input delta
		EffectiveDelta = AxisDir * Rotation.Roll;
		// Adjust the input delta according to how much rotation was actually applied
		InOutDelta = FVector(EffectiveDelta.X, -EffectiveDelta.Y, 0.0f);
		// Need to get the delta rotation in the current coordinate space of the widget
		OutRotation = (CustomCoordSystem.Inverse() * FRotationMatrix(Rotation) * CustomCoordSystem).Rotator();
	}
	else if (CurrentAxis == EAxisList::Y)
	{
		FRotator Rotation;
		FVector2D EffectiveDelta;
		// TODO: Determine why -TangentDir is necessary here, and fix whatever is causing it
		const FVector2D AxisDir = bIsOrthoDrawingFullRing ? -TangentDir : YAxisDir;

		Rotation = FRotator(FVector2D::DotProduct(AxisDir, DragDir), 0, 0);
		FSnappingUtils::SnapRotatorToGrid(Rotation);

		CurrentDeltaRotation = static_cast<float>(Rotation.Pitch);
		EffectiveDelta = AxisDir * Rotation.Pitch;
		// Adjust the input delta according to how much rotation was actually applied
		InOutDelta = FVector(EffectiveDelta.X, -EffectiveDelta.Y, 0.0f);
		// Need to get the delta rotation in the current coordinate space of the widget
		OutRotation = (CustomCoordSystem.Inverse() * FRotationMatrix(Rotation) * CustomCoordSystem).Rotator();
	}
	else if (CurrentAxis == EAxisList::Z)
	{
		FRotator Rotation;
		FVector2D EffectiveDelta;
		const FVector2D AxisDir = bIsOrthoDrawingFullRing ? TangentDir : ZAxisDir;

		Rotation = FRotator(0, FVector2D::DotProduct(AxisDir, DragDir), 0);
		FSnappingUtils::SnapRotatorToGrid(Rotation);

		CurrentDeltaRotation = static_cast<float>(Rotation.Yaw);
		EffectiveDelta = AxisDir * Rotation.Yaw;
		// Adjust the input delta according to how much rotation was actually applied
		InOutDelta = FVector(EffectiveDelta.X, -EffectiveDelta.Y, 0.0f);
		// Need to get the delta rotation in the current coordinate space of the widget
		OutRotation = (CustomCoordSystem.Inverse() * FRotationMatrix(Rotation) * CustomCoordSystem).Rotator();
	}
	else if (CurrentAxis == EAxisList::XYZ) //arcball rotate
	{
		//For Arball rotate we have three states we can be in
		// If within the OUTER_AXIS_CIRCLE_RADIUS, we hit test the rays from current and old camera to the widget with that circle radius, and then find the angle
		// between the vectors form the center of the widget to those intersections.
		// If ouside the arcball, we still want it to rotate like an arcball since our is smaller than other DCC's and doesnt' feel right.
		// We use a CVAR to specify how far out to do this extra pseudo arcball rotation, 2.0 feels right for animators.
		// So in this case the Axis is the cross product of the current ray from eye to pixel in world space with the previous ray.
		// The Angle is angle amount we rotate from the object's location to the imaginary sphere that matches up with the difference 
		// between the current and previous ray from the Camera. Those rays form a triangle, which we can bisect with a common side, to find the angle
		// This gives us an arcball like rotation beyond the normal arcball.
		// Finally if outside the specified cvar distance we do a screen rotate like other DCC's
		FVector2D MousePosition(InViewportClient->Viewport->GetMouseX(), InViewportClient->Viewport->GetMouseY());
		FViewportCursorLocation OldMouseViewportRay(InView, InViewportClient, LastDragPos.X, LastDragPos.Y);
		FViewportCursorLocation MouseViewportRay(InView, InViewportClient, MousePosition.X, MousePosition.Y);
		LastDragPos = MousePosition;
		FVector DirectionToWidget = InViewportClient->GetWidgetLocation() - MouseViewportRay.GetOrigin();
		float Length = DirectionToWidget.Size();
		if (!FMath::IsNearlyZero(Length)) //degenerate check
		{
			//Find screen space distance to the arcball size
			DirectionToWidget /= Length;
			const FVector CameraToPixelDir = MouseViewportRay.GetDirection();
			const FVector OldCameraToPixelDir = OldMouseViewportRay.GetDirection();
			FVector RotationAxis = FVector::CrossProduct(CameraToPixelDir, OldCameraToPixelDir);
			RotationAxis.Normalize();
			float RotationAngle = 0.0f;
			FVector4 ScreenLocation = InView->WorldToScreen(InViewportClient->GetWidgetLocation());
			FVector2D PixelLocation;
			InView->ScreenToPixel(ScreenLocation, PixelLocation);
			float Distance = FVector2D::Distance(PixelLocation, MousePosition);
			const float ExternalScale = EditorModeTools ? EditorModeTools->GetWidgetScale() : 1.0f;
			const float CircleRadius = OUTER_AXIS_CIRCLE_RADIUS * ExternalScale;
			const float ArcballLimit = CVarArcballLimit.GetValueOnGameThread() * 2.0f;
			const float MaxDiff = ArcballLimit * CircleRadius;
			//If within Arcball do rotate the ball based on angle difference on hit tested sphere 
			if (Distance < CircleRadius)
			{
				const float ScaleInScreen = InView->WorldToScreen(InViewportClient->GetWidgetLocation()).W * (4.0f / InView->UnscaledViewRect.Width() / InView->ViewMatrices.GetViewToClip().M[0][0]);
				const float SphereRadius = CircleRadius * ScaleInScreen  + GetDefault<UTransformGizmoEditorSettings>()->TransformGizmoSize;
				FVector OldLocation, NewLocation;
				FMath::SphereDistToLine(InViewportClient->GetWidgetLocation(), SphereRadius, MouseViewportRay.GetOrigin(), OldCameraToPixelDir, OldLocation);
				FMath::SphereDistToLine(InViewportClient->GetWidgetLocation(), SphereRadius, MouseViewportRay.GetOrigin(), CameraToPixelDir, NewLocation);
				FVector OldLineToCenter = OldLocation - InViewportClient->GetWidgetLocation();
				OldLineToCenter.Normalize();
				FVector NewLineToCenter = NewLocation - InViewportClient->GetWidgetLocation();
				NewLineToCenter.Normalize();
				const FQuat QuatRotation = FindQuatBetweenNormals(OldLineToCenter, NewLineToCenter);
				OutRotation = FRotator(QuatRotation);
			}
			else if (Distance > CircleRadius && Distance < MaxDiff) //do 
			{
				const float Scale = ScreenLocation.W *
					(4.0f / InView->UnscaledViewRect.Width() / InView->ViewMatrices.GetViewToClip().M[0][0]);
				const float Radius = (CircleRadius * Scale) +
					GetDefault<UTransformGizmoEditorSettings>()->TransformGizmoSize;
				const float LengthOfAdjacent = Length - Radius;
				RotationAngle = FMath::Acos(FVector::DotProduct(OldCameraToPixelDir, CameraToPixelDir));
				const float OppositeSize = FMath::Tan(RotationAngle) * LengthOfAdjacent;
				RotationAngle = FMath::Atan2(OppositeSize, Radius);
				const FQuat QuatRotation(RotationAxis, RotationAngle);
				OutRotation = FRotator(QuatRotation);
			}
			else if (Distance > MaxDiff) 	//If outside radius, do screen rotate instead, like other DCC's
			{
				FPlane Plane(InViewportClient->GetWidgetLocation(), DirectionToWidget);
				FVector StartOnPlane =
					FMath::RayPlaneIntersection(MouseViewportRay.GetOrigin(), CameraToPixelDir, Plane);
				FVector OldOnPlane =
					FMath::RayPlaneIntersection(MouseViewportRay.GetOrigin(), OldCameraToPixelDir, Plane);
				StartOnPlane -= InViewportClient->GetWidgetLocation();
				OldOnPlane -= InViewportClient->GetWidgetLocation();
				StartOnPlane.Normalize();
				OldOnPlane.Normalize();
				RotationAngle = FMath::Acos(FVector::DotProduct(StartOnPlane, OldOnPlane));

				FVector Cross = FVector::CrossProduct(OldCameraToPixelDir, CameraToPixelDir);
				if (FVector::DotProduct(DirectionToWidget, Cross) < 0.0f)
				{
					RotationAngle *= -1.0f;
				}
				RotationAxis = DirectionToWidget;
				const FQuat QuatRotation(RotationAxis, RotationAngle);
				OutRotation = FRotator(QuatRotation);
			}
		}
		return;
	}
	else if (CurrentAxis == EAxisList::Screen)
	{
		FVector2D MousePosition(InViewportClient->Viewport->GetMouseX(), InViewportClient->Viewport->GetMouseY());
		FViewportCursorLocation OldMouseViewportRay(InView, InViewportClient, LastDragPos.X, LastDragPos.Y);
		FViewportCursorLocation MouseViewportRay(InView, InViewportClient, MousePosition.X, MousePosition.Y);

		LastDragPos = MousePosition;
		FVector DirectionToWidget = InViewportClient->GetWidgetLocation() - MouseViewportRay.GetOrigin();
		float Length = DirectionToWidget.Size();

		if (!FMath::IsNearlyZero(Length))
		{
			DirectionToWidget /= Length;

			const FVector CameraToPixelDir = MouseViewportRay.GetDirection();
			const FVector OldCameraToPixelDir = OldMouseViewportRay.GetDirection();
			FPlane Plane(InViewportClient->GetWidgetLocation(), DirectionToWidget);
			FVector StartOnPlane = FMath::RayPlaneIntersection(MouseViewportRay.GetOrigin(), CameraToPixelDir, Plane);
			FVector OldOnPlane = FMath::RayPlaneIntersection(MouseViewportRay.GetOrigin(), OldCameraToPixelDir, Plane);
			StartOnPlane -= InViewportClient->GetWidgetLocation();
			OldOnPlane -= InViewportClient->GetWidgetLocation();
			StartOnPlane.Normalize();
			OldOnPlane.Normalize();
			float RotationAngle = FMath::Acos(FVector::DotProduct(StartOnPlane, OldOnPlane));
			FVector Cross = FVector::CrossProduct(OldCameraToPixelDir, CameraToPixelDir);
			if (FVector::DotProduct(DirectionToWidget, Cross) < 0.0f)
			{
				RotationAngle *= -1.0f;
			}
			const FQuat QuatRotation(DirectionToWidget, RotationAngle);
			OutRotation = FRotator(QuatRotation);
		}
		return;
	}
}

void FWidget::ConvertMouseToAxis_Scale(FVector2D DragDir, FVector& InOutDelta, FVector& OutScale)
{
	FVector2D AxisDir = FVector2D::ZeroVector;

	if (CurrentAxis & EAxisList::X)
	{
		AxisDir += XAxisDir;
	}

	if (CurrentAxis & EAxisList::Y)
	{
		AxisDir += YAxisDir;
	}

	if (CurrentAxis & EAxisList::Z)
	{
		AxisDir += ZAxisDir;
	}

	AxisDir.Normalize();
	const float ScaleDelta = FVector2D::DotProduct(AxisDir, DragDir);

	OutScale =
	    FVector((CurrentAxis & EAxisList::X) ? ScaleDelta : 0.0f, (CurrentAxis & EAxisList::Y) ? ScaleDelta : 0.0f,
	            (CurrentAxis & EAxisList::Z) ? ScaleDelta : 0.0f);

	// Snap to grid in widget axis space
	const FVector GridSize = FVector(GEditor->GetGridSize());
	FSnappingUtils::SnapScale(OutScale, GridSize);

	// Convert to effective screen space delta, and replace input delta, adjusted for inverted screen space Y axis
	const float ScaleMax           = OutScale.GetMax();
	const float ScaleMin           = OutScale.GetMin();
	const float ScaleApplied       = (ScaleMax > -ScaleMin) ? ScaleMax : ScaleMin;
	const FVector2D EffectiveDelta = AxisDir * ScaleApplied;
	InOutDelta                     = FVector(EffectiveDelta.X, -EffectiveDelta.Y, 0.0f);
}

void FWidget::ConvertMouseToAxis_TranslateRotateZ(FVector2D TangentDir, FVector2D DragDir, FVector& InOutDelta,
                                                  FVector& OutDrag, FRotator& OutRotation)
{
	if (CurrentAxis == EAxisList::ZRotation)
	{
		const FVector2D AxisDir = bIsOrthoDrawingFullRing ? TangentDir : ZAxisDir;
		FRotator Rotation       = FRotator(0, FVector2D::DotProduct(AxisDir, DragDir), 0);
		FSnappingUtils::SnapRotatorToGrid(Rotation);
		CurrentDeltaRotation = Rotation.Yaw;

		const FVector2D EffectiveDelta = AxisDir * Rotation.Yaw;
		InOutDelta                     = FVector(EffectiveDelta.X, -EffectiveDelta.Y, 0.0f);

		OutRotation = (CustomCoordSystem.Inverse() * FRotationMatrix(Rotation) * CustomCoordSystem).Rotator();
	}
	else
	{
		// Get drag delta in widget axis space
		OutDrag = FVector((CurrentAxis & EAxisList::X) ? FVector2D::DotProduct(XAxisDir, DragDir) : 0.0f,
		                  (CurrentAxis & EAxisList::Y) ? FVector2D::DotProduct(YAxisDir, DragDir) : 0.0f,
		                  (CurrentAxis & EAxisList::Z) ? FVector2D::DotProduct(ZAxisDir, DragDir) : 0.0f);

		// Snap to grid in widget axis space
		const FVector GridSize = FVector(GEditor->GetGridSize());
		FSnappingUtils::SnapPointToGrid(OutDrag, GridSize);

		// Convert to effective screen space delta, and replace input delta, adjusted for inverted screen space Y axis
		const FVector2D EffectiveDelta = OutDrag.X * XAxisDir + OutDrag.Y * YAxisDir + OutDrag.Z * ZAxisDir;
		InOutDelta                     = FVector(EffectiveDelta.X, -EffectiveDelta.Y, 0.0f);

		if (AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward)
		{
			OutDrag.Y = -OutDrag.Y;
		}

		// Transform drag delta into world space
		OutDrag = CustomCoordSystem.TransformPosition(OutDrag);
	}
}

void FWidget::ConvertMouseToAxis_WM_2D(FVector2D TangentDir, FVector2D DragDir, FVector& InOutDelta, FVector& OutDrag,
                                       FRotator& OutRotation)
{
	if (CurrentAxis == EAxisList::Rotate2D)
	{
		// TODO: Determine why -TangentDir is necessary here, and fix whatever is causing it
		const FVector2D AxisDir = bIsOrthoDrawingFullRing ? -TangentDir : YAxisDir;

		FRotator Rotation = FRotator(FVector2D::DotProduct(AxisDir, DragDir), 0, 0);
		FSnappingUtils::SnapRotatorToGrid(Rotation);

		CurrentDeltaRotation     = Rotation.Pitch;
		FVector2D EffectiveDelta = AxisDir * Rotation.Pitch;


		// Adjust the input delta according to how much rotation was actually applied
		InOutDelta = FVector(EffectiveDelta.X, -EffectiveDelta.Y, 0.0f);

		// Need to get the delta rotation in the current coordinate space of the widget
		OutRotation = (CustomCoordSystem.Inverse() * FRotationMatrix(Rotation) * CustomCoordSystem).Rotator();
	}
	else
	{
		// Get drag delta in widget axis space
		OutDrag = FVector((CurrentAxis & EAxisList::X) ? FVector2D::DotProduct(XAxisDir, DragDir) : 0.0f,
		                  (CurrentAxis & EAxisList::Y) ? FVector2D::DotProduct(YAxisDir, DragDir) : 0.0f,
		                  (CurrentAxis & EAxisList::Z) ? FVector2D::DotProduct(ZAxisDir, DragDir) : 0.0f);

		// Snap to grid in widget axis space
		const FVector GridSize = FVector(GEditor->GetGridSize());
		FSnappingUtils::SnapPointToGrid(OutDrag, GridSize);

		// Convert to effective screen space delta, and replace input delta, adjusted for inverted screen space Y axis
		const FVector2D EffectiveDelta = OutDrag.X * XAxisDir + OutDrag.Y * YAxisDir + OutDrag.Z * ZAxisDir;
		InOutDelta                     = FVector(EffectiveDelta.X, -EffectiveDelta.Y, 0.0f);

		// Transform drag delta into world space
		OutDrag = CustomCoordSystem.TransformPosition(OutDrag);
	}
}

/**
 * Converts mouse movement on the screen to widget axis movement/rotation.
 */
void FWidget::ConvertMouseMovementToAxisMovement(FSceneView* InView, FEditorViewportClient* InViewportClient,
                                                 bool bInUsedDragModifier, FVector& InOutDelta, FVector& OutDrag,
                                                 FRotator& OutRotation, FVector& OutScale)
{
	OutDrag     = FVector::ZeroVector;
	OutRotation = FRotator::ZeroRotator;
	OutScale    = FVector::ZeroVector;

	const int32 WidgetMode = InViewportClient->GetWidgetMode();
	
	// Get input delta as 2D vector, adjusted for inverted screen space Y axis
	const FVector2D DragDir = FVector2D(InOutDelta.X, -InOutDelta.Y);

	// Get offset of the drag start position from the widget origin
	const FVector2D DirectionToMousePos = FVector2D(DragStartPos - Origin).GetSafeNormal();

	// For rotations which display as a full ring, calculate the tangent direction representing a clockwise movement
	FVector2D TangentDir = bInUsedDragModifier ?
	    // If a drag modifier has been used, this implies we are not actually touching the widget, so don't attempt to
	    // calculate the tangent dir based on the relative offset of the cursor from the widget location.
	    FVector2D(1, 1).GetSafeNormal() :
	    // Treat the tangent dir as perpendicular to the relative offset of the cursor from the widget location.
	    FVector2D(-DirectionToMousePos.Y, DirectionToMousePos.X);

	switch (WidgetMode)
	{
	case UE::Widget::EWidgetMode::WM_Translate:
		ConvertMouseToAxis_Translate(DragDir, InOutDelta, OutDrag);
		break;
	case UE::Widget::EWidgetMode::WM_Rotate:
		ConvertMouseToAxis_Rotate(TangentDir, DragDir, InView, InViewportClient, InOutDelta, OutRotation);
		break;
	case UE::Widget::EWidgetMode::WM_Scale:
		ConvertMouseToAxis_Scale(DragDir, InOutDelta, OutScale);
		break;
	case UE::Widget::EWidgetMode::WM_TranslateRotateZ:
		ConvertMouseToAxis_TranslateRotateZ(TangentDir, DragDir, InOutDelta, OutDrag, OutRotation);
		break;
	case UE::Widget::EWidgetMode::WM_2D:
		ConvertMouseToAxis_WM_2D(TangentDir, DragDir, InOutDelta, OutDrag, OutRotation);
		break;
	default:
		break;
	}
}

/**
 * For axis movement, get the "best" planar normal and axis mask
 * @param InAxis - Axis of movement
 * @param InDirToPixel -
 * @param OutPlaneNormal - Normal of the plane to project the mouse onto
 * @param OutMask - Used to mask out the component of the planar movement we want
 */
void GetAxisPlaneNormalAndMask(const FMatrix& InCoordSystem, const FVector& InAxis, const FVector& InDirToPixel,
                               FVector& OutPlaneNormal, FVector& NormalToRemove)
{
	FVector XAxis = InCoordSystem.TransformVector(FVector(1, 0, 0));
	FVector YAxis = InCoordSystem.TransformVector(FVector(0, 1, 0));
	FVector ZAxis = InCoordSystem.TransformVector(FVector(0, 0, 1));

	float XDot = FMath::Abs(InDirToPixel | XAxis);
	float YDot = FMath::Abs(InDirToPixel | YAxis);
	float ZDot = FMath::Abs(InDirToPixel | ZAxis);

	if ((InAxis | XAxis) > .1f)
	{
		OutPlaneNormal = (YDot > ZDot) ? YAxis : ZAxis;
		NormalToRemove = (YDot > ZDot) ? ZAxis : YAxis;
	}
	else if ((InAxis | YAxis) > .1f)
	{
		OutPlaneNormal = (XDot > ZDot) ? XAxis : ZAxis;
		NormalToRemove = (XDot > ZDot) ? ZAxis : XAxis;
	}
	else
	{
		OutPlaneNormal = (XDot > YDot) ? XAxis : YAxis;
		NormalToRemove = (XDot > YDot) ? YAxis : XAxis;
	}
}

/**
 * For planar movement, get the "best" planar normal and axis mask
 * @param InAxis - Axis of movement
 * @param OutPlaneNormal - Normal of the plane to project the mouse onto
 * @param OutMask - Used to mask out the component of the planar movement we want
 */
void GetPlaneNormalAndMask(const FVector& InAxis, FVector& OutPlaneNormal, FVector& NormalToRemove)
{
	OutPlaneNormal = InAxis;
	NormalToRemove = InAxis;
}

void FWidget::AbsoluteConvertMouseToAxis_Translate(FSceneView* InView, const FMatrix& InputCoordSystem,
                                                   FAbsoluteMovementParams& InOutParams, FVector& OutDrag)
{
	switch (CurrentAxis)
	{
	case EAxisList::X:
		GetAxisPlaneNormalAndMask(InputCoordSystem, InOutParams.XAxis, InOutParams.CameraDir, InOutParams.PlaneNormal,
		                          InOutParams.NormalToRemove);
		break;
	case EAxisList::Y:
		GetAxisPlaneNormalAndMask(InputCoordSystem, InOutParams.YAxis, InOutParams.CameraDir, InOutParams.PlaneNormal,
		                          InOutParams.NormalToRemove);
		break;
	case EAxisList::Z:
		GetAxisPlaneNormalAndMask(InputCoordSystem, InOutParams.ZAxis, InOutParams.CameraDir, InOutParams.PlaneNormal,
		                          InOutParams.NormalToRemove);
		break;
	case EAxisList::XY:
		GetPlaneNormalAndMask(InOutParams.ZAxis, InOutParams.PlaneNormal, InOutParams.NormalToRemove);
		break;
	case EAxisList::XZ:
		GetPlaneNormalAndMask(InOutParams.YAxis, InOutParams.PlaneNormal, InOutParams.NormalToRemove);
		break;
	case EAxisList::YZ:
		GetPlaneNormalAndMask(InOutParams.XAxis, InOutParams.PlaneNormal, InOutParams.NormalToRemove);
		break;
	case EAxisList::Screen:
		InOutParams.XAxis = InView->ViewMatrices.GetWorldToView().GetColumn(0);
		InOutParams.YAxis = InView->ViewMatrices.GetWorldToView().GetColumn(1);
		InOutParams.ZAxis = InView->ViewMatrices.GetWorldToView().GetColumn(2);
		GetPlaneNormalAndMask(InOutParams.ZAxis, InOutParams.PlaneNormal, InOutParams.NormalToRemove);
		//do not damp the movement in this case, we also want to snap
		InOutParams.bMovementLockedToCamera = false;
		break;
	}

	OutDrag = GetAbsoluteTranslationDelta(InOutParams);
}

void FWidget::AbsoluteConvertMouseToAxis_WM_2D(const FMatrix& InputCoordSystem, FAbsoluteMovementParams& InOutParams,
                                               FVector& OutDrag, FRotator& OutRotation)
{
	switch (CurrentAxis)
	{
	case EAxisList::X: {
		GetAxisPlaneNormalAndMask(InputCoordSystem, InOutParams.XAxis, InOutParams.CameraDir, InOutParams.PlaneNormal,
		                          InOutParams.NormalToRemove);
		OutDrag = GetAbsoluteTranslationDelta(InOutParams);
		break;
	}
	case EAxisList::Z: {
		GetAxisPlaneNormalAndMask(InputCoordSystem, InOutParams.ZAxis, InOutParams.CameraDir, InOutParams.PlaneNormal,
		                          InOutParams.NormalToRemove);
		OutDrag = GetAbsoluteTranslationDelta(InOutParams);
		break;
	}
	case EAxisList::XZ: {
		GetPlaneNormalAndMask(InOutParams.YAxis, InOutParams.PlaneNormal, InOutParams.NormalToRemove);
		OutDrag = GetAbsoluteTranslationDelta(InOutParams);
		break;
	}

	//Rotate about the y-axis
	case EAxisList::Rotate2D: {
		//no position snapping, we'll handle the rotation snapping elsewhere
		InOutParams.bPositionSnapping = false;

		GetPlaneNormalAndMask(InOutParams.YAxis, InOutParams.PlaneNormal, InOutParams.NormalToRemove);
		//No DAMPING
		InOutParams.bMovementLockedToCamera = false;
		//this is the one movement type where we want to always use the widget origin and
		//NOT the "first click" origin
		FVector XZPlaneProjectedPosition = GetAbsoluteTranslationDelta(InOutParams) + InitialTranslationOffset;

		//remove the component along the normal we want to mute
		float MovementAlongMutedAxis = XZPlaneProjectedPosition | InOutParams.NormalToRemove;
		XZPlaneProjectedPosition     = XZPlaneProjectedPosition - (InOutParams.NormalToRemove * MovementAlongMutedAxis);

		if (!XZPlaneProjectedPosition.Normalize())
		{
			XZPlaneProjectedPosition = InOutParams.YAxis;
		}

		//NOW, find the rotation around the PlaneNormal to make the xaxis point at InDrag
		OutRotation = FRotator::ZeroRotator;

		float PitchDegrees = -FMath::Atan2(-XZPlaneProjectedPosition.Z, XZPlaneProjectedPosition.X) * 180.f / PI;
		OutRotation.Pitch  = PitchDegrees - (EditorModeTools ? EditorModeTools->TranslateRotate2DAngle : 0);

		if (bSnapEnabled)
		{
			FSnappingUtils::SnapRotatorToGrid(OutRotation);
		}

		break;
	}
	}
}
void FWidget::AbsoluteConvertMouseToAxis_TranslateRotateZ(const FMatrix& InputCoordSystem,
                                                          FAbsoluteMovementParams& InOutParams, FVector& OutDrag,
                                                          FRotator& OutRotation)
{
	switch (CurrentAxis)
	{
	case EAxisList::X: {
		GetAxisPlaneNormalAndMask(InputCoordSystem, InOutParams.XAxis, InOutParams.CameraDir, InOutParams.PlaneNormal,
		                          InOutParams.NormalToRemove);
		OutDrag = GetAbsoluteTranslationDelta(InOutParams);
		break;
	}
	case EAxisList::Y: {
		GetAxisPlaneNormalAndMask(InputCoordSystem, InOutParams.YAxis, InOutParams.CameraDir, InOutParams.PlaneNormal,
		                          InOutParams.NormalToRemove);
		OutDrag = GetAbsoluteTranslationDelta(InOutParams);
		break;
	}
	case EAxisList::Z: {
		GetAxisPlaneNormalAndMask(InputCoordSystem, InOutParams.ZAxis, InOutParams.CameraDir, InOutParams.PlaneNormal,
		                          InOutParams.NormalToRemove);
		OutDrag = GetAbsoluteTranslationDelta(InOutParams);
		break;
	}
	case EAxisList::XY: {
		GetPlaneNormalAndMask(InOutParams.ZAxis, InOutParams.PlaneNormal, InOutParams.NormalToRemove);
		OutDrag = GetAbsoluteTranslationDelta(InOutParams);
		break;
	}
	//Rotate about the z-axis
	case EAxisList::ZRotation: {
		//no position snapping, we'll handle the rotation snapping elsewhere
		InOutParams.bPositionSnapping = false;

		//find new point on the
		GetPlaneNormalAndMask(InOutParams.ZAxis, InOutParams.PlaneNormal, InOutParams.NormalToRemove);
		//No DAMPING
		InOutParams.bMovementLockedToCamera = false;
		//this is the one movement type where we want to always use the widget origin and
		//NOT the "first click" origin
		FVector XYPlaneProjectedPosition = GetAbsoluteTranslationDelta(InOutParams) + InitialTranslationOffset;

		//remove the component along the normal we want to mute
		float MovementAlongMutedAxis = XYPlaneProjectedPosition | InOutParams.NormalToRemove;
		XYPlaneProjectedPosition     = XYPlaneProjectedPosition - (InOutParams.NormalToRemove * MovementAlongMutedAxis);

		if (!XYPlaneProjectedPosition.Normalize())
		{
			XYPlaneProjectedPosition = InOutParams.XAxis;
		}

		//NOW, find the rotation around the PlaneNormal to make the xaxis point at InDrag
		OutRotation = FRotator::ZeroRotator;

		OutRotation.Yaw = XYPlaneProjectedPosition.Rotation().Yaw -
		    (EditorModeTools ? EditorModeTools->TranslateRotateXAxisAngle : 0);

		if (bSnapEnabled)
		{
			FSnappingUtils::SnapRotatorToGrid(OutRotation);
		}

		break;
	}
	default:
		break;
	}
}

/**
 * Absolute Translation conversion from mouse movement on the screen to widget axis movement/rotation.
 */
void FWidget::AbsoluteTranslationConvertMouseMovementToAxisMovement(FSceneView* InView,
                                                                    FEditorViewportClient* InViewportClient,
                                                                    const FVector& InLocation,
                                                                    const FVector2D& InMousePosition, FVector& OutDrag,
                                                                    FRotator& OutRotation, FVector& OutScale)
{
	// Compute a world space ray from the screen space mouse coordinates
	FViewportCursorLocation MouseViewportRay(InView, InViewportClient, InMousePosition.X, InMousePosition.Y);

	FAbsoluteMovementParams Params;
	Params.EyePos    = MouseViewportRay.GetOrigin();
	Params.PixelDir  = MouseViewportRay.GetDirection();
	Params.CameraDir = InView->GetViewDirection();
	Params.Position  = InLocation;
	//dampen by
	Params.bMovementLockedToCamera = InViewportClient->IsShiftPressed();
	Params.bPositionSnapping       = true;

	FMatrix InputCoordSystem = InViewportClient->GetWidgetCoordSystem();

	Params.XAxis = InputCoordSystem.TransformVector(FVector(1, 0, 0));
	Params.YAxis = InputCoordSystem.TransformVector(FVector(0, 1, 0));
	Params.ZAxis = InputCoordSystem.TransformVector(FVector(0, 0, 1));

	switch (InViewportClient->GetWidgetMode())
	{
	case UE::Widget::EWidgetMode::WM_Translate:
		AbsoluteConvertMouseToAxis_Translate(InView, InputCoordSystem, Params, OutDrag);
		break;

	case UE::Widget::EWidgetMode::WM_2D:
		AbsoluteConvertMouseToAxis_WM_2D(InputCoordSystem, Params, OutDrag, OutRotation);
		break;
	case UE::Widget::EWidgetMode::WM_TranslateRotateZ:
		AbsoluteConvertMouseToAxis_TranslateRotateZ(InputCoordSystem, Params, OutDrag, OutRotation);
		break;
	case UE::Widget::EWidgetMode::WM_Rotate:
	case UE::Widget::EWidgetMode::WM_Scale:
	case UE::Widget::EWidgetMode::WM_None:
	case UE::Widget::EWidgetMode::WM_Max:
		break;
	}
}

/** Only some modes support Absolute Translation Movement */
bool FWidget::AllowsAbsoluteTranslationMovement(UE::Widget::EWidgetMode WidgetMode)
{
	if ((WidgetMode == UE::Widget::EWidgetMode::WM_Translate) || (WidgetMode == UE::Widget::EWidgetMode::WM_TranslateRotateZ) || (WidgetMode == UE::Widget::EWidgetMode::WM_2D))
	{
		return true;
	}
	return false;
}

/** Only some modes support Absolute Rotation Movement/arcball*/
bool FWidget::AllowsAbsoluteRotationMovement(UE::Widget::EWidgetMode WidgetMode, EAxisList::Type InAxisType)
{
	if (WidgetMode == UE::Widget::EWidgetMode::WM_Rotate && (InAxisType == EAxisList::XYZ || InAxisType == EAxisList::Screen))
	{
		return true;
	}
	return false;
}

/**
 * Serializes the widget references so they don't get garbage collected.
 *
 * @param Ar	FArchive to serialize with
 */
void FWidget::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(AxisMaterialX);
	Collector.AddReferencedObject(AxisMaterialY);
	Collector.AddReferencedObject(AxisMaterialZ);
	Collector.AddReferencedObject(OpaquePlaneMaterialXY);
	Collector.AddReferencedObject(TransparentPlaneMaterialXY);
	Collector.AddReferencedObject(GridMaterial);
	Collector.AddReferencedObject(CurrentAxisMaterial);
}

#define CAMERA_LOCK_DAMPING_FACTOR .1f
#define MAX_CAMERA_MOVEMENT_SPEED 512.0f
/**
 * Returns the Delta from the current position that the absolute movement system wants the object to be at
 * @param InParams - Structure containing all the information needed for absolute movement
 * @return - The requested delta from the current position
 */
FVector FWidget::GetAbsoluteTranslationDelta(const FAbsoluteMovementParams& InParams)
{
	FPlane MovementPlane(InParams.Position, InParams.PlaneNormal);
	FVector ProposedEndofEyeVector =
	    InParams.EyePos + (InParams.PixelDir * (InParams.Position - InParams.EyePos).Size());

	//default to not moving
	FVector RequestedPosition = InParams.Position;

	float DotProductWithPlaneNormal = InParams.PixelDir | InParams.PlaneNormal;
	//check to make sure we're not co-planar
	if (FMath::Abs(DotProductWithPlaneNormal) > DELTA)
	{
		//Get closest point on plane
		RequestedPosition = FMath::LinePlaneIntersection(InParams.EyePos, ProposedEndofEyeVector, MovementPlane);
	}

	//drag is a delta position, so just update the different between the previous position and the new position
	FVector DeltaPosition = RequestedPosition - InParams.Position;

	//Retrieve the initial offset, passing in the current requested position and the current position
	FVector InitialOffset = GetAbsoluteTranslationInitialOffset(RequestedPosition, InParams.Position);

	//subtract off the initial offset (where the widget was clicked) to prevent popping
	DeltaPosition -= InitialOffset;

	//remove the component along the normal we want to mute
	float MovementAlongMutedAxis = DeltaPosition | InParams.NormalToRemove;
	FVector OutDrag              = DeltaPosition - (InParams.NormalToRemove * MovementAlongMutedAxis);

	if (InParams.bMovementLockedToCamera)
	{
		//DAMPEN ABSOLUTE MOVEMENT when the camera is locked to the object
		OutDrag *= CAMERA_LOCK_DAMPING_FACTOR;
		OutDrag.X = FMath::Clamp<FVector::FReal>(OutDrag.X, -MAX_CAMERA_MOVEMENT_SPEED, MAX_CAMERA_MOVEMENT_SPEED);
		OutDrag.Y = FMath::Clamp<FVector::FReal>(OutDrag.Y, -MAX_CAMERA_MOVEMENT_SPEED, MAX_CAMERA_MOVEMENT_SPEED);
		OutDrag.Z = FMath::Clamp<FVector::FReal>(OutDrag.Z, -MAX_CAMERA_MOVEMENT_SPEED, MAX_CAMERA_MOVEMENT_SPEED);
	}

	//the they requested position snapping and we're not moving with the camera
	if (InParams.bPositionSnapping && !InParams.bMovementLockedToCamera && bSnapEnabled)
	{
		FVector MovementAlongAxis =
		    FVector(OutDrag | InParams.XAxis, OutDrag | InParams.YAxis, OutDrag | InParams.ZAxis);
		//translation (either xy plane or z)
		FSnappingUtils::SnapPointToGrid(
		    MovementAlongAxis, FVector(GEditor->GetGridSize(), GEditor->GetGridSize(), GEditor->GetGridSize()));
		OutDrag = MovementAlongAxis.X * InParams.XAxis + MovementAlongAxis.Y * InParams.YAxis +
		    MovementAlongAxis.Z * InParams.ZAxis;
	}

	//get the distance from the original position to the new proposed position
	FVector DeltaFromStart = InParams.Position + OutDrag - InitialTranslationPosition;

	//Get the vector from the eye to the proposed new position (to make sure it's not behind the camera
	FVector EyeToNewPosition        = (InParams.Position + OutDrag) - InParams.EyePos;
	float BehindTheCameraDotProduct = EyeToNewPosition | InParams.CameraDir;

	//Don't let the requested position go behind the camera
	if (BehindTheCameraDotProduct <= 0)
	{
		OutDrag = OutDrag.ZeroVector;
	}
	return OutDrag;
}

/**
 * Returns the offset from the initial selection point
 */
FVector FWidget::GetAbsoluteTranslationInitialOffset(const FVector& InNewPosition, const FVector& InCurrentPosition)
{
	if (!bAbsoluteTranslationInitialOffsetCached)
	{
		bAbsoluteTranslationInitialOffsetCached = true;
		InitialTranslationOffset                = InNewPosition - InCurrentPosition;
		InitialTranslationPosition              = InCurrentPosition;
	}
	return InitialTranslationOffset;
}


/**
 * Returns true if we're in Local Space editing mode
 */
bool FWidget::IsRotationLocalSpace() const
{
	return (CustomCoordSystemSpace == COORD_Local);
}

void FWidget::UpdateDeltaRotation()
{
	TotalDeltaRotation += CurrentDeltaRotation;
	if ((TotalDeltaRotation <= -360.f) || (TotalDeltaRotation >= 360.f))
	{
		TotalDeltaRotation = FRotator::ClampAxis(TotalDeltaRotation);
	}
}

/**
 * Returns the angle in degrees representation of how far we have just rotated
 */
float FWidget::GetDeltaRotation() const
{
	return TotalDeltaRotation;
}

uint32 FWidget::GetDominantAxisIndex(const FVector& InDiff, FEditorViewportClient* ViewportClient) const
{
	uint32 DominantIndex = 0;
	if (FMath::Abs(InDiff.X) < FMath::Abs(InDiff.Y))
	{
		DominantIndex = 1;
	}

	const int32 WidgetMode = ViewportClient->GetWidgetMode();

	switch (WidgetMode)
	{
	case UE::Widget::EWidgetMode::WM_Translate:
		switch (ViewportClient->ViewportType)
		{
		case LVT_OrthoXY:
			if (CurrentAxis == EAxisList::X)
			{
				DominantIndex = 0;
			}
			else if (CurrentAxis == EAxisList::Y)
			{
				DominantIndex = 1;
			}
			break;
		case LVT_OrthoXZ:
			if (CurrentAxis == EAxisList::X)
			{
				DominantIndex = 0;
			}
			else if (CurrentAxis == EAxisList::Z)
			{
				DominantIndex = 1;
			}
			break;
		case LVT_OrthoYZ:
			if (CurrentAxis == EAxisList::Y)
			{
				DominantIndex = 0;
			}
			else if (CurrentAxis == EAxisList::Z)
			{
				DominantIndex = 1;
			}
			break;
		default:
			break;
		}
		break;
	default:
		break;
	}

	return DominantIndex;
}

bool FWidget::IsWidgetDisabled() const
{
	return EditorModeTools ? (EditorModeTools->IsDefaultModeActive() && GEditor->HasSelectedMovementLockedItems()) : false;
}

HWidgetAxis::HWidgetAxis(EAxisList::Type InAxis, bool InbDisabled, EHitProxyPriority InHitProxy)
	: HHitProxy(InHitProxy)
	, Axis(InAxis)
	, bDisabled(InbDisabled)
{
}

EMouseCursor::Type HWidgetAxis::GetMouseCursor()
{
	if (bDisabled)
	{
		return EMouseCursor::SlashedCircle;
	}
	return EMouseCursor::CardinalCross;
}

bool HWidgetAxis::AlwaysAllowsTranslucentPrimitives() const
{
	return true;
}
