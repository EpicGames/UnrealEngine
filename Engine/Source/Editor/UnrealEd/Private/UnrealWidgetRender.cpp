// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnrealWidget.h"
#include "UnrealWidgetFwd.h"
#include "Materials/Material.h"
#include "CanvasItem.h"
#include "SceneView.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "EditorViewportClient.h"
#include "EdMode.h"
#include "EditorModeManager.h"
#include "SnappingUtils.h"
#include "DynamicMeshBuilder.h"
#include "CanvasTypes.h"
#include "EditorInteractiveGizmoManager.h"
#include "TransformGizmoEditorSettings.h"
#include "Misc/AxisDisplayInfo.h"
#include "Tools/EdModeInteractiveToolsContext.h"

/*
 *  Simple struct used to create and group data related to the current window's / viewport's space,
 *  orientation, and scale.
 */
struct FSpaceDescriptor
{
	/*
	*  Creates a new FSpaceDescriptor.
	*
	*  @param View         The virtual view for the space.
	*  @param Viewport     The real viewport for the space.
	*  @param InLocation   The location of the camera in the virtual space.
	*  @param ExternalScale ExtraScale Factor to apply to the UniformScale to allow for the Widget To Get Scaled.
	*/
	FSpaceDescriptor(const FSceneView* View, const FEditorViewportClient* Viewport, const FVector& InLocation, const float ExternalScale) :
		bIsPerspective(View->ViewMatrices.GetViewToClip().M[3][3] < 1.0f),
		bIsLocalSpace(Viewport->GetWidgetCoordSystemSpace() == COORD_Local),
		bIsOrthoXY(!bIsPerspective && FMath::Abs(View->ViewMatrices.GetWorldToView().M[2][2]) > 0.0f),
		bIsOrthoXZ(!bIsPerspective && FMath::Abs(View->ViewMatrices.GetWorldToView().M[1][2]) > 0.0f),
		bIsOrthoYZ(!bIsPerspective && FMath::Abs(View->ViewMatrices.GetWorldToView().M[0][2]) > 0.0f),
		UniformScale(ExternalScale * View->WorldToScreen(InLocation).W * (4.0f / View->UnscaledViewRect.Width() / View->ViewMatrices.GetViewToClip().M[0][0])),
		Scale(CreateScale())
	{
	}

    // Wether or not the view is perspective.
    const bool bIsPerspective;

    // Whether or not the view is in local space.
    const bool bIsLocalSpace;

    // Whether or not the view is orthogonal to the XY plane.
    const bool bIsOrthoXY;

    // Whether or not the view is orthogonal to the XZ plane.
    const bool bIsOrthoXZ;

    // Whether or not the view is orthogonal to the YZ plane.
    const bool bIsOrthoYZ;

    // The uniform scale for the space.
    const float UniformScale;

    // The scale vector for the space based on orientation.
    const FVector Scale;

    /*
     *  Used to determine whether or not the X axis should be drawn.
     *
     *  @param AxisToDraw   The desired axis to draw.
     *
     *  @return True if the axis should be drawn. False otherwise.
     */
    bool ShouldDrawAxisX(const EAxisList::Type AxisToDraw)
    {
        return ShouldDrawAxis(EAxisList::X, AxisToDraw, bIsOrthoYZ);
    }

    /*
     *  Used to determine whether or not the Y axis should be drawn.
     *
     *  @param AxisToDraw   The desired axis to draw.
     *
     *  @return True if the axis should be drawn. False otherwise.
     */
    bool ShouldDrawAxisY(const EAxisList::Type AxisToDraw)
    {
        return ShouldDrawAxis(EAxisList::Y, AxisToDraw, bIsOrthoXZ);
    }

    /*
     *  Used to determine whether or not the Z axis should be drawn.
     *
     *  @param AxisToDraw   The desired axis to draw.
     *
     *  @return True if the axis should be drawn. False otherwise.
     */
    bool ShouldDrawAxisZ(const EAxisList::Type AxisToDraw)
    {
        return ShouldDrawAxis(EAxisList::Z, AxisToDraw, bIsOrthoXY);
    }

private:

    /*
     *  Creates a space scale vector from the determined orientation and uniform scale.
     *
     *  @return Space scale vector.
     */
    FVector CreateScale()
    {
        if (bIsOrthoXY)
        {
            return FVector(UniformScale, UniformScale, 1.0f);
        }
        else if (bIsOrthoXZ)
        {
            return FVector(UniformScale, 1.0f, UniformScale);
        }
        else if (bIsOrthoYZ)
        {
            return FVector(1.0f, UniformScale, UniformScale);
        }
        else
        {
            return FVector(UniformScale, UniformScale, UniformScale);
        }
    }

    /*
     *  Used to determine whether or not a specific axis should be drawn.
     *
     *  @param AxisToCheck  The axis to check.
     *  @param AxisToDraw   The desired axis to draw.
     *  @param bIsOrtho     Whether or not the axis to check is orthogonal to the viewing orientation.
     *
     *  @return True if the axis should be drawn. False otherwise.
     */
    bool ShouldDrawAxis(const EAxisList::Type AxisToCheck, const EAxisList::Type AxisToDraw, const bool bIsOrtho)
    {
        return (AxisToCheck & AxisToDraw) && (bIsPerspective || bIsLocalSpace || !bIsOrtho);
    }
};

/**
 * Renders any widget specific HUD text
 * @param Canvas - Canvas to use for 2d rendering
 */
void FWidget::DrawHUD (FCanvas* Canvas)
{
	if (HUDString.Len())
	{
		const float DPIScale = Canvas->GetDPIScale();
		int32 StringPosX = FMath::FloorToInt(HUDInfoPos.X/DPIScale);
		int32 StringPosY = FMath::FloorToInt(HUDInfoPos.Y/DPIScale);

		//measure string size
		int32 StringSizeX, StringSizeY;
		StringSize(GEngine->GetSmallFont(), StringSizeX, StringSizeY, *HUDString);
		
		//add some padding to the outside
		const int32 Border = 5;
		int32 FillMinX = StringPosX - Border - (StringSizeX>>1);
		int32 FillMinY = StringPosY - Border;// - (StringSizeY>>1);
		StringSizeX += 2*Border;
		StringSizeY += 2*Border;

		//mostly alpha'ed black
		FCanvasTileItem TileItem( FVector2D( FillMinX, FillMinY), GWhiteTexture, FVector2D( StringSizeX, StringSizeY ), FLinearColor( 0.0f, 0.0f, 0.0f, .25f ) );
		TileItem.BlendMode = SE_BLEND_Translucent;
		Canvas->DrawItem( TileItem );
		FCanvasTextItem TextItem( FVector2D( StringPosX, StringPosY), FText::FromString( HUDString ), GEngine->GetSmallFont(), FLinearColor::White );
		TextItem.bCentreX = true;
		Canvas->DrawItem( TextItem );	
	}
}

void FWidget::Render( const FSceneView* View,FPrimitiveDrawInterface* PDI, FEditorViewportClient* ViewportClient )
{
	check(ViewportClient);


	//reset HUD text
	HUDString.Empty();

	const bool bShowFlagsSupportsWidgetDrawing = View->Family->EngineShowFlags.ModeWidgets;
	const bool bEditorModeToolsSupportsWidgetDrawing = EditorModeTools ? EditorModeTools->GetShowWidget() : true;

	// Use legacy widget for TranslateRotateZ or 2D modes or when UseLegacyWidget cvar is true
	UE::Widget::EWidgetMode WidgetMode = ViewportClient->GetWidgetMode();
	bool bUseLegacyWidget = (WidgetMode == UE::Widget::EWidgetMode::WM_TranslateRotateZ || WidgetMode == UE::Widget::EWidgetMode::WM_2D);
	if (!bUseLegacyWidget)
	{
		bUseLegacyWidget = !UEditorInteractiveGizmoManager::AreEditorGizmosAllowed(ViewportClient->GetModeTools());
	}

	bool bDrawWidget;

	// Because the movement routines use the widget axis to determine how to transform mouse movement into
	// editor object movement, we need to still run through the Render routine even though widget drawing may be
	// disabled.  So we keep a flag that is used to determine whether or not to actually render anything.  This way
	// we can still update the widget axis' based on the Context's transform matrices, even though drawing is disabled.
	if(bDefaultVisibility && bShowFlagsSupportsWidgetDrawing && bEditorModeToolsSupportsWidgetDrawing && bUseLegacyWidget)
	{
		bDrawWidget = true;

		// See if there is a custom coordinate system we should be using, only change it if we are drawing widgets.
		CustomCoordSystem = ViewportClient->GetWidgetCoordSystem();
	}
	else
	{
		bDrawWidget = false;
	}

	CustomCoordSystemSpace = ViewportClient->GetWidgetCoordSystemSpace();

	// If the current modes don't want to use the widget, don't draw it.
	if( EditorModeTools && !EditorModeTools->UsesTransformWidget() )
	{
		CurrentAxis = EAxisList::None;
		return;
	}

	FVector WidgetLocation = ViewportClient->GetWidgetLocation();
	FVector2D NewOrigin;
	if (View->ScreenToPixel(View->WorldToScreen(WidgetLocation), NewOrigin))
	{
		// Only update the viewport-space origin if the position was in front of the camera
		Origin = NewOrigin;
	}

	switch( WidgetMode )
	{
		case UE::Widget::EWidgetMode::WM_Translate:
			Render_Translate(View, PDI, ViewportClient, WidgetLocation, bDrawWidget);
			break;

		case UE::Widget::EWidgetMode::WM_Rotate:
			Render_Rotate(View, PDI, ViewportClient, WidgetLocation, bDrawWidget);
			break;

		case UE::Widget::EWidgetMode::WM_Scale:
			Render_Scale(View, PDI, ViewportClient, WidgetLocation, bDrawWidget);
			break;

		case UE::Widget::EWidgetMode::WM_TranslateRotateZ:
			Render_TranslateRotateZ(View, PDI, ViewportClient, WidgetLocation, bDrawWidget);
			break;

		case UE::Widget::EWidgetMode::WM_2D:
			Render_2D(View, PDI, ViewportClient, WidgetLocation, bDrawWidget);
			break;

		default:
			break;
	}
};

/**
 * Draws an arrow head line for a specific axis.
 */
void FWidget::Render_Axis(const FSceneView* View, FPrimitiveDrawInterface* PDI, EAxisList::Type InAxis, FMatrix& InMatrix, UMaterialInterface* InMaterial, const FLinearColor& InColor, FVector2D& OutAxisDir, const FVector& InScale, bool bDrawWidget, bool bCubeHead, float AxisLengthOffset)
{
	FMatrix AxisRotation = FMatrix::Identity;
	const bool bUsingLUFCoordinateSysem = AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward;
	if( InAxis == EAxisList::Y )
	{
		AxisRotation = FRotationMatrix::MakeFromXZ(FVector(0, bUsingLUFCoordinateSysem ? -1 : 1, 0), FVector(0, 0, 1));
	}
	else if( InAxis == EAxisList::Z )
	{
		AxisRotation = FRotationMatrix::MakeFromXY(FVector(0, 0, 1), FVector(0, bUsingLUFCoordinateSysem ? -1 : 1, 0));
	}

	FMatrix ArrowToWorld = AxisRotation * InMatrix;

	// The scale that is passed in potentially leaves one component with a scale of 1, if that happens
	// we need to extract the inform scale and use it to construct the scale that transforms the primitives
	float UniformScale = InScale.GetMax() > 1.0f ? InScale.GetMax() : InScale.GetMin() < 1.0f ? InScale.GetMin() : 1.0f;
	// After the primitives have been scaled and transformed, we apply this inverse scale that flattens the dimension
	// that was scaled up to prevent it from intersecting with the near plane.  In perspective this won't have any effect,
	// but in the ortho viewports it will prevent scaling in the direction of the camera and thus intersecting the near plane.
	FVector FlattenScale = FVector(InScale.Component(0) == 1.0f ? 1.0f / UniformScale : 1.0f, InScale.Component(1) == 1.0f ? 1.0f / UniformScale : 1.0f, InScale.Component(2) == 1.0f ? 1.0f / UniformScale : 1.0f);

	FScaleMatrix Scale(UniformScale);
	ArrowToWorld = Scale * ArrowToWorld;

	if( bDrawWidget )
	{
		const bool bDisabled = EditorModeTools ? (EditorModeTools->IsDefaultModeActive() && GEditor->HasSelectedMovementLockedItems() ) : false;
		PDI->SetHitProxy( new HWidgetAxis( InAxis, bDisabled) );

		const float AxisLength = AXIS_LENGTH + GetDefault<UTransformGizmoEditorSettings>()->TransformGizmoSize - (AxisLengthOffset * 2);
		const float HalfHeight = AxisLength/2.0f;
		const float CylinderRadius = 1.2f;
		const FVector Offset(0, 0, HalfHeight + AxisLengthOffset);

		switch( InAxis )
		{
			case EAxisList::X:
			{
				DrawCylinder(PDI, ( Scale * FRotationMatrix(FRotator(-90, 0.f, 0)) * InMatrix ) * FScaleMatrix(FlattenScale), Offset, FVector(1, 0, 0), FVector(0, 1, 0), FVector(0, 0, 1), CylinderRadius, HalfHeight, 16, InMaterial->GetRenderProxy(), SDPG_Foreground);
				break;
			}
			case EAxisList::Y:
			{
				DrawCylinder(PDI, (Scale * FRotationMatrix(FRotator(0, 0, bUsingLUFCoordinateSysem ? -90 : 90)) * InMatrix)* FScaleMatrix(FlattenScale), Offset, FVector(1, 0, 0), FVector(0, 1, 0), FVector(0, 0, 1), CylinderRadius, HalfHeight, 16, InMaterial->GetRenderProxy(), SDPG_Foreground );
				break;
			}
			case EAxisList::Z:
			{
				DrawCylinder(PDI, ( Scale * InMatrix ) * FScaleMatrix(FlattenScale), Offset, FVector(1, 0, 0), FVector(0, 1, 0), FVector(0, 0, 1), CylinderRadius, HalfHeight, 16, InMaterial->GetRenderProxy(), SDPG_Foreground);
				break;
			}
		}

		if ( bCubeHead )
		{
			const float CubeHeadOffset = 3.0f;
			FVector RootPos(AxisLength + CubeHeadOffset + AxisLengthOffset, 0, 0);

			Render_Cube(PDI, (FTranslationMatrix(RootPos) * ArrowToWorld) * FScaleMatrix(FlattenScale), InMaterial, FVector(4.0f));
		}
		else
		{
			const float ConeHeadOffset = 12.0f;
			FVector RootPos(AxisLength + ConeHeadOffset + AxisLengthOffset, 0, 0);

			float Angle = FMath::DegreesToRadians( PI * 5 );
			DrawCone(PDI, ( FScaleMatrix(-13) * FTranslationMatrix(RootPos) * ArrowToWorld ) * FScaleMatrix(FlattenScale), Angle, Angle, 32, false, FColor::White, InMaterial->GetRenderProxy(), SDPG_Foreground);
		}
	
		PDI->SetHitProxy( nullptr );
	}

	FVector2D NewOrigin;
	FVector2D AxisEnd;
	const FVector AxisEndWorld = ArrowToWorld.TransformPosition(FVector(64, 0, 0));
	const FVector WidgetOrigin = InMatrix.GetOrigin();

	if (View->ScreenToPixel(View->WorldToScreen(WidgetOrigin), NewOrigin) &&
		View->ScreenToPixel(View->WorldToScreen(AxisEndWorld), AxisEnd))
	{
		// If both the origin and the axis endpoint are in front of the camera, trivially calculate the viewport space axis direction
		OutAxisDir = (AxisEnd - NewOrigin).GetSafeNormal();
	}
	else
	{
		// If either the origin or axis endpoint are behind the camera, translate the entire widget in front of the camera in the view direction before performing the
		// viewport space calculation
		const FMatrix InvViewMatrix = View->ViewMatrices.GetViewToWorld();
		const FVector ViewLocation = InvViewMatrix.GetOrigin();
		const FVector ViewDirection = InvViewMatrix.GetUnitAxis(EAxis::Z);
		const FVector Offset = ViewDirection * (FVector::DotProduct(ViewLocation - WidgetOrigin, ViewDirection) + 100.0f);
		const FVector AdjustedWidgetOrigin = WidgetOrigin + Offset;
		const FVector AdjustedWidgetAxisEnd = AxisEndWorld + Offset;

		if (View->ScreenToPixel(View->WorldToScreen(AdjustedWidgetOrigin), NewOrigin) &&
			View->ScreenToPixel(View->WorldToScreen(AdjustedWidgetAxisEnd), AxisEnd))
		{
			OutAxisDir = -(AxisEnd - NewOrigin).GetSafeNormal();
		}
	}
}

void FWidget::Render_Cube( FPrimitiveDrawInterface* PDI, const FMatrix& InMatrix, const UMaterialInterface* InMaterial, const FVector& InScale )
{
	const FMatrix CubeToWorld = FScaleMatrix(InScale) * InMatrix;
	DrawBox( PDI, CubeToWorld, FVector(1,1,1), InMaterial->GetRenderProxy(), SDPG_Foreground );
}

void DrawCornerHelper( FPrimitiveDrawInterface* PDI, const FMatrix& LocalToWorld, const FVector& Length, float Thickness, const FMaterialRenderProxy* MaterialRenderProxy,uint8 DepthPriorityGroup  )
{
	const float TH = Thickness;

	float TX = Length.X/2;
	float TY = Length.Y/2;
	float TZ = Length.Z/2;

	FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());

	// Top
	{
		int32 VertexIndices[4];
		VertexIndices[0] = MeshBuilder.AddVertex( FVector3f(-TX, -TY, +TZ), FVector2f::ZeroVector, FVector3f(1,0,0), FVector3f(0,1,0), FVector3f(0,0,1), FColor::White );
		VertexIndices[1] = MeshBuilder.AddVertex( FVector3f(-TX, +TY, +TZ), FVector2f::ZeroVector, FVector3f(1,0,0), FVector3f(0,1,0), FVector3f(0,0,1), FColor::White );
		VertexIndices[2] = MeshBuilder.AddVertex( FVector3f(+TX, +TY, +TZ), FVector2f::ZeroVector, FVector3f(1,0,0), FVector3f(0,1,0), FVector3f(0,0,1), FColor::White );
		VertexIndices[3] = MeshBuilder.AddVertex( FVector3f(+TX, -TY, +TZ), FVector2f::ZeroVector, FVector3f(1,0,0), FVector3f(0,1,0), FVector3f(0,0,1), FColor::White );

		MeshBuilder.AddTriangle(VertexIndices[0],VertexIndices[1],VertexIndices[2]);
		MeshBuilder.AddTriangle(VertexIndices[0],VertexIndices[2],VertexIndices[3]);
	}

	//Left
	{
		int32 VertexIndices[4];
		VertexIndices[0] = MeshBuilder.AddVertex( FVector3f(-TX,  -TY, TZ-TH),	FVector2f::ZeroVector, FVector3f(0,0,1), FVector3f(0,1,0), FVector3f(-1,0,0), FColor::White );
		VertexIndices[1] = MeshBuilder.AddVertex( FVector3f(-TX, -TY, TZ),		FVector2f::ZeroVector, FVector3f(0,0,1), FVector3f(0,1,0), FVector3f(-1,0,0), FColor::White );
		VertexIndices[2] = MeshBuilder.AddVertex( FVector3f(-TX, +TY, TZ),		FVector2f::ZeroVector, FVector3f(0,0,1), FVector3f(0,1,0), FVector3f(-1,0,0), FColor::White );
		VertexIndices[3] = MeshBuilder.AddVertex( FVector3f(-TX, +TY, TZ-TH),	FVector2f::ZeroVector, FVector3f(0,0,1), FVector3f(0,1,0), FVector3f(-1,0,0), FColor::White );


		MeshBuilder.AddTriangle(VertexIndices[0],VertexIndices[1],VertexIndices[2]);
		MeshBuilder.AddTriangle(VertexIndices[0],VertexIndices[2],VertexIndices[3]);
	}

	// Front
	{
		int32 VertexIndices[5];
		VertexIndices[0] = MeshBuilder.AddVertex( FVector3f(-TX,	+TY, TZ-TH),	FVector2f::ZeroVector, FVector3f(1,0,0), FVector3f(0,0,-1), FVector3f(0,1,0), FColor::White );
		VertexIndices[1] = MeshBuilder.AddVertex( FVector3f(-TX,	+TY, +TZ  ),	FVector2f::ZeroVector, FVector3f(1,0,0), FVector3f(0,0,-1), FVector3f(0,1,0), FColor::White );
		VertexIndices[2] = MeshBuilder.AddVertex( FVector3f(+TX-TH, +TY, +TX  ),	FVector2f::ZeroVector, FVector3f(1,0,0), FVector3f(0,0,-1), FVector3f(0,1,0), FColor::White );
		VertexIndices[3] = MeshBuilder.AddVertex( FVector3f(+TX,	+TY, +TZ  ),	FVector2f::ZeroVector, FVector3f(1,0,0), FVector3f(0,0,-1), FVector3f(0,1,0), FColor::White );
		VertexIndices[4] = MeshBuilder.AddVertex( FVector3f(+TX-TH, +TY, TZ-TH),	FVector2f::ZeroVector, FVector3f(1,0,0), FVector3f(0,0,-1), FVector3f(0,1,0), FColor::White );

		MeshBuilder.AddTriangle(VertexIndices[0],VertexIndices[1],VertexIndices[2]);
		MeshBuilder.AddTriangle(VertexIndices[0],VertexIndices[2],VertexIndices[4]);
		MeshBuilder.AddTriangle(VertexIndices[4],VertexIndices[2],VertexIndices[3]);
	}

	// Back
	{
		int32 VertexIndices[5];
		VertexIndices[0] = MeshBuilder.AddVertex( FVector3f(-TX,	-TY, TZ-TH),	FVector2f::ZeroVector, FVector3f(1,0,0), FVector3f(0,0,1), FVector3f(0,-1,0), FColor::White );
		VertexIndices[1] = MeshBuilder.AddVertex( FVector3f(-TX,	-TY, +TZ),		FVector2f::ZeroVector, FVector3f(1,0,0), FVector3f(0,0,1), FVector3f(0,-1,0), FColor::White );
		VertexIndices[2] = MeshBuilder.AddVertex( FVector3f(+TX-TH, -TY, +TX),		FVector2f::ZeroVector, FVector3f(1,0,0), FVector3f(0,0,1), FVector3f(0,-1,0), FColor::White );
		VertexIndices[3] = MeshBuilder.AddVertex( FVector3f(+TX,	-TY, +TZ),		FVector2f::ZeroVector, FVector3f(1,0,0), FVector3f(0,0,1), FVector3f(0,-1,0), FColor::White );
		VertexIndices[4] = MeshBuilder.AddVertex( FVector3f(+TX-TH, -TY, TZ-TH),	FVector2f::ZeroVector, FVector3f(1,0,0), FVector3f(0,0,1), FVector3f(0,-1,0), FColor::White );

		MeshBuilder.AddTriangle(VertexIndices[0],VertexIndices[1],VertexIndices[2]);
		MeshBuilder.AddTriangle(VertexIndices[0],VertexIndices[2],VertexIndices[4]);
		MeshBuilder.AddTriangle(VertexIndices[4],VertexIndices[2],VertexIndices[3]);
	}
	// Bottom
	{
		int32 VertexIndices[4];
		VertexIndices[0] = MeshBuilder.AddVertex( FVector3f(-TX, -TY, TZ-TH),		FVector2f::ZeroVector, FVector3f(1,0,0), FVector3f(0,0,-1), FVector3f(0,0,1), FColor::White );
		VertexIndices[1] = MeshBuilder.AddVertex( FVector3f(-TX, +TY, TZ-TH),		FVector2f::ZeroVector, FVector3f(1,0,0), FVector3f(0,0,-1), FVector3f(0,0,1), FColor::White );
		VertexIndices[2] = MeshBuilder.AddVertex( FVector3f(+TX-TH, +TY, TZ-TH),	FVector2f::ZeroVector, FVector3f(1,0,0), FVector3f(0,0,-1), FVector3f(0,0,1), FColor::White );
		VertexIndices[3] = MeshBuilder.AddVertex( FVector3f(+TX-TH, -TY, TZ-TH),	FVector2f::ZeroVector, FVector3f(1,0,0), FVector3f(0,0,-1), FVector3f(0,0,1), FColor::White );

		MeshBuilder.AddTriangle(VertexIndices[0],VertexIndices[1],VertexIndices[2]);
		MeshBuilder.AddTriangle(VertexIndices[0],VertexIndices[2],VertexIndices[3]);
	}
	MeshBuilder.Draw(PDI,LocalToWorld,MaterialRenderProxy,DepthPriorityGroup,0.f);
}

void DrawDualAxis( FPrimitiveDrawInterface* PDI, const FMatrix& BoxToWorld,const FVector& Length, float Thickness, const FMaterialRenderProxy* AxisMat,const FMaterialRenderProxy* Axis2Mat )
{
	DrawCornerHelper(PDI, BoxToWorld, Length, Thickness, Axis2Mat, SDPG_Foreground);
	DrawCornerHelper(PDI, FScaleMatrix(FVector(-1, 1, 1)) * FRotationMatrix(FRotator(-90, 0, 0)) * BoxToWorld, Length, Thickness, AxisMat, SDPG_Foreground);
}
/**
 * Draws the translation widget.
 */
void FWidget::Render_Translate( const FSceneView* View, FPrimitiveDrawInterface* PDI, FEditorViewportClient* ViewportClient, const FVector& InLocation, bool bDrawWidget )
{
	// Figure out axis colors
	const FLinearColor& XColor = ( CurrentAxis&EAxisList::X ? (FLinearColor)CurrentColor : AxisColorX );
	const FLinearColor& YColor = ( CurrentAxis&EAxisList::Y ? (FLinearColor)CurrentColor : AxisColorY );
	const FLinearColor& ZColor = ( CurrentAxis&EAxisList::Z ? (FLinearColor)CurrentColor : AxisColorZ );
	FColor CurrentScreenColor = ( CurrentAxis & EAxisList::Screen ? CurrentColor : ScreenSpaceColor );

	// Figure out axis matrices
	FMatrix WidgetMatrix = CustomCoordSystem * FTranslationMatrix( InLocation );
	const EAxisList::Type DrawAxis = GetAxisToDraw( ViewportClient->GetWidgetMode() );
	const bool bDisabled = IsWidgetDisabled();

	const float ExternalScale = EditorModeTools ? EditorModeTools->GetWidgetScale() : 1.0f;
	FSpaceDescriptor Space(View, ViewportClient, InLocation, ExternalScale);

	// Draw the axis lines with arrow heads
	if( Space.ShouldDrawAxisX(DrawAxis) )
	{
		UMaterialInstanceDynamic* XMaterial = ( CurrentAxis&EAxisList::X ? CurrentAxisMaterial : AxisMaterialX );
		Render_Axis( View, PDI, EAxisList::X, WidgetMatrix, XMaterial, XColor, XAxisDir, Space.Scale, bDrawWidget );
	}

	if( Space.ShouldDrawAxisY(DrawAxis) )
	{
		UMaterialInstanceDynamic* YMaterial = ( CurrentAxis&EAxisList::Y ? CurrentAxisMaterial : AxisMaterialY );
		Render_Axis( View, PDI, EAxisList::Y, WidgetMatrix, YMaterial, YColor, YAxisDir, Space.Scale, bDrawWidget );
	}

	if( Space.ShouldDrawAxisZ(DrawAxis) )
	{
		UMaterialInstanceDynamic* ZMaterial = ( CurrentAxis&EAxisList::Z ? CurrentAxisMaterial : AxisMaterialZ );
		Render_Axis( View, PDI, EAxisList::Z, WidgetMatrix, ZMaterial, ZColor, ZAxisDir, Space.Scale, bDrawWidget );
	}

	// Draw the grabbers
	if( bDrawWidget )
	{
		FVector CornerPos = FVector(7, 0, 7) * Space.UniformScale;
		FVector AxisSize = FVector(12, 1.2, 12) * Space.UniformScale;
		float CornerLength = 1.2f * Space.UniformScale;

		// After the primitives have been scaled and transformed, we apply this inverse scale that flattens the dimension
		// that was scaled up to prevent it from intersecting with the near plane.  In perspective this won't have any effect,
		// but in the ortho viewports it will prevent scaling in the direction of the camera and thus intersecting the near plane.
		FVector FlattenScale = FVector(Space.Scale.Component(0) == 1.0f ? 1.0f / Space.UniformScale : 1.0f, Space.Scale.Component(1) == 1.0f ? 1.0f / Space.UniformScale : 1.0f, Space.Scale.Component(2) == 1.0f ? 1.0f / Space.UniformScale : 1.0f);

		const bool bUsingLUFCoordinateSysem = AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward;

		if (Space.bIsPerspective || Space.bIsLocalSpace || Space.bIsOrthoXY)
		{
			if( (DrawAxis&EAxisList::XY) == EAxisList::XY )							// Top
			{
				UMaterialInstanceDynamic* XMaterial = ( (CurrentAxis&EAxisList::XY) == EAxisList::XY ? CurrentAxisMaterial : AxisMaterialX );
				UMaterialInstanceDynamic* YMaterial = ( (CurrentAxis&EAxisList::XY) == EAxisList::XY? CurrentAxisMaterial : AxisMaterialY );

				PDI->SetHitProxy( new HWidgetAxis(EAxisList::XY, bDisabled) );
				{
					DrawDualAxis(PDI, ( FTranslationMatrix(CornerPos) * FRotationMatrix(FRotator(0, 0, bUsingLUFCoordinateSysem ? -90 : 90)) * WidgetMatrix ) * FScaleMatrix(FlattenScale), AxisSize, CornerLength, XMaterial->GetRenderProxy(), YMaterial->GetRenderProxy());
				}
				PDI->SetHitProxy( nullptr );
			}
		}

		if (Space.bIsPerspective || Space.bIsLocalSpace || Space.bIsOrthoXZ)		// Front
		{
			if( (DrawAxis&EAxisList::XZ) == EAxisList::XZ ) 
			{
				UMaterialInstanceDynamic* XMaterial = ( (CurrentAxis&EAxisList::XZ) == EAxisList::XZ ? CurrentAxisMaterial : AxisMaterialX );
				UMaterialInstanceDynamic* ZMaterial = ( (CurrentAxis&EAxisList::XZ) == EAxisList::XZ ? CurrentAxisMaterial : AxisMaterialZ );

				PDI->SetHitProxy( new HWidgetAxis(EAxisList::XZ, bDisabled) );
				{
					DrawDualAxis(PDI, (FTranslationMatrix(CornerPos) * WidgetMatrix) * FScaleMatrix(FlattenScale), AxisSize, CornerLength, XMaterial->GetRenderProxy(), ZMaterial->GetRenderProxy() );
				}
				PDI->SetHitProxy( nullptr );
			}
		}

		if( Space.bIsPerspective || Space.bIsLocalSpace || Space.bIsOrthoYZ )		// Side
		{
			if( (DrawAxis&EAxisList::YZ) == EAxisList::YZ ) 
			{
				UMaterialInstanceDynamic* YMaterial = ( (CurrentAxis&EAxisList::YZ) == EAxisList::YZ ? CurrentAxisMaterial : AxisMaterialY );
				UMaterialInstanceDynamic* ZMaterial = ( (CurrentAxis&EAxisList::YZ) == EAxisList::YZ ? CurrentAxisMaterial : AxisMaterialZ );

				PDI->SetHitProxy( new HWidgetAxis(EAxisList::YZ, bDisabled) );
				{
					DrawDualAxis(PDI, (FTranslationMatrix(CornerPos) * FRotationMatrix(FRotator(0, bUsingLUFCoordinateSysem ? -90 : 90, 0)) * WidgetMatrix) * FScaleMatrix(FlattenScale), AxisSize, CornerLength, YMaterial->GetRenderProxy(), ZMaterial->GetRenderProxy() );
				}
				PDI->SetHitProxy( nullptr );
			}
		}
	}

	// Draw screen-space movement handle (circle)
	if( bDrawWidget && ( DrawAxis & EAxisList::Screen ) && Space.bIsPerspective )
	{
		PDI->SetHitProxy( new HWidgetAxis(EAxisList::Screen, bDisabled) );
		const FVector CameraXAxis = View->ViewMatrices.GetWorldToView().GetColumn(0);
		const FVector CameraYAxis = View->ViewMatrices.GetWorldToView().GetColumn(1);
		const FVector CameraZAxis = View->ViewMatrices.GetWorldToView().GetColumn(2);

		UMaterialInstanceDynamic* XYZMaterial = ( CurrentAxis&EAxisList::Screen) ? CurrentAxisMaterial : OpaquePlaneMaterialXY;
		DrawSphere( PDI, WidgetMatrix.GetOrigin(), FRotator::ZeroRotator, 4.0f * Space.Scale, 10, 5, XYZMaterial->GetRenderProxy(), SDPG_Foreground );

		PDI->SetHitProxy( nullptr );
	}
}

uint8 LargeInnerAlpha = 0x3f;
uint8 SmallInnerAlpha = 0x0f;
uint8 LargeOuterAlpha = 0x7f;
uint8 SmallOuterAlpha = 0x0f;

void FWidget::DrawColoredSphere(FPrimitiveDrawInterface* PDI, const FVector& Center, const FRotator& Orientation, FColor Color, const FVector& Radii, int32 NumSides, int32 NumRings, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority, bool bDisableBackfaceCulling)
{
	// Use a mesh builder to draw the sphere.
	FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());
	{
		// The first/last arc are on top of each other.
		int32 NumVerts = (NumSides + 1) * (NumRings + 1);
		FDynamicMeshVertex* Verts = (FDynamicMeshVertex*)FMemory::Malloc(NumVerts * sizeof(FDynamicMeshVertex));

		// Calculate verts for one arc
		FDynamicMeshVertex* ArcVerts = (FDynamicMeshVertex*)FMemory::Malloc((NumRings + 1) * sizeof(FDynamicMeshVertex));

		for (int32 i = 0; i < NumRings + 1; i++)
		{
			FDynamicMeshVertex* ArcVert = &ArcVerts[i];

			float angle = ((float)i / NumRings) * PI;

			ArcVert->Color = Color;
			// Note- unit sphere, so position always has mag of one. We can just use it for normal!			
			ArcVert->Position.X = 0.0f;
			ArcVert->Position.Y = FMath::Sin(angle);
			ArcVert->Position.Z = FMath::Cos(angle);

			ArcVert->SetTangents(
				FVector3f(1, 0, 0),
				FVector3f(0.0f, -ArcVert->Position.Z, ArcVert->Position.Y),
				ArcVert->Position
			);

			ArcVert->TextureCoordinate[0].X = 0.0f;
			ArcVert->TextureCoordinate[0].Y = ((float)i / NumRings);
		}

		// Then rotate this arc NumSides+1 times.
		for (int32 s = 0; s < NumSides + 1; s++)
		{
			FRotator3f ArcRotator(0, 360.f * (float)s / NumSides, 0);
			FRotationMatrix44f ArcRot(ArcRotator);
			float XTexCoord = ((float)s / NumSides);

			for (int32 v = 0; v < NumRings + 1; v++)
			{
				int32 VIx = (NumRings + 1)*s + v;
				Verts[VIx].Color = Color;
				Verts[VIx].Position = ArcRot.TransformPosition(ArcVerts[v].Position);

				Verts[VIx].SetTangents(
					ArcRot.TransformVector(ArcVerts[v].TangentX.ToFVector3f()),
					ArcRot.TransformVector(ArcVerts[v].GetTangentY()),
					ArcRot.TransformVector(ArcVerts[v].TangentZ.ToFVector3f())
				);

				Verts[VIx].TextureCoordinate[0].X = XTexCoord;
				Verts[VIx].TextureCoordinate[0].Y = ArcVerts[v].TextureCoordinate[0].Y;
			}
		}

		// Add all of the vertices we generated to the mesh builder.
		for (int32 VertIdx = 0; VertIdx < NumVerts; VertIdx++)
		{
			MeshBuilder.AddVertex(Verts[VertIdx]);
		}

		// Add all of the triangles we generated to the mesh builder.
		for (int32 s = 0; s < NumSides; s++)
		{
			int32 a0start = (s + 0) * (NumRings + 1);
			int32 a1start = (s + 1) * (NumRings + 1);

			for (int32 r = 0; r < NumRings; r++)
			{
				MeshBuilder.AddTriangle(a0start + r + 0, a1start + r + 0, a0start + r + 1);
				MeshBuilder.AddTriangle(a1start + r + 0, a1start + r + 1, a0start + r + 1);
			}
		}

		// Free our local copy of verts and arc verts
		FMemory::Free(Verts);
		FMemory::Free(ArcVerts);
	}
	MeshBuilder.Draw(PDI, FScaleMatrix(Radii) * FRotationMatrix(Orientation) * FTranslationMatrix(Center), MaterialRenderProxy, DepthPriority, bDisableBackfaceCulling);
}
//CVAR for the arcball size, so animators can adjust it
//DEPRECRATREDin 5.1, since changing the size will cause hit test problems.
static TAutoConsoleVariable<float> CVarArcballSize(
	TEXT("r.Editor.ArcballSize"),
	1.0,
	TEXT("DEPRECRATED in 5.1)"),
	ECVF_RenderThreadSafe
);

/**
 * Draws the rotation widget.
 */
void FWidget::Render_Rotate( const FSceneView* View,FPrimitiveDrawInterface* PDI, FEditorViewportClient* ViewportClient, const FVector& InLocation, bool bDrawWidget )
{
	float Scale = View->WorldToScreen(InLocation).W * (4.0f / View->UnscaledViewRect.Width() / View->ViewMatrices.GetViewToClip().M[0][0]);
	const float ExternalScale = EditorModeTools ? EditorModeTools->GetWidgetScale() : 1.0f;
	Scale *= ExternalScale;
	//get the axes 
	FVector XAxis = CustomCoordSystem.TransformVector(FVector(1, 0, 0));
	FVector YAxis = CustomCoordSystem.TransformVector(FVector(0, 1, 0));
	FVector ZAxis = CustomCoordSystem.TransformVector(FVector(0, 0, 1));
	FMatrix WidgetMatrix = CustomCoordSystem * FTranslationMatrix( InLocation );

	EAxisList::Type DrawAxis = GetAxisToDraw(ViewportClient->GetWidgetMode());

	FVector DirectionToWidget = View->IsPerspectiveProjection() ? (InLocation - View->ViewMatrices.GetViewOrigin()) : -View->GetViewDirection();
	DirectionToWidget.Normalize();

	// Draw a circle for each axis
	if (bDrawWidget || bDragging)
	{
		bIsOrthoDrawingFullRing = false;
		//now draw the arc segments, only if we are not dragging and moving the arcball per animators request
		if ((DrawAxis & EAxisList::XYZ && CurrentAxis == EAxisList::XYZ && bDragging) == false) 
		{

			if (DrawAxis & EAxisList::X)
			{
				DrawRotationArc(View, PDI, EAxisList::X, WidgetMatrix.GetOrigin(), ZAxis, YAxis, DirectionToWidget, AxisColorX.ToFColor(true), Scale, XAxisDir);
			}

			if (DrawAxis & EAxisList::Y)
			{
				DrawRotationArc(View, PDI, EAxisList::Y, WidgetMatrix.GetOrigin(), XAxis, ZAxis, DirectionToWidget, AxisColorY.ToFColor(true), Scale, YAxisDir);
			}

			if (DrawAxis & EAxisList::Z)
			{
				DrawRotationArc(View, PDI, EAxisList::Z, WidgetMatrix.GetOrigin(), XAxis, YAxis, DirectionToWidget, AxisColorZ.ToFColor(true), Scale, ZAxisDir);
			}
			
		}

		if (DrawAxis&EAxisList::XYZ  && (!bDragging || CurrentAxis == EAxisList::XYZ) && GetDefault<UTransformGizmoEditorSettings>()->bEnableArcballRotate)
		{
			FVector Center = InLocation;
			FRotator Orientation = FRotator::ZeroRotator;
			const float ArcballSize = 1.0f; //always constant, otherwise can cause hit testing issues

			const float InnerDistance = (INNER_AXIS_CIRCLE_RADIUS * ArcballSize * Scale) + GetDefault<UTransformGizmoEditorSettings>()->TransformGizmoSize;
			FVector Radii(InnerDistance, InnerDistance, InnerDistance);
			const bool bDisabled = IsWidgetDisabled();
			FColor ArcColor = ArcBallColor;
			ArcColor.A = (CurrentAxis==EAxisList::XYZ) ? ArcBallColor.A + 5 : ArcBallColor.A; //less transparent if selected

			//set priority to foreground so axis get hit first
			PDI->SetHitProxy(new HWidgetAxis(EAxisList::XYZ, bDisabled, HPP_Foreground));
			{
				DrawColoredSphere(PDI, WidgetMatrix.GetOrigin(), FRotator::ZeroRotator, ArcColor, Radii, 32, 24, TransparentPlaneMaterialXY->GetRenderProxy(), SDPG_Foreground, true);
			}
			PDI->SetHitProxy(nullptr);
		}
		
		if(DrawAxis&EAxisList::Screen && (!bDragging || CurrentAxis == EAxisList::Screen) && GetDefault<ULevelEditorViewportSettings>()->bAllowScreenRotate)
		{
			const bool bIsPerspective = (View->ViewMatrices.GetViewToClip().M[3][3] < 1.0f);
			const bool bIsOrtho = !bIsPerspective;

			const FVector Axis0 = View->GetViewUp();
			const FVector Axis1 = View->GetViewRight();
			
			const float OuterRadius = (OUTER_AXIS_CIRCLE_RADIUS * 1.25f * Scale) + GetDefault<UTransformGizmoEditorSettings>()->TransformGizmoSize;
			const float InnerRadius = ((OUTER_AXIS_CIRCLE_RADIUS -1) * 1.25f * Scale) + GetDefault<UTransformGizmoEditorSettings>()->TransformGizmoSize;

			FColor ArcColor = (CurrentAxis==EAxisList::Screen) ? CurrentColor : ScreenAxisColor.ToFColor(true);

			const bool bDisabled = IsWidgetDisabled();

			PDI->SetHitProxy(new HWidgetAxis(EAxisList::Screen, bDisabled));
			{
				FThickArcParams OuterArcParams(PDI, InLocation, TransparentPlaneMaterialXY, InnerRadius, OuterRadius);
				//Pass through alpha
				DrawThickArc(OuterArcParams, Axis0, Axis1, 0.0f, 2.0f * PI, ArcColor, DirectionToWidget, !bIsPerspective);
			}
			PDI->SetHitProxy(nullptr);
		}
	}
}

/**
 * Draws the scaling widget.
 */
void FWidget::Render_Scale( const FSceneView* View,FPrimitiveDrawInterface* PDI, FEditorViewportClient* ViewportClient, const FVector& InLocation, bool bDrawWidget)
{
	// Figure out axis colors
	const FLinearColor& XColor = ( CurrentAxis&EAxisList::X ? (FLinearColor)CurrentColor : AxisColorX );
	const FLinearColor& YColor = ( CurrentAxis&EAxisList::Y ? (FLinearColor)CurrentColor : AxisColorY );
	const FLinearColor& ZColor = ( CurrentAxis&EAxisList::Z ? (FLinearColor)CurrentColor : AxisColorZ );
	FColor CurrentScreenColor = ( CurrentAxis & EAxisList::Screen ? CurrentColor : ScreenSpaceColor );

	// Figure out axis materials

	UMaterialInstanceDynamic* XMaterial = ( CurrentAxis&EAxisList::X ? CurrentAxisMaterial : AxisMaterialX );
	UMaterialInstanceDynamic* YMaterial = ( CurrentAxis&EAxisList::Y ? CurrentAxisMaterial : AxisMaterialY );
	UMaterialInstanceDynamic* ZMaterial = ( CurrentAxis&EAxisList::Z ? CurrentAxisMaterial : AxisMaterialZ );
	UMaterialInstanceDynamic* XYZMaterial = ( CurrentAxis&EAxisList::XYZ ? CurrentAxisMaterial : OpaquePlaneMaterialXY );

	FMatrix WidgetMatrix = CustomCoordSystem * FTranslationMatrix( InLocation );
	const EAxisList::Type DrawAxis = GetAxisToDraw( ViewportClient->GetWidgetMode() );
	const float ExternalScale = EditorModeTools ? EditorModeTools->GetWidgetScale() : 1.0f;
	FSpaceDescriptor Space(View, ViewportClient, InLocation, ExternalScale);

	// Use a constant uniform scale for this widget since orthographic view for it is not supported.
	const FVector UniformScale(Space.UniformScale);

	// Draw the axis lines with cube heads	
    if (Space.ShouldDrawAxisX(DrawAxis))
    {
        Render_Axis(View, PDI, EAxisList::X, WidgetMatrix, XMaterial, XColor, XAxisDir, UniformScale, bDrawWidget, true, AXIS_LENGTH_SCALE_OFFSET);
    }

    if (Space.ShouldDrawAxisY(DrawAxis))
    {
        Render_Axis(View, PDI, EAxisList::Y, WidgetMatrix, YMaterial, YColor, YAxisDir, UniformScale, bDrawWidget, true, AXIS_LENGTH_SCALE_OFFSET);
    }

    if (Space.ShouldDrawAxisZ(DrawAxis))
    {
        Render_Axis(View, PDI, EAxisList::Z, WidgetMatrix, ZMaterial, ZColor, ZAxisDir, UniformScale, bDrawWidget, true, AXIS_LENGTH_SCALE_OFFSET);
    }

	// Draw grabber handles and center cube
	if ( bDrawWidget )
	{
		const bool bDisabled = IsWidgetDisabled();
		const bool bUsingLUFCoordinateSysem = AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward;

		// Since the Y axis gets flipped in the LUF coord system, flip the Y position of the grabber handles as well
		const int8_t GrabberHandlesHalfYPos = bUsingLUFCoordinateSysem ? -12 : 12;
		const int8_t GrabberHandlesYPos = bUsingLUFCoordinateSysem ? -24 : 24;

		// Grabber handles - since orthographic scale widgets are not supported, we should always draw grabber handles if we're drawing
		// the corresponding axes.
		if( (DrawAxis&(EAxisList::X|EAxisList::Y)) == (EAxisList::X|EAxisList::Y) )
		{
			PDI->SetHitProxy( new HWidgetAxis(EAxisList::XY, bDisabled) );
			{
				PDI->DrawLine( WidgetMatrix.TransformPosition(FVector(24,0,0) * UniformScale), WidgetMatrix.TransformPosition(FVector(12,GrabberHandlesHalfYPos,0) * UniformScale), XColor, SDPG_Foreground );
				PDI->DrawLine( WidgetMatrix.TransformPosition(FVector(12,GrabberHandlesHalfYPos,0) * UniformScale), WidgetMatrix.TransformPosition(FVector(0,GrabberHandlesYPos,0) * UniformScale), YColor, SDPG_Foreground );
			}
			PDI->SetHitProxy( nullptr );
		}

		if( (DrawAxis&(EAxisList::X|EAxisList::Z)) == (EAxisList::X|EAxisList::Z) )
		{
			PDI->SetHitProxy( new HWidgetAxis(EAxisList::XZ, bDisabled) );
			{
				PDI->DrawLine( WidgetMatrix.TransformPosition(FVector(24,0,0) * UniformScale), WidgetMatrix.TransformPosition(FVector(12,0,12) * UniformScale), XColor, SDPG_Foreground );
				PDI->DrawLine( WidgetMatrix.TransformPosition(FVector(12,0,12) * UniformScale), WidgetMatrix.TransformPosition(FVector(0,0,24) * UniformScale), ZColor, SDPG_Foreground );
			}
			PDI->SetHitProxy( nullptr );
		}

		if( (DrawAxis&(EAxisList::Y|EAxisList::Z)) == (EAxisList::Y|EAxisList::Z) )
		{
			PDI->SetHitProxy( new HWidgetAxis(EAxisList::YZ, bDisabled) );
			{
				PDI->DrawLine( WidgetMatrix.TransformPosition(FVector(0,GrabberHandlesYPos,0) * UniformScale), WidgetMatrix.TransformPosition(FVector(0,GrabberHandlesHalfYPos,12) * UniformScale), YColor, SDPG_Foreground );
				PDI->DrawLine( WidgetMatrix.TransformPosition(FVector(0,GrabberHandlesHalfYPos,12) * UniformScale), WidgetMatrix.TransformPosition(FVector(0,0,24) * UniformScale), ZColor, SDPG_Foreground );
			}
			PDI->SetHitProxy( nullptr );
		}

		// Center cube
		if( (DrawAxis&(EAxisList::XYZ)) == EAxisList::XYZ )
		{
			PDI->SetHitProxy( new HWidgetAxis(EAxisList::XYZ, bDisabled) );

			Render_Cube(PDI, WidgetMatrix, XYZMaterial, UniformScale * 4 );

			PDI->SetHitProxy( nullptr );
		}
	}
}

/**
* Draws the Translate & Rotate Z widget.
*/

void FWidget::Render_TranslateRotateZ( const FSceneView* View, FPrimitiveDrawInterface* PDI, FEditorViewportClient* ViewportClient, const FVector& InLocation, bool bDrawWidget )
{
	// Figure out axis colors

	FColor XYPlaneColor  = ( (CurrentAxis&EAxisList::XY) ==  EAxisList::XY) ? CurrentColor : PlaneColorXY;
	FColor ZRotateColor  = ( ( CurrentAxis&EAxisList::ZRotation ) == EAxisList::ZRotation ) ? CurrentColor : AxisColorZ.ToFColor(true);
	FColor XColor        = ( ( CurrentAxis&EAxisList::X ) == EAxisList::X ) ? CurrentColor : AxisColorX.ToFColor(true);
	FColor YColor        = ( ( CurrentAxis&EAxisList::Y ) == EAxisList::Y && CurrentAxis != EAxisList::ZRotation ) ? CurrentColor : AxisColorY.ToFColor(true);
	FColor ZColor        = ( ( CurrentAxis&EAxisList::Z ) == EAxisList::Z ) ? CurrentColor : AxisColorZ.ToFColor(true);

	// Figure out axis materials
	UMaterialInterface* ZRotateMaterial = (CurrentAxis&EAxisList::ZRotation) == EAxisList::ZRotation? CurrentAxisMaterial : AxisMaterialZ;
	UMaterialInterface* XMaterial = CurrentAxis&EAxisList::X? CurrentAxisMaterial : AxisMaterialX;
	UMaterialInterface* YMaterial = ( CurrentAxis&EAxisList::Y && CurrentAxis != EAxisList::ZRotation ) ? CurrentAxisMaterial : AxisMaterialY;
	UMaterialInterface* ZMaterial = CurrentAxis&EAxisList::Z ? CurrentAxisMaterial : AxisMaterialZ;

	// Figure out axis matrices
	FMatrix AxisMatrix = CustomCoordSystem * FTranslationMatrix( InLocation );
	EAxisList::Type DrawAxis = GetAxisToDraw( ViewportClient->GetWidgetMode() );

	const float ExternalScale = EditorModeTools ? EditorModeTools->GetWidgetScale() : 1.0f;
	FSpaceDescriptor Space(View, ViewportClient, InLocation, ExternalScale);

	// Draw the grabbers
	if( bDrawWidget )
	{
		// Draw the axis lines with arrow heads
		if( DrawAxis&EAxisList::X && (Space.bIsPerspective || Space.bIsLocalSpace || View->ViewMatrices.GetWorldToView().M[0][2] != -1.f) )
		{
			Render_Axis( View, PDI, EAxisList::X, AxisMatrix, XMaterial, XColor, XAxisDir, Space.Scale, bDrawWidget );
		}

		if( DrawAxis&EAxisList::Y && (Space.bIsPerspective || Space.bIsLocalSpace || View->ViewMatrices.GetWorldToView().M[1][2] != -1.f) )
		{
			Render_Axis( View, PDI, EAxisList::Y, AxisMatrix, YMaterial, YColor, YAxisDir, Space.Scale, bDrawWidget );
		}

		if( DrawAxis&EAxisList::Z && (Space.bIsPerspective || Space.bIsLocalSpace || View->ViewMatrices.GetWorldToView().M[0][1] != 1.f) )
		{
			Render_Axis( View, PDI, EAxisList::Z, AxisMatrix, ZMaterial, ZColor, ZAxisDir, Space.Scale, bDrawWidget );
		}

		const bool bDisabled = IsWidgetDisabled();

		const float ScaledRadius = (TRANSLATE_ROTATE_AXIS_CIRCLE_RADIUS * Space.UniformScale) + GetDefault<UTransformGizmoEditorSettings>()->TransformGizmoSize;

		//ZRotation
		if( DrawAxis&EAxisList::ZRotation && (Space.bIsPerspective || Space.bIsLocalSpace || View->ViewMatrices.GetWorldToView().M[0][2] != -1.f) )
		{
			PDI->SetHitProxy( new HWidgetAxis(EAxisList::ZRotation, bDisabled) );
			{
				FVector XAxis;
				FVector YAxis;
				if (Space.bIsLocalSpace)
				{
					// In local space, only need to set axis
					XAxis = CustomCoordSystem.TransformPosition(FVector::ForwardVector);
					YAxis = CustomCoordSystem.TransformPosition(FVector::RightVector);
				}
				else
				{
					// In world space, need to both set and modify axis
					XAxis = CustomCoordSystem.TransformPosition(FVector::ForwardVector.RotateAngleAxis((EditorModeTools ? EditorModeTools->TranslateRotateXAxisAngle : 0), FVector::UpVector));
					YAxis = CustomCoordSystem.TransformPosition(FVector::RightVector.RotateAngleAxis((EditorModeTools ? EditorModeTools->TranslateRotateXAxisAngle : 0), FVector::UpVector));
				}

				FVector BaseArrowPoint = InLocation + XAxis * ScaledRadius;
				DrawFlatArrow(PDI, BaseArrowPoint, XAxis, YAxis, ZRotateColor, ScaledRadius, ScaledRadius*.5f, ZRotateMaterial->GetRenderProxy(), SDPG_Foreground);
			}
			PDI->SetHitProxy( nullptr );
		}

		//XY Plane
		if( Space.bIsPerspective || Space.bIsLocalSpace || View->ViewMatrices.GetWorldToView().M[0][1] != 1.f )
		{
			if( (DrawAxis & EAxisList::XY) == EAxisList::XY ) 
			{
				// Add more sides to the circle if we've been scaled up to keep the circle looking circular
				// An extra side for every 5 extra unreal units seems to produce a nice result
				const int32 CircleSides = (GetDefault<UTransformGizmoEditorSettings>()->TransformGizmoSize > 0) 
					? AXIS_CIRCLE_SIDES + (GetDefault<UTransformGizmoEditorSettings>()->TransformGizmoSize / 5)
					: AXIS_CIRCLE_SIDES;

				PDI->SetHitProxy( new HWidgetAxis(EAxisList::XY, bDisabled) );
				{
					DrawCircle( PDI, InLocation, CustomCoordSystem.TransformPosition( FVector(1,0,0) ), CustomCoordSystem.TransformPosition( FVector(0,1,0) ), XYPlaneColor, ScaledRadius, CircleSides, SDPG_Foreground );
					XYPlaneColor.A = ((CurrentAxis&EAxisList::XY) == EAxisList::XY) ? 0x3f : 0x0f;	//make the disc transparent
					DrawDisc  ( PDI, InLocation, CustomCoordSystem.TransformPosition( FVector(1,0,0) ), CustomCoordSystem.TransformPosition( FVector(0,1,0) ), XYPlaneColor, ScaledRadius, CircleSides, TransparentPlaneMaterialXY->GetRenderProxy(), SDPG_Foreground );
				}
				PDI->SetHitProxy( nullptr );
			}
		}
	}
}

/**
* Draws the 2D widget.
*/

void FWidget::Render_2D(const FSceneView* View, FPrimitiveDrawInterface* PDI, FEditorViewportClient* ViewportClient, const FVector& InLocation, bool bDrawWidget)
{
	//////////////////////////////////////////////////////////////////////////
	// Translation subwidget
	//////////////////////////////////////////////////////////////////////////

	// Figure out axis colors
	const FLinearColor& XColor = (CurrentAxis&EAxisList::X ? (FLinearColor)CurrentColor : AxisColorX);
	const FLinearColor& YColor = (CurrentAxis&EAxisList::Y ? (FLinearColor)CurrentColor : AxisColorY);
	const FLinearColor& ZColor = (CurrentAxis&EAxisList::Z ? (FLinearColor)CurrentColor : AxisColorZ);
	FColor CurrentScreenColor = (CurrentAxis & EAxisList::Screen ? CurrentColor : ScreenSpaceColor);

	// Figure out axis matrices
	FMatrix WidgetMatrix = CustomCoordSystem * FTranslationMatrix(InLocation);

	const float ExternalScale = EditorModeTools ? EditorModeTools->GetWidgetScale() : 1.0f;
	FSpaceDescriptor Space(View, ViewportClient, InLocation, ExternalScale);

	EAxisList::Type DrawAxis = EAxisList::None;
	if (Space.bIsOrthoXY)
	{
		DrawAxis = EAxisList::X;
	}
	else if (Space.bIsOrthoXZ)
	{
		DrawAxis = EAxisList::XZ;
	}
	else if (Space.bIsOrthoYZ)
	{
		DrawAxis = EAxisList::Z;
	}
	else if (Space.bIsPerspective)
	{
		// Find the best plane to move on
		const FVector CameraZAxis = View->ViewMatrices.GetWorldToView().GetColumn(2);
		const FVector LargestAxis = CameraZAxis.GetAbs();
		if (LargestAxis.X > LargestAxis.Y)
		{
			if (LargestAxis.Z > LargestAxis.X)
			{
				DrawAxis = EAxisList::X;
			}
			else
			{
				DrawAxis = EAxisList::Z;
			}
		}
		else
		{
			if (LargestAxis.Y > LargestAxis.Z)
			{
				DrawAxis = EAxisList::XZ;
			}
			else
			{
				DrawAxis = EAxisList::X;
			}
		}
	}

	const bool bDisabled = IsWidgetDisabled();

	// Radius
	const float ScaledRadius = (TWOD_AXIS_CIRCLE_RADIUS * Space.UniformScale) + GetDefault<UTransformGizmoEditorSettings>()->TransformGizmoSize;
	const int32 CircleSides = (GetDefault<UTransformGizmoEditorSettings>()->TransformGizmoSize > 0)
		? AXIS_CIRCLE_SIDES + (GetDefault<UTransformGizmoEditorSettings>()->TransformGizmoSize / 5)
		: AXIS_CIRCLE_SIDES;

	// Draw the grabbers
	if (bDrawWidget)
	{
		FVector CornerPos = FVector(7, 0, 7) * Space.UniformScale;
		FVector AxisSize = FVector(12, 1.2, 12) * Space.UniformScale;
		float CornerLength = 1.2f * Space.UniformScale;

		// Draw the axis lines with arrow heads
		if (Space.ShouldDrawAxisX(DrawAxis))
		{
			UMaterialInstanceDynamic* XMaterial = (CurrentAxis&EAxisList::X ? CurrentAxisMaterial : AxisMaterialX);
			Render_Axis(View, PDI, EAxisList::X, WidgetMatrix, XMaterial, XColor, XAxisDir, Space.Scale, bDrawWidget);
		}

		if (Space.ShouldDrawAxisY(DrawAxis))
		{
			UMaterialInstanceDynamic* YMaterial = (CurrentAxis&EAxisList::Y ? CurrentAxisMaterial : AxisMaterialY);
			Render_Axis(View, PDI, EAxisList::Y, WidgetMatrix, YMaterial, YColor, YAxisDir, Space.Scale, bDrawWidget);
		}

		if (Space.ShouldDrawAxisZ(DrawAxis))
		{
			UMaterialInstanceDynamic* ZMaterial = (CurrentAxis&EAxisList::Z ? CurrentAxisMaterial : AxisMaterialZ);
			Render_Axis(View, PDI, EAxisList::Z, WidgetMatrix, ZMaterial, ZColor, ZAxisDir, Space.Scale, bDrawWidget);
		}

		// After the primitives have been scaled and transformed, we apply this inverse scale that flattens the dimension
		// that was scaled up to prevent it from intersecting with the near plane.  In perspective this won't have any effect,
		// but in the ortho viewports it will prevent scaling in the direction of the camera and thus intersecting the near plane.
		FVector FlattenScale = FVector(Space.Scale.Component(0) == 1.0f ? 1.0f / Space.UniformScale : 1.0f, Space.Scale.Component(1) == 1.0f ? 1.0f / Space.UniformScale : 1.0f, Space.Scale.Component(2) == 1.0f ? 1.0f / Space.UniformScale : 1.0f);
		float ArrowRadius = ScaledRadius * 2.0f;
		float ArrowStartRadius = ScaledRadius * 1.3f;

		uint8 HoverAlpha = 0xff;
		uint8 NormalAlpha = 0x2f;

		if (((DrawAxis&EAxisList::XZ) == EAxisList::XZ) && (Space.bIsPerspective || Space.bIsLocalSpace || Space.bIsOrthoXZ)) // Front
		{
			uint8 Alpha = ((CurrentAxis&EAxisList::XZ) == EAxisList::XZ ? HoverAlpha : NormalAlpha);
			PDI->SetHitProxy(new HWidgetAxis(EAxisList::XZ, bDisabled));
			{
				FColor Color = YColor.ToFColor(true);
				DrawCircle(PDI, InLocation, CustomCoordSystem.TransformPosition(FVector(1, 0, 0)), CustomCoordSystem.TransformPosition(FVector(0, 0, 1)), Color, ScaledRadius, CircleSides, SDPG_Foreground);
				Color.A = Alpha;
				DrawDisc(PDI, InLocation, CustomCoordSystem.TransformPosition(FVector(1, 0, 0)), CustomCoordSystem.TransformPosition(FVector(0, 0, 1)), Color, ScaledRadius, CircleSides, TransparentPlaneMaterialXY->GetRenderProxy(), SDPG_Foreground);
			}
			PDI->SetHitProxy(nullptr);

			PDI->SetHitProxy(new HWidgetAxis(EAxisList::Rotate2D, bDisabled));
			{
				FColor Color = YColor.ToFColor(true);
				Color.A = ((CurrentAxis&EAxisList::Rotate2D) == EAxisList::Rotate2D ? HoverAlpha : NormalAlpha);

				FVector XAxis = CustomCoordSystem.TransformPosition(FVector(1, 0, 0).RotateAngleAxis((EditorModeTools ? EditorModeTools->TranslateRotate2DAngle : 0), FVector(0, -1, 0)));
				FVector YAxis = CustomCoordSystem.TransformPosition(FVector(0, 0, 1).RotateAngleAxis((EditorModeTools ? EditorModeTools->TranslateRotate2DAngle : 0), FVector(0, -1, 0)));
				FVector BaseArrowPoint = InLocation + XAxis * ArrowStartRadius;
				DrawFlatArrow(PDI, BaseArrowPoint, XAxis, YAxis, Color, ArrowRadius, ArrowRadius*.5f, TransparentPlaneMaterialXY->GetRenderProxy(), SDPG_Foreground);
			}
			PDI->SetHitProxy(nullptr);
		}
	}
}

/**
 * Caches off HUD text to display after 3d rendering is complete
 * @param View - Information about the scene/camera/etc
 * @param PDI - Drawing interface
 * @param InLocation - The Origin of the widget
 * @param Axis0 - The Axis that describes a 0 degree rotation
 * @param Axis1 - The Axis that describes a 90 degree rotation
 * @param AngleOfAngle - angle we've rotated so far (in degrees)
 */
void FWidget::CacheRotationHUDText(const FSceneView* View, FPrimitiveDrawInterface* PDI, const FVector& InLocation, const FVector& Axis0, const FVector& Axis1, const float AngleOfChange, const float InScale)
{
	const float TextDistance = (ROTATION_TEXT_RADIUS * InScale) + GetDefault<UTransformGizmoEditorSettings>()->TransformGizmoSize;

	FVector AxisVectors[4] = { Axis0, Axis1, -Axis0, -Axis1};

	for (int i = 0 ; i < 4; ++i)
	{
		FVector PotentialTextPosition = InLocation + (TextDistance)*AxisVectors[i];
		if(View->ScreenToPixel(View->WorldToScreen(PotentialTextPosition), HUDInfoPos))
		{
			if (FMath::IsWithin<float>(HUDInfoPos.X, 0, View->UnscaledViewRect.Width()) && FMath::IsWithin<float>(HUDInfoPos.Y, 0, View->UnscaledViewRect.Height()))
			{
				//only valid screen locations get a valid string
				HUDString = FString::Printf(TEXT("%3.2f"), AngleOfChange);
				break;
			}
		}
	}
}

/**
 * If actively dragging, draws a ring representing the potential rotation of the selected objects, snap ticks, and "delta" markers
 * If not actively dragging, draws a quarter ring representing the closest quadrant to the camera
 * @param View - Information about the scene/camera/etc
 * @param PDI - Drawing interface
 * @param InAxis - Enumeration of axis to rotate about
 * @param InLocation - The Origin of the widget
 * @param Axis0 - The Axis that describes a 0 degree rotation
 * @param Axis1 - The Axis that describes a 90 degree rotation
 * @param InDirectionToWidget - Direction from camera to the widget
 * @param InColor - The color associated with the axis of rotation
 * @param InScale - Multiplier to maintain a constant screen size for rendering the widget
 */
void FWidget::DrawRotationArc(const FSceneView* View, FPrimitiveDrawInterface* PDI, EAxisList::Type InAxis, const FVector& InLocation, const FVector& Axis0, const FVector& Axis1, const FVector& InDirectionToWidget, const FColor& InColor, const float InScale, FVector2D& OutAxisDir)
{
	bool bIsPerspective = ( View->ViewMatrices.GetViewToClip().M[3][3] < 1.0f );
	bool bIsOrtho = !bIsPerspective;

	//if we're in an ortho viewport and the ring is perpendicular to the camera (both Axis0 & Axis1 are perpendicular)
	bIsOrthoDrawingFullRing |= bIsOrtho && (FMath::Abs(Axis0|InDirectionToWidget) < KINDA_SMALL_NUMBER) && (FMath::Abs(Axis1|InDirectionToWidget) < KINDA_SMALL_NUMBER);

	FColor ArcColor = InColor;
	ArcColor.A = LargeOuterAlpha;

	if (bDragging || (bIsOrthoDrawingFullRing))
	{
		if ((CurrentAxis&InAxis) || (bIsOrthoDrawingFullRing))
		{
			bool bDrawingArcBallAlso = CurrentAxis == EAxisList::XYZ;

			float DeltaRotation = GetDeltaRotation();
			float AdjustedDeltaRotation = IsRotationLocalSpace() ? -DeltaRotation : DeltaRotation;
			float AbsRotation = FRotator::ClampAxis(FMath::Abs(DeltaRotation));
			float AngleOfChangeRadians (AbsRotation * PI / 180.f);

			//always draw clockwise, so if we're negative we need to flip the angle
			float StartAngle = AdjustedDeltaRotation < 0.0f ? -AngleOfChangeRadians : 0.0f;
			float FilledAngle = AngleOfChangeRadians;

			//the axis of rotation
			FVector ZAxis = Axis0 ^ Axis1;

			ArcColor.A = LargeOuterAlpha;
			DrawPartialRotationArc(View, PDI, InAxis, InLocation,  Axis0, Axis1, StartAngle, StartAngle + FilledAngle, ArcColor, InScale, InDirectionToWidget);
			ArcColor.A = SmallOuterAlpha;
			DrawPartialRotationArc(View, PDI, InAxis, InLocation,  Axis0, Axis1, StartAngle + FilledAngle, StartAngle + 2*PI, ArcColor, InScale, InDirectionToWidget);

			ArcColor = (CurrentAxis&InAxis && !bDrawingArcBallAlso) ? CurrentColor : ArcColor;
			if (!bDrawingArcBallAlso)
			{
				//Hallow Arrow
				ArcColor.A = 0;
				DrawStartStopMarker(PDI, InLocation, Axis0, Axis1, 0, ArcColor, InScale);
				//Filled Arrow
				ArcColor.A = LargeOuterAlpha;
				DrawStartStopMarker(PDI, InLocation, Axis0, Axis1, AdjustedDeltaRotation, ArcColor, InScale);
				ArcColor.A = 255;


				FVector SnapLocation = InLocation;

				if (GetDefault<ULevelEditorViewportSettings>()->RotGridEnabled)
				{
					float DeltaAngle = GEditor->GetRotGridSize().Yaw;
					//every 22.5 degrees
					float TickMarker = 22.5f;
					for (float Angle = 0; Angle < 360.f; Angle += DeltaAngle)
					{
						FVector GridAxis = Axis0.RotateAngleAxis(Angle, ZAxis);
						float PercentSize = (FMath::Fmod(Angle, TickMarker) == 0) ? .75f : .25f;
						if (FMath::Fmod(Angle, 90.f) != 0)
						{
							DrawSnapMarker(PDI, SnapLocation, GridAxis, FVector::ZeroVector, ArcColor, InScale, 0.0f, PercentSize);
						}
					}
				}

				//draw axis tick marks
				FColor AxisColor = InColor;
				//Rotate Colors to match Axis 0
				Swap(AxisColor.R, AxisColor.G);
				Swap(AxisColor.B, AxisColor.R);
				AxisColor.A = (AdjustedDeltaRotation == 0) ? MAX_uint8 : LargeOuterAlpha;
				DrawSnapMarker(PDI, SnapLocation, Axis0, Axis1, AxisColor, InScale, .25f);
				AxisColor.A = (AdjustedDeltaRotation == 180.f) ? MAX_uint8 : LargeOuterAlpha;
				DrawSnapMarker(PDI, SnapLocation, -Axis0, -Axis1, AxisColor, InScale, .25f);

				//Rotate Colors to match Axis 1
				Swap(AxisColor.R, AxisColor.G);
				Swap(AxisColor.B, AxisColor.R);
				AxisColor.A = (AdjustedDeltaRotation == 90.f) ? MAX_uint8 : LargeOuterAlpha;
				DrawSnapMarker(PDI, SnapLocation, Axis1, -Axis0, AxisColor, InScale, .25f);
				AxisColor.A = (AdjustedDeltaRotation == 270.f) ? MAX_uint8 : LargeOuterAlpha;
				DrawSnapMarker(PDI, SnapLocation, -Axis1, Axis0, AxisColor, InScale, .25f);

				if (bDragging)
				{
					float OffsetAngle = IsRotationLocalSpace() ? 0 : AdjustedDeltaRotation;

					if (AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward)
					{
						if (InAxis == EAxisList::Y || InAxis == EAxisList::Z)
						{
							DeltaRotation = -DeltaRotation;
						}
					}
					CacheRotationHUDText(View, PDI, InLocation, Axis0.RotateAngleAxis(OffsetAngle, ZAxis), Axis1.RotateAngleAxis(OffsetAngle, ZAxis), DeltaRotation, InScale);
				}
			}
		}
	}
	else
	{
		//Reverse the axes based on camera view
		bool bMirrorAxis0 = ((Axis0 | InDirectionToWidget) <= 0.0f);
		bool bMirrorAxis1 = ((Axis1 | InDirectionToWidget) <= 0.0f);

		FVector RenderAxis0 = bMirrorAxis0 ? Axis0 : -Axis0;
		FVector RenderAxis1 = bMirrorAxis1 ? Axis1 : -Axis1;
		float Direction = (bMirrorAxis0 ^ bMirrorAxis1) ? -1.0f : 1.0f;

		DrawPartialRotationArc(View, PDI, InAxis, InLocation, RenderAxis0, RenderAxis1, 0, PI/2, ArcColor, InScale, InDirectionToWidget);

		FVector2D Axis0ScreenLocation;
		if (!View->ScreenToPixel(View->WorldToScreen(InLocation + RenderAxis0 * 64.0f), Axis0ScreenLocation))
		{
			Axis0ScreenLocation.X = Axis0ScreenLocation.Y = 0;
		}

		FVector2D Axis1ScreenLocation;
		if (!View->ScreenToPixel(View->WorldToScreen(InLocation + RenderAxis1 * 64.0f), Axis1ScreenLocation))
		{
			Axis1ScreenLocation.X = Axis1ScreenLocation.Y = 0;
		}

		OutAxisDir = ((Axis1ScreenLocation - Axis0ScreenLocation) * Direction).GetSafeNormal();
	}
}

/**
 * If actively dragging, draws a ring representing the potential rotation of the selected objects, snap ticks, and "delta" markers
 * If not actively dragging, draws a quarter ring representing the closest quadrant to the camera
 * @param View - Information about the scene/camera/etc
 * @param PDI - Drawing interface
 * @param InAxis - Enumeration of axis to rotate about
 * @param InLocation - The Origin of the widget
 * @param Axis0 - The Axis that describes a 0 degree rotation
 * @param Axis1 - The Axis that describes a 90 degree rotation
 * @param InStartAngle - The starting angle about (Axis0^Axis1) to render the arc, in radians
 * @param InEndAngle - The ending angle about (Axis0^Axis1) to render the arc, in radians
 * @param InColor - The color associated with the axis of rotation
 * @param InScale - Multiplier to maintain a constant screen size for rendering the widget
 */
void FWidget::DrawPartialRotationArc(const FSceneView* View, FPrimitiveDrawInterface* PDI, EAxisList::Type InAxis, const FVector& InLocation, const FVector& Axis0, const FVector& Axis1, const float InStartAngle, const float InEndAngle, const FColor& InColor, const float InScale, const FVector& InDirectionToWidget )
{
	const float InnerRadius = (INNER_AXIS_CIRCLE_RADIUS * InScale) + GetDefault<UTransformGizmoEditorSettings>()->TransformGizmoSize;
	const float OuterRadius = (OUTER_AXIS_CIRCLE_RADIUS * InScale) + GetDefault<UTransformGizmoEditorSettings>()->TransformGizmoSize;

	bool bIsPerspective = ( View->ViewMatrices.GetViewToClip().M[3][3] < 1.0f );
	PDI->SetHitProxy( new HWidgetAxis( InAxis ) );
	{
		FThickArcParams OuterArcParams(PDI, InLocation, TransparentPlaneMaterialXY, InnerRadius, OuterRadius);
		FColor OuterColor = ( (CurrentAxis&InAxis  && ( CurrentAxis != EAxisList::XYZ ) )? CurrentColor : InColor );
		//Pass through alpha
		OuterColor.A = InColor.A;
		DrawThickArc(OuterArcParams, Axis0, Axis1, InStartAngle, InEndAngle, OuterColor, InDirectionToWidget, !bIsPerspective );
	}
	PDI->SetHitProxy( nullptr );

	const bool bIsHitProxyView = View->Family->EngineShowFlags.HitProxies;
	if (bIsPerspective && !bIsHitProxyView && !PDI->IsHitTesting())
	{
		FThickArcParams InnerArcParams(PDI, InLocation, GridMaterial, 0.0f, InnerRadius);
		FColor InnerColor = InColor;
		//if something is selected and it's not this
		InnerColor.A = ((CurrentAxis & InAxis) && !bDragging) ? LargeInnerAlpha : SmallInnerAlpha;
		DrawThickArc(InnerArcParams, Axis0, Axis1, InStartAngle, InEndAngle, InnerColor, InDirectionToWidget, false );
	}
}

/**
 * Renders a portion of an arc for the rotation widget
 * @param InParams - Material, Radii, etc
 * @param InStartAxis - Start of the arc, in radians
 * @param InEndAxis - End of the arc, in radians
 * @param InColor - Color to use for the arc
 */
void FWidget::DrawThickArc (const FThickArcParams& InParams, const FVector& Axis0, const FVector& Axis1, const float InStartAngle, const float InEndAngle, const FColor& InColor, const FVector& InDirectionToWidget, bool bIsOrtho )
{
	if (InColor.A == 0)
	{
		return;
	}

	// Add more sides to the circle if we've been scaled up to keep the circle looking circular
	// An extra side for every 5 extra unreal units seems to produce a nice result
	const int32 CircleSides = (GetDefault<UTransformGizmoEditorSettings>()->TransformGizmoSize > 0) 
		? AXIS_CIRCLE_SIDES + (GetDefault<UTransformGizmoEditorSettings>()->TransformGizmoSize / 5)
		: AXIS_CIRCLE_SIDES;
	const int32 NumPoints = FMath::TruncToInt(CircleSides * (InEndAngle-InStartAngle)/(PI/2)) + 1;

	FColor TriangleColor = InColor;
	FColor RingColor = InColor;
	RingColor.A = MAX_uint8;

	FVector ZAxis = Axis0 ^ Axis1;
	FVector LastWorldVertex;

	FDynamicMeshBuilder MeshBuilder(InParams.PDI->View->GetFeatureLevel());

	for (int32 RadiusIndex = 0; RadiusIndex < 2; ++RadiusIndex)
	{
		float Radius = (RadiusIndex == 0) ? InParams.OuterRadius : InParams.InnerRadius;
		float TCRadius = Radius / (float) InParams.OuterRadius;
		//Compute vertices for base circle.
		for(int32 VertexIndex = 0;VertexIndex <= NumPoints;VertexIndex++)
		{
			float Percent = VertexIndex/(float)NumPoints;
			float Angle = FMath::Lerp(InStartAngle, InEndAngle, Percent);
			float AngleDeg = FRotator::ClampAxis(Angle * 180.f / PI);

			FVector VertexDir = Axis0.RotateAngleAxis(AngleDeg, ZAxis);
			VertexDir.Normalize();

			float TCAngle = Percent*(PI/2);
			FVector2f TC(TCRadius*FMath::Cos(Angle), TCRadius*FMath::Sin(Angle));

			// Keep the vertices in local space so that we don't lose precision when dealing with LWC
			// The local-to-world transform is handled in the MeshBuilder.Draw() call at the end of this function
			const FVector VertexPosition = VertexDir*Radius;
			FVector Normal = VertexPosition;
			Normal.Normalize();

			FDynamicMeshVertex MeshVertex;
			MeshVertex.Position = (FVector3f)VertexPosition;
			MeshVertex.Color = TriangleColor;
			MeshVertex.TextureCoordinate[0] = TC;

			MeshVertex.SetTangents(
				(FVector3f)-ZAxis,
				FVector3f((-ZAxis) ^ Normal),
				(FVector3f)Normal
				);

			MeshBuilder.AddVertex(MeshVertex); //Add bottom vertex

			// Push out the arc line borders so they dont z-fight with the mesh arcs
			// DrawLine needs vertices in world space, but this is fine because it takes FVectors and works with LWC well
			FVector EndLinePos = VertexPosition + InParams.Position;
			if (VertexIndex != 0)
			{
				InParams.PDI->DrawLine(LastWorldVertex,EndLinePos,RingColor,SDPG_Foreground);
			}
			LastWorldVertex = EndLinePos;
		}
	}

	//Add top/bottom triangles, in the style of a fan.
	int32 InnerVertexStartIndex = NumPoints + 1;
	for(int32 VertexIndex = 0; VertexIndex < NumPoints; VertexIndex++)
	{
		MeshBuilder.AddTriangle(VertexIndex, VertexIndex+1, InnerVertexStartIndex+VertexIndex);
		MeshBuilder.AddTriangle(VertexIndex+1, InnerVertexStartIndex+VertexIndex+1, InnerVertexStartIndex+VertexIndex);
	}

	MeshBuilder.Draw(InParams.PDI, FTranslationMatrix(InParams.Position), InParams.Material->GetRenderProxy(),SDPG_Foreground,0.f);
}

/**
 * Draws protractor like ticks where the rotation widget would snap too.
 * Also, used to draw the wider axis tick marks
 * @param PDI - Drawing interface
 * @param InLocation - The Origin of the widget
 * @param Axis0 - The Axis that describes a 0 degree rotation
 * @param Axis1 - The Axis that describes a 90 degree rotation
 * @param InAngle - The Angle to rotate about the axis of rotation, the vector (Axis0 ^ Axis1)
 * @param InColor - The color to use for line/poly drawing
 * @param InScale - Multiplier to maintain a constant screen size for rendering the widget
 * @param InWidthPercent - The percent of the distance between the outer ring and inner ring to use for tangential thickness
 * @param InPercentSize - The percent of the distance between the outer ring and inner ring to use for radial distance
 */
void FWidget::DrawSnapMarker(FPrimitiveDrawInterface* PDI, const FVector& InLocation, const FVector& Axis0, const FVector& Axis1, const FColor& InColor, const float InScale, const float InWidthPercent, const float InPercentSize)
{
	const float InnerDistance = (INNER_AXIS_CIRCLE_RADIUS * InScale) + GetDefault<UTransformGizmoEditorSettings>()->TransformGizmoSize;
	const float OuterDistance = (OUTER_AXIS_CIRCLE_RADIUS * InScale) + GetDefault<UTransformGizmoEditorSettings>()->TransformGizmoSize;
	const float MaxMarkerHeight = OuterDistance - InnerDistance;
	const float MarkerWidth = MaxMarkerHeight*InWidthPercent;
	const float MarkerHeight = MaxMarkerHeight*InPercentSize;

	FVector LocalVertices[4];
	LocalVertices[0] = (OuterDistance)*Axis0 - (MarkerWidth*.5)*Axis1;
	LocalVertices[1] = LocalVertices[0] + (MarkerWidth)*Axis1;
	LocalVertices[2] = (OuterDistance-MarkerHeight)*Axis0 - (MarkerWidth*.5)*Axis1;
	LocalVertices[3] = LocalVertices[2] + (MarkerWidth)*Axis1;

	//draw at least one line
	// DrawLine needs vertices in world space, but this is fine because it takes FVectors and works with LWC well
	PDI->DrawLine(LocalVertices[0] + InLocation, LocalVertices[2] + InLocation, InColor, SDPG_Foreground);

	//if there should be thickness, draw the other lines
	if (InWidthPercent > 0.0f)
	{
		PDI->DrawLine(LocalVertices[0] + InLocation, LocalVertices[1] + InLocation, InColor, SDPG_Foreground);
		PDI->DrawLine(LocalVertices[1] + InLocation, LocalVertices[3] + InLocation, InColor, SDPG_Foreground);
		PDI->DrawLine(LocalVertices[2] + InLocation, LocalVertices[3] + InLocation, InColor, SDPG_Foreground);

		//fill in the box
		FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());

		for(int32 VertexIndex = 0;VertexIndex < 4; VertexIndex++)
		{
			FDynamicMeshVertex MeshVertex;
			MeshVertex.Position = (FVector3f)LocalVertices[VertexIndex];
			MeshVertex.Color = InColor;
			MeshVertex.TextureCoordinate[0] = FVector2f(0.0f, 0.0f);
			MeshVertex.SetTangents(
				(FVector3f)Axis0,
				(FVector3f)Axis1,
				FVector3f((Axis0) ^ Axis1)
				);
			MeshBuilder.AddVertex(MeshVertex); //Add bottom vertex
		}

		MeshBuilder.AddTriangle(0, 1, 2);
		MeshBuilder.AddTriangle(1, 3, 2);
		MeshBuilder.Draw(PDI, FTranslationMatrix(InLocation), TransparentPlaneMaterialXY->GetRenderProxy(),SDPG_Foreground,0.f);
	}
}

/**
 * Draw Start/Stop Marker to show delta rotations along the arc of rotation
 * @param PDI - Drawing interface
 * @param InLocation - The Origin of the widget
 * @param Axis0 - The Axis that describes a 0 degree rotation
 * @param Axis1 - The Axis that describes a 90 degree rotation
 * @param InAngle - The Angle to rotate about the axis of rotation, the vector (Axis0 ^ Axis1), units are degrees
 * @param InColor - The color to use for line/poly drawing
 * @param InScale - Multiplier to maintain a constant screen size for rendering the widget
 */
void FWidget::DrawStartStopMarker(FPrimitiveDrawInterface* PDI, const FVector& InLocation, const FVector& Axis0, const FVector& Axis1, const float InAngle, const FColor& InColor, const float InScale)
{
	const float ArrowHeightPercent = .8f;
	const float InnerDistance = (INNER_AXIS_CIRCLE_RADIUS * InScale) + GetDefault<UTransformGizmoEditorSettings>()->TransformGizmoSize;
	const float OuterDistance = (OUTER_AXIS_CIRCLE_RADIUS * InScale) + GetDefault<UTransformGizmoEditorSettings>()->TransformGizmoSize;
	const float RingHeight = OuterDistance - InnerDistance;
	const float ArrowHeight = RingHeight*ArrowHeightPercent;
	const float ThirtyDegrees = PI / 6.0f;
	const float HalfArrowidth = ArrowHeight*FMath::Tan(ThirtyDegrees);

	FVector ZAxis = Axis0 ^ Axis1;
	FVector RotatedAxis0 = Axis0.RotateAngleAxis(InAngle, ZAxis);
	FVector RotatedAxis1 = Axis1.RotateAngleAxis(InAngle, ZAxis);

	FVector LocalVertices[3];
	LocalVertices[0] = (OuterDistance)*RotatedAxis0;
	LocalVertices[1] = LocalVertices[0] + (ArrowHeight)*RotatedAxis0 - HalfArrowidth*RotatedAxis1;
	LocalVertices[2] = LocalVertices[1] + (2*HalfArrowidth)*RotatedAxis1;

	// DrawLine needs vertices in world space, but this is fine because it takes FVectors and works with LWC well
	PDI->DrawLine(LocalVertices[0] + InLocation, LocalVertices[1] + InLocation, InColor, SDPG_Foreground);
	PDI->DrawLine(LocalVertices[1] + InLocation, LocalVertices[2] + InLocation, InColor, SDPG_Foreground);
	PDI->DrawLine(LocalVertices[0] + InLocation, LocalVertices[2] + InLocation, InColor, SDPG_Foreground);

	if (InColor.A > 0)
	{
		//fill in the box
		FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());

		for(int32 VertexIndex = 0;VertexIndex < 3; VertexIndex++)
		{
			FDynamicMeshVertex MeshVertex;
			MeshVertex.Position = (FVector3f)LocalVertices[VertexIndex];
			MeshVertex.Color = InColor;
			MeshVertex.TextureCoordinate[0] = FVector2f(0.0f, 0.0f);
			MeshVertex.SetTangents(
				(FVector3f)RotatedAxis0,
				(FVector3f)RotatedAxis1,
				FVector3f((RotatedAxis0) ^ RotatedAxis1)
				);
			MeshBuilder.AddVertex(MeshVertex); //Add bottom vertex
		}

		MeshBuilder.AddTriangle(0, 1, 2);
		MeshBuilder.Draw(PDI, FTranslationMatrix(InLocation), TransparentPlaneMaterialXY->GetRenderProxy(),SDPG_Foreground,0.f);
	}
}

EAxisList::Type FWidget::GetAxisToDraw( UE::Widget::EWidgetMode WidgetMode ) const
{
	return EditorModeTools ? EditorModeTools->GetWidgetAxisToDraw( WidgetMode ) : EAxisList::All;
}
