// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserDefinedStructure/UserDefinedStructEditorData.h"
#include "Misc/ITransactionObjectAnnotation.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "UObject/UnrealType.h"
#include "UObject/ObjectSaveContext.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Kismet2/StructureEditorUtils.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Blueprint/BlueprintSupport.h"
#include "Editor.h"
#include "EdGraphSchema_K2.h"
#include "Internationalization/TextPackageNamespaceUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UserDefinedStructEditorData)

#define LOCTEXT_NAMESPACE "UserDefinedStructEditorData"

#if USE_STABLE_LOCALIZATION_KEYS 

namespace
{
	void LocalizationUpdateTextsPackageNamespaceCallback(UObject* Object, TextNamespaceUtil::FUpdateTextsPackageNamespace& UpdateTextsPackageNamespace)
	{
		// User Defined Structs need some special handling as they store their default data in a way that serialize doesn't pick up
		if (UUserDefinedStruct* const UserDefinedStruct = Cast<UUserDefinedStruct>(Object))
		{
			if (UUserDefinedStructEditorData* UDSEditorData = Cast<UUserDefinedStructEditorData>(UserDefinedStruct->EditorData))
			{
				for (FStructVariableDescription& StructVariableDesc : UDSEditorData->VariablesDescriptions)
				{
					if (StructVariableDesc.Category == UEdGraphSchema_K2::PC_Text)
					{
						FText StructVariableValue;
						if (FTextStringHelper::ReadFromBuffer(*StructVariableDesc.DefaultValue, StructVariableValue) && UpdateTextsPackageNamespace.UpdateSingleText(StructVariableValue))
						{
							FTextStringHelper::WriteToBuffer(StructVariableDesc.DefaultValue, StructVariableValue);
						}
					}
				}
			}
		}

		UpdateTextsPackageNamespace.Serialize(Object);
	}

	static const TextNamespaceUtil::FAutoRegisterLocalizationUpdateTextsPackageNamespaceCallback AutomaticRegistrationOfLocalizationUpdateTextsPackageNamespace(UUserDefinedStruct::StaticClass(), &LocalizationUpdateTextsPackageNamespaceCallback);
}

#endif // USE_STABLE_LOCALIZATION_KEYS

const FName FStructVariableMetaData::ClampMin("ClampMin");
const FName FStructVariableMetaData::ClampMax("ClampMax");
const FName FStructVariableMetaData::UIMin("UIMin");
const FName FStructVariableMetaData::UIMax("UIMax");

void FStructVariableDescription::PostSerialize(const FArchive& Ar)
{
	if (ContainerType == EPinContainerType::None)
	{
		ContainerType = FEdGraphPinType::ToPinContainerType(bIsArray_DEPRECATED, bIsSet_DEPRECATED, bIsMap_DEPRECATED);
	}

	if (Ar.UEVer() < VER_UE4_ADDED_SOFT_OBJECT_PATH)
	{
		// Fix up renamed categories
		if (Category == TEXT("asset"))
		{
			Category = TEXT("softobject");
		}
		else if (Category == TEXT("assetclass"))
		{
			Category = TEXT("softclass");
		}
	}

	bool FixupPinCategories =
		Ar.IsLoading() &&
		((Category == UEdGraphSchema_K2::PC_Double) || (Category == UEdGraphSchema_K2::PC_Float));

	if (FixupPinCategories)
	{
		Category = UEdGraphSchema_K2::PC_Real;
		SubCategory = UEdGraphSchema_K2::PC_Double;
	}

	bool FixupPinValueCategories =
		Ar.IsLoading() &&
		((PinValueType.TerminalCategory == UEdGraphSchema_K2::PC_Double) || (PinValueType.TerminalCategory == UEdGraphSchema_K2::PC_Float));

	if (FixupPinValueCategories)
	{
		PinValueType.TerminalCategory = UEdGraphSchema_K2::PC_Real;
		PinValueType.TerminalSubCategory = UEdGraphSchema_K2::PC_Double;
	}
}

bool FStructVariableDescription::SetPinType(const FEdGraphPinType& VarType)
{
	Category = VarType.PinCategory;
	SubCategory = VarType.PinSubCategory;
	SubCategoryObject = VarType.PinSubCategoryObject.Get();
	PinValueType = VarType.PinValueType;
	ContainerType = VarType.ContainerType;

	return !VarType.bIsReference && !VarType.bIsWeakPointer;
}

FEdGraphPinType FStructVariableDescription::ToPinType() const
{
	return FEdGraphPinType(Category, SubCategory, SubCategoryObject.LoadSynchronous(), ContainerType, false, PinValueType);
}

bool FStructVariableDescription::HasSameType(const FStructVariableDescription& Other) const
{
	return Category == Other.Category
		&& SubCategory == Other.SubCategory
		&& SubCategoryObject == Other.SubCategoryObject
		&& PinValueType == Other.PinValueType
		&& ContainerType == Other.ContainerType;
}

UUserDefinedStructEditorData::UUserDefinedStructEditorData(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer), UniqueNameId(0), CachedStructureChange()
{
}

uint32 UUserDefinedStructEditorData::GenerateUniqueNameIdForMemberVariable()
{
	const uint32 Result = UniqueNameId;
	++UniqueNameId;
	return Result;
}

UUserDefinedStruct* UUserDefinedStructEditorData::GetOwnerStruct() const
{
	return Cast<UUserDefinedStruct>(GetOuter());
}

void UUserDefinedStructEditorData::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
}

void UUserDefinedStructEditorData::PostUndo(bool bSuccess)
{
	GEditor->UnregisterForUndo(this);
	// TODO: In the undo case we might want to flip the change type since an add is now a remove and vice versa
	FStructureEditorUtils::OnStructureChanged(GetOwnerStruct(), CachedStructureChange);
	CachedStructureChange = FStructureEditorUtils::Unknown;
}

void UUserDefinedStructEditorData::ConsolidatedPostEditUndo(const FStructureEditorUtils::EStructureEditorChangeInfo TransactedStructureChange)
{
	ensure(CachedStructureChange == FStructureEditorUtils::Unknown);
	CachedStructureChange = TransactedStructureChange;
	GEditor->RegisterForUndo(this);
}

void UUserDefinedStructEditorData::PostEditUndo()
{
	Super::PostEditUndo();
	ConsolidatedPostEditUndo(FStructureEditorUtils::Unknown);
}

class FStructureTransactionAnnotation : public ITransactionObjectAnnotation
{
public:
	FStructureTransactionAnnotation()
		: ActiveChange(FStructureEditorUtils::Unknown)
	{
	}

	explicit FStructureTransactionAnnotation(FStructureEditorUtils::EStructureEditorChangeInfo ChangeInfo)
		: ActiveChange(ChangeInfo)
	{
	}

	//~ ITransactionObjectAnnotation interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override {}
	virtual void Serialize(FArchive& Ar) override
	{
		enum class EVersion : uint8
		{
			InitialVersion = 0,
			// -----<new versions can be added above this line>-------------------------------------------------
			VersionPlusOne,
			LatestVersion = VersionPlusOne - 1
		};

		EVersion Version = EVersion::LatestVersion;
		Ar << Version;

		if (Version > EVersion::LatestVersion)
		{
			Ar.SetError();
			return;
		}

		int32 ActiveChangeInt = (int32)ActiveChange;
		Ar << ActiveChangeInt;
		ActiveChange = (FStructureEditorUtils::EStructureEditorChangeInfo)ActiveChangeInt;
	}

	FStructureEditorUtils::EStructureEditorChangeInfo GetActiveChange()
	{
		return ActiveChange;
	}

protected:
	FStructureEditorUtils::EStructureEditorChangeInfo ActiveChange;
};

TSharedPtr<ITransactionObjectAnnotation> UUserDefinedStructEditorData::FactoryTransactionAnnotation(const ETransactionAnnotationCreationMode InCreationMode) const
{
	if (InCreationMode == UObject::ETransactionAnnotationCreationMode::DefaultInstance)
	{
		return MakeShared<FStructureTransactionAnnotation>();
	}

	return MakeShared<FStructureTransactionAnnotation>(FStructureEditorUtils::FStructEditorManager::ActiveChange);
}

void UUserDefinedStructEditorData::PostEditUndo(TSharedPtr<ITransactionObjectAnnotation> TransactionAnnotation)
{
	Super::PostEditUndo();
	FStructureEditorUtils::EStructureEditorChangeInfo TransactedStructureChange = FStructureEditorUtils::Unknown;

	if (TransactionAnnotation.IsValid())
	{
		TSharedPtr<FStructureTransactionAnnotation> StructAnnotation = StaticCastSharedPtr<FStructureTransactionAnnotation>(TransactionAnnotation);
		if (StructAnnotation.IsValid())
		{
			TransactedStructureChange = StructAnnotation->GetActiveChange();
		}
	}
	ConsolidatedPostEditUndo(TransactedStructureChange);
}

void UUserDefinedStructEditorData::PostLoad()
{
	Super::PostLoad();

	if (!bMetaDataConverted && !HasAnyFlags(RF_ClassDefaultObject))
	{
		const UUserDefinedStruct* ScriptStruct = GetOwnerStruct();
		if (ensure(ScriptStruct != nullptr))
		{
			TMap<FName, FString>* Map = GetPackage()->GetMetaData().GetMapForObject(ScriptStruct);
			if (Map)
			{
				for (const TPair<FName, FString>& Data : *Map)
				{
					if (Data.Key != FBlueprintMetadata::MD_Tooltip)
					{
						MetaData.Add(Data.Key, Data.Value);
					}
				}
			}
			bMetaDataConverted = true;
		}
	}
}

void UUserDefinedStructEditorData::PostLoadSubobjects(FObjectInstancingGraph* OuterInstanceGraph)
{
	Super::PostLoadSubobjects(OuterInstanceGraph);

	for (FStructVariableDescription& VarDesc : VariablesDescriptions)
	{
		VarDesc.bInvalidMember = !FStructureEditorUtils::CanHaveAMemberVariableOfType(GetOwnerStruct(), VarDesc.ToPinType());
	}
}

void UUserDefinedStructEditorData::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	// Prior to saving, ensure that editor data matches up with the default instance data.
	RefreshValuesFromDefaultInstance();
}

const uint8* UUserDefinedStructEditorData::GetDefaultInstance() const
{
	return GetOwnerStruct()->GetDefaultInstance();
}

void UUserDefinedStructEditorData::RecreateDefaultInstance(FString* OutLog)
{
	UUserDefinedStruct* ScriptStruct = GetOwnerStruct();
	ScriptStruct->DefaultStructInstance.Recreate(ScriptStruct);
	ReinitializeDefaultInstance(OutLog);
}

void UUserDefinedStructEditorData::ReinitializeDefaultInstance(FString* OutLog)
{
	UUserDefinedStruct* ScriptStruct = GetOwnerStruct();
	uint8* StructData = ScriptStruct->DefaultStructInstance.GetStructMemory();
	ensure(ScriptStruct->DefaultStructInstance.IsValid() && ScriptStruct->DefaultStructInstance.GetStruct() == ScriptStruct);
	if (ScriptStruct->DefaultStructInstance.IsValid() && StructData)
	{
		// When loading, the property's default value may end up being filled with a placeholder. 
		// This tracker object allows the linker to track the actual object that is being filled in 
		// so it can calculate an offset to the property and write in the placeholder value:
		FScopedPlaceholderRawContainerTracker TrackDefaultObject(StructData);

		ScriptStruct->DefaultStructInstance.SetPackage(ScriptStruct->GetOutermost());

		for (TFieldIterator<FProperty> It(ScriptStruct); It; ++It)
		{
			FProperty* Property = *It;
			if (Property)
			{
				const FGuid VarGuid = FStructureEditorUtils::GetGuidFromPropertyName(Property->GetFName());
				FStructVariableDescription* VarDesc = VariablesDescriptions.FindByPredicate(FStructureEditorUtils::FFindByGuidHelper<FStructVariableDescription>(VarGuid));
				if (VarDesc == nullptr)
				{
					VarDesc = VariablesDescriptions.FindByPredicate(FStructureEditorUtils::FFindByNameHelper<FStructVariableDescription>(Property->GetFName()));
				}

				if (VarDesc && !VarDesc->CurrentDefaultValue.IsEmpty())
				{
					if (!FBlueprintEditorUtils::PropertyValueFromString(Property, VarDesc->CurrentDefaultValue, StructData, ScriptStruct, PPF_DuplicateVerbatim))
					{
						const FString Message = FString::Printf(TEXT("Cannot parse value. Property: %s String: \"%s\" ")
							, *Property->GetDisplayNameText().ToString()
							, *VarDesc->CurrentDefaultValue);
						UE_LOGF(LogClass, Warning, "UUserDefinedStructEditorData::RecreateDefaultInstance %ls Struct: %ls ", *Message, *GetPathNameSafe(ScriptStruct));
						if (OutLog)
						{
							OutLog->Append(Message);
						}
					}
				}
			}
		}
	}

	// Make sure StructFlags are in sync with the new default instance:
	ScriptStruct->UpdateStructFlags();
}

void UUserDefinedStructEditorData::CleanDefaultInstance()
{
	UUserDefinedStruct* ScriptStruct = GetOwnerStruct();
	ensure(!ScriptStruct->DefaultStructInstance.IsValid() || ScriptStruct->DefaultStructInstance.GetStruct() == GetOwnerStruct());
	ScriptStruct->DefaultStructInstance.Destroy();
}

void UUserDefinedStructEditorData::RefreshValuesFromDefaultInstance()
{
	UUserDefinedStruct* ScriptStruct = GetOwnerStruct();
	if (!ScriptStruct || !ScriptStruct->DefaultStructInstance.IsValid())
	{
		return;
	}

	ensure(ScriptStruct->DefaultStructInstance.GetStruct() == ScriptStruct);
	if (uint8* StructData = ScriptStruct->DefaultStructInstance.GetStructMemory())
	{
		for (TFieldIterator<FProperty> It(ScriptStruct); It; ++It)
		{
			if (const FProperty* Property = *It)
			{
				const FGuid VarGuid = FStructureEditorUtils::GetGuidFromPropertyName(Property->GetFName());
				FStructVariableDescription* VarDesc = VariablesDescriptions.FindByPredicate(FStructureEditorUtils::FFindByGuidHelper<FStructVariableDescription>(VarGuid));
				if (VarDesc == nullptr)
				{
					VarDesc = VariablesDescriptions.FindByPredicate(FStructureEditorUtils::FFindByNameHelper<FStructVariableDescription>(Property->GetFName()));
				}

				if (VarDesc)
				{
					// If the default value has been changed elsewhere, don't refresh it here. This is typically the result of an edit action prior to compilation.
					const bool bHasDefaultValueChanged = (VarDesc->DefaultValue != VarDesc->CurrentDefaultValue);

					FBlueprintEditorUtils::PropertyValueToString(Property, StructData, VarDesc->CurrentDefaultValue, ScriptStruct);
					if (!bHasDefaultValueChanged)
					{
						VarDesc->DefaultValue = VarDesc->CurrentDefaultValue;
					}
				}
			}
		}
	}
}

FProperty* UUserDefinedStructEditorData::FindProperty(const UUserDefinedStruct* Struct, const FName Name) const
{
	const FGuid PropertyGuid = FStructureEditorUtils::GetGuidFromPropertyName(Name);
	FProperty* EditorProperty = PropertyGuid.IsValid()
		? FStructureEditorUtils::GetPropertyByGuid(Struct, PropertyGuid)
		: FStructureEditorUtils::GetPropertyByFriendlyName(Struct, Name.ToString());

	ensure(!EditorProperty || !PropertyGuid.IsValid() || PropertyGuid == FStructureEditorUtils::GetGuidForProperty(EditorProperty));
	return EditorProperty;
}

FString UUserDefinedStructEditorData::GetFriendlyNameForProperty(const UUserDefinedStruct* Struct, const FProperty* Property) const
{
	return FStructureEditorUtils::GetVariableFriendlyNameForProperty(Struct, Property);
}

FString UUserDefinedStructEditorData::GetTooltip() const
{
	return ToolTip;
}

#undef LOCTEXT_NAMESPACE
