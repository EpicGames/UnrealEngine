// Copyright Epic Games, Inc. All Rights Reserved.


#include "Utils.h"
#include "EditorViewportClient.h"
#include "SceneView.h"

#if PLATFORM_WINDOWS
// Needed for showing balloon messages
#include "Windows/AllowWindowsPlatformTypes.h"
		#include <ShellAPI.h>
#include "Windows/HideWindowsPlatformTypes.h"
#endif

DEFINE_LOG_CATEGORY(LogUtils);

IMPLEMENT_HIT_PROXY(HWidgetUtilProxy,HHitProxy);

float UnrealEd_WidgetSize = 0.15f; // Proportion of the viewport the widget should fill

/** Utility for calculating drag direction when you click on this widget. */
void HWidgetUtilProxy::CalcVectors(FSceneView* SceneView, const FViewportClick& Click, FVector& LocalManDir, FVector& WorldManDir, float& DragDirX, float& DragDirY)
{
	if(Axis == EAxisList::X)
	{
		WorldManDir = WidgetMatrix.GetScaledAxis( EAxis::X );
		LocalManDir = FVector(1,0,0);
	}
	else if(Axis == EAxisList::Y)
	{
		WorldManDir = WidgetMatrix.GetScaledAxis( EAxis::Y );
		LocalManDir = FVector(0,1,0);
	}
	else
	{
		WorldManDir = WidgetMatrix.GetScaledAxis( EAxis::Z );
		LocalManDir = FVector(0,0,1);
	}

	FVector WorldDragDir = WorldManDir;

	if(Mode == WMM_Rotate)
	{
		if( FMath::Abs(Click.GetDirection() | WorldManDir) > KINDA_SMALL_NUMBER ) // If click direction and circle plane are parallel.. can't resolve.
		{
			// First, find actual position we clicking on the circle in world space.
			const FVector ClickPosition = FMath::LinePlaneIntersection(	Click.GetOrigin(),
																	Click.GetOrigin() + Click.GetDirection(),
																	WidgetMatrix.GetOrigin(),
																	WorldManDir );

			// Then find Radial direction vector (from center to widget to clicked position).
			FVector RadialDir = ( ClickPosition - WidgetMatrix.GetOrigin() );
			RadialDir.Normalize();

			// Then tangent in plane is just the cross product. Should always be unit length again because RadialDir and WorlManDir should be orthogonal.
			WorldDragDir = RadialDir ^ WorldManDir;
		}
	}

	// Transform world-space drag dir to screen space.
	FVector ScreenDir = SceneView->ViewMatrices.GetWorldToView().TransformVector(WorldDragDir);
	ScreenDir.Z = 0.0f;

	if( ScreenDir.IsZero() )
	{
		DragDirX = 0.0f;
		DragDirY = 0.0f;
	}
	else
	{
		ScreenDir.Normalize();
		DragDirX = static_cast<float>(ScreenDir.X);
		DragDirY = static_cast<float>(ScreenDir.Y);
	}
}

/** 
 *	Utility function for drawing manipulation widget in a 3D viewport. 
 *	If we are hit-testing will create HWidgetUtilProxys for each axis, filling in InInfo1 and InInfo2 as passed in by user. 
 */
void FUnrealEdUtils::DrawWidget(const FSceneView* View,FPrimitiveDrawInterface* PDI, const FMatrix& WidgetMatrix, int32 InInfo1, int32 InInfo2, EAxisList::Type HighlightAxis, EWidgetMovementMode bInMode)
{
	DrawWidget( View, PDI, WidgetMatrix, InInfo1, InInfo2, HighlightAxis, bInMode, PDI->IsHitTesting() );
}

void FUnrealEdUtils::DrawWidget(const FSceneView* View,FPrimitiveDrawInterface* PDI, const FMatrix& WidgetMatrix, int32 InInfo1, int32 InInfo2, EAxisList::Type HighlightAxis, EWidgetMovementMode bInMode, bool bHitTesting)
{
	const FVector WidgetOrigin = WidgetMatrix.GetOrigin();

	// Calculate size to draw widget so it takes up the same screen space.
	const double ZoomFactor = FMath::Min<double>(View->ViewMatrices.GetViewToClip().M[0][0], View->ViewMatrices.GetViewToClip().M[1][1]);
	const double WidgetRadius = View->Project(WidgetOrigin).W * (UnrealEd_WidgetSize / ZoomFactor);

	// Choose its color. Highlight manipulated axis in yellow.
	FColor XColor(FColor::Red);
	FColor YColor(FColor::Green);
	FColor ZColor(FColor::Blue);

	if (HighlightAxis == EAxisList::X)
	{
		XColor = FColor::Yellow;
	}
	else if (HighlightAxis == EAxisList::Y)
	{
		YColor = FColor::Yellow;
	}
	else if (HighlightAxis == EAxisList::Z)
	{
		ZColor = FColor::Yellow;
	}

	const FVector XAxis = WidgetMatrix.GetScaledAxis( EAxis::X ); 
	const FVector YAxis = WidgetMatrix.GetScaledAxis( EAxis::Y ); 
	const FVector ZAxis = WidgetMatrix.GetScaledAxis( EAxis::Z );

	if(bInMode == WMM_Rotate)
	{
		if(bHitTesting) PDI->SetHitProxy( new HWidgetUtilProxy(InInfo1, InInfo2, EAxisList::X, WidgetMatrix, bInMode) );
		DrawCircle(PDI,WidgetOrigin, YAxis, ZAxis, XColor, WidgetRadius, 24, SDPG_Foreground);
		if(bHitTesting) PDI->SetHitProxy( NULL );

		if(bHitTesting) PDI->SetHitProxy( new HWidgetUtilProxy(InInfo1, InInfo2, EAxisList::Y, WidgetMatrix, bInMode) );
		DrawCircle(PDI,WidgetOrigin, XAxis, ZAxis, YColor, WidgetRadius, 24, SDPG_Foreground);
		if(bHitTesting) PDI->SetHitProxy( NULL );

		if(bHitTesting) PDI->SetHitProxy( new HWidgetUtilProxy(InInfo1, InInfo2, EAxisList::Z, WidgetMatrix, bInMode) );
		DrawCircle(PDI,WidgetOrigin, XAxis, YAxis, ZColor, WidgetRadius, 24, SDPG_Foreground);
		if(bHitTesting) PDI->SetHitProxy( NULL );
	}
	else
	{
		FMatrix WidgetTM;

		// Draw the widget arrows.
		if(bHitTesting) PDI->SetHitProxy( new HWidgetUtilProxy(InInfo1, InInfo2, EAxisList::X, WidgetMatrix, bInMode) );
		WidgetTM = FMatrix(XAxis, YAxis, ZAxis, WidgetOrigin);
		DrawDirectionalArrow(PDI,WidgetTM, XColor, WidgetRadius, 1.f, SDPG_Foreground);
		if(bHitTesting) PDI->SetHitProxy( NULL );

		if(bHitTesting) PDI->SetHitProxy( new HWidgetUtilProxy(InInfo1, InInfo2, EAxisList::Y, WidgetMatrix, bInMode) );
		WidgetTM = FMatrix(YAxis, ZAxis, XAxis, WidgetOrigin);
		DrawDirectionalArrow(PDI,WidgetTM, YColor, WidgetRadius, 1.f, SDPG_Foreground);
		if(bHitTesting) PDI->SetHitProxy( NULL );

		if(bHitTesting) PDI->SetHitProxy( new HWidgetUtilProxy(InInfo1, InInfo2, EAxisList::Z, WidgetMatrix, bInMode) );
		WidgetTM = FMatrix(ZAxis, XAxis, YAxis, WidgetOrigin);
		DrawDirectionalArrow(PDI,WidgetTM, ZColor, WidgetRadius, 1.f, SDPG_Foreground);
		if(bHitTesting) PDI->SetHitProxy( NULL );

		if(bInMode == WMM_Scale)
		{
			FVector AlongX = WidgetOrigin + (XAxis * WidgetRadius * 0.3f);
			FVector AlongY = WidgetOrigin + (YAxis * WidgetRadius * 0.3f);
			FVector AlongZ = WidgetOrigin + (ZAxis * WidgetRadius * 0.3f);

			PDI->DrawLine(AlongX, AlongY, FColor::White, SDPG_Foreground);
			PDI->DrawLine(AlongY, AlongZ, FColor::White, SDPG_Foreground);
			PDI->DrawLine(AlongZ, AlongX, FColor::White, SDPG_Foreground);
		}
	}
}
