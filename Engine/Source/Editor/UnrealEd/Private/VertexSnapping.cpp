// Copyright Epic Games, Inc. All Rights Reserved.

#include "VertexSnapping.h"
#include "GameFramework/Actor.h"
#include "Misc/App.h"
#include "Model.h"
#include "SceneView.h"
#include "Components/PrimitiveComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Components/SkinnedMeshComponent.h"
#include "Editor/GroupActor.h"
#include "Components/BrushComponent.h"
#include "Engine/Selection.h"
#include "EngineUtils.h"
#include "EditorViewportClient.h"
#include "LevelEditorViewport.h"
#include "EditorModeManager.h"
#include "StaticMeshResources.h"
#include "Engine/Polys.h"
#include "Rendering/SkeletalMeshRenderData.h"

namespace VertexSnappingConstants
{
	const float MaxSnappingDistance = 300.0f;
	const float MaxSquaredDistanceFromCamera = FMath::Square( 5000.0f );
	const float FadeTime = 0.15f;
	const FLinearColor VertexHelperColor = FColor(  17, 105, 238, 255 );
};

/**
 * Base class for an interator that iterates through the vertices on a component
 */
class FVertexIterator
{
public:
	virtual ~FVertexIterator(){};

	/** Advances to the next vertex */
	void operator++()
	{
		Advance();
	}
	
	/** @return True if there are more vertices on the component */
	explicit operator bool() const
	{
		return HasMoreVertices();
	}

	/**
	 * @return The position in world space of the current vertex
	 */
	virtual FVector Position() const = 0;

	/**
	 * @return The position in world space of the current vertex normal
	 */
	virtual FVector Normal() const = 0;

protected:
	/**
	 * @return True if there are more vertices on the component
	 */
	virtual bool HasMoreVertices() const = 0;

	/**
	 * Advances to the next vertex
	 */
	virtual void Advance() = 0;
};

/**
 * Iterates through the vertices of a static mesh
 */
class FStaticMeshVertexIterator : public FVertexIterator
{
public:
	FStaticMeshVertexIterator( UStaticMeshComponent* SMC )
		: ComponentToWorldIT( SMC->GetComponentTransform().ToInverseMatrixWithScale().GetTransposed() )
		, StaticMeshComponent( SMC )
		, PositionBuffer( SMC->GetStaticMesh()->GetRenderData()->GetCurrentFirstLOD(SMC->GetStaticMesh()->GetMinLODIdx())->VertexBuffers.PositionVertexBuffer )
		, VertexBuffer( SMC->GetStaticMesh()->GetRenderData()->GetCurrentFirstLOD(SMC->GetStaticMesh()->GetMinLODIdx())->VertexBuffers.StaticMeshVertexBuffer )
		, CurrentVertexIndex( 0 )
	{

	}

	/** FVertexIterator interface */
	virtual FVector Position() const override
	{
		return StaticMeshComponent->GetComponentTransform().TransformPosition( (FVector)PositionBuffer.VertexPosition( CurrentVertexIndex ) );
	}

	virtual FVector Normal() const override
	{
		return ComponentToWorldIT.TransformVector( FVector4(VertexBuffer.VertexTangentZ( CurrentVertexIndex )) );
	}

protected:
	virtual void Advance() override
	{
		++CurrentVertexIndex;
	}

	virtual bool HasMoreVertices() const override
	{
		return CurrentVertexIndex < PositionBuffer.GetNumVertices();
	}
private:
	/** Component To World Inverse Transpose matrix */
	FMatrix ComponentToWorldIT;
	/** Component containing the mesh that we are getting vertices from */
	const UStaticMeshComponent* StaticMeshComponent;
	/** The static meshes position vertex buffer */
	const FPositionVertexBuffer& PositionBuffer;
	/** The static meshes vertex buffer for normals */
	const FStaticMeshVertexBuffer& VertexBuffer;
	/** Current vertex index */
	uint32 CurrentVertexIndex;
};

/**
 * Vertex iterator for brush components
 */
class FBrushVertexIterator : public FVertexIterator
{
public:
	FBrushVertexIterator( UBrushComponent* InBrushComponent )
		: BrushComponent( InBrushComponent )
		, CurrentVertexIndex( 0 )
	{
		// Build up a list of vertices
		UModel* Model = BrushComponent->Brush;
		for( int32 PolyIndex = 0; PolyIndex < Model->Polys->Element.Num(); ++PolyIndex )
		{
			FPoly& Poly = Model->Polys->Element[PolyIndex];
			for( int32 VertexIndex = 0;VertexIndex < Poly.Vertices.Num();++VertexIndex )
			{
				Vertices.Add( (FVector)Poly.Vertices[VertexIndex] );
			}
		}
	}

	/** FVertexIterator interface */
	virtual FVector Position() const override
	{
		return BrushComponent->GetComponentTransform().TransformPosition( Vertices[CurrentVertexIndex] );
	}

	/** FVertexIterator interface */
	virtual FVector Normal() const override
	{
		return FVector::ZeroVector;
	}

protected:
	virtual void Advance() override
	{
		++CurrentVertexIndex;
	}

	virtual bool HasMoreVertices() const override
	{
		return Vertices.IsValidIndex( CurrentVertexIndex );
	}
private:
	/** The brush component getting vertices from */
	UBrushComponent* BrushComponent;
	/** All brush component vertices */
	TArray<FVector> Vertices;
	/** Current vertex index the iterator is on */
	uint32 CurrentVertexIndex;
	/** The number of vertices to iterate through */
	uint32 NumVertices;
};

/**
 * Iterates through the vertices on a component
 */
class FSkeletalMeshVertexIterator : public FVertexIterator
{
public:
	FSkeletalMeshVertexIterator( USkinnedMeshComponent* InSkinnedMeshComp )
		: ComponentToWorldIT( InSkinnedMeshComp->GetComponentTransform().ToInverseMatrixWithScale().GetTransposed() )
		, SkinnedMeshComponent( InSkinnedMeshComp )
		, LODData( InSkinnedMeshComp->GetSkeletalMeshRenderData()->LODRenderData[0] )
		, VertexIndex( 0 )
	{
	}
	
	/** FVertexIterator interface */
	virtual FVector Position() const override
	{
		const FVector VertPos = (FVector)LODData.StaticVertexBuffers.PositionVertexBuffer.VertexPosition(VertexIndex);
		return SkinnedMeshComponent->GetComponentTransform().TransformPosition(VertPos);
	}

	virtual FVector Normal() const override
	{
		FPackedNormal TangentZ = LODData.StaticVertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(VertexIndex);
		FVector VertNormal = TangentZ.ToFVector();
		return ComponentToWorldIT.TransformVector(VertNormal);
	}

protected:
	virtual void Advance() override
	{
		VertexIndex++;
	}

	virtual bool HasMoreVertices() const override
	{
		return VertexIndex < LODData.StaticVertexBuffers.PositionVertexBuffer.GetNumVertices();
	}

private:
	/** Component To World inverse transpose matrix */
	FMatrix ComponentToWorldIT;
	/** The component getting vertices from */
	USkinnedMeshComponent* SkinnedMeshComponent;
	/** Skeletal mesh render data */
	FSkeletalMeshLODRenderData& LODData;
	/** Current Soft vertex index the iterator is on */
	uint32 VertexIndex; 
};


/**
 * Makes a vertex iterator from the specified component
 */
static TSharedPtr<FVertexIterator> MakeVertexIterator( UPrimitiveComponent* Component )
{
	UStaticMeshComponent* SMC = Cast<UStaticMeshComponent>( Component );
	if( SMC && SMC->GetStaticMesh())
	{
		return MakeShareable( new FStaticMeshVertexIterator( SMC ) );
	}

	UBrushComponent* BrushComponent = Cast<UBrushComponent>( Component );
	if( BrushComponent && BrushComponent->Brush )
	{
		return MakeShareable( new FBrushVertexIterator( BrushComponent ) );
	}

	USkinnedMeshComponent* SkinnedComponent = Cast<USkinnedMeshComponent>( Component );
	if( SkinnedComponent && SkinnedComponent->GetSkinnedAsset() && SkinnedComponent->MeshObject )
	{
		return MakeShareable( new FSkeletalMeshVertexIterator( SkinnedComponent ) );
	}

	return NULL;
}


FVertexSnappingImpl::FVertexSnappingImpl()
{
}

void FVertexSnappingImpl::ClearSnappingHelpers( bool bClearImmediately )
{
	if( bClearImmediately )
	{
		ActorVertsToFade.Empty();
		ActorVertsToDraw.Reset();
	}
	else if( ActorVertsToDraw.IsValid() )
	{
		// Fade out previous verts
		ActorVertsToFade.Add( ActorVertsToDraw, FApp::GetCurrentTime() );
		ActorVertsToDraw.Reset();
	}
}

static void DrawSnapVertices( AActor* Actor, float PointSize, FPrimitiveDrawInterface* PDI )
{
	// Get the closest vertex on each component
	for (UActorComponent* Component : Actor->GetComponents())
	{
		TSharedPtr<FVertexIterator> VertexGetter = MakeVertexIterator( Cast<UPrimitiveComponent>(Component) );
		if( VertexGetter.IsValid() )
		{
			FVertexIterator& VertexGetterRef = *VertexGetter;
			for( ; VertexGetterRef; ++VertexGetterRef )
			{
				PDI->DrawPoint( VertexGetterRef.Position(), VertexSnappingConstants::VertexHelperColor, PointSize, SDPG_World );
			}
		}
		else
		{
			PDI->DrawPoint( Actor->GetActorLocation(), VertexSnappingConstants::VertexHelperColor, PointSize, SDPG_World );
		}
	}
}

void FVertexSnappingImpl::DrawSnappingHelpers(const FSceneView* View,FPrimitiveDrawInterface* PDI)
{
	if( ActorVertsToDraw.IsValid() )
	{
		float PointSize = View->IsPerspectiveProjection() ? 4.0f : 5.0f;
		DrawSnapVertices( ActorVertsToDraw.Get(), PointSize, PDI );
	}

	for( auto It = ActorVertsToFade.CreateIterator(); It; ++It )
	{
		TWeakObjectPtr<AActor> Actor = It.Key();
		double FadeStart = It.Value();

		if( Actor.IsValid() )
		{
			float PointSize = View->IsPerspectiveProjection() ? 4.0f : 5.0f;

			if( FApp::GetCurrentTime()-FadeStart <= VertexSnappingConstants::FadeTime )
			{
				PointSize = FMath::Lerp( PointSize, 0.0f, (FApp::GetCurrentTime()-FadeStart)/VertexSnappingConstants::FadeTime );

				DrawSnapVertices( Actor.Get(), PointSize, PDI );
			}
		}
	
		if( !Actor.IsValid() || FApp::GetCurrentTime()-FadeStart > VertexSnappingConstants::FadeTime )
		{
			It.RemoveCurrent();
		}
	}
}

struct FVertexSnappingArgs
{
	FVertexSnappingArgs( const FPlane& InActorPlane, const FVector& InCurrentLocation, TNotNull<FEditorViewportClient*> InViewportClient, const FSceneView* InView, const FVector2D& InMousePosition, EAxisList::Type InCurrentAxis, bool bInDrawVertexHelpers )
		: ActorPlane( InActorPlane )
		, CurrentLocation( InCurrentLocation )
		, ViewportClient( InViewportClient )
		, MousePosition( InMousePosition )
		, SceneView( InView )
		, CurrentAxis( InCurrentAxis )
		, bDrawVertexHelpers( bInDrawVertexHelpers )
	{}

	/** Plane that the actor is on.  For checking distances and culling vertices behind the plane */
	const FPlane& ActorPlane;
	/** Current pre-snap location that is being snapped */
	const FVector& CurrentLocation;
	/** Viewport client being used */
	TNotNull<FEditorViewportClient*> ViewportClient;
	/** 2D position of the mouse */
	const FVector2D MousePosition;
	/** Our current view */
	const FSceneView* SceneView;
	/** The current axis being dragged */
	EAxisList::Type CurrentAxis;
	/** Whether or not to draw vertex helpers */
	bool bDrawVertexHelpers;
};

bool FVertexSnappingImpl::GetClosestVertexOnComponent( const FSnapActor& SnapActor, UPrimitiveComponent* Component, const FVertexSnappingArgs& InArgs, FSnappingVertex& OutClosestLocation )
{
	// Current closest distance 
	double ClosestDistance = std::numeric_limits<double>::max();
	bool bHasAnyVerts = false;

	const FPlane& ActorPlane = InArgs.ActorPlane;
	EAxisList::Type CurrentAxis = InArgs.CurrentAxis;
	const FSceneView* View = InArgs.SceneView;
	const FVector& CurrentLocation = InArgs.CurrentLocation;
	const FVector2D& MousePosition = InArgs.MousePosition;

	// If no close vertex is found, do not snap
	OutClosestLocation.Position = CurrentLocation;

	TSharedPtr<FVertexIterator> VertexGetter = MakeVertexIterator( Component );
	if( VertexGetter.IsValid() )
	{
		FVertexIterator& VertexGetterRef = *VertexGetter;
		for( ; VertexGetterRef; ++VertexGetterRef )
		{
			bHasAnyVerts = true;

			FVector Position = VertexGetterRef.Position();
			FVector Normal = VertexGetterRef.Normal();

			if( CurrentAxis == EAxisList::Screen && View->IsPerspectiveProjection() && !SnapActor.AllowedSnappingBox.IsInside( Position ) )
			{
				// Vertex is outside the snapping bounding box
				continue;
			}

			// Ignore backface vertices when translating in screen space
			bool bIsBackface = false;
			bool bOutside = false;
			double Distance = 0.0;
			double DistanceFromCamera = 0.0;
			if( CurrentAxis != EAxisList::Screen )
			{
				// Compute the distance to the plane the actor is on
				Distance = ActorPlane.PlaneDot( Position );
			}
			else
			{
				// When moving in screen space compute the vertex closest to the mouse location for more accuracy

				FVector ViewToVertex = View->ViewMatrices.GetViewOrigin() - Position;

				// Ignore backface vertices 
				if( View->IsPerspectiveProjection() && Normal != FVector::ZeroVector && FVector::DotProduct( ViewToVertex, Normal ) < 0 )
				{
					bIsBackface = true;
				}

				if( !bIsBackface )
				{				
					FVector2D PixelPos;
					View->WorldToPixel( Position, PixelPos );

					// Ensure the vertex is inside the view
					bOutside = PixelPos.X < 0.0f || PixelPos.X > View->UnscaledViewRect.Width() || PixelPos.Y < 0.0f || PixelPos.Y > View->UnscaledViewRect.Height();

					if( !bOutside )
					{
						DistanceFromCamera = View->IsPerspectiveProjection() ? FVector::DistSquared( Position, View->ViewMatrices.GetViewOrigin() ) : 0.0;
						Distance = FVector::DistSquared( FVector( MousePosition, 0 ), FVector( PixelPos, 0 ) );
					}
				}
				
				if( !bIsBackface && !bOutside && DistanceFromCamera <= VertexSnappingConstants::MaxSquaredDistanceFromCamera )
				{
					// Draw a snapping helper for visible vertices
					FSnappingVertex NewVertex( Position, Normal );
				}
			}
			
			if( // Vertex cannot be facing away from the camera
				!bIsBackface 
				// Vertex cannot be outside the view
				&& !bOutside 
				// In screen space the distance of the vertex must not be too far from the camera.  In any other axis the vertex cannot be beind the actor
				&& ( ( CurrentAxis == EAxisList::Screen && DistanceFromCamera <= VertexSnappingConstants::MaxSquaredDistanceFromCamera ) || ( CurrentAxis != EAxisList::Screen && !( Distance < 0.0f ) ) )
				// The vertex must be closer than the current closest vertex
				&& Distance < ClosestDistance )
			{
				// Update the closest point
				ClosestDistance = Distance;
				OutClosestLocation.Position = Position;
				OutClosestLocation.Normal = Normal;
			}
		}
	}

	return bHasAnyVerts;
}


FSnappingVertex FVertexSnappingImpl::GetClosestVertex( const TArray<FSnapActor>& Actors, const FVertexSnappingArgs& InArgs )
{
	// The current closest distance
	double ClosestDistance = std::numeric_limits<double>::max();

	const FPlane& ActorPlane = InArgs.ActorPlane;
	EAxisList::Type CurrentAxis = InArgs.CurrentAxis;
	const FSceneView* View = InArgs.SceneView;
	const FVector& CurrentLocation = InArgs.CurrentLocation;
	const FVector2D& MousePosition = InArgs.MousePosition;

	FSnappingVertex ClosestLocation( CurrentLocation );

	AActor* ClosestActor = NULL;
	// Find the closest vertex on each actor and then from that list find the closest vertex
	for( int32 ActorIndex = 0; ActorIndex < Actors.Num(); ++ActorIndex )
	{
		const FSnapActor& SnapActor = Actors[ActorIndex];
		AActor* Actor = SnapActor.Actor;

		// Get the closest vertex on each component
		for (UActorComponent* Component : Actor->GetComponents())
		{
			UPrimitiveComponent* PrimComp = Cast<UPrimitiveComponent>(Component);
			if (PrimComp == nullptr)
			{
				continue;
			}

			FSnappingVertex ClosestLocationOnComponent( CurrentLocation );
			if( !GetClosestVertexOnComponent( SnapActor, PrimComp, InArgs, ClosestLocationOnComponent ) )
			{
				ClosestLocationOnComponent.Position = Actor->GetActorLocation();
				ClosestLocationOnComponent.Normal = FVector::ZeroVector;
			}

			double Distance = 0.0;
			if( CurrentAxis != EAxisList::Screen )
			{
				// Compute the distance from the point being snapped.  When not in screen space we snap to the plane created by the current closest vertex
				Distance = ActorPlane.PlaneDot( ClosestLocationOnComponent.Position );
			}
			else
			{
				// Favor the vertex closest to the mouse in screen space
				FVector2D ComponentLocPixel;
				if( View->WorldToPixel( ClosestLocationOnComponent.Position, ComponentLocPixel ) )
				{
					Distance = FVector::DistSquared( FVector( MousePosition, 0 ), FVector( ComponentLocPixel, 0 ) );
				}
			}

			if( 
				// A close vertex must have been found
				ClosestLocationOnComponent.Position != CurrentLocation 
				// we must have made some movement
				&& !FMath::IsNearlyZero(Distance) 
				// If not in screen space the vertex cannot be behind the point being snapped
				&& ( CurrentAxis == EAxisList::Screen || !( Distance < 0.0f ) )
				// The vertex must be closer than the current closest vertex
				&& Distance < ClosestDistance )
			{
				ClosestActor = Actor;
				ClosestDistance = Distance;
				ClosestLocation = ClosestLocationOnComponent;
			}
		}
	}

	if( InArgs.bDrawVertexHelpers )
	{
		if(ActorVertsToDraw.IsValid())
		{
			ActorVertsToFade.Add(ActorVertsToDraw, FApp::GetCurrentTime());
		}

		ActorVertsToFade.Remove(ClosestActor);
		ActorVertsToDraw = ClosestActor;
	}
	else
	{
		ActorVertsToDraw = nullptr;
		ActorVertsToFade.Empty();
	}


	return ClosestLocation;
}

void FVertexSnappingImpl::GetPossibleSnapActors( const FBox& AllowedBox, FIntPoint MouseLocation, TNotNull<FEditorViewportClient*> ViewportClient, const FSceneView* View, EAxisList::Type CurrentAxis, const TSet< TWeakObjectPtr<AActor> >& ActorsToIgnore, TArray<FSnapActor>& OutActorsInBox )
{
	if( CurrentAxis == EAxisList::Screen && !ViewportClient->IsOrtho() )
	{
		HHitProxy* HitProxy = ViewportClient->Viewport->GetHitProxy( MouseLocation.X, MouseLocation.Y );
		if( HitProxy && HitProxy->IsA(HActor::StaticGetType()) )
		{
			AActor* HitProxyActor = static_cast<HActor*>(HitProxy)->Actor ;
			if( HitProxyActor && !ActorsToIgnore.Contains(HitProxyActor)  )
			{
				OutActorsInBox.Add( FSnapActor( HitProxyActor, HitProxyActor->GetComponentsBoundingBox(true) ) );
			}
		}
	}
				
	if( OutActorsInBox.Num() == 0 )
	{
		GetActorsInsideBox( AllowedBox, ViewportClient->GetWorld(), OutActorsInBox, ActorsToIgnore, View );
	}
}
void FVertexSnappingImpl::GetActorsInsideBox( const FBox& Box, UWorld* World, TArray<FSnapActor>& OutActorsInBox, const TSet< TWeakObjectPtr<AActor> >& ActorsToIgnore, const FSceneView* View )
{
	for( FActorIterator It(World); It; ++It )
	{
		AActor* Actor = *It;
		// Ignore the builder brush, hidden actors and forcefully ignored actors (actors being moved)
		if( Actor != World->GetDefaultBrush() && It->IsHiddenEd() == false && !ActorsToIgnore.Contains( Actor ) )
		{
			const bool bNonColliding = true;
			FBox ActorBoundingBox = Actor->GetComponentsBoundingBox(true);

			// Actors must be within the bounding box and within the view frustum
			if( Box.Intersect( ActorBoundingBox ) && View->ViewFrustum.IntersectBox( ActorBoundingBox.GetCenter(), ActorBoundingBox.GetExtent() ) ) 
			{
				OutActorsInBox.Add( FSnapActor( Actor, Box ) );
			}
		}
	}
}

bool FVertexSnappingImpl::SnapLocationToNearestVertex(FVector& Location, const FVector2D& MouseLocation, TNotNull<FEditorViewportClient*> ViewportClient,
	FVector& OutVertexNormal, bool bDrawVertexHelpers, const TSet<TWeakObjectPtr<AActor>>& ActorsToIgnore)
{
	bool bSnapped = false;

	// Make a box around the actor which is the area we are allowed to snap in
	FBox AllowedSnappingBox = FBox( Location-VertexSnappingConstants::MaxSnappingDistance, Location+VertexSnappingConstants::MaxSnappingDistance );

	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		ViewportClient->Viewport, 
		ViewportClient->GetScene(),
		ViewportClient->EngineShowFlags )
		.SetRealtimeUpdate( ViewportClient->IsRealtime() ));

	FSceneView* View = ViewportClient->CalcSceneView( &ViewFamily );

	TArray<FSnapActor> ActorsInBox;
	GetPossibleSnapActors( AllowedSnappingBox, MouseLocation.IntPoint(), ViewportClient, View, EAxisList::Screen, ActorsToIgnore, ActorsInBox );

	FViewportCursorLocation Cursor(View, ViewportClient, static_cast<int32>(MouseLocation.X), static_cast<int32>(MouseLocation.Y));

	FPlane ActorPlane( Location, Cursor.GetDirection() );

	FVertexSnappingArgs Args
	( 
		ActorPlane, 
		Location,
		ViewportClient,
		View,
		Cursor.GetCursorPos(),
		EAxisList::Screen,
		bDrawVertexHelpers
	);
	
	// Snap to the nearest vertex
	FSnappingVertex ClosestVertex = GetClosestVertex( ActorsInBox, Args );

	Location = ClosestVertex.Position;
	OutVertexNormal = ClosestVertex.Normal;
	bSnapped = true;

	return bSnapped;
}

static void GetActorsToIgnore( AActor* Actor, TSet< TWeakObjectPtr<AActor> >& ActorsToIgnore )
{
	if( !ActorsToIgnore.Contains( Actor ) )
	{
		ActorsToIgnore.Add( Actor );

		// We cannot snap to any attached children or actors in the same group as moving this actor will also move the children as we are snapping to them, 
		// causing a cascading effect and unexpected results
		const TArray<USceneComponent*>& AttachedChildren = Actor->GetRootComponent()->GetAttachChildren();

		for (USceneComponent* Child : AttachedChildren)
		{
			if( Child && Child->GetOwner() )
			{
				ActorsToIgnore.Add( Child->GetOwner() );
			}
		}

		AGroupActor* ParentGroup = AGroupActor::GetRootForActor(Actor, true, true);
		if( ParentGroup ) 
		{
			TArray<AActor*> GroupActors;
			ParentGroup->GetGroupActors(GroupActors, true);
			for( int32 GroupActorIndex = 0; GroupActorIndex < GroupActors.Num(); ++GroupActorIndex )
			{
				ActorsToIgnore.Add( GroupActors[GroupActorIndex] );
			}
		}
	}
}

bool FVertexSnappingImpl::SnapDraggedActorsToNearestVertex( FVector& DragDelta, TNotNull<FEditorViewportClient*> ViewportClient )
{
	int32 MouseX = ViewportClient->Viewport->GetMouseX();
	int32 MouseY = ViewportClient->Viewport->GetMouseY();
	FVector2D MousePosition = FVector2D( MouseX, MouseY ) ;
	
	EAxisList::Type CurrentAxis = ViewportClient->GetCurrentWidgetAxis();

	bool bSnapped = false;
	if( !DragDelta.IsNearlyZero() )
	{
		FVector Direction = DragDelta.GetSafeNormal();

		FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
			ViewportClient->Viewport, 
			ViewportClient->GetScene(),
			ViewportClient->EngineShowFlags )
			.SetRealtimeUpdate( ViewportClient->IsRealtime() ));

		FSceneView* View = ViewportClient->CalcSceneView( &ViewFamily );

		FVector StartLocation = ViewportClient->GetModeTools()->PivotLocation;

		FVector DesiredUnsnappedLocation = StartLocation+DragDelta;
					
		// Plane facing in the direction of axis movement.  This is for clipping actors which are behind the desired location (they should not be considered for snap)
		FPlane ActorPlane( DesiredUnsnappedLocation, Direction );

		TSet< TWeakObjectPtr<AActor> > ActorsToIgnore;
	
		// Snap each selected actor
		for ( FSelectionIterator It( GEditor->GetSelectedActorIterator() ) ; It ; ++It )
		{
			// Build up a region that actors must be in for snapping
			FBoxSphereBounds SnappingAreaBox( FBox( DesiredUnsnappedLocation-VertexSnappingConstants::MaxSnappingDistance, DesiredUnsnappedLocation+VertexSnappingConstants::MaxSnappingDistance )  );

			AActor* Actor = CastChecked<AActor>( *It );

			if( Actor->GetRootComponent() != NULL )
			{
				// Get a bounding box around the actor
				const bool bNonColliding = true;
				FBoxSphereBounds ActorBounds = Actor->GetComponentsBoundingBox(bNonColliding);

				// The allowed snapping box is a box around the selected actor plus a region around the actor that other actors must be in for snapping
				FBox AllowedSnappingBox = ActorBounds.GetBox();

				// Extend the box to include the drag point
				AllowedSnappingBox += SnappingAreaBox.GetBox();

				GetActorsToIgnore( Actor, ActorsToIgnore );

				FVertexSnappingArgs Args
				( 
					ActorPlane, 
					DesiredUnsnappedLocation,
					ViewportClient,
					View,
					MousePosition,
					CurrentAxis,
					true
				);

				// Snap the drag delta
				SnapDragDelta( Args, StartLocation, AllowedSnappingBox, ActorsToIgnore, DragDelta );
			}
		}
	}

	return bSnapped;
}

void FVertexSnappingImpl::SnapDragDelta( FVertexSnappingArgs& InArgs, const FVector& StartLocation, const FBox& AllowedSnappingBox, TSet< TWeakObjectPtr<AActor> >& ActorsToIgnore, FVector& DragDelta )
{	
	const FSceneView* View = InArgs.SceneView;
	const FVector& DesiredUnsnappedLocation = InArgs.CurrentLocation;
	const FVector2D& MousePosition = InArgs.MousePosition;
	const EAxisList::Type CurrentAxis = InArgs.CurrentAxis;
	const FPlane ActorPlane = InArgs.ActorPlane;
	TNotNull<FEditorViewportClient*> ViewportClient = InArgs.ViewportClient;

	TArray<FSnapActor> PossibleSnapPointActors;
	GetPossibleSnapActors( AllowedSnappingBox, MousePosition.IntPoint(), ViewportClient, View, CurrentAxis, ActorsToIgnore, PossibleSnapPointActors );

	FVector Direction = FVector( ActorPlane.X, ActorPlane.Y, ActorPlane.Z );

	if( PossibleSnapPointActors.Num() > 0 )
	{
		// Get the closest vertex to the desired location (before snapping)
		FVector ClosestPoint = GetClosestVertex( PossibleSnapPointActors, InArgs ).Position;

		FVector PrevDragDelta = DragDelta;
		double Distance = 0;
		if( CurrentAxis != EAxisList::Screen )
		{
			// Compute a distance from the stat location to the snap point.  
			// When not using the screen space translation we snap to the plane along the movement axis that the nearest vertex is on and not the vertex itself
			FPlane RealPlane( StartLocation, Direction );
			Distance = RealPlane.PlaneDot( ClosestPoint );
						
			// Snap to the plane
			DragDelta = Distance*Direction;
		}
		else
		{
			// Snap to the nearest vertex
			DragDelta = ClosestPoint-StartLocation;
			Distance = DragDelta.Size();			
		}

		const FVector& PreSnapLocation = StartLocation;

		// Compute snapped location after computing the new drag delta
		FVector SnappedLocation = StartLocation+DragDelta;
				
		if( ViewportClient->IsPerspective() )
		{
			// Distance from start location to the location the actor would be in without snapping
			double DistFromPreSnapToDesiredUnsnapped = FVector::DistSquared( PreSnapLocation, DesiredUnsnappedLocation );

			// Distance from the new location of the actor without snapping to the location with snapping
			double DistFromDesiredUnsnappedToSnapped = FVector::DistSquared( DesiredUnsnappedLocation, SnappedLocation );

			// Only snap if the distance to the snapped location is less than the distance to the unsnapped location.  
			// This allows the user to control the speed of snapping based on how fast they move the mouse and also avoids jerkiness when the mouse is behind the snap location
			if( (CurrentAxis != EAxisList::Screen && DistFromDesiredUnsnappedToSnapped >= DistFromPreSnapToDesiredUnsnapped) || ClosestPoint == DesiredUnsnappedLocation )
			{
				DragDelta = FVector::ZeroVector;
			}
		}
		else
		{
			FVector2D PreSnapLocationPixel;
			View->WorldToPixel( PreSnapLocation, PreSnapLocationPixel );

			FVector2D SnappedLocationPixel;
			View->WorldToPixel( SnappedLocation, SnappedLocationPixel );

			FVector2D SLtoML = SnappedLocationPixel-MousePosition;
			FVector2D PStoML = MousePosition-PreSnapLocationPixel;

			// Only snap if the distance to the snapped location is less than the distance to the unsnapped location
			double Dist2 = PStoML.SizeSquared();
			double Dist1 = SLtoML.SizeSquared();
			if( Dist1 >= Dist2 || ClosestPoint == DesiredUnsnappedLocation )
			{
				DragDelta = FVector::ZeroVector;
			}
		}
	}
}

bool FVertexSnappingImpl::SnapDragLocationToNearestVertex( const FVector& BaseLocation, FVector& DragDelta, TNotNull<FEditorViewportClient*> ViewportClient )
{
	int32 MouseX = ViewportClient->Viewport->GetMouseX();
	int32 MouseY = ViewportClient->Viewport->GetMouseY();
	FVector2D MousePosition = FVector2D( MouseX, MouseY ) ;
	
	EAxisList::Type CurrentAxis = ViewportClient->GetCurrentWidgetAxis();

	bool bSnapped = false;
	if( !DragDelta.IsNearlyZero() )
	{
		FVector Direction = DragDelta.GetSafeNormal();

		FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
			ViewportClient->Viewport, 
			ViewportClient->GetScene(),
			ViewportClient->EngineShowFlags )
			.SetRealtimeUpdate( ViewportClient->IsRealtime() ));
		FSceneView* View = ViewportClient->CalcSceneView( &ViewFamily );

		FVector DesiredUnsnappedLocation = BaseLocation+DragDelta;
		
		FBoxSphereBounds SnappingAreaBox( FBox( DesiredUnsnappedLocation-VertexSnappingConstants::MaxSnappingDistance, DesiredUnsnappedLocation+VertexSnappingConstants::MaxSnappingDistance )  );

		FBox AllowedSnappingBox = SnappingAreaBox.GetBox();
		
		AllowedSnappingBox += DragDelta;
			
		FPlane ActorPlane( DesiredUnsnappedLocation, Direction );

		TSet< TWeakObjectPtr<AActor> > NoActorsToIgnore;

		FVertexSnappingArgs Args
		( 
			ActorPlane, 
			DesiredUnsnappedLocation,
			ViewportClient,
			View,
			MousePosition,
			CurrentAxis,
			true
		);

		SnapDragDelta( Args, BaseLocation, AllowedSnappingBox, NoActorsToIgnore, DragDelta );
	}

	return bSnapped;
}
