// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"

class AActor;
class FEditorViewportClient;
class FLevelEditorViewportClient;
class FPrimitiveDrawInterface;
class FSceneView;
class UPrimitiveComponent;
struct FVertexSnappingArgs;

struct FSnappingVertex
{
	FSnappingVertex( const FVector& InPosition, const FVector& InNormal )
		: Position( InPosition )
		, Normal( InNormal )
		, FadeOutTime( 0 )
		, FadeInTime( 0 )
		, Distance( 0 )
	{}

	FSnappingVertex( const FVector& InPosition )
		: Position( InPosition )
		, Normal( FVector::ZeroVector )
		, FadeOutTime( 0 )
		, FadeInTime( 0 )
		, Distance( 0 )
	{}

	FSnappingVertex()
		: Position()
		, Normal()
		, FadeOutTime( 0 )
		, FadeInTime( 0 )
		, Distance( 0 )
	{}

	friend uint32 GetTypeHash( const FSnappingVertex& Vertex )
	{
		return GetTypeHash(Vertex.Position);
	}

	bool operator==( const FSnappingVertex& Other ) const
	{
		return Position == Other.Position && Normal == Other.Normal;
	}

	bool operator<( const FSnappingVertex& Other ) const
	{
		return Distance < Other.Distance;
	}

	FVector Position;
	FVector Normal;
	double FadeOutTime;
	double FadeInTime;
	float Distance;
};

template<> struct TIsPODType<FSnappingVertex> { enum { Value = true }; };

struct FSnapActor
{
	FSnapActor( AActor* InActor, const FBox& InAllowedSnappingBox )
		: Actor( InActor )
		, AllowedSnappingBox( InAllowedSnappingBox )
	{}

	AActor* Actor;
	FBox	AllowedSnappingBox;
};

class FVertexSnappingImpl
{
public:
	FVertexSnappingImpl();

	/**
	 * Draws snapping helpers 
	 *
	 * @param View	The current view of the scene
	 * @param PDI	Drawing interface
	 */
	void DrawSnappingHelpers(const FSceneView* View,FPrimitiveDrawInterface* PDI);

	/**
	 * Clears all vertices being drawn to help a user snap
	 *
	 * @param bClearImmediatley	true to clear helpers without fading them out
	 */
	void ClearSnappingHelpers( bool bClearImmediately );

	/**
	 * Snaps actors to the nearest vertex on another actor
	 *
	 * @param DragDelta			The current world space drag amount that will be modified to account for snapping to a vertex
	 * @param ViewportClient	The viewport client the user is dragging in
	 * @return true if anything was snapped
	 */
	bool SnapDraggedActorsToNearestVertex( FVector& DragDelta, TNotNull<FEditorViewportClient*> ViewportClient );

	/**
	 * Snaps a delta drag movement to the nearest vertex
	 *
	 * @param BaseLocation		Location that should be snapped before any drag is applied
	 * @param DragDelta			Delta drag movement that should be snapped.  This value will be updated such that BaseLocation+DragDelta is the nearest snapped verted
	 * @param ViewportClient	The viewport client being dragged in.
	 * @return true if anything was snapped
	 */
	bool SnapDragLocationToNearestVertex( const FVector& BaseLocation, FVector& DragDelta, TNotNull<FEditorViewportClient*> ViewportClient );

	/**
	 * Snaps a location to the nearest vertex
	 *
	 * @param Location			The location to snap
	 * @param MouseLocation		The current 2d mouse location.  Vertices closer to the mouse are favored
	 * @param ViewportClient	The viewport client being used
	 * @param OutVertexNormal	The normal at the closest vertex
	 * @return true if anything was snapped
	 */
	bool SnapLocationToNearestVertex( FVector& Location, const FVector2D& MouseLocation, TNotNull<FEditorViewportClient*> ViewportClient, FVector& OutVertexNormal, bool bDrawVertexHelpers, const TSet<TWeakObjectPtr<AActor>>& ActorsToIgnore );
private:
	/**
	 * Gets the closest vertex on a primitive component
	 *
	 * @param SnapActor				The actor that owns the component
	 * @param Component				The component to get vertices from
	 * @param InArgs				Parameters for vertex snapping
	 * @param OutClosestLocation	The closest vertex location on the component
	 * @return true if any vertices were found, falsse otherwise
	 */
	bool GetClosestVertexOnComponent( const FSnapActor& SnapActor, UPrimitiveComponent* Component, const struct FVertexSnappingArgs& InArgs, FSnappingVertex& OutClosestLocation );

	/**
	 * Gets the closest vertex from the list of actors
	 *
	 * @param Actors	The actors to check
	 * @param InArgs	Parameters for vertex snapping
	 * @return The closest vertex
	 */
	FSnappingVertex GetClosestVertex( const TArray<FSnapActor>& Actors, const struct FVertexSnappingArgs& InArgs );

	/**
	 * Get all possible actors to vertex snap to within a region
	 *
	 * @param AllowedBox		The allowed bounding box that actors can be within
	 * @param MouseLocation		The current cursor location. Actors over the current cursor location are favored
	 * @param ViewportClient	The viewport client snapping is taking place in
	 * @param View				The current view of the scene
	 * @param CurrentAxis		The current axis being used
	 * @param ActorsToIgnore	Set of actors to ignore when checking for vertices
	 * @param OutSnapActors		The populated list of possible snap actors
	 */
	void GetPossibleSnapActors( const FBox& AllowedBox, FIntPoint MouseLocation, TNotNull<FEditorViewportClient*> ViewportClient, const FSceneView* View, EAxisList::Type CurrentAxis, const TSet< TWeakObjectPtr<AActor> >& ActorsToIgnore, TArray<FSnapActor>& OutSnapActors );

	/**
	 * Gets all actors inside a bounding box
	 * 
	 * @param Box				The bounding box to check for actors
	 * @param World				The world that actors must be in
	 * @param OutActorsInBox	All the actors inside the box
	 * @param ActorsToIgnore	Set of actors to ignore even if inside the box
	 * @param View				The current view of the scene (used for frustum culling )
	 */
	void GetActorsInsideBox( const FBox& Box, UWorld* World, TArray<FSnapActor>& OutActorsInBox, const TSet< TWeakObjectPtr<AActor> >& ActorsToIgnore, const FSceneView* View );

	/**
	 * Snaps and updates a delta drag movement to a vertex
	 *
	 * @param InArgs				Parameters for vertex snapping
	 * @param StartLocation			The start location.  The area around this location is considered for snapping
	 * @param AllowedSnappingBox	Vertices must be inside this box to be considered for snapping
	 * @param ActorsToIgnroe		Actors to ignore when snapping
	 * @param DragDelta				Delta drag amount to snap.  This value will be updated with the snapped drag
	 */
	void SnapDragDelta( FVertexSnappingArgs& InArgs, const FVector& StartLocation, const FBox& AllowedSnappingBox, TSet< TWeakObjectPtr<AActor> >& ActorsToIgnore, FVector& DragDelta );

private:
	/** Actor whose verts are being snapped to.  Draw the verts on this actor to help the user pick which vertices to snap to*/
	TWeakObjectPtr<AActor> ActorVertsToDraw;
	/** Map of actors with vertices that have previously been drawn to their initial fade start time.   Fade these vertices over time.  */
	TMap<TWeakObjectPtr<AActor>, double> ActorVertsToFade;
};
