// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportClientNavigationHelper.h"

FViewportClientNavigationHelper::FViewportClientNavigationHelper()
	: bIsMouseLooking(false)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bCameraFollowsGizmo = false;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	ResetTransformDelta();
	ResetImpulseData();
}

bool FViewportClientNavigationHelper::HasTransformDelta() const
{
	return !RotationDelta.IsNearlyZero() || !LocationDelta.IsNearlyZero() || !OrbitDelta.IsNearlyZero();
}

void FViewportClientNavigationHelper::ConsumeTransformDelta(FVector& OutLocationDelta, FRotator& OutRotationDelta, FVector& OutOrbitDelta)
{
	OutLocationDelta = LocationDelta;
	OutRotationDelta = RotationDelta;
	OutOrbitDelta = OrbitDelta;

	// Reset after use
	ResetTransformDelta();
	OrbitDelta = FVector::ZeroVector;
}

void FViewportClientNavigationHelper::ResetTransformDelta()
{
	LocationDelta = FVector::ZeroVector;
	RotationDelta = FRotator::ZeroRotator;
	OrbitDelta = FVector::ZeroVector;
}

void FViewportClientNavigationHelper::ConsumeImpulseData(FCameraControllerUserImpulseData& OutImpulseData)
{
	// Translation
	OutImpulseData.MoveForwardBackwardImpulse += ImpulseDataDelta.MoveForwardBackwardImpulse;
	OutImpulseData.MoveRightLeftImpulse += ImpulseDataDelta.MoveRightLeftImpulse;
	OutImpulseData.MoveWorldUpDownImpulse += ImpulseDataDelta.MoveWorldUpDownImpulse;
	OutImpulseData.MoveLocalUpDownImpulse += ImpulseDataDelta.MoveLocalUpDownImpulse;

	// Rotation
	OutImpulseData.RotateYawImpulse += ImpulseDataDelta.RotateYawImpulse;
	OutImpulseData.RotatePitchImpulse += ImpulseDataDelta.RotatePitchImpulse;
	OutImpulseData.RotateRollImpulse += ImpulseDataDelta.RotateRollImpulse;
	OutImpulseData.RotateYawVelocityModifier += ImpulseDataDelta.RotateYawVelocityModifier;
	OutImpulseData.RotatePitchVelocityModifier += ImpulseDataDelta.RotatePitchVelocityModifier;
	OutImpulseData.RotateRollVelocityModifier += ImpulseDataDelta.RotateRollVelocityModifier;

	// FOV
	OutImpulseData.ZoomOutInImpulse += ImpulseDataDelta.ZoomOutInImpulse;

	// Reset after use
	ResetImpulseData();
}

void FViewportClientNavigationHelper::ConsumeOrbitDelta(FVector& OutOrbitDelta)
{
	OutOrbitDelta = OrbitDelta;
	OrbitDelta = FVector::ZeroVector;
}

void FViewportClientNavigationHelper::ResetImpulseData()
{
	ImpulseDataDelta.MoveForwardBackwardImpulse = 0.0f;
	ImpulseDataDelta.MoveRightLeftImpulse = 0.0f;
	ImpulseDataDelta.MoveWorldUpDownImpulse = 0.0f;
	ImpulseDataDelta.MoveLocalUpDownImpulse = 0.0f;
	ImpulseDataDelta.RotateYawImpulse = 0.0f;
	ImpulseDataDelta.RotatePitchImpulse = 0.0f;
	ImpulseDataDelta.RotateRollImpulse = 0.0f;
	ImpulseDataDelta.RotateYawVelocityModifier = 0.0f;
	ImpulseDataDelta.RotatePitchVelocityModifier = 0.0f;
	ImpulseDataDelta.RotateRollVelocityModifier = 0.0f;
	ImpulseDataDelta.ZoomOutInImpulse = 0.0f;
}
