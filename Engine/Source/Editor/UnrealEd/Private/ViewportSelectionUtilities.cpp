// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportSelectionUtilities.h"
#include "ActorEditorUtils.h"
#include "AssetSelection.h"
#include "Components/PointLightComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Dialogs/Dialogs.h"
#include "Editor.h"
#include "EditorModeManager.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Engine/Brush.h"
#include "Engine/PointLight.h"
#include "Engine/Selection.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/TargetPoint.h"
#include "EngineUtils.h"
#include "GameFramework/Actor.h"
#include "ILevelEditor.h"
#include "InputCoreTypes.h"
#include "LevelEditorViewport.h"
#include "LightMap.h"
#include "Logging/MessageLog.h"
#include "Materials/MaterialInterface.h"
#include "Model.h"
#include "SEditorViewport.h"
#include "ScopedTransaction.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "SnappingUtils.h"
#include "StaticLightingSystem/StaticLightingPrivate.h"
#include "UObject/Class.h"

DEFINE_LOG_CATEGORY_STATIC(LogViewportSelectionUtilities, Log, All);
#define LOCTEXT_NAMESPACE "ViewportSelectionUtilities"

namespace UE::Editor::ViewportSelectionUtilities
{
static const FEditorModeTools* GetModeTools(const FEditorViewportClient* InEditorViewportClient)
{
	if (InEditorViewportClient)
	{
		return InEditorViewportClient->GetModeTools();
	}

	return nullptr;
}

static UTypedElementSelectionSet* GetElementSelectionSet(const FEditorViewportClient* InEditorViewportClient)
{
	if (const FEditorModeTools* const ModeTools = GetModeTools(InEditorViewportClient))
	{
		return ModeTools->GetEditorSelectionSet();
	}

	return nullptr;
}

/**
 * Creates an actor of the specified type, trying first to find an actor factory,
 * falling back to "ACTOR ADD" exec and SpawnActor if no factory is found.
 * Does nothing if ActorClass is NULL.
 */
static AActor* PrivateAddActor(UClass* ActorClass)
{
	return FActorFactoryAssetProxy::AddActorForAsset(ActorClass);
}

/**
 * This function picks a color from under the mouse in the viewport and adds a light with that color.
 * This is to make it easy for LDs to add lights that fake radiosity.
 * @param Viewport	Viewport to pick color from.
 * @param Click		A class that has information about where and how the user clicked on the viewport.
 */
void PickColorAndAddLight(FViewport* Viewport, const FViewportClick& Click)
{
	// Read pixels from viewport.
	TArray<FColor> OutputBuffer;

	// We need to redraw the viewport before reading pixels otherwise we may be reading back from an old buffer.
	Viewport->Draw();
	Viewport->ReadPixels(OutputBuffer);

	// Sample the color we want.
	const int32 ClickX = Click.GetClickPos().X;
	const int32 ClickY = Click.GetClickPos().Y;
	const int32 PixelIdx = ClickX + ClickY * (int32)Viewport->GetSizeXY().X;

	if (PixelIdx < OutputBuffer.Num())
	{
		const FColor PixelColor = OutputBuffer[PixelIdx];

		AActor* NewActor = PrivateAddActor(APointLight::StaticClass());

		APointLight* Light = CastChecked<APointLight>(NewActor);
		Light->SetMobility(EComponentMobility::Stationary);
		UPointLightComponent* PointLightComponent = Cast<UPointLightComponent>(Light->GetLightComponent());
		PointLightComponent->LightColor = PixelColor;
	}
}

bool ClickElement(FEditorViewportClient* InViewportClient, const FTypedElementHandle& InHitElement, const FViewportClick& Click)
{
	// Pivot snapping
	if (Click.GetKey() == EKeys::MiddleMouseButton && Click.IsAltDown())
	{
		// GEditor->SetPivot(GEditor->ClickLocation, true, false, true); // TODO: This last param is only for actor
		// pivots return true;
		return false; // Let actor and component clicks handle pivots for now
	}

	UTypedElementSelectionSet* ElementSelectionSet = GetElementSelectionSet(InViewportClient);
	if (!ElementSelectionSet)
	{
		return false;
	}

	bool bHandledClick = false;

	const bool bIsLeftClickSelection = Click.GetKey() == EKeys::LeftMouseButton
									&& !(InViewportClient->Viewport->KeyState(EKeys::T)
										 || InViewportClient->Viewport->KeyState(EKeys::L)
										 || InViewportClient->Viewport->KeyState(EKeys::S)
										 || InViewportClient->Viewport->KeyState(EKeys::A));
	const bool bIsRightClickSelection = Click.GetKey() == EKeys::RightMouseButton && !Click.IsControlDown()
									 && !InViewportClient->Viewport->KeyState(EKeys::LeftMouseButton);

	if (bIsLeftClickSelection || bIsRightClickSelection)
	{
		const ETypedElementSelectionMethod SelectionMethod = Click.GetEvent() == IE_DoubleClick
															   ? ETypedElementSelectionMethod::Secondary
															   : ETypedElementSelectionMethod::Primary;
		if (const FTypedElementHandle ResolvedElement =
				ElementSelectionSet->GetSelectionElement(InHitElement, SelectionMethod))
		{
			

			const FTypedElementSelectionOptions SelectionOptions =
				FTypedElementSelectionOptions().SetAllowHidden(true).SetWarnIfLocked(true)
					.SetNameForTEDSIntegration(ElementSelectionSet->GetNameForTedsIntegration());

			bool bNeedViewportRefresh = false;

			if (ElementSelectionSet->CanSelectElement(ResolvedElement, SelectionOptions))
			{
				bHandledClick = true;
				const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "ClickingOnElements", "Clicking on Elements"));

				const bool bAllowSelectionModifiers =
					bIsLeftClickSelection && ElementSelectionSet->AllowSelectionModifiers(ResolvedElement);
				if (Click.IsControlDown() && bAllowSelectionModifiers)
				{
					if (ElementSelectionSet->IsElementSelected(
							ResolvedElement,
							FTypedElementIsSelectedOptions().SetAllowIndirect(true)
								.SetNameForTEDSIntegration(ElementSelectionSet->GetNameForTedsIntegration())
						))
					{
						ElementSelectionSet->DeselectElement(ResolvedElement, SelectionOptions);
					}
					else
					{
						ElementSelectionSet->SelectElement(ResolvedElement, SelectionOptions);
					}
				}
				else if (Click.IsShiftDown() && bAllowSelectionModifiers)
				{
					ElementSelectionSet->SelectElement(ResolvedElement, SelectionOptions);
				}
				else
				{
					// Skip the clear if we're doing a RMB select and this actor is already selected, as we want to summon the menu for the current selection
					if (bIsLeftClickSelection
						|| !ElementSelectionSet->IsElementSelected(
							ResolvedElement,
							FTypedElementIsSelectedOptions().SetAllowIndirect(true)
								.SetNameForTEDSIntegration(ElementSelectionSet->GetNameForTedsIntegration())
						))
					{
						bNeedViewportRefresh =
							bIsRightClickSelection; // Refresh the viewport so the user will see what they just clicked while the menu is open
						GEditor->DeselectAllSurfaces();
						ElementSelectionSet->ClearSelection(SelectionOptions);
					}
					ElementSelectionSet->SelectElement(ResolvedElement, SelectionOptions);
				}

				// Notify any pending selection change now, as this avoids the visual pivot location "lagging" behind the actual selection,
				// and also ensures that the pivot is at the correct location prior to opening any context menus (which block the update)
				ElementSelectionSet->NotifyPendingChanges();
			}

			if (bNeedViewportRefresh)
			{
				// Redraw the viewport so the user can see which object was clicked on
				InViewportClient->Viewport->Draw();
				FlushRenderingCommands();
			}

			if (bIsRightClickSelection)
			{
				InViewportClient->SummonContextMenu(ResolvedElement);
			}
		}
	}

	return bHandledClick;
}

bool ClickActor(FEditorViewportClient* InViewportClient, AActor* InActor, const FViewportClick& Click, bool bAllowSelectionChange)
{
	// Pivot snapping
	if (Click.GetKey() == EKeys::MiddleMouseButton && Click.IsAltDown())
	{
		GEditor->SetPivot(GEditor->ClickLocation, true, false, true);

		return true;
	}
	// Handle selection.
	else if (Click.GetKey() == EKeys::RightMouseButton && !Click.IsControlDown()
			 && !InViewportClient->Viewport->KeyState(EKeys::LeftMouseButton))
	{
		bool bNeedViewportRefresh = false;
		if (InActor)
		{
			const FScopedTransaction Transaction(
				NSLOCTEXT("UnrealEd", "ClickingOnActorsContextMenu", "Clicking on Actors (context menu)")
			);
			UE_LOGF(LogViewportSelectionUtilities, Log,  "Clicking on Actor (context menu): %ls (%ls)", *InActor->GetClass()->GetName(), *InActor->GetActorLabel() );

			GEditor->GetSelectedActors()->Modify();

			if (bAllowSelectionChange && GEditor->CanSelectActor(InActor, true, true))
			{
				// If the actor the user clicked on was already selected, then we won't bother clearing the selection
				if (!InActor->IsActorOrSelectionParentSelected())
				{
					GEditor->SelectNone(false, true);
					bNeedViewportRefresh = true;
				}

				// Select the actor the user clicked on
				GEditor->SelectActor(InActor, true, true);
			}
		}

		if (bNeedViewportRefresh)
		{
			// Redraw the viewport so the user can see which object was right clicked on
			InViewportClient->Viewport->Draw();
			FlushRenderingCommands();
		}

		InViewportClient->SummonContextMenu(
			InActor ? UEngineElementsLibrary::AcquireEditorActorElementHandle(InActor) : FTypedElementHandle()
		);
		return true;
	}
	else if (Click.GetEvent() == IE_DoubleClick && Click.GetKey() == EKeys::LeftMouseButton && !Click.IsControlDown()
			 && !Click.IsShiftDown())
	{
		if (InActor)
		{
			const FScopedTransaction Transaction(
				NSLOCTEXT("UnrealEd", "ClickingOnActorsDouble-Click", "Clicking on Actors (double-click)")
			);
			UE_LOGF(LogViewportSelectionUtilities, Log,  "Clicking on Actor (double click): %ls (%ls)", *InActor->GetClass()->GetName(), *InActor->GetActorLabel());

			GEditor->GetSelectedActors()->Modify();

			if (bAllowSelectionChange && GEditor->CanSelectActor(InActor, true, true))
			{
				// Clear the selection
				GEditor->SelectNone(false, true);

				// Select the actor the user clicked on
				GEditor->SelectActor(InActor, true, true);
			}
		}

		return true;
	}
	else if (Click.GetKey() != EKeys::RightMouseButton)
	{
		if (Click.GetKey() == EKeys::LeftMouseButton && InViewportClient->Viewport->KeyState(EKeys::T) && InActor)
		{
			TArray<UActorComponent*> Components;
			InActor->GetComponents(Components);
			SetDebugLightmapSample(&Components, NULL, 0, GEditor->ClickLocation);
		}
		else if (Click.GetKey() == EKeys::LeftMouseButton && InViewportClient->Viewport->KeyState(EKeys::L))
		{
			// If shift is down, we pick a color from under the mouse in the viewport and create a light with that color.
			if (Click.IsControlDown())
			{
				PickColorAndAddLight(InViewportClient->Viewport, Click);
			}
			else
			{
				// Create a point light (they default to stationary)
				PrivateAddActor(APointLight::StaticClass());
			}

			return true;
		}
		else if (Click.GetKey() == EKeys::LeftMouseButton && InViewportClient->Viewport->KeyState(EKeys::S))
		{
			// Create a static mesh.
			PrivateAddActor(AStaticMeshActor::StaticClass());

			return true;
		}
		else if (Click.GetKey() == EKeys::LeftMouseButton && InViewportClient->Viewport->KeyState(EKeys::A))
		{
			// Create an actor of the selected class.
			UClass* SelectedClass = GEditor->GetSelectedObjects()->GetTop<UClass>();
			if (SelectedClass)
			{
				PrivateAddActor(SelectedClass);
			}

			return true;
		}
		else if (InActor)
		{
			if (bAllowSelectionChange && GEditor->CanSelectActor(InActor, true, true, true))
			{
				const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "ClickingOnActors", "Clicking on Actors"));
				GEditor->GetSelectedActors()->Modify();

				// Ctrl- or shift- clicking an actor is the same as regular clicking when components are selected
				const bool bComponentSelected = GEditor->GetSelectedComponentCount() > 0;

				if (Click.IsControlDown() && !bComponentSelected)
				{
					const bool bSelect = !InActor->IsSelected();
					if (bSelect)
					{
						UE_LOGF(LogViewportSelectionUtilities, Log,  "Clicking on Actor (CTRL LMB): %ls (%ls)", *InActor->GetClass()->GetName(), *InActor->GetActorLabel());
					}
					GEditor->SelectActor(InActor, bSelect, true, true);
				}
				else if (Click.IsShiftDown() && !bComponentSelected)
				{
					if (!InActor->IsSelected())
					{
						const bool bSelect = true;
						GEditor->SelectActor(InActor, bSelect, true, true);
					}
				}
				else
				{
					// check to see how many actors need deselecting first - and warn as appropriate
					int32 NumSelectedActors = GEditor->GetSelectedActors()->Num();
					if (NumSelectedActors >= EditorActorSelectionDefs::MaxActorsToSelectBeforeWarning)
					{
						const FText ConfirmText = FText::Format(
							NSLOCTEXT(
								"UnrealEd", "Warning_ManyActorsToSelectOne", "There are {0} selected actors. Selecting this actor will deselect them all. Are you sure?"
							),
							FText::AsNumber(NumSelectedActors)
						);

						FSuppressableWarningDialog::FSetupInfo Info(
							ConfirmText, NSLOCTEXT("UnrealEd", "Warning_ManyActors", "Warning: Many Actors"), "Warning_ManyActors"
						);
						Info.ConfirmText = NSLOCTEXT("ModalDialogs", "ManyActorsToSelectOneConfirm", "Continue Selection");
						Info.CancelText =
							NSLOCTEXT("ModalDialogs", "ManyActorsToSelectOneCancel", "Keep Current Selection");

						FSuppressableWarningDialog ManyActorsWarning(Info);
						if (ManyActorsWarning.ShowModal() == FSuppressableWarningDialog::Cancel)
						{
							return false;
						}
					}

					GEditor->SelectNone(false, true, false);
					UE_LOGF(LogViewportSelectionUtilities, Log,  "Clicking on Actor (LMB): %ls (%ls)", *InActor->GetClass()->GetName(), *InActor->GetActorLabel());
					GEditor->SelectActor(InActor, true, true, true);
				}
			}

			return false;
		}
	}

	return false;
}

bool ClickComponent(FEditorViewportClient* ViewportClient, HActor* ActorHitProxy, const FViewportClick& Click)
{
	//@todo hotkeys for component placement?

	bool bComponentClicked = false;

	USceneComponent* Component = nullptr;

	if (ActorHitProxy->Actor->IsChildActor())
	{
		AActor* TestActor = ActorHitProxy->Actor;
		do
		{
			Component = TestActor->GetParentComponent();
			TestActor = TestActor->GetParentActor();
		}
		while (TestActor->IsChildActor());
	}
	else
	{
		UPrimitiveComponent* TestComponent = ConstCast(ActorHitProxy->PrimComponent);
		if (ActorHitProxy->Actor->GetComponents().Contains(TestComponent))
		{
			Component = TestComponent;
		}
	}

	// If the component selected is a visualization component, we want to select the non-visualization component it's attached to
	while (Component != nullptr && Component->IsVisualizationComponent())
	{
		Component = Component->GetAttachParent();
	}

	if (Component == nullptr)
	{
		// It's possible to have a null component here if the primitive component contained in the hit proxy is not part
		// of the actor contained in the hit proxy. In that case, component click is not possible
		//  (but actor click can still be used as a fallback) :
		return false;
	}

	// Pivot snapping
	if (Click.GetKey() == EKeys::MiddleMouseButton && Click.IsAltDown())
	{
		GEditor->SetPivot(GEditor->ClickLocation, true, false);

		return true;
	}
	// Selection + context menu
	else if (Click.GetKey() == EKeys::RightMouseButton && !Click.IsControlDown()
			 && !ViewportClient->Viewport->KeyState(EKeys::LeftMouseButton))
	{
		const FScopedTransaction Transaction(
			NSLOCTEXT("UnrealEd", "ClickingOnComponentContextMenu", "Clicking on Component (context menu)")
		);
		UE_LOGF(LogViewportSelectionUtilities, Log, "Clicking on Component (context menu): %ls (%ls)", *Component->GetClass()->GetName(), *Component->GetName());

		const auto EditorComponentSelection = GEditor->GetSelectedComponents();
		EditorComponentSelection->Modify();

		// If the component the user clicked on was already selected, then we won't bother clearing the selection
		bool bNeedViewportRefresh = false;
		if (!EditorComponentSelection->IsSelected(Component))
		{
			EditorComponentSelection->DeselectAll();
			bNeedViewportRefresh = true;
		}

		GEditor->SelectComponent(Component, true, true);

		if (bNeedViewportRefresh)
		{
			// Redraw the viewport so the user can see which object was right clicked on
			ViewportClient->Viewport->Draw();
			FlushRenderingCommands();
		}

		ViewportClient->SummonContextMenu();
		bComponentClicked = true;
	}
	// Selection only
	else if (Click.GetKey() == EKeys::LeftMouseButton)
	{
		const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "ClickingOnComponents", "Clicking on Components"));
		GEditor->GetSelectedComponents()->Modify();

		if (Click.IsControlDown())
		{
			const bool bSelect = !Component->IsSelected();
			if (bSelect)
			{
				UE_LOGF(LogViewportSelectionUtilities, Log, "Clicking on Component (CTRL LMB): %ls (%ls)", *Component->GetClass()->GetName(), *Component->GetName());
			}
			GEditor->SelectComponent(Component, bSelect, true, true);
			bComponentClicked = true;
		}
		else if (Click.IsShiftDown())
		{
			if (!Component->IsSelected())
			{
				UE_LOGF(LogViewportSelectionUtilities, Log, "Clicking on Component (SHIFT LMB): %ls (%ls)", *Component->GetClass()->GetName(), *Component->GetName());
				GEditor->SelectComponent(Component, true, true, true);
			}
			bComponentClicked = true;
		}
		else
		{
			GEditor->GetSelectedComponents()->DeselectAll();
			UE_LOGF(LogViewportSelectionUtilities, Log, "Clicking on Component (LMB): %ls (%ls)", *Component->GetClass()->GetName(), *Component->GetName());
			GEditor->SelectComponent(Component, true, true, true);
			bComponentClicked = true;
		}
	}

	return bComponentClicked;
}

void ClickBrushVertex(FEditorViewportClient* ViewportClient, ABrush* InBrush, FVector* InVertex, const FViewportClick& InClick)
{
	// Pivot snapping
	if (InClick.GetKey() == EKeys::MiddleMouseButton && InClick.IsAltDown())
	{
		GEditor->SetPivot(GEditor->ClickLocation, true, false, true);
	}
	else if (InClick.GetKey() == EKeys::RightMouseButton)
	{
		const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "ClickingOnBrushVertex", "Clicking on Brush Vertex"));
		const FTransform ActorToWorld = InBrush->ActorToWorld();
		GEditor->SetPivot(ActorToWorld.TransformPosition(*InVertex), false, false);

		const FVector World = ActorToWorld.TransformPosition(*InVertex);
		FVector Snapped = World;
		FSnappingUtils::SnapPointToGrid(Snapped, FVector(GEditor->GetGridSize()));
		const FVector Delta = Snapped - World;
		GEditor->SetPivot(Snapped, false, false);


		if (ViewportClient->GetModeTools()->IsDefaultModeActive())
		{
			// All selected actors need to move by the delta.
			for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
			{
				AActor* Actor = static_cast<AActor*>(*It);
				checkSlow(Actor->IsA(AActor::StaticClass()));

				Actor->Modify();

				FVector ActorLocation = Actor->GetActorLocation() + Delta;
				Actor->SetActorLocation(ActorLocation, false);
			}
		}

		ViewportClient->Invalidate(true, true);

		// Update Bsp
		GEditor->RebuildAlteredBSP();
	}
}

void ClickStaticMeshVertex(FEditorViewportClient* ViewportClient, AActor* InActor, FVector& InVertex, const FViewportClick& InClick)
{
	// Pivot snapping
	if (InClick.GetKey() == EKeys::MiddleMouseButton && InClick.IsAltDown())
	{
		GEditor->SetPivot(GEditor->ClickLocation, true, false, true);
	}
	else if (InClick.GetKey() == EKeys::RightMouseButton)
	{
		const FScopedTransaction Transaction(
			NSLOCTEXT("UnrealEd", "ClickingOnStaticMeshVertex", "Clicking on Static Mesh Vertex")
		);

		FVector Snapped = InVertex;
		FSnappingUtils::SnapPointToGrid(Snapped, FVector(GEditor->GetGridSize()));
		const FVector Delta = Snapped - InVertex;
		GEditor->SetPivot(Snapped, false, true);

		// All selected actors need to move by the delta.
		for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
		{
			AActor* Actor = static_cast<AActor*>(*It);
			checkSlow(Actor->IsA(AActor::StaticClass()));

			Actor->Modify();

			FVector ActorLocation = Actor->GetActorLocation() + Delta;
			Actor->SetActorLocation(ActorLocation, false);
		}

		ViewportClient->Invalidate(true, true);
	}
}
static FBspSurf GSaveSurf;

void ClickSurface(FEditorViewportClient* ViewportClient, UModel* Model, int32 InSurf, const FViewportClick& Click)
{
	// Gizmos can cause BSP surfs to become selected without this check
	if (Click.GetKey() == EKeys::RightMouseButton && Click.IsControlDown())
	{
		return;
	}

	// Remember hit location for actor-adding.
	FBspSurf& Surf = Model->Surfs[InSurf];

	// Pivot snapping
	if (Click.GetKey() == EKeys::MiddleMouseButton && Click.IsAltDown())
	{
		GEditor->SetPivot(GEditor->ClickLocation, true, false, true);
	}
	else if (Click.GetKey() == EKeys::LeftMouseButton && Click.IsShiftDown() && Click.IsControlDown())
	{

		if (!GetDefault<ULevelEditorViewportSettings>()->bClickBSPSelectsBrush)
		{
			// Add to the actor selection set the brush actor that belongs to this BSP surface.
			// Check Surf.Actor, as it can be NULL after deleting brushes and before rebuilding BSP.
			if (Surf.Actor)
			{
				const FScopedTransaction Transaction(
					NSLOCTEXT("UnrealEd", "SelectBrushFromSurface", "Select Brush from Surface")
				);

				// If the builder brush is selected, first deselect it.
				USelection* SelectedActors = GEditor->GetSelectedActors();
				for (FSelectionIterator It(*SelectedActors); It; ++It)
				{
					ABrush* Brush = Cast<ABrush>(*It);
					if (Brush && FActorEditorUtils::IsABuilderBrush(Brush))
					{
						GEditor->SelectActor(Brush, false, false);
						break;
					}
				}

				GEditor->SelectActor(Surf.Actor, true, true);
			}
		}
		else
		{
			// Select or deselect surfaces.
			{
				const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "SelectSurfaces", "Select Surfaces"));
				Model->ModifySurf(InSurf, false);
				Surf.PolyFlags ^= PF_Selected;
			}
			GEditor->NoteSelectionChange();
		}
	}
	else if (Click.GetKey() == EKeys::LeftMouseButton && Click.IsShiftDown())
	{
		FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();

		// Apply texture to all selected.
		const FScopedTransaction Transaction(
			NSLOCTEXT("UnrealEd", "ApplyMaterialToSelectedSurfaces", "Apply Material to Selected Surfaces")
		);

		UMaterialInterface* SelectedMaterialInstance = GEditor->GetSelectedObjects()->GetTop<UMaterialInterface>();
		for (int32 i = 0; i < Model->Surfs.Num(); i++)
		{
			if (Model->Surfs[i].PolyFlags & PF_Selected)
			{
				Model->ModifySurf(i, 1);
				Model->Surfs[i].Material = SelectedMaterialInstance;
				const bool bUpdateTexCoords = false;
				const bool bOnlyRefreshSurfaceMaterials = true;
				GEditor->polyUpdateBrush(Model, i, bUpdateTexCoords, bOnlyRefreshSurfaceMaterials);
			}
		}
	}
	else if (Click.GetKey() == EKeys::LeftMouseButton && ViewportClient->Viewport->KeyState(EKeys::A))
	{
		// Create an actor of the selected class.
		UClass* SelectedClass = GEditor->GetSelectedObjects()->GetTop<UClass>();
		if (SelectedClass)
		{
			PrivateAddActor(SelectedClass);
		}
	}
	else if (Click.GetKey() == EKeys::LeftMouseButton && ViewportClient->Viewport->KeyState(EKeys::L))
	{
		// If shift is down, we pick a color from under the mouse in the viewport and create a light with that color.
		if (Click.IsControlDown())
		{
			PickColorAndAddLight(ViewportClient->Viewport, Click);
		}
		else
		{
			// Create a point light (they default to stationary)
			PrivateAddActor(APointLight::StaticClass());
		}
	}
	else if (IsTexelDebuggingEnabled() && Click.GetKey() == EKeys::LeftMouseButton
			 && ViewportClient->Viewport->KeyState(EKeys::T))
	{
		SetDebugLightmapSample(NULL, Model, InSurf, GEditor->ClickLocation);
	}

	else if (Click.GetKey() == EKeys::LeftMouseButton && ViewportClient->Viewport->KeyState(EKeys::S))
	{
		// Create a static mesh.
		PrivateAddActor(AStaticMeshActor::StaticClass());
	}
	else if (Click.GetKey() == EKeys::LeftMouseButton && ViewportClient->Viewport->KeyState(EKeys::Semicolon))
	{
		PrivateAddActor(ATargetPoint::StaticClass());
	}
	else if (Click.IsAltDown() && Click.GetKey() == EKeys::RightMouseButton)
	{
		// Grab the texture.
		GEditor->GetSelectedObjects()->DeselectAll(UMaterialInterface::StaticClass());

		if (Surf.Material)
		{
			GEditor->GetSelectedObjects()->Select(Surf.Material);
		}
		GSaveSurf = Surf;
	}
	else if (Click.IsAltDown() && Click.GetKey() == EKeys::LeftMouseButton)
	{
		FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();

		// Apply texture to the one polygon clicked on.
		const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "ApplyMaterialToSurface", "Apply Material to Surface"));
		Model->ModifySurf(InSurf, true);
		Surf.Material = GEditor->GetSelectedObjects()->GetTop<UMaterialInterface>();
		if (Click.IsControlDown())
		{
			Surf.vTextureU = GSaveSurf.vTextureU;
			Surf.vTextureV = GSaveSurf.vTextureV;
			if (Surf.vNormal == GSaveSurf.vNormal)
			{
				UE_LOGF(LogViewportSelectionUtilities, Log, "WARNING: the texture coordinates were not parallel to the surface." );
			}
			Surf.PolyFlags = GSaveSurf.PolyFlags;
			const bool bUpdateTexCoords = true;
			const bool bOnlyRefreshSurfaceMaterials = true;
			GEditor->polyUpdateBrush(Model, InSurf, bUpdateTexCoords, bOnlyRefreshSurfaceMaterials);
		}
		else
		{
			const bool bUpdateTexCoords = false;
			const bool bOnlyRefreshSurfaceMaterials = true;
			GEditor->polyUpdateBrush(Model, InSurf, bUpdateTexCoords, bOnlyRefreshSurfaceMaterials);
		}
	}
	else if (Click.GetKey() == EKeys::RightMouseButton && !Click.IsControlDown())
	{
		// Select surface and display context menu
		check(Model);

		bool bNeedViewportRefresh = false;
		bool bSelectionChanged = !Surf.Actor || !Surf.Actor->IsSelected();
		{
			const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "SelectSurfaces", "Select Surfaces"));

			USelection* SelectedActors = GEditor->GetSelectedActors();
			SelectedActors->BeginBatchSelectOperation();

			// We only need to unselect surfaces if the surface the user clicked on was not already selected
			if (!(Surf.PolyFlags & PF_Selected))
			{
				GEditor->SelectNone(false, true);
				bNeedViewportRefresh = true;
				bSelectionChanged = true;
			}

			// Select the surface the user clicked on
			Model->ModifySurf(InSurf, false);
			Surf.PolyFlags |= PF_Selected;

			GEditor->SelectActor(Surf.Actor, true, false);
			SelectedActors->EndBatchSelectOperation(false);

			if (bSelectionChanged)
			{
				GEditor->NoteSelectionChange();
			}
		}

		if (bNeedViewportRefresh)
		{
			// Redraw the viewport so the user can see which object was right clicked on
			ViewportClient->Viewport->Draw();
			FlushRenderingCommands();
		}

		ViewportClient->SummonContextMenu();
	}
	else if (Click.GetEvent() == IE_DoubleClick && Click.GetKey() == EKeys::LeftMouseButton && !Click.IsControlDown())
	{
		{
			const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "SelectSurface", "Select Surface"));

			// Clear the selection
			GEditor->SelectNone(false, true);

			// Select the surface
			const uint32 SelectMask = Surf.PolyFlags & PF_Selected;
			Model->ModifySurf(InSurf, false);
			Surf.PolyFlags = (Surf.PolyFlags & ~PF_Selected) | (SelectMask ^ PF_Selected);
		}
		GEditor->NoteSelectionChange();

		// Display the surface properties window
		GEditor->Exec(ViewportClient->GetWorld(), TEXT("EDCALLBACK SURFPROPS"));
	}
	else
	{
		const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "SelectBrushSurface", "Select Brush Surface"));
		bool bDeselectAlreadyHandled = false;
		bool bSelectionChanged = !Surf.Actor || !Surf.Actor->IsSelected();

		USelection* SelectedActors = GEditor->GetSelectedActors();
		SelectedActors->BeginBatchSelectOperation();

		// We are going to handle the notification ourselves
		const bool bNotify = false;
		if (GetDefault<ULevelEditorViewportSettings>()->bClickBSPSelectsBrush)
		{
			// Add to the actor selection set the brush actor that belongs to this BSP surface.
			// Check Surf.Actor, as it can be NULL after deleting brushes and before rebuilding BSP.
			if (Surf.Actor)
			{
				if (!Click.IsControlDown())
				{
					GEditor->SelectNone(false, true);
					bDeselectAlreadyHandled = true;
				}
				// If the builder brush is selected, first deselect it.
				for (FSelectionIterator It(*SelectedActors); It; ++It)
				{
					ABrush* Brush = Cast<ABrush>(*It);
					if (Brush && FActorEditorUtils::IsABuilderBrush(Brush))
					{
						GEditor->SelectActor(Brush, false, bNotify);
						break;
					}
				}

				GEditor->SelectActor(Surf.Actor, true, bNotify);
			}
		}

		// Select or deselect surfaces.
		{
			if (Click.IsControlDown() || !(Surf.PolyFlags & PF_Selected))
			{
				bSelectionChanged = true;
			}

			if (!Click.IsControlDown() && !bDeselectAlreadyHandled)
			{
				GEditor->SelectNone(false, true);
			}
			Model->ModifySurf(InSurf, false);
			Surf.PolyFlags ^= PF_Selected;

			// If there are no surfaces selected now, deselect the actor
			if (!Model->HasSelectedSurfaces())
			{
				GEditor->SelectActor(Surf.Actor, false, bNotify);
				bSelectionChanged = true;
			}
		}

		SelectedActors->EndBatchSelectOperation(false);

		if (bSelectionChanged)
		{
			GEditor->NoteSelectionChange();
		}
	}
}

void ClickBackdrop(FEditorViewportClient* ViewportClient, const FViewportClick& InClick)
{
	// Pivot snapping
	if (InClick.GetKey() == EKeys::MiddleMouseButton && InClick.IsAltDown())
	{
		GEditor->SetPivot(GEditor->ClickLocation, true, false, true);
	}
	else if (InClick.GetKey() == EKeys::LeftMouseButton && ViewportClient->Viewport->KeyState(EKeys::A))
	{
		// Create an actor of the selected class.
		UClass* SelectedClass = GEditor->GetSelectedObjects()->GetTop<UClass>();
		if (SelectedClass)
		{
			PrivateAddActor(SelectedClass);
		}
	}
	else if (IsTexelDebuggingEnabled() && InClick.GetKey() == EKeys::LeftMouseButton
			 && ViewportClient->Viewport->KeyState(EKeys::T))
	{
		SetDebugLightmapSample(NULL, NULL, 0, GEditor->ClickLocation);
	}

	else if (InClick.GetKey() == EKeys::LeftMouseButton && ViewportClient->Viewport->KeyState(EKeys::L))
	{
		// If shift is down, we pick a color from under the mouse in the viewport and create a light with that color.
		if (InClick.IsControlDown())
		{
			PickColorAndAddLight(ViewportClient->Viewport, InClick);
		}
		else
		{
			// Create a point light (they default to stationary)
			PrivateAddActor(APointLight::StaticClass());
		}
	}
	else if (InClick.GetKey() == EKeys::LeftMouseButton && ViewportClient->Viewport->KeyState(EKeys::S))
	{
		// Create a static mesh.
		PrivateAddActor(AStaticMeshActor::StaticClass());
	}
	else if (InClick.GetKey() == EKeys::RightMouseButton && !InClick.IsControlDown()
			 && !ViewportClient->Viewport->KeyState(EKeys::LeftMouseButton))
	{
		// NOTE: We intentionally do not deselect selected actors here even though the user right
		//		clicked on an empty background.  This is because LDs often use wireframe modes to
		//		interact with brushes and such, and it's easier to summon the context menu for
		//		these actors when right clicking *anywhere* will not deselect things.

		// Redraw the viewport so the user can see which object was right clicked on
		ViewportClient->Viewport->Draw();
		FlushRenderingCommands();

		ViewportClient->SummonContextMenu();
	}
	else if (InClick.GetKey() == EKeys::LeftMouseButton)
	{
		if (!InClick.IsControlDown())
		{
			const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "ClickingBackground", "Clicking Background"));
			GEditor->SelectNone(true, true);
		}
	}
}

void ClickLevelSocket(FEditorViewportClient* ViewportClient, HHitProxy* HitProxy, const FViewportClick& Click)
{
	const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "LevelSocketClicked", "Level Socket Clicked"));

	FMessageLog EditorErrors("EditorErrors");
	EditorErrors.NewPage(LOCTEXT("SocketClickedNewPage", "Socket Clicked"));

	// Attach the selected actors to the socket that was clicked
	HLevelSocketProxy* SocketProxy = static_cast<HLevelSocketProxy*>(HitProxy);
	check(SocketProxy->SceneComponent);
	check(SocketProxy->Actor);

	for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
	{
		if (AActor* Actor = Cast<AActor>(*It))
		{
			// Parent actors and handle socket snapping.
			// Will cause editor to refresh viewport.
			FText ReasonText;
			if (GEditor->CanParentActors(SocketProxy->Actor, Actor, &ReasonText) == false)
			{
				EditorErrors.Error(ReasonText);
			}
			else
			{
				GEditor->ParentActors(SocketProxy->Actor, Actor, SocketProxy->SocketName, SocketProxy->SceneComponent);
			}
		}
	}

	// Report errors
	EditorErrors.Notify(NSLOCTEXT("ActorAttachmentError", "AttachmentsFailed", "Attachments Failed!"));
}
} // namespace UE::Editor::ViewportUtilities



// Used by Deprecated code
namespace LevelViewportClickHandlers
{
	static void PrivateSummonViewportMenu( FLevelEditorViewportClient* ViewportClient )
	{
		if (ViewportClient->ParentLevelEditor.IsValid())
		{
			ViewportClient->ParentLevelEditor.Pin()->SummonLevelViewportViewOptionMenu(LVT_Perspective);
		}
	}

	bool ClickViewport(FLevelEditorViewportClient* ViewportClient, const FViewportClick& Click)
	{
		if (Click.GetKey() == EKeys::MiddleMouseButton && Click.IsControlDown())
		{
			PrivateSummonViewportMenu(ViewportClient);
			return true;
		}
		return false;
	}

	bool ClickElement(FLevelEditorViewportClient* ViewportClient, const FTypedElementHandle& HitElement, const FViewportClick& Click)
	{
		return UE::Editor::ViewportSelectionUtilities::ClickElement(ViewportClient, HitElement, Click);
	}

	bool ClickActor(FLevelEditorViewportClient* ViewportClient,AActor* Actor,const FViewportClick& Click,bool bAllowSelectionChange)
	{
		return UE::Editor::ViewportSelectionUtilities::ClickActor(ViewportClient, Actor, Click, bAllowSelectionChange);
	}

	bool ClickComponent(FLevelEditorViewportClient* ViewportClient, HActor* ActorHitProxy, const FViewportClick& Click)
	{
		return UE::Editor::ViewportSelectionUtilities::ClickComponent(ViewportClient, ActorHitProxy, Click);
	}

	void ClickBrushVertex(FLevelEditorViewportClient* ViewportClient,ABrush* InBrush,FVector* InVertex,const FViewportClick& Click)
	{
		UE::Editor::ViewportSelectionUtilities::ClickBrushVertex(ViewportClient, InBrush, InVertex, Click);
	}

	void ClickStaticMeshVertex(FLevelEditorViewportClient* ViewportClient,AActor* InActor,FVector& InVertex,const FViewportClick& Click)
	{
		UE::Editor::ViewportSelectionUtilities::ClickStaticMeshVertex(ViewportClient, InActor, InVertex, Click);
	}


	void ClickSurface(FLevelEditorViewportClient* ViewportClient,UModel* Model,int32 iSurf,const FViewportClick& Click)
	{
		UE::Editor::ViewportSelectionUtilities::ClickSurface(ViewportClient, Model, iSurf, Click);
	}

	void ClickBackdrop(FLevelEditorViewportClient* ViewportClient,const FViewportClick& Click)
	{
		UE::Editor::ViewportSelectionUtilities::ClickBackdrop(ViewportClient, Click);
	}

	void ClickLevelSocket(FLevelEditorViewportClient* ViewportClient, HHitProxy* HitProxy, const FViewportClick& Click)
	{
		UE::Editor::ViewportSelectionUtilities::ClickLevelSocket(ViewportClient, HitProxy, Click);
	}
}


#undef LOCTEXT_NAMESPACE
