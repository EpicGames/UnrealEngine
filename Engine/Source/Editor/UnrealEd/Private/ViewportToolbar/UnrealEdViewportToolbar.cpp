// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportToolbar/UnrealEdViewportToolbar.h"

#include "AdvancedPreviewSceneCommands.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DebugViewModeHelpers.h"
#include "EditorInteractiveGizmoManager.h"
#include "Editor/EditorPerformanceSettings.h"
#include "EditorViewportClient.h"
#include "EditorViewportCommands.h"
#include "EditorModeManager.h"
#include "Engine/GameViewportClient.h"
#include "Framework/Commands/GenericCommands.h"
#include "GPUSkinCache.h"
#include "GPUSkinCacheVisualizationMenuCommands.h"
#include "IPreviewProfileController.h"
#include "IPreviewLODController.h"
#include "ISettingsModule.h"
#include "LevelEditor.h"
#include "LevelEditorActions.h"
#include "SEditorViewport.h"
#include "SScalabilitySettings.h"
#include "ShowFlagMenuCommands.h"
#include "Settings/EditorProjectSettings.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Styling/SlateIconFinder.h"
#include "Templates/SharedPointer.h"
#include "ToolMenu.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "ToolMenusEditor.h"
#include "TransformGizmoEditorSettings.h"
#include "Tests/ToolMenusTestUtilities.h"
#include "ViewportToolbar/UnrealEdViewportToolbarContext.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SSlider.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SToolTip.h"
#include "RenderUtils.h"

#define LOCTEXT_NAMESPACE "UnrealEdViewportToolbar"

namespace UE::UnrealEd::Private
{
int32 CVarToolMenusViewportToolbarsValue = 2;

struct ViewModesSubmenu
{
	static void AddModeIfSupported(
		const IsViewModeSupportedDelegate& InIsViewModeSupported,
		FToolMenuSection& InMenuSection,
		const TSharedPtr<FUICommandInfo>& InModeCommandInfo,
		EViewModeIndex InViewModeIndex,
		const TAttribute<FText>& InToolTipOverride = TAttribute<FText>(),
		const TAttribute<FSlateIcon>& InIconOverride = TAttribute<FSlateIcon>()
	)
	{
		if (!InIsViewModeSupported.IsBound() || InIsViewModeSupported.Execute(InViewModeIndex))
		{
			InMenuSection.AddMenuEntry(
				InModeCommandInfo, UViewModeUtils::GetViewModeDisplayName(InViewModeIndex), InToolTipOverride, InIconOverride
			);
		}
	}

	static bool IsMenuSectionAvailable(const UUnrealEdViewportToolbarContext* InContext, EHidableViewModeMenuSections InMenuSection)
	{
		if (!InContext->DoesViewModeMenuShowSection.IsBound())
		{
			return true;
		}

		return InContext->DoesViewModeMenuShowSection.Execute(InMenuSection);
	}
};

FToolUIAction DisabledAction()
{
	static FToolUIAction Action;
	Action.CanExecuteAction = FToolMenuCanExecuteAction::CreateLambda(
		[](const FToolMenuContext&)
		{
			return false;
		}
	);

	return Action;
}

void ToggleSurfaceSnapping()
{
	if (ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>())
	{
		ViewportSettings->SnapToSurface.bEnabled = !ViewportSettings->SnapToSurface.bEnabled;
	}
}

bool IsSurfaceSnappingEnabled()
{
	if (const ULevelEditorViewportSettings* const ViewportSettings = GetDefault<ULevelEditorViewportSettings>())
	{
		return ViewportSettings->SnapToSurface.bEnabled;
	}

	return false;
}

FToolMenuEntry CreateSurfaceSnapOffsetEntry()
{
	FText Label = LOCTEXT("SurfaceOffsetLabel", "Surface Offset");
	FText Tooltip = LOCTEXT("SurfaceOffsetTooltip", "The amount of offset to apply when snapping to surfaces");

	const FMargin WidgetsMargin(2.0f, 0.0f, 3.0f, 0.0f);

	FToolMenuEntry SurfaceOffset = FToolMenuEntry::InitMenuEntry(
		"SurfaceOffset",
		FUIAction(),
		// clang-format off
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.Padding(WidgetsMargin)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(Label)
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(WidgetsMargin)
		.AutoWidth()
		[
			SNew(SBox)
			.Padding(WidgetsMargin)
			.MinDesiredWidth(100.0f)
			[
				// Min/Max/Slider values taken from STransformViewportToolbar.cpp
				SNew(SNumericEntryBox<float>)
				.ToolTipText(Tooltip)
				.MinValue(0.0f)
				.MaxValue(static_cast<float>(HALF_WORLD_MAX))
				.MaxSliderValue(1000.0f)
				.AllowSpin(true)
				.MaxFractionalDigits(2)
				.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
				.OnValueChanged_Lambda([](float InNewValue)
				{
					ULevelEditorViewportSettings* Settings =
						GetMutableDefault<ULevelEditorViewportSettings>();
					Settings->SnapToSurface.SnapOffsetExtent = InNewValue;

					// If user is editing surface snapping values, we assume they also want surface snapping turned on
					if (!UE::UnrealEd::Private::IsSurfaceSnappingEnabled())
					{
						UE::UnrealEd::Private::ToggleSurfaceSnapping();
					}
				})
				.Value_Lambda([]()
				{
					return GetDefault<ULevelEditorViewportSettings>()->SnapToSurface.SnapOffsetExtent;
				})
			]
		]
		// clang-format on
	);

	return SurfaceOffset;
}

FToolMenuEntry CreateSurfaceSnapCheckboxMenu(const FToolMenuContext& InContext)
{
	const TSharedPtr<FUICommandInfo> Command = FEditorViewportCommands::Get().SurfaceSnapping;

	FUIAction Action;
	
	if (const FUIAction* FoundAction = InContext.GetActionForCommand(Command))
	{
		Action = *FoundAction;
	}
	else
	{
		// Provide a default implementation
		// TODO: This would be better handled by ensuring each editor has a bound command.
		Action.ExecuteAction.BindLambda([] { ToggleSurfaceSnapping(); });
		Action.GetActionCheckState.BindLambda([]
		{
			return IsSurfaceSnappingEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		});
	}
	
	FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
		"SurfaceSnapping",
		LOCTEXT("SurfaceSnapLabel", "Surface"),
		Command->GetDescription(),
		FNewToolMenuDelegate::CreateLambda([](UToolMenu* Submenu)
		{
			FToolMenuSection& SurfaceSnappingSection =
				Submenu->FindOrAddSection("SurfaceSnapping", LOCTEXT("SurfaceSnappingLabel", "Surface Snapping"));

			SurfaceSnappingSection.AddMenuEntry(FEditorViewportCommands::Get().RotateToSurfaceNormal);
			SurfaceSnappingSection.AddEntry(CreateSurfaceSnapOffsetEntry());
		}),
		Action,
		EUserInterfaceActionType::ToggleButton
	);
	
	if (UUnrealEdViewportToolbarContext* Context = InContext.FindContext<UUnrealEdViewportToolbarContext>())
	{
		TWeakObjectPtr<UUnrealEdViewportToolbarContext> WeakContext = Context;
		Entry.Visibility = [WeakContext]
		{
			if (WeakContext.IsValid())
			{
				return WeakContext->bShowSurfaceSnap;
			}
			return true;
		};
	}

	Entry.InputBindingLabel = TAttribute<FText>::CreateLambda(
		[]
		{
			if (const TSharedPtr<FUICommandInfo>& Command = FEditorViewportCommands::Get().SurfaceSnapping)
			{
				return Command->GetInputText().ToUpper();
			}

			return FText();
		}
	);

	Entry.ToolBarData.LabelOverride = TAttribute<FText>::CreateLambda([]
	{
		const ULevelEditorViewportSettings* const Settings = GetMutableDefault<ULevelEditorViewportSettings>();
		return FText::AsNumber(Settings->SnapToSurface.SnapOffsetExtent);
	});
	
	Entry.Icon = TAttribute<FSlateIcon>::CreateLambda([]
	{
		// todo: dynamic update works in real time for the icon in the raised entry, but not within the menu itself.
		// In order to se the icon within the menu update, the menu needs to close and re-open
		return GetDefault<ULevelEditorViewportSettings>()->SnapToSurface.bSnapRotation
				 ? FSlateIcon(FAppStyle::GetAppStyleSetName(), "ViewportToolbar.SurfaceSnapRotateToNormal")
				 : FSlateIcon(FAppStyle::GetAppStyleSetName(), "ViewportToolbar.SurfaceSnap");
	});

	return Entry;
}

FToolMenuEntry CreateLocationSnapCheckboxMenu(const FToolMenuContext& InContext)
{
	const TSharedPtr<FUICommandInfo> Command = FEditorViewportCommands::Get().LocationGridSnap;

	FUIAction Action;
	
	if (const FUIAction* FoundAction = InContext.GetActionForCommand(Command))
	{
		Action = *FoundAction;
	}
	else
	{
		// Provide a default implementation
		// TODO: This would be better handled by ensuring each editor has a bound command.
		Action.ExecuteAction.BindStatic(&FLevelEditorActionCallbacks::LocationGridSnap_Clicked);
		Action.GetActionCheckState.BindLambda([]
		{
			return FLevelEditorActionCallbacks::LocationGridSnap_IsChecked() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		});
	}

	FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
		"GridSnapping",
		LOCTEXT("LocationSnapLabel", "Location"),
		Command->GetDescription(),
		FNewToolMenuDelegate::CreateLambda([](UToolMenu* ToolMenu)
		{
			UUnrealEdViewportToolbarContext* Context = ToolMenu->FindContext<UUnrealEdViewportToolbarContext>();
			if (!Context) return;
					
			FToolMenuSection& Section = ToolMenu->AddSection("Snap", LOCTEXT("LocationSnapText", "Snap Sizes"));
					
			const TArray<float> GridSizes = Context->GetGridSnapSizes();
					
			for (int32 GridSizeIndex = 0; GridSizeIndex < GridSizes.Num(); ++GridSizeIndex)
			{
				const float GridSize = GridSizes[GridSizeIndex];
				Section.AddMenuEntry(
					NAME_None,
					FText::AsNumber(GridSize),
					FText::Format(
						LOCTEXT("LocationGridSize_ToolTip", "Sets grid size to {0}"), FText::AsNumber(GridSize)
					),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateUObject(Context, &UUnrealEdViewportToolbarContext::SetGridSnapSize, GridSizeIndex),
						FCanExecuteAction(),
						FIsActionChecked::CreateUObject(Context, &UUnrealEdViewportToolbarContext::IsGridSnapSizeActive, GridSizeIndex)
					),
					EUserInterfaceActionType::RadioButton
				);
			}
		}),
		Action,
		EUserInterfaceActionType::ToggleButton
	);
			
	Entry.UserInterfaceActionType = EUserInterfaceActionType::ToggleButton;
	
	if (UUnrealEdViewportToolbarContext* Context = InContext.FindContext<UUnrealEdViewportToolbarContext>())
	{
		TAttribute<FText> LabelOverride;
		LabelOverride.BindUObject(Context, &UUnrealEdViewportToolbarContext::GetGridSnapLabel);
			
		Entry.ToolBarData.LabelOverride = LabelOverride;
	}
	
	Entry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ViewportToolbar.SnapLocation");
	Entry.InputBindingLabel = TAttribute<FText>::CreateLambda(
		[]
		{
			const TSharedPtr<FUICommandInfo>& Command = FEditorViewportCommands::Get().LocationGridSnap;
			if (Command.IsValid())
			{
				return Command->GetInputText().ToUpper();
			}

			return FText();
		}
	);
	
	return Entry;
}

TSharedRef<SWidget> CreateRotationGridSnapList(
	UUnrealEdViewportToolbarContext* Context,
	const FText& Heading,
	const TArray<float>& Sizes,
	ERotationGridMode GridMode
)
{
	FMenuBuilder Menu(true, nullptr);
	
	Menu.BeginSection(NAME_None, Heading);
	
	for (int32 Index = 0; Index < Sizes.Num(); ++Index)
	{
		FText Label = FText::Format(
			LOCTEXT("RotationGridAngle", "{0}\u00b0"), // \u00b0 is the degree symbol
			FText::AsNumber(Sizes[Index])
		);
		
		FText ToolTip = FText::Format(
			LOCTEXT("RotationGridAngle_ToolTip", "Sets rotation grid angle to {0}"),
			Label
		);
	
		Menu.AddMenuEntry(
			Label,
			ToolTip,
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateUObject(Context, &UUnrealEdViewportToolbarContext::SetRotationSnapSize, Index, GridMode),
				FCanExecuteAction(),
				FIsActionChecked::CreateUObject(Context, &UUnrealEdViewportToolbarContext::IsRotationSnapActive, Index, GridMode)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
	
	Menu.EndSection();
	
	return Menu.MakeWidget();
}

TSharedRef<SWidget> CreateRotationGridSnapWidget(UUnrealEdViewportToolbarContext* Context)
{
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();

	return SNew(SUniformGridPanel)
		+ SUniformGridPanel::Slot(0, 0)
		[
			CreateRotationGridSnapList(
				Context,
				LOCTEXT("RotationCommonText", "Rotation Increment"),
				ViewportSettings->CommonRotGridSizes,
				GridMode_Common
			)
		]
		+ SUniformGridPanel::Slot(1, 0)
		[
			CreateRotationGridSnapList(
				Context,
				LOCTEXT("RotationDivisions360DegreesText", "Divisions of 360\u00b0"),
				ViewportSettings->DivisionsOf360RotGridSizes,
				GridMode_DivisionsOf360
			)
		];
}

FToolMenuEntry CreateRotationSnapCheckboxMenu(const FToolMenuContext& InContext)
{
	const TSharedPtr<FUICommandInfo> Command = FEditorViewportCommands::Get().RotationGridSnap;
	
	FUIAction Action;
	
	if (const FUIAction* FoundAction = InContext.GetActionForCommand(Command))
	{
		Action = *FoundAction;
	}
	else
	{
		// Provide a default implementation
		// TODO: This would be better handled by ensuring each editor has a bound command.
		Action.ExecuteAction.BindStatic(&FLevelEditorActionCallbacks::RotationGridSnap_Clicked);
		Action.GetActionCheckState.BindLambda([]
		{
			return FLevelEditorActionCallbacks::RotationGridSnap_IsChecked()
				? ECheckBoxState::Checked
				: ECheckBoxState::Unchecked;
		});
	}
	
	FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
		"RotationSnapping",
		LOCTEXT("RotationSnapLabel", "Rotation"),
		FEditorViewportCommands::Get().RotationGridSnap->GetDescription(),
		FNewToolMenuDelegate::CreateLambda([](UToolMenu* InToolMenu)
		{
			UUnrealEdViewportToolbarContext* Context = InToolMenu->FindContext<UUnrealEdViewportToolbarContext>();
			if (!Context)
			{
				return;
			}
			
			InToolMenu->AddMenuEntry(
				"RotationSnap",
				FToolMenuEntry::InitWidget(
					"RotationSnap",
					CreateRotationGridSnapWidget(Context),
					FText()
				)
			);
		}),
		Action,
		EUserInterfaceActionType::ToggleButton
	);

	TAttribute<FText> LabelOverride;
	
	if (UUnrealEdViewportToolbarContext* Context = InContext.FindContext<UUnrealEdViewportToolbarContext>())
	{
		LabelOverride.BindUObject(Context, &UUnrealEdViewportToolbarContext::GetRotationSnapLabel);
	}

	Entry.InputBindingLabel = TAttribute<FText>::CreateLambda(
		[]
		{
			const TSharedPtr<FUICommandInfo>& Command = FEditorViewportCommands::Get().RotationGridSnap;
			if (Command.IsValid())
			{
				return Command->GetInputText().ToUpper();
			}

			return FText();
		}
	);

	Entry.ToolBarData.LabelOverride = LabelOverride;
	
	Entry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ViewportToolbar.SnapRotation");

	return Entry;
}

FToolMenuEntry CreateScaleSnapCheckboxMenu(const FToolMenuContext& InContext)
{
	const TSharedPtr<FUICommandInfo> Command = FEditorViewportCommands::Get().ScaleGridSnap;
	
	FUIAction Action;
	
	if (const FUIAction* FoundAction = InContext.GetActionForCommand(Command))
	{
		Action = *FoundAction;
	}
	else
	{
		// Provide a default implementation
		// TODO: This would be better handled by ensuring each editor has a bound command.
		Action.ExecuteAction.BindStatic(&FLevelEditorActionCallbacks::ScaleGridSnap_Clicked);
		Action.GetActionCheckState.BindLambda([]
		{
			return FLevelEditorActionCallbacks::ScaleGridSnap_IsChecked() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		});
	}

	const FName ScaleSnapName = "ScaleSnap";
	
	FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
		"ScaleSnapping",
		LOCTEXT("ScaleSnapLabel", "Scale"),
		Command->GetDescription(),
		FNewToolMenuDelegate::CreateLambda([ScaleSnapName](UToolMenu* InToolMenu)
		{
			UUnrealEdViewportToolbarContext* Context = InToolMenu->FindContext<UUnrealEdViewportToolbarContext>();
			if (!Context)
			{
				return;
			}
			
			FToolMenuSection& Section = InToolMenu->FindOrAddSection("ScaleSnap", LOCTEXT("ScaleSnapSizesSectionLabel", "Snap Sizes"));
		
			TArray<float> ScaleSnapSizes = Context->GetScaleSnapSizes();
			
			FNumberFormattingOptions NumberFormattingOptions;
			NumberFormattingOptions.MaximumFractionalDigits = 5;
			
			for (int32 ScaleSnapSizeIndex = 0; ScaleSnapSizeIndex < ScaleSnapSizes.Num(); ++ScaleSnapSizeIndex)
			{
				const float CurGridAmount = ScaleSnapSizes[ScaleSnapSizeIndex];

				FText Label;
				FText ToolTip;

				if (GEditor->UsePercentageBasedScaling())
				{
					Label = FText::AsPercent(CurGridAmount / 100.0f, &NumberFormattingOptions);
					ToolTip =
						FText::Format(LOCTEXT("ScaleGridAmountOld_ToolTip", "Snaps scale values to {0}"), Label);
				}
				else
				{
					Label = FText::AsNumber(CurGridAmount, &NumberFormattingOptions);
					ToolTip = FText::Format(
						LOCTEXT("ScaleGridAmount_ToolTip", "Snaps scale values to increments of {0}"), Label
					);
				}
				
				Section.AddMenuEntry(
					NAME_None,
					Label,
					ToolTip,
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateUObject(Context, &UUnrealEdViewportToolbarContext::SetScaleSnapSize, ScaleSnapSizeIndex),
						FCanExecuteAction(),
						FIsActionChecked::CreateUObject(Context, &UUnrealEdViewportToolbarContext::IsScaleSnapActive, ScaleSnapSizeIndex)
					),
					EUserInterfaceActionType::RadioButton
				);
			}
		}),
		Action,
		EUserInterfaceActionType::ToggleButton
	);

	TAttribute<FText> LabelOverride;
	
	if (UUnrealEdViewportToolbarContext* Context = InContext.FindContext<UUnrealEdViewportToolbarContext>())
	{
		LabelOverride.BindUObject(Context, &UUnrealEdViewportToolbarContext::GetScaleSnapLabel);
	}

	Entry.InputBindingLabel = TAttribute<FText>::CreateLambda(
		[]
		{
			const TSharedPtr<FUICommandInfo>& Command = FEditorViewportCommands::Get().ScaleGridSnap;
			if (Command.IsValid())
			{
				return Command->GetInputText().ToUpper();
			}

			return FText();
		}
	);

	Entry.ToolBarData.LabelOverride = LabelOverride;
	Entry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ViewportToolbar.SnapScale");

	return Entry;
}

FToolMenuEntry CreateLayer2DSnapCheckboxMenu(const FToolMenuContext& InContext)
{
	TSharedPtr<FUICommandInfo> Command = FEditorViewportCommands::Get().Layer2DSnap;

	FUIAction Action;
	
	if (const FUIAction* FoundAction = InContext.GetActionForCommand(Command))
	{
		Action = *FoundAction;
	}
	else
	{
		// Provide a default implementation
		// TODO: This would be better handled by ensuring each editor has a bound command.
		Action.ExecuteAction.BindLambda([]
		{
			ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>();
			const ULevelEditor2DSettings* Settings2D = GetDefault<ULevelEditor2DSettings>();
			if (!ViewportSettings->bEnableLayerSnap && (Settings2D->SnapLayers.Num() > 0))
			{
				ViewportSettings->bEnableLayerSnap = true;
				ViewportSettings->ActiveSnapLayerIndex = FMath::Clamp(ViewportSettings->ActiveSnapLayerIndex, 0, Settings2D->SnapLayers.Num() - 1);
			}
			else
			{
				ViewportSettings->bEnableLayerSnap = false;
			}
			ViewportSettings->PostEditChange();
		});
		Action.GetActionCheckState.BindLambda([]
		{
			const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
			const ULevelEditor2DSettings* Settings2D = GetDefault<ULevelEditor2DSettings>();
			const bool bChecked = ViewportSettings->bEnableLayerSnap && Settings2D->SnapLayers.IsValidIndex(ViewportSettings->ActiveSnapLayerIndex);
			return bChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		});
	}
	
	TAttribute<FText> Label = TAttribute<FText>::CreateLambda([]
	{
		const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
		const ULevelEditor2DSettings* Settings2D = GetDefault<ULevelEditor2DSettings>();
		if (Settings2D->SnapLayers.IsValidIndex(ViewportSettings->ActiveSnapLayerIndex))
		{
			return FText::FromString(Settings2D->SnapLayers[ViewportSettings->ActiveSnapLayerIndex].Name);
		}

		return FText();
	});

	FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
		"Layer2DSnap",
		Label,
		Command->GetDescription(),
		FNewToolMenuDelegate::CreateLambda([](UToolMenu* Menu)
		{
			const ULevelEditor2DSettings* Settings2D = GetDefault<ULevelEditor2DSettings>();
		
			{
				FToolMenuSection& LayerSection = Menu->AddSection("2DLayers", LOCTEXT("Layer2DLayerHeader", "Layers")); 
				const int32 LayerCount = Settings2D->SnapLayers.Num();
			
				for (int32 LayerIndex = 0; LayerIndex < LayerCount; ++LayerIndex)
				{
					FName LayerName(*Settings2D->SnapLayers[LayerIndex].Name);

					FUIAction Action(
						FExecuteAction::CreateLambda([LayerIndex]
						{
							ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>();
							ViewportSettings->bEnableLayerSnap = true;
							ViewportSettings->ActiveSnapLayerIndex = LayerIndex;
							ViewportSettings->PostEditChange();
						}),
						FCanExecuteAction(),
						FIsActionChecked::CreateLambda([LayerIndex]
						{
							const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
							return (ViewportSettings->ActiveSnapLayerIndex == LayerIndex);
						})
					);
				
					LayerSection.AddMenuEntry(
						LayerName,
						FText::FromName(LayerName),
						FText::GetEmpty(),
						FSlateIcon(),
						Action,
						EUserInterfaceActionType::RadioButton
					);
				}
				
				LayerSection.AddMenuEntry(
					"GoToSettings",
					LOCTEXT("2DSnap_EditLayer", "Edit Layers..."),
					FText::GetEmpty(),
					FSlateIcon(),
					FExecuteAction::CreateLambda([]
					{
						if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
						{
							SettingsModule->ShowViewer("Project", "Editor", "LevelEditor2DSettings");
						}
					})
				);
			}
			
			{
				FToolMenuSection& ActionSection = Menu->AddSection("Actions", LOCTEXT("Layer2DActionHeader", "Actions"));
				
				ActionSection.AddMenuEntry(FLevelEditorCommands::Get().SnapTo2DLayer);

				ActionSection.AddSeparator(NAME_None);
				ActionSection.AddMenuEntry(FLevelEditorCommands::Get().MoveSelectionToTop2DLayer);
				ActionSection.AddMenuEntry(FLevelEditorCommands::Get().MoveSelectionUpIn2DLayers);
				ActionSection.AddMenuEntry(FLevelEditorCommands::Get().MoveSelectionDownIn2DLayers);
				ActionSection.AddMenuEntry(FLevelEditorCommands::Get().MoveSelectionToBottom2DLayer);
				
				ActionSection.AddSeparator(NAME_None);
				ActionSection.AddMenuEntry(FLevelEditorCommands::Get().Select2DLayerAbove);
				ActionSection.AddMenuEntry(FLevelEditorCommands::Get().Select2DLayerBelow);
			}
			
		}),
		Action,
		EUserInterfaceActionType::ToggleButton
	);
	
	Entry.Icon = Command->GetIcon();
	Entry.ToolBarData.LabelOverride = Label;
	
	Entry.Visibility = TAttribute<bool>::CreateLambda([]
	{
		return GetDefault<ULevelEditor2DSettings>()->bEnableSnapLayers;
	});
	
	return Entry;
}

bool IsViewModeSupported(EViewModeIndex InViewModeIndex)
{
	switch (InViewModeIndex)
	{
	case VMI_PrimitiveDistanceAccuracy:
	case VMI_MaterialTextureScaleAccuracy:
	case VMI_RequiredTextureResolution:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	case VMI_Lit_Wireframe:
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
		return false;
	default:
		return true;
	}
}

void AddGizmoSizeSlider(FToolMenuSection& InMenuSection)
{
	const FSlateIcon WidgetIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ViewportToolbar.GizmoScale");

	// Helper struct, used by both Legacy and New Gizmo scale sliders to specify their lambdas
	struct FWidgetLambdas
	{
		TFunction<FText()> Tooltip = []() { return FText::GetEmpty(); };
		TFunction<float()> Value = []() { return 0.0f; };
		TFunction<void(float)> OnValueChanged = [](float) {};
		TFunction<void(float, ETextCommit::Type)> OnCommit = TFunction<void(float, ETextCommit::Type)>();
	};

	// Creates a gizmo scale widget with a slider using the lambdas specified by InWidgetLambdas
	auto MakeGizmoScaleWidget = [](float InMin, float InMax, FWidgetLambdas InWidgetLambdas)
	{
		// Commit lambda is not set, so we use OnValueChanged for that too
		if (!InWidgetLambdas.OnCommit)
		{
			InWidgetLambdas.OnCommit = [OnValueChanged = InWidgetLambdas.OnValueChanged](float InValue, ETextCommit::Type)
			{
				OnValueChanged(InValue);
			};
		}

		// clang-format off
		return SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
			.WidthOverride(100.0f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
				.Padding(FMargin(1.0f))
				[
					SNew(SSpinBox<float>)
					.Style(&FAppStyle::Get(), "Menu.SpinBox")
					.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
					.MinValue(InMin)
					.MaxValue(InMax)
					.ToolTipText_Lambda(MoveTemp(InWidgetLambdas.Tooltip))
					.Value_Lambda(MoveTemp(InWidgetLambdas.Value))
					.OnValueChanged_Lambda(MoveTemp(InWidgetLambdas.OnValueChanged))
					.OnValueCommitted_Lambda(MoveTemp(InWidgetLambdas.OnCommit))
				]
			]
		];
		// clang-format on
	};

	// Legacy Gizmo Scale widget
	{
		FWidgetLambdas LegacyWidgetLambdas;
		LegacyWidgetLambdas.Tooltip = []()
		{
			return FText::AsNumber(GetDefault<UTransformGizmoEditorSettings>()->TransformGizmoSize);
		};
		LegacyWidgetLambdas.Value = []()
		{
			return GetDefault<UTransformGizmoEditorSettings>()->TransformGizmoSize;
		};
		LegacyWidgetLambdas.OnValueChanged = [](float InValue)
		{
			UTransformGizmoEditorSettings* ViewportSettings = GetMutableDefault<UTransformGizmoEditorSettings>();
			ViewportSettings->SetTransformGizmoSize(InValue);
		};

		TSharedRef<SWidget> LegacyGizmoScaleWidget = MakeGizmoScaleWidget(-10.0f,150.0f, LegacyWidgetLambdas);

		FToolMenuEntry& GizmoScaleEntry = InMenuSection.AddEntry(FToolMenuEntry::InitWidget("LegacyGizmoScale", LegacyGizmoScaleWidget, LOCTEXT("LegacyGizmoScaleLabel", "Gizmo Scale")));
		GizmoScaleEntry.Icon = WidgetIcon;
		GizmoScaleEntry.Visibility = TAttribute<bool>::CreateLambda([]
		{
			// Legacy only
			return UEditorInteractiveGizmoManager::GetTransformGizmoVersion() == 0;
		});
	}

	// New Gizmo Scale widget
	{
		FWidgetLambdas WidgetLambdas;
		WidgetLambdas.Tooltip = []()
		{
			return FText::AsNumber(GetDefault<UTransformGizmoEditorSettings>()->GetTransformGizmoAxisMultiplier());
		};
		WidgetLambdas.Value = []()
		{
			return GetDefault<UTransformGizmoEditorSettings>()->GetTransformGizmoAxisMultiplier();
		};
		WidgetLambdas.OnValueChanged = [](float InValue)
		{
			// Interactive: SetTransformGizmoAxisMultiplier will not store the new value in config.
			// Grants better performance while dragging the slider
			constexpr bool bInteractive = true;
			UTransformGizmoEditorSettings* ViewportSettings = GetMutableDefault<UTransformGizmoEditorSettings>();
			ViewportSettings->SetTransformGizmoAxisMultiplier(InValue, bInteractive);
		};
		WidgetLambdas.OnCommit = [](float InValue, ETextCommit::Type)
		{
			UTransformGizmoEditorSettings* ViewportSettings = GetMutableDefault<UTransformGizmoEditorSettings>();
			ViewportSettings->SetTransformGizmoAxisMultiplier(InValue);
		};

		TSharedRef<SWidget> GizmoScaleWidget = MakeGizmoScaleWidget(0.1f,10.0f, WidgetLambdas);

		FToolMenuEntry& GizmoScaleEntry = InMenuSection.AddEntry(FToolMenuEntry::InitWidget("GizmoScale", GizmoScaleWidget, LOCTEXT("GizmoScaleLabel", "Gizmo Scale")));
		GizmoScaleEntry.Icon = WidgetIcon;
		GizmoScaleEntry.Visibility = TAttribute<bool>::CreateLambda([]
		{
			return UEditorInteractiveGizmoManager::GetTransformGizmoVersion() >= 1;
		});
	}
}

class SOrthographicViewPlaneSlider : public SCompoundWidget
{
public:
	enum class EMode
	{
		Near,
		Far
	};

	SLATE_BEGIN_ARGS(SOrthographicViewPlaneSlider) {}
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, TWeakPtr<FEditorViewportClient> InClient, EMode InMode)
	{
		WeakClient = InClient;
		Mode = InMode;
		
		ChildSlot
		[
			SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
			.HAlign(HAlign_Right)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
				.Padding(FMargin(1.0f))
				[
					SNew(SSpinBox<float>)
					.Style(&FAppStyle::Get(), "Menu.SpinBox")
					.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
					.MinValue(this, &SOrthographicViewPlaneSlider::GetMin)
					.MaxValue(this, &SOrthographicViewPlaneSlider::GetMax)
					.OnGetDisplayValue(this, &SOrthographicViewPlaneSlider::GetDisplayValue)
					.Value(this, &SOrthographicViewPlaneSlider::GetValue)
					.OnValueChanged(this, &SOrthographicViewPlaneSlider::OnValueChanged)
					.MaxFractionalDigits(2)
					.MinDesiredWidth(100)
				]
			]
		];
	}
	
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
	{
		if (TSharedPtr<FEditorViewportClient> Client = WeakClient.Pin())
		{
			const FBox Bounds = Client->GetWorldBounds();
			if (Bounds.IsValid)
			{
				const SDepthBar::FDepthSpace DepthSpace = GetDepthSpace();
				const double BoundsMin = DepthSpace.RelativeToBoundsPosition(0.0f);
				const double BoundsMax = DepthSpace.RelativeToBoundsPosition(1.0f);
				
				Min = BoundsMin;
				Max = BoundsMax;
			}
		}
	
		SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	}
	
private:
	TOptional<float> GetMin() const
	{
		return Min < Max ? Min : Max;
	}
	
	TOptional<float> GetMax() const
	{
		return Min < Max ? Max : Min;
	}
	
	TOptional<double> GetRawValue() const
	{
		if (TSharedPtr<FEditorViewportClient> Client = WeakClient.Pin())
		{
			if (Mode == EMode::Far)
			{
				return Client->GetOrthographicFarPlaneOverride();
			}
			return Client->GetOrthographicNearPlaneOverride();
		}
	
		return TOptional<double>();
	}
	
	float GetValue() const
	{
		TOptional<double> ValueD = GetRawValue();
		if (const double* Value = ValueD.GetPtrOrNull())
		{
			// The overrides are "aligned" offsets relative to the camera position
			// we want to present them as world coordinates.
			const SDepthBar::FDepthSpace DepthSpace = GetDepthSpace();
			const double RelativePosition = DepthSpace.AlignedToRelativePosition(*Value);
			return static_cast<float>(DepthSpace.RelativeToBoundsPosition(RelativePosition));
		}
		return Mode == EMode::Far ? Max : Min;
	}
	
	void OnValueChanged(float NewValue)
	{
		if (TSharedPtr<FEditorViewportClient> Client = WeakClient.Pin())
		{
			const SDepthBar::FDepthSpace DepthSpace = GetDepthSpace();
			const double RelativePosition = DepthSpace.BoundsToRelativePosition(NewValue);
			
			if (Mode == EMode::Near && FMath::IsNearlyEqual(RelativePosition, 0.0))
			{
				Client->SetOrthographicNearPlaneOverride(TOptional<double>());
			}
			else if (Mode == EMode::Far && FMath::IsNearlyEqual(RelativePosition, 1.0f))
			{
				Client->SetOrthographicFarPlaneOverride(TOptional<double>());
			}
			else
			{
				const double AlignedPosition = DepthSpace.RelativeToAlignedPosition(RelativePosition);
			
				if (Mode == EMode::Far)
				{
					Client->SetOrthographicFarPlaneOverride(AlignedPosition);
					if (const double* NearPlane = Client->GetOrthographicNearPlaneOverride().GetPtrOrNull())
					{
						if (*NearPlane >= AlignedPosition)
						{
							Client->SetOrthographicNearPlaneOverride(AlignedPosition - 1.0f);
						}
					}
				}
				else
				{
					Client->SetOrthographicNearPlaneOverride(AlignedPosition);
					if (const double* FarPlane = Client->GetOrthographicFarPlaneOverride().GetPtrOrNull())
					{
						if (*FarPlane <= AlignedPosition)
						{
							Client->SetOrthographicFarPlaneOverride(AlignedPosition + 1.0f);
						}
					}
				}
			}
			
			Client->Invalidate();
		}
	}
	
	SDepthBar::FDepthSpace GetDepthSpace() const
	{
		TSharedPtr<FEditorViewportClient> Client = WeakClient.Pin();
		
		return SDepthBar::FDepthSpace(Client->GetViewLocation(), Client->GetForwardVector(), Client->GetWorldBounds());
	}
	
	TOptional<FText> GetDisplayValue(float Value) const
	{
		if (!GetRawValue().IsSet())
		{
			return TOptional<FText>(LOCTEXT("Infinity", "Infinity"));
		}
		return TOptional<FText>();
	}

	TWeakPtr<FEditorViewportClient> WeakClient;
	EMode Mode = EMode::Near;
	
	float Min = -10000.0f;
	float Max = 10000.0f;
};

} // namespace UE::UnrealEd::Private

// The accessor functions are deprecated, but the functionality remains so that licensee implementers can swap back and forth easily while transitioning.
static FAutoConsoleVariableRef CVarToolMenusViewportToolbars(
	TEXT("ToolMenusViewportToolbars"),
	UE::UnrealEd::Private::CVarToolMenusViewportToolbarsValue,
	TEXT("Control whether the new ToolMenus-based viewport toolbars are enabled across the editor. Set to 0 "
		 "to show only the old viewport toolbars. Set to 1 for side-by-side mode where both the old and new viewport "
		 "toolbars are shown. Set to 2 (default) to show only the new viewport toolbars."),
	ECVF_Default
);

namespace UE::UnrealEd
{

bool ShowOldViewportToolbars()
{
	return Private::CVarToolMenusViewportToolbarsValue <= 1;
}

bool ShowNewViewportToolbars()
{
	return Private::CVarToolMenusViewportToolbarsValue >= 1;
}

FSlateIcon GetIconFromCoordSystem(ECoordSystem InCoordSystem)
{
	if (InCoordSystem == COORD_World)
	{
		static FName WorldIcon("EditorViewport.RelativeCoordinateSystem_World");
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), WorldIcon);
	}
	else if (InCoordSystem == COORD_Parent)
	{
		static const FName ParentIcon("EditorViewport.RelativeCoordinateSystem_Parent");
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), ParentIcon);
	}
	else if (InCoordSystem == COORD_Explicit)
	{
		static const FName ExplicitIcon("EditorViewport.RelativeCoordinateSystem_Explicit");
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), ExplicitIcon);
	}
	else
	{
		static FName LocalIcon("EditorViewport.RelativeCoordinateSystem_Local");
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), LocalIcon);
	}
}

FText GetNameForCoordSystem(ECoordSystem InCoordSystem)
{
	switch (InCoordSystem)
	{
	case COORD_World:
		return LOCTEXT("COORD_World", "World Space");
	case COORD_Local:
		return LOCTEXT("COORD_Local", "Local Space");
	case COORD_Parent:
		return LOCTEXT("COORD_Parent", "Parent Space");
	case COORD_Explicit:
		return LOCTEXT("COORD_Explicit", "Explicit Space");
	default:
		return FText::GetEmpty();
	}
}

FToolMenuEntry CreateViewportToolbarTransformsSection()
{
	return CreateTransformsSubmenu();
}

FToolMenuEntry CreateTransformsSubmenu()
{
	FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
		"Transform",
		LOCTEXT("TransformsSubmenuLabel", "Transform"),
		LOCTEXT("TransformsSubmenuTooltip", "Viewport-related transforms tools"),
		FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* Submenu) -> void
			{
				{
					FToolMenuEntryToolBarData ToolBarData;
					ToolBarData.StyleNameOverride = "ViewportToolbar.TransformTools";
				
					FToolMenuSection& TransformToolsSection =
						Submenu->FindOrAddSection("TransformTools", LOCTEXT("TransformToolsLabel", "Transform Tools"));

					FToolMenuEntry SelectMode = FToolMenuEntry::InitMenuEntry(FEditorViewportCommands::Get().SelectMode);
					SelectMode.SetShowInToolbarTopLevel(true);
					SelectMode.ToolBarData = ToolBarData;
					
					TransformToolsSection.AddEntry(SelectMode);

					FToolMenuEntry TranslateMode =
						FToolMenuEntry::InitMenuEntry(FEditorViewportCommands::Get().TranslateMode);
					TranslateMode.SetShowInToolbarTopLevel(true);
					TranslateMode.ToolBarData = ToolBarData;
					TransformToolsSection.AddEntry(TranslateMode);

					FToolMenuEntry RotateMode = FToolMenuEntry::InitMenuEntry(FEditorViewportCommands::Get().RotateMode);
					RotateMode.SetShowInToolbarTopLevel(true);
					RotateMode.ToolBarData = ToolBarData;
					TransformToolsSection.AddEntry(RotateMode);

					FToolMenuEntry ScaleMode = FToolMenuEntry::InitMenuEntry(FEditorViewportCommands::Get().ScaleMode);
					ScaleMode.SetShowInToolbarTopLevel(true);
					ScaleMode.ToolBarData = ToolBarData;
					TransformToolsSection.AddEntry(ScaleMode);

					FToolMenuEntry TranslateRotateMode =
						FToolMenuEntry::InitMenuEntry(FEditorViewportCommands::Get().TranslateRotateMode);
					TranslateRotateMode.SetShowInToolbarTopLevel(true);
					TranslateRotateMode.ToolBarData = ToolBarData;
					TransformToolsSection.AddEntry(TranslateRotateMode);

					FToolMenuEntry TranslateRotate2DMode =
						FToolMenuEntry::InitMenuEntry(FEditorViewportCommands::Get().TranslateRotate2DMode);
					TranslateRotate2DMode.SetShowInToolbarTopLevel(true);
					TranslateRotate2DMode.ToolBarData = ToolBarData;
					TransformToolsSection.AddEntry(TranslateRotate2DMode);

					// Build a submenu for selecting the coordinate system to use.
					{
						if (const UUnrealEdViewportToolbarContext* const Context =
								Submenu->FindContext<UUnrealEdViewportToolbarContext>())
						{
							if (TSharedPtr<SEditorViewport> Viewport = Context->Viewport.Pin())
							{
								if (Context->bShowCoordinateSystemControls)
								{
									// Build a submenu for selecting the coordinate system to use.
									{
										TransformToolsSection.AddSeparator("CoordinateSystemSeparator");

										FToolMenuEntry& CoordinateSystemSubmenu = TransformToolsSection.AddSubMenu(
											"CoordinateSystem",
											LOCTEXT("CoordinateSystemLabel", "Coordinate System"),
											LOCTEXT("CoordinateSystemTooltip", "Select between coordinate systems"),
											FNewToolMenuDelegate::CreateLambda(
												[](UToolMenu* InSubmenu)
												{
													FToolMenuSection& UnnamedSection =
														InSubmenu->FindOrAddSection(NAME_None);

													FToolMenuEntry WorldCoords = FToolMenuEntry::InitMenuEntry(
														FEditorViewportCommands::Get().RelativeCoordinateSystem_World
													);
													UnnamedSection.AddEntry(WorldCoords);

													FToolMenuEntry LocalCoords = FToolMenuEntry::InitMenuEntry(
														FEditorViewportCommands::Get().RelativeCoordinateSystem_Local
													);
													UnnamedSection.AddEntry(LocalCoords);

													if (TSharedPtr<FEditorViewportClient> Client = UUnrealEdViewportToolbarContext::GetEditorViewportClient(InSubmenu->Context))
													{
														if (UEditorInteractiveGizmoManager::AreEditorGizmosAllowed(Client->GetModeTools()))
														{
															FToolMenuEntry ParentCoords = FToolMenuEntry::InitMenuEntry(
																FEditorViewportCommands::Get().RelativeCoordinateSystem_Parent
															);
															UnnamedSection.AddEntry(ParentCoords);

															if (UEditorInteractiveGizmoManager::IsExplicitModeEnabled())
															{
																FToolMenuEntry ExplicitCoords = FToolMenuEntry::InitMenuEntry(
																   FEditorViewportCommands::Get().RelativeCoordinateSystem_Explicit
															   );
															   UnnamedSection.AddEntry(ExplicitCoords);
															}
														}
													}
												}
											)
										);

										// Set the icon based on the current coordinate system and fall back to the Local icon.
										{
											CoordinateSystemSubmenu.Icon = TAttribute<FSlateIcon>::CreateLambda(
												[WeakViewport = Viewport.ToWeakPtr()]() -> FSlateIcon
												{
													ECoordSystem CoordSystem = ECoordSystem::COORD_Local;
													if (const TSharedPtr<SEditorViewport> EditorViewport =
															WeakViewport.Pin())
													{
														CoordSystem =
															EditorViewport->GetViewportClient()->GetWidgetCoordSystemSpace();
													}
													return GetIconFromCoordSystem(CoordSystem);
												}
											);
										}

										// Have the Tooltip show the current hotkey(s) for cycling coord spaces
										{
											CoordinateSystemSubmenu.ToolTip = TAttribute<FText>::CreateLambda(
												[WeakViewport = Viewport.ToWeakPtr()]()
												{
													ECoordSystem CoordSystem = ECoordSystem::COORD_Local;
													if (const TSharedPtr<SEditorViewport> EditorViewport = WeakViewport.Pin())
													{
														CoordSystem = EditorViewport->GetViewportClient()->GetWidgetCoordSystemSpace();
													}
													
													FText CoordSystemText = GetNameForCoordSystem(CoordSystem);
												
													const FInputChord& PrimaryChord =
														*FEditorViewportCommands::Get().CycleTransformGizmoCoordSystem->GetActiveChord(
															EMultipleKeyBindingIndex::Primary
														);

													const FInputChord& SecondaryChord =
														*FEditorViewportCommands::Get().CycleTransformGizmoCoordSystem->GetActiveChord(
															EMultipleKeyBindingIndex::Secondary
														);

													if (PrimaryChord.IsValidChord() || SecondaryChord.IsValidChord())
													{

														// Both Chords are available
														if (PrimaryChord.IsValidChord() && SecondaryChord.IsValidChord())
														{
															FFormatNamedArguments Args;
															Args.Add(TEXT("PrimaryChord"), PrimaryChord.GetInputText());
															Args.Add(TEXT("SecondaryChord"), SecondaryChord.GetInputText());

															return FText::Format(
																LOCTEXT(
																	"CoordinateSystemTooltipWithBothChords", "Select between coordinate systems. \n{PrimaryChord} or {SecondaryChord} to cycle between them."
																),
																Args
															);
														}

														// If we got here, only one chord is available (primary or secondary)
														FText ChordText;
														if (PrimaryChord.IsValidChord())
														{
															ChordText = PrimaryChord.GetInputText();
														}
														else if (SecondaryChord.IsValidChord())
														{
															ChordText = SecondaryChord.GetInputText();
														}

														return FText::Format(
															LOCTEXT(
																"CoordinateSystemTooltipSingleChord", "{0} Coordinates\n Click or press {1} to cycle between coordinate systems."
															),
															CoordSystemText,
															ChordText
														);
													}

													return FText::Format(
														LOCTEXT(
															"CoordinateSystemTooltipNoChords", "{0} Coordinates\n Click to cycle between coordinate systems."
														),
														CoordSystemText
													);
												}
											);
										}

										FToolUIAction CycleCoordSystemAction;
										{
											CycleCoordSystemAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda(
												[](const FToolMenuContext& InContext)
												{
													if (const FUIAction* Action = InContext.GetActionForCommand(FEditorViewportCommands::Get().CycleTransformGizmoCoordSystem))
													{
														Action->Execute();
													}
												});
										}

										CoordinateSystemSubmenu.ToolBarData = ToolBarData;
										CoordinateSystemSubmenu.ToolBarData.LabelOverride = FText();
										CoordinateSystemSubmenu.ToolBarData.ActionOverride = CycleCoordSystemAction;
										CoordinateSystemSubmenu.SetShowInToolbarTopLevel(true);
									}
								}
							}
						}
					}
				}

				{
					FToolMenuSection& GizmoSection = Submenu->FindOrAddSection("Gizmo", LOCTEXT("GizmoLabel", "Gizmo"));

					Private::AddGizmoSizeSlider(GizmoSection);

					{
						FUIAction PreserveNonUniformScaleAction;
						PreserveNonUniformScaleAction.ExecuteAction.BindLambda([]
						{
							ULevelEditorViewportSettings* Settings = GetMutableDefault<ULevelEditorViewportSettings>();
							Settings->PreserveNonUniformScale = !Settings->PreserveNonUniformScale;
						});
						PreserveNonUniformScaleAction.GetActionCheckState.BindLambda([]
						{
							return GetDefault<ULevelEditorViewportSettings>()->PreserveNonUniformScale ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
						});
						
						GizmoSection.AddMenuEntry(
							"PreserveNonUniformScale",
							LOCTEXT("ScaleGridPreserveNonUniformScale", "Preserve Non-Uniform Scale"),
							LOCTEXT(
								"ScaleGridPreserveNonUniformScale_ToolTip",
								"When this option is checked, scaling objects that have a non-uniform scale will preserve the ratios between each axis, snapping the axis with the largest value."
							),
							FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.PreserveNonUniformScale"),
							PreserveNonUniformScaleAction,
							EUserInterfaceActionType::ToggleButton
						);
					}
				}
			}
		)
	);
	Entry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.SelectMode");
	Entry.ToolBarData.LabelOverride = FText();
	Entry.ToolBarData.ResizeParams.ClippingPriority = 1000;
	Entry.StyleNameOverride = "ViewportToolbar.TransformTools";

	return Entry;
}

// The code in this function was moved to FLevelEditorMenu::RegisterSelectMenu.
// This function just duplicates that until it is removed after its deprecation period.
FToolMenuEntry CreateViewportToolbarSelectSection()
{
	FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
		"Select",
		LOCTEXT("SelectonSubmenuLabel", "Select"),
		LOCTEXT("SelectionSubmenuTooltip", "Viewport-related selection tools"),
		FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* Submenu) -> void
			{
				{
					FToolMenuSection& UnnamedSection = Submenu->FindOrAddSection(NAME_None);

					UnnamedSection.AddMenuEntry(
						FGenericCommands::Get().SelectAll,
						FGenericCommands::Get().SelectAll->GetLabel(),
						FGenericCommands::Get().SelectAll->GetDescription(),
						FSlateIconFinder::FindIcon("FoliageEditMode.SelectAll")
					);

					UnnamedSection.AddMenuEntry(
						FLevelEditorCommands::Get().SelectNone,
						FLevelEditorCommands::Get().SelectNone->GetLabel(),
						FLevelEditorCommands::Get().SelectNone->GetDescription(),
						FSlateIconFinder::FindIcon("Cross")
					);

					UnnamedSection.AddMenuEntry(
						FLevelEditorCommands::Get().InvertSelection,
						FLevelEditorCommands::Get().InvertSelection->GetLabel(),
						FLevelEditorCommands::Get().InvertSelection->GetDescription(),
						FSlateIconFinder::FindIcon("FoliageEditMode.DeselectAll")
					);

					// Hierarchy based selection
					{
						UnnamedSection.AddSubMenu(
							"Hierarchy",
							LOCTEXT("HierarchyLabel", "Hierarchy"),
							LOCTEXT("HierarchyTooltip", "Hierarchy selection tools"),
							FNewToolMenuDelegate::CreateLambda(
								[](UToolMenu* HierarchyMenu)
								{
									FToolMenuSection& HierarchySection = HierarchyMenu->FindOrAddSection(
										"SelectAllHierarchy", LOCTEXT("SelectAllHierarchyLabel", "Hierarchy")
									);

									HierarchySection.AddMenuEntry(
										FLevelEditorCommands::Get().SelectImmediateChildren,
										LOCTEXT("HierarchySelectImmediateChildrenLabel", "Immediate Children")
									);

									HierarchySection.AddMenuEntry(
										FLevelEditorCommands::Get().SelectAllDescendants,
										LOCTEXT("HierarchySelectAllDescendantsLabel", "All Descendants")
									);
								}
							),
							false,
							FSlateIconFinder::FindIcon("BTEditor.SwitchToBehaviorTreeMode")
						);
					}

					UnnamedSection.AddSeparator("Advanced");

					UnnamedSection.AddMenuEntry(
						FLevelEditorCommands::Get().SelectAllActorsOfSameClass,
						LOCTEXT("AdvancedSelectAllActorsOfSameClassLabel", "All of Same Class"),
						FLevelEditorCommands::Get().SelectAllActorsOfSameClass->GetDescription(),
						FSlateIconFinder::FindIcon("PlacementBrowser.Icons.All")
					);
				}

				{
					FToolMenuSection& ByTypeSection =
						Submenu->FindOrAddSection("ByTypeSection", LOCTEXT("ByTypeSectionLabel", "By Type"));

					ByTypeSection.AddSubMenu(
						"BSP",
						LOCTEXT("BspLabel", "BSP"),
						LOCTEXT("BspTooltip", "BSP-related tools"),
						FNewToolMenuDelegate::CreateLambda(
							[](UToolMenu* BspMenu)
							{
								FToolMenuSection& SelectAllSection = BspMenu->FindOrAddSection(
									"SelectAllBSP", LOCTEXT("SelectAllBSPLabel", "Select All BSP")
								);

								SelectAllSection.AddMenuEntry(
									FLevelEditorCommands::Get().SelectAllAddditiveBrushes,
									LOCTEXT("BSPSelectAllAdditiveBrushesLabel", "Addditive Brushes")
								);

								SelectAllSection.AddMenuEntry(
									FLevelEditorCommands::Get().SelectAllSubtractiveBrushes,
									LOCTEXT("BSPSelectAllSubtractiveBrushesLabel", "Subtractive Brushes")
								);

								SelectAllSection.AddMenuEntry(
									FLevelEditorCommands::Get().SelectAllSurfaces,
									LOCTEXT("BSPSelectAllAllSurfacesLabel", "Surfaces")
								);
							}
						),
						false,
						FSlateIconFinder::FindIcon("ShowFlagsMenu.BSP")
					);

					ByTypeSection.AddSubMenu(
						"Emitters",
						LOCTEXT("EmittersLabel", "Emitters"),
						LOCTEXT("EmittersTooltip", "Emitters-related tools"),
						FNewToolMenuDelegate::CreateLambda(
							[](UToolMenu* EmittersMenu)
							{
								FToolMenuSection& SelectAllSection = EmittersMenu->FindOrAddSection(
									"SelectAllEmitters", LOCTEXT("SelectAllEmittersLabel", "Select All Emitters")
								);

								SelectAllSection.AddMenuEntry(
									FLevelEditorCommands::Get().SelectMatchingEmitter,
									LOCTEXT("EmittersSelectMatchingEmitterLabel", "Matching Emitters")
								);
							}
						),
						false,
						FSlateIconFinder::FindIcon("ClassIcon.Emitter")
					);

					ByTypeSection.AddSubMenu(
						"GeometryCollections",
						LOCTEXT("GeometryCollectionsLabel", "Geometry Collections"),
						LOCTEXT("GeometryCollectionsTooltip", "GeometryCollections-related tools"),
						FNewToolMenuDelegate::CreateLambda(
							[](UToolMenu* GeometryCollectionsMenu)
							{
								// This one will be filled by extensions from GeometryCollectionEditorPlugin
								// Hook is "SelectGeometryCollections"
								FToolMenuSection& SelectAllSection = GeometryCollectionsMenu->FindOrAddSection(
									"SelectGeometryCollections",
									LOCTEXT("SelectGeometryCollectionsLabel", "Geometry Collections")
								);
							}
						),
						false,
						FSlateIconFinder::FindIcon("ClassIcon.GeometryCollection")
					);

					ByTypeSection.AddSubMenu(
						"HLOD",
						LOCTEXT("HLODLabel", "HLOD"),
						LOCTEXT("HLODTooltip", "HLOD-related tools"),
						FNewToolMenuDelegate::CreateLambda(
							[](UToolMenu* HLODMenu)
							{
								FToolMenuSection& SelectAllSection = HLODMenu->FindOrAddSection(
									"SelectAllHLOD", LOCTEXT("SelectAllHLODLabel", "Select All HLOD")
								);

								SelectAllSection.AddMenuEntry(
									FLevelEditorCommands::Get().SelectOwningHierarchicalLODCluster,
									LOCTEXT("HLODSelectOwningHierarchicalLODClusterLabel", "Owning HLOD Cluster")
								);
							}
						),
						false,
						FSlateIconFinder::FindIcon("WorldPartition.ShowHLODActors")
					);

					ByTypeSection.AddSubMenu(
						"Lights",
						LOCTEXT("LightsLabel", "Lights"),
						LOCTEXT("LightsTooltip", "Lights-related tools"),
						FNewToolMenuDelegate::CreateLambda(
							[](UToolMenu* LightsMenu)
							{
								FToolMenuSection& SelectAllSection = LightsMenu->FindOrAddSection(
									"SelectAllLights", LOCTEXT("SelectAllLightsLabel", "Select All Lights")
								);

								SelectAllSection.AddMenuEntry(
									FLevelEditorCommands::Get().SelectAllLights,
									LOCTEXT("LightsSelectAllLightsLabel", "All Lights")
								);

								SelectAllSection.AddMenuEntry(
									FLevelEditorCommands::Get().SelectRelevantLights,
									LOCTEXT("LightsSelectRelevantLightsLabel", "Relevant Lights")
								);

								SelectAllSection.AddMenuEntry(
									FLevelEditorCommands::Get().SelectStationaryLightsExceedingOverlap,
									LOCTEXT("LightsSelectStationaryLightsExceedingOverlapLabel", "Stationary Lights Exceeding Overlap")
								);
							}
						),
						false,
						FSlateIconFinder::FindIcon("PlacementBrowser.Icons.Lights")
					);

					ByTypeSection.AddSubMenu(
						"Material",
						LOCTEXT("MaterialLabel", "Material"),
						LOCTEXT("MaterialTooltip", "Material-related tools"),
						FNewToolMenuDelegate::CreateLambda(
							[](UToolMenu* MaterialMenu)
							{
								FToolMenuSection& SelectAllSection = MaterialMenu->FindOrAddSection(
									"SelectAllMaterial", LOCTEXT("SelectAllMaterialLabel", "Select All Material")
								);

								SelectAllSection.AddMenuEntry(
									FLevelEditorCommands::Get().SelectAllWithSameMaterial,
									LOCTEXT("MaterialSelectAllWithSameMaterialLabel", "With Same Material")
								);
							}
						),
						false,
						FSlateIconFinder::FindIcon("ClassIcon.Material")
					);

					ByTypeSection.AddSubMenu(
						"SkeletalMeshes",
						LOCTEXT("SkeletalMeshesLabel", "Skeletal Meshes"),
						LOCTEXT("SkeletalMeshesTooltip", "SkeletalMeshes-related tools"),
						FNewToolMenuDelegate::CreateLambda(
							[](UToolMenu* SkeletalMeshesMenu)
							{
								FToolMenuSection& SelectAllSection = SkeletalMeshesMenu->FindOrAddSection(
									"SelectAllSkeletalMeshes",
									LOCTEXT("SelectAllSkeletalMeshesLabel", "Select All SkeletalMeshes")
								);

								SelectAllSection.AddMenuEntry(
									FLevelEditorCommands::Get().SelectSkeletalMeshesOfSameClass,
									LOCTEXT(
										"SkeletalMeshesSelectSkeletalMeshesOfSameClassLabel",
										"Using Selected Skeletal Meshes (Selected Actor Types)"
									)
								);

								SelectAllSection.AddMenuEntry(
									FLevelEditorCommands::Get().SelectSkeletalMeshesAllClasses,
									LOCTEXT(
										"SkeletalMeshesSelectSkeletalMeshesAllClassesLabel",
										"Using Selected Skeletal Meshes (All Actor Types)"
									)
								);
							}
						),
						false,
						FSlateIconFinder::FindIcon("SkeletonTree.Bone")
					);

					ByTypeSection.AddSubMenu(
						"StaticMeshes",
						LOCTEXT("StaticMeshesLabel", "Static Meshes"),
						LOCTEXT("StaticMeshesTooltip", "StaticMeshes-related tools"),
						FNewToolMenuDelegate::CreateLambda(
							[](UToolMenu* StaticMeshesMenu)
							{
								FToolMenuSection& SelectAllSection = StaticMeshesMenu->FindOrAddSection(
									"SelectAllStaticMeshes", LOCTEXT("SelectAllStaticMeshesLabel", "Select All StaticMeshes")
								);

								SelectAllSection.AddMenuEntry(
									FLevelEditorCommands::Get().SelectStaticMeshesOfSameClass,
									LOCTEXT("StaticMeshesSelectStaticMeshesOfSameClassLabel", "Matching Selected Class")
								);

								SelectAllSection.AddMenuEntry(
									FLevelEditorCommands::Get().SelectStaticMeshesAllClasses,
									LOCTEXT("StaticMeshesSelectStaticMeshesAllClassesLabel", "Matching All Classes")
								);
							}
						),
						false,
						FSlateIconFinder::FindIcon("ShowFlagsMenu.StaticMeshes")
					);
				}
			}
		)
	);

	Entry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.SelectMode");
	return Entry;
}

FToolMenuEntry CreateViewportToolbarSnappingSubmenu()
{
	return CreateSnappingSubmenu();
}

FToolMenuEntry CreateSnappingSubmenu()
{
	FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
		"Snapping",
		LOCTEXT("SnappingSubmenuLabel", "Snapping"),
		LOCTEXT("SnappingSubmenuTooltip", "Viewport-related snapping settings"),
		FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* Submenu) -> void
			{
				FToolMenuSection& SnappingSection =
					Submenu->FindOrAddSection("Snapping", LOCTEXT("SnappingLabel", "Snapping"));

				SnappingSection.AddEntry(Private::CreateSurfaceSnapCheckboxMenu(Submenu->Context)).SetShowInToolbarTopLevel(true);
				SnappingSection.AddEntry(Private::CreateLocationSnapCheckboxMenu(Submenu->Context)).SetShowInToolbarTopLevel(true);
				SnappingSection.AddEntry(Private::CreateRotationSnapCheckboxMenu(Submenu->Context)).SetShowInToolbarTopLevel(true);
				SnappingSection.AddEntry(Private::CreateScaleSnapCheckboxMenu(Submenu->Context)).SetShowInToolbarTopLevel(true);
				SnappingSection.AddEntry(Private::CreateLayer2DSnapCheckboxMenu(Submenu->Context)).SetShowInToolbarTopLevel(true);
			}
		)
	);

	Entry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ViewportToolbar.Snap");
	Entry.ToolBarData.LabelOverride = FText();
	Entry.ToolBarData.ResizeParams.ClippingPriority = 900;
	return Entry;
}

FText GetViewModesSubmenuLabel(TWeakPtr<SEditorViewport> InViewport)
{
	FText Label = LOCTEXT("ViewMenuTitle_Default", "View");
	if (TSharedPtr<SEditorViewport> PinnedViewport = InViewport.Pin())
	{
		const TSharedPtr<FEditorViewportClient> ViewportClient = PinnedViewport->GetViewportClient();
		check(ViewportClient.IsValid());
		const EViewModeIndex ViewMode = ViewportClient->GetViewMode();
		// If VMI_VisualizeBuffer, return its subcategory name
		if (ViewMode == VMI_VisualizeBuffer)
		{
			Label = ViewportClient->GetCurrentBufferVisualizationModeDisplayName();
		}
		else if (ViewMode == VMI_VisualizeNanite)
		{
			Label = ViewportClient->GetCurrentNaniteVisualizationModeDisplayName();
		}
		else if (ViewMode == VMI_VisualizeLumen)
		{
			Label = ViewportClient->GetCurrentLumenVisualizationModeDisplayName();
		}
		else if (ViewMode == VMI_VisualizeMegaLights)
		{
			Label = ViewportClient->GetCurrentMegaLightsVisualizationModeDisplayName();
		}
		else if (ViewMode == VMI_VisualizeSubstrate)
		{
			Label = ViewportClient->GetCurrentSubstrateVisualizationModeDisplayName();
		}
		else if (ViewMode == VMI_VisualizeGroom)
		{
			Label = ViewportClient->GetCurrentGroomVisualizationModeDisplayName();
		}
		else if (ViewMode == VMI_VisualizeVirtualShadowMap)
		{
			Label = ViewportClient->GetCurrentVirtualShadowMapVisualizationModeDisplayName();
		}
		else if (ViewMode == VMI_VisualizeVirtualTexture)
		{
			Label = ViewportClient->GetCurrentVirtualTextureVisualizationModeDisplayName();
		}
		else if (ViewMode == VMI_VisualizeActorColoration)
		{
			Label = ViewportClient->GetCurrentActorColorationVisualizationModeDisplayName();
		}
		else if (ViewMode == VMI_VisualizeGPUSkinCache)
		{
			Label = ViewportClient->GetCurrentGPUSkinCacheVisualizationModeDisplayName();
		}
		// For any other category, return its own name
		else
		{
			Label = UViewModeUtils::GetViewModeDisplayName(ViewMode);
		}
	}

	return Label;
}

FText GetViewModesSubmenuLabel(const UGameViewportClient* InViewportClient)
{
	FText Label = LOCTEXT("ViewMenuTitlePIE_Default", "View");
	if (InViewportClient)
	{
		Label = UViewModeUtils::GetViewModeDisplayName((EViewModeIndex)InViewportClient->ViewModeIndex);
	}
	return Label;
}

FToolMenuEntry GetMeshEdgesMenuEntry(const TSharedPtr<SEditorViewport>& EditorViewport)
{
	const FNewToolMenuDelegate OpacityMenu = FNewToolMenuDelegate::CreateLambda(
		[EditorViewportWeak = EditorViewport.ToWeakPtr()](UToolMenu* Submenu)
		{
			const TSharedPtr<SEditorViewport>& EditorViewport = EditorViewportWeak.Pin();
			if (!EditorViewport)
			{
				return;
			}

			Submenu->AddMenuEntry("WireframeOpacitySubmenu", FToolMenuEntry::InitWidget(
				"WireframeOpacity", EditorViewport->BuildWireframeMenu(), LOCTEXT("WireframeOpacity", "Opacity")
			));
		}
	);
	
	const TSharedPtr<const FUICommandInfo> MeshCommand = FShowFlagMenuCommands::Get().FindCommandForFlag(FEngineShowFlags::EShowFlag::SF_MeshEdges);
	const FToolUIActionChoice ShowMeshMenuAction = FToolUIActionChoice(MeshCommand, *EditorViewport->GetCommandList());
	const FSlateIcon& Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.MeshEdges");

	FToolMenuEntry MeshMenuEntry = FToolMenuEntry::InitSubMenu(
		"MeshEdgesSubMenu",
		LOCTEXT("ViewMenuMeshEdges", "Mesh edges"),
		MeshCommand->GetDescription(),
		OpacityMenu,
		ShowMeshMenuAction,
		EUserInterfaceActionType::ToggleButton,
		false,
		Icon
	);
	MeshMenuEntry.InputBindingLabel = MeshCommand->GetInputText();
	return MeshMenuEntry;
}

FSlateIcon GetViewModesSubmenuIcon(const TWeakPtr<SEditorViewport>& InViewport)
{
	if (TSharedPtr<SEditorViewport> PinnedViewport = InViewport.Pin())
	{
		const TSharedPtr<FEditorViewportClient> ViewportClient = PinnedViewport->GetViewportClient();
		check(ViewportClient.IsValid());

		return UViewModeUtils::GetViewModeDisplaySlateIcon(ViewportClient->GetViewMode());
	}

	return FSlateIcon();
}

FSlateIcon GetViewModesSubmenuIcon(const UGameViewportClient* InViewportClient)
{
	if (InViewportClient)
	{	
		return UViewModeUtils::GetViewModeDisplaySlateIcon((EViewModeIndex)InViewportClient->ViewModeIndex);
	}
	return FSlateIcon();
}

FGetActionCheckState BuildParentCheckStateForCommands(
	const FToolMenuContext& Context,
	const TArray<TSharedPtr<FUICommandInfo>>& Commands
)
{
	FGetActionCheckState Result;
	
	TSharedPtr<const FUICommandList> CommandList;
	
	for (const TSharedPtr<FUICommandInfo>& Command : Commands)
	{
		Context.GetActionForCommand(Command, CommandList);
		if (CommandList)
		{
			break;
		}
	}
	
	if (CommandList)
	{
		Result.BindLambda([WeakCommandList = CommandList.ToWeakPtr(), Commands = Commands]
		{
			if (TSharedPtr<const FUICommandList> CommandList = WeakCommandList.Pin())
			{
				for (const TSharedPtr<FUICommandInfo>& Command : Commands)
				{
					if (const FUIAction* Action = CommandList->GetActionForCommand(Command))
					{
						if (Action->GetCheckState() != ECheckBoxState::Unchecked)
						{
							return ECheckBoxState::Checked;
						}
					}
				}
			}
			return ECheckBoxState::Unchecked;
		});
	}
	
	return Result;
}

void PopulateViewModesMenu(UToolMenu* InMenu)
{
	UUnrealEdViewportToolbarContext* const Context = InMenu->FindContext<UUnrealEdViewportToolbarContext>();
	if (!Context)
	{
		return;
	}

	const TSharedPtr<SEditorViewport> EditorViewport = Context->Viewport.Pin();

	if (!EditorViewport)
	{
		return;
	}

	const FEditorViewportCommands& BaseViewportActions = FEditorViewportCommands::Get();

	IsViewModeSupportedDelegate IsViewModeSupported = Context->IsViewModeSupported;

	// View modes
	{
		FToolMenuSection& Section = InMenu->AddSection("ViewMode", LOCTEXT("ViewModeHeader", "View Mode"));
		{
			UE::UnrealEd::Private::ViewModesSubmenu::AddModeIfSupported(
				IsViewModeSupported, Section, BaseViewportActions.LitMode, VMI_Lit
			);
			UE::UnrealEd::Private::ViewModesSubmenu::AddModeIfSupported(
				IsViewModeSupported, Section, BaseViewportActions.UnlitMode, VMI_Unlit
			);
			UE::UnrealEd::Private::ViewModesSubmenu::AddModeIfSupported(
				IsViewModeSupported, Section, BaseViewportActions.WireframeMode, VMI_BrushWireframe
			);
			UE::UnrealEd::Private::ViewModesSubmenu::AddModeIfSupported(
				IsViewModeSupported, Section, BaseViewportActions.DetailLightingMode, VMI_Lit_DetailLighting
			);
			UE::UnrealEd::Private::ViewModesSubmenu::AddModeIfSupported(
				IsViewModeSupported, Section, BaseViewportActions.LightingOnlyMode, VMI_LightingOnly
			);
			UE::UnrealEd::Private::ViewModesSubmenu::AddModeIfSupported(
				IsViewModeSupported, Section, BaseViewportActions.ReflectionOverrideMode, VMI_ReflectionOverride
			);
			UE::UnrealEd::Private::ViewModesSubmenu::AddModeIfSupported(
				IsViewModeSupported, Section, BaseViewportActions.CollisionPawn, VMI_CollisionPawn
			);
			UE::UnrealEd::Private::ViewModesSubmenu::AddModeIfSupported(
				IsViewModeSupported, Section, BaseViewportActions.CollisionVisibility, VMI_CollisionVisibility
			);
		}

		if (IsRayTracingEnabled())
		{
			static auto PathTracingCvar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.PathTracing"));
			const bool bPathTracingSupported = FDataDrivenShaderPlatformInfo::GetSupportsPathTracing(GMaxRHIShaderPlatform);
			const bool bPathTracingEnabled = PathTracingCvar && PathTracingCvar->GetValueOnAnyThread() != 0;
			if (bPathTracingSupported && bPathTracingEnabled)
			{
				UE::UnrealEd::Private::ViewModesSubmenu::AddModeIfSupported(
					IsViewModeSupported, Section, BaseViewportActions.PathTracingMode, VMI_PathTracing
				);
			}
		}

		FToolMenuSection& AdvancedSection = InMenu->AddSection("Advanced_CustomSection", LOCTEXT("AdvancedHeader", "Advanced"));
		AdvancedSection.AddEntry(GetMeshEdgesMenuEntry(EditorViewport));
		AdvancedSection.InsertPosition = FToolMenuInsert("ViewMode", EToolMenuInsertType::After);

		// Optimization
		{
			struct Local
			{
				static void BuildOptimizationMenu(UToolMenu* Menu, IsViewModeSupportedDelegate IsViewModeSupported)
				{
					const FEditorViewportCommands& BaseViewportActions = FEditorViewportCommands::Get();

					UWorld* World = GWorld;
					const ERHIFeatureLevel::Type FeatureLevel = (IsInGameThread() && World)
																  ? (ERHIFeatureLevel::Type)World->GetFeatureLevel()
																  : GMaxRHIFeatureLevel;

					{
						FToolMenuSection& Section = Menu->AddSection(
							"OptimizationViewmodes", LOCTEXT("OptimizationSubMenuHeader", "Optimization Viewmodes")
						);
						if (FeatureLevel >= ERHIFeatureLevel::SM5)
						{
							UE::UnrealEd::Private::ViewModesSubmenu::AddModeIfSupported(
								IsViewModeSupported, Section, BaseViewportActions.LightComplexityMode, VMI_LightComplexity
							);

							if (IsStaticLightingAllowed())
							{
								UE::UnrealEd::Private::ViewModesSubmenu::AddModeIfSupported(
									IsViewModeSupported, Section, BaseViewportActions.LightmapDensityMode, VMI_LightmapDensity
								);
							}

							UE::UnrealEd::Private::ViewModesSubmenu::AddModeIfSupported(
								IsViewModeSupported, Section, BaseViewportActions.StationaryLightOverlapMode, VMI_StationaryLightOverlap
							);
						}

						UE::UnrealEd::Private::ViewModesSubmenu::AddModeIfSupported(
							IsViewModeSupported, Section, BaseViewportActions.ShaderComplexityMode, VMI_ShaderComplexity
						);

						if (AllowDebugViewShaderMode(
								DVSM_ShaderComplexityContainedQuadOverhead, GMaxRHIShaderPlatform, FeatureLevel
							))
						{
							UE::UnrealEd::Private::ViewModesSubmenu::AddModeIfSupported(
								IsViewModeSupported,
								Section,
								BaseViewportActions.ShaderComplexityWithQuadOverdrawMode,
								VMI_ShaderComplexityWithQuadOverdraw
							);
						}

						if (AllowDebugViewShaderMode(DVSM_QuadComplexity, GMaxRHIShaderPlatform, FeatureLevel))
						{
							UE::UnrealEd::Private::ViewModesSubmenu::AddModeIfSupported(
								IsViewModeSupported, Section, BaseViewportActions.QuadOverdrawMode, VMI_QuadOverdraw
							);
						}

						if (AllowDebugViewShaderMode(DVSM_LWCComplexity, GMaxRHIShaderPlatform, FeatureLevel))
						{
							UE::UnrealEd::Private::ViewModesSubmenu::AddModeIfSupported(
								IsViewModeSupported,
								Section,
								BaseViewportActions.VisualizeLWCComplexity,
								VMI_LWCComplexity,
								TAttribute<FText>(),
								FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.LWCComplexityMode")
							);
						}
					}

					{
						FToolMenuSection& Section = Menu->AddSection(
							"TextureStreaming", LOCTEXT("TextureStreamingHeader", "Texture Streaming Accuracy")
						);

						if (AllowDebugViewShaderMode(DVSM_PrimitiveDistanceAccuracy, GMaxRHIShaderPlatform, FeatureLevel))
						{
							UE::UnrealEd::Private::ViewModesSubmenu::AddModeIfSupported(
								IsViewModeSupported, Section, BaseViewportActions.TexStreamAccPrimitiveDistanceMode, VMI_PrimitiveDistanceAccuracy
							);
						}

						if (AllowDebugViewShaderMode(DVSM_MeshUVDensityAccuracy, GMaxRHIShaderPlatform, FeatureLevel))
						{
							UE::UnrealEd::Private::ViewModesSubmenu::AddModeIfSupported(
								IsViewModeSupported, Section, BaseViewportActions.TexStreamAccMeshUVDensityMode, VMI_MeshUVDensityAccuracy
							);
						}

						// TexCoordScale accuracy viewmode requires shaders that are only built in the
						// TextureStreamingBuild, which requires the new metrics to be enabled.
						if (AllowDebugViewShaderMode(DVSM_MaterialTextureScaleAccuracy, GMaxRHIShaderPlatform, FeatureLevel)
							&& CVarStreamingUseNewMetrics.GetValueOnAnyThread() != 0)
						{
							UE::UnrealEd::Private::ViewModesSubmenu::AddModeIfSupported(
								IsViewModeSupported,
								Section,
								BaseViewportActions.TexStreamAccMaterialTextureScaleMode,
								VMI_MaterialTextureScaleAccuracy
							);
						}

						if (AllowDebugViewShaderMode(DVSM_RequiredTextureResolution, GMaxRHIShaderPlatform, FeatureLevel))
						{
							UE::UnrealEd::Private::ViewModesSubmenu::AddModeIfSupported(
								IsViewModeSupported, Section, BaseViewportActions.RequiredTextureResolutionMode, VMI_RequiredTextureResolution
							);
						}
					}
				}

				static bool ViewModesShouldShowOptimizationEntries(const IsViewModeSupportedDelegate& InIsViewModeSupported)
				{
					if (!InIsViewModeSupported.IsBound())
					{
						return true;
					}

					return InIsViewModeSupported.Execute(VMI_LightComplexity)
						|| InIsViewModeSupported.Execute(VMI_LightmapDensity)
						|| InIsViewModeSupported.Execute(VMI_StationaryLightOverlap)
						|| InIsViewModeSupported.Execute(VMI_ShaderComplexity)
						|| InIsViewModeSupported.Execute(VMI_ShaderComplexityWithQuadOverdraw)
						|| InIsViewModeSupported.Execute(VMI_QuadOverdraw)
						|| InIsViewModeSupported.Execute(VMI_PrimitiveDistanceAccuracy)
						|| InIsViewModeSupported.Execute(VMI_MeshUVDensityAccuracy)
						|| InIsViewModeSupported.Execute(VMI_MaterialTextureScaleAccuracy)
						|| InIsViewModeSupported.Execute(VMI_RequiredTextureResolution);
				}
			};

			if (Local::ViewModesShouldShowOptimizationEntries(IsViewModeSupported))
			{
				FUIAction MenuAction;
				MenuAction.GetActionCheckState = BuildParentCheckStateForCommands(InMenu->Context, {
					BaseViewportActions.LightComplexityMode,
					BaseViewportActions.LightmapDensityMode,
					BaseViewportActions.StationaryLightOverlapMode,
					BaseViewportActions.ShaderComplexityMode,
					BaseViewportActions.ShaderComplexityWithQuadOverdrawMode,
					BaseViewportActions.QuadOverdrawMode,
					BaseViewportActions.TexStreamAccPrimitiveDistanceMode,
					BaseViewportActions.TexStreamAccMeshUVDensityMode,
					BaseViewportActions.TexStreamAccMaterialTextureScaleMode,
					BaseViewportActions.RequiredTextureResolutionMode
				});
				
				Section.AddSubMenu(
					"OptimizationSubMenu",
					LOCTEXT("OptimizationSubMenu", "Optimization Viewmodes"),
					LOCTEXT("Optimization_ToolTip", "Select optimization visualizer"),
					FNewToolMenuDelegate::CreateStatic(&Local::BuildOptimizationMenu, IsViewModeSupported),
					MenuAction,
					EUserInterfaceActionType::RadioButton,
					/* bInOpenSubMenuOnClick = */ false,
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.QuadOverdrawMode")
				);
			}
		}

		{
			struct Local
			{
				static void BuildLODMenu(UToolMenu* Menu, IsViewModeSupportedDelegate IsViewModeSupported)
				{
					{
						FToolMenuSection& Section = Menu->AddSection(
							"LevelViewportLODColoration", LOCTEXT("LODModesHeader", "Level of Detail Coloration")
						);

						UE::UnrealEd::Private::ViewModesSubmenu::AddModeIfSupported(
							IsViewModeSupported, Section, FEditorViewportCommands::Get().LODColorationMode, VMI_LODColoration
						);

						UE::UnrealEd::Private::ViewModesSubmenu::AddModeIfSupported(
							IsViewModeSupported, Section, FEditorViewportCommands::Get().HLODColorationMode, VMI_HLODColoration
						);
					}
				}
			};

			if (!IsViewModeSupported.IsBound()
				|| (IsViewModeSupported.Execute(VMI_LODColoration) || IsViewModeSupported.Execute(VMI_HLODColoration)))
			{
				FUIAction MenuAction;
				MenuAction.GetActionCheckState = BuildParentCheckStateForCommands(InMenu->Context, {
					FEditorViewportCommands::Get().LODColorationMode,
					FEditorViewportCommands::Get().HLODColorationMode
				});
				
				Section.AddSubMenu(
					"VisualizeGroupedLOD",
					LOCTEXT("VisualizeGroupedLODDisplayName", "Level of Detail Coloration"),
					LOCTEXT("GroupedLODMenu_ToolTip", "Select a mode for LOD Coloration"),
					FNewToolMenuDelegate::CreateStatic(&Local::BuildLODMenu, IsViewModeSupported),
					MenuAction,
					EUserInterfaceActionType::RadioButton,
					/* bInOpenSubMenuOnClick = */ false,
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.GroupLODColorationMode")
				);
			}
		}

		{
			struct Local
			{
				static void BuildStreamingDeficitMenu(UToolMenu* Menu, IsViewModeSupportedDelegate IsViewModeSupported)
				{
					{
						FToolMenuSection& Section = Menu->AddSection(
							"LevelViewportStreamingDeficit", LOCTEXT("StreamingDeficitModesHeader", "Streaming Deficit")
						);

						UE::UnrealEd::Private::ViewModesSubmenu::AddModeIfSupported(
							IsViewModeSupported, Section, FEditorViewportCommands::Get().StreamingTextureDeficitMode, VMI_StreamingTextureDeficit
						);

						UE::UnrealEd::Private::ViewModesSubmenu::AddModeIfSupported(
							IsViewModeSupported, Section, FEditorViewportCommands::Get().StreamingTextureResidencyMode, VMI_StreamingTextureResidency
						);

						// Mesh LOD streaming modes are not shown in editor — mesh LODs are always fully loaded in editor.
						// Use console commands (viewmode StreamingMeshLODDeficit/StreamingMeshLODResidency) in cooked builds.
					}
				}
			};

			if (!IsViewModeSupported.IsBound()
				|| (IsViewModeSupported.Execute(VMI_StreamingTextureDeficit) || IsViewModeSupported.Execute(VMI_StreamingMeshLODDeficit)))
			{
				FUIAction MenuAction;
				MenuAction.GetActionCheckState = BuildParentCheckStateForCommands(InMenu->Context, {
					FEditorViewportCommands::Get().StreamingTextureDeficitMode,
					FEditorViewportCommands::Get().StreamingTextureResidencyMode
				});

				Section.AddSubMenu(
					"VisualizeGroupedStreamingDeficit",
					LOCTEXT("VisualizeGroupedStreamingDeficitDisplayName", "Streaming Deficit"),
					LOCTEXT("GroupedStreamingDeficitMenu_ToolTip", "Select a mode for streaming deficit visualization"),
					FNewToolMenuDelegate::CreateStatic(&Local::BuildStreamingDeficitMenu, IsViewModeSupported),
					MenuAction,
					EUserInterfaceActionType::RadioButton,
					/* bInOpenSubMenuOnClick = */ false,
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.LOD")
				);
			}
		}

		{
			struct Local
			{
				static void BuildGeometryInspectionMenu(UToolMenu* Menu,
					IsViewModeSupportedDelegate IsViewModeSupported,
					const TWeakPtr<SEditorViewport> EditorViewportWeakPtr)
				{
					{
						FToolMenuSection& Section = Menu->AddSection(
							"GeometryInspection", LOCTEXT("GeometryInspectionModesHeader", "Geometry Inspection")
						);

						UE::UnrealEd::Private::ViewModesSubmenu::AddModeIfSupported(
							IsViewModeSupported, Section, FEditorViewportCommands::Get().GeometryInspectionClayMode, VMI_Clay
						);

						FUIAction MenuAction;
						MenuAction.GetActionCheckState = BuildParentCheckStateForCommands(Menu->Context, {
							FEditorViewportCommands::Get().GeometryInspectionZebraHorizontalMode,
							FEditorViewportCommands::Get().GeometryInspectionZebraVerticalMode
						});

						Section.AddSubMenu(
							"Zebra",
							LOCTEXT("GeometryInspectionZebraDisplayName", "Zebra"),
							LOCTEXT("GeometryInspectionZebra_ToolTip", "Geometry Inspection Zebra View Modes"),
							FNewToolMenuDelegate::CreateStatic(&BuildZebraMenu, IsViewModeSupported, EditorViewportWeakPtr),
							MenuAction,
							EUserInterfaceActionType::RadioButton,
							/* bInOpenSubMenuOnClick = */ false,
							FSlateIcon()
						);

						UE::UnrealEd::Private::ViewModesSubmenu::AddModeIfSupported(
							IsViewModeSupported, Section, FEditorViewportCommands::Get().GeometryInspectionFrontBackFaceMode, VMI_FrontBackFace
						);

						UE::UnrealEd::Private::ViewModesSubmenu::AddModeIfSupported(
							IsViewModeSupported, Section, FEditorViewportCommands::Get().GeometryInspectionRandomColorMode, VMI_RandomColor
						);
					}

					{
						FToolMenuSection& Section = Menu->AddSection(
							"GeometryInspectionWireframe", LOCTEXT("GeometryInspectionWireframeHeader", "Wireframe")
						);

						Section.AddMenuEntry(FEditorViewportCommands::Get().GeometryInspectionShowWireframe);

						if (TSharedPtr<SEditorViewport> EditorViewport = EditorViewportWeakPtr.Pin())
						{
							Section.AddEntry(FToolMenuEntry::InitWidget(
						"WireframeOpacity", EditorViewport->BuildWireframeMenu(), LOCTEXT("WireframeOpacity", "Opacity")
							));
						}
					}
				}

				static void BuildZebraMenu(UToolMenu* Menu, IsViewModeSupportedDelegate IsViewModeSupported, const TWeakPtr<SEditorViewport> EditorViewportWeakPtr)
				{
					FToolMenuSection& Section = Menu->AddSection(
						"GeometryInspectionZebra", LOCTEXT("GeometryInspectionZebraModesHeader", "Zebra Mode")
					);

					Section.AddMenuEntry(FEditorViewportCommands::Get().GeometryInspectionZebraHorizontalMode);
					Section.AddMenuEntry(FEditorViewportCommands::Get().GeometryInspectionZebraVerticalMode);

					Menu->AddMenuEntry(TEXT("Zebra Configuration"), 
						FToolMenuEntry::InitWidget(
						"StripeCount",
						BuildZebraStripeCountWidget(Menu, EditorViewportWeakPtr),
					LOCTEXT("GeometryInspectionStripeCountLabel", "Stripe Count")));
				}

				static TSharedRef<SWidget> BuildZebraStripeCountWidget(UToolMenu* Menu, const TWeakPtr<SEditorViewport>& EditorViewportWeakPtr)
				{
					return SNew( SBox )
					.HAlign( HAlign_Right )
					[
						SNew( SBox )
						.Padding( FMargin(0.0f, 0.0f, 0.0f, 0.0f) )
						.WidthOverride( 100.0f )
						[
							SNew ( SBorder )
							.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
							.Padding(FMargin(1.0f))
							[
								SNew(SSpinBox<uint8>)
								.Style(&FAppStyle::Get(), "Menu.SpinBox")
								.Font( FAppStyle::GetFontStyle( TEXT( "MenuItem.Font" ) ) )
								.MinValue(1.f)
								.MaxValue(64.f)
								.MaxSliderValue(32.f)
								.SupportDynamicSliderMaxValue(false)
								.Value_Static ( &FEditorViewportClient::GetZebraViewModeStripeCount )
								.OnValueChanged_Lambda( [EditorViewportWeakPtr](uint16 NewValue)
								{
									if (TSharedPtr<SEditorViewport> EditorViewport = EditorViewportWeakPtr.Pin())
									{
										if (TSharedPtr<FEditorViewportClient> Client = StaticCastSharedPtr<FEditorViewportClient>(
											EditorViewport->GetViewportClient()))
										{
											Client->SetZebraViewModeStripeCount(NewValue);
										}
									}
								})
								.ToolTipText(LOCTEXT("WireframeOpacity_ToolTip", "Adjust number of the stripes in zebra mode."))
							]
						]
					];
				}

			};

			if (!IsViewModeSupported.IsBound()
				|| (IsViewModeSupported.Execute(VMI_Clay) || IsViewModeSupported.Execute(VMI_Zebra)
				|| (IsViewModeSupported.Execute(VMI_FrontBackFace) || IsViewModeSupported.Execute(VMI_RandomColor))))
			{
				FUIAction MenuAction;
				MenuAction.GetActionCheckState = BuildParentCheckStateForCommands(InMenu->Context, {
					FEditorViewportCommands::Get().GeometryInspectionClayMode,
					FEditorViewportCommands::Get().GeometryInspectionFrontBackFaceMode,
					FEditorViewportCommands::Get().GeometryInspectionRandomColorMode,
					FEditorViewportCommands::Get().GeometryInspectionZebraHorizontalMode,
					FEditorViewportCommands::Get().GeometryInspectionZebraVerticalMode
				});

				Section.AddSubMenu(
					"GeometryInspection",
					LOCTEXT("GeometryInspectionDisplayName", "Geometry Inspection"),
					LOCTEXT("GeometryInspection_ToolTip", "Geometry Inspection View Modes"),
					FNewToolMenuDelegate::CreateStatic(&Local::BuildGeometryInspectionMenu, IsViewModeSupported, EditorViewport.ToWeakPtr()),
					MenuAction,
					EUserInterfaceActionType::RadioButton,
					/* bInOpenSubMenuOnClick = */ false,
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.GeometryInspection")
				);
			}
		}

		if (FGPUSkinCache::IsGPUSkinCacheEnabled(GMaxRHIFeatureLevel)
			&& UE::UnrealEd::Private::ViewModesSubmenu::IsMenuSectionAvailable(
				Context, EHidableViewModeMenuSections::GPUSkinCache
			))
		{
			FUIAction MenuAction;
			if (const FUIAction* FoundAction = InMenu->Context.GetActionForCommand(FEditorViewportCommands::Get().VisualizeGPUSkinCacheMode))
			{
				MenuAction.GetActionCheckState = FoundAction->GetActionCheckState;
			}
		
			Section.AddSubMenu(
				"VisualizeGPUSkinCacheViewMode",
				LOCTEXT("VisualizeGPUSkinCacheViewModeDisplayName", "GPU Skin Cache"),
				LOCTEXT("GPUSkinCacheVisualizationMenu_ToolTip", "Select a mode for GPU Skin Cache visualization."),
				FNewMenuDelegate::CreateStatic(&FGPUSkinCacheVisualizationMenuCommands::BuildVisualisationSubMenu),
				MenuAction,
				EUserInterfaceActionType::RadioButton,
				/* bInOpenSubMenuOnClick = */ false,
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.VisualizeGPUSkinCacheMode")
			);
		}
	}
	
	// View Mode options
	{
		FToolMenuSection& Section = InMenu->AddSection("ViewModeOptions", LOCTEXT("ViewModeOptionsHeader", "View Mode Options"), FToolMenuInsert("ViewMode", EToolMenuInsertType::After));
		Section.Visibility = TAttribute<bool>::CreateLambda([WeakViewport = EditorViewport.ToWeakPtr()]
		{
			if (TSharedPtr<SEditorViewport> Viewport = WeakViewport.Pin())
			{
				if (TSharedPtr<FEditorViewportClient> Client = Viewport->GetViewportClient())
				{
					EViewModeIndex ViewMode = Client->GetViewMode();
					return ViewMode == VMI_MeshUVDensityAccuracy || ViewMode == VMI_MaterialTextureScaleAccuracy || ViewMode == VMI_RequiredTextureResolution;	
				}
			}
			return false;
		});
		
		TAttribute<FText> ViewOptionsLabel = TAttribute<FText>::CreateLambda([WeakViewport = EditorViewport.ToWeakPtr()]
		{
			if (TSharedPtr<SEditorViewport> Viewport = WeakViewport.Pin())
			{
				if (TSharedPtr<FEditorViewportClient> Client = Viewport->GetViewportClient())
				{
					return ::GetViewModeOptionsMenuLabel(Client->GetViewMode());
				}
			}
			return LOCTEXT("ViewModeOptionsLabel", "View Mode Options");
		});
		
		FToolMenuEntry& OptionsEntry = Section.AddSubMenu(
			"ViewModeOptions",
			ViewOptionsLabel,
			LOCTEXT("ViewModeOptionsToolTip", "Options for the current view mode."),
			FNewMenuDelegate::CreateLambda([WeakViewport = EditorViewport.ToWeakPtr()](FMenuBuilder& MenuBuilder)
			{
				if (TSharedPtr<SEditorViewport> Viewport = WeakViewport.Pin())
				{
					if (TSharedPtr<FEditorViewportClient> Client = Viewport->GetViewportClient())
					{
						const UWorld* World = Client->GetWorld();
						const FViewModeOptionsArgs ViewModeArgs
						{
							.ViewModeIndex = Client->GetViewMode(),
							.FeatureLevel = World ? World->GetFeatureLevel() : GMaxRHIFeatureLevel,
							.ParamNameMap = Client->GetViewModeParamNameMap(),
							.ViewportClient = Client
						};
						BuildViewModeOptionsMenu(MenuBuilder, ViewModeArgs);
					}
				}
			})
		);
					
		OptionsEntry.SetShowInToolbarTopLevel(true); 
	}
}

UUnrealEdViewportToolbarContext* CreateViewportToolbarDefaultContext(const TWeakPtr<SEditorViewport>& InViewport)
{
	UUnrealEdViewportToolbarContext* const ContextObject = NewObject<UUnrealEdViewportToolbarContext>();
	ContextObject->Viewport = InViewport;

	// Hook up our toolbar's filter for supported view modes.
	ContextObject->IsViewModeSupported =
		UE::UnrealEd::IsViewModeSupportedDelegate::CreateStatic(&UE::UnrealEd::Private::IsViewModeSupported);

	return ContextObject;
}

FToolMenuEntry CreateViewportToolbarViewModesSubmenu()
{
	return CreateViewModesSubmenu();
}

FToolMenuEntry CreateViewModesSubmenu()
{
	// This has to be a dynamic entry for the ViewModes submenu's label to be able to access the context.
	return FToolMenuEntry::InitDynamicEntry(
		"DynamicViewModes",
		FNewToolMenuSectionDelegate::CreateLambda(
			[](FToolMenuSection& InDynamicSection) -> void
			{
				// Base the label on the current view mode.
				TAttribute<FText> LabelAttribute = UE::UnrealEd::GetViewModesSubmenuLabel(nullptr);
				TAttribute<FSlateIcon> IconAttribute = TAttribute<FSlateIcon>();
				if (UUnrealEdViewportToolbarContext* const Context =
						InDynamicSection.FindContext<UUnrealEdViewportToolbarContext>())
				{
					LabelAttribute = TAttribute<FText>::CreateLambda(
						[WeakViewport = Context->Viewport]()
						{
							return UE::UnrealEd::GetViewModesSubmenuLabel(WeakViewport);
						}
					);

					IconAttribute = TAttribute<FSlateIcon>::CreateLambda(
						[WeakViewport = Context->Viewport]()
						{
							return UE::UnrealEd::GetViewModesSubmenuIcon(WeakViewport);
						}
					);
				}

				FToolMenuEntry& Entry = InDynamicSection.AddSubMenu(
					"ViewModes",
					LabelAttribute,
					LOCTEXT("ViewModesSubmenuTooltip", "View mode settings for the current viewport."),
					FNewToolMenuDelegate::CreateLambda(
						[](UToolMenu* Submenu) -> void
						{
							PopulateViewModesMenu(Submenu);
						}
					),
					false,
					IconAttribute
				);
				Entry.ToolBarData.ResizeParams.ClippingPriority = 800;
			}
		)
	);
}

TSharedRef<SWidget> BuildRotationGridCheckBoxList(
	FName InExtentionHook,
	const FText& InHeading,
	const TArray<float>& InGridSizes,
	ERotationGridMode InGridMode,
	const FRotationGridCheckboxListExecuteActionDelegate& InExecuteAction,
	const FRotationGridCheckboxListIsCheckedDelegate& InIsActionChecked,
	const TSharedPtr<FUICommandList>& InCommandList
)
{
	const FName CheckboxListMenu("RotationGridCheckboxList." + InExtentionHook.ToString());
	if (!UToolMenus::Get()->IsMenuRegistered(CheckboxListMenu))
	{
		if (UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(CheckboxListMenu, NAME_None))
		{
			FToolMenuSection& Section = Menu->AddSection(InExtentionHook, InHeading);
			for (int32 CurrGridAngleIndex = 0; CurrGridAngleIndex < InGridSizes.Num(); ++CurrGridAngleIndex)
			{
				const float CurrGridAngle = InGridSizes[CurrGridAngleIndex];

				FText MenuName = FText::Format(
					LOCTEXT("RotationGridAngle", "{0}\u00b0"), FText::AsNumber(CurrGridAngle)
				); /*degree symbol*/
				FText ToolTipText = FText::Format(
					LOCTEXT("RotationGridAngle_ToolTip", "Sets rotation grid angle to {0}"), MenuName
				); /*degree symbol*/

				Section.AddMenuEntry(
					NAME_None,
					MenuName,
					ToolTipText,
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda(
							[CurrGridAngleIndex, InGridMode, InExecuteAction]()
							{
								InExecuteAction.Execute(CurrGridAngleIndex, InGridMode);
							}
						),
						FCanExecuteAction(),
						FIsActionChecked::CreateLambda(
							[CurrGridAngleIndex, InGridMode, InIsActionChecked]()
							{
								return InIsActionChecked.Execute(CurrGridAngleIndex, InGridMode);
							}
						)
					),
					EUserInterfaceActionType::RadioButton
				);
			}
		}
	}

	FToolMenuContext MenuContext;
	{
		MenuContext.AppendCommandList(InCommandList);
	}

	return UToolMenus::Get()->GenerateWidget(CheckboxListMenu, MenuContext);
}

FText GetRotationGridLabel()
{
	return FText::Format(
		LOCTEXT("GridRotation - Number - DegreeSymbol", "{0}\u00b0"), FText::AsNumber(GEditor->GetRotGridSize().Pitch)
	);
}

TSharedRef<SWidget> CreateRotationGridSnapMenu(
	const FRotationGridCheckboxListExecuteActionDelegate& InExecuteDelegate,
	const FRotationGridCheckboxListIsCheckedDelegate& InIsCheckedDelegate,
	const TAttribute<bool>& InIsEnabledDelegate,
	const TSharedPtr<FUICommandList>& InCommandList
)
{
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();

	// clang-format off
	return SNew(SUniformGridPanel)
		.IsEnabled(InIsEnabledDelegate)
		+ SUniformGridPanel::Slot(0, 0)
		[
			UnrealEd::BuildRotationGridCheckBoxList("Common",LOCTEXT("RotationCommonText", "Rotation Increment")
				, ViewportSettings->CommonRotGridSizes, GridMode_Common,
				InExecuteDelegate,
				InIsCheckedDelegate,
				InCommandList
			)
		]
		+ SUniformGridPanel::Slot(1, 0)
		[
			UnrealEd::BuildRotationGridCheckBoxList("Div360",LOCTEXT("RotationDivisions360DegreesText", "Divisions of 360\u00b0")
				, ViewportSettings->DivisionsOf360RotGridSizes, GridMode_DivisionsOf360,
				InExecuteDelegate,
				InIsCheckedDelegate,
				InCommandList
			)
		];
	// clang-format on
}

FText GetLocationGridLabel()
{
	return FText::AsNumber(GEditor->GetGridSize());
}

TSharedRef<SWidget> CreateLocationGridSnapMenu(
	const FLocationGridCheckboxListExecuteActionDelegate& InExecuteDelegate,
	const FLocationGridCheckboxListIsCheckedDelegate& InIsCheckedDelegate,
	const TArray<float>& InGridSizes,
	const TAttribute<bool>& InIsEnabledDelegate,
	const TSharedPtr<FUICommandList>& InCommandList
)
{
	FLocationGridSnapMenuOptions MenuOptions;
	MenuOptions.MenuName = "LocationGridCheckboxList";
	MenuOptions.ExecuteDelegate = InExecuteDelegate;
	MenuOptions.IsCheckedDelegate = InIsCheckedDelegate;
	MenuOptions.IsEnabledDelegate = InIsEnabledDelegate;
	MenuOptions.CommandList = InCommandList;

	return CreateLocationGridSnapMenu(MenuOptions);
}

TSharedRef<SWidget> CreateLocationGridSnapMenu(const FLocationGridSnapMenuOptions& InMenuOptions)
{
	const FName MenuName = InMenuOptions.MenuName.IsNone() ? "LocationGridCheckboxList" : InMenuOptions.MenuName;

	if (!UToolMenus::Get()->IsMenuRegistered(MenuName))
	{
		if (UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(MenuName, NAME_None))
		{
			Menu->AddDynamicSection(
				NAME_None,
				FNewToolMenuDelegate::CreateLambda(
					[InMenuOptions](UToolMenu* InMenu)
					{
						TAttribute<bool> IsEnabledDelegate = InMenuOptions.IsEnabledDelegate;

						UnrealEd::FLocationGridCheckboxListExecuteActionDelegate ExecuteDelegate =
							InMenuOptions.ExecuteDelegate.IsBound()
								? InMenuOptions.ExecuteDelegate
								: UnrealEd::FLocationGridCheckboxListExecuteActionDelegate::CreateUObject(
									  GEditor, &UEditorEngine::SetGridSize
								  );

						UnrealEd::FLocationGridCheckboxListIsCheckedDelegate IsCheckedDelegate =
							InMenuOptions.IsCheckedDelegate.IsBound()
								? InMenuOptions.IsCheckedDelegate
								: UnrealEd::FLocationGridCheckboxListIsCheckedDelegate::CreateLambda(
									  [](int CurrGridSizeIndex)
									  {
										  const ULevelEditorViewportSettings* ViewportSettings =
											  GetDefault<ULevelEditorViewportSettings>();
										  return ViewportSettings->CurrentPosGridSize == CurrGridSizeIndex;
									  }
								  );

						FLocationGridValuesArrayDelegate GridValuesDelegate =
							InMenuOptions.GridValuesArrayDelegate.IsBound()
								? InMenuOptions.GridValuesArrayDelegate
								: FLocationGridValuesArrayDelegate::CreateLambda(
									  []()
									  {
										  const ULevelEditorViewportSettings* ViewportSettings =
											  GetDefault<ULevelEditorViewportSettings>();
										  TArray<float> GridSizes = ViewportSettings->bUsePowerOf2SnapSize
																	  ? ViewportSettings->Pow2GridSizes
																	  : ViewportSettings->DecimalGridSizes;

										  return GridSizes;
									  }
								  );

						const TArray<float> InGridSizes = GridValuesDelegate.Execute();

						FToolMenuSection& Section = InMenu->AddSection("Snap", LOCTEXT("LocationSnapText", "Snap Sizes"));

						for (int32 CurrGridSizeIndex = 0; CurrGridSizeIndex < InGridSizes.Num(); ++CurrGridSizeIndex)
						{
							const float CurGridSize = InGridSizes[CurrGridSizeIndex];

							Section.AddMenuEntry(
								NAME_None,
								FText::AsNumber(CurGridSize),
								FText::Format(
									LOCTEXT("LocationGridSize_ToolTip", "Sets grid size to {0}"), FText::AsNumber(CurGridSize)
								),
								FSlateIcon(),
								FUIAction(
									FExecuteAction::CreateLambda(
										[CurrGridSizeIndex, ExecuteDelegate]()
										{
											ExecuteDelegate.Execute(CurrGridSizeIndex);
										}
									),
									FCanExecuteAction::CreateLambda(
										[IsEnabledDelegate]()
										{
											return IsEnabledDelegate.Get();
										}
									),
									FIsActionChecked::CreateLambda(
										[CurrGridSizeIndex, IsCheckedDelegate]()
										{
											return IsCheckedDelegate.Execute(CurrGridSizeIndex);
										}
									)
								),
								EUserInterfaceActionType::RadioButton
							);
						}
					}
				)
			);
		}
	}

	FToolMenuContext MenuContext;
	{
		MenuContext.AppendCommandList(InMenuOptions.CommandList);
	}

	return UToolMenus::Get()->GenerateWidget(MenuName, MenuContext);
}

FText GetScaleGridLabel()
{
	FNumberFormattingOptions NumberFormattingOptions;
	NumberFormattingOptions.MaximumFractionalDigits = 5;

	const float CurGridAmount = GEditor->GetScaleGridSize();
	return (GEditor->UsePercentageBasedScaling()) ? FText::AsPercent(CurGridAmount / 100.0f, &NumberFormattingOptions)
												  : FText::AsNumber(CurGridAmount, &NumberFormattingOptions);
}

TSharedRef<SWidget> CreateScaleGridSnapMenu(
	const FScaleGridCheckboxListExecuteActionDelegate& InExecuteDelegate,
	const FScaleGridCheckboxListIsCheckedDelegate& InIsCheckedDelegate,
	const TArray<float>& InGridSizes,
	const TAttribute<bool>& InIsEnabledDelegate,
	const TSharedPtr<FUICommandList>& InCommandList,
	const TAttribute<bool>& ShowPreserveNonUniformScaleOption,
	const FUIAction& PreserveNonUniformScaleUIAction
)
{
	const FName CheckboxListMenu("ScaleGridCheckboxList");
	if (!UToolMenus::Get()->IsMenuRegistered(CheckboxListMenu))
	{
		if (UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(CheckboxListMenu, NAME_None))
		{

			FNumberFormattingOptions NumberFormattingOptions;
			NumberFormattingOptions.MaximumFractionalDigits = 5;

			FToolMenuSection& Section = Menu->AddSection("ScaleSnapOptions");
			for (int32 CurrGridAmountIndex = 0; CurrGridAmountIndex < InGridSizes.Num(); ++CurrGridAmountIndex)
			{
				const float CurGridAmount = InGridSizes[CurrGridAmountIndex];

				FText MenuText;
				FText ToolTipText;

				if (GEditor->UsePercentageBasedScaling())
				{
					MenuText = FText::AsPercent(CurGridAmount / 100.0f, &NumberFormattingOptions);
					ToolTipText =
						FText::Format(LOCTEXT("ScaleGridAmountOld_ToolTip", "Snaps scale values to {0}"), MenuText);
				}
				else
				{
					MenuText = FText::AsNumber(CurGridAmount, &NumberFormattingOptions);
					ToolTipText = FText::Format(
						LOCTEXT("ScaleGridAmount_ToolTip", "Snaps scale values to increments of {0}"), MenuText
					);
				}

				Section.AddMenuEntry(
					NAME_None,
					MenuText,
					ToolTipText,
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda(
							[CurrGridAmountIndex, InExecuteDelegate]()
							{
								InExecuteDelegate.Execute(CurrGridAmountIndex);
							}
						),
						FCanExecuteAction::CreateLambda(
							[InIsEnabledDelegate]()
							{
								return InIsEnabledDelegate.Get();
							}
						),
						FIsActionChecked::CreateLambda(
							[CurrGridAmountIndex, InIsCheckedDelegate]()
							{
								return InIsCheckedDelegate.Execute(CurrGridAmountIndex);
							}
						)
					),
					EUserInterfaceActionType::RadioButton
				);
			}

			Menu->AddDynamicSection(
				NAME_None,
				FNewToolMenuDelegate::CreateLambda(
					[ShowPreserveNonUniformScaleOption, PreserveNonUniformScaleUIAction](UToolMenu* InMenu)
					{
						if (!GEditor->UsePercentageBasedScaling() && ShowPreserveNonUniformScaleOption.Get())
						{
							FToolMenuSection& GeneralOptionsSection =
								InMenu->AddSection("ScaleGeneralOptions", LOCTEXT("ScaleOptions", "Scaling Options"));
							GeneralOptionsSection.AddMenuEntry(
								NAME_None,
								LOCTEXT("ScaleGridPreserveNonUniformScale", "Preserve Non-Uniform Scale"),
								LOCTEXT(
									"ScaleGridPreserveNonUniformScale_ToolTip", "When this option is checked, scaling objects that have a non-uniform scale will preserve the ratios between each axis, snapping the axis with the largest value."
								),
								FSlateIcon(),
								PreserveNonUniformScaleUIAction,
								EUserInterfaceActionType::ToggleButton
							);
						}
					}
				)
			);
		}
	}

	FToolMenuContext MenuContext;
	{
		MenuContext.AppendCommandList(InCommandList);
	}

	return UToolMenus::Get()->GenerateWidget(CheckboxListMenu, MenuContext);
}

FToolMenuEntry CreateCheckboxSubmenu(
	const FName InName,
	const TAttribute<FText>& InLabel,
	const TAttribute<FText>& InToolTip,
	const FToolMenuExecuteAction& InCheckboxExecuteAction,
	const FToolMenuCanExecuteAction& InCheckboxCanExecuteAction,
	const FToolMenuGetActionCheckState& InCheckboxActionCheckState,
	const FNewToolMenuChoice& InMakeMenu
)
{
	FToolUIAction CheckboxMenuAction;
	{
		CheckboxMenuAction.ExecuteAction = InCheckboxExecuteAction;
		CheckboxMenuAction.CanExecuteAction = InCheckboxCanExecuteAction;
		CheckboxMenuAction.GetActionCheckState = InCheckboxActionCheckState;
	}

	FToolMenuEntry CheckBoxSubmenu = FToolMenuEntry::InitSubMenu(
		InName, InLabel, InToolTip, InMakeMenu, CheckboxMenuAction, EUserInterfaceActionType::ToggleButton
	);
	
	return CheckBoxSubmenu;
}

TSharedRef<SWidget> CreateNumericEntryWidget(
	const TSharedRef<SWidget>& InNumericBoxWidget,
	const FText& InLabel)
{
	const FMargin WidgetsMargin(2.0f, 0.0f, 3.0f, 0.0f);

	// clang-format off
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.Padding(WidgetsMargin)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(InLabel)
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		.Padding(FMargin(6.0f, 0))
		.FillContentWidth(1.0)
		[
			SNew(SBox)
			.Padding(WidgetsMargin)
			.MinDesiredWidth(80.0f)
			[
				InNumericBoxWidget
			]
		];
	// clang-format on
}

// float version of numeric entry widget
FToolMenuEntry CreateNumericEntry(
	const FName InName,
	const FText& InLabel,
	const FText& InTooltip,
	const FCanExecuteAction& InCanExecuteAction,
	const FNumericEntryExecuteActionDelegate& InOnValueChanged,
	const TAttribute<float>& InGetValue,
	float InMinValue,
	float InMaxValue,
	int32 InMaxFractionalDigits
)
{
	// clang-format off
	TSharedRef<SWidget> NumericEntryWidget =
		SNew(SNumericEntryBox<float>)
		.ToolTipText(InTooltip)
		.MinValue(InMinValue)
		.MaxValue(InMaxValue)
		.MaxSliderValue(InMaxValue)
		.AllowSpin(true)
		.MaxFractionalDigits(InMaxFractionalDigits)
		.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
		.OnValueChanged_Lambda([InOnValueChanged](float InValue)
		{
			InOnValueChanged.Execute(InValue);
		})
		.Value_Lambda([InGetValue]()
		{
			return InGetValue.Get();
		});
	// clang-format on

	FToolMenuEntry NumericEntry = FToolMenuEntry::InitMenuEntry(
		InName,
		FUIAction(FExecuteAction(), InCanExecuteAction),
		CreateNumericEntryWidget(NumericEntryWidget, InLabel)
	);
	
	NumericEntry.ToolTip = InTooltip;

	return NumericEntry;
}

// int32 version of numeric entry widget
FToolMenuEntry CreateNumericEntry(
	const FName InName,
	const FText& InLabel,
	const FText& InTooltip,
	const FCanExecuteAction& InCanExecuteAction,
	const FNumericEntryExecuteActionDelegateInt32& InOnValueChanged,
	const TAttribute<int32>& InGetValue,
	int32 InMinValue,
	int32 InMaxValue
)
{
	// clang-format off
	TSharedRef<SWidget> NumericEntryWidget =
		SNew(SNumericEntryBox<int32>)
		.ToolTipText(InTooltip)
		.MinValue(InMinValue)
		.MaxValue(InMaxValue)
		.MaxSliderValue(InMaxValue)
		.AllowSpin(true)
		.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
		.OnValueChanged_Lambda([InOnValueChanged](int32 InValue)
		{
			InOnValueChanged.Execute(InValue);
		})
		.Value_Lambda([InGetValue]()
		{
			return InGetValue.Get();
		});
	// clang-format on

	FToolMenuEntry NumericEntry = FToolMenuEntry::InitMenuEntry(
		InName,
		FUIAction(FExecuteAction(), InCanExecuteAction),
		CreateNumericEntryWidget(NumericEntryWidget, InLabel)
	);

	return NumericEntry;
}

TSharedRef<SWidget> CreateCameraMenuWidget(const TSharedRef<SEditorViewport>& InViewport, bool bInShowExposureSettings)
{
	// We generate a menu via UToolMenus, so we can use FillShowSubmenu call from both old and new toolbar
	FName OldShowMenuName = "LevelEditor.OldViewportToolbar.CameraOptions";

	if (!UToolMenus::Get()->IsMenuRegistered(OldShowMenuName))
	{
		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(OldShowMenuName, NAME_None, EMultiBoxType::Menu, false);
		Menu->AddDynamicSection(
			"BaseSection",
			FNewToolMenuDelegate::CreateLambda(
				[](UToolMenu* InMenu)
				{
					PopulateCameraMenu(InMenu);
				}
			)
		);
	}

	FToolMenuContext MenuContext;
	{
		MenuContext.AppendCommandList(InViewport->GetCommandList());

		// Add the UnrealEd viewport toolbar context.
		{
			UUnrealEdViewportToolbarContext* const ContextObject =
				UE::UnrealEd::CreateViewportToolbarDefaultContext(InViewport);

			MenuContext.AddObject(ContextObject);
		}
	}

	return UToolMenus::Get()->GenerateWidget(OldShowMenuName, MenuContext);
}

TSharedRef<SWidget> CreateFOVMenuWidget(const TSharedRef<SEditorViewport>& InViewport)
{
	constexpr float FOVMin = 5.0f;
	constexpr float FOVMax = 170.0f;

	TWeakPtr<FEditorViewportClient> ViewportClientWeak = InViewport->GetViewportClient();

	return
		// clang-format off
		SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
			.WidthOverride(100.0f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
				.Padding(FMargin(1.0f))
				[
					SNew(SSpinBox<float>)
					.Style(&FAppStyle::Get(), "Menu.SpinBox")
					.Font( FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
					.MinValue(FOVMin)
					.MaxValue(FOVMax)
					.Value_Lambda([ViewportClientWeak]()
					{
						if (TSharedPtr<FEditorViewportClient> ViewportClient = ViewportClientWeak.Pin())
						{
							return ViewportClient->ViewFOV;
						}
						return 90.0f;
					})
					.OnValueChanged_Lambda([ViewportClientWeak](float InNewValue)
					{
						if (TSharedPtr<FEditorViewportClient> ViewportClient = ViewportClientWeak.Pin())
						{
							ViewportClient->FOVAngle = InNewValue;
							ViewportClient->ViewFOV = InNewValue;
							ViewportClient->Invalidate();
						}
					})
				]
			]
		];
	// clang-format on
}

TSharedRef<SWidget> CreateNearViewPlaneMenuWidget(const TSharedRef<SEditorViewport>& InViewport)
{
	TWeakPtr<FEditorViewportClient> ViewportClientWeak = InViewport->GetViewportClient();

	return
		// clang-format off
		SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
			.WidthOverride(100.0f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
				.Padding(FMargin(1.0f))
				[
					SNew(SSpinBox<float>)
					.Style(&FAppStyle::Get(), "Menu.SpinBox")
					.ToolTipText(LOCTEXT("NearViewPlaneTooltip", "Distance to use as the near view plane"))
					.MinValue(0.001f)
					.MaxValue(100.0f)
					.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
					.Value_Lambda([ViewportClientWeak]()
					{
						if (TSharedPtr<FEditorViewportClient> ViewportClient = ViewportClientWeak.Pin())
						{
							return ViewportClient->GetNearClipPlane();
						}

						return 1.0f;
					})
					.OnValueChanged_Lambda([ViewportClientWeak](float InNewValue)
					{
						if (TSharedPtr<FEditorViewportClient> ViewportClient = ViewportClientWeak.Pin())
						{
							ViewportClient->OverrideNearClipPlane(InNewValue);
							ViewportClient->Invalidate();
						}
					})
				]
			]
		];
	// clang-format on
}

TSharedRef<SWidget> CreateFarViewPlaneMenuWidget(const TSharedRef<SEditorViewport>& InViewport)
{
	TWeakPtr<FEditorViewportClient> ViewportClientWeak = InViewport->GetViewportClient();

	/*
	 * FEditorViewportClient treats a far clip plane value of 0.0 as "infinity".
	 * This Spin Box transforms the maximum value to that 0.0 and back again,
	 * allowing the maximum value to be treated as infinity and creating a more
	 * natural interface.
	 */
	constexpr float MaxValue = 100000.0f;

	return
		// clang-format off
		SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
			.WidthOverride(100.0f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
				.Padding(FMargin(1.0f))
				[
					SNew(SSpinBox<float>)
					.Style(&FAppStyle::Get(), "Menu.SpinBox")
					.ToolTipText(LOCTEXT("FarViewPlaneTooltip", "Distance to use as the far view plane"))
					.MinValue(0.01f)
					.MaxValue(MaxValue)
					.SliderExponent(3.0f) // Gives better precision for smaller ranges
					.OnGetDisplayValue_Lambda([](float InValue)
					{
						if (InValue >= MaxValue)
						{
							return TOptional<FText>(LOCTEXT("Infinity", "Infinity"));
						}
						return TOptional<FText>();
					})
					.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
					.Value_Lambda([ViewportClientWeak]()
					{
						if (TSharedPtr<FEditorViewportClient> ViewportClient = ViewportClientWeak.Pin())
						{
							const float Override = ViewportClient->GetFarClipPlaneOverride(); 
							if (Override > 0.0f)
							{
								return Override;
							}
						}

						return MaxValue;
					})
					.OnValueChanged_Lambda([ViewportClientWeak](float InNewValue)
					{
						if (TSharedPtr<FEditorViewportClient> ViewportClient = ViewportClientWeak.Pin())
						{
							ViewportClient->OverrideFarClipPlane(InNewValue >= MaxValue ? 0.0f : InNewValue);
							ViewportClient->Invalidate();
						}
					})
				]
			]
		];
	// clang-format on
}

TSharedRef<SWidget> CreateCameraSpeedWidget(const TWeakPtr<SEditorViewport>& WeakViewport)
{
	if (TSharedPtr<SEditorViewport> Viewport = WeakViewport.Pin())
	{
		return CreateCameraSpeedWidget(Viewport->GetViewportClient());
	}
	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> CreateCameraSpeedWidget(const TSharedPtr<FEditorViewportClient>& ViewportClient)
{
	TWeakPtr<FEditorViewportClient> WeakClient = ViewportClient.ToWeakPtr();

	// clang-format off
	return
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(24.0f, 0.0f, 4.0f, 0.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			[
				SNew(STextBlock)
				.ColorAndOpacity(FLinearColor::White)
				.Text(LOCTEXT("CameraSpeedWidgetLabel", "Speed"))
				.TextStyle(FAppStyle::Get(), "Menu.Label")
			]
			+ SHorizontalBox::Slot()
			.Padding(4.0f, 0.0f)
			.FillWidth(1.0f)
			[
				SNew(SNumericEntryBox<float>)
				.AllowSpin(true)
				.MinValue_Lambda([WeakClient]
				{
					if (TSharedPtr<FEditorViewportClient> Client = WeakClient.Pin())
					{
						return Client->GetCameraSpeedSettings().GetAbsoluteMinSpeed();
					}
					return 0.0f;
				})
				.MaxValue_Lambda([WeakClient]
				{
					if (TSharedPtr<FEditorViewportClient> Client = WeakClient.Pin())
					{
						return Client->GetCameraSpeedSettings().GetAbsoluteMaxSpeed();
					}
					return 1000.0f;
				})
				.MinSliderValue_Lambda([WeakClient]
				{
					if (TSharedPtr<FEditorViewportClient> Client = WeakClient.Pin())
					{
						return Client->GetCameraSpeedSettings().GetMinUISpeed();
					}
					return 0.0f;
				})
				.MaxSliderValue_Lambda([WeakClient]
				{
					if (TSharedPtr<FEditorViewportClient> Client = WeakClient.Pin())
					{
						return Client->GetCameraSpeedSettings().GetMaxUISpeed();
					}
					return 32.0f;
				})
				.SliderExponent(3.0f)
				.MaxFractionalDigits_Lambda([WeakClient]
				{
					if (TSharedPtr<FEditorViewportClient> Client = WeakClient.Pin())
					{
						const float Speed = Client->GetCameraSpeedSettings().GetCurrentSpeed();
						if (Speed > 10.0f)
						{
							return 0;
						}
						if (Speed > 1.0f)
						{
							return 1;
						}
						if (Speed > .1f)
						{
							return 2;
						}
					}
					return 3;
				})
				.Value_Lambda([WeakClient]()
				{
					if (TSharedPtr<FEditorViewportClient> Client = WeakClient.Pin())
					{
						return Client->GetCameraSpeedSettings().GetCurrentSpeed();
					}
					return 0.0f;
				})
				.OnValueChanged_Lambda([WeakClient](float NewValue)
				{
					if (TSharedPtr<FEditorViewportClient> Client = WeakClient.Pin())
					{
						FEditorViewportCameraSpeedSettings Settings = Client->GetCameraSpeedSettings();
						Settings.SetCurrentSpeed(NewValue);
						Client->SetCameraSpeedSettings(Settings);
					}
				})
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(24.0f, 4.0f, 8.0f, 4.0f)
		[
			SNew(STextBlock)
			.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			.Text(LOCTEXT("CameraSpeedSliderDescription", "Hold either mouse button and use the scroll wheel to adjust the speed on the fly."))
			.WrapTextAt(260.f)
		]
		;
	// clang-format on
}

FText GetCameraSpeedLabel(const TWeakPtr<SEditorViewport>& WeakViewport)
{
	if (const TSharedPtr<SEditorViewport> Viewport = WeakViewport.Pin())
	{
		return GetCameraSpeedLabel(Viewport->GetViewportClient());
	}

	return FText();
}

FText GetCameraSpeedLabel(const TSharedPtr<FEditorViewportClient>& ViewportClient)
{
	if (ViewportClient)
	{
		const float CameraSpeed = ViewportClient->GetCameraSpeed();
		FNumberFormattingOptions FormattingOptions = FNumberFormattingOptions::DefaultNoGrouping();
		FormattingOptions.MaximumFractionalDigits = CameraSpeed > 1 ? 1 : 3;
		return FText::AsNumber(CameraSpeed, &FormattingOptions);
	}
	return FText();
}

FText GetCameraSubmenuLabelFromViewportType(const ELevelViewportType ViewportType)
{
	FText Label = LOCTEXT("CameraMenuTitle_Default", "Camera");
	switch (ViewportType)
	{
	case LVT_Perspective:
		Label = LOCTEXT("CameraMenuTitle_Perspective", "Perspective");
		break;

	case LVT_OrthoTop:
		Label = LOCTEXT("CameraMenuTitle_Top", "Top");
		break;

	case LVT_OrthoLeft:
		Label = LOCTEXT("CameraMenuTitle_Left", "Left");
		break;

	case LVT_OrthoFront:
		Label = LOCTEXT("CameraMenuTitle_Front", "Front");
		break;

	case LVT_OrthoBottom:
		Label = LOCTEXT("CameraMenuTitle_Bottom", "Bottom");
		break;

	case LVT_OrthoRight:
		Label = LOCTEXT("CameraMenuTitle_Right", "Right");
		break;

	case LVT_OrthoBack:
		Label = LOCTEXT("CameraMenuTitle_Back", "Back");
		break;
	case LVT_OrthoFreelook:
		break;
	}

	return Label;
}

FName GetCameraSubmenuIconFNameFromViewportType(const ELevelViewportType ViewportType)
{
	// Use the raw camera icon rather than the perspective icon
	// so that in the default state, the camera menu is easily recognizable.
	static FName PerspectiveIcon("ClassIcon.CameraComponent");
	static FName TopIcon("EditorViewport.Top");
	static FName LeftIcon("EditorViewport.Left");
	static FName FrontIcon("EditorViewport.Front");
	static FName BottomIcon("EditorViewport.Bottom");
	static FName RightIcon("EditorViewport.Right");
	static FName BackIcon("EditorViewport.Back");

	FName Icon = NAME_None;

	switch (ViewportType)
	{
	case LVT_Perspective:
		Icon = PerspectiveIcon;
		break;

	case LVT_OrthoTop:
		Icon = TopIcon;
		break;

	case LVT_OrthoLeft:
		Icon = LeftIcon;
		break;

	case LVT_OrthoFront:
		Icon = FrontIcon;
		break;

	case LVT_OrthoBottom:
		Icon = BottomIcon;
		break;

	case LVT_OrthoRight:
		Icon = RightIcon;
		break;

	case LVT_OrthoBack:
		Icon = BackIcon;
		break;
	case LVT_OrthoFreelook:
		break;
	}

	return Icon;
}

FToolMenuEntry CreateViewportToolbarCameraSubmenu()
{
	return CreateCameraSubmenu();
}

FViewportCameraMenuOptions::FViewportCameraMenuOptions()
	: bShowCameraMovement(false)
	, bShowFieldOfView(false)
	, bShowNearAndFarPlanes(false)
{
}

FViewportCameraMenuOptions& FViewportCameraMenuOptions::ShowAll()
{
	bShowCameraMovement = true;
	bShowFieldOfView = true;
	bShowNearAndFarPlanes = true;
	return *this;
}

FViewportCameraMenuOptions& FViewportCameraMenuOptions::ShowCameraMovement()
{
	bShowCameraMovement = true;
	return *this;
}

FViewportCameraMenuOptions& FViewportCameraMenuOptions::ShowLensControls()
{
	bShowFieldOfView = true;
	bShowNearAndFarPlanes = true;
	return *this;
}

FToolMenuEntry CreateCameraSubmenu(const FViewportCameraMenuOptions& InOptions)
{
	return FToolMenuEntry::InitDynamicEntry(
		"DynamicCameraOptions",
		FNewToolMenuSectionDelegate::CreateLambda(
			[Options = InOptions](FToolMenuSection& InDynamicSection) -> void
			{
				TWeakPtr<SEditorViewport> WeakViewport;
				if (UUnrealEdViewportToolbarContext* const EditorViewportContext =
						InDynamicSection.FindContext<UUnrealEdViewportToolbarContext>())
				{
					WeakViewport = EditorViewportContext->Viewport;
				}

				const TAttribute<FText> Label = TAttribute<FText>::CreateLambda(
					[WeakViewport]()
					{
						if (TSharedPtr<SEditorViewport> Viewport = WeakViewport.Pin())
						{
							return UE::UnrealEd::GetCameraSubmenuLabelFromViewportType(Viewport->GetViewportClient()->ViewportType
							);
						}
						return LOCTEXT("CameraSubmenuLabel", "Camera");
					}
				);

				const TAttribute<FSlateIcon> Icon = TAttribute<FSlateIcon>::CreateLambda(
					[WeakViewport]()
					{
						if (TSharedPtr<SEditorViewport> Viewport = WeakViewport.Pin())
						{
							const FName IconFName = UE::UnrealEd::GetCameraSubmenuIconFNameFromViewportType(
								Viewport->GetViewportClient()->ViewportType
							);
							return FSlateIcon(FAppStyle::GetAppStyleSetName(), IconFName);
						}
						return FSlateIcon();
					}
				);

				FToolMenuEntry& Entry = InDynamicSection.AddSubMenu(
					"Camera",
					Label,
					LOCTEXT("CameraSubmenuTooltip", "Camera options"),
					FNewToolMenuDelegate::CreateLambda(
						[Options](UToolMenu* Submenu) -> void
						{
							PopulateCameraMenu(Submenu, Options);
						}
					),
					false,
					Icon
				);
				Entry.ToolBarData.ResizeParams.ClippingPriority = 800;
			}
		)
	);
}

FToolMenuEntry CreateViewportToolbarAssetViewerProfileSubmenu()
{
	return CreateAssetViewerProfileSubmenu();
}

FToolMenuEntry CreateAssetViewerProfileSubmenu()
{
	return FToolMenuEntry::InitDynamicEntry(
		"AssetViewerProfile",
		FNewToolMenuSectionDelegate::CreateLambda(
			[](FToolMenuSection& InnerSection) -> void
			{
				UUnrealEdViewportToolbarContext* const EditorViewportContext =
					InnerSection.FindContext<UUnrealEdViewportToolbarContext>();
				if (!EditorViewportContext)
				{
					return;
				}

				TSharedPtr<IPreviewProfileController> PreviewProfileController = EditorViewportContext->GetPreviewProfileController();
				if (!PreviewProfileController)
				{
					return;
				}

				TWeakPtr<IPreviewProfileController> PreviewProfileControllerWeak = PreviewProfileController;

				FToolMenuEntry& Entry = InnerSection.AddSubMenu(
					"AssetViewerProfile",
					FText::GetEmpty(),
					TAttribute<FText>::CreateLambda([PreviewProfileControllerWeak]
					{
						if (TSharedPtr<IPreviewProfileController> PreviewProfileController = PreviewProfileControllerWeak.Pin())
						{
							return FText::Format(
								LOCTEXT("AssetViewerProfileSelectionSectionTooltipFormat", "Preview Scene: {0}"),
								FText::FromString(PreviewProfileController->GetActiveProfile())
							);
						}
						return LOCTEXT("AssetViewerProfileSelectionSectionNoneTooltip", "Select the Preview Scene Profile for this viewport.");	
					}),
					FNewToolMenuDelegate::CreateLambda(
						[PreviewProfileControllerWeak](UToolMenu* Submenu) -> void
						{
							TSharedPtr<IPreviewProfileController> PreviewProfileController =
								PreviewProfileControllerWeak.Pin();

							if (!PreviewProfileController)
							{
								return;
							}

							FToolMenuSection& PreviewProfilesSelectionSection = Submenu->FindOrAddSection(
								"AssetViewerProfileSelectionSection",
								LOCTEXT("AssetViewerProfileSelectionSectionLabel", "Preview Scene Profiles")
							);

							int32 CurrProfileIndex = 0;
							const TArray<FString>& PreviewProfiles =
								PreviewProfileController->GetPreviewProfiles(CurrProfileIndex);

							for (int32 ProfileIndex = 0; ProfileIndex < PreviewProfiles.Num(); ProfileIndex++)
							{
								const FString& ProfileName = PreviewProfiles[ProfileIndex];
								PreviewProfilesSelectionSection.AddMenuEntry(
									NAME_None,
									FText::FromString(ProfileName),
									TAttribute<FText>::CreateLambda([]()->FText
										{
											const FInputBindingManager& InputManager = FInputBindingManager::Get();
											const TSharedPtr<FUICommandInfo> PreviousProfile = InputManager.FindCommandInContext(TEXT("AdvancedPreviewScene"), TEXT("PreviousProfile"));
											const TSharedPtr<FUICommandInfo> NextProfile = InputManager.FindCommandInContext(TEXT("AdvancedPreviewScene"), TEXT("NextProfile"));
			
											if (ensureMsgf(PreviousProfile, TEXT("AdvancedPreviewScene PreviousProfile command is not valid"))
												&& ensureMsgf(NextProfile, TEXT("AdvancedPreviewScene NextProfile command is not valid")))
											{
												return FText::FormatOrdered(
													LOCTEXT("AssetViewerProfileTooltip", "Set this profile as active\nUse {0} to navigate to the previous profile\nUse {1} to navigate to the next profile"),
													PreviousProfile->GetInputText(),
													NextProfile->GetInputText()
												);
											}

											return FText::GetEmpty();
										}),
									FSlateIcon(),
									FUIAction(
										FExecuteAction::CreateLambda(
											[PreviewProfileControllerWeak, ProfileIndex, PreviewProfiles]()
											{
												if (TSharedPtr<IPreviewProfileController> PreviewProfileController =
														PreviewProfileControllerWeak.Pin())
												{
													PreviewProfileController->SetActiveProfile(PreviewProfiles[ProfileIndex]);
												}
											}
										),
										FCanExecuteAction(),
										FIsActionChecked::CreateLambda(
											[PreviewProfileControllerWeak, ProfileIndex]()
											{
												if (TSharedPtr<IPreviewProfileController> PreviewProfileController =
														PreviewProfileControllerWeak.Pin())
												{
													int32 CurrentlySelectedProfileIndex;
													PreviewProfileController->GetPreviewProfiles(CurrentlySelectedProfileIndex
													);

													return ProfileIndex == CurrentlySelectedProfileIndex;
												}

												return false;
											}
										)
									),
									EUserInterfaceActionType::RadioButton
								);
							}
						}
					),
					false,
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "ViewportToolbar.PreviewSceneSettings")
				);
				Entry.ToolBarData.ResizeParams.ClippingPriority = 1100;
			}
		)
	);
}

FToolMenuEntry CreatePreviewLODSelectionSubmenu(TWeakPtr<IPreviewLODController> LODController)
{
	return FToolMenuEntry::InitSubMenu(
		"LOD",
		TAttribute<FText>::CreateLambda([LODController]
		{
			if (TSharedPtr<IPreviewLODController> Controller = LODController.Pin())
			{
				const int32 CurrentLOD = Controller->GetCurrentLOD();
				if (CurrentLOD >= 0)
				{
					return FText::Format(LOCTEXT("LODMenu_LabelFormat", "LOD {0}"), CurrentLOD);
				}
			}
			return LOCTEXT("LODMenu_AutoLabel", "LOD Auto");
		}),
		LOCTEXT("LODMenu_Tooltip", "Set the Level of Detail of the viewport."),
		FNewToolMenuDelegate::CreateStatic(&FillPreviewLODSelectionSubmenu, LODController),
		false,
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.LOD")
	);
}

void FillPreviewLODSelectionSubmenu(UToolMenu* Menu, TWeakPtr<IPreviewLODController> LODController)
{
	if (TSharedPtr<IPreviewLODController> Controller = LODController.Pin())
	{
		FToolMenuSection& Section = Menu->FindOrAddSection("LOD", LOCTEXT("LODMenuSectionHeader", "Preview LODs"));
				
		// Allow some LOD items to be handled by commands
		TArray<TSharedPtr<FUICommandInfo>> Commands;
		Controller->FillLODCommands(Commands);
		for (const TSharedPtr<FUICommandInfo>& Command : Commands)
		{
			Section.AddMenuEntry(Command);
		}
				
		// Fill in remaining LOD levels with menu entries
		const int32 LODCount = Controller->GetLODCount();
		for (int32 LODIndex = Controller->GetAutoLODStartingIndex(); LODIndex < LODCount; ++LODIndex)
		{
			FText Label;
			FText Tooltip;
			
			if (LODIndex == INDEX_NONE)
			{
				Label = LOCTEXT("LODMenu_LODAutoLabel", "LOD Auto");
				Tooltip = LOCTEXT("LODMenu_LODAutoTooltip", "Automatically select the the Level of Detail.");
			}
			else
			{
				Label = FText::Format(LOCTEXT("LODMenu_LODLabelFormat", "LOD {0}"), LODIndex);
				Tooltip = FText::Format(LOCTEXT("LODMenu_LODTooltipFormat", "Sets the Level of Detail to {0}"), LODIndex);
			}
		
			Section.AddMenuEntry(
				NAME_None,
				Label,
				Tooltip,
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([LODController, LODIndex]
					{
						if (TSharedPtr<IPreviewLODController> Controller = LODController.Pin())
						{
							Controller->SetLODLevel(LODIndex);
						}
					}),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([LODController, LODIndex]
					{
						if (TSharedPtr<IPreviewLODController> Controller = LODController.Pin())
						{
							return Controller->IsLODSelected(LODIndex);
						}
						return false;
					})
				),
				EUserInterfaceActionType::RadioButton
			);
		}
	}
}

void ExtendPreviewSceneSettingsWithTabEntry(FName InAssetViewerProfileSubmenuName)
{
	UToolMenu* const Submenu = UToolMenus::Get()->ExtendMenu(InAssetViewerProfileSubmenuName);
	if (!Submenu)
	{
		return;
	}
	
	FToolMenuSection& PreviewMeshSection = Submenu->FindOrAddSection("PreviewSceneTabOpeningSection");
	PreviewMeshSection.Alignment = EToolMenuSectionAlign::Last;

	PreviewMeshSection.AddSeparator("PreviewSceneTabOpeningSeparator");

	FToolUIAction UIAction;
	UIAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda(
		[](const FToolMenuContext& InContext)
		{
			UUnrealEdViewportToolbarContext* Context = Cast<UUnrealEdViewportToolbarContext>(InContext.FindByClass(UUnrealEdViewportToolbarContext::StaticClass()));
			if (!Context)
			{
				return;
			}
			if (TSharedPtr<FAssetEditorToolkit> ToolkitPinned = Context->AssetEditorToolkit.Pin())
			{
				if (TSharedPtr<FTabManager> TabManager = ToolkitPinned->GetTabManager())
				{
					TabManager->TryInvokeTab(Context->PreviewSettingsTabId);
				}
			}
		}
	);

	PreviewMeshSection.AddEntry(FToolMenuEntry::InitMenuEntry(
		"OpenPreviewSceneSettingsTab",
		LOCTEXT("OpenPreviewSceneSettingsTabLabel", "Preview Scene Settings..."),
		LOCTEXT("OpenPreviewSceneSettingsTabTooltip", "Opens a details tab with Preview Scene Settings."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "ViewportToolbar.PreviewSceneSettings"),
		UIAction,
		EUserInterfaceActionType::Button
	));
}

void AddExposureSection(UToolMenu* InMenu, const TSharedPtr<SEditorViewport>& EditorViewport)
{
	const FEditorViewportCommands& BaseViewportCommands = FEditorViewportCommands::Get();

	TSharedRef<SWidget> FixedEV100Menu = EditorViewport->BuildFixedEV100Menu();
	TSharedPtr<FEditorViewportClient> EditorViewportClient = EditorViewport->GetViewportClient();
	const bool bIsLevelEditor = EditorViewportClient.IsValid() && EditorViewportClient->IsLevelEditorClient();

	FToolMenuSection& Section = InMenu->AddSection("Exposure", LOCTEXT("ExposureHeader", "Exposure"));
	Section.AddMenuEntry(bIsLevelEditor ? BaseViewportCommands.ToggleInGameExposure : BaseViewportCommands.ToggleAutoExposure);
	Section.AddEntry(FToolMenuEntry::InitWidget("FixedEV100", FixedEV100Menu, LOCTEXT("FixedEV100", "EV100")));
}

void PopulateCameraMenu(UToolMenu* InMenu, const FViewportCameraMenuOptions& InOptions)
{
	UUnrealEdViewportToolbarContext* const EditorViewportContext = InMenu->FindContext<UUnrealEdViewportToolbarContext>();
	if (!EditorViewportContext)
	{
		return;
	}

	const TSharedPtr<SEditorViewport> EditorViewport = EditorViewportContext->Viewport.Pin();
	if (!EditorViewport)
	{
		return;
	}

	// Perspective
	{
		FToolMenuSection& PerspectiveCameraSection =
			InMenu->FindOrAddSection("LevelViewportCameraType_Perspective", LOCTEXT("PerspectiveLabel", "Perspective"));
		PerspectiveCameraSection.AddMenuEntry(FEditorViewportCommands::Get().Perspective);

		FToolMenuSection& OrthographicCameraSection =
			InMenu->FindOrAddSection("LevelViewportCameraType_Ortho", LOCTEXT("CameraTypeHeader_Ortho", "Orthographic"));
		OrthographicCameraSection.AddMenuEntry(FEditorViewportCommands::Get().Top);
		OrthographicCameraSection.AddMenuEntry(FEditorViewportCommands::Get().Bottom);
		OrthographicCameraSection.AddMenuEntry(FEditorViewportCommands::Get().Left);
		OrthographicCameraSection.AddMenuEntry(FEditorViewportCommands::Get().Right);
		OrthographicCameraSection.AddMenuEntry(FEditorViewportCommands::Get().Front);
		OrthographicCameraSection.AddMenuEntry(FEditorViewportCommands::Get().Back);
	}

	// Movement
	{
		FToolMenuSection& MovementSection = InMenu->FindOrAddSection("Movement", LOCTEXT("CameraMovementSectionLabel", "Movement"));
		if (InOptions.bShowCameraMovement)
		{
			MovementSection.AddEntry(CreateCameraSpeedMenu());

			FToolMenuEntry& FrameEntry = MovementSection.AddMenuEntry(FEditorViewportCommands::Get().FocusViewportToSelection);
			FrameEntry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.FrameActor");
			
			FToolMenuEntry& FrameAndClip = MovementSection.AddMenuEntry(FEditorViewportCommands::Get().FocusAndClipViewportToSelection);
			FrameAndClip.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.FrameActor");
		}
	}

	// View
	{
		FToolMenuSection& ViewSection = InMenu->FindOrAddSection("View", LOCTEXT("CameraViewSectionLabel", "View"));
		const TAttribute<bool> IsPerspectiveAttribute = UE::UnrealEd::GetIsPerspectiveAttribute(EditorViewport->GetViewportClient());  

		if (InOptions.bShowFieldOfView)
		{
			FToolMenuEntry CameraFOV = FToolMenuEntry::InitWidget(
				"CameraFOV",
				UE::UnrealEd::CreateFOVMenuWidget(EditorViewport.ToSharedRef()),
				LOCTEXT("CameraSubmenu_FieldOfViewLabel", "Field of View")
			);
			CameraFOV.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ViewportToolbar.FieldOfView");
			CameraFOV.ToolTip = LOCTEXT("CameraMovementTooltip", "Sets the field of view of the viewport's camera.");
			CameraFOV.Visibility = IsPerspectiveAttribute;
			ViewSection.AddEntry(CameraFOV);
		}

		if (InOptions.bShowNearAndFarPlanes)
		{
			FToolMenuEntry CameraNearViewPlane = FToolMenuEntry::InitWidget(
				"CameraNearViewPlane",
				UE::UnrealEd::CreateNearViewPlaneMenuWidget(EditorViewport.ToSharedRef()),
				LOCTEXT("CameraSubmenu_NearViewPlaneLabel", "Near View Plane")
			);
			CameraNearViewPlane.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ViewportToolbar.NearViewPlane");
			CameraNearViewPlane.Visibility = IsPerspectiveAttribute;
			ViewSection.AddEntry(CameraNearViewPlane);

			FToolMenuEntry CameraFarViewPlane = FToolMenuEntry::InitWidget(
				"CameraFarViewPlane",
				UE::UnrealEd::CreateFarViewPlaneMenuWidget(EditorViewport.ToSharedRef()),
				LOCTEXT("CameraSubmenu_FarViewPlaneLabel", "Far View Plane")
			);
			CameraFarViewPlane.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ViewportToolbar.FarViewPlane");
			CameraFarViewPlane.Visibility = IsPerspectiveAttribute;
			ViewSection.AddEntry(CameraFarViewPlane);
					
			FToolMenuEntry& OrthoSubmenu = ViewSection.AddEntry(CreateOrthographicClippingPlanesSubmenu());
			OrthoSubmenu.Visibility = UE::UnrealEd::GetIsOrthographicAttribute(EditorViewport->GetViewportClient());
		}
	}
	
	// Auto Exposure
	{
		if (UE::UnrealEd::Private::ViewModesSubmenu::IsMenuSectionAvailable(
				EditorViewportContext, EHidableViewModeMenuSections::Exposure
			))
		{
			const FEditorViewportCommands& BaseViewportCommands = FEditorViewportCommands::Get();

			TSharedRef<SWidget> FixedEV100Menu = EditorViewport->BuildFixedEV100Menu();
			TSharedPtr<FEditorViewportClient> EditorViewportClient = EditorViewport->GetViewportClient();
			const bool bIsLevelEditor = EditorViewportClient.IsValid() && EditorViewportClient->IsLevelEditorClient();

			FToolMenuSection& Section = InMenu->AddSection("Exposure", LOCTEXT("ExposureHeader", "Exposure"));
			Section.AddMenuEntry(
				bIsLevelEditor ? BaseViewportCommands.ToggleInGameExposure : BaseViewportCommands.ToggleAutoExposure
			);
			FToolMenuEntry& EVEntry = Section.AddEntry(FToolMenuEntry::InitWidget("FixedEV100", FixedEV100Menu, LOCTEXT("FixedEV100", "EV100")));
			EVEntry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ViewportToolbar.EV100");
		}
	}
}

void ExtendCameraSubmenu(FName InCameraOptionsSubmenuName, bool bInShowViewPlaneEntries /* = true */)
{
	UToolMenu* const Submenu = UToolMenus::Get()->ExtendMenu(InCameraOptionsSubmenuName);

	Submenu->AddDynamicSection(
		"EditorCameraExtensionDynamicSection",
		FNewToolMenuDelegate::CreateLambda(
			[bInShowViewPlaneEntries](UToolMenu* InDynamicMenu)
			{
				UUnrealEdViewportToolbarContext* const EditorViewportContext =
					InDynamicMenu->FindContext<UUnrealEdViewportToolbarContext>();
				if (!EditorViewportContext)
				{
					return;
				}

				const TSharedPtr<SEditorViewport> EditorViewport = EditorViewportContext->Viewport.Pin();
				if (!EditorViewport)
				{
					return;
				}

				FToolMenuInsert InsertPosition("LevelViewportCameraType_Ortho", EToolMenuInsertType::After);

				FToolMenuSection& PostOrthoSection = InDynamicMenu->FindOrAddSection("CameraSubmenuPostOrtho", FText(), InsertPosition);
				PostOrthoSection.AddSeparator("CameraSubmenuSeparator");

				FToolMenuEntry CameraFOV = FToolMenuEntry::InitWidget(
					"CameraFOV",
					UE::UnrealEd::CreateFOVMenuWidget(EditorViewport.ToSharedRef()),
					LOCTEXT("CameraSubmenu_FieldOfViewLabel", "Field of View")
				);
				CameraFOV.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ViewportToolbar.FieldOfView");
				PostOrthoSection.AddEntry(CameraFOV);

				if (bInShowViewPlaneEntries)
				{
					FToolMenuEntry CameraNearViewPlane = FToolMenuEntry::InitWidget(
						"CameraNearViewPlane",
						UE::UnrealEd::CreateNearViewPlaneMenuWidget(EditorViewport.ToSharedRef()),
						LOCTEXT("CameraSubmenu_NearViewPlaneLabel", "Near View Plane")
					);
					CameraNearViewPlane.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ViewportToolbar.NearViewPlane");
					PostOrthoSection.AddEntry(CameraNearViewPlane);
				
					FToolMenuEntry CameraFarViewPlane = FToolMenuEntry::InitWidget(
						"CameraFarViewPlane",
						UE::UnrealEd::CreateFarViewPlaneMenuWidget(EditorViewport.ToSharedRef()),
						LOCTEXT("CameraSubmenu_FarViewPlaneLabel", "Far View Plane")
					);
					CameraFarViewPlane.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ViewportToolbar.FarViewPlane");
					PostOrthoSection.AddEntry(CameraFarViewPlane);
				}
			}
		)
	);
}

void GenerateViewportTypeMenu(UToolMenu* InMenu)
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));

	int32 NumCustomViewportTypes = 0;
	LevelEditorModule.IterateViewportTypes(
		[&NumCustomViewportTypes](FName, const FViewportTypeDefinition&)
		{
			++NumCustomViewportTypes;
		}
	);

	FText ViewportTypesHeading = LOCTEXT("ViewportTypes", "Viewport Type");
	constexpr uint32 MaxViewportTypesInTopLevelMenu = 4;

	auto PupoulateSection = [&LevelEditorModule](FToolMenuSection& InSection)
	{
		LevelEditorModule.IterateViewportTypes(
			[&InSection](FName ViewportTypeName, const FViewportTypeDefinition& InDefinition)
			{
				if (InDefinition.ActivationCommand.IsValid())
				{
					InSection.AddMenuEntry(
						*FString::Printf(TEXT("ViewportType_%s"), *ViewportTypeName.ToString()), InDefinition.ActivationCommand
					);
				}
			}
		);
	};

	if (NumCustomViewportTypes > MaxViewportTypesInTopLevelMenu)
	{
		FToolMenuSection& Section = InMenu->AddSection("ViewportTypes");
		Section.AddSubMenu(
			"ViewportTypes",
			ViewportTypesHeading,
			FText(),
			FNewToolMenuDelegate::CreateLambda(
				[PupoulateSection](UToolMenu* InSubmenu)
				{
					FToolMenuSection& Section = InSubmenu->AddSection(NAME_None);
					PupoulateSection(Section);
				}
			)
		);
	}
	else
	{
		FToolMenuSection& Section = InMenu->AddSection("ViewportTypes", ViewportTypesHeading);
		PupoulateSection(Section);
	}
}

static FFormatNamedArguments GetScreenPercentageFormatArguments(const FEditorViewportClient& ViewportClient)
{
	const UEditorPerformanceProjectSettings* EditorProjectSettings = GetDefault<UEditorPerformanceProjectSettings>();
	const UEditorPerformanceSettings* EditorUserSettings = GetDefault<UEditorPerformanceSettings>();
	const FEngineShowFlags& EngineShowFlags = ViewportClient.EngineShowFlags;

	const EViewStatusForScreenPercentage ViewportRenderingMode = ViewportClient.GetViewStatusForScreenPercentage();
	const bool bViewModeSupportsScreenPercentage = ViewportClient.SupportsPreviewResolutionFraction();
	const bool bIsPreviewScreenPercentage = ViewportClient.IsPreviewingScreenPercentage();

	float DefaultScreenPercentage = FMath::Clamp(
										ViewportClient.GetDefaultPrimaryResolutionFractionTarget(),
										ISceneViewFamilyScreenPercentage::kMinTSRResolutionFraction,
										ISceneViewFamilyScreenPercentage::kMaxTSRResolutionFraction
									)
								  * 100.0f;
	float PreviewScreenPercentage = float(ViewportClient.GetPreviewScreenPercentage());
	float FinalScreenPercentage = bIsPreviewScreenPercentage ? PreviewScreenPercentage : DefaultScreenPercentage;

	FFormatNamedArguments FormatArguments;
	FormatArguments.Add(TEXT("ViewportMode"), UEnum::GetDisplayValueAsText(ViewportRenderingMode));

	EScreenPercentageMode ProjectSetting = EScreenPercentageMode::Manual;
	EEditorUserScreenPercentageModeOverride UserPreference = EEditorUserScreenPercentageModeOverride::ProjectDefault;
	IConsoleVariable* CVarDefaultScreenPercentage = nullptr;
	if (ViewportRenderingMode == EViewStatusForScreenPercentage::PathTracer)
	{
		ProjectSetting = EditorProjectSettings->PathTracerScreenPercentageMode;
		UserPreference = EditorUserSettings->PathTracerScreenPercentageMode;
		CVarDefaultScreenPercentage = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Editor.Viewport.ScreenPercentageMode.PathTracer"));
	}
	else if (ViewportRenderingMode == EViewStatusForScreenPercentage::VR)
	{
		ProjectSetting = EditorProjectSettings->VRScreenPercentageMode;
		UserPreference = EditorUserSettings->VRScreenPercentageMode;
		CVarDefaultScreenPercentage = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Editor.Viewport.ScreenPercentageMode.VR"));
	}
	else if (ViewportRenderingMode == EViewStatusForScreenPercentage::Mobile)
	{
		ProjectSetting = EditorProjectSettings->MobileScreenPercentageMode;
		UserPreference = EditorUserSettings->MobileScreenPercentageMode;
		CVarDefaultScreenPercentage = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Editor.Viewport.ScreenPercentageMode.Mobile"));
	}
	else if (ViewportRenderingMode == EViewStatusForScreenPercentage::Desktop)
	{
		ProjectSetting = EditorProjectSettings->RealtimeScreenPercentageMode;
		UserPreference = EditorUserSettings->RealtimeScreenPercentageMode;
		CVarDefaultScreenPercentage = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Editor.Viewport.ScreenPercentageMode.RealTime"));
	}
	else if (ViewportRenderingMode == EViewStatusForScreenPercentage::NonRealtime)
	{
		ProjectSetting = EditorProjectSettings->NonRealtimeScreenPercentageMode;
		UserPreference = EditorUserSettings->NonRealtimeScreenPercentageMode;
		CVarDefaultScreenPercentage = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Editor.Viewport.ScreenPercentageMode.NonRealTime"));
	}
	else
	{
		unimplemented();
	}

	EScreenPercentageMode FinalScreenPercentageMode = EScreenPercentageMode::Manual;
	if (!bViewModeSupportsScreenPercentage)
	{
		FormatArguments.Add(
			TEXT("SettingSource"), LOCTEXT("ScreenPercentage_SettingSource_UnsupportedByViewMode", "Unsupported by View mode")
		);
		FinalScreenPercentageMode = EScreenPercentageMode::Manual;
		FinalScreenPercentage = 100;
	}
	else if (bIsPreviewScreenPercentage)
	{
		FormatArguments.Add(
			TEXT("SettingSource"), LOCTEXT("ScreenPercentage_SettingSource_ViewportOverride", "Viewport Override")
		);
		FinalScreenPercentageMode = EScreenPercentageMode::Manual;
	}
	else if ((CVarDefaultScreenPercentage->GetFlags() & ECVF_SetByMask) > ECVF_SetByProjectSetting)
	{
		FormatArguments.Add(TEXT("SettingSource"), LOCTEXT("ScreenPercentage_SettingSource_Cvar", "Console Variable"));
		FinalScreenPercentageMode = EScreenPercentageMode(CVarDefaultScreenPercentage->GetInt());
	}
	else if (UserPreference == EEditorUserScreenPercentageModeOverride::ProjectDefault)
	{
		FormatArguments.Add(
			TEXT("SettingSource"), LOCTEXT("ScreenPercentage_SettingSource_ProjectSettigns", "Project Settings")
		);
		FinalScreenPercentageMode = ProjectSetting;
	}
	else
	{
		FormatArguments.Add(
			TEXT("SettingSource"), LOCTEXT("ScreenPercentage_SettingSource_EditorPreferences", "Editor Preferences")
		);
		if (UserPreference == EEditorUserScreenPercentageModeOverride::BasedOnDPIScale)
		{
			FinalScreenPercentageMode = EScreenPercentageMode::BasedOnDPIScale;
		}
		else if (UserPreference == EEditorUserScreenPercentageModeOverride::BasedOnDisplayResolution)
		{
			FinalScreenPercentageMode = EScreenPercentageMode::BasedOnDisplayResolution;
		}
		else
		{
			FinalScreenPercentageMode = EScreenPercentageMode::Manual;
		}
	}

	if (FinalScreenPercentageMode == EScreenPercentageMode::BasedOnDPIScale)
	{
		FormatArguments.Add(TEXT("Setting"), LOCTEXT("ScreenPercentage_Setting_BasedOnDPIScale", "Based on OS's DPI scale"));
	}
	else if (FinalScreenPercentageMode == EScreenPercentageMode::BasedOnDisplayResolution)
	{
		FormatArguments.Add(
			TEXT("Setting"), LOCTEXT("ScreenPercentage_Setting_BasedOnDisplayResolution", "Based on display resolution")
		);
	}
	else
	{
		FormatArguments.Add(TEXT("Setting"), LOCTEXT("ScreenPercentage_Setting_Manual", "Manual"));
	}

	FormatArguments.Add(
		TEXT("CurrentScreenPercentage"),
		FText::FromString(FString::Printf(TEXT("%3.1f"), FMath::RoundToFloat(FinalScreenPercentage * 10.0f) / 10.0f))
	);

	{
		float FinalResolutionFraction = (FinalScreenPercentage / 100.0f);
		FIntPoint DisplayResolution = ViewportClient.Viewport->GetSizeXY();
		FIntPoint RenderingResolution;
		RenderingResolution.X = FMath::CeilToInt(DisplayResolution.X * FinalResolutionFraction);
		RenderingResolution.Y = FMath::CeilToInt(DisplayResolution.Y * FinalResolutionFraction);

		FormatArguments.Add(
			TEXT("ResolutionFromTo"),
			FText::FromString(FString::Printf(
				TEXT("%dx%d -> %dx%d"),
				RenderingResolution.X,
				RenderingResolution.Y,
				DisplayResolution.X,
				DisplayResolution.Y
			))
		);
	}

	return FormatArguments;
}

static const FMargin ScreenPercentageMenuCommonPadding(26.0f, 3.0f);

TSharedRef<SWidget> CreateCurrentPercentageWidget(FEditorViewportClient& InViewportClient)
{
	// clang-format off
	return SNew(SBox)
		.Padding(ScreenPercentageMenuCommonPadding)
		[
			SNew(STextBlock)
			.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			.Text_Lambda([&InViewportClient]()
			{
				FFormatNamedArguments FormatArguments = GetScreenPercentageFormatArguments(InViewportClient);
				return FText::Format(LOCTEXT("ScreenPercentageCurrent_Display", "Current Screen Percentage: {CurrentScreenPercentage}"), FormatArguments);
			})
			.ToolTip(SNew(SToolTip).Text(LOCTEXT("ScreenPercentageCurrent_ToolTip", "Current Screen Percentage the viewport is rendered with. The primary screen percentage can either be a spatial or temporal upscaler based of your anti-aliasing settings.")))
		];
	// clang-format on
}

TSharedRef<SWidget> CreateResolutionsWidget(FEditorViewportClient& InViewportClient)
{
	// clang-format off
	return SNew(SBox)
	.Padding(ScreenPercentageMenuCommonPadding)
	[
		SNew(STextBlock)
		.ColorAndOpacity(FSlateColor::UseSubduedForeground())
		.Text_Lambda([&InViewportClient]()
		{
			FFormatNamedArguments FormatArguments = GetScreenPercentageFormatArguments(InViewportClient);
			return FText::Format(LOCTEXT("ScreenPercentageResolutions", "Resolution: {ResolutionFromTo}"), FormatArguments);
		})
	];
	// clang-format on
}

TSharedRef<SWidget> CreateActiveViewportWidget(FEditorViewportClient& InViewPortClient)
{
	// clang-format off
	return SNew(SBox)
		.Padding(ScreenPercentageMenuCommonPadding)
		[
			SNew(STextBlock)
			.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			.Text_Lambda([&InViewPortClient]()
			{
				FFormatNamedArguments FormatArguments = GetScreenPercentageFormatArguments(InViewPortClient);
				return FText::Format(LOCTEXT("ScreenPercentageActiveViewport", "Active Viewport: {ViewportMode}"), FormatArguments);
			})
		];
	// clang-format on
}

TSharedRef<SWidget> CreateSetFromWidget(FEditorViewportClient& InViewPortClient)
{
	// clang-format off
	return SNew(SBox)
		.Padding(ScreenPercentageMenuCommonPadding)
		[
			SNew(STextBlock)
			.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			.Text_Lambda([&InViewPortClient]()
			{
				FFormatNamedArguments FormatArguments = GetScreenPercentageFormatArguments(InViewPortClient);
				return FText::Format(LOCTEXT("ScreenPercentageSetFrom", "Set From: {SettingSource}"), FormatArguments);
			})
		];
	// clang-format on
}

TSharedRef<SWidget> CreateCurrentScreenPercentageSettingWidget(FEditorViewportClient& InViewPortClient)
{
	// clang-format off
	return SNew(SBox)
		.Padding(ScreenPercentageMenuCommonPadding)
		[
			SNew(STextBlock)
			.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			.Text_Lambda([&InViewPortClient]()
			{
				FFormatNamedArguments FormatArguments = GetScreenPercentageFormatArguments(InViewPortClient);
				return FText::Format(LOCTEXT("ScreenPercentageSetting", "Setting: {Setting}"), FormatArguments);
			})
		];
	// clang-format on
}

TSharedRef<SWidget> CreateCurrentScreenPercentageWidget(FEditorViewportClient& InViewPortClient)
{
	constexpr int32 PreviewScreenPercentageMin = ISceneViewFamilyScreenPercentage::kMinTSRResolutionFraction * 100.0f;
	constexpr int32 PreviewScreenPercentageMax = ISceneViewFamilyScreenPercentage::kMaxTSRResolutionFraction * 100.0f;

	// clang-format off
	return SNew(SBox)
		.HAlign(HAlign_Right)
		.IsEnabled_Lambda([&InViewPortClient]()
		{
			return InViewPortClient.IsPreviewingScreenPercentage() && InViewPortClient.SupportsPreviewResolutionFraction();
		})
		[
			SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
			.WidthOverride(100.0f)
			[
				SNew(SBorder)
				.Padding(FMargin(1.0f))
				[
					SNew(SSpinBox<int32>)
					.Style(&FAppStyle::Get(), "Menu.SpinBox")
					.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
					.MinSliderValue(PreviewScreenPercentageMin)
					.MaxSliderValue(PreviewScreenPercentageMax)
					.Value_Lambda([&InViewPortClient]()
					{
						return InViewPortClient.GetPreviewScreenPercentage();
					})
					.OnValueChanged_Lambda([&InViewPortClient](int32 NewValue)
					{
						InViewPortClient.SetPreviewScreenPercentage(NewValue);
						InViewPortClient.Invalidate();
					})
				]
			]
		];
	// clang-format on
}

void ConstructScreenPercentageMenu(UToolMenu* InMenu)
{
	UUnrealEdViewportToolbarContext* const LevelViewportContext = InMenu->FindContext<UUnrealEdViewportToolbarContext>();
	if (!LevelViewportContext)
	{
		return;
	}

	const TSharedPtr<SEditorViewport> LevelViewport = LevelViewportContext->Viewport.Pin();
	if (!LevelViewport)
	{
		return;
	}

	FEditorViewportClient& ViewportClient = *LevelViewport->GetViewportClient();

	const FEditorViewportCommands& BaseViewportCommands = FEditorViewportCommands::Get();

	// Summary
	{
		FToolMenuSection& SummarySection = InMenu->FindOrAddSection("Summary", LOCTEXT("Summary", "Summary"));
		SummarySection.AddEntry(FToolMenuEntry::InitWidget(
			"ScreenPercentageCurrent", CreateCurrentPercentageWidget(ViewportClient), FText::GetEmpty()
		));

		SummarySection.AddEntry(FToolMenuEntry::InitWidget(
			"ScreenPercentageResolutions", CreateResolutionsWidget(ViewportClient), FText::GetEmpty()
		));

		SummarySection.AddEntry(FToolMenuEntry::InitWidget(
			"ScreenPercentageActiveViewport", CreateActiveViewportWidget(ViewportClient), FText::GetEmpty()
		));

		SummarySection.AddEntry(
			FToolMenuEntry::InitWidget("ScreenPercentageSetFrom", CreateSetFromWidget(ViewportClient), FText::GetEmpty())
		);

		SummarySection.AddEntry(FToolMenuEntry::InitWidget(
			"ScreenPercentageSetting", CreateCurrentScreenPercentageSettingWidget(ViewportClient), FText::GetEmpty()
		));
	}

	// Screen Percentage
	{
		FToolMenuSection& ScreenPercentageSection =
			InMenu->FindOrAddSection("ScreenPercentage", LOCTEXT("ScreenPercentage_ViewportOverride", "Viewport Override"));

		ScreenPercentageSection.AddMenuEntry(BaseViewportCommands.ToggleOverrideViewportScreenPercentage);

		ScreenPercentageSection.AddEntry(FToolMenuEntry::InitWidget(
			"PreviewScreenPercentage",
			CreateCurrentScreenPercentageWidget(ViewportClient),
			LOCTEXT("ScreenPercentage", "Screen Percentage")
		));
	}

	// Screen Percentage Settings
	{
		FToolMenuSection& ScreenPercentageSettingsSection = InMenu->FindOrAddSection(
			"ScreenPercentageSettings", LOCTEXT("ScreenPercentage_ViewportSettings", "Viewport Settings")
		);

		ScreenPercentageSettingsSection.AddMenuEntry(
			BaseViewportCommands.OpenEditorPerformanceProjectSettings,
			/* InLabelOverride = */ TAttribute<FText>(),
			/* InToolTipOverride = */ TAttribute<FText>(),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "ProjectSettings.TabIcon")
		);

		ScreenPercentageSettingsSection.AddMenuEntry(
			BaseViewportCommands.OpenEditorPerformanceEditorPreferences,
			/* InLabelOverride = */ TAttribute<FText>(),
			/* InToolTipOverride = */ TAttribute<FText>(),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorPreferences.TabIcon")
		);
	}
}

FToolMenuEntry CreateScreenPercentageSubmenu()
{
	FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
		"ScreenPercentage",
		LOCTEXT("ScreenPercentageSubMenu", "Screen Percentage"),
		LOCTEXT("ScreenPercentageSubMenu_ToolTip", "Customize the viewport's screen percentage"),
		FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* InMenu) -> void
			{
				ConstructScreenPercentageMenu(InMenu);
			}
		),
		false,
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "AssetEditor.ScreenPercentage")
	);

	return Entry;
}

FToolMenuEntry CreateScalabilitySubmenu()
{
	return FToolMenuEntry::InitSubMenu(
		"Scalability",
		LOCTEXT("ScalabilitySubMenu", "Viewport Scalability"),
		TAttribute<FText>::CreateLambda([]()
		{
			if (UE::UnrealEd::IsScalabilityWarningVisible())
			{
				return UE::UnrealEd::GetScalabilityWarningTooltip();
			}

			return LOCTEXT("ScalabilitySubMenu", "Viewport Scalability");
		}),
		FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* InMenu) -> void
			{
				FToolMenuSection& Section = InMenu->FindOrAddSection(NAME_None);
				Section.AddEntry(FToolMenuEntry::InitWidget(
					"ScalabilitySettings", SNew(SScalabilitySettings), FText(), true
				));
			}
		),
		FToolUIAction(),
		EUserInterfaceActionType::Button,
		false,
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.ViewportScalability")
	);
}

FToolMenuEntry CreateResetScalabilitySubmenu()
{
	// Hide this entry in non-toolbar contexts.
	// The actual Defaults button is handled by SScalabilitySettings
	FUIAction InvisibleAction;
	InvisibleAction.IsActionVisibleDelegate.BindLambda([] { return false; });
				
	FToolMenuEntry ResetToDefaultsEntry = FToolMenuEntry::InitMenuEntry(
		"ResetToDefault",
		LOCTEXT("ResetToDefaultLabel", "Reset To Defaults"),
		GetScalabilityWarningTooltip(),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.ViewportScalabilityReset"),
		FUIAction(InvisibleAction)
	);
				
	FToolUIAction ToolbarAction;
	ToolbarAction.ExecuteAction.BindLambda([](const FToolMenuContext& Context)
	{
		Scalability::ResetQualityLevelsToDefault();
		Scalability::SaveState(GEditorSettingsIni);
		GEditor->RedrawAllViewports();
	});
				
	ResetToDefaultsEntry.ToolBarData.ActionOverride = ToolbarAction;
	ResetToDefaultsEntry.ToolBarData.ResizeParams.AllowClipping = TAttribute<bool>::CreateLambda([]
	{
		return !UE::UnrealEd::IsScalabilityWarningVisible();
	});
	ResetToDefaultsEntry.SetShowInToolbarTopLevel(TAttribute<bool>::Create(&UE::UnrealEd::IsScalabilityWarningVisible));
	ResetToDefaultsEntry.StyleNameOverride = "ViewportToolbarWarning";
	
	return ResetToDefaultsEntry;
}

FText GetCameraSpeedTooltip()
{
	return LOCTEXT("CameraSpeedSubMenu_ToolTip", "Set the camera speed.\nShortcut: Hold either mouse button and use the scroll wheel.");
}

FToolMenuEntry CreateOrthographicClippingPlanesSubmenu(const FOrthographicClippingPlanesSubmenuOptions& InOptions)
{
	FToolUIAction Action;

	if (InOptions.bDisplayAsToggle)
	{
		Action.ExecuteAction.BindLambda([](const FToolMenuContext& Context)
		{
			if (const FUIAction* Action = Context.GetActionForCommand(FEditorViewportCommands::Get().UseOrthographicClippingPlanes))
			{
				Action->Execute();
			}
		});
		
		Action.GetActionCheckState.BindLambda([](const FToolMenuContext& Context)
		{
			if (const FUIAction* Action = Context.GetActionForCommand(FEditorViewportCommands::Get().UseOrthographicClippingPlanes))
			{
				return Action->GetCheckState();
			}
			return ECheckBoxState::Unchecked;
		});
	}

	return FToolMenuEntry::InitSubMenu(
		"OrthographicClippingPlanes",
		LOCTEXT("OrthographicClippingPlanesSubMenuLabel", "Ortho Clipping Planes"),
		LOCTEXT("OrthographicClippingPlanesSubMenuTooltip", "Control the Orthographic Clipping Planes"),
		FNewToolMenuDelegate::CreateLambda([](UToolMenu* Menu)
		{
			TSharedPtr<SEditorViewport> Viewport = UUnrealEdViewportToolbarContext::GetEditorViewport(Menu->Context);
			if (!Viewport || !Viewport->GetViewportClient())
			{
				return;
			}
			
			FToolMenuSection& ClippingSection = Menu->AddSection("ClippingPlanes", LOCTEXT("OrthographicClippingPlanesSectionLabel", "Ortho Clipping Planes"));
			
			TSharedRef<UE::UnrealEd::Private::SOrthographicViewPlaneSlider> FarSlider = SNew(
				UE::UnrealEd::Private::SOrthographicViewPlaneSlider,
				Viewport->GetViewportClient(),
				UE::UnrealEd::Private::SOrthographicViewPlaneSlider::EMode::Far
			);
			
			TSharedRef<UE::UnrealEd::Private::SOrthographicViewPlaneSlider> NearSlider = SNew(
				UE::UnrealEd::Private::SOrthographicViewPlaneSlider,
				Viewport->GetViewportClient(),
				UE::UnrealEd::Private::SOrthographicViewPlaneSlider::EMode::Near
			);
				
			FToolMenuEntry& FarPlaneEntry = ClippingSection.AddEntry(FToolMenuEntry::InitWidget("FarPlane", FarSlider, LOCTEXT("OrthoFarPlaneLabel", "Far Clip Location")));
			FarPlaneEntry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ViewportToolbar.FarViewPlane");
			
			FToolMenuEntry& NearPlaneEntry = ClippingSection.AddEntry(FToolMenuEntry::InitWidget("NearPlane", NearSlider, LOCTEXT("OrthoNearPlaneLabel", "Near Clip Location")));
			NearPlaneEntry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ViewportToolbar.NearViewPlane");
		}),
		Action,
		InOptions.bDisplayAsToggle ? EUserInterfaceActionType::ToggleButton : EUserInterfaceActionType::None,
		false,
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.OrthographicClippingSettings")	
	);
}

bool ShouldShowViewportRealtimeWarning(const FEditorViewportClient& ViewportClient)
{
	// Almost all usages of viewport realtime overrides are for overriding realtime to be true.
	// There are two exceptions:
	//  - Editor tools running in PIE, where they configure their client to be realtime and then override it to be true.
	//  - The Editor itself overriding realtime to false all viewport clients when the app goes into the background.
	// When realtime is disabled and Unreal goes into the background, the `false` override will match.
	// The realtime warning should be visible in this case, otherwise it will appear and disappear depending on whether
	// Unreal is the foreground app or not.
	return ViewportClient.DoRealtimeAndOverridesMatch(false) && ViewportClient.IsPerspective();
}

FToolMenuEntry CreatePerformanceAndScalabilitySubmenu()
{
	FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
		"PerformanceAndScalability",
		LOCTEXT("PerformanceAndScalabilitySubmenuLabel", "Performance and Scalability"),
		LOCTEXT("PerformanceAndScalabilitySubmenuTooltip", "Performance and scalability tools tied to this viewport."),
		FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* Submenu) -> void
			{
				FToolMenuSection& UnnamedSection = Submenu->FindOrAddSection(NAME_None);

				UnnamedSection.AddEntry(CreateToggleRealtimeEntry());

				if (const UUnrealEdViewportToolbarContext* Context = Submenu->FindContext<UUnrealEdViewportToolbarContext>())
				{
					UnnamedSection.AddEntry(CreateRemoveRealtimeOverrideEntry(Context->Viewport));
				}

				Submenu->AddMenuEntry("ScreenPercentageSubmenu", CreateScreenPercentageSubmenu());
			}
		)
	);
	Entry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Scalability");
	Entry.ToolBarData.LabelOverride = FText();
	Entry.ToolBarData.ResizeParams.ClippingPriority = 800;

	return Entry;
}

bool IsScalabilityWarningVisible()
{
	if (!GetDefault<UEditorPerformanceSettings>()->bEnableScalabilityWarningIndicator)
	{
		return false;
	}
	
	Scalability::FQualityLevels Current = Scalability::GetQualityLevels();
	Scalability::FQualityLevels Default = Scalability::GetDefaultQualityLevels();
	
	if (FMath::IsNearlyZero(Current.ResolutionQuality))
	{
		// Resolution quality gets initialized to 0.0 in a fresh project, thereby using project defaults.
		// This should still show up as "defaulted"
		Current.ResolutionQuality = 0.0f;
		Default.ResolutionQuality = 0.0f;	
	}
	
	return Current != Default;
}

FText GetScalabilityWarningLabel()
{
	if (UE::UnrealEd::IsScalabilityWarningVisible())
	{
		const int32 QualityLevel = Scalability::GetQualityLevels().GetMinQualityLevel();
		if (QualityLevel >= 0)
		{
			return FText::Format(
				LOCTEXT("ScalabilityWarning", "Scalability: {0}"),
				Scalability::GetScalabilityNameFromQualityLevel(QualityLevel)
			);
		}
	}

	return FText();
}

FText GetScalabilityWarningTooltip()
{
	return LOCTEXT(
		"ScalabilityWarning_ToolTip", "Non-default scalability settings could be affecting what is shown in this viewport.\nFor example you may experience lower visual quality, reduced particle counts, and other artifacts that don't match what the scene would look like when running outside of the editor.\n\nClick to reset scalability settings to default."
	);
}

FToolMenuEntry CreateShowSubmenu(const FNewToolMenuChoice& InSubmenuChoice)
{
	FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
    	"Show",
    	LOCTEXT("ShowSubmenuLabel", "Show"),
    	LOCTEXT("ShowSubmenuTooltip", "Show flags related to the current viewport"),
    	InSubmenuChoice
    );

    Entry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Level.VisibleHighlightIcon16x");
    Entry.ToolBarData.LabelOverride = FText();
    Entry.ToolBarData.ResizeParams.ClippingPriority = 800;

    return Entry;
}

FToolMenuEntry CreateDefaultShowSubmenu()
{
	return CreateShowSubmenu(FNewToolMenuDelegate::CreateStatic(&AddDefaultShowFlags));
}

void AddDefaultShowFlags(UToolMenu* InMenu)
{
	FShowFlagFilter ShowFlagFilter = FShowFlagFilter(FShowFlagFilter::IncludeAllFlagsByDefault);
	if (UUnrealEdViewportToolbarContext* EditorViewportContext = InMenu->FindContext<UUnrealEdViewportToolbarContext>())
	{
		for (const FEngineShowFlags::EShowFlag& Flag : EditorViewportContext->ExcludedShowMenuFlags)
		{
			ShowFlagFilter.ExcludeFlag(Flag);
		}

		for (const EShowFlagGroup& Flag : EditorViewportContext->ExcludedShowMenuGroupFlags)
		{
			ShowFlagFilter.ExcludeGroup(Flag);
		}
	}

	{
		FToolMenuSection& CommonShowFlagsSection =
			InMenu->FindOrAddSection("CommonShowFlags", LOCTEXT("CommonShowFlagsLabel", "Common Show Flags"));

		FShowFlagMenuCommands::Get().PopulateCommonShowFlagsSection(CommonShowFlagsSection, ShowFlagFilter);
	}

	{
		FToolMenuSection& AllShowFlagsSection =
			InMenu->FindOrAddSection("AllShowFlags", LOCTEXT("AllShowFlagsLabel", "All Show Flags"));

		FShowFlagMenuCommands::Get().PopulateAllShowFlagsSection(AllShowFlagsSection, ShowFlagFilter);
	}
}

FToolMenuEntry CreateToggleRealtimeEntry()
{
	return FToolMenuEntry::InitDynamicEntry(
		"ToggleRealtimeDynamicSection",
		FNewToolMenuSectionDelegate::CreateLambda(
			[](FToolMenuSection& InnerSection) -> void
			{
				UUnrealEdViewportToolbarContext* const EditorViewportContext = InnerSection.FindContext<UUnrealEdViewportToolbarContext>();
				if (!EditorViewportContext)
				{
					return;
				}

				TWeakPtr<SEditorViewport> EditorViewportWeak = EditorViewportContext->Viewport;
				
				FToolUIAction RealtimeToggleAction;
				RealtimeToggleAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda(
					[EditorViewportWeak](const FToolMenuContext& Context) -> void
					{
						if (TSharedPtr<SEditorViewport> EditorViewport = EditorViewportWeak.Pin())
						{
							EditorViewport->OnToggleRealtime();
						}
					}
				);

				RealtimeToggleAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateLambda(
					[EditorViewportWeak](const FToolMenuContext& Context) -> bool
					{
						if (TSharedPtr<SEditorViewport> EditorViewport = EditorViewportWeak.Pin())
						{
							return !EditorViewport->GetViewportClient()->IsRealtimeOverrideSet();
						}
						return true;
					}
				);

				RealtimeToggleAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateLambda(
					[EditorViewportWeak](const FToolMenuContext& Context) -> ECheckBoxState
					{
						if (TSharedPtr<SEditorViewport> EditorViewport = EditorViewportWeak.Pin())
						{
							return EditorViewport->IsRealtime() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
						}
						return ECheckBoxState::Undetermined;
					}
				);

				TAttribute<FText> Tooltip = TAttribute<FText>::CreateLambda([EditorViewportWeak]
				{
					if (const TSharedPtr<SEditorViewport> EditorViewport = EditorViewportWeak.Pin())
					{
						if (EditorViewport->GetViewportClient()->IsRealtimeOverrideSet())
						{
							const FText Format = LOCTEXT(
								"ToggleRealtimeTooltip_RealtimeOverrides",
								"Realtime rendering cannot be toggled because an override has been set: {0}"
							);
							const FText Message =
								EditorViewport->GetViewportClient()->GetRealtimeOverrideMessage();

							return FText::Format(Format, Message);
						}
						
						if (!EditorViewport->IsRealtime())
						{
							return LOCTEXT(
								"ToggleRealtimeTooltip_WarnRealtimeOff", "Warning: This viewport is not updating in realtime. Click to turn on realtime mode."
							);
						}
					}

					return LOCTEXT("ToggleRealtimeTooltip", "Toggle realtime rendering of the viewport");
				});
				
				TAttribute<FSlateIcon> Icon = TAttribute<FSlateIcon>::CreateLambda([EditorViewportWeak]
				{
					if (const TSharedPtr<SEditorViewport> EditorViewport = EditorViewportWeak.Pin())
					{
						if (TSharedPtr<FEditorViewportClient> Client = EditorViewport->GetViewportClient())
						{
							if (Client->IsRealtimeOverrideSet())
							{
								return FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.ToggleRealTimeLocked");
							}
						}
					}

					return FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.RealTimeReset");
				});

				FToolMenuEntry ToggleRealtime = FToolMenuEntry::InitMenuEntry(
					"ToggleRealtime",
					LOCTEXT("ToggleRealtimeLabel", "Realtime Viewport"),
					Tooltip,
					Icon,
					RealtimeToggleAction,
					EUserInterfaceActionType::ToggleButton
				);

				ToggleRealtime.SetShowInToolbarTopLevel(TAttribute<bool>::CreateLambda(
					[WeakViewport = EditorViewportContext->Viewport]() -> bool
					{
						if (const TSharedPtr<SEditorViewport> EditorViewport = WeakViewport.Pin())
						{
							return ShouldShowViewportRealtimeWarning(*EditorViewport->GetViewportClient());
						}
						return false;
					}
				));

				ToggleRealtime.ToolBarData.ResizeParams.AllowClipping = TAttribute<bool>::CreateLambda(
					[EditorViewportWeak]() -> bool
					{
						if (const TSharedPtr<SEditorViewport> EditorViewport = EditorViewportWeak.Pin())
						{
							// Don't allow this entry to be clipped from the toolbar when the viewport isn't realtime.
							// This to avoid that the non-relatime-viewport warning is hidden as the toolbar is shrunk.
							if (!EditorViewport->IsRealtime())
							{
								return false;
							}
						}

						return true;
					}
				);

				// We need the warning style when we've raised this entry to the top-level toolbar to draw
				// attention to it. However, styles are not driven by an attribute, so they cannot be updated after an
				// FToolMenuEntry has been created. Therefor we always apply the style instead because the style
				// will not make this entry look different when it appears in a menu, but when it appears in a toolbar
				// we get the desired warning effect.
				ToggleRealtime.StyleNameOverride = "ViewportToolbarWarning";

				InnerSection.AddEntry(ToggleRealtime);
			}
		)
	);
}

FToolMenuEntry CreateRemoveRealtimeOverrideEntry(TWeakPtr<SEditorViewport> WeakViewport)
{
	FUIAction Action;
	Action.ExecuteAction = FExecuteAction::CreateLambda(
		[WeakViewport]()
		{
			if (TSharedPtr<SEditorViewport> ViewportPinned = WeakViewport.Pin())
			{
				ViewportPinned->GetViewportClient()->PopRealtimeOverride();
			}
		}
	);
	Action.IsActionVisibleDelegate = FIsActionButtonVisible::CreateLambda(
		[WeakViewport]() -> bool
		{
			if (TSharedPtr<SEditorViewport> ViewportPinned = WeakViewport.Pin())
			{
				return ViewportPinned->GetViewportClient()->IsRealtimeOverrideSet();
			}

			return false;
		}
	);

	TAttribute<FText> Tooltip = TAttribute<FText>::CreateLambda(
		[WeakViewport]()
		{
			if (TSharedPtr<SEditorViewport> ViewportPinned = WeakViewport.Pin())
			{
				return FText::Format(
					LOCTEXT(
						"DisableRealtimeOverrideToolTip", "Realtime is currently overridden by \"{0}\". Click to remove that override."
					),
					ViewportPinned->GetViewportClient()->GetRealtimeOverrideMessage()
				);
			}

			return FText::GetEmpty();
		}
	);

	return FToolMenuEntry::InitMenuEntry(
		"DisableRealtimeOverride", LOCTEXT("DisableRealtimeOverride", "Disable Realtime Override"), Tooltip, FSlateIcon(), Action
	);
}

FOnViewportClientCamSpeedChanged& OnViewportClientCamSpeedChanged()
{
	static FOnViewportClientCamSpeedChanged OnViewportClientCamSpeedChangedDelegate;
	return OnViewportClientCamSpeedChangedDelegate;
}

FOnViewportClientCamSpeedScalarChanged& OnViewportClientCamSpeedScalarChanged()
{
	static FOnViewportClientCamSpeedScalarChanged OnViewportClientCamSpeedScalarChangedDelegate;
	return OnViewportClientCamSpeedScalarChangedDelegate;
}

FToolMenuEntry CreateCameraSpeedEntry(const TWeakPtr<::SEditorViewport>& InEditorViewportWeak)
{
	FToolMenuEntry CameraSpeedSliderEntry = FToolMenuEntry::InitWidget(
		"CameraSpeed",
		CreateCameraSpeedWidget(InEditorViewportWeak),
		FText::GetEmpty()
	);
	
	CameraSpeedSliderEntry.ToolTip = LOCTEXT("CameraSpeedSliderTooltip", "The speed of the camera when moving around using the keyboard (WASD).");

	return CameraSpeedSliderEntry;
}

FToolMenuEntry CreateCameraSpeedMenu()
{
	return FToolMenuEntry::InitDynamicEntry(
		"CameraSpeedMenuDynamicSection",
		FNewToolMenuSectionDelegate::CreateLambda(
			[](FToolMenuSection& InDynamicSection)
			{
				FToolMenuEntry& Entry = InDynamicSection.AddSubMenu(
					"CameraSpeed",
					LOCTEXT("CameraSpeedSubMenu", "Camera Speed"),
					UE::UnrealEd::GetCameraSpeedTooltip(),
					FNewToolMenuDelegate::CreateLambda(
						[](UToolMenu* InMenu)
						{
							FToolMenuSection& CameraSpeedSection =
								InMenu->AddSection("CameraSpeed", LOCTEXT("CameraSpeedLabel", "Camera Speed"));

							if (UUnrealEdViewportToolbarContext* const Context =
									InMenu->FindContext<UUnrealEdViewportToolbarContext>())
							{
								CameraSpeedSection.AddEntry(CreateCameraSpeedEntry(Context->Viewport));
							}
							
							FToolMenuSection& SettingsSection = InMenu->AddSection("Settings", LOCTEXT("CameraSpeedSettingsSectionLabel", "Settings"));
							SettingsSection.AddMenuEntry(FEditorViewportCommands::Get().ToggleDistanceBasedCameraSpeed);
						}
					),
					false,
					FSlateIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ViewportToolbar.CameraSpeed"))
				);

				if (UUnrealEdViewportToolbarContext* Context =
						InDynamicSection.FindContext<UUnrealEdViewportToolbarContext>())
				{
					Entry.ToolBarData.LabelOverride = TAttribute<FText>::CreateLambda(
						[WeakViewport = Context->Viewport]()
						{
							return UE::UnrealEd::GetCameraSpeedLabel(WeakViewport);
						}
					);

					TAttribute<bool> IsPerspectiveLambda = TAttribute<bool>::CreateLambda(
						[WeakViewport = Context->Viewport]() -> bool
						{
							if (const TSharedPtr<SEditorViewport>& ViewportPinned = WeakViewport.Pin())
							{
								if (const TSharedPtr<FEditorViewportClient>& ViewportClient =
										ViewportPinned->GetViewportClient())
								{
									return ViewportClient->IsPerspective();
								}
							}

							return false;
						}
					);

					// Don't show camera speed entries (in-menu and raised) if viewport is orthographic (or missing)
					Entry.SetShowInToolbarTopLevel(true);
					Entry.Visibility = IsPerspectiveLambda;

					// Reset camera speed entry
					{
						// This entry is a toolbar only warning, telling the user that camera speed is extremely low and might cause imperceptible camera movements.

						FUIAction HideInMenuAction;
						HideInMenuAction.IsActionVisibleDelegate.BindLambda([]{ return false; });

						FToolMenuEntry ResetSpeedEntry = FToolMenuEntry::InitMenuEntry(
							"LowCameraSpeedWarning",
							LOCTEXT("LowCameraSpeedWarningLabel", "Reset Camera Speed"),
							LOCTEXT("LowCameraSpeedWarningTooltip", "Camera speed is extremely low. This might cause imperceptible camera movements.\nClick to reset camera speed to its default."),
							FSlateIcon(FAppStyle::GetAppStyleSetName(), "ViewportToolbar.CameraSpeedReset"),
							HideInMenuAction
						);

						ResetSpeedEntry.ToolBarData.ActionOverride = FToolMenuExecuteAction::CreateLambda(
							[WeakViewport = Context->Viewport](const FToolMenuContext&)
							{
								if (const TSharedPtr<SEditorViewport> ViewportPinned = WeakViewport.Pin())
								{
									if (const TSharedPtr<FEditorViewportClient> ViewportClient = ViewportPinned->GetViewportClient())
									{
										const float DefaultSpeed = FEditorViewportCameraSpeedSettings{}.GetCurrentSpeed();
										FEditorViewportCameraSpeedSettings Settings = ViewportClient->GetCameraSpeedSettings();
										Settings.SetCurrentSpeed(DefaultSpeed);
										ViewportClient->SetCameraSpeedSettings(Settings);
									}
								}
							}
						);

						ResetSpeedEntry.SetShowInToolbarTopLevel(
							TAttribute<bool>::CreateLambda(
								[WeakViewport = Context->Viewport]() -> bool
								{
									if (const TSharedPtr<SEditorViewport> ViewportPinned = WeakViewport.Pin())
									{
										if (const TSharedPtr<FEditorViewportClient> ViewportClient = ViewportPinned->GetViewportClient())
										{
											if (ViewportClient->IsPerspective())
											{
												const float CurrentSpeed = ViewportClient->GetCameraSpeedSettings().GetCurrentSpeed();
												const float LowSpeedThreshold = ViewportClient->GetCameraSpeedSettings().GetLowSpeedWarningThreshold();
												return CurrentSpeed <= LowSpeedThreshold;
											}
										}
									}
									return false;
								}
							)
						);
						ResetSpeedEntry.StyleNameOverride = "ViewportToolbarWarning";

						InDynamicSection.AddEntry(ResetSpeedEntry);
					}
				}

				Entry.ToolBarData.PlacementOverride = MenuPlacement_BelowRightAnchor;
			}
		)
	);
}

TAttribute<EVisibility> GetPerspectiveOnlyVisibility(const TSharedPtr<FEditorViewportClient>& InViewportClient)
{
	return TAttribute<EVisibility>::CreateLambda([IsPerspective = GetIsPerspectiveAttribute(InViewportClient)]
	{
		return IsPerspective.Get() ? EVisibility::Visible : EVisibility::Collapsed;
	});
}

TAttribute<bool> GetIsPerspectiveAttribute(const TSharedPtr<FEditorViewportClient>& InViewportClient)
{
	if (!InViewportClient.IsValid())
	{
		return false;
	}

	return TAttribute<bool>::CreateLambda(
		[WeakViewport = InViewportClient.ToWeakPtr()]
		{
			if (const TSharedPtr<FEditorViewportClient>& ViewportClient = WeakViewport.Pin())
			{
				return ViewportClient->IsPerspective();
			}
			return false;
		}
	);
}

TAttribute<bool> GetIsOrthographicAttribute(const TSharedPtr<FEditorViewportClient>& InViewportClient)
{
	if (!InViewportClient.IsValid())
	{
		return false;
	}

	return TAttribute<bool>::CreateLambda(
		[WeakViewport = InViewportClient.ToWeakPtr()]
		{
			if (const TSharedPtr<FEditorViewportClient>& ViewportClient = WeakViewport.Pin())
			{
				return ViewportClient->IsOrtho();
			}
			return false;
		}
	);
}

} // namespace UE::UnrealEd

#undef LOCTEXT_NAMESPACE
