// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportToolbar/UnrealEdViewportToolbarContext.h"

#include "SEditorViewport.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "ToolMenuContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UnrealEdViewportToolbarContext)

#define LOCTEXT_NAMESPACE "UnrealEdViewportToolbar"

TSharedPtr<SEditorViewport> UUnrealEdViewportToolbarContext::GetEditorViewport(const FToolMenuContext& Context)
{
	if (const UUnrealEdViewportToolbarContext* ContextObject = Context.FindContext<UUnrealEdViewportToolbarContext>())
	{
		return ContextObject->Viewport.Pin();
	}
	return nullptr;
}

TSharedPtr<FEditorViewportClient> UUnrealEdViewportToolbarContext::GetEditorViewportClient(const FToolMenuContext& Context)
{
	if (TSharedPtr<SEditorViewport> Viewport = GetEditorViewport(Context))
	{
		return Viewport->GetViewportClient();
	}
	return nullptr;
}

TSharedPtr<IPreviewProfileController> UUnrealEdViewportToolbarContext::GetPreviewProfileController() const
{
	if (const TSharedPtr<SEditorViewport> EditorViewport = Viewport.Pin())
	{
		return EditorViewport->GetPreviewProfileController();	
	}
	return nullptr;
}

void UUnrealEdViewportToolbarContext::RefreshViewport()
{
	if (TSharedPtr<SEditorViewport> EditorViewport = Viewport.Pin())
	{
		if (TSharedPtr<FEditorViewportClient> Client = EditorViewport->GetViewportClient())
		{
			Client->Invalidate();
		}
	}
}

FText UUnrealEdViewportToolbarContext::GetGridSnapLabel() const
{
	return FText::AsNumber(GEditor->GetGridSize());
}

TArray<float> UUnrealEdViewportToolbarContext::GetGridSnapSizes() const
{
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
	if (ViewportSettings->bUsePowerOf2SnapSize)
	{
		return ViewportSettings->Pow2GridSizes;
	}
	return ViewportSettings->DecimalGridSizes;
}

bool UUnrealEdViewportToolbarContext::IsGridSnapSizeActive(int32 GridSizeIndex) const
{
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
	return ViewportSettings->CurrentPosGridSize == GridSizeIndex;
}

void UUnrealEdViewportToolbarContext::SetGridSnapSize(int32 GridSizeIndex)
{
	GEditor->SetGridSize(GridSizeIndex);
}

FText UUnrealEdViewportToolbarContext::GetRotationSnapLabel() const
{
	return FText::Format(
		LOCTEXT("GridRotation - Number - DegreeSymbol", "{0}\u00b0"),
		FText::AsNumber(GEditor->GetRotGridSize().Pitch)
	);
}

bool UUnrealEdViewportToolbarContext::IsRotationSnapActive(int32 RotationIndex, ERotationGridMode RotationMode) const
{
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
	return ViewportSettings->CurrentRotGridSize == RotationIndex
		&& ViewportSettings->CurrentRotGridMode == RotationMode;
}

void UUnrealEdViewportToolbarContext::SetRotationSnapSize(int32 RotationIndex, ERotationGridMode RotationMode)
{
	GEditor->SetRotGridSize(RotationIndex, RotationMode);
}

FText UUnrealEdViewportToolbarContext::GetScaleSnapLabel() const
{
	return UE::UnrealEd::GetScaleGridLabel();
}

TArray<float> UUnrealEdViewportToolbarContext::GetScaleSnapSizes() const
{
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
	return ViewportSettings->ScalingGridSizes;
}

bool UUnrealEdViewportToolbarContext::IsScaleSnapActive(int32 ScaleIndex) const
{
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
	return ViewportSettings->CurrentScalingGridSize == ScaleIndex;
}

void UUnrealEdViewportToolbarContext::SetScaleSnapSize(int32 ScaleIndex)
{
	GEditor->SetScaleGridSize(ScaleIndex);
}

#undef LOCTEXT_NAMESPACE
