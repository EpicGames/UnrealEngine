// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	VolumeTexturePreview.h: Implementation for previewing Volume textures.
==============================================================================*/

#include "VolumeTexturePreview.h"
#include "Engine/Texture2D.h"
#include "Shader.h"
#include "GlobalShader.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"
#include "SimpleElementShaders.h"
#include "ShaderParameterUtils.h"
#include "PipelineStateCache.h"
#include "Editor.h"
#include "TextureResource.h"
#include "GlobalRenderResources.h"
#include "DataDrivenShaderPlatformInfo.h"

UNREALED_API void GetBestFitForNumberOfTiles(int32 InSize, int32& OutNumTilesX, int32& OutNumTilesY)
{
	const float Ratios[] = { 1.f, 1.2f, 1.25f, 1.33f, 1.5f, 1.77f, 2.f, 3.f };

	OutNumTilesX = InSize;
	OutNumTilesY = 1;

	int32 OutError = InSize;

	for (float Ratio : Ratios)
	{
		int32 NumTilesY = (int32)FMath::RoundToInt(FMath::Sqrt((float)InSize / Ratio));
		int32 NumTilesX = (int32)FMath::RoundToInt((float)NumTilesY * Ratio);
		int32 Error = NumTilesX * NumTilesY - InSize;

		if (Error >= 0 && Error < OutError)
		{
			OutError = Error;
			OutNumTilesX = NumTilesX;
			OutNumTilesY = NumTilesY;
		}
	}
}

/*------------------------------------------------------------------------------
	Batched element shaders for previewing 2d textures.
------------------------------------------------------------------------------*/
/**
 * Simple pixel shader for previewing volume textures at a specified mip level
 */
class FSimpleElementVolumeTexturePreviewPS : public FGlobalShader
{
	DECLARE_INLINE_TYPE_LAYOUT(FSimpleElementVolumeTexturePreviewPS, NonVirtual);
public:

	FSimpleElementVolumeTexturePreviewPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer) :
		FGlobalShader(Initializer)
	{
		InTexture.Bind(Initializer.ParameterMap,TEXT("InTexture"), SPF_Mandatory);
		InTextureSampler.Bind(Initializer.ParameterMap,TEXT("InTextureSampler"));	
		BadTexture.Bind(Initializer.ParameterMap,TEXT("BadTexture"));
		BadTextureSampler.Bind(Initializer.ParameterMap,TEXT("BadTextureSampler"));	
		TextureComponentReplicate.Bind(Initializer.ParameterMap,TEXT("TextureComponentReplicate"));
		TextureComponentReplicateAlpha.Bind(Initializer.ParameterMap,TEXT("TextureComponentReplicateAlpha"));
		ColorWeights.Bind(Initializer.ParameterMap,TEXT("ColorWeights"));
		PackedParameters.Bind(Initializer.ParameterMap,TEXT("PackedParams"));
		NumTilesPerSideParameter.Bind(Initializer.ParameterMap,TEXT("NumTilesPerSide"));
		TraceVolumeScalingParameter.Bind(Initializer.ParameterMap,TEXT("TraceVolumeScaling"));
		TextureDimensionParameter.Bind(Initializer.ParameterMap,TEXT("TextureDimension"));
		TraceViewMatrixParameter.Bind(Initializer.ParameterMap,TEXT("TraceViewMatrix"));
	}

	FSimpleElementVolumeTexturePreviewPS() {}

	/** Should the shader be cached? Always. */
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) && !IsConsolePlatform(Parameters.Platform);
	}
	
	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FTexture* TextureValue, const FMatrix44f& ColorWeightsValue, float GammaValue, float MipLevel, float Opacity, const FRotator& TraceOrientation, bool bUsePointSampling)
	{
		FRHISamplerState* SamplerState = bUsePointSampling ? TStaticSamplerState<SF_Point>::GetRHI() : TextureValue->SamplerStateRHI.GetReference();
		SetTextureParameter(BatchedParameters, InTexture, InTextureSampler, SamplerState, TextureValue->TextureRHI);

		if (GEditor && GEditor->Bad)
		{
			SetTextureParameter(BatchedParameters, BadTexture, BadTextureSampler, GEditor->Bad->GetResource());
		}
		else
		{
			SetTextureParameter(BatchedParameters, BadTexture, GWhiteTexture->TextureRHI);
		}
		SetShaderValue(BatchedParameters,ColorWeights,ColorWeightsValue);

		const int32 SizeZ = TextureValue->GetSizeZ();
		const int32 MipSizeZ = MipLevel >= 0 ? FMath::Max<int32>(SizeZ >> FMath::FloorToInt(MipLevel), 1) : SizeZ;
		FVector4f PackedParametersValue(GammaValue, MipLevel, (float)MipSizeZ, Opacity);
		SetShaderValue(BatchedParameters, PackedParameters, PackedParametersValue);

		int32 NumTilesX = 0;
		int32 NumTilesY = 0;
		GetBestFitForNumberOfTiles(MipSizeZ, NumTilesX, NumTilesY);
		SetShaderValue(BatchedParameters, NumTilesPerSideParameter, FVector4f((float)NumTilesX, (float)NumTilesY, 0 ,0));

		SetShaderValue(BatchedParameters,TextureComponentReplicate,TextureValue->bGreyScaleFormat ? FLinearColor(1,0,0,0) : FLinearColor(0,0,0,0));
		SetShaderValue(BatchedParameters,TextureComponentReplicateAlpha,TextureValue->bGreyScaleFormat ? FLinearColor(1,0,0,0) : FLinearColor(0,0,0,1));

		const FVector3f TextureDimension((float)TextureValue->GetSizeX(), (float)TextureValue->GetSizeY(), (float)SizeZ);
		const float OneOverMinDimension = 1.f / FMath::Max(TextureDimension.GetMin(), 1.f);
		SetShaderValue(BatchedParameters, TraceVolumeScalingParameter, FVector4f(
				TextureDimension.X * OneOverMinDimension, 
				TextureDimension.Y * OneOverMinDimension, 
				TextureDimension.Z * OneOverMinDimension, 
				TextureDimension.GetMax() * OneOverMinDimension * .5f) // Extent
			);

		SetShaderValue(BatchedParameters, TextureDimensionParameter, FVector3f(TextureDimension.X, TextureDimension.Y, TextureDimension.Z));

		SetShaderValue(BatchedParameters, TraceViewMatrixParameter, FMatrix44f(FRotationMatrix::Make(TraceOrientation)));
	}

private:
	
	LAYOUT_FIELD(FShaderResourceParameter, InTexture);
	LAYOUT_FIELD(FShaderResourceParameter, InTextureSampler);
	LAYOUT_FIELD(FShaderResourceParameter, BadTexture);
	LAYOUT_FIELD(FShaderResourceParameter, BadTextureSampler);
	LAYOUT_FIELD(FShaderParameter, TextureComponentReplicate);
	LAYOUT_FIELD(FShaderParameter, TextureComponentReplicateAlpha);
	LAYOUT_FIELD(FShaderParameter, ColorWeights);
	LAYOUT_FIELD(FShaderParameter, PackedParameters);
	LAYOUT_FIELD(FShaderParameter, NumTilesPerSideParameter);
	LAYOUT_FIELD(FShaderParameter, TraceVolumeScalingParameter);
	LAYOUT_FIELD(FShaderParameter, TextureDimensionParameter);
	LAYOUT_FIELD(FShaderParameter, TraceViewMatrixParameter);
};

class FVolumeTextureTilePreviewPS : public FSimpleElementVolumeTexturePreviewPS
{
	DECLARE_SHADER_TYPE(FVolumeTextureTilePreviewPS,Global);
public:
	FVolumeTextureTilePreviewPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer) : FSimpleElementVolumeTexturePreviewPS(Initializer) {}
	FVolumeTextureTilePreviewPS() {}
};

class FVolumeTextureTracePreviewPS : public FSimpleElementVolumeTexturePreviewPS
{
	DECLARE_SHADER_TYPE(FVolumeTextureTracePreviewPS,Global);
public:
	FVolumeTextureTracePreviewPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer) : FSimpleElementVolumeTexturePreviewPS(Initializer) {}
	FVolumeTextureTracePreviewPS() {}
};


IMPLEMENT_SHADER_TYPE(,FVolumeTextureTilePreviewPS,TEXT("/Engine/Private/SimpleElementVolumeTexturePreviewPixelShader.usf"),TEXT("TileMain"),SF_Pixel);
IMPLEMENT_SHADER_TYPE(,FVolumeTextureTracePreviewPS,TEXT("/Engine/Private/SimpleElementVolumeTexturePreviewPixelShader.usf"),TEXT("TraceMain"),SF_Pixel);

/** Binds vertex and pixel shaders for this element */
void FBatchedElementVolumeTexturePreviewParameters::BindShaders(
	FRHICommandList& RHICmdList,
	FGraphicsPipelineStateInitializer& GraphicsPSOInit,
	ERHIFeatureLevel::Type InFeatureLevel,
	const FMatrix& InTransform,
	const float InGamma,
	const FMatrix& InColorWeights,
	const FTexture* Texture)
{
	TShaderMapRef<FSimpleElementVS> VertexShader(GetGlobalShaderMap(InFeatureLevel));

	TShaderRef<FSimpleElementVolumeTexturePreviewPS> PixelShader ;
	if (bViewModeAsDepthSlices)
	{
		TShaderMapRef<FVolumeTextureTilePreviewPS> TileShader(GetGlobalShaderMap(InFeatureLevel));
		PixelShader = TileShader;
	}
	else
	{
		TShaderMapRef<FVolumeTextureTracePreviewPS> TileShader(GetGlobalShaderMap(InFeatureLevel));
		PixelShader = TileShader;
	}


	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GSimpleElementVertexDeclaration.VertexDeclarationRHI;
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;

	if (!bViewModeAsDepthSlices)
	{
		GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One>::GetRHI();
	}

	FMatrix44f ColorWeights = FMatrix44f(InColorWeights);
	if (!bViewModeAsDepthSlices && ColorWeights.M[3][3] == 0)
	{
		const float XWeight = ColorWeights.M[0][0] + ColorWeights.M[1][0] + ColorWeights.M[2][0];
		const float YWeight = ColorWeights.M[0][1] + ColorWeights.M[1][1] + ColorWeights.M[2][1];
		const float ZWeight = ColorWeights.M[0][2] + ColorWeights.M[1][2] + ColorWeights.M[2][2];
		const float OneOverWeightSum = 1.f / FMath::Max(SMALL_NUMBER, XWeight + YWeight + ZWeight);
		ColorWeights.M[3][0] = XWeight * OneOverWeightSum;
		ColorWeights.M[3][1] = YWeight * OneOverWeightSum;
		ColorWeights.M[3][2] = ZWeight * OneOverWeightSum;
	}

	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

	SetShaderParametersLegacyVS(RHICmdList, VertexShader, InTransform);
	SetShaderParametersLegacyPS(RHICmdList, PixelShader, Texture, ColorWeights, InGamma, MipLevel, Opacity, TraceOrientation, bUsePointSampling);
}
