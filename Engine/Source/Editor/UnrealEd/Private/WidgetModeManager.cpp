// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetModeManager.h"

#include "EditorGizmos/EditorTransformGizmoUtil.h"
#include "Snapping/EditorSnappingManager.h"
#include "ToolContexts/WidgetToolsContext.h"

//////////////////////////////////////////////////////////////////////////
// FWidgetModeManager

FWidgetModeManager::FWidgetModeManager()
	: FEditorModeTools()
{
	// Since we can't call virtuals in constructor, need free and re-create InteractiveToolsContext resource
	if (UObjectInitialized())
	{
		UE::EditorTransformGizmoUtil::UnregisterTransformGizmoContextObject(this);
		UE::Editor::Gizmos::DeregisterSceneSnappingManager(InteractiveToolsContext);
		InteractiveToolsContext->ShutdownContext();
		InteractiveToolsContext = nullptr;
	}

	CachedWidgetToolContext = NewObject<UWidgetToolsContext>(GetTransientPackage(), UWidgetToolsContext::StaticClass(), NAME_None, RF_Transient);
	InteractiveToolsContext = CachedWidgetToolContext;
	InteractiveToolsContext->InitializeContextWithEditorModeManager(this);
	UE::EditorTransformGizmoUtil::RegisterTransformGizmoContextObject(this);
	UE::Editor::Gizmos::RegisterSceneSnappingManager(InteractiveToolsContext);
}

bool FWidgetModeManager::OnKeyChar(const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent)
{
	if (CachedWidgetToolContext)
	{
		return CachedWidgetToolContext->OnKeyChar(MyGeometry, InCharacterEvent);
	}

	return false;
}

bool FWidgetModeManager::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CachedWidgetToolContext)
	{
		return CachedWidgetToolContext->OnKeyDown(MyGeometry, InKeyEvent);
	}

	return false;
}

bool FWidgetModeManager::OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CachedWidgetToolContext)
	{
		return CachedWidgetToolContext->OnKeyUp(MyGeometry, InKeyEvent);
	}

	return false;
}

bool FWidgetModeManager::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (CachedWidgetToolContext)
	{
		return CachedWidgetToolContext->OnMouseButtonDown(MyGeometry, MouseEvent);
	}

	return false;
}

bool FWidgetModeManager::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (CachedWidgetToolContext)
	{
		return CachedWidgetToolContext->OnMouseButtonUp(MyGeometry, MouseEvent);
	}

	return false;
}

bool FWidgetModeManager::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (CachedWidgetToolContext)
	{
		return CachedWidgetToolContext->OnMouseMove(MyGeometry, MouseEvent);
	}

	return false;
}

bool FWidgetModeManager::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (CachedWidgetToolContext)
	{
		return CachedWidgetToolContext->OnMouseWheel(MyGeometry, MouseEvent);
	}

	return false;
}

bool FWidgetModeManager::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (CachedWidgetToolContext)
	{
		return CachedWidgetToolContext->OnMouseButtonDoubleClick(InMyGeometry, InMouseEvent);
	}

	return false;
}

bool FWidgetModeManager::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (CachedWidgetToolContext)
	{
		return CachedWidgetToolContext->OnDragDetected(MyGeometry, MouseEvent);
	}

	return false;
}

void FWidgetModeManager::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	if (CachedWidgetToolContext)
	{
		CachedWidgetToolContext->OnMouseCaptureLost(CaptureLostEvent);
	}
}

int32 FWidgetModeManager::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled)
{
	if (CachedWidgetToolContext)
	{
		return CachedWidgetToolContext->OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	}

	return LayerId;
}

void FWidgetModeManager::SetManagedWidget(TSharedPtr<SWidget> InManagedWidget)
{
	ManagedWidget = InManagedWidget;
}

TSharedPtr<SWidget> FWidgetModeManager::GetManagedWidget() const
{
	return ManagedWidget.Pin();
}
