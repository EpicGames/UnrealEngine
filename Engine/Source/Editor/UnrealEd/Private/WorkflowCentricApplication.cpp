// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorkflowOrientedApp/WorkflowCentricApplication.h"
#include "Framework/Application/SlateApplication.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"
#include "WorkflowOrientedApp/ApplicationMode.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "WorkflowCentricApplication"

DEFINE_LOG_CATEGORY_STATIC(LogWorkflowCentricApplication, Log, All);

/////////////////////////////////////////////////////
// FWorkflowCentricApplication

TArray<FWorkflowApplicationModeExtender> FWorkflowCentricApplication::ModeExtenderList;

void FWorkflowCentricApplication::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	if (CurrentAppModePtr.IsValid())
	{
		CurrentAppModePtr->RegisterTabFactoriesWithManager(InTabManager);
	}
}

void FWorkflowCentricApplication::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	InTabManager->UnregisterAllTabSpawners();
}

FName FWorkflowCentricApplication::GetToolMenuToolbarName(FName& OutParentName) const
{
	return GetToolMenuToolbarNameForMode(GetCurrentMode(), OutParentName);
}

FName FWorkflowCentricApplication::GetToolMenuToolbarNameForMode(const FName InModeName, FName& OutParentName) const
{
	const FName BaseMenuName = FAssetEditorToolkit::GetToolMenuToolbarName(OutParentName);
	if (InModeName != NAME_None)
	{
		OutParentName = BaseMenuName;
		return *(BaseMenuName.ToString() + TEXT(".") + InModeName.ToString());
	}

	return BaseMenuName;
}

UToolMenu* FWorkflowCentricApplication::RegisterModeToolbarIfUnregistered(const FName InModeName)
{
	FName ParentToolbarName;
	const FName ModeSpecificToolbarName = GetToolMenuToolbarNameForMode(InModeName, ParentToolbarName);
	if (!UToolMenus::Get()->IsMenuRegistered(ModeSpecificToolbarName))
	{
		return UToolMenus::Get()->RegisterMenu(ModeSpecificToolbarName, ParentToolbarName, EMultiBoxType::ToolBar);
	}

	return nullptr;
}

FName FWorkflowCentricApplication::GetCurrentMode() const
{
	return CurrentAppModePtr.IsValid() ? CurrentAppModePtr->GetModeName() : NAME_None;
}

void FWorkflowCentricApplication::SetCurrentMode(FName NewMode)
{
	const bool bModeAlreadyActive = CurrentAppModePtr.IsValid() && (NewMode == CurrentAppModePtr->GetModeName());

	if (!bModeAlreadyActive)
	{
		check(TabManager.IsValid());

		TSharedPtr<FApplicationMode> NewModePtr = ApplicationModeList.FindRef(NewMode);

		LayoutExtenders.Reset();

		if (NewModePtr.IsValid())
		{
			if (NewModePtr->LayoutExtender.IsValid())
			{
				LayoutExtenders.Add(NewModePtr->LayoutExtender);
			}
			
			// Deactivate the old mode
			if (CurrentAppModePtr.IsValid())
			{
				check(TabManager.IsValid());
				CurrentAppModePtr->PreDeactivateMode();
				CurrentAppModePtr->DeactivateMode();
				RemoveToolbarExtender(CurrentAppModePtr->GetToolbarExtender());
				RemoveAllToolbarWidgets();
			}

			// Unregister tab spawners
			TabManager->UnregisterAllTabSpawners();

			//@TODO: Should do some validation here
			CurrentAppModePtr = NewModePtr;

			// Establish the workspace menu category for the new mode
			TabManager->ClearLocalWorkspaceMenuCategories();
			TabManager->AddLocalWorkspaceMenuItem( CurrentAppModePtr->GetWorkspaceMenuCategory() );

			// Activate the new layout
			const TSharedRef<FTabManager::FLayout> NewLayout = CurrentAppModePtr->ActivateMode();
			RestoreFromLayout(NewLayout);

			// Give the new mode a chance to do init
			CurrentAppModePtr->PostActivateMode();

			AddToolbarExtender(NewModePtr->GetToolbarExtender());
			RegenerateMenusAndToolbars();
		}
	}
}

void FWorkflowCentricApplication::PushTabFactories(FWorkflowAllowedTabSet& FactorySetToPush) const
{
	check(TabManager.IsValid());
	for (auto FactoryIt = FactorySetToPush.CreateConstIterator(); FactoryIt; ++FactoryIt)
	{
		FactoryIt.Value()->RegisterTabSpawner(TabManager.ToSharedRef(), CurrentAppModePtr.Get());
	}
}

bool FWorkflowCentricApplication::OnRequestClose(EAssetEditorCloseReason InCloseReason)
{
	return FSlateApplication::Get().IsNormalExecution();
}

void FWorkflowCentricApplication::OnClose()
{
	if (CurrentAppModePtr.IsValid())
	{
		check(TabManager.IsValid());

		// Deactivate the old mode
		CurrentAppModePtr->PreDeactivateMode();
		CurrentAppModePtr->DeactivateMode();
		RemoveToolbarExtender(CurrentAppModePtr->GetToolbarExtender());
		RemoveAllToolbarWidgets();

		// Unregister tab spawners
		TabManager->UnregisterAllTabSpawners();
	}
}

void FWorkflowCentricApplication::AddApplicationMode(FName ModeName, TSharedRef<FApplicationMode> Mode)
{
	// The given mode must not belong to another host.
	check(!Mode->GetHost() || Mode->GetHost().Get() == this);

	// Ensure the mode has a unique reference to this host.
	Mode->ShareHost(nullptr);
	Mode->SetHost(StaticCastWeakPtr<FWorkflowCentricApplication>(AsWeak()));

	for (int32 Index = 0; Index < ModeExtenderList.Num(); ++Index)
	{
		TSharedRef<FApplicationMode> ExtendedMode = ModeExtenderList[Index].Execute(ModeName, Mode);

		// Share the mode's host with the extended mode.
		ExtendedMode->ShareHost(&Mode.Get());

		Swap(ExtendedMode, Mode);
	}
	ApplicationModeList.Add(ModeName, Mode);
}

bool FWorkflowCentricApplication::RemoveApplicationMode(FName ModeName)
{
	if (GetCurrentMode() != ModeName)
	{
		TSharedPtr<FApplicationMode> Mode;
		if (ApplicationModeList.RemoveAndCopyValue(ModeName, Mode))
		{
			if (Mode)
			{
				check(this == Mode->GetHost().Get());
				Mode->SetHost(nullptr);
			}
			return true;
		}
		return false;
	}
	else
	{
		UE_LOGF(LogWorkflowCentricApplication, Warning, "Cannot remove application mode \"%ls\", because it's the current mode.", *ModeName.ToString());
		return false;
	}
}

void FWorkflowCentricApplication::RemoveAllApplicationModes()
{
	for (auto& Entry : ApplicationModeList)
	{
		if (Entry.Value)
		{
			check(this == Entry.Value->GetHost().Get());
			Entry.Value->SetHost(nullptr);
		}
	}
	ApplicationModeList.Reset();
}

const TMap<FName, TSharedPtr<FApplicationMode>>& FWorkflowCentricApplication::GetApplicationModeList() const
{
	return ApplicationModeList;
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
