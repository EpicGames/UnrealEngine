// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorkflowOrientedApp/SContentReference.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Styling/AppStyle.h"
#include "Editor.h"
#include "ContentBrowserModule.h"

#define LOCTEXT_NAMESPACE "SContentReference"

//////////////////////////////////////////////////////////////////////////
// SContentReference

void SContentReference::Construct(const FArguments& InArgs)
{
	// Save off the attributes
	AssetReference = InArgs._AssetReference;
	ShowFindInBrowserButton = InArgs._ShowFindInBrowserButton;
	ShowToolsButton = InArgs._ShowToolsButton;
	AllowSelectingNewAsset = InArgs._AllowSelectingNewAsset;
	AllowClearingReference = InArgs._AllowClearingReference;
	AllowedClass = InArgs._AllowedClass;
	AssetPickerSizeOverride = InArgs._AssetPickerSizeOverride;
	InitialAssetViewType = InArgs._InitialAssetViewType;

	// Save off delegates
	OnShouldFilterAsset = InArgs._OnShouldFilterAsset;
	OnSetReference = InArgs._OnSetReference;

	// Cache resources
	BorderImageNormal = FAppStyle::GetBrush(InArgs._Style, ".Background.Normal");
	BorderImageHovered = FAppStyle::GetBrush(InArgs._Style, ".Background.Hovered");

	static const FName InvertedForegroundName("InvertedForeground");

	PickerComboButton = SNew(SComboButton)
		.ButtonStyle(FAppStyle::Get(), "NoBorder")
		.ContentPadding(1.f)
		.Visibility(this, &SContentReference::GetPickButtonVisibility)
		.OnGetMenuContent(this, &SContentReference::MakeAssetPickerMenu)
		.HasDownArrow(false)
		.ToolTipText(LOCTEXT("PickAsset", "Pick an asset from a popup menu"))
		.ButtonContent()
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush(InArgs._Style, ".PickAsset"))
		];

	// Create the widgets
	ChildSlot
	[
		SNew(SHorizontalBox)

		// Text box containing the name of the asset
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(1,0)
		[
			SAssignNew(AssetReferenceNameBorderWidget, SBorder)
			.BorderImage(this, &SContentReference::GetBorderImage)
			.Padding(FAppStyle::GetMargin(InArgs._Style, ".BorderPadding"))
			.BorderBackgroundColor( FLinearColor::White )
			.ForegroundColor(FAppStyle::GetSlateColor(InvertedForegroundName))
			.ToolTipText(this, &SContentReference::GetAssetFullName)
			.OnMouseDoubleClick(this, &SContentReference::OnDoubleClickedOnAssetName)
			[
				SNew(SBox)
				.WidthOverride(InArgs._WidthOverride)
				[
					SNew(STextBlock)
					.Text(this, &SContentReference::GetAssetShortName)
				]
			]
		]

		// Pick an asset
		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Padding(1,0)
		[
			PickerComboButton.ToSharedRef()
		]
			
		// Find in content browser button
		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Padding(1,0)
		[
			SNew(SButton)
			.ButtonStyle( FAppStyle::Get(), "NoBorder" )
			.OnClicked(this, &SContentReference::OnClickFindButton)
			.ContentPadding(0.0f)
			.Visibility(this, &SContentReference::GetFindButtonVisibility)
			.ToolTipText(LOCTEXT("Find", "Find in content browser"))
			[
				SNew(SImage)
				.Image( FAppStyle::GetBrush(InArgs._Style, ".FindInContentBrowser") )
			]
		]
			
		// Clear button
		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Padding(1,0)
		[
			SNew(SButton)
			.ButtonStyle( FAppStyle::Get(), "NoBorder" )
			.OnClicked(this, &SContentReference::OnClickClearButton)
			.ContentPadding(1.f)
			.Visibility( this, &SContentReference::GetClearButtonVisibility)
			.ToolTipText(LOCTEXT("Clear", "Clear"))
			[
				SNew(SImage)
				.Image( FAppStyle::GetBrush(InArgs._Style, ".Clear") )
			]
		]
		
		// Tools button
		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Padding(1,0)
		[
			SNew(SButton)
			.ButtonStyle( FAppStyle::Get(), "NoBorder" )
			.OnClicked(InArgs._OnClickedTools)
			.ContentPadding(1.f)
			.Visibility(this, &SContentReference::GetToolsButtonVisibility)
			.ToolTipText(LOCTEXT("Tools", "Tools"))
			[
				SNew(SImage)
				.Image( FAppStyle::GetBrush(InArgs._Style, ".Tools") )
			]
		]
	];
}

void SContentReference::OpenAssetPickerMenu()
{
	PickerComboButton->SetIsOpen(true);
}

EVisibility SContentReference::GetUseButtonVisibility() const
{
	return AllowSelectingNewAsset.Get() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SContentReference::GetPickButtonVisibility() const
{
	return AllowSelectingNewAsset.Get() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SContentReference::GetFindButtonVisibility() const
{
	return ShowFindInBrowserButton.Get() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SContentReference::GetClearButtonVisibility() const
{
	return AllowClearingReference.Get() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SContentReference::GetToolsButtonVisibility() const
{
	return ShowToolsButton.Get() ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply SContentReference::OnClickUseButton()
{
	TArray<FAssetData> Objects;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	ContentBrowserModule.Get().GetSelectedAssets(Objects);

	for(auto It(Objects.CreateIterator());It;++It)
	{
		OnSetReference.ExecuteIfBound(It->GetAsset());
	}
	return FReply::Handled();
}

FReply SContentReference::OnClickFindButton()
{
	FindObjectInContentBrowser(AssetReference.Get());
	return FReply::Handled();
}

FReply SContentReference::OnClickClearButton()
{
	OnSetReference.ExecuteIfBound(NULL);
	return FReply::Handled();
}

TSharedRef<SWidget> SContentReference::MakeAssetPickerMenu()
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	FAssetPickerConfig AssetPickerConfig;

	UClass* FilterClass = AllowedClass.Get();
	if (FilterClass != NULL)
	{
		AssetPickerConfig.Filter.ClassPaths.Add(FilterClass->GetClassPathName());
		AssetPickerConfig.Filter.bRecursiveClasses = true;
	}

	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SContentReference::OnAssetSelectedFromPicker);
	AssetPickerConfig.OnShouldFilterAsset = OnShouldFilterAsset;
	AssetPickerConfig.bAllowNullSelection = true;
	AssetPickerConfig.ThumbnailLabel = EThumbnailLabel::ClassName;
	AssetPickerConfig.InitialAssetViewType = InitialAssetViewType;

	return SNew(SBox)
		.WidthOverride(static_cast<float>(AssetPickerSizeOverride.Get().X))
		.HeightOverride(static_cast<float>(AssetPickerSizeOverride.Get().Y))
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		];
}

void SContentReference::OnAssetSelectedFromPicker(const FAssetData& AssetData)
{
	PickerComboButton->SetIsOpen(false);
	OnSetReference.ExecuteIfBound(AssetData.GetAsset());
}

FText SContentReference::GetAssetShortName() const
{
	if (UObject* Asset = AssetReference.Get())
	{
		return FText::FromString(Asset->GetName());
	}
	else
	{
		return LOCTEXT("NullReference", "(None)");
	}
}

FText SContentReference::GetAssetFullName() const
{
	if (UObject* Asset = AssetReference.Get())
	{
		return FText::FromString(Asset->GetFullName());
	}
	else
	{
		return LOCTEXT("NullReferenceTooltip", "(None)");
	}
}

FReply SContentReference::OnDoubleClickedOnAssetName(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	OpenAssetToEdit();
	return FReply::Handled();
}

void SContentReference::OpenAssetToEdit()
{
	if (UObject* Asset = AssetReference.Get())
	{
		GEditor->EditObject(Asset);
	}
}

const FSlateBrush* SContentReference::GetBorderImage() const
{
	return AssetReferenceNameBorderWidget->IsHovered() ? BorderImageHovered : BorderImageNormal;
}

void SContentReference::FindObjectInContentBrowser(UObject* Object)
{
	if ((Object != NULL) && (Object->IsAsset()))
	{
		TArray<UObject*> ObjectsToSyncTo;
		ObjectsToSyncTo.Add(Object);

		GEditor->SyncBrowserToObjects(ObjectsToSyncTo);
	}
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
