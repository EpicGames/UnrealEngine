// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorkflowOrientedApp/SModeWidget.h"

#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

//////////////////////////////////////////////////////////////////////////
// SModeWidget

void SModeWidget::Construct(const FArguments& InArgs, const FText& InText, const FName InMode)
{
	// Copy arguments
	ModeText = InText;
	ThisMode = InMode;
	OnGetActiveMode = InArgs._OnGetActiveMode;
	CanBeSelected = InArgs._CanBeSelected;
	OnSetActiveMode = InArgs._OnSetActiveMode;

	// Load resources
	InactiveModeBorderImage = FAppStyle::GetBrush("ModeSelector.ToggleButton.Normal");
	ActiveModeBorderImage = FAppStyle::GetBrush("ModeSelector.ToggleButton.Pressed");
	HoverBorderImage = FAppStyle::GetBrush("ModeSelector.ToggleButton.Hovered");
	
	TSharedRef<SHorizontalBox> InnerRow = SNew(SHorizontalBox);

	FMargin IconPadding(4.0f, 0.0f, 4.0f, 0.0f);
	FMargin BodyPadding(0.0f, 0.0f, 0.0f, 0.0f);
	
	if (InArgs._IconImage.IsSet())
	{
		InnerRow->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(IconPadding)
			[
				SNew(SImage)
				.Image(InArgs._IconImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
			];
	}

	// Label + content
	InnerRow->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(BodyPadding)
		[
			SNew(SVerticalBox)

			// Mode 'tab'
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				//Mode Name
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(ModeText)
				]

				//Dirty flag
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(3)
				[
					SNew(SImage)
					.Image(InArgs._DirtyMarkerBrush)
				]
			]

			// Body of 'ribbon'
			+SVerticalBox::Slot()
			.AutoHeight()
			[

				InArgs._ShortContents.Widget
			]
		];


	// Create the widgets
	ChildSlot
	.VAlign(VAlign_Fill)
	[
		SNew(SCheckBox)
		.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
		.IsChecked(this, &SModeWidget::GetModeCheckState)
		.OnCheckStateChanged(this, &SModeWidget::OnModeTabClicked)
		[
			InnerRow
		]
	];

	SetEnabled(CanBeSelected);
}

ECheckBoxState SModeWidget::GetModeCheckState() const
{
	if (IsActiveMode())
	{
		return ECheckBoxState::Checked;
	}
	else
	{
		return ECheckBoxState::Unchecked;
	}
}

bool SModeWidget::IsActiveMode() const
{
	return OnGetActiveMode.Get() == ThisMode;
}

void SModeWidget::OnModeTabClicked(ECheckBoxState CheckState)
{
	// Try to change the mode
	if (!IsActiveMode() && (CanBeSelected.Get() == true))
	{
		OnSetActiveMode.ExecuteIfBound(ThisMode);
	}
}
