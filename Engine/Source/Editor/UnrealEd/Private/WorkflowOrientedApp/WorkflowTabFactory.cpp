// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/AppStyle.h"
#include "WorkflowOrientedApp/ApplicationMode.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"
#include "IDocumentation.h"
#include "Widgets/Docking/SDockTab.h"
#include "Framework/Docking/TabManager.h"
#include "Misc/NamePermissionList.h"

/////////////////////////////////////////////////////
// FWorkflowTabFactory

FWorkflowTabFactory::FWorkflowTabFactory(FName InIdentifier, TSharedPtr<class FAssetEditorToolkit> InHostingApp)
	: TabIdentifier(InIdentifier)
	, TabRole(ETabRole::PanelTab)
	, TabIcon(FSlateIcon())
	, InsideTabPadding(0.0f)
	, bIsSingleton(false)
	, bShouldAutosize(false)
	, HostingApp(InHostingApp)
{
}

void FWorkflowTabFactory::EnableTabPadding()
{
	InsideTabPadding = 4.0f;
}

TSharedRef<SDockTab> FWorkflowTabFactory::SpawnTab(const FWorkflowTabSpawnInfo& Info) const
{
	// Get the tab body content
	TSharedRef<SWidget> TabBody = CreateTabBody(Info);

	// Pad the content if requested
	if (InsideTabPadding > 0.0f)
	{
		// propagate tag from original content, or use TabId
		TSharedPtr<FTagMetaData> MetaData = TabBody->GetMetaData<FTagMetaData>();

		TabBody = SNew(SBorder)
			.Padding(InsideTabPadding)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.AddMetaData<FTagMetaData>(MetaData.IsValid() ? MetaData->Tag : this->TabIdentifier)
			[
				TabBody
			];
	}

	// Spawn the tab
	TSharedRef<SDockTab> NewTab = SNew(SDockTab)
		.TabRole(this->TabRole)
		.Label(ConstructTabName(Info))
		.LabelSuffix(ConstructTabLabelSuffix(Info))
		.ShouldAutosize(bShouldAutosize)
		[
			TabBody
		];
	const TAttribute<const FSlateBrush*> TabIconAttribute = TAttribute<const FSlateBrush*>::CreateLambda([this, Info]()
		{
			return GetTabIcon(Info);
		});
	NewTab->SetTabIcon(TabIconAttribute);
	NewTab->SetTabToolTipWidget(CreateTabToolTipWidget(Info));

	return NewTab;
}

TSharedRef<SDockTab> FWorkflowTabFactory::SpawnBlankTab() const
{
	// Spawn the tab
	TSharedRef<SDockTab> NewTab = SNew(SDockTab)
		.TabRole(this->TabRole)
		.ShouldAutosize(bShouldAutosize);

	return NewTab;
}


void FWorkflowTabFactory::UpdateTab(TSharedPtr<SDockTab> InDockTab, const FWorkflowTabSpawnInfo& Info, TSharedPtr< SWidget > InContent) const
{
	// Get the tab body content
	TSharedRef<SWidget> TabBody = InContent.ToSharedRef();

	// Pad the content if requested
	if (InsideTabPadding > 0.0f)
	{
		TabBody = SNew(SBorder)
			.Padding(InsideTabPadding)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				TabBody
			];
	}

	InDockTab->SetContent(TabBody);
	InDockTab->SetLabel(ConstructTabName(Info));
	InDockTab->SetTabLabelSuffix(ConstructTabLabelSuffix(Info));
	InDockTab->SetTabIcon(GetTabIcon(Info));
	InDockTab->SetTabToolTipWidget(CreateTabToolTipWidget(Info));
}

TSharedPtr<SToolTip> FWorkflowTabFactory::CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const
{
	const FString DocLink = TEXT("Shared/Editors/Common/EditorTabs");
	const FString DocExcerptName = TabIdentifier.ToString();
	return IDocumentation::Get()->CreateToolTip(GetTabToolTipText(Info), NULL, DocLink, DocExcerptName);
}

TSharedRef<SDockTab> FWorkflowTabFactory::OnSpawnTab(const FSpawnTabArgs& SpawnArgs, TWeakPtr<FTabManager> WeakTabManager) const
{
	FWorkflowTabSpawnInfo SpawnInfo;
	SpawnInfo.TabManager = WeakTabManager.Pin();

	return SpawnInfo.TabManager.IsValid() ? SpawnTab(SpawnInfo) : SNew(SDockTab);
}

bool FWorkflowTabFactory::CanSpawnTab(const FSpawnTabArgs& SpawnArgs, TWeakPtr<FTabManager> WeakTabManager) const
{
	TSharedPtr<FTabManager> TabManager = WeakTabManager.Pin();
	if (TabManager.IsValid())
	{
		return TabManager->GetTabPermissionList()->PassesFilter(SpawnArgs.GetTabId().TabType);
	}

	return true;
}

FTabSpawnerEntry& FWorkflowTabFactory::RegisterTabSpawner(TSharedRef<FTabManager> InTabManager, const FApplicationMode* CurrentApplicationMode) const
{
	FWorkflowTabSpawnInfo SpawnInfo;
	SpawnInfo.TabManager = InTabManager;

	TWeakPtr<FTabManager> WeakTabManager(InTabManager);
	FTabSpawnerEntry& SpawnerEntry = InTabManager->RegisterTabSpawner(GetIdentifier(), FOnSpawnTab::CreateSP(this, &FWorkflowTabFactory::OnSpawnTab, WeakTabManager), FCanSpawnTab::CreateSP(this, &FWorkflowTabFactory::CanSpawnTab, WeakTabManager))
		.SetDisplayName(ConstructTabName(SpawnInfo).Get())
		.SetTooltipText(GetTabToolTipText(SpawnInfo));

	if (Group)
	{
		SpawnerEntry.SetGroup(Group.ToSharedRef());
	}
	else if (CurrentApplicationMode)
	{
		SpawnerEntry.SetGroup(CurrentApplicationMode->GetWorkspaceMenuCategory());
	}
	
	// Add the tab icon to the menu entry if one was provided
	const FSlateIcon& TabSpawnerIcon = GetTabSpawnerIcon(SpawnInfo);
	if (TabSpawnerIcon.IsSet())
	{
		SpawnerEntry.SetIcon(TabSpawnerIcon);
	}

	return SpawnerEntry;
}

TAttribute<FText> FWorkflowTabFactory::ConstructTabName(const FWorkflowTabSpawnInfo& Info) const
{
	return TabLabel;
}

TAttribute<FText> FWorkflowTabFactory::ConstructTabLabelSuffix(const FWorkflowTabSpawnInfo& Info) const
{
	return FText::GetEmpty();
}

TSharedRef<SWidget> FWorkflowTabFactory::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(STextBlock).Text(NSLOCTEXT("WorkflowTabFactory", "EmptyTabMessage", "EmptyBody"));
}

const FSlateBrush* FWorkflowTabFactory::GetTabIcon(const FWorkflowTabSpawnInfo& Info) const
{
	return TabIcon.GetIcon();
}

const FSlateIcon& FWorkflowTabFactory::GetTabSpawnerIcon(const FWorkflowTabSpawnInfo& Info) const
{
	return TabIcon;
}

void FWorkflowTabFactory::CreateViewMenuEntry(FMenuBuilder& MenuBuilder, const FUIAction& Action) const
{
	MenuBuilder.AddMenuEntry(ViewMenuDescription, ViewMenuTooltip, FSlateIcon(), Action);
}

/////////////////////////////////////////////////////
// FDocumentTabFactory

FDocumentTabFactory::FDocumentTabFactory(FName InIdentifier, TSharedPtr<class FAssetEditorToolkit> InHostingApp) : FWorkflowTabFactory(InIdentifier, InHostingApp)
{
	TabRole = ETabRole::DocumentTab;
}

TSharedRef<FGenericTabHistory> FDocumentTabFactory::CreateTabHistoryNode(TSharedPtr<FTabPayload> Payload)
{
	return MakeShareable(new FGenericTabHistory(SharedThis(this), Payload));
}
