// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorkflowOrientedApp/WorkflowTabManager.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "WorkflowOrientedApp/WorkflowUObjectDocuments.h"
#include "Widgets/Docking/SDockTab.h"
#include "Misc/NamePermissionList.h"

/////////////////////////////////////////////////////
// FTabInfo

namespace WorkflowTabManagerHelpers
{
	/** Max number of history items that can be stored.  Once the max is reached, the oldest history item is removed */
	const int32 MaxHistoryEntries = 300;
}

FTabInfo::FTabInfo(const TSharedRef<SDockTab>& InTab, const TSharedPtr<FDocumentTabFactory>& InSpawner, const TSharedPtr<class FDocumentTracker>& InTracker)
	: Tab(InTab)
	, WeakTracker(InTracker)
{
}

bool FTabInfo::PayloadMatches(const TSharedPtr<FTabPayload> TestPayload) const
{
	return PayloadMatches(GetPayload(), TestPayload);
}

bool FTabInfo::PayloadMatches(TSharedPtr<FTabPayload> A, TSharedPtr<FTabPayload> B)
{
	if (A.IsValid() && B.IsValid())
	{
		return A->IsEqual(B.ToSharedRef());
	}
	else if (!B.IsValid() && !A.IsValid())
	{
		return true;
	}
	else
	{
		return false;
	}
}

void FTabInfo::AddTabHistory(TSharedPtr< struct FGenericTabHistory > InHistoryNode, bool bInSaveHistory/* = true*/)
{
	TSharedPtr<FDocumentTracker> Tracker = WeakTracker.Pin();

	if (Tracker.IsValid())
	{
		if (Tracker->CurrentHistoryIndex == Tracker->History.Num() - 1)
		{
			// History added to the end
			if (Tracker->History.Num() == WorkflowTabManagerHelpers::MaxHistoryEntries)
			{
				// If max history entries has been reached
				// remove the oldest history
				Tracker->History.RemoveAt(0);
			}
		}
		else
		{
			// Clear out any history that is in front of the current location in the history list
			Tracker->History.RemoveAt(Tracker->CurrentHistoryIndex + 1, Tracker->History.Num() - (Tracker->CurrentHistoryIndex + 1), EAllowShrinking::Yes);
		}

		Tracker->History.Add(InHistoryNode);
		Tracker->CurrentHistoryIndex = Tracker->History.Num() - 1;

		SetCurrentHistory(InHistoryNode, bInSaveHistory, false);
	}
}

TSharedPtr<struct FGenericTabHistory> FTabInfo::GetCurrentHistory() const
{
	return CurrentHistory;
}

void FTabInfo::SetCurrentHistory(TSharedPtr<FGenericTabHistory> NewHistory, bool bInSaveHistory, bool bShouldRestore)
{
	bool bPayloadMatches = false;

	if (CurrentHistory.IsValid() && NewHistory.IsValid())
	{
		if (bInSaveHistory)
		{
			CurrentHistory->SaveHistory();
		}

		bPayloadMatches = PayloadMatches(CurrentHistory->GetPayload(), NewHistory->GetPayload());
	}
	CurrentHistory = NewHistory;

	if (CurrentHistory.IsValid())
	{
		CurrentHistory->BindToTab(AsShared());

		// This creates the tab widget but does not foreground it
		CurrentHistory->EvokeHistory(AsShared(), bPayloadMatches);
		if (bShouldRestore)
		{
			CurrentHistory->RestoreHistory();
		}
		TSharedPtr<FDocumentTabFactory> Factory = CurrentHistory->GetFactory().Pin();
		if (Factory.IsValid())
		{
			// Notify listeners that tab contents have changed
			Factory->OnTabActivated(Tab.Pin());
		}
		
	}
}

FReply FTabInfo::OnGoForwardInHistory()
{
	TSharedPtr<FDocumentTracker> Tracker = WeakTracker.Pin();

	if (Tracker.IsValid())
	{
		int32 NextHistoryIndex = Tracker->CurrentHistoryIndex;
		while (NextHistoryIndex < Tracker->History.Num() - 1)
		{
			++NextHistoryIndex;
				
			if (Tracker->NavigateToTabHistory(NextHistoryIndex))
			{
				break;
			}
		}
	}
	return FReply::Handled();
}

FReply FTabInfo::OnGoBackInHistory()
{
	TSharedPtr<FDocumentTracker> Tracker = WeakTracker.Pin();

	if (Tracker.IsValid())
	{
		int32 PreviousHistoryIndex = Tracker->CurrentHistoryIndex;
		while (PreviousHistoryIndex > 0)
		{
			--PreviousHistoryIndex;
				
			if (Tracker->NavigateToTabHistory(PreviousHistoryIndex))
			{
				break;
			}
		}
	}
	return FReply::Handled();
}

void FTabInfo::JumpToNearestValidHistoryData()
{
	// Each tab only knows current history, close if it's invalid
	if (!CurrentHistory.IsValid() || !CurrentHistory->IsHistoryValid())
	{
		if (Tab.IsValid())
		{
			Tab.Pin()->RequestCloseTab();
		}
	}
}

TWeakPtr<FDocumentTabFactory> FTabInfo::GetFactory() const
{
	if (CurrentHistory.IsValid())
	{
		return CurrentHistory->GetFactory();
	}
	return nullptr;
}

TSharedPtr<FTabPayload> FTabInfo::GetPayload() const
{
	if (CurrentHistory.IsValid())
	{
		return CurrentHistory->GetPayload();
	}
	return nullptr;
}

void FTabInfo::GoToHistoryIndex(int32 InHistoryIdx)
{
	TSharedPtr<FDocumentTracker> Tracker = WeakTracker.Pin();
	if (Tracker.IsValid())
	{
		Tracker->NavigateToTabHistory(InHistoryIdx);
	}
}

TSharedRef<SWidget> FTabInfo::CreateHistoryMenu(bool bInBackHistory) const
{
	TSharedPtr<FDocumentTracker> Tracker = WeakTracker.Pin();
	if (!Tracker.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	FMenuBuilder MenuBuilder(true, nullptr);
	if(bInBackHistory)
	{
		int32 HistoryIdx = Tracker->CurrentHistoryIndex - 1;
		while( HistoryIdx >= 0 )
		{
			TSharedPtr<FGenericTabHistory> HistoryItem = Tracker->History[HistoryIdx];
			if(HistoryItem->IsHistoryValid())
			{
				MenuBuilder.AddMenuEntry(HistoryItem->GetHistoryTitle().Get(), FText(), FSlateIcon(), 
					FUIAction(
					FExecuteAction::CreateRaw(const_cast<FTabInfo*>(this), &FTabInfo::GoToHistoryIndex, HistoryIdx)
					), 
					NAME_None, EUserInterfaceActionType::Button);
			}

			--HistoryIdx;
		}
	}
	else
	{
		int32 HistoryIdx = Tracker->CurrentHistoryIndex + 1;
		while( HistoryIdx < Tracker->History.Num() )
		{
			TSharedPtr<FGenericTabHistory> HistoryItem = Tracker->History[HistoryIdx];
			if(HistoryItem->IsHistoryValid())
			{
				MenuBuilder.AddMenuEntry(HistoryItem->GetHistoryTitle().Get(), FText(), FSlateIcon(), 
					FUIAction(
					FExecuteAction::CreateRaw(const_cast<FTabInfo*>(this), &FTabInfo::GoToHistoryIndex, HistoryIdx)
					), 
					NAME_None, EUserInterfaceActionType::Button);
			}

			++HistoryIdx;
		}
	}

	return MenuBuilder.MakeWidget();
}

bool FTabInfo::CanStepBackwardInHistory() const
{
	TSharedPtr<FDocumentTracker> Tracker = WeakTracker.Pin();
	if (!Tracker.IsValid())
	{
		return false;
	}

	int32 HistoryIdx = Tracker->CurrentHistoryIndex - 1;
	while( HistoryIdx >= 0 )
	{
		if(Tracker->History[HistoryIdx]->IsHistoryValid())
		{
			return true;
		}

		--HistoryIdx;
	}
	return false;
}

bool FTabInfo::CanStepForwardInHistory() const
{
	TSharedPtr<FDocumentTracker> Tracker = WeakTracker.Pin();
	if (!Tracker.IsValid())
	{
		return false;
	}

	int32 HistoryIdx = Tracker->CurrentHistoryIndex + 1;
	while( HistoryIdx < Tracker->History.Num() )
	{
		if(Tracker->History[HistoryIdx]->IsHistoryValid())
		{
			return true;
		}

		++HistoryIdx;
	}
	return false;
}

FReply FTabInfo::OnMouseDownHistory( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, TWeakPtr< SMenuAnchor > InMenuAnchor )
{
	if(MouseEvent.GetEffectingButton() == EKeys::RightMouseButton && InMenuAnchor.IsValid())
	{
		InMenuAnchor.Pin()->SetIsOpen(true);
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

TSharedRef< SWidget > FTabInfo::CreateHistoryNavigationWidget()
{
	if(!HistoryNavigationWidget.IsValid())
	{
		const TSharedRef<SMenuAnchor> BackMenuAnchorPtr =
			SNew(SMenuAnchor)
			.Placement( MenuPlacement_BelowAnchor )
			.OnGetMenuContent( this, &FTabInfo::CreateHistoryMenu, true )
			[
				SNew(SButton)
				.OnClicked( this, &FTabInfo::OnGoBackInHistory )
				.ButtonStyle( FAppStyle::Get(), "GraphBreadcrumbButton" )
				.IsEnabled(this, &FTabInfo::CanStepBackwardInHistory)
				.ToolTipText(NSLOCTEXT("WorkflowNavigationBrowser", "Backward_Tooltip", "Step backward in the tab history. Right click to see full history."))
				[
					SNew(SImage)
					.Image( FAppStyle::GetBrush("GraphBreadcrumb.BrowseBack") )
				]
			];

		const TSharedRef<SMenuAnchor> FwdMenuAnchorPtr =
			SNew(SMenuAnchor)
			.Placement( MenuPlacement_BelowAnchor )
			.OnGetMenuContent( this, &FTabInfo::CreateHistoryMenu, false )
			[
				SNew(SButton)
				.OnClicked( this, &FTabInfo::OnGoForwardInHistory )
				.ButtonStyle( FAppStyle::Get(), "GraphBreadcrumbButton" )
				.IsEnabled(this, &FTabInfo::CanStepForwardInHistory)
				.ToolTipText(NSLOCTEXT("WorkflowNavigationBrowser", "Forward_Tooltip", "Step forward in the tab history. Right click to see full history."))
				[
					SNew(SImage)
					.Image( FAppStyle::GetBrush("GraphBreadcrumb.BrowseForward") )
				]
			];

		HistoryNavigationWidget = 
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBorder)
				.OnMouseButtonDown(this, &FTabInfo::OnMouseDownHistory, BackMenuAnchorPtr.ToWeakPtr())
				.BorderImage( FAppStyle::GetBrush("NoBorder") )
				[
					BackMenuAnchorPtr
				]
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBorder)
				.OnMouseButtonDown(this, &FTabInfo::OnMouseDownHistory, FwdMenuAnchorPtr.ToWeakPtr())
				.BorderImage( FAppStyle::GetBrush("NoBorder") )
				[
					FwdMenuAnchorPtr
				]
			];
	}

	return HistoryNavigationWidget.ToSharedRef();
}

/////////////////////////////////////////////////////
// FWorkflowAllowedTabSet

// Searches this set for a factory with the specified ID, or returns NULL.
TSharedPtr<class FWorkflowTabFactory> FWorkflowAllowedTabSet::GetFactory(FName FactoryID) const
{
	return Factories.FindRef(FactoryID);
}

// Registers a factory with this set
void FWorkflowAllowedTabSet::RegisterFactory(TSharedPtr<class FWorkflowTabFactory> Factory)
{
	FName NewIdentifier = Factory->GetIdentifier();
	check(!Factories.Contains(NewIdentifier));
	Factories.Add(NewIdentifier, Factory);
}

void FWorkflowAllowedTabSet::UnregisterFactory(FName FactoryID)
{
	int32 Removed = Factories.Remove(FactoryID);
	check(Removed != 0);
}

// Merges in a set of factories into this set
void FWorkflowAllowedTabSet::MergeInSet(const FWorkflowAllowedTabSet& OtherSet)
{
	Factories.Append(OtherSet.Factories);
}

// Clears the set
void FWorkflowAllowedTabSet::Clear()
{
	Factories.Empty();
}

TMap< FName, TSharedPtr<class FWorkflowTabFactory> >::TIterator FWorkflowAllowedTabSet::CreateIterator()
{
	return Factories.CreateIterator();
}

TMap< FName, TSharedPtr<class FWorkflowTabFactory> >::TConstIterator FWorkflowAllowedTabSet::CreateConstIterator() const
{
	return Factories.CreateConstIterator();
}

/////////////////////////////////////////////////////
// FDocumentTracker

void FDocumentTracker::ClearDocumentFactories()
{
	PotentialTabFactories.Empty();
}

void FDocumentTracker::RegisterDocumentFactory(TSharedPtr<class FDocumentTabFactory> Factory)
{
	FName NewIdentifier = Factory->GetIdentifier();
	check(!PotentialTabFactories.Contains(NewIdentifier));
	PotentialTabFactories.Add(NewIdentifier, Factory);
}

FDocumentTracker::FDocumentTracker(FName InDefaultDocumentId)
	: DefaultDocumentId(InDefaultDocumentId)
{
	// Make sure we know when tabs become active
	OnActiveTabChangedDelegateHandle = FGlobalTabmanager::Get()->OnActiveTabChanged_Subscribe( FOnActiveTabChanged::FDelegate::CreateRaw( this, &FDocumentTracker::OnActiveTabChanged ) );
	TabForegroundedDelegateHandle = FGlobalTabmanager::Get()->OnTabForegrounded_Subscribe(FOnActiveTabChanged::FDelegate::CreateRaw(this, &FDocumentTracker::OnTabForegrounded));

	CurrentHistoryIndex = INDEX_NONE;
}

FDocumentTracker::~FDocumentTracker()
{
	FGlobalTabmanager::Get()->OnActiveTabChanged_Unsubscribe(OnActiveTabChangedDelegateHandle);
	FGlobalTabmanager::Get()->OnTabForegrounded_Unsubscribe(TabForegroundedDelegateHandle);
}

// Called by the global active tab changed callback; dispatches to individually registered callbacks
void FDocumentTracker::OnActiveTabChanged(TSharedPtr<SDockTab> PreviouslyActive, TSharedPtr<SDockTab> NewlyActivated)
{
	FTabList& List = GetSpawnedList();
	for (auto ListIt = List.CreateIterator(); ListIt; ++ListIt)
	{
		// Get the factory (can't fail; the tabs had to come from somewhere; failing means a tab survived a mode transition to a mode where it is not allowed!)
		TSharedPtr<FDocumentTabFactory> Factory = (*ListIt)->GetFactory().Pin();
		if (ensure(Factory.IsValid()))
		{
			TSharedPtr<SDockTab> Tab = (*ListIt)->GetTab().Pin();
			if (Tab == NewlyActivated)
			{
				WeakLastEditedTabInfo = *ListIt;
				Factory->OnTabActivated(Tab);
			}
		}
	}
}

void FDocumentTracker::OnTabForegrounded(TSharedPtr<SDockTab> ForegroundedTab, TSharedPtr<SDockTab> BackgroundedTab)
{
	TSharedPtr<FTabInfo> NewTabInfo;
	TSharedPtr<SDockTab> OwnedForeground, OwnedBackground;
	TSharedPtr<FDocumentTabFactory> ForegroundFactory, BackgroundFactory;

	FTabList& List = GetSpawnedList();
	for ( const TSharedPtr<FTabInfo>& TabInfo : List )
	{
		// Get the factory (can't fail; the tabs had to come from somewhere; failing means a tab survived a mode transition to a mode where it is not allowed!)
		TSharedPtr<FDocumentTabFactory> Factory = TabInfo->GetFactory().Pin();
		if ( ensure(Factory.IsValid()) )
		{
			TSharedPtr<SDockTab> Tab = TabInfo->GetTab().Pin();
			if ( Tab == ForegroundedTab )
			{
				NewTabInfo = TabInfo;
				OwnedForeground = Tab;
				ForegroundFactory = Factory;
			}
			else if ( Tab == BackgroundedTab )
			{
				OwnedBackground = Tab;
				BackgroundFactory = Factory;
			}
		}
	}

	if ( OwnedBackground.IsValid() )
	{
		BackgroundFactory->OnTabBackgrounded(OwnedBackground);
	}

	if ( OwnedForeground.IsValid() )
	{
		ForegroundFactory->OnTabForegrounded(OwnedForeground);
	}

	TSharedPtr<FGenericTabHistory> CurrentTabHistory = GetCurrentTabHistory();
	TSharedPtr<FGenericTabHistory> NewTabHistory;
	if (NewTabInfo.IsValid())
	{
		NewTabHistory = NewTabInfo->GetCurrentHistory();
	}

	if ( ForegroundFactory.IsValid() && NewTabHistory.IsValid() && NewTabHistory->IsHistoryValid() && NewTabHistory != CurrentTabHistory )
	{
		// If a tab was manually foregrounded, need to add tab history
		NewTabInfo->AddTabHistory(ForegroundFactory->CreateTabHistoryNode(NewTabInfo->GetPayload()), true);
	}
}

FDocumentTracker::FTabList& FDocumentTracker::GetSpawnedList()
{
	// Remove any closed tabs
	{
		struct
		{ 
			bool operator()(const TSharedPtr<FTabInfo>& RemovalCandidate) const
			{
				return !RemovalCandidate->GetTab().IsValid();
			}
		} CullClosedTabs;
	
		SpawnedTabs.RemoveAll( CullClosedTabs );
	}

	return SpawnedTabs;
}

TSharedPtr<FTabInfo> FDocumentTracker::GetLastEditedTabInfo()
{
	TSharedPtr<FTabInfo> LastEditedTabInfo = WeakLastEditedTabInfo.Pin();
	if (!LastEditedTabInfo.IsValid() || !LastEditedTabInfo->GetTab().IsValid())
	{
		// Clear if either the info or actual tab are gone
		WeakLastEditedTabInfo = nullptr;
		return nullptr;
	}
	return LastEditedTabInfo;
}

void FDocumentTracker::Initialize(TSharedPtr<FAssetEditorToolkit> InHostingApp )
{
	check(!HostingAppPtr.IsValid());
	HostingAppPtr = InHostingApp;
}

void FDocumentTracker::SetTabManager( const TSharedRef<FTabManager>& InTabManager )
{
	TabManager = InTabManager;
}

TSharedPtr<SDockTab> FDocumentTracker::OpenDocument(TSharedPtr<FTabPayload> InPayload, EOpenDocumentCause InOpenCause)
{
	// Managed document spawn can occur at times like 'Ctrl + Z' to undo, don't force them to spawn a new tab.
	if(FSlateApplication::Get().GetModifierKeys().IsControlDown() && InOpenCause != SpawnManagedDocument)
	{
		InOpenCause = FDocumentTracker::ForceOpenNewDocument;
	}

	if(InOpenCause == CreateHistoryEvent)
	{
		// Deprecated, all opens now save history
		InOpenCause = FDocumentTracker::OpenNewDocument;
	}

	if(InOpenCause == NavigatingCurrentDocument || InOpenCause == QuickNavigateCurrentDocument || InOpenCause == NavigateBackwards || InOpenCause == NavigateForwards)
	{
		return NavigateCurrentTab(InPayload, InOpenCause);
	}

	// Spawning or restoring a tab, so the factory has to be valid
	TSharedPtr<FDocumentTabFactory> Factory = FindSupportingFactory(InPayload.ToSharedRef());
	if(!Factory.IsValid())
	{
		return nullptr;
	}

	if(InOpenCause == OpenNewDocument || InOpenCause == RestorePreviousDocument || InOpenCause == SpawnManagedDocument)
	{
		// If the current tab matches we'll re-use it.
		TSharedPtr<FTabInfo> LastEditedTabInfo = GetLastEditedTabInfo();
		if(LastEditedTabInfo.IsValid() && LastEditedTabInfo->PayloadMatches(InPayload))
		{
			TSharedPtr<SDockTab> Tab = LastEditedTabInfo->GetTab().Pin();
			if (Tab.IsValid())
			{
				LastEditedTabInfo->AddTabHistory(Factory->CreateTabHistoryNode(InPayload), true);
				// Ensure that the tab appears if the tab isn't currently in the foreground.
				Tab->ActivateInParent(ETabActivationCause::SetDirectly);
			}
			return Tab;
		}
		else
		{
			// Check if the payload is currently open in any tab
			FTabList& List = GetSpawnedList();
			for (const TSharedPtr<FTabInfo>& TabInfo : List)
			{
				TSharedPtr<SDockTab> Tab = TabInfo->GetTab().Pin();
				if (TabInfo->PayloadMatches(InPayload))
				{
					// Manually opening an existing tab, add to history
					TabInfo->AddTabHistory(Factory->CreateTabHistoryNode(InPayload), true);
					TabManager->DrawAttention(Tab.ToSharedRef());
					return Tab;
				}
			}
		}
	}

	// Occurs when forcing open a new tab, or if it failed to find existing one
	return OpenNewTab(Factory->CreateTabHistoryNode(InPayload), InOpenCause);
}

TWeakPtr< FTabInfo > FDocumentTracker::FindTabInForeground()
{
	// Find a tab that is in the foreground
	FTabList& List = GetSpawnedList();
	for (auto ListIt = List.CreateIterator(); ListIt; ++ListIt)
	{
		if((*ListIt)->GetTab().Pin()->IsForeground())
		{
			return *ListIt;
		}
	}

	return nullptr;
}

bool FDocumentTracker::NavigateToTabHistory(int32 InHistoryIdx)
{
	if (History.IsValidIndex(InHistoryIdx))
	{
		TSharedPtr<FGenericTabHistory> NewHistory = History[InHistoryIdx];
		if (NewHistory->IsHistoryValid())
		{
			CurrentHistoryIndex = InHistoryIdx;

			TSharedPtr<FTabInfo> FoundPayloadTab, FoundBoundTab;
			FTabList& List = GetSpawnedList();
			for (TSharedPtr<FTabInfo>& TabInfo : List)
			{
				if (TabInfo->GetCurrentHistory() == NewHistory)
				{
					// If it's literally the same history item that is already active, just focus the tab
					TabManager->DrawAttention(TabInfo->GetTab().Pin().ToSharedRef());
					return true;
				}
				else if (TabInfo->GetCurrentHistory()->GetBoundTab() == NewHistory->GetBoundTab())
				{
					FoundBoundTab = TabInfo;
				}
				else if (TabInfo->PayloadMatches(NewHistory->GetPayload()))
				{
					FoundPayloadTab = TabInfo;
				}
			}

			// If no bound tab but found matching payload, use that
			if (!FoundBoundTab.IsValid() && FoundPayloadTab.IsValid())
			{
				FoundBoundTab = FoundPayloadTab;
			}

			if (FoundBoundTab.IsValid())
			{
				// Found a mostly matching tab we want to restore in
				FoundBoundTab->SetCurrentHistory(NewHistory, true, true);
				TabManager->DrawAttention(FoundBoundTab->GetTab().Pin().ToSharedRef());
				return true;
			}

			// The tab was closed, so spawn a new one
			OpenNewTab(NewHistory, EOpenDocumentCause::NavigatingHistory);
			return true;
		}
	}

	return false;
}

TSharedPtr<FGenericTabHistory> FDocumentTracker::GetCurrentTabHistory()
{
	if (History.IsValidIndex(CurrentHistoryIndex))
	{
		return History[CurrentHistoryIndex];
	}
	return nullptr;
}

TSharedPtr<SDockTab> FDocumentTracker::NavigateCurrentTab(TSharedPtr<FTabPayload> InPayload, EOpenDocumentCause InNavigateCause)
{
	ensure(InNavigateCause == NavigatingCurrentDocument || InNavigateCause == QuickNavigateCurrentDocument || InNavigateCause == NavigateBackwards || InNavigateCause == NavigateForwards);

	FTabList& List = GetSpawnedList();
	if(List.Num())
	{
		// Make sure we find an available tab to navigate, there are ones available.
		TSharedPtr<FTabInfo> LastEditedTabInfo = GetLastEditedTabInfo();
		if(!LastEditedTabInfo.IsValid())
		{
			WeakLastEditedTabInfo = FindTabInForeground();

			// Check if we still do not have a valid tab, if we do not, activate the first tab
			// Any invalid tabs would have been deleted inside GetSpawnedList so we know 0 is valid
			if(!WeakLastEditedTabInfo.IsValid())
			{
				WeakLastEditedTabInfo = List[0];
				List[0]->GetTab().Pin()->ActivateInParent(ETabActivationCause::SetDirectly);
			}

			LastEditedTabInfo = GetLastEditedTabInfo();
		}

		check(LastEditedTabInfo.IsValid());
		TSharedPtr<SDockTab> LastEditedTab = LastEditedTabInfo->GetTab().Pin();

		if(InNavigateCause == NavigatingCurrentDocument || InNavigateCause == QuickNavigateCurrentDocument)
		{
			TSharedPtr<FDocumentTabFactory> Factory = FindSupportingFactory(InPayload.ToSharedRef());
			if (Factory.IsValid())
			{
				// If doing a Quick navigate of the document, do not save history data as it's likely still at the default values. The object is always saved
				LastEditedTabInfo->AddTabHistory(Factory->CreateTabHistoryNode(InPayload), InNavigateCause != QuickNavigateCurrentDocument);
				// Ensure that the tab appears if the tab isn't currently in the foreground.
				LastEditedTab->ActivateInParent(ETabActivationCause::SetDirectly);
			}
		}
		else if(InNavigateCause == NavigateBackwards)
		{
			LastEditedTabInfo->OnGoBackInHistory();
		}
		else if(InNavigateCause == NavigateForwards)
		{
			LastEditedTabInfo->OnGoForwardInHistory();
		}

		return LastEditedTab;
	}
	
	// Open in new tab
	TSharedPtr<FDocumentTabFactory> Factory = FindSupportingFactory(InPayload.ToSharedRef());
	if (Factory.IsValid())
	{
		return OpenNewTab(Factory->CreateTabHistoryNode(InPayload), OpenNewDocument);
	}
	
	return nullptr;
}

TSharedPtr<SDockTab> FDocumentTracker::OpenNewTab(TSharedPtr<FGenericTabHistory> InTabHistory, EOpenDocumentCause InOpenCause)
{
	ensure(InOpenCause == ForceOpenNewDocument || InOpenCause == OpenNewDocument || InOpenCause == RestorePreviousDocument || InOpenCause == NavigatingHistory || InOpenCause == SpawnManagedDocument);

	TSharedPtr<FDocumentTabFactory> Factory = InTabHistory->GetFactory().Pin();
	TSharedPtr<SDockTab> NewTab;

	if(Factory.IsValid())
	{
		TSharedPtr<FAssetEditorToolkit> HostingApp = HostingAppPtr.Pin();
		FWorkflowTabSpawnInfo SpawnInfo;
		SpawnInfo.Payload = InTabHistory->GetPayload();

		NewTab = Factory->SpawnBlankTab();

		TSharedPtr<FTabInfo> NewTabInfo = MakeShareable( new FTabInfo(NewTab.ToSharedRef(), Factory, AsShared()) );
		SpawnedTabs.Add( NewTabInfo );

		if (InOpenCause == NavigatingHistory)
		{
			// Don't want to create a new history entry when going back/forward
			NewTabInfo->SetCurrentHistory(InTabHistory, true, true);
		}
		else
		{
			NewTabInfo->AddTabHistory(InTabHistory);
		}

		const FName DocumentId = (DefaultDocumentId != NAME_None) ? DefaultDocumentId : Factory->GetIdentifier();

		if (InOpenCause == ForceOpenNewDocument  || InOpenCause == OpenNewDocument)
		{
			TabManager->InsertNewDocumentTab( DocumentId, FTabManager::ESearchPreference::RequireClosedTab, NewTab.ToSharedRef() );
		}
		else if (InOpenCause == RestorePreviousDocument)
		{
			TabManager->RestoreDocumentTab( DocumentId, FTabManager::ESearchPreference::RequireClosedTab, NewTab.ToSharedRef() );

			// Clear tab history before this so previous restores don't show up
			History.RemoveAt(0, History.Num() - 1, EAllowShrinking::Yes);
			CurrentHistoryIndex = History.Num() - 1;
		}
	}

	return NewTab;
}

void FDocumentTracker::CloseTab(TSharedPtr<FTabPayload> Payload)
{
	FTabList List = GetSpawnedList();
	for (auto ListIt = List.CreateIterator(); ListIt; ++ListIt)
	{
		if ((*ListIt)->PayloadMatches(Payload))
		{
			TSharedPtr<SDockTab> Tab = (*ListIt)->GetTab().Pin();
			Tab->RequestCloseTab();
		}
	}
}

void FDocumentTracker::CleanInvalidTabs()
{
	FTabList List = GetSpawnedList();
	for (auto ListIt = List.CreateIterator(); ListIt; ++ListIt)
	{
		(*ListIt)->JumpToNearestValidHistoryData();
	}
}

// Finds a factory that can deal with the supplied payload
TSharedPtr<FDocumentTabFactory> FDocumentTracker::FindSupportingFactory(TSharedRef<FTabPayload> Payload) const
{
	for (auto FactoryIt = PotentialTabFactories.CreateConstIterator(); FactoryIt; ++FactoryIt)
	{
		TSharedPtr<FDocumentTabFactory> Factory = FactoryIt.Value();

		if (Factory->IsPayloadSupported(Payload) && TabManager->GetTabPermissionList()->PassesFilter(Factory->GetIdentifier()))
		{
			return Factory;
		}
	}

	return TSharedPtr<FDocumentTabFactory>();
}

// Finds all tabs that match the specified identifier and payload, placing them in the specified array
void FDocumentTracker::FindMatchingTabs( TSharedPtr<FTabPayload> Payload, /*inout*/ TArray< TSharedPtr<SDockTab> >& Results)
{
	FTabList& List = GetSpawnedList();
	for (auto ListIt = List.CreateIterator(); ListIt; ++ListIt)
	{
		if ((*ListIt)->PayloadMatches(Payload))
		{
			Results.Add((*ListIt)->GetTab().Pin());
		}
	}
}

// Finds all tabs that match the specified identifier, placing them in the specified array
void FDocumentTracker::FindAllTabsForFactory(const TWeakPtr<FDocumentTabFactory>& Factory, /*inout*/ TArray< TSharedPtr<SDockTab> >& Results)
{
	FTabList& List = GetSpawnedList();
	for (auto ListIt = List.CreateIterator(); ListIt; ++ListIt)
	{
		if ((*ListIt)->GetFactory() == Factory)
		{
			Results.Add((*ListIt)->GetTab().Pin());
		}
		
	}
}

// Saves the state of all tabs
//@TODO: DOCMANAGEMENT: Basically a total copy of RefreshAllTabs; need to make this sort of pattern be a delegate for-all-tabs
void FDocumentTracker::SaveAllState()
{
	for (auto ToolPanelIt = SpawnedTabs.CreateIterator(); ToolPanelIt; ++ToolPanelIt)
	{
		// Get the factory (can't fail; the tabs had to come from somewhere; failing means a tab survived a mode transition to a mode where it is not allowed!)
		TSharedPtr<FDocumentTabFactory> Factory = (*ToolPanelIt)->GetFactory().Pin();
		if (ensure(Factory.IsValid()) && (*ToolPanelIt)->GetTab().IsValid() && !(*ToolPanelIt)->GetTab().Pin()->IsForeground())
		{
			Factory->SaveState((*ToolPanelIt)->GetTab().Pin(), (*ToolPanelIt)->GetPayload());
		}
	}
	// Save the foreground tabs after saving the background tabs
	// This ensures foreground tabs always are restored after the background tabs.
	for (auto ToolPanelIt = SpawnedTabs.CreateIterator(); ToolPanelIt; ++ToolPanelIt)
	{
		TSharedPtr<FDocumentTabFactory> Factory = (*ToolPanelIt)->GetFactory().Pin();
		if (ensure(Factory.IsValid()) && (*ToolPanelIt)->GetTab().IsValid() && (*ToolPanelIt)->GetTab().Pin()->IsForeground())
		{
			Factory->SaveState((*ToolPanelIt)->GetTab().Pin(), (*ToolPanelIt)->GetPayload());
		}
	}
}

// Calls OnTabRefreshed for each open tab (on the factories that created them)
void FDocumentTracker::RefreshAllTabs() const
{
	for (auto ToolPanelIt = SpawnedTabs.CreateConstIterator(); ToolPanelIt; ++ToolPanelIt)
	{
		// Get the factory (can't fail; the tabs had to come from somewhere; failing means a tab survived a mode transition to a mode where it is not allowed!)
		TSharedPtr<FDocumentTabFactory> Factory = (*ToolPanelIt)->GetFactory().Pin();
		// Run thru the open tabs for this one
		if (ensure(Factory.IsValid()))
		{
			TSharedPtr<SDockTab> Tab = (*ToolPanelIt)->GetTab().Pin();
			if (Tab.IsValid())
			{
				Factory->OnTabRefreshed(Tab);
			}			
		}
	}
}

// Replaces the open payload in the specified tab with a new one; recreating the contents
void FDocumentTracker::ReplacePayloadInTab(TSharedPtr<SDockTab> TargetTab, TSharedPtr<FTabPayload> NewPayload)
{
	// Find the existing tab
	for (auto ToolPanelIt = SpawnedTabs.CreateIterator(); ToolPanelIt; ++ToolPanelIt)
	{
		TSharedPtr<SDockTab> Tab = (*ToolPanelIt)->GetTab().Pin();

		if (Tab == TargetTab)
		{
			// Get the factory (can't fail; the tabs had to come from somewhere; failing means a tab survived a mode transition to a mode where it is not allowed!)
			TSharedPtr<FDocumentTabFactory> Factory = (*ToolPanelIt)->GetFactory().Pin();
			if (ensure(Factory.IsValid()) && ensure(Factory->IsPayloadSupported(NewPayload.ToSharedRef())))
			{
				FWorkflowTabSpawnInfo Info;
				Info.Payload = NewPayload;
				Tab->SetContent(Factory->CreateTabBody(Info));

				return;
			}
		}
	}

}

TArray< TSharedPtr<SDockTab> > FDocumentTracker::GetAllDocumentTabs() const
{
	TArray< TSharedPtr<SDockTab> > AllSpawnedDocuments;
	for (int32 DocIndex=0; DocIndex < SpawnedTabs.Num(); ++DocIndex)
	{
		if (SpawnedTabs[DocIndex]->GetTab().IsValid())
		{
			AllSpawnedDocuments.Add(SpawnedTabs[DocIndex]->GetTab().Pin());
		}
	}

	return AllSpawnedDocuments;
}

TSharedPtr<SDockTab> FDocumentTracker::GetActiveTab() const
{
	if(WeakLastEditedTabInfo.IsValid())
	{
		return WeakLastEditedTabInfo.Pin()->GetTab().Pin();
	}

	return nullptr;
}

FReply FDocumentTracker::OnNavigateTab(FDocumentTracker::EOpenDocumentCause InCause)
{
	NavigateCurrentTab(FTabPayload_UObject::Make(nullptr), InCause);
	return FReply::Handled();
}
