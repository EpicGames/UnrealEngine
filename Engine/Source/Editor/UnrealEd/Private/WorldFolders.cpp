// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldFolders.h"
#include "ActorFolders/ActorFolderColumns.h"
#include "ActorFolders/TedsActorFolderUtils.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "DataStorage/Features.h"
#include "Dom/JsonObject.h"
#include "Engine/World.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "EngineUtils.h"
#include "EditorActorFolders.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "UObject/Package.h"
#include "UObject/WeakObjectPtr.h"
#include "Editor.h"
#include "Elements/Columns/TypedElementUIColumns.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldFolders)

#define LOCTEXT_NAMESPACE "UnrealEd.WorldFolders"

DEFINE_LOG_CATEGORY(LogWorldFolders);

namespace UE::Editor::WorldFolders
{
	static bool bRegisterFoldersInTEDS = false;

	// Cvar to control registration of actor folders in TEDS
	static FAutoConsoleVariableRef CvarRegisterFoldersInTEDS(
		TEXT("TEDS.Feature.Folders"),
		bRegisterFoldersInTEDS,
		TEXT("Populate FFolders and Actor Folders in TEDS. Must be set at startup."));
}

void FActorPlacementFolder::Reset()
{
	Path = NAME_None;
	RootObjectPtr.Reset();
	ActorFolderGuid.Invalidate();
}

FActorPlacementFolder& FActorPlacementFolder::operator= (const FFolder& InOtherFolder)
{
	Path = InOtherFolder.GetPath();
	RootObjectPtr = InOtherFolder.GetRootObjectPtr();
	ActorFolderGuid = InOtherFolder.GetActorFolderGuid();
	return *this;
}

FFolder FActorPlacementFolder::GetFolder() const
{
	FFolder::FRootObject RootObject = FFolder::FRootObject(RootObjectPtr.Get());
	return ActorFolderGuid.IsValid() ? FFolder(RootObject, ActorFolderGuid) : FFolder(RootObject, Path);
}

void UWorldFolders::Initialize(UWorld* InWorld)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldFolders::Initialize);

	check(!World.IsValid());
	check(IsValidChecked(InWorld));
	
	World = InWorld;
	SetFlags(RF_Transactional);
	
	PersistentFolders = MakeUnique<FWorldPersistentFolders>(*this);
	TransientFolders = MakeUnique<FWorldTransientFolders>(*this);
	
	if (UE::Editor::WorldFolders::CvarRegisterFoldersInTEDS->GetBool())
	{
		using namespace UE::Editor::DataStorage;

		auto OnDataStorage = [this]
		{
			check(IsInGameThread());
				
			DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
			if (DataStorage)
			{
				// Sync folders that were added before DataStorage was available
				for (const FFolder& Folder : PendingTEDSRegistrations)
				{
					ActorFolders::RegisterFolderInTeds(DataStorage, Folder, GetWorld());
				}
				PendingTEDSRegistrations.Empty();
			}
		};

		if (AreEditorDataStorageFeaturesEnabled())
		{
			OnDataStorage();
		}
		else
		{
			OnDataStorageInitialized = OnEditorDataStorageFeaturesEnabled().AddLambda(OnDataStorage);
		}
	}

	bQueuedRebuild = false;
	RebuildList();
	
	LoadState();
}

void UWorldFolders::RebuildList()
{
	if (GetWorld()->GetIsInBlockTillLevelStreamingCompleted())
	{
		if (!bQueuedRebuild)
		{
			bQueuedRebuild = true;

			GEditor->GetTimerManager()->SetTimerForNextTick([WeakThis = TWeakObjectPtr<UWorldFolders>(this)]()
			{
				if (WeakThis.IsValid())
				{
					// Test if the rebuild is still queued...
					if (WeakThis->bQueuedRebuild)
					{
						WeakThis->RebuildListInternal();
					}
				}
			});
		}
	}
	else
	{
		RebuildListInternal();
	}
}

void UWorldFolders::RebuildListInternal()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldFolders::RebuildList);

	// No need to rebuild if world is not valid or if it's a game world
	// Because life cycle of UWorldFolders is decoupled from its World, it's 
	// possible that the world is not valid anymore (see FActorFolders::Housekeeping 
	// and FActorFolders::AddReferencedObjects).
	if (!GetWorld() || GetWorld()->IsGameWorld())
	{
		return;
	}

	Modify();
	
	// Clear folders with a Root Object.
	TArray<FFolder> FoldersToRemove;
	ForEachFolder([&FoldersToRemove](const FFolder& Folder)
	{
		if (!Folder.IsRootObjectPersistentLevel())
		{
			FoldersToRemove.Add(Folder);
		}
		return true;
	});

	for (const FFolder& Folder : FoldersToRemove)
	{
		RemoveFolder(Folder);
	}

	// Iterate over every actor in memory. WARNING: This is potentially very expensive!
	for (FActorIterator ActorIt(GetWorld()); ActorIt; ++ActorIt)
	{
		AddFolder(ActorIt->GetFolder());
	}

	// Add levels ActorFolders as they are still needed for unloaded actors (only in editor)
	if (!GetWorld()->IsGameWorld())
	{
		for (ULevel* Level : GetWorld()->GetLevels())
		{
			const bool bIsLevelVisibleOrAssociating = (Level->bIsVisible && !Level->bIsBeingRemoved) || Level->bIsAssociatingLevel || Level->bIsDisassociatingLevel;
			if (bIsLevelVisibleOrAssociating)
			{
				Level->ForEachActorFolder([this](UActorFolder* ActorFolder)
				{
					AddFolder(ActorFolder->GetFolder());
					return true;
				}, /*bSkipDeleted*/ true);
			}
		}
	}

	bQueuedRebuild = false;
}

void UWorldFolders::AddFolder_Internal(const FFolder& InFolder, const FActorFolderProps& FolderProperties)
{
	FoldersProperties.Add(InFolder, FolderProperties);
	
	if (UE::Editor::WorldFolders::CvarRegisterFoldersInTEDS->GetBool())
	{
		using namespace UE::Editor::DataStorage;
		if (DataStorage)
		{
			ActorFolders::RegisterFolderInTeds(DataStorage, InFolder, GetWorld());
		}
		else
		{
			PendingTEDSRegistrations.Add(InFolder);
		}
	}
}

void UWorldFolders::RemoveFolder_Internal(const FFolder& InFolder)
{
	if (UE::Editor::WorldFolders::CvarRegisterFoldersInTEDS->GetBool())
	{
		using namespace UE::Editor::DataStorage;
		if (DataStorage)
		{
			ActorFolders::UnregisterFolderFromTeds(DataStorage, InFolder);
		}
		else
		{
			PendingTEDSRegistrations.Remove(InFolder);
		}
	}

	FoldersProperties.Remove(InFolder);
}

void UWorldFolders::ReAddFolder_Internal(const FFolder& InOldFolder, const FFolder& InNewFolder, const FActorFolderProps& InNewFolderProperties)
{
	FoldersProperties.Remove(InOldFolder);
	FoldersProperties.Add(InNewFolder, InNewFolderProperties);

	if (UE::Editor::WorldFolders::CvarRegisterFoldersInTEDS->GetBool())
	{
		using namespace UE::Editor::DataStorage;
		if (DataStorage)
		{
			ActorFolders::RemapFolder(DataStorage, InOldFolder, InNewFolder, GetWorld());
		}
		else
		{
			PendingTEDSRegistrations.Remove(InOldFolder);
			PendingTEDSRegistrations.Add(InNewFolder);
		}
	}
}

UWorld* UWorldFolders::GetWorld() const
{
	return World.Get();
}

void UWorldFolders::OnRemoved()
{
	// This function is called when the FActorFolders bookkeeping decides that a particular UWorldFolders object isn't needed anymore
	// so it's a good spot to unregister all folders belonging to a world from TEDS without relying on GC timing
	if (UE::Editor::WorldFolders::CvarRegisterFoldersInTEDS->GetBool())
	{
		using namespace UE::Editor::DataStorage;

		OnEditorDataStorageFeaturesEnabled().Remove(OnDataStorageInitialized);
		if (DataStorage)
		{
			ForEachFolder([this](const FFolder& InFolder)
			{
				ActorFolders::UnregisterFolderFromTeds(DataStorage, InFolder);
				return true;
			});
		}
	}
}

bool UWorldFolders::AddFolder(const FFolder& InFolder)
{
	if (!InFolder.IsNone())
	{
		if (!FoldersProperties.Contains(InFolder))
		{
			// Add the parent as well
			const FFolder ParentFolder = InFolder.GetParent();
			if (!ParentFolder.IsNone())
			{
				AddFolder(ParentFolder);
			}

			Modify();
			
			bool bAdded = GetImpl(InFolder).AddFolder(InFolder);

			FActorFolderProps* LoadedFolderProps = LoadedStateFoldersProperties.Find(InFolder);
			AddFolder_Internal(InFolder, LoadedFolderProps ? *LoadedFolderProps : FActorFolderProps());
			
			return bAdded;
		}
		// The FFolder key (RootObject + Path) is stable across a load/unload of the underlying UActorFolder UObject, e.g. when a Level Instance 
		// is edited, the editable inner level is unloaded and reloaded, replacing each UActorFolder with a new UObject that has the
		// same GUID and label. TEDS Compat drops the row when the old UObject is GC'd, but the FoldersProperties entry is keyed on FFolder and 
		// survives, so the rebroadcast hits the FoldersProperties.Contains() guard and the new UObject never makes it into TEDS. 
		// Detect that, and re-register only the TEDS row without touching the rest of the folder state.
		if (InFolder.GetActorFolder() && UE::Editor::WorldFolders::CvarRegisterFoldersInTEDS->GetBool() && DataStorage)
		{
			UE::Editor::DataStorage::ActorFolders::RegisterFolderInTeds(DataStorage, InFolder, GetWorld());
		}
	}
	
	return false;
}

bool UWorldFolders::RemoveFolder(const FFolder& InFolder, bool bShouldDeleteFolder)
{
	if (FoldersProperties.Contains(InFolder))
	{
		Modify();
		RemoveFolder_Internal(InFolder);
		return GetImpl(InFolder).RemoveFolder(InFolder, bShouldDeleteFolder);
	}
	return false;
}

bool UWorldFolders::RenameFolder(const FFolder& InOldFolder, const FFolder& InNewFolder)
{
	Modify();

	check(IsValid(World.Get()));
	check(InOldFolder.GetRootObject() == InNewFolder.GetRootObject());

	bool bNeedsReselect = false;
	if (UE::Editor::WorldFolders::CvarRegisterFoldersInTEDS->GetBool() && DataStorage)
	{
		bNeedsReselect = UE::Editor::DataStorage::ActorFolders::DeselectFolderIfSelected(DataStorage, InOldFolder);
	}

	bool bSuccess = GetImpl(InOldFolder).RenameFolder(InOldFolder, InNewFolder);
	if (bSuccess)
	{
		bool bChanged = false;
		for (int StackIndex=0; StackIndex < CurrentFolderStack.Num(); ++StackIndex)
		{
			FActorPlacementFolder& StackElement = CurrentFolderStack[StackIndex];
			if (StackElement.GetFolder() == InOldFolder)
			{
				StackElement.Path = InNewFolder.GetPath();
				bChanged = true;
			}
		}
		if (CurrentFolder.GetFolder() == InOldFolder)
		{
			CurrentFolder.Path = InNewFolder.GetPath();
			bChanged = true;
		}
		if (bChanged)
		{
			FActorFolders::Get().BroadcastOnActorEditorContextClientChanged(*World.Get());
		}

		// The label for folders in TEDS is updated by a processor, add the sync tag so that it triggers for legacy FFolders as well
		using namespace UE::Editor::DataStorage;
		if (UE::Editor::WorldFolders::CvarRegisterFoldersInTEDS->GetBool() && DataStorage)
		{
			RowHandle NewFolderRow = ActorFolders::LookupMappedRow(DataStorage, InNewFolder);
			DataStorage->AddColumn<FTypedElementSyncFromWorldTag>(NewFolderRow);

			if (bNeedsReselect)
			{
				ActorFolders::SelectFolder(DataStorage, InNewFolder);
			}
		}
	}
	return bSuccess;
}

void UWorldFolders::BroadcastOnActorFolderCreated(const FFolder& InFolder)
{
	check(World.IsValid());
	FActorFolders::Get().BroadcastOnActorFolderCreated(*World, InFolder);
}

void UWorldFolders::BroadcastOnActorFolderDeleted(const FFolder& InFolder)
{
	check(World.IsValid());
	FActorFolders::Get().BroadcastOnActorFolderDeleted(*World, InFolder);
}

void UWorldFolders::BroadcastOnActorFolderMoved(const FFolder& InSrcFolder, const FFolder& InDstFolder)
{
	check(World.IsValid());
	FActorFolders::Get().BroadcastOnActorFolderMoved(*World, InSrcFolder, InDstFolder);
}

bool UWorldFolders::IsFolderExpanded(const FFolder& InFolder) const
{
	const FActorFolderProps* FolderProps = FoldersProperties.Find(InFolder);
	return FolderProps ? FolderProps->bIsExpanded : false;
}

bool UWorldFolders::SetIsFolderExpanded(const FFolder& InFolder, bool bIsExpanded)
{
	if (FActorFolderProps* FolderProps = FoldersProperties.Find(InFolder))
	{
		FolderProps->bIsExpanded = bIsExpanded;
		
		if (UE::Editor::WorldFolders::CvarRegisterFoldersInTEDS->GetBool() && DataStorage)
		{
			using namespace UE::Editor::DataStorage;
			
			RowHandle FolderRow = ActorFolders::LookupMappedRow(DataStorage, InFolder);
			bIsExpanded ? DataStorage->AddColumn<FExpandedInUITag>(FolderRow) : DataStorage->RemoveColumn<FExpandedInUITag>(FolderRow);
		}
		
		return true;
	}
	return false;
}

FFolder UWorldFolders::GetActorEditorContextFolder(bool bMustMatchCurrentLevel) const
{
	FFolder Folder = CurrentFolder.GetFolder();
	if (ContainsFolder(Folder))
	{
		ULevelInstanceSubsystem* LevelInstanceSubsystem = GetWorld()->GetSubsystem<ULevelInstanceSubsystem>();
		ILevelInstanceInterface* EditingLevelInstance = LevelInstanceSubsystem ? LevelInstanceSubsystem->GetEditingLevelInstance() : nullptr;
		if (UObject* EditingLevelInstanceObject = EditingLevelInstance ? CastChecked<UObject>(EditingLevelInstance) : nullptr)
		{
			FFolder::FRootObject RootObject = FFolder::FRootObject(EditingLevelInstanceObject);
			if (Folder.GetRootObject() == RootObject)
			{
				return Folder;
			}
		}
		else if (!bMustMatchCurrentLevel || (Folder.GetRootObject() == FFolder::FRootObject(GetWorld()->GetCurrentLevel())))
		{
			return Folder;
		}
	}
	return FFolder::GetWorldRootFolder(GetWorld());
}

bool UWorldFolders::SetActorEditorContextFolder(const FFolder& InFolder)
{
	if (InFolder != CurrentFolder.GetFolder())
	{
		Modify();
		CurrentFolder = InFolder;
		return true;
	}
	return false;
}

void UWorldFolders::PushActorEditorContext(bool bDuplicateContext)
{
	Modify();
	CurrentFolderStack.Push(CurrentFolder);
	if (!bDuplicateContext)
	{
		CurrentFolder.Reset();
	}
}

void UWorldFolders::PopActorEditorContext()
{
	check(!CurrentFolderStack.IsEmpty());

	Modify();
	CurrentFolder = CurrentFolderStack.Pop();
}

bool UWorldFolders::ContainsFolder(const FFolder& InFolder) const
{
	return InFolder.IsValid() && !InFolder.IsNone() && GetImpl(InFolder).ContainsFolder(InFolder);
}

void UWorldFolders::ForEachFolder(TFunctionRef<bool(const FFolder&)> Operation)
{
	for (const auto& Pair : FoldersProperties)
	{
		if (!Operation(Pair.Key))
		{
			break;
		}
	}
}

void UWorldFolders::ForEachFolderWithRootObject(const FFolder::FRootObject& InFolderRootObject, TFunctionRef<bool(const FFolder&)> Operation)
{
	for (const auto& Pair : FoldersProperties)
	{
		const FFolder& Folder = Pair.Key;
		if (Folder.GetRootObject() == InFolderRootObject)
		{
			if (!Operation(Folder))
			{
				break;
			}
		}
	}
}

void UWorldFolders::Serialize(FArchive& Ar)
{
	if (IsTemplate())
	{
		return;
	}

	check(PersistentFolders.IsValid());
	Ar << FoldersProperties;
	
	Super::Serialize(Ar);
}

void UWorldFolders::PreEditUndo()
{
	Super::PreEditUndo();
	
	// We are currently mirroring folder data in TEDS, however with current implementation of UWorldFolders the individual folders don't receive
	// any notification when their data changes via undo/redo and neither are any delegates fired. The per-world UWorldFolders object is the one
	// that is modified/restored using the transaction system.
	
	// To make sure these changes are tracked in TEDS, we need to keep a copy of all folders before and after the undo/redo to compare and figure out
	// which folders changed
	if (UE::Editor::WorldFolders::CvarRegisterFoldersInTEDS->GetBool() && DataStorage)
	{
		PreUndoFolders.Empty();
		
		// Cache the list of folders we have before the undo/redo operation to compare after
		for (const TPair<FFolder, FActorFolderProps>& FolderIt : FoldersProperties)
		{
			// Actor folders are handled by TEDS compat since they are UObjects, plus their key is not stable across renames so this logic cannot
			// track them properly
			if (!FolderIt.Key.GetActorFolder())
			{
				PreUndoFolders.Add(FolderIt.Key);
			}
		}
	}
}

void UWorldFolders::PostEditUndo()
{
	Super::PostEditUndo();
	
	// Compare the pre-undo list with the post-undo list to find out which folders were added/removed because of the operation
	if (UE::Editor::WorldFolders::CvarRegisterFoldersInTEDS->GetBool() && DataStorage)
	{
		using namespace UE::Editor::DataStorage;
		
		// Get the list of folders we have after the undo/redo operation
		TSet<FFolder> PostUndoFolders;
		PostUndoFolders.Reserve(FoldersProperties.Num());
		
		for (const TPair<FFolder, FActorFolderProps>& FolderIt : FoldersProperties)
		{
			// Actor folders are handled by TEDS compat since they are UObjects, plus their key is not stable across renames so this logic cannot
			// track them properly
			if (!FolderIt.Key.GetActorFolder())
			{
				PostUndoFolders.Add(FolderIt.Key);
			}
			
			// We also want to add the sync tag to all folders, since undo/redo can cause data changes (e.g label) and not just remove/add which
			// needs to re-run processors (similar to how actors get this tag automatically on undo/redo because of TEDS compat)
			
			// While actor folders also go through TEDS compat and get this tag added, that isn't true for all operations since some data is still
			// stored in the FFolder for actor folders which does not modify the UObject. So we add it manually just to be sure
			RowHandle FolderRow = ActorFolders::LookupMappedRow(DataStorage, FolderIt.Key);
			DataStorage->AddColumn<FTypedElementSyncFromWorldTag>(FolderRow);
		}
	
		// Any items that exist post-undo but did not exist pre-undo were "added" because of the undo
		TSet<FFolder> AddedFolders = PostUndoFolders.Difference(PreUndoFolders);
		// Add those folders to TEDS
		for (const FFolder& AddedFolder : AddedFolders)
		{
			ActorFolders::RegisterFolderInTeds(DataStorage, AddedFolder, GetWorld());
		}
		
		// Any items that existed pre-undo but do not exist post-undo were "removed" because of the undo
		TSet<FFolder> RemovedFolders = PreUndoFolders.Difference(PostUndoFolders);
		// Remove those folders from TEDS
		for (const FFolder& RemoveFolder : RemovedFolders)
		{
			ActorFolders::UnregisterFolderFromTeds(DataStorage, RemoveFolder);
		}
	}
}

FString UWorldFolders::GetWorldStateFilename() const
{
	if (World->IsGameWorld() || World->IsInstanced() || FPackageName::IsTempPackage(World->GetPackage()->GetName()))
	{
		return FString();
	}
	UPackage* Package = World->GetOutermost();
	const FString PathName = Package->GetPathName();
	const uint32 PathNameCrc = FCrc::MemCrc32(*PathName, sizeof(TCHAR) * PathName.Len());
	return FPaths::Combine(*FPaths::ProjectSavedDir(), TEXT("Config"), TEXT("WorldState"), *FString::Printf(TEXT("%u.json"), PathNameCrc));
}

void UWorldFolders::LoadState()
{
	const FString Filename = GetWorldStateFilename();
	if (Filename.IsEmpty())
	{
		return;
	}

	// Attempt to load the folder properties from user's saved world state directory and apply them.
	TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileReader(*Filename));
	if (Ar)
	{
		TSharedPtr<FJsonObject> RootObject = MakeShareable(new FJsonObject);
		auto Reader = TJsonReaderFactory<TCHAR>::Create(Ar.Get());
		if (FJsonSerializer::Deserialize(Reader, RootObject))
		{
			FFolder WorldDefaultFolder = FFolder::GetWorldRootFolder(World.Get());
			check(WorldDefaultFolder.IsRootObjectValid());
			const FFolder::FRootObject WorldRootObject = WorldDefaultFolder.GetRootObject();

			const TSharedPtr<FJsonObject>& JsonFolders = RootObject->GetObjectField(TEXT("Folders"));
			for (const auto& KeyValue : JsonFolders->Values)
			{
				// Only pull in the folder's properties if this folder still exists in the world.
				// This means that old stale folders won't re-appear in the world (they'll won't get serialized when the world is saved anyway)
				auto FolderProperties = KeyValue.Value->AsObject();
				const FFolder Folder(WorldRootObject, *KeyValue.Key);
				const bool bIsExpanded = FolderProperties->GetBoolField(TEXT("bIsExpanded"));
				if (!SetIsFolderExpanded(Folder, bIsExpanded))
				{
					FActorFolderProps& LoadedFolderProps = LoadedStateFoldersProperties.FindOrAdd(Folder);
					LoadedFolderProps.bIsExpanded = bIsExpanded;
				}
			}
		}
		Ar->Close();
	}
}

void UWorldFolders::SaveState()
{
	const FString Filename = GetWorldStateFilename();
	if (Filename.IsEmpty())
	{
		return;
	}

	TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileWriter(*Filename));
	if (Ar)
	{
		TSharedRef<FJsonObject> RootObject = MakeShareable(new FJsonObject);
		TSharedRef<FJsonObject> JsonFolders = MakeShareable(new FJsonObject);

		ForEachFolder([this, &JsonFolders](const FFolder& Folder)
		{
			// Only write for World root
			if (Folder.IsRootObjectPersistentLevel())
			{
				TSharedRef<FJsonObject> JsonFolder = MakeShareable(new FJsonObject);
				JsonFolder->SetBoolField(TEXT("bIsExpanded"), IsFolderExpanded(Folder));
				JsonFolders->SetObjectField(Folder.ToString(), JsonFolder);
			}
			return true;
		});

		RootObject->SetObjectField(TEXT("Folders"), JsonFolders);
		{
			auto Writer = TJsonWriterFactory<TCHAR>::Create(Ar.Get());
			FJsonSerializer::Serialize(RootObject, Writer);
			Ar->Close();
		}
	}
}

bool UWorldFolders::IsUsingPersistentFolders(const FFolder& InFolder) const
{
	ULevel* Level = FWorldPersistentFolders::GetRootObjectContainer(InFolder, GetWorld());
	return Level ? Level->IsUsingActorFolders() : false;
}

FWorldFoldersImplementation& UWorldFolders::GetImpl(const FFolder& InFolder) const
{
	if (IsUsingPersistentFolders(InFolder))
	{
		return *PersistentFolders;
	}
	else
	{
		return *TransientFolders;
	}
}

////////////////////////////////////////////
//~ Begin Deprecated
FActorFolderProps* UWorldFolders::GetFolderProperties(const FFolder& InFolder)
{
	return FoldersProperties.Find(InFolder);
}
//~ End Deprecated
////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE 
