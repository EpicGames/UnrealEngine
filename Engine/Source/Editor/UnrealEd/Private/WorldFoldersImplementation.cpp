// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldFoldersImplementation.h"

#include "Containers/Map.h"
#include "WorldFolders.h"

FWorldFoldersImplementation::FWorldFoldersImplementation(UWorldFolders& InWorldFolders)
	: Owner(InWorldFolders)
{
}

UWorld* FWorldFoldersImplementation::GetWorld() const
{
	return Owner.GetWorld();
}

bool FWorldFoldersImplementation::ContainsFolder(const FFolder& InFolder) const
{
	return Owner.FoldersProperties.Contains(InFolder);
}