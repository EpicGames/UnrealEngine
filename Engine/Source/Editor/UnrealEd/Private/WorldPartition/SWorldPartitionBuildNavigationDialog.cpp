// Copyright Epic Games, Inc. All Rights Reserved.
#include "WorldPartition/SWorldPartitionBuildNavigationDialog.h"

#include "DetailsViewArgs.h"
#include "IDetailsView.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Types/SlateEnums.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "WorldPartition/WorldPartitionBuildNavigationOptions.h"

#define LOCTEXT_NAMESPACE "WorldPartitionBuildNavigationDialog"

const FVector2D SWorldPartitionBuildNavigationDialog::DEFAULT_WINDOW_SIZE = FVector2D(600, 350);

void SWorldPartitionBuildNavigationDialog::Construct(const FArguments& InArgs)
{
	ParentWindowPtr = InArgs._ParentWindow.Get();
	BuildNavigationOptions = InArgs._BuildNavigationOptions.Get();
	bClickedOk = false;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bAllowMultipleTopLevelObjects = false;

	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	
	// Set provided objects on SDetailsView
	DetailsView->SetObject(BuildNavigationOptions.Get(), true);

	this->ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.MaxHeight(500.0f)
				[
					DetailsView->AsShared()
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(1.f)
					[
						SNew(SSpacer)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2.f)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.IsEnabled(this, &SWorldPartitionBuildNavigationDialog::IsOkEnabled)
						.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
						.OnClicked(this, &SWorldPartitionBuildNavigationDialog::OnOkClicked)
						.Text(LOCTEXT("OkButton", "Ok"))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2.f)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
						.OnClicked(this, &SWorldPartitionBuildNavigationDialog::OnCancelClicked)
						.Text(LOCTEXT("CancelButton", "Cancel"))
					]
				]
			]
		];
}

bool SWorldPartitionBuildNavigationDialog::IsOkEnabled() const
{
	return true;
}

FReply SWorldPartitionBuildNavigationDialog::OnOkClicked()
{
	bClickedOk = true;
	ParentWindowPtr.Pin()->RequestDestroyWindow();
	return FReply::Handled();
}

FReply SWorldPartitionBuildNavigationDialog::OnCancelClicked()
{
	bClickedOk = false;
	ParentWindowPtr.Pin()->RequestDestroyWindow();
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
