// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionBuilder.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "CoreMinimal.h"
#include "Editor.h"
#include "EditorWorldUtils.h"
#include "EngineModule.h"
#include "UObject/Linker.h"
#include "HAL/PlatformFileManager.h"
#include "Engine/CoreSettings.h"
#include "Engine/World.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/LoaderAdapter/LoaderAdapterShape.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "Math/IntVector.h"
#include "UObject/SavePackage.h"
#include "Algo/ForEach.h"
#include "Algo/Transform.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionBuilder)

DEFINE_LOG_CATEGORY_STATIC(LogWorldPartitionBuilder, All, All);

namespace FWorldPartitionBuilderLog
{
	void Error(bool bErrorsAsWarnings, const FString& Msg)
	{
		if (bErrorsAsWarnings)
		{
			UE_LOGF(LogWorldPartitionBuilder, Warning, "%ls", *Msg);
		}
		else
		{
			UE_LOGF(LogWorldPartitionBuilder, Error, "%ls", *Msg);
		}
	}
}

FCellInfo::FCellInfo()
	: Location(ForceInitToZero)
	, Bounds(ForceInitToZero)
	, EditorBounds(ForceInitToZero)
	, IterativeCellSize(102400)
{
}

FString UWorldPartitionBuilder::Args;

UWorldPartitionBuilder::UWorldPartitionBuilder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UWorld::InitializationValues UWorldPartitionBuilder::GetWorldInitializationValues() const
{
	UWorld::InitializationValues IVS;
	IVS.RequiresHitProxies(false);
	IVS.ShouldSimulatePhysics(false);
	IVS.EnableTraceCollision(false);
	IVS.CreateNavigation(false);
	IVS.CreateAISystem(false);
	IVS.AllowAudioPlayback(false);
	IVS.CreatePhysicsScene(true);
	return IVS;
}

bool UWorldPartitionBuilder::RunBuilder(UWorld* World)
{
	// Load configuration file & builder configuration
	const FString WorldConfigFilename = FPackageName::LongPackageNameToFilename(World->GetPackage()->GetName(), TEXT(".ini"));
	if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*WorldConfigFilename))
	{
		LoadConfig(GetClass(), *WorldConfigFilename);
	}

	// Validate builder settings
	if (IsRunningCommandlet() && RequiresCommandletRendering() && !IsAllowCommandletRendering())
	{
		UE_LOGF(LogWorldPartitionBuilder, Error, "The option \"-AllowCommandletRendering\" must be provided for the %ls process to work", *GetClass()->GetName());
		return false;
	}

	// Verify if the builder want to skip processing the given world.
	if (!ShouldProcessWorld(World))
	{
		return true;
	}

	// Wait for the asset registry to finish scanning
	IAssetRegistry& AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
	AssetRegistry.WaitForCompletion();

	FPackageSourceControlHelper SCCHelper;

	// Perform builder pre world initialisation
	if (!PreWorldInitialization(World, SCCHelper))
	{
		UE_LOGF(LogWorldPartitionBuilder, Error, "PreWorldInitialization failed");
		return false;
	}

	bool bResult = true;
	// Setup the world
	{
		TUniquePtr<FScopedEditorWorld> EditorWorld;
		if (World->bIsWorldInitialized)
		{
			// Skip initialization, but ensure we're dealing with an editor world.
			check(World->WorldType == EWorldType::Editor);
		}
		else
		{
			EditorWorld = MakeUnique<FScopedEditorWorld>(World, GetWorldInitializationValues());
		}

		// Make sure the world is partitioned if required
		if (UWorld::IsPartitionedWorld(World) || CanProcessNonPartitionedWorlds())
		{
			FWorldContext& WorldContext = GEditor->GetEditorWorldContext(true /*bEnsureIsGWorld*/);
			WorldContext.SetCurrentWorld(World);
			UWorld* PrevGWorld = GWorld;
			GWorld = World;

			// Run builder
			bResult = Run(World, SCCHelper);

			// Restore previous world
			WorldContext.SetCurrentWorld(PrevGWorld);
			GWorld = PrevGWorld;

			// Save default configuration
			if (bResult)
			{
				if (!FPlatformFileManager::Get().GetPlatformFile().FileExists(*WorldConfigFilename) ||
					!FPlatformFileManager::Get().GetPlatformFile().IsReadOnly(*WorldConfigFilename))
				{
					SaveConfig(CPF_Config, *WorldConfigFilename);
				}
			}
		}
		else
		{
			UE_LOGF(LogWorldPartitionBuilder, Log, "WorldPartition builder '%ls' only works on partitioned worlds - skipping processing of '%ls'.", *GetClass()->GetName(), *World->GetPathName());
		}
	}

	if (bResult)
	{
		bResult = PostWorldTeardown(SCCHelper);
	}

	return bResult;
}

FWorldBuilderCellCoord FCellInfo::GetCellCoord(const FVector& InPos, const int32 InCellSize)
{
	return FWorldBuilderCellCoord(
		FMath::FloorToInt(InPos.X / InCellSize),
		FMath::FloorToInt(InPos.Y / InCellSize),
		FMath::FloorToInt(InPos.Z / InCellSize)
	);
}

FWorldBuilderCellCoord FCellInfo::GetCellCount(const FBox& InBounds, const int32 InCellSize)
{
	const FWorldBuilderCellCoord MinCellCoords = GetCellCoord(InBounds.Min, InCellSize);
	const FWorldBuilderCellCoord MaxCellCoords = FWorldBuilderCellCoord(FMath::CeilToInt(InBounds.Max.X / InCellSize),
												FMath::CeilToInt(InBounds.Max.Y / InCellSize),
												FMath::CeilToInt(InBounds.Max.Z / InCellSize));
	return MaxCellCoords - MinCellCoords;
}

bool UWorldPartitionBuilder::Run(UWorld* World, FPackageSourceControlHelper& PackageHelper)
{
	// As we manage GC frequency ourselves during builds, make sure to turn off automated GC based on other settings
	TGuardValue<int32> Guard_GLevelStreamingContinuouslyIncrementalGCWhileLevelsPendingPurge(GLevelStreamingContinuouslyIncrementalGCWhileLevelsPendingPurge, 0);
	TGuardValue<int32> Guard_GLevelStreamingContinuouslyIncrementalGCWhileActorsPendingPurge(GLevelStreamingContinuouslyIncrementalGCWhileActorsPendingPurge, 0);
	TGuardValue<int32> Guard_GLevelStreamingLowMemoryPendingPurgeCount(GLevelStreamingLowMemoryPendingPurgeCount, MAX_int32);
	TGuardValue<int32> Guard_GLevelStreamingLowMemoryActorsPurgeCount(GLevelStreamingLowMemoryActorsPendingPurgeCount, MAX_int32);
	TGuardValue<int32> Guard_GLevelStreamingForceGCAfterLevelStreamedOut(GLevelStreamingForceGCAfterLevelStreamedOut, 0);

	UWorldPartition* WorldPartition = World->GetWorldPartition();
	check(WorldPartition || CanProcessNonPartitionedWorlds());

	// Notify derived classes that partition building process starts
	bool bResult = PreRun(World, PackageHelper);

	// Load data layers
	LoadDataLayers(World);
	
	static const FBox BoxEntireWorld = FBox(FVector(-HALF_WORLD_MAX, -HALF_WORLD_MAX, -HALF_WORLD_MAX), FVector(HALF_WORLD_MAX, HALF_WORLD_MAX, HALF_WORLD_MAX));
	const ELoadingMode LoadingMode = WorldPartition ? GetLoadingMode() : ELoadingMode::Custom;
		
	FCellInfo CellInfo;
	CellInfo.EditorBounds = IterativeWorldBounds.IsValid ? IterativeWorldBounds : WorldPartition ? WorldPartition->GetEditorWorldBounds() : BoxEntireWorld;
	CellInfo.IterativeCellSize = IterativeCellSize;

	if ((LoadingMode == ELoadingMode::IterativeCells) || (LoadingMode == ELoadingMode::IterativeCells2D))
	{
		// do partial loading loop that calls RunInternal
		const FWorldBuilderCellCoord MinCellCoords = FCellInfo::GetCellCoord(CellInfo.EditorBounds.Min, IterativeCellSize);
		const FWorldBuilderCellCoord NumCellsIterations = FCellInfo::GetCellCount(CellInfo.EditorBounds, IterativeCellSize);
		const FWorldBuilderCellCoord BeginCellCoords = MinCellCoords;
		const FWorldBuilderCellCoord EndCellCoords = BeginCellCoords + NumCellsIterations;

		auto CanIterateZ = [&BeginCellCoords, &EndCellCoords, LoadingMode](const bool bInResult, const int64 InZ) -> bool
		{
			if (LoadingMode == ELoadingMode::IterativeCells2D)
			{
				return bInResult && (InZ == BeginCellCoords.Z);
			}

			return bInResult && (InZ < EndCellCoords.Z);
		};

		const int64 IterationCount = ((LoadingMode == ELoadingMode::IterativeCells2D) ? 1 : NumCellsIterations.Z) * NumCellsIterations.Y * NumCellsIterations.X;
		int64 IterationIndex = 0;

		UE_LOGF(LogWorldPartitionBuilder, Display, "Iterative Cell Mode");
		UE_LOGF(LogWorldPartitionBuilder, Display, "Cell Size:       %d", IterativeCellSize);
		UE_LOGF(LogWorldPartitionBuilder, Display, "Cell Overlap:    %d", IterativeCellOverlapSize);
		UE_LOGF(LogWorldPartitionBuilder, Display, "WorldBounds:     Min %ls, Max %ls", *CellInfo.EditorBounds.Min.ToString(), *CellInfo.EditorBounds.Max.ToString());
		UE_LOGF(LogWorldPartitionBuilder, Display, "Iteration Count: %lld", IterationCount);
		
		TArray<TUniquePtr<IWorldPartitionActorLoaderInterface::ILoaderAdapter>> LoaderAdapters;

		for (int64 z = BeginCellCoords.Z; CanIterateZ(bResult, z); z++)
		{
			for (int64 y = BeginCellCoords.Y; bResult && (y < EndCellCoords.Y); y++)
			{
				for (int64 x = BeginCellCoords.X; bResult && (x < EndCellCoords.X); x++)
				{
					IterationIndex++;
					
					if (ShouldSkipCell(FWorldBuilderCellCoord(x, y, z)))
					{
						UE_LOGF(LogWorldPartitionBuilder, Display, "[%lld / %lld] Processing cells... (skipped)", IterationIndex, IterationCount);
						continue;
					}
					
					UE_LOGF(LogWorldPartitionBuilder, Display, "[%lld / %lld] Processing cells...", IterationIndex, IterationCount);

					double MinX = static_cast<double>(x * IterativeCellSize);
					double MinY = static_cast<double>(y * IterativeCellSize);
					double MinZ = static_cast<double>(z * IterativeCellSize);

					FVector Min(MinX, MinY, MinZ);
					FVector Max = Min + FVector(IterativeCellSize);

					if (LoadingMode == ELoadingMode::IterativeCells2D)
					{
						Min.Z = CellInfo.EditorBounds.Min.Z;
						Max.Z = CellInfo.EditorBounds.Max.Z;
					}

					FBox BoundsToLoad(Min, Max);
					BoundsToLoad = BoundsToLoad.ExpandBy(IterativeCellOverlapSize);

					CellInfo.Location = FWorldBuilderCellCoord(x, y, z);
					CellInfo.Bounds = BoundsToLoad;

					UE_LOGF(LogWorldPartitionBuilder, Verbose, "Loading Bounds: Min %ls, Max %ls", *BoundsToLoad.Min.ToString(), *BoundsToLoad.Max.ToString());

					LoaderAdapters.Emplace_GetRef(MakeUnique<FLoaderAdapterShape>(World, BoundsToLoad, TEXT("Loaded Region")))->Load();

					// Simulate an engine tick
					FWorldPartitionHelpers::FakeEngineTick(World);

					bResult = RunInternal(World, CellInfo, PackageHelper);

					if (FWorldPartitionHelpers::ShouldCollectGarbage())
					{
						LoaderAdapters.Empty();
						FWorldPartitionHelpers::DoCollectGarbage();
					}
				}
			}
		}
	}
	else if (LoadingMode == ELoadingMode::EntireWorld)
	{
		CellInfo.Bounds = BoxEntireWorld;

		TUniquePtr<FLoaderAdapterShape> LoaderAdapterShape = MakeUnique<FLoaderAdapterShape>(World, CellInfo.Bounds, TEXT("Loaded Region"));
		LoaderAdapterShape->Load();

		bResult = RunInternal(World, CellInfo, PackageHelper);

		LoaderAdapterShape.Reset();
	}
	else if (LoadingMode == ELoadingMode::Custom)
	{
		CellInfo.Bounds.Init();
		bResult = RunInternal(World, CellInfo, PackageHelper);
	}
	else
	{
		checkNoEntry();
	}

	return PostRun(World, PackageHelper, bResult);
}

bool UWorldPartitionBuilder::IsForceIncludedDataLayer(const UDataLayerInstance* InDataLayerInstance) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const FName DataLayerShortName(InDataLayerInstance->GetDataLayerShortName());
	return IncludedDataLayers.Contains(InDataLayerInstance->GetAsset()) || DataLayerShortNames.Contains(DataLayerShortName);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UWorldPartitionBuilder::IsForceExcludedDataLayer(const UDataLayerInstance* InDataLayerInstance) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const FName DataLayerShortName(InDataLayerInstance->GetDataLayerShortName());
	return ExcludedDataLayers.Contains(InDataLayerInstance->GetAsset()) || ExcludedDataLayerShortNames.Contains(DataLayerShortName);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UWorldPartitionBuilder::LoadDataLayers(UWorld* InWorld)
{
	if (UWorldPartition* WorldPartition = InWorld->GetWorldPartition())
	{
		// Properly Setup DataLayers for Builder
		UDataLayerManager* DataLayerManager = WorldPartition->GetDataLayerManager();
	
		// Load Data Layers
		bool bUpdateEditorCells = false;
		DataLayerManager->ForEachDataLayerInstance([&bUpdateEditorCells, this](UDataLayerInstance* DataLayer)
		{
			// Forced inclusion
			const bool bForceIncludeEditorDataLayer = bLoadNonDynamicDataLayers && !DataLayer->IsRuntime();
			const bool bForceIncludeInitiallyActivatedRuntimeDataLayer = bIncludeInitiallyActivatedRuntimeDataLayers && DataLayer->IsRuntime() && DataLayer->GetInitialRuntimeState() == EDataLayerRuntimeState::Activated;
			const bool bIsForceIncluded = IsForceIncludedDataLayer(DataLayer) || bForceIncludeEditorDataLayer || bForceIncludeInitiallyActivatedRuntimeDataLayer;

			// Forced exclusion
			const bool bForceExcludeNonInitiallyActivatedRuntimeDataLayer = (bExcludeNonInitiallyActivatedRuntimeDataLayers && DataLayer->IsRuntime()) ? DataLayer->GetInitialRuntimeState() != EDataLayerRuntimeState::Activated : false;
			const bool bIsForceExcluded = IsForceExcludedDataLayer(DataLayer) || bForceExcludeNonInitiallyActivatedRuntimeDataLayer;

			// Default state
			const bool bIsDefaultLoaded = DataLayer->IsLoadedInEditor();

			// Final state
			const bool bShouldBeLoaded = !bIsForceExcluded && (bIsForceIncluded || bIsDefaultLoaded);
			if (DataLayer->IsLoadedInEditor() != bShouldBeLoaded)
			{
				bUpdateEditorCells = true;
				DataLayer->SetIsLoadedInEditor(bShouldBeLoaded, /*bFromUserChange*/false);
				if (RequiresCommandletRendering() && bShouldBeLoaded)
				{
					DataLayer->SetIsInitiallyVisible(true);
				}
			}
			
			UE_LOGF(LogWorldPartitionBuilder, Display, "DataLayer '%ls' Loaded: %d", *UDataLayerInstance::GetDataLayerText(DataLayer).ToString(), bShouldBeLoaded ? 1 : 0);
			
			return true;
		});
	
		if (bUpdateEditorCells)
		{
			UE_LOGF(LogWorldPartitionBuilder, Display, "DataLayer load state changed refreshing editor cells");
			FDataLayersEditorBroadcast::StaticOnActorDataLayersEditorLoadingStateChanged(false);
		}
	}
}

bool UWorldPartitionBuilder::SavePackages(const TArray<UPackage*>& Packages, FPackageSourceControlHelper& PackageHelper, bool bErrorsAsWarnings)
{
	if (Packages.IsEmpty())
	{
		return true;
	}
	UE_LOGF(LogWorldPartitionBuilder, Display, "Saving %d packages...", Packages.Num());

	const TArray<FString> PackageFilenames = SourceControlHelpers::PackageFilenames(Packages);

	TArray<FString> PackagesToCheckout;
	TArray<FString> PackagesToAdd;
	TArray<FString> PackageNames;

	PackageNames.Reserve(Packages.Num());
	Algo::Transform(Packages, PackageNames, [](UPackage* InPackage) { return InPackage->GetName(); });

	// Display all affected files : that can be useful when analyzing what was saved (or at least what we tried to) by this commandlet : 
	TStringBuilder<0> PackageNamesString;
	Algo::ForEach(PackageNames, [&PackageNamesString](const FString& InPackageName) { PackageNamesString.Appendf(TEXT("\t%s\n"), *InPackageName); });
	UE_LOGF(LogWorldPartitionBuilder, Display, "%ls", PackageNamesString.ToString());

	bool bSuccess = PackageHelper.GetDesiredStatesForModification(PackageNames, PackagesToCheckout, PackagesToAdd, bErrorsAsWarnings);
	if (!bSuccess && !bErrorsAsWarnings)
	{
		return false;
	}
		
	if (PackagesToCheckout.Num())
	{
		if (!PackageHelper.Checkout(PackagesToCheckout, bErrorsAsWarnings))
		{
			bSuccess = false;
			if (!bErrorsAsWarnings)
			{
				return false;
			}
		}
	}

	// Load existing thumbnails to be able to resave them properly
	for (UPackage* PackageToSave : Packages)
	{
		EnsureLoadingComplete(PackageToSave);
	}

	ResetLoaders(TArray<UObject*>(Packages));

	for (int PackageIndex = 0; PackageIndex < Packages.Num(); ++PackageIndex)
	{
		const FString& PackageFilename = PackageFilenames[PackageIndex];
		// Check readonly flag in case some checkouts failed
		if (!IPlatformFile::GetPlatformPhysical().IsReadOnly(*PackageFilename))
		{
			// Save package
			FSavePackageArgs SaveArgs;
			SaveArgs.TopLevelFlags = RF_Standalone;
			if (!UPackage::SavePackage(Packages[PackageIndex], nullptr, *PackageFilenames[PackageIndex], SaveArgs))
			{
				bSuccess = false;
				FWorldPartitionBuilderLog::Error(bErrorsAsWarnings, FString::Printf(TEXT("Error saving package %s."), *Packages[PackageIndex]->GetName()));
				if(!bErrorsAsWarnings)
				{
					return false;
				}
			}
		}
		else
		{
			check(bErrorsAsWarnings);
		}
	}

	if (PackagesToAdd.Num())
	{
		if (!PackageHelper.AddToSourceControl(PackagesToAdd, bErrorsAsWarnings))
		{
			return false;
		}
	}

	return bSuccess;
}

bool UWorldPartitionBuilder::DeletePackages(const TArray<UPackage*>& Packages, FPackageSourceControlHelper& PackageHelper, bool bErrorsAsWarnings)
{
	TArray<FString> PackagesNames;
	Algo::Transform(Packages, PackagesNames, [](UPackage* InPackage) { return InPackage->GetName(); });
	return DeletePackages(PackagesNames, PackageHelper, bErrorsAsWarnings);
}

bool UWorldPartitionBuilder::DeletePackages(const TArray<FString>& PackageNames, FPackageSourceControlHelper& PackageHelper, bool bErrorsAsWarnings)
{
	if (PackageNames.Num() > 0)
	{
		UE_LOGF(LogWorldPartitionBuilder, Display, "Deleting %d packages...", PackageNames.Num());
		return PackageHelper.Delete(PackageNames, bErrorsAsWarnings);
	}

	return true;
}

bool UWorldPartitionBuilder::AutoSubmitPackages(const TArray<UPackage*>& InModifiedPackages, const FString& InChangelistDescription) const
{
	OnPackagesModified(InModifiedPackages, InChangelistDescription);
	return true;
}

bool UWorldPartitionBuilder::AutoSubmitFiles(const TArray<FString>& InModifiedFiles, const FString& InChangelistDescription) const
{
	OnFilesModified(InModifiedFiles, InChangelistDescription);
	return true;
}

bool UWorldPartitionBuilder::OnPackagesModified(const TArray<UPackage*>& InModifiedPackages, const FString& InChangelistDescription) const
{
	TArray<FString> ModifiedFiles;
	Algo::Transform(InModifiedPackages, ModifiedFiles, [](const UPackage* InPackage) { return USourceControlHelpers::PackageFilename(InPackage); });
	return OnFilesModified(ModifiedFiles, InChangelistDescription);
}

bool UWorldPartitionBuilder::OnFilesModified(const TArray<FString>& InModifiedFiles, const FString& InChangelistDescription) const
{
	if (ModifiedFilesHandler.IsBound())
	{
		return ModifiedFilesHandler.Execute(InModifiedFiles, InChangelistDescription);
	}

	return true;
}

void UWorldPartitionBuilder::SetModifiedFilesHandler(const UWorldPartitionBuilder::FModifiedFilesHandler& InModifiedFilesHandler)
{
	ModifiedFilesHandler = InModifiedFilesHandler;
}
