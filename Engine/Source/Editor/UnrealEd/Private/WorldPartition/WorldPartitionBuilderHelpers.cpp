// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionBuilderHelpers.h"

#include "CoreMinimal.h"
#include "HAL/FileManager.h"
#include "Logging/LogMacros.h"
#include "WorldPartition/WorldPartitionBuilder.h"
#include "Misc/FileHelper.h"
#include "Algo/ForEach.h"
#include "UObject/SavePackage.h"

#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "SourceControlOperations.h"

DEFINE_LOG_CATEGORY(LogWorldPartitionBuilderSourceControlHelper);

void FBuilderModifiedFiles::Add(EFileOperation FileOp, const FString& File)
{
	Files[FileOp].Add(File);
}

const TSet<FString>& FBuilderModifiedFiles::Get(EFileOperation FileOp) const
{
	return Files[FileOp];
}

void FBuilderModifiedFiles::Append(EFileOperation FileOp, const TArray<FString>& InFiles)
{
	Files[FileOp].Append(InFiles);
}

void FBuilderModifiedFiles::Append(const FBuilderModifiedFiles& Other)
{
	Files[EFileOperation::FileAdded].Append(Other.Files[EFileOperation::FileAdded]);
	Files[EFileOperation::FileEdited].Append(Other.Files[EFileOperation::FileEdited]);
	Files[EFileOperation::FileDeleted].Append(Other.Files[EFileOperation::FileDeleted]);
}

void FBuilderModifiedFiles::Empty()
{
	Files[EFileOperation::FileAdded].Empty();
	Files[EFileOperation::FileEdited].Empty();
	Files[EFileOperation::FileDeleted].Empty();
}

TArray<FString> FBuilderModifiedFiles::GetAllFiles() const
{
	TArray<FString> AllFiles;
	AllFiles.Append(Files[EFileOperation::FileAdded].Array());
	AllFiles.Append(Files[EFileOperation::FileEdited].Array());
	AllFiles.Append(Files[EFileOperation::FileDeleted].Array());
	return AllFiles;
}


FSourceControlHelper::FSourceControlHelper(FPackageSourceControlHelper& InPackageHelper, FBuilderModifiedFiles& InModifiedFiles)
	: PackageHelper(InPackageHelper)
	, ModifiedFiles(InModifiedFiles)
{}

FSourceControlHelper::~FSourceControlHelper()
{}

FString FSourceControlHelper::GetFilename(const FString& PackageName) const 
{
	return SourceControlHelpers::PackageFilename(PackageName);
}

FString FSourceControlHelper::GetFilename(UPackage* Package) const 
{
	return SourceControlHelpers::PackageFilename(Package);
}

bool FSourceControlHelper::Checkout(UPackage* Package) const 
{
	bool bCheckedOut = PackageHelper.Checkout(Package);
	if (bCheckedOut)
	{
		const FString Filename = GetFilename(Package);
		const bool bAdded = ModifiedFiles.Get(FBuilderModifiedFiles::EFileOperation::FileAdded).Contains(Filename);
		if (!bAdded)
		{
			ModifiedFiles.Add(FBuilderModifiedFiles::EFileOperation::FileEdited, Filename);
		}		
	}
	return bCheckedOut;
}

bool FSourceControlHelper::Checkout(const FString& Filename) const
{
	return Checkout(TArray<FString>({ Filename }));
}

bool FSourceControlHelper::Checkout(const TArray<FString>& Filenames) const
{
	bool bCheckedOut = PackageHelper.Checkout(Filenames);
	if (bCheckedOut)
	{
		for (const FString& Filename : Filenames)
		{
			const bool bAdded = ModifiedFiles.Get(FBuilderModifiedFiles::EFileOperation::FileAdded).Contains(Filename);
			if (!bAdded)
			{
				ModifiedFiles.Add(FBuilderModifiedFiles::EFileOperation::FileEdited, Filename);
			}
		}
	}
	return bCheckedOut;
}

bool FSourceControlHelper::Add(UPackage* Package) const 
{
	bool bAdded = PackageHelper.AddToSourceControl(Package);
	if (bAdded)
	{
		ModifiedFiles.Add(FBuilderModifiedFiles::EFileOperation::FileAdded, GetFilename(Package));
	}
	return bAdded;
}

bool FSourceControlHelper::Add(const FString& Filename) const
{
	return Add(TArray<FString>({ Filename }));
}

bool FSourceControlHelper::Add(const TArray<FString>& Filenames) const
{
	bool bAdded = PackageHelper.AddToSourceControl(Filenames);
	if (bAdded)
	{
		ModifiedFiles.Append(FBuilderModifiedFiles::EFileOperation::FileAdded, Filenames);
	}
	return bAdded;
}

bool FSourceControlHelper::Delete(const FString& PackageName) const 
{
	bool bDeleted = PackageHelper.Delete(PackageName);
	if (bDeleted)
	{
		ModifiedFiles.Add(FBuilderModifiedFiles::EFileOperation::FileDeleted, PackageName);
	}
	return bDeleted;
}

bool FSourceControlHelper::Delete(const TArray<FString>& PackageNames) const
{
	bool bDeleted = PackageHelper.Delete(PackageNames);
	if (bDeleted)
	{
		ModifiedFiles.Append(FBuilderModifiedFiles::EFileOperation::FileDeleted, PackageNames);
	}
	return bDeleted;
}

bool FSourceControlHelper::Delete(UPackage* Package) const 
{
	FString PackageName = GetFilename(Package);
	bool bDeleted = PackageHelper.Delete(Package);
	if (bDeleted)
	{
		ModifiedFiles.Add(FBuilderModifiedFiles::EFileOperation::FileDeleted, PackageName);
	}
	return bDeleted;
}

bool FSourceControlHelper::Delete(const TArray<UPackage*>& Packages) const
{
	TArray<FString> PackageNames;
	Algo::Transform(Packages, PackageNames, [this](UPackage* Package)
	{
		return GetFilename(Package);
	});
	
	bool bDeleted = PackageHelper.Delete(Packages);
	if (bDeleted)
	{
		ModifiedFiles.Append(FBuilderModifiedFiles::FileDeleted, PackageNames);
	}
	return bDeleted;
}

bool FSourceControlHelper::Save(UPackage* Package) const 
{
	return Save(TArray<UPackage*>({ Package }));
}

bool FSourceControlHelper::Save(const TArray<UPackage*>& Packages) const
{
	TArray<FString> PackagesToCheckout;
	TArray<FString> PackagesToAdd;

	for (UPackage* Package : Packages)
	{
		if (!Package)
		{
			return false;
		}

		FString PackageFileName = GetFilename(Package);
		if (IPlatformFile::GetPlatformPhysical().FileExists(*PackageFileName))
		{
			PackagesToCheckout.Add(PackageFileName);
		}
		else
		{
			PackagesToAdd.Add(PackageFileName);
		}
	}
	
	// Checkout packages
	if (PackagesToCheckout.Num() > 0)
	{
		if (!Checkout(PackagesToCheckout))
		{
			UE_LOGF(LogWorldPartitionBuilderSourceControlHelper, Error, "Error checking out packages");
			return false;
		}
	}

	// Save packages
	for (UPackage* Package : Packages)
	{
		Package->MarkAsFullyLoaded();

		FString PackageFileName = GetFilename(Package);
		FSavePackageArgs SaveArgs;
		SaveArgs.TopLevelFlags = RF_Standalone;
		SaveArgs.SaveFlags = PackageHelper.UseSourceControl() ? ESaveFlags::SAVE_None : ESaveFlags::SAVE_Async;
		if (!UPackage::SavePackage(Package, nullptr, *PackageFileName, SaveArgs))
		{
			UE_LOGF(LogWorldPartitionBuilderSourceControlHelper, Error, "Error saving package %ls.", *Package->GetName());
			return false;
		}
	}

	// Add new packages to source control
	if (PackagesToAdd.Num() > 0)
	{
		if (!Add(PackagesToAdd))
		{
			UE_LOGF(LogWorldPartitionBuilderSourceControlHelper, Error, "Error adding packages to revision control.");
			return false;
		}
	}

	return true;
}

bool FSourceControlHelper::Copy(const FString& SrcFilePath, const FString& DstFilePath) const
{
	IPlatformFile& PlatformFile = IPlatformFile::GetPlatformPhysical();

	// Check if the source file exists
	if (!PlatformFile.FileExists(*SrcFilePath))
	{
		UE_LOGF(LogWorldPartitionBuilderSourceControlHelper, Error, "Source file does not exist: %ls", *SrcFilePath);
		return false;
	}

	// Check if destination file already exists
	const bool bDstExists = PlatformFile.FileExists(*DstFilePath);

	// If the destination exists, check it out from source control
	if (bDstExists)
	{
		if (!Checkout(DstFilePath))
		{
			UE_LOGF(LogWorldPartitionBuilderSourceControlHelper, Error, "Error checking out destination file %ls.", *DstFilePath);
			return false;
		}
	}

	// Attempt to copy the file
	if (!PlatformFile.CopyFile(*DstFilePath, *SrcFilePath))
	{
		UE_LOGF(LogWorldPartitionBuilderSourceControlHelper, Error, "Error copying file from %ls to %ls.", *SrcFilePath, *DstFilePath);
		return false;
	}

	// If the destination file didn’t exist before, add it to source control
	if (!bDstExists)
	{
		if (!Add(DstFilePath))
		{
			UE_LOGF(LogWorldPartitionBuilderSourceControlHelper, Error, "Error adding file %ls to revision control.", *DstFilePath);
			return false;
		}
	}

	return true;
}
