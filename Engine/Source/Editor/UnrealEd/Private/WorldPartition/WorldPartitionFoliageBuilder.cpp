// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionFoliageBuilder.h"

#include "WorldPartition/ActorDescContainerInstance.h"
#include "WorldPartition/ActorPartition/PartitionActorDesc.h"
#include "WorldPartition/DataLayer/DataLayerEditorContext.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "PackageSourceControlHelper.h"
#include "SourceControlHelpers.h"
#include "HAL/PlatformFileManager.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "GameFramework/WorldSettings.h"
#include "InstancedFoliageActor.h"
#include "ActorPartition/ActorPartitionSubsystem.h"
#include "UObject/SavePackage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionFoliageBuilder)

DEFINE_LOG_CATEGORY_STATIC(LogWorldPartitionFoliageBuilder, All, All);

UWorldPartitionFoliageBuilder::UWorldPartitionFoliageBuilder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, NewGridSize(0)
	, bRepair(false)
{
	GetParamValue("NewGridSize=", NewGridSize);
	bRepair = HasParam("Repair");
}

bool UWorldPartitionFoliageBuilder::RunInternal(UWorld* World, const FCellInfo& InCellInfo, FPackageSourceControlHelper& PackageHelper)
{			
	UWorldPartition* WorldPartition = World->GetWorldPartition();
	if (!WorldPartition)
	{
		UE_LOGF(LogWorldPartitionFoliageBuilder, Error, "Failed to retrieve WorldPartition.");
		return false;
	}

	// Validate New Grid Size
	if (bRepair)
	{
		if (NewGridSize)
		{
			UE_LOGF(LogWorldPartitionFoliageBuilder, Error, "Cannot repair foliage actors while specifying NewGridSize.");
			return false;
		}
	}
	else if (World->GetWorldSettings()->InstancedFoliageGridSize == NewGridSize)
	{
		UE_LOGF(LogWorldPartitionFoliageBuilder, Display, "Foliage Grid Size already is %d.", NewGridSize);
		return true;
	}
	else if (NewGridSize == 0)
	{
		UE_LOGF(LogWorldPartitionFoliageBuilder, Error, "Foliage Grid Size is invalid %d. Please specify NewGridSize parameter.", NewGridSize);
		return false;
	}

	if (!bRepair)
	{
		if (WorldPartition->GetActorDescContainerCount() > 1)
		{ 
			// @todo_ow: This commandlet is currently unsupported for worldpartitions having multiple containers.
			// ForEachActorWithLoading below loops on all actors of all containers of a WorldPartition and saves new actors.
			// The new actors are saved within the main container WorldPartition container. Newly saved actors coming from different actor desc container should be saved in the right container. 
			UE_LOGF(LogWorldPartitionFoliageBuilder, Error, "FoliageBuilder Commandlet is unsupported on WorldPartition using more than 1 ExternalActor Folder.");
			return false;
		}

		UE_LOGF(LogWorldPartitionFoliageBuilder, Display, "Changing Foliage Grid Size from %d to %d", World->GetWorldSettings()->InstancedFoliageGridSize, NewGridSize);
	}
			
	// Update Setting
	World->GetWorldSettings()->Modify();

	World->GetWorldSettings()->InstancedFoliageGridSize = NewGridSize;
	World->GetWorldSettings()->bIncludeGridSizeInNameForFoliageActors = true;
		
	FPackageSourceControlHelper SCCHelper;
	TArray<FGuid> ExistingActors;
			
	// Snapshot existing ActorDescs
	FWorldPartitionHelpers::ForEachActorDescInstance(WorldPartition, AInstancedFoliageActor::StaticClass(), [&ExistingActors](const FWorldPartitionActorDescInstance* ActorDescInstance)
	{
		ExistingActors.Add(ActorDescInstance->GetGuid());
		return true;
	});

	TMap<UActorPartitionSubsystem::FCellCoord, FGuid> NewActors;
	UActorPartitionSubsystem* ActorPartitionSubsystem = World->GetSubsystem<UActorPartitionSubsystem>();
	int32 NumInstances = 0;
	int32 NumInstancesProcessed = 0;

	// This is the loop:
	// - Load 1 Existing IFA (OldGridSize)
	// - Copy all its instance data and compute its overall instance bounds
	// - Iterate over its bounds using the new grid size and create/reload new IFAs (NewGridSize).
	// - Add instance data to the new IFAs
	// - Save new/reload IFAs that were modified
	// - Unload the Existing IFA and all the new IFAs
	// - Repeat for next Existing IFA
	FWorldPartitionHelpers::FForEachActorWithLoadingParams ForEachActorWithLoadingParams;
	ForEachActorWithLoadingParams.ActorClasses = { AInstancedFoliageActor::StaticClass() };
	ForEachActorWithLoadingParams.ActorGuids.Append(ExistingActors);

	FWorldPartitionHelpers::ForEachActorWithLoading(WorldPartition, [this, World, WorldPartition, ActorPartitionSubsystem, &SCCHelper, &NewActors, &NumInstances, &NumInstancesProcessed](const FWorldPartitionActorDescInstance* ActorDescInstance)
	{
		TMap<UFoliageType*, TArray<FFoliageInstance>> FoliageToAdd;
		FBox InstanceBounds(ForceInit);

		const FPartitionActorDesc* PartitionActorDesc = (FPartitionActorDesc*)ActorDescInstance->GetActorDesc();
		AInstancedFoliageActor* IFA = Cast<AInstancedFoliageActor>(ActorDescInstance->GetActor());
		if (!IFA)
		{
			UE_LOGF(LogWorldPartitionFoliageBuilder, Error, "Foliage actor failed to load: %ls (%ls)", *PartitionActorDesc->GetActorNameString(), *PartitionActorDesc->GetActorPackage().ToString());
			return false;
		}

		if (bRepair)
		{
			FPackageSourceControlHelper PackageHelper;

			const FDataLayerEditorContext DataLayerEditorContext(World, ActorDescInstance->GetDataLayerInstanceNames().ToArray());
			const FActorPartitionIdentifier ActorPartitionIdentifier(PartitionActorDesc->GetActorNativeClass(), IFA->GetGridGuid(), DataLayerEditorContext.GetHash());
			const bool bShouldIncludeGridSizeInName = ActorPartitionIdentifier.GetClass()->GetDefaultObject<APartitionActor>()->ShouldIncludeGridSizeInName(World, ActorPartitionIdentifier);
			const FString ExpectedActorName = APartitionActor::GetActorName(World, ActorPartitionIdentifier, PartitionActorDesc->GridSize, 
				PartitionActorDesc->GridIndexX, PartitionActorDesc->GridIndexY, PartitionActorDesc->GridIndexZ);

			if (PartitionActorDesc->GetActorName() != ExpectedActorName)
			{
				UPackage* PackageToModify = nullptr;
				UPackage* PackageToDelete = nullptr;
				UPackage* PackageToAdd = nullptr;

				UE_LOGF(LogWorldPartitionFoliageBuilder, Display, "Repairing invalid foliage actor: %ls->%ls ", *PartitionActorDesc->GetActorNameString(), *ExpectedActorName);

				const FString ExpectedActorPath = FString::Printf(TEXT("%s.%s"), *World->PersistentLevel->GetPathName(), *ExpectedActorName);
				
				if (const FWorldPartitionActorDescInstance* DupActorDescInstance = WorldPartition->GetActorDescInstanceByPath(ExpectedActorPath))
				{
					// Merge with existing
					FWorldPartitionReference DupActorRef(WorldPartition, DupActorDescInstance->GetGuid());

					AInstancedFoliageActor* DupIFA = Cast<AInstancedFoliageActor>(DupActorDescInstance->GetActor());
					if (!DupIFA)
					{
						UE_LOGF(LogWorldPartitionFoliageBuilder, Error, "Foliage actor failed to load: %ls (%ls)", *DupActorDescInstance->GetActorNameString(), *DupActorDescInstance->GetActorPackage().ToString());
						return false;
					}

					PackageToDelete = IFA->GetPackage();
					IFA->ForEachFoliageInfo([DupIFA](UFoliageType* FoliageType, FFoliageInfo& FoliageInfo)
					{
						FFoliageInfo* NewFoliageInfo = nullptr;
						UFoliageType* NewFoliageType = DupIFA->AddFoliageType(FoliageType, &NewFoliageInfo);

						for (const FFoliageInstance& Instance : FoliageInfo.Instances)
						{
							NewFoliageInfo->AddInstance(NewFoliageType, Instance);
						}

						return true;
					});
					PackageToModify = DupIFA->GetPackage();
				}
				else
				{
					// Rename existing
					PackageToDelete = IFA->GetPackage();
					IFA->SetPackageExternal(false);
					IFA->Rename(*ExpectedActorName, nullptr);
					IFA->SetActorLabel(*ExpectedActorName);
					IFA->SetPackageExternal(true);
					PackageToAdd = IFA->GetPackage();
				}

				// Delete
				if (PackageToDelete && !PackageHelper.Delete(PackageToDelete))
				{
					return false;
				}

				// Checkout
				if (PackageToModify)
				{
					FString PackageFileName = SourceControlHelpers::PackageFilename(PackageToModify);
					if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*PackageFileName))
					{
						if (!PackageHelper.Checkout(PackageToModify))
						{
							return false;
						}
					}
				}

				// Save
				auto SavePackage = [](UPackage* PackageToSave)
				{
					FString PackageFileName = SourceControlHelpers::PackageFilename(PackageToSave);
					FSavePackageArgs SaveArgs;
					SaveArgs.TopLevelFlags = RF_Standalone;
					return UPackage::SavePackage(PackageToSave, nullptr, *PackageFileName, SaveArgs);
				};

				if (PackageToModify && !SavePackage(PackageToModify))
				{
					return false;
				}

				if (PackageToAdd && !SavePackage(PackageToAdd))
				{
					return false;
				}

				// Add
				if (PackageToAdd && !PackageHelper.AddToSourceControl(PackageToAdd))
				{
					return false;
				}
			}
		}
		else
		{	
			check(IFA->GetGridSize() != NewGridSize);
			// Harvest Instances from existing IFA and build up instance bounds
			UE_LOGF(LogWorldPartitionFoliageBuilder, Display, "Processing existing foliage actor: %ls (%ls)", *PartitionActorDesc->GetActorNameString(), *PartitionActorDesc->GetActorPackage().ToString());
		
			IFA->ForEachFoliageInfo([IFA, &FoliageToAdd, &InstanceBounds, &NumInstances](UFoliageType* FoliageType, FFoliageInfo& FoliageInfo)
			{
				check(FoliageType->GetTypedOuter<AInstancedFoliageActor>() == nullptr);
				FoliageToAdd.FindOrAdd(FoliageType).Append(FoliageInfo.Instances);
				for (const FFoliageInstance& Instance : FoliageInfo.Instances)
				{
					InstanceBounds += Instance.GetInstanceWorldTransform().GetLocation();
				}

				if(FoliageInfo.Instances.Num() > 0)
				{
					UE_LOGF(LogWorldPartitionFoliageBuilder, Display, "    FoliageType: %ls Count: %d", *FoliageType->GetName(), FoliageInfo.Instances.Num());
				}
				NumInstances += FoliageInfo.Instances.Num();

				return true;
			});

			TMap<UActorPartitionSubsystem::FCellCoord, AInstancedFoliageActor*> IntersectingActors;

			// Maintain a list of loaded 
			TSet<FWorldPartitionReference> ActorReferences;

			// Create/Reload new actors
			FActorPartitionGridHelper::ForEachIntersectingCell(AInstancedFoliageActor::StaticClass(), InstanceBounds, World->PersistentLevel, [&](const UActorPartitionSubsystem::FCellCoord& InCellCoord, const FBox& InCellBounds)
			{
				// Create or Get (might have already been created and not GCed)
				if (AInstancedFoliageActor* NewOrExistingActor = Cast<AInstancedFoliageActor>(ActorPartitionSubsystem->GetActor(AInstancedFoliageActor::StaticClass(), InCellCoord, /*bInCreate=*/true)))
				{
					IntersectingActors.Add(InCellCoord, NewOrExistingActor);
				} 
				else // If we haven't created or found the actor it means it was created/saved/unloaded and should be in the NewActorDescs so we can reload it
				{
					const FGuid& NewActorGuid = NewActors.FindChecked(InCellCoord);
					FWorldPartitionReference Reference(WorldPartition, NewActorGuid);
					ActorReferences.Add(Reference);
					IntersectingActors.Add(InCellCoord, CastChecked<AInstancedFoliageActor>(Reference.GetActor()));
				}

				return true;
			});
                        
			// Maintain list of modified IFAs so we don't save empty intersecting actors
			TSet<AInstancedFoliageActor*> ModifiedIFAs;
            
			// Add Foliage to those actors
			for (auto& InstancesPerFoliageType : FoliageToAdd)
			{
				for (const FFoliageInstance& Instance : InstancesPerFoliageType.Value)
				{
					// Here we should get back one of the IntersectingActors that was just created or reloaded so that is why we pass in false to bCreateIfNone
					AInstancedFoliageActor* ModifiedIFA = AInstancedFoliageActor::Get(World, /*bCreateIfNone=*/false, World->PersistentLevel, Instance.Location);
					check(ModifiedIFA);
					ModifiedIFAs.Add(ModifiedIFA);
					check(ModifiedIFA->GetGridSize() == NewGridSize);
					FFoliageInfo* NewFoliageInfo = nullptr;
					UFoliageType* NewFoliageType = ModifiedIFA->AddFoliageType(InstancesPerFoliageType.Key, &NewFoliageInfo);
					NewFoliageInfo->AddInstance(NewFoliageType, Instance);
					NumInstancesProcessed++;
				}
			}

			// Save only modified ones
			for (auto Pair : IntersectingActors)
			{
				AInstancedFoliageActor* IntersectingActor = Pair.Value;
				// No need to save actor as it hasn't changed
				if (!ModifiedIFAs.Contains(IntersectingActor))
				{
					continue;
				}

				UPackage* Package = IntersectingActor->GetExternalPackage();
				check(Package);

				FString PackageFileName = SourceControlHelpers::PackageFilename(Package);
                                
				// Save package
				FSavePackageArgs SaveArgs;
				SaveArgs.TopLevelFlags = RF_Standalone;
				if (!UPackage::SavePackage(Package, nullptr, *PackageFileName, SaveArgs))
				{
					UE_LOGF(LogWorldPartitionFoliageBuilder, Error, "Error saving package %ls.", *Package->GetName());
					return false;
				}

				// Actor Desc didn't exist yet which means we need to add the file to SCC and add it to the new actors descs and reference it so it gets unloaded
				if (!NewActors.Contains(Pair.Key))
				{
					if (!SCCHelper.AddToSourceControl(Package))
					{
						// It is possible the resave can't checkout everything. Continue processing.
						UE_LOGF(LogWorldPartitionFoliageBuilder, Error, "Error adding package to revision control %ls.", *Package->GetName());
						return false;
					}

					FGuid NewActorGuid = IntersectingActor->GetActorGuid();
					FWorldPartitionActorDescInstance* NewActorDescInstance = WorldPartition->GetActorDescInstance(NewActorGuid);
					check(NewActorDescInstance);
					NewActors.Add(Pair.Key, NewActorGuid);

					// Add ref so actor gets unloaded 
					FWorldPartitionReference Reference(WorldPartition, NewActorGuid);
					ActorReferences.Add(Reference);
				}
                
				UE_LOGF(LogWorldPartitionFoliageBuilder, Display, "Saved foliage actor %ls (%ls).", *IntersectingActor->GetName(), *Package->GetName());
			}
		}
					
		return true;
	}, ForEachActorWithLoadingParams);

	if (!bRepair)
	{
		check(NumInstances == NumInstancesProcessed);

		// Checkout World for WorldSettings change
		UPackage* WorldSettingsPackage = World->GetWorldSettings()->GetPackage();
		if (!SCCHelper.Checkout(WorldSettingsPackage))
		{
			UE_LOGF(LogWorldPartitionFoliageBuilder, Error, "Error checking out package %ls.", *WorldSettingsPackage->GetName());
			return false;
		}

		// Save World 
		{
			FString PackageFileName = SourceControlHelpers::PackageFilename(WorldSettingsPackage);
			FSavePackageArgs SaveArgs;
			SaveArgs.TopLevelFlags = RF_Standalone;
			if (!UPackage::SavePackage(World->GetPackage(), nullptr, *PackageFileName, SaveArgs))
			{
				UE_LOGF(LogWorldPartitionFoliageBuilder, Error, "Error saving package %ls.", *WorldSettingsPackage->GetName());
				return false;
			}
		}

		// Delete old IFAs
		for (const FGuid& ActorGuid : ExistingActors)
		{
			FWorldPartitionActorDescInstance* ActorDescInstance = WorldPartition->GetActorDescInstance(ActorGuid);
			check(ActorDescInstance);
			const FString PackageToDelete = ActorDescInstance->GetActorPackage().ToString();
			if (!SCCHelper.Delete(PackageToDelete))
			{
				UE_LOGF(LogWorldPartitionFoliageBuilder, Error, "Error deleting package %ls.", *PackageToDelete);
				return false;
			}
		}
	}

	return true;
}
