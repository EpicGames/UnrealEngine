// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionHLODsBuilder.h"

#include "CoreMinimal.h"
#include "DiffUtils.h"
#include "HAL/FileManager.h"
#include "Logging/LogMacros.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/EngineVersion.h"
#include "Misc/FileHelper.h"
#include "Misc/MessageDialog.h"
#include "Misc/OutputDeviceNull.h"
#include "Misc/BufferedOutputDevice.h"
#include "Misc/ScopedSlowTask.h"
#include "Algo/ForEach.h"
#include "UObject/Linker.h"
#include "UObject/GCObjectScopeGuard.h"
#include "UObject/SavePackage.h"

#include "ActorFolder.h"
#include "EditorWorldUtils.h"
#include "Engine/Engine.h"
#include "Engine/LevelStreamingGCHelper.h"
#include "EngineUtils.h"
#include "SourceControlHelpers.h"
#include "SourceControlOperations.h"
#include "ISourceControlModule.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "DerivedDataCacheInterface.h"
#include "DirectoryWatcherModule.h"
#include "IDirectoryWatcher.h"
#include "ProfilingDebugging/ScopedTimers.h"

#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/DataLayer/ExternalDataLayerEngineSubsystem.h"
#include "WorldPartition/ActorDescContainerInstance.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/HLOD/HLODActor.h"
#include "WorldPartition/HLOD/HLODActorDesc.h"
#include "WorldPartition/HLOD/HLODCreationFilter.h"
#include "WorldPartition/HLOD/HLODSourceActors.h"
#include "WorldPartition/HLOD/HLODLayer.h"
#include "WorldPartition/HLOD/HLODProviderInterface.h"
#include "WorldPartition/HLOD/HLODSourceActorsFromCell.h"
#include "WorldPartition/HLOD/IWorldPartitionHLODUtilities.h"
#include "WorldPartition/HLOD/IWorldPartitionHLODUtilitiesModule.h"
#include "WorldPartition/HLOD/StandaloneHLODSubsystem.h"
#include "WorldPartition/IWorldPartitionEditorModule.h"
#include "WorldPartition/WorldPartitionRuntimeHash.h"

#include "NavSystemConfigOverride.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionHLODsBuilder)

#define LOCTEXT_NAMESPACE "WorldPartitionHLODsBuilder"

DEFINE_LOG_CATEGORY_STATIC(LogWorldPartitionHLODsBuilder, Log, All);

static const FString DistributedBuildWorkingDirName = TEXT("HLODTemp");
static const FString DistributedBuildManifestName = TEXT("HLODBuildManifest.ini");
static const FString BuildProductsFileName = TEXT("BuildProducts.txt");

FString GetHLODBuilderFolderName(uint32 BuilderIndex) { return FString::Printf(TEXT("HLODBuilder%d"), BuilderIndex); }
FString GetToSubmitFolderName() { return TEXT("ToSubmit"); }
FString GetDistributedBuildBaseDir() { return FPaths::ProjectIntermediateDir() / DistributedBuildWorkingDirName / TEXT(""); }

void CacheUnsavedHLODActors(UWorld* InWorld, TMap<FGuid, TWeakObjectPtr<AWorldPartitionHLOD> >& OutUnsavedHLODActors)
{
	if (InWorld)
	{
		for (TActorIterator<AWorldPartitionHLOD> It(InWorld); It; ++It)
		{
			if (UPackage* Package = It->GetPackage())
			{
				if (Package->IsDirty())
				{
					OutUnsavedHLODActors.Add(It->GetActorGuid(), *It);
				}
			}
		}
	}
}

namespace BuildEvaluationStats
{
	struct FStats
	{
		int32 Evaluated = 0;
		int32 ReusedFromParentBranch = 0;
		int32 NotFoundInParentBranch = 0;
		int32 Built = 0;
		int64 DataSyncedBytes = 0;
		double TimeEvaluate = 0;
		double TimeSyncParent = 0;
		double TimeLoadParent = 0;

		void Print(const FString& StatsName)
		{
			if (Evaluated > 0)
			{
				UE_LOGF(LogWorldPartitionHLODsBuilder, Display, "##### BuildEvaluationStats: %ls ####", *StatsName);
				UE_LOGF(LogWorldPartitionHLODsBuilder, Display, " * Evaluated = %d", Evaluated);
				UE_LOGF(LogWorldPartitionHLODsBuilder, Display, " * ReusedFromParentBranch = %d", ReusedFromParentBranch);
				UE_LOGF(LogWorldPartitionHLODsBuilder, Display, " * NotFoundInParentBranch = %d", NotFoundInParentBranch);
				UE_LOGF(LogWorldPartitionHLODsBuilder, Display, " * Built = %d", Built);
				UE_LOGF(LogWorldPartitionHLODsBuilder, Display, " * DataSyncedBytes = %lld", DataSyncedBytes);
				UE_LOGF(LogWorldPartitionHLODsBuilder, Display, " * TimeEvaluate = %f", TimeEvaluate);
				UE_LOGF(LogWorldPartitionHLODsBuilder, Display, " * TimeSyncParent = %f", TimeSyncParent);
				UE_LOGF(LogWorldPartitionHLODsBuilder, Display, " * TimeLoadParent = %f", TimeLoadParent);
				UE_LOGF(LogWorldPartitionHLODsBuilder, Display, "###############################");
			}
		}

		FStats& operator+=(const FStats& Other)
		{
			Evaluated += Other.Evaluated;
			ReusedFromParentBranch += Other.ReusedFromParentBranch;
			NotFoundInParentBranch += Other.NotFoundInParentBranch;
			Built += Other.Built;
			DataSyncedBytes += Other.DataSyncedBytes;
			TimeEvaluate += Other.TimeEvaluate;
			TimeSyncParent += Other.TimeSyncParent;
			TimeLoadParent += Other.TimeLoadParent;
			return *this;
		}
	};

	static TMap<FName, FStats> StatsByHLODLayer;

	void PrintStats()
	{
		FStats StatsGlobal;

		for (auto [LayerName, Stats] : StatsByHLODLayer)
		{
			StatsGlobal += Stats;
			Stats.Print(LayerName.ToString());
		}

		StatsGlobal.Print("Global");
	}
}

UWorldPartitionHLODsBuilder::FOnWorldPartitionHLODBuildWithFiltersCompleted UWorldPartitionHLODsBuilder::OnWorldPartitionHLODBuildWithFiltersCompleted;

UWorldPartitionHLODsBuilder::UWorldPartitionHLODsBuilder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, BuilderIdx(INDEX_NONE)
	, BuilderCount(INDEX_NONE)
	, bBuildingStandaloneHLOD(false)
	, bUseSlowTask(false)
	, bSaveActors(true)
	, bConsiderUnsavedHLODActors(false)
{
	if (!IsTemplate())
	{
		BuildOptions = HasParam("SetupHLODs") ? EHLODBuildStep::HLOD_Setup : EHLODBuildStep::None;
		BuildOptions |= HasParam("BuildHLODs") ? EHLODBuildStep::HLOD_Build : EHLODBuildStep::None;
		BuildOptions |= HasParam("RebuildHLODs") ? EHLODBuildStep::HLOD_Build : EHLODBuildStep::None;
		BuildOptions |= HasParam("DeleteHLODs") ? EHLODBuildStep::HLOD_Delete : EHLODBuildStep::None;
		BuildOptions |= HasParam("FinalizeHLODs") ? EHLODBuildStep::HLOD_Finalize : EHLODBuildStep::None;
		BuildOptions |= HasParam("DumpStats") ? EHLODBuildStep::HLOD_Stats : EHLODBuildStep::None;

		bResumeBuild = GetParamValue("ResumeBuild=", ResumeBuildIndex);

		bDistributedBuild = HasParam("DistributedBuild");
		bForceBuild = HasParam("RebuildHLODs");
		bReportOnly = HasParam("ReportOnly");
		bReuseParentBranchHLODs = HasParam("ReuseParentBranchHLODs");

		GetParamValue("BuildManifest=", BuildManifest);
		GetParamValue("BuilderIdx=", BuilderIdx);
		GetParamValue("BuilderCount=", BuilderCount);
		GetParamValue("BuildHLODLayer=", HLODLayerToBuild);
		GetParamValue("BuildSingleHLOD=", HLODActorToBuild);

		if (!HLODActorToBuild.IsNone() || !HLODLayerToBuild.IsNone())
		{
			BuildOptions |= EHLODBuildStep::HLOD_Build;
			bForceBuild = bForceBuild || !HLODActorToBuild.IsNone();
		}

		// Default behavior without any option is to setup + build
		if (BuildOptions == EHLODBuildStep::None)
		{
			BuildOptions = EHLODBuildStep::HLOD_Setup | EHLODBuildStep::HLOD_Build;
		}

		UExternalDataLayerEngineSubsystem::Get().OnExternalDataLayerOverrideInjection.AddUObject(this, &UWorldPartitionHLODsBuilder::AllowExternalDataLayerInjection);
	}
}

bool UWorldPartitionHLODsBuilder::BuildWithFilters(UWorld* InWorld, const TArray<TSharedPtr<IHLODCreationFilter>>& InFilters)
{
	bool bResult = false;

	// Make sure that a newly created world was saved
	if (InWorld->GetPackage()->HasAnyPackageFlags(PKG_NewlyCreated))
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("NewMap", "New world must be saved before performing this operation."));
		return false;
	}

	bool bUnsavedActorsPassingFilters = false;
	for (TActorIterator<AActor> It(InWorld); It; ++It)
	{
		AActor* Actor = *It;
		if (Actor && !Actor->IsA<AWorldPartitionHLOD>() && Actor->GetPackage()->IsDirty())
		{
			FBox RuntimeBounds;
			FBox EditorBounds;
			Actor->GetStreamingBounds(RuntimeBounds, EditorBounds);

			FHLODCreationFilterContext FilterContext;
			FilterContext.Bounds = RuntimeBounds;
			if (UE::HLOD::CreationFilter::PassesFilters(InFilters, FilterContext))
			{
				bUnsavedActorsPassingFilters = true;
				break;
			}
		}
	}

	if (bUnsavedActorsPassingFilters)
	{
		EAppReturnType::Type Response = FMessageDialog::Open(EAppMsgType::YesNo,
			LOCTEXT("HLODBuildUnsavedActorsMessage", "There are unsaved actors that won't be included in this HLOD build. Do you want to continue?"));

		if (Response == EAppReturnType::No || Response == EAppReturnType::Cancel)
		{
			return false;
		}
	}

	UWorldPartitionHLODsBuilder* Builder = NewObject<UWorldPartitionHLODsBuilder>(GetTransientPackage(), UWorldPartitionHLODsBuilder::StaticClass());
	Builder->SetUseSlowTask(true);
	Builder->SetSaveActors(false);
	Builder->SetConsiderUnsavedHLODActors(true);
	Builder->SetFilters(InFilters);

	{
		FGCObjectScopeGuard BuilderGuard(Builder);
		bResult = Builder->RunBuilder(InWorld);
	}

	GEditor->RedrawLevelEditingViewports();

	GetOnWorldPartitionHLODBuildWithFiltersCompleted().Broadcast();

	return bResult;
}

void UWorldPartitionHLODsBuilder::AllowExternalDataLayerInjection(const UWorld* InWorld, const UExternalDataLayerAsset* InExternalDataLayerAsset, bool& bOutAllowInjection)
{
	// Always allow EDL injections during HLOD builds
	bOutAllowInjection = true;
}

bool UWorldPartitionHLODsBuilder::RequiresCommandletRendering() const
{
	// Commandlet requires rendering only for building HLODs
	// Building will occur either if -BuildHLODs is provided or no explicit step arguments are provided
	return EnumHasAnyFlags(BuildOptions, EHLODBuildStep::HLOD_Build);
}

bool UWorldPartitionHLODsBuilder::ShouldRunStep(const EHLODBuildStep BuildStep) const
{
	return (BuildOptions & BuildStep) == BuildStep;
}

bool UWorldPartitionHLODsBuilder::ValidateParams() const
{
	if (ShouldRunStep(EHLODBuildStep::HLOD_Setup) && IsUsingBuildManifest())
	{
		if (BuilderCount <= 0)
		{
			UE_LOGF(LogWorldPartitionHLODsBuilder, Error, "Missing parameter -BuilderCount=N (where N > 0), exiting...");
			return false;
		}
	}

	if (ShouldRunStep(EHLODBuildStep::HLOD_Build) && IsUsingBuildManifest())
	{
		if (BuilderIdx < 0)
		{
			UE_LOGF(LogWorldPartitionHLODsBuilder, Error, "Missing parameter -BuilderIdx=i, exiting...");
			return false;
		}

		if (!FPaths::FileExists(BuildManifest))
		{
			UE_LOGF(LogWorldPartitionHLODsBuilder, Error, "Build manifest file \"%ls\" not found, exiting...", *BuildManifest);
			return false;
		}

		FString CurrentEngineVersion = FEngineVersion::Current().ToString();
		FString ManifestEngineVersion = TEXT("unknown");

		FConfigFile ConfigFile;
		ConfigFile.Read(BuildManifest);
		ConfigFile.GetString(TEXT("General"), TEXT("EngineVersion"), ManifestEngineVersion);
		if (ManifestEngineVersion != CurrentEngineVersion)
		{
			UE_LOGF(LogWorldPartitionHLODsBuilder, Error, "Build manifest engine version doesn't match current engine version (%ls vs %ls), exiting...", *ManifestEngineVersion, *CurrentEngineVersion);
			return false;
		}
	}

	return true;
}

FString GetDistributedBuildWorkingDir(UWorld* InWorld)
{
	uint32 WorldPackageHash = GetTypeHash(InWorld->GetPackage()->GetFullName());
	return FString::Printf(TEXT("%s%08x"), *GetDistributedBuildBaseDir(), WorldPackageHash);
}

bool UWorldPartitionHLODsBuilder::ShouldProcessWorld(UWorld* InWorld) const
{
	bool bShouldProcessWorld = true;

	// When building HLODs in a distributed build, if there is no config section for the given builder index
	// it means that the builder can skip processing this world altogether.
	if (bDistributedBuild && ShouldRunStep(EHLODBuildStep::HLOD_Build))
	{
		const FString BuildManifestDirName = GetDistributedBuildWorkingDir(InWorld);
		const FString BuildManifestFileName = BuildManifestDirName / DistributedBuildManifestName;

		FConfigFile ConfigFile;
		ConfigFile.Read(BuildManifestFileName);

		FString SectionName = GetHLODBuilderFolderName(BuilderIdx);

		const FConfigSection* ConfigSection = ConfigFile.FindSection(SectionName);
		if (!ConfigSection || ConfigSection->IsEmpty())
		{
			bShouldProcessWorld = false;
		}
	}

	return bShouldProcessWorld;
}

bool UWorldPartitionHLODsBuilder::ShouldProcessAdditionalWorlds(UWorld* InWorld, TArray<FString>& OutPackageNames) const
{
	// If during Build step and if building standalone HLOD, we want to run the builder on standalone HLOD levels,
	// so that the HLOD Actors, which were created in those levels can be built
	if (ShouldRunStep(EHLODBuildStep::HLOD_Build) == false)
	{
		return false;
	}

	UWorldPartition* WP = InWorld->GetWorldPartition();
	if (WP && WP->HasStandaloneHLOD() && WP->RuntimeHash)
	{
		FString FolderPath, PackagePrefix;
		UWorldPartitionStandaloneHLODSubsystem::GetStandaloneHLODFolderPathAndPackagePrefix(InWorld->GetPackage()->GetName(), FolderPath, PackagePrefix);

		int32 HLODDepth = WP->RuntimeHash->ComputeHLODHierarchyDepth();

		for (int32 HLODSetupIndex = 0; HLODSetupIndex < HLODDepth; HLODSetupIndex++)
		{
			const FString HLODLevelPackageName = FString::Printf(TEXT("%s/%s%d"), *FolderPath, *PackagePrefix, HLODSetupIndex);
			OutPackageNames.Add(HLODLevelPackageName);
		}

		return true;
	}
	return false;
}

bool UWorldPartitionHLODsBuilder::PreWorldInitialization(UWorld* InWorld, FPackageSourceControlHelper& PackageHelper)
{
	if (bDistributedBuild)
	{
		DistributedBuildWorkingDir = GetDistributedBuildWorkingDir(InWorld);
		DistributedBuildManifest = DistributedBuildWorkingDir / DistributedBuildManifestName;

		if (!BuildManifest.IsEmpty())
		{
			UE_LOGF(LogWorldPartitionHLODsBuilder, Warning, "Ignoring parameter -BuildManifest when a distributed build is performed");
		}

		BuildManifest = DistributedBuildManifest;
	}

	if (!ValidateParams())
	{
		return false;
	}

	bool bRet = true;

	// When running a distributed build, retrieve relevant build products from the previous steps
	if (IsDistributedBuild() && (ShouldRunStep(EHLODBuildStep::HLOD_Build) || ShouldRunStep(EHLODBuildStep::HLOD_Finalize)))
	{
		FString WorkingDirFolder = ShouldRunStep(EHLODBuildStep::HLOD_Build) ? GetHLODBuilderFolderName(BuilderIdx) : GetToSubmitFolderName();
		bRet = CopyFilesFromWorkingDir(WorkingDirFolder);
	}

	return bRet;
}

bool UWorldPartitionHLODsBuilder::RunInternal(UWorld* InWorld, const FCellInfo& InCellInfo, FPackageSourceControlHelper& PackageHelper)
{
	World = InWorld;
	WorldPartition = World->GetWorldPartition();

	FScopedSlowTask SlowTask(2.f, LOCTEXT("BuildingHLODs", "Building HLODs..."), bUseSlowTask);
	SlowTask.MakeDialog(true);

	if (WorldPartition != nullptr)
	{
		bBuildingStandaloneHLOD = WorldPartition->HasStandaloneHLOD();
	}
	
	if (IsRunningCommandlet())
	{
		// Allows HLOD Streaming levels to be GCed properly
		FLevelStreamingGCHelper::EnableForCommandlet();
	}

	SourceControlHelper = new FSourceControlHelper(PackageHelper, ModifiedFiles);

	bool bRet = true;

	if (bRet && ShouldRunStep(EHLODBuildStep::HLOD_Setup))
	{
		SlowTask.EnterProgressFrame(1.f, LOCTEXT("SetupHLODActors", "Setup HLOD Actors"));
		bRet = SetupHLODActors();
	}

	if (!bReportOnly)
	{
		if (bRet && ShouldRunStep(EHLODBuildStep::HLOD_Build))
		{
			SlowTask.EnterProgressFrame(1.f, LOCTEXT("BuildHLODActors", "Build HLOD Actors"));
			bRet = BuildHLODActors();
		}

		if (bRet && ShouldRunStep(EHLODBuildStep::HLOD_Delete))
		{
			bRet = DeleteHLODActors();
		}

		if (bRet && ShouldRunStep(EHLODBuildStep::HLOD_Finalize))
		{
			bRet = SubmitHLODActors();
		}

		if (bRet && ShouldRunStep(EHLODBuildStep::HLOD_Stats))
		{
			bRet = DumpStats();
		}
	}

	WorldPartition = nullptr;
	delete SourceControlHelper;

	return bRet;
}

bool UWorldPartitionHLODsBuilder::SetupHLODActors()
{
	// No setup needed for non partitioned worlds and standalone HLOD worlds
	if (WorldPartition && !WorldPartition->IsStandaloneHLODWorld())
	{
		auto ActorFolderAddedDelegateHandle = GEngine->OnActorFolderAdded().AddLambda([this](UActorFolder* InActorFolder)
		{
			UPackage* ActorFolderPackage = InActorFolder->GetPackage();
			const bool bIsTempPackage = FPackageName::IsTempPackage(ActorFolderPackage->GetName());
			if (!bIsTempPackage && InActorFolder->IsInitiallyExpanded())
			{
				// We don't want the HLOD folders to be expanded by default
				InActorFolder->SetIsInitiallyExpanded(false);
				SourceControlHelper->Save(InActorFolder->GetPackage());
			}
		});
	
		ON_SCOPE_EXIT
		{
			GEngine->OnActorFolderAdded().Remove(ActorFolderAddedDelegateHandle);
		};

		UWorldPartition::FSetupHLODActorsParams SetupHLODActorsParams = UWorldPartition::FSetupHLODActorsParams()
			.SetSourceControlHelper(SourceControlHelper)
			.SetReportOnly(bReportOnly)
			.SetConsiderUnsavedHLODActors(bConsiderUnsavedHLODActors)
			.SetSaveActors(bSaveActors)
			.SetFilters(Filters);

		WorldPartition->SetupHLODActors(SetupHLODActorsParams);

		if (bBuildingStandaloneHLOD)
		{
			// Retrieve additional Standalone HLOD levels that have to be processed
			AdditionalWorldPartitionsForStandaloneHLOD = MoveTemp(SetupHLODActorsParams.OutAdditionalWorldPartitionsForStandaloneHLOD);
			if (IsDistributedBuild())
			{
				// Generate working dirs for additional Standalone HLOD levels
				StandaloneHLODWorkingDirs.SetNum(AdditionalWorldPartitionsForStandaloneHLOD.Num());
				for (int32 Index = 0; Index < StandaloneHLODWorkingDirs.Num(); Index++)
				{
					StandaloneHLODWorkingDirs[Index] = GetDistributedBuildWorkingDir(AdditionalWorldPartitionsForStandaloneHLOD[Index]->GetWorld());
				}
			}

			// Refresh Asset Registry to include Standalone HLOD levels that were created during SetupHLODActors
			FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::Get().LoadModuleChecked<FDirectoryWatcherModule>(TEXT("DirectoryWatcher"));
			DirectoryWatcherModule.Get()->Tick(-1.0f);

			FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
			IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

			for (const TObjectPtr<UWorldPartition>& AdditionalWorldPartition : AdditionalWorldPartitionsForStandaloneHLOD)
			{
				FString WorldName = AdditionalWorldPartition->GetWorld()->GetPackage()->GetName();
				TArray<FString> ExternalObjectsPaths = ULevel::GetExternalObjectsPaths(WorldName);

				AssetRegistry.ScanModifiedAssetFiles({WorldName});
				AssetRegistry.ScanPathsSynchronous(ExternalObjectsPaths, true);
			}
		}

		// When performing a distributed build, ensure our work folder is empty
		if (IsDistributedBuild())
		{
			IFileManager::Get().DeleteDirectory(*DistributedBuildWorkingDir, false, true);
		}

		UE_LOGF(LogWorldPartitionHLODsBuilder, Display, "#### World HLOD actors ####");

		int32 NumActors = 0;
		TFunction<void(UWorldPartition*)> ListHLODActors = [&NumActors, bConsiderUnsavedHLODActors = bConsiderUnsavedHLODActors](UWorldPartition* WorldPartitionToProcess)
		{
			for (FActorDescContainerInstanceCollection::TIterator<AWorldPartitionHLOD> HLODIterator(WorldPartitionToProcess); HLODIterator; ++HLODIterator)
			{
				FWorldPartitionActorDescInstance* HLODActorDescInstance = *HLODIterator;
				FString PackageName = HLODActorDescInstance->GetActorPackage().ToString();

				UE_LOGF(LogWorldPartitionHLODsBuilder, Display, "    [%d] %ls", NumActors, *PackageName);

				NumActors++;
			}

			if (bConsiderUnsavedHLODActors)
			{
				if (UWorld* World = WorldPartitionToProcess->GetWorld())
				{
					for (TActorIterator<AWorldPartitionHLOD> It(World); It; ++It)
					{
						if (!WorldPartitionToProcess->GetActorDescInstance(It->GetActorGuid()))
						{
							UE_LOGF(LogWorldPartitionHLODsBuilder, Display, "    [%d] %ls", NumActors, *It->GetPackage()->GetName());
							NumActors++;
						}
					}
				}
			}
		};

		ListHLODActors(WorldPartition);

		if (bBuildingStandaloneHLOD)
		{
			for (const TObjectPtr<UWorldPartition>& AdditionalWorldPartition : AdditionalWorldPartitionsForStandaloneHLOD)
			{
				ListHLODActors(AdditionalWorldPartition);
			}
		}

		UE_LOGF(LogWorldPartitionHLODsBuilder, Display, "#### World contains %d HLOD actors ####", NumActors);
	}

	if (IsUsingBuildManifest())
	{
		// With Standalone HLOD levels we might be generating work for multiple builders across multiple worlds,
		// keep track of Builder Index and World Index for each file
		TMap<FString, TPair<int32, int32>> FilesToBuilderAndWorldIndexMap;
		bool bGenerated = GenerateBuildManifest(FilesToBuilderAndWorldIndexMap);
		if (!bGenerated)
		{
			return false;
		}

		// When performing a distributed build, move modified files to the temporary working dir, to be submitted later in the last "submit" step
		if (IsDistributedBuild())
		{
			// Ensure we don't hold on to packages of always loaded actors
			// When running distributed builds, we wanna leave the machine clean, so added files are deleted, check'd out files are reverted
			// and deleted files are restored.
			bool bCollectGarbage = false;
			if (WorldPartition)
			{
				WorldPartition->Uninitialize();
				bCollectGarbage = true;
			}
			
			// Clean up Standalone HLOD levels
			for (const TObjectPtr<UWorldPartition>& AdditionalWorldPartition : AdditionalWorldPartitionsForStandaloneHLOD)
			{
				if (AdditionalWorldPartition)
				{
					AdditionalWorldPartition->Uninitialize();
					bCollectGarbage = true;
				}
			}
			AdditionalWorldPartitionsForStandaloneHLOD.Empty();
			
			if (bCollectGarbage)
			{
				FWorldPartitionHelpers::DoCollectGarbage();
			}

			TArray<TArray<FBuilderModifiedFiles>> BuildersFilesPerWorld;
			BuildersFilesPerWorld.SetNum(bBuildingStandaloneHLOD ? StandaloneHLODWorkingDirs.Num() : 1);
			for (TArray<FBuilderModifiedFiles>& BuildersFiles : BuildersFilesPerWorld)
			{
				BuildersFiles.SetNum(BuilderCount);
			}

			for (int32 i = 0; i < FBuilderModifiedFiles::EFileOperation::NumFileOperations; i++)
			{
				FBuilderModifiedFiles::EFileOperation FileOp = (FBuilderModifiedFiles::EFileOperation)i;
				for (const FString& ModifiedFile : ModifiedFiles.Get(FileOp))
				{
					// Key - Builder Index
					// Value - World Index
					TPair<int32, int32>* Idx = FilesToBuilderAndWorldIndexMap.Find(ModifiedFile);
					if (Idx)
					{
						BuildersFilesPerWorld[Idx->Value][Idx->Key].Add(FileOp, ModifiedFile);
					}
					else
					{
						// Add general files to the last builder, first world
						BuildersFilesPerWorld[0].Last().Add(FileOp, ModifiedFile);
					}
				}
			}

			// Gather build product to ensure intermediary files are copied between the different HLOD generation steps
			TArray<FString> BuildProducts;

			// Copy files that will be handled by the different builders
			for (int32 WorldIndex = 0; WorldIndex < BuildersFilesPerWorld.Num(); WorldIndex++)
			{
				FString WorkingDir;
				if (bBuildingStandaloneHLOD)
				{
					WorkingDir = StandaloneHLODWorkingDirs[WorldIndex];
				}
				else
				{
					WorkingDir = DistributedBuildWorkingDir;
				}

				for (int32 Idx = 0; Idx < BuilderCount; Idx++)
				{
					if (!CopyFilesToWorkingDir(GetHLODBuilderFolderName(Idx), BuildersFilesPerWorld[WorldIndex][Idx], WorkingDir, BuildProducts))
					{
						return false;
					}
				}
			}

			// The build manifest must also be included as a build product to be available in the next steps
			// Store as relative path for portability
			FString RelativeBuildManifest = BuildManifest;
			FPaths::MakePathRelativeTo(RelativeBuildManifest, *GetDistributedBuildBaseDir());
			BuildProducts.Add(RelativeBuildManifest);

			// Write build products to a file
			if (!AddBuildProducts(BuildProducts))
			{
				return false;
			}
		}
	}

	// Clean up Standalone HLOD levels if not cleaned up before
	for (const TObjectPtr<UWorldPartition>& AdditionalWorldPartition : AdditionalWorldPartitionsForStandaloneHLOD)
	{
		if (AdditionalWorldPartition)
		{
			AdditionalWorldPartition->Uninitialize();
		}
	}
	AdditionalWorldPartitionsForStandaloneHLOD.Empty();
	FWorldPartitionHelpers::DoCollectGarbage();

	return true;
}

bool UWorldPartitionHLODsBuilder::BuildHLODActors()
{
	IWorldPartitionHLODUtilities* WPHLODUtilities = FModuleManager::LoadModuleChecked<IWorldPartitionHLODUtilitiesModule>("WorldPartitionHLODUtilities").GetUtilities();
	
	if (bReuseParentBranchHLODs)
	{
		WPHLODUtilities->SetHLODBuildEvaluator(IWorldPartitionHLODUtilities::FHLODBuildEvaluator::CreateUObject(this, &UWorldPartitionHLODsBuilder::EvaluateHLODBuildConditions));
	}

	// Disable nav system config overrides which can possible reenable navigation on our world. This leads to log spamming and extra overhead during our actors loading.
	for (TActorIterator<ANavSystemConfigOverride> NavSystemConfigOverrideActor(World); NavSystemConfigOverrideActor; ++NavSystemConfigOverrideActor)
	{
		NavSystemConfigOverrideActor->UnregisterAllComponents();
	}

	ON_SCOPE_EXIT
	{
		if (bReuseParentBranchHLODs)
		{
			WPHLODUtilities->SetHLODBuildEvaluator(nullptr);
			BuildEvaluationStats::PrintStats();
		}

		// Reenable nav system config overrides
		for (TActorIterator<ANavSystemConfigOverride> NavSystemConfigOverrideActor(World); NavSystemConfigOverrideActor; ++NavSystemConfigOverrideActor)
		{
			NavSystemConfigOverrideActor->RegisterAllComponents();
		}
	};

	auto SaveHLODActor = [this](AWorldPartitionHLOD* HLODActor)
	{
		if (bSaveActors)
		{
			TFunction<bool(UPackage*, const FString&)> SaveHLODPackage = [this, ActorLabel = HLODActor->GetActorLabel()](UPackage* Package, const FString& Description) -> bool
			{
				if (Package && Package->IsDirty())
				{
					UE_LOGF(LogWorldPartitionHLODsBuilder, Display, "%ls %ls was modified, saving...", *Description, *ActorLabel);

					bool bSaved = SourceControlHelper->Save(Package);
					if (!bSaved)
					{
						UE_LOGF(LogWorldPartitionHLODsBuilder, Error, "Failed to save %ls, exiting...", *USourceControlHelpers::PackageFilename(Package));
						return false;
					}
				}

				return true;
			};

			if (!SaveHLODPackage(HLODActor->GetPackage(), TEXT("HLOD Actor")))
			{
				return false;
			}

			// If HLOD resources are stored in a separate package, save it as well
			if (HLODActor->GetHLODResourcesPackage() != HLODActor->GetPackage())
			{
				return SaveHLODPackage(HLODActor->GetHLODResourcesPackage(), TEXT("HLOD resources package"));
			}
		}

		return true;
	};

	if (WorldPartition)
	{
		TArray<FGuid> HLODActorsToBuild;
		if (!GetHLODActorsToBuild(HLODActorsToBuild))
		{
			return false;
		}

		FHLODWorkload WorkloadToValidate;
		WorkloadToValidate.PerWorldHLODWorkloads.Add(HLODActorsToBuild);
		if (!ValidateWorkload(WorkloadToValidate, /*bShouldConsiderExternalHLODActors=*/false))
		{
			return false;
		}

		FScopedSlowTask SlowTask(static_cast<float>(HLODActorsToBuild.Num() - ResumeBuildIndex), LOCTEXT("BuildingHLODActors", "Building HLOD Actors"), bUseSlowTask);

		UE_LOGF(LogWorldPartitionHLODsBuilder, Display, "#### Building %d HLOD actors ####", HLODActorsToBuild.Num());
		if (bResumeBuild)
		{
			UE_LOGF(LogWorldPartitionHLODsBuilder, Display, "#### Resuming build at %d ####", ResumeBuildIndex);
		}

		// Cache all unsaved HLOD actors
		TMap<FGuid, TWeakObjectPtr<AWorldPartitionHLOD>> UnsavedHLODActors;
		if (bConsiderUnsavedHLODActors)
		{
			CacheUnsavedHLODActors(WorldPartition->GetWorld(), UnsavedHLODActors);
		}

		int32 BuiltHLODActors = 0;
		for (int32 CurrentActor = ResumeBuildIndex; CurrentActor < HLODActorsToBuild.Num(); ++CurrentActor)
		{
			if (SlowTask.ShouldCancel())
			{
				UE_LOGF(LogWorldPartitionHLODsBuilder, Display, "#### HLOD build cancelled ####");
				break;
			}

			TRACE_BOOKMARK(TEXT("BuildHLOD Start - %d"), CurrentActor);

			UPackage* HLODActorPackage = nullptr;
			{
				const FGuid& HLODActorGuid = HLODActorsToBuild[CurrentActor];

				FWorldPartitionReference ActorRef(WorldPartition, HLODActorGuid);
				AWorldPartitionHLOD* HLODActor = nullptr;
				if (ActorRef.IsValid())
				{ 
					HLODActor = CastChecked<AWorldPartitionHLOD>(ActorRef.GetActor());
				}
				else if (bConsiderUnsavedHLODActors)
				{
					if (TWeakObjectPtr<AWorldPartitionHLOD>* UnsavedHLODActorPtr = UnsavedHLODActors.Find(HLODActorGuid))
					{
						HLODActor = UnsavedHLODActorPtr->Get();
					}

				}
				check(HLODActor);

				SlowTask.EnterProgressFrame(1.f, FText::Format(LOCTEXT("BuildingHLODActorWithLabel", "Building HLOD actor: {0}"), FText::FromString(HLODActor->GetActorLabel())));

				HLODActorPackage = HLODActor->GetPackage();

				UE_LOGF(LogWorldPartitionHLODsBuilder, Display, "[%d / %d] %ls %ls...", CurrentActor + 1, HLODActorsToBuild.Num(), *LOCTEXT("BuildingHLODActor", "Building HLOD actor").ToString(), *HLODActor->GetActorLabel());

				if (IsRunningCommandlet())
				{
					// Simulate an engine tick to make sure engine & render resources that are queued for deletion are processed.
					FWorldPartitionHelpers::FakeEngineTick(World);
				}

				HLODActor->BuildHLOD(bForceBuild);

				if (ParentBranchHLODFileToCopy.IsEmpty())
				{
					bool bSaved = SaveHLODActor(HLODActor);
					if (!bSaved)
					{
						return false;
					}
				}
			}

			if (!ParentBranchHLODFileToCopy.IsEmpty())
			{
				CopyParentBranchHLODFile(HLODActorPackage);
			}

			TRACE_BOOKMARK(TEXT("BuildHLOD End - %d"), CurrentActor);

			if (FWorldPartitionHelpers::ShouldCollectGarbage())
			{
				FWorldPartitionHelpers::DoCollectGarbage();
			}

			BuiltHLODActors++;
		}

		UE_LOGF(LogWorldPartitionHLODsBuilder, Display, "#### Built %d HLOD actors ####", BuiltHLODActors);
	}
	else
	{
		IWorldPartitionHLODProvider::FBuildHLODActorParams BuildHLODActorParams;
		BuildHLODActorParams.bForceRebuild = bForceBuild;
		BuildHLODActorParams.OnPackageProcessed.BindLambda([this](UPackage* ProcessedPackage)
		{
			bool bSuccess = true;

			if (!ParentBranchHLODFileToCopy.IsEmpty())
			{
				CopyParentBranchHLODFile(ProcessedPackage);
			}
			else if (ProcessedPackage->IsDirty())
			{
				bSuccess = SourceControlHelper->Save(ProcessedPackage);
				if (!bSuccess)
				{
					UE_LOGF(LogWorldPartitionHLODsBuilder, Error, "Failed to save %ls, exiting...", *USourceControlHelpers::PackageFilename(ProcessedPackage));
				}
			}

			return bSuccess;
		});

		TArray<IWorldPartitionHLODProvider*> HLODProviders;

		// Gather all HLOD providers
		for (TActorIterator<AActor> ActorIt(World); ActorIt; ++ActorIt)
		{
			if (IWorldPartitionHLODProvider* HLODProvider = Cast<IWorldPartitionHLODProvider>(*ActorIt))
			{
				HLODProviders.Add(HLODProvider);
			}
		}

		// Process them one by one
		for (IWorldPartitionHLODProvider* HLODProvider : HLODProviders)
		{
			bool bBuildResult = HLODProvider->BuildHLODActor(BuildHLODActorParams);
			if (!bBuildResult)
			{
				return false;
			}
		}

		UE_LOGF(LogWorldPartitionHLODsBuilder, Display, "#### Built %d HLOD actor ####", HLODProviders.Num());
	}


	// Move modified files to the temporary working dir, to be submitted later in the final "submit" pass, from a single machine.
	if (IsDistributedBuild())
	{
		// Ensure we don't hold on to packages of always loaded actors
		// When running distributed builds, we wanna leave the machine clean, so added files are deleted, check'd out files are reverted
		// and deleted files are restored.
		if (WorldPartition)
		{
			WorldPartition->Uninitialize();
			FWorldPartitionHelpers::DoCollectGarbage();
		}

		TArray<FString> BuildProducts;

		if (!CopyFilesToWorkingDir("ToSubmit", ModifiedFiles, DistributedBuildWorkingDir, BuildProducts))
		{
			return false;
		}

		// Write build products to a file
		if (!AddBuildProducts(BuildProducts))
		{
			return false;
		}
	}

	return true;
}

bool UWorldPartitionHLODsBuilder::DeleteHLODActors()
{
	UE_LOGF(LogWorldPartitionHLODsBuilder, Display, "#### Deleting HLOD actors ####");

	TArray<UClass*> HLODActorClasses =
	{
		AWorldPartitionHLOD::StaticClass(),
		FindObject<UClass>(nullptr, TEXT("/Script/Engine.SpatialHashRuntimeGridInfo"))
	};

	TArray<FString> PackagesToDelete;

	if (bBuildingStandaloneHLOD)
	{
		// Find all Stadalone HLOD levels and delete them and all their external actors
		UWorld* SourceWorld = WorldPartition->GetWorld();
		FString FolderPath, PackagePrefix;
		UWorldPartitionStandaloneHLODSubsystem::GetStandaloneHLODFolderPathAndPackagePrefix(SourceWorld->GetPackage()->GetName(), FolderPath, PackagePrefix);

		TArray<FString> Packages;
		FPackageName::FindPackagesInDirectory(Packages, FolderPath);

		for (const FString& Package : Packages)
		{
			if (!Package.Contains(PackagePrefix))
			{
				continue;
			}

			FString PackageName = FPackageName::FilenameToLongPackageName(Package);
			PackagesToDelete.Add(PackageName);

			const TArray<FString> ExternalObjectsPaths = ULevel::GetExternalObjectsPaths(PackageName);
			for (const FString& ExternaObjectsPath : ExternalObjectsPaths)
			{
				FString ExternalObjectsDirectoryPath = FPackageName::LongPackageNameToFilename(ExternaObjectsPath);
				if (IFileManager::Get().DirectoryExists(*ExternalObjectsDirectoryPath))
				{
					const bool bSuccess = IFileManager::Get().IterateDirectoryRecursively(*ExternalObjectsDirectoryPath, [&PackagesToDelete](const TCHAR* FilenameOrDirectory, bool bIsDirectory)
					{
						if (!bIsDirectory)
						{
							PackagesToDelete.Add(FilenameOrDirectory);
						}

						return true;
					});

					if (!bSuccess)
					{
						UE_LOGF(LogWorldPartitionHLODsBuilder, Log, "Failed to iterate external package folder: %ls", *ExternalObjectsDirectoryPath);
					}
				}
			}
		}
	}

	for (FActorDescContainerInstanceCollection::TIterator<> Iterator(WorldPartition); Iterator; ++Iterator)
	{
		if (HLODActorClasses.FindByPredicate([ActorClass = Iterator->GetActorNativeClass()](const UClass* HLODClass) { return ActorClass->IsChildOf(HLODClass); }))
		{
			FString PackageName = Iterator->GetActorPackage().ToString();
			PackagesToDelete.Add(PackageName);
		}
	}

	// Ensure we don't hold on to packages of always loaded actors
	// When running distributed builds, we wanna leave the machine clean, so added files are deleted, checked out files are reverted
	// and deleted files are restored.
	WorldPartition->Uninitialize();
	FWorldPartitionHelpers::DoCollectGarbage();

	bool bDeleted = SourceControlHelper->Delete(PackagesToDelete);
	if (bDeleted)
	{
		for (int32 PackageIndex = 0; PackageIndex < PackagesToDelete.Num(); PackageIndex++)
		{
			UE_LOGF(LogWorldPartitionHLODsBuilder, Display, "[%d / %d] Deleting %ls...", PackageIndex + 1, PackagesToDelete.Num(), *PackagesToDelete[PackageIndex]);
		}
	}
	else
	{
		UE_LOGF(LogWorldPartitionHLODsBuilder, Error, "Failed to delete HLOD actors, exiting...");
		return false;
	}

	UE_LOGF(LogWorldPartitionHLODsBuilder, Display, "#### Deleted %d HLOD actors ####", PackagesToDelete.Num());

	return true;
}

bool UWorldPartitionHLODsBuilder::SubmitHLODActors()
{
	// Wait for pending async file writes before submitting
	UPackage::WaitForAsyncFileWrites();

	// Check in all modified files
	const FString ChangeDescription = FString::Printf(TEXT("Rebuilt HLODs for %s"), *World->GetPackage()->GetName());
	return OnFilesModified(ModifiedFiles.GetAllFiles(), ChangeDescription);
}

bool UWorldPartitionHLODsBuilder::DumpStats()
{
	const FString HLODStatsOutputFilename = FPaths::ProjectSavedDir() / TEXT("WorldPartition") / FString::Printf(TEXT("HLODStats-%08x.csv"), FPlatformProcess::GetCurrentProcessId());

	IWorldPartitionEditorModule::FWriteHLODStatsParams StatsParams;
	StatsParams.Filename = HLODStatsOutputFilename;
	StatsParams.World = World;
	StatsParams.StatsType = IWorldPartitionEditorModule::FWriteHLODStatsParams::EStatsType::Default;
	return IWorldPartitionEditorModule::Get().WriteHLODStats(StatsParams);
}

bool UWorldPartitionHLODsBuilder::GetHLODActorsToBuild(TArray<FGuid>& HLODActorsToBuild) const
{
	bool bRet = true;

	if (!BuildManifest.IsEmpty())
	{
		// Get HLOD actors to build from the BuildManifest file
		FConfigFile ConfigFile;
		ConfigFile.Read(BuildManifest);

		FString SectionName = GetHLODBuilderFolderName(BuilderIdx);

		const FConfigSection* ConfigSection = ConfigFile.FindSection(SectionName);
		if (ConfigSection)
		{
			TArray<FString> HLODActorGuidStrings;
			ConfigSection->MultiFind(TEXT("+HLODActorGuid"), HLODActorGuidStrings, /*bMaintainOrder=*/true);

			for (const FString& HLODActorGuidString : HLODActorGuidStrings)
			{
				FGuid HLODActorGuid;
				bRet = FGuid::Parse(HLODActorGuidString, HLODActorGuid);
				if (bRet)
				{
					HLODActorsToBuild.Add(HLODActorGuid);
				}
				else
				{
					UE_LOGF(LogWorldPartitionHLODsBuilder, Error, "Error parsing section [%ls] in config file \"%ls\"", *SectionName, *BuildManifest);
					break;
				}
			}
		}
		else
		{
			UE_LOGF(LogWorldPartitionHLODsBuilder, Log, "No section [%ls] found in config file \"%ls\", assuming no HLOD needs to be built.", *SectionName, *BuildManifest);
			bRet = false;
		}
	}
	else
	{
		// When getting HLOD Workloads during Build step, we don't want to consider Standalone HLOD Actors in Standalone HLOD Levels,
		// as they'll be considered when the builder runs directly on those levels
		TArray<FHLODWorkload> HLODWorkloads = GetHLODWorkloads(1, /*bShouldConsiderExternalHLODActors=*/false);
		HLODActorsToBuild = MoveTemp(HLODWorkloads[0].PerWorldHLODWorkloads[0]);
	}

	return bRet;
}

TArray<UWorldPartitionHLODsBuilder::FHLODWorkload> UWorldPartitionHLODsBuilder::GetHLODWorkloads(int32 NumWorkloads, bool bShouldConsiderExternalHLODActors) const
{
	if (!WorldPartition)
	{
		FHLODWorkload HLODWorkload;
		HLODWorkload.PerWorldHLODWorkloads.Add( { FGuid() } );
		return { HLODWorkload };
	}

	// Build a mapping of HLODActor to WorldPartition Index to be used when splitting actors into workloads
	// 0 - World Partition currently processed by the builder
	// 1 ... N - World Partitions from AdditionalWorldPartitionsForStandaloneHLOD array
	TMap<FGuid, uint32> HLODActorToWorldPartitionIndex;

	// Build a mapping of 1 HLOD[Level] -> N HLOD[Level - 1]
	TMap<FGuid, TArray<FGuid>>	HLODParenting;

	TMap<FGuid, TWeakObjectPtr<AWorldPartitionHLOD> > UnsavedHLODActors;
	if (bConsiderUnsavedHLODActors)
	{
		CacheUnsavedHLODActors(WorldPartition->GetWorld(), UnsavedHLODActors);
	}

	TFunction<void(const FHLODActorDesc&, const FGuid&, uint32)> ProcessHLODActor = [this, &HLODParenting, &HLODActorToWorldPartitionIndex, bShouldConsiderExternalHLODActors](const FHLODActorDesc& HLODActorDesc, const FGuid& ActorGuid, uint32 WorldPartitionIndex)
	{
		// Filter by HLOD actor
		if (!HLODActorToBuild.IsNone() && HLODActorDesc.GetActorLabel() != HLODActorToBuild)
		{
			return;
		}

		// Filter by HLOD layer
		if (!HLODLayerToBuild.IsNone() && HLODActorDesc.GetSourceHLODLayer().GetAssetName() != HLODLayerToBuild)
		{
			return;
		}
		
		FHLODCreationFilterContext FilterContext;
		FilterContext.Bounds = HLODActorDesc.GetRuntimeBounds();
		const bool bPassesAllFilters = UE::HLOD::CreationFilter::PassesFilters(Filters, FilterContext);

		if (!bPassesAllFilters)
		{
			return;
		}

		if (bBuildingStandaloneHLOD && IsDistributedBuild())
		{
			HLODActorToWorldPartitionIndex.Add(ActorGuid, WorldPartitionIndex);
		}

		// When requested to build a single HLOD Layer, skip the child actors
		if (HLODLayerToBuild.IsNone())
		{
			TArray<FGuid>& ChildActors = HLODParenting.Add(ActorGuid, HLODActorDesc.GetChildHLODActors());

			if (bShouldConsiderExternalHLODActors)
			{
				ChildActors.Append(HLODActorDesc.GetExternalChildHLODActors());
			}
		}
		else
		{
			HLODParenting.Add(ActorGuid);
		}
	};

	TFunction<void(UWorldPartition*, uint32)> ProcessWorldPartition = [this, &HLODParenting, &HLODActorToWorldPartitionIndex, &UnsavedHLODActors, bShouldConsiderExternalHLODActors, &ProcessHLODActor](UWorldPartition* WorldPartitionToProcess, uint32 WorldPartitionIndex)
	{
		// If considering unsaved HLOD Actors, process them first to use current state for filtering
		// For remaining actors, use their saved ActorDesc
		TSet<FGuid> ProcessedLoadedActors;
		if (bConsiderUnsavedHLODActors)
		{
			for (const TPair<FGuid, TWeakObjectPtr<AWorldPartitionHLOD>>& Pair : UnsavedHLODActors)
			{
				if (AWorldPartitionHLOD* HLODActor = Cast<AWorldPartitionHLOD>(Pair.Value))
				{
					ProcessedLoadedActors.Add(HLODActor->GetActorGuid());
					TUniquePtr<FWorldPartitionActorDesc> HLODActorDescPtr = HLODActor->CreateActorDesc();
					const FHLODActorDesc& HLODActorDesc = *(FHLODActorDesc*)HLODActorDescPtr.Get();
					ProcessHLODActor(HLODActorDesc, HLODActor->GetActorGuid(), WorldPartitionIndex);

					// Unsaved HLOD actors might use other unsaved HLOD actors as ChildActors. If a ChildActor was never saved, it won't appear in ChildActors list generated by CreateActorDesc. Add them now.
					if (UWorldPartitionHLODSourceActorsFromCell* SourceActors = Cast<UWorldPartitionHLODSourceActorsFromCell>(HLODActor->GetSourceActors()))
					{
						if (TArray<FGuid>* ChildActorsPtr = HLODParenting.Find(HLODActor->GetActorGuid()))
						{
							TArray<FGuid>& ChildActors = *ChildActorsPtr;
							for (const FWorldPartitionRuntimeCellObjectMapping& SourceActor : SourceActors->GetActors())
							{
								if (UnsavedHLODActors.Contains(SourceActor.ActorInstanceGuid) && !ChildActors.Contains(SourceActor.ActorInstanceGuid))
								{
									ChildActors.Add(SourceActor.ActorInstanceGuid);
								}
							}
						}
					}
				}
			}
		}

		for (FActorDescContainerInstanceCollection::TIterator<AWorldPartitionHLOD> HLODIterator(WorldPartitionToProcess); HLODIterator; ++HLODIterator)
		{
			if (bConsiderUnsavedHLODActors && ProcessedLoadedActors.Contains(HLODIterator->GetGuid()))
			{
				continue;
			}

			const FHLODActorDesc& HLODActorDesc = *(FHLODActorDesc*)HLODIterator->GetActorDesc();
			ProcessHLODActor(HLODActorDesc, HLODIterator->GetGuid(), WorldPartitionIndex);
		}
	};

	// In distributed builds all workloads are prepared during the Setup step, which doesn't run on Standalone HLOD Levels, so we have to generate the workloads for them as well
	// If building Standalone HLODs, all HLOD actors are in Standalone HLOD Levels, so we can skip processing the main world
	// In non-distributed builds, workloads are generated during the Build step, which runs on Standalone HLOD Levels
	if (bBuildingStandaloneHLOD && IsDistributedBuild())
	{
		for (int32 WorldIndex = 0; WorldIndex < AdditionalWorldPartitionsForStandaloneHLOD.Num(); WorldIndex++)
		{
			ProcessWorldPartition(AdditionalWorldPartitionsForStandaloneHLOD[WorldIndex], WorldIndex);
		}
	}
	else
	{
		ProcessWorldPartition(WorldPartition, 0);
	}

	// All child HLODs must be built before their parent HLOD
	// Create groups to ensure those will be processed in the correct order, on the same builder
	TMap<FGuid, TArray<FGuid>> HLODGroups;
	TSet<FGuid>				   TriagedHLODs;

	TFunction<void(const FGuid&, const FGuid&)> RecursiveAdd = [&TriagedHLODs, &HLODParenting, &HLODGroups, &RecursiveAdd](const FGuid& GroupGuid, const FGuid& HLODGuid)
	{
		if (!TriagedHLODs.Contains(HLODGuid))
		{
			TriagedHLODs.Add(HLODGuid);
			HLODGroups.FindChecked(GroupGuid).Insert(HLODGuid, 0); // Child will come first in the list, as they need to be built first...
			TArray<FGuid>* ChildHLODs = HLODParenting.Find(HLODGuid);
			if (ChildHLODs)
			{
				for (const auto& ChildGuid : *ChildHLODs)
				{
					RecursiveAdd(GroupGuid, ChildGuid);
				}
			}
		}
		else
		{
			HLODGroups.FindChecked(GroupGuid).Insert(MoveTemp(HLODGroups.FindChecked(HLODGuid)), 0);
			HLODGroups.Remove(HLODGuid);
		}
	};

	for (const auto& Pair : HLODParenting)
	{
		if (!TriagedHLODs.Contains(Pair.Key))
		{
			HLODGroups.Add(Pair.Key);
			RecursiveAdd(Pair.Key, Pair.Key);
		}
	}

	// Sort groups by number of HLOD actors
	HLODGroups.ValueSort([](const TArray<FGuid>& GroupA, const TArray<FGuid>& GroupB) { return GroupA.Num() > GroupB.Num(); });

	// Dispatch them in multiple lists and try to balance the workloads as much as possible
	TArray<FHLODWorkload> Workloads;
	Workloads.SetNum(NumWorkloads);

	for (FHLODWorkload& Workload : Workloads)
	{
		if (bBuildingStandaloneHLOD && IsDistributedBuild())
		{
			Workload.PerWorldHLODWorkloads.SetNum(AdditionalWorldPartitionsForStandaloneHLOD.Num());
		}
		else
		{
			Workload.PerWorldHLODWorkloads.SetNum(1);
		}
	}

	int32 Idx = 0;
	for (const auto& Pair : HLODGroups)
	{
		int32 WorkloadNum = Idx % NumWorkloads;
		for (const FGuid& HLODActorGuid : Pair.Value)
		{
			int32 WorldIndex;
			if (bBuildingStandaloneHLOD && IsDistributedBuild())
			{
				// We might be generating workloads for a few Worlds at the same time. Find which one, so that we can assign actor to the right workload
				const uint32* IndexPtr = HLODActorToWorldPartitionIndex.Find(HLODActorGuid);
				check(IndexPtr);
				WorldIndex = *IndexPtr;
			}
			else
			{
				WorldIndex = 0;
			}
			
			Workloads[WorkloadNum].PerWorldHLODWorkloads[WorldIndex].Add(HLODActorGuid);
		}
		Idx++;
	}

	// Validate workloads to ensure our meshes are built in the correct order
	for (const FHLODWorkload& Workload : Workloads)
	{
		check(ValidateWorkload(Workload, bShouldConsiderExternalHLODActors));
	}

	return Workloads;
}

bool UWorldPartitionHLODsBuilder::ValidateWorkload(const FHLODWorkload& Workload, bool bShouldConsiderExternalHLODActors) const
{
	check(WorldPartition);

	uint32 NumHLODs = 0;
	for (const TArray<FGuid>& HLODActorArray : Workload.PerWorldHLODWorkloads)
	{
		NumHLODs += HLODActorArray.Num();
	}

	TSet<FGuid> ProcessedHLOD;
	ProcessedHLOD.Reserve(NumHLODs);

	// For each HLOD entry in the workload, validate that its children are found before itself
	for (int32 WorldIndex = 0; WorldIndex < Workload.PerWorldHLODWorkloads.Num(); WorldIndex++)
	{
		UWorldPartition* CurrentWorldPartition = (bBuildingStandaloneHLOD && IsDistributedBuild()) ? AdditionalWorldPartitionsForStandaloneHLOD[WorldIndex].Get() : WorldPartition;

		TMap<FGuid, TWeakObjectPtr<AWorldPartitionHLOD>> UnsavedHLODActors;
		if (bConsiderUnsavedHLODActors)
		{
			CacheUnsavedHLODActors(CurrentWorldPartition->GetWorld(), UnsavedHLODActors);
		}

		for (const FGuid& HLODActorGuid : Workload.PerWorldHLODWorkloads[WorldIndex])
		{
			const FWorldPartitionActorDescInstance* ActorDescInstance = CurrentWorldPartition->GetActorDescInstance(HLODActorGuid);
			bool bUsingUnsavedHLODActor = false;
			TUniquePtr<FWorldPartitionActorDesc> UnsavedHLODActorDesc;
			if (bConsiderUnsavedHLODActors && !ActorDescInstance)
			{
				if (TWeakObjectPtr<AWorldPartitionHLOD>* UnsavedHLODActorPtr = UnsavedHLODActors.Find(HLODActorGuid))
				{
					UnsavedHLODActorDesc = (*UnsavedHLODActorPtr)->CreateActorDesc();
					bUsingUnsavedHLODActor = true;
				}
			}

			if (!bUsingUnsavedHLODActor)
			{
				if (!ActorDescInstance)
				{
					UE_LOGF(LogWorldPartitionHLODsBuilder, Error, "Unknown actor guid found (\"%ls\"), your HLOD actors are probably out of date. Run with -SetupHLODs to fix this. Exiting...", *HLODActorGuid.ToString());
					return false;
				}

				if (!ActorDescInstance->GetActorNativeClass()->IsChildOf<AWorldPartitionHLOD>())
				{
					UE_LOGF(LogWorldPartitionHLODsBuilder, Error, "Unexpected actor guid found in HLOD workload (\"%ls\"), exiting...", *HLODActorGuid.ToString());
					return false;
				}
			}

			// When requested to build a single HLOD Layer, do not validate that child actors are included
			// Don't validate when using Volume Filter, as not all child actors have to pass the volume Volume Filter
			const bool bHasAnyActiveFilters = UE::HLOD::CreationFilter::HasAnyActiveFilters(Filters);
			if (HLODLayerToBuild.IsNone() && !bHasAnyActiveFilters)
			{
				const FHLODActorDesc* HLODActorDesc = static_cast<const FHLODActorDesc*>(bUsingUnsavedHLODActor ? UnsavedHLODActorDesc.Get() : ActorDescInstance->GetActorDesc());

				for (const FGuid& ChildHLODActorGuid : HLODActorDesc->GetChildHLODActors())
				{
					if (!ProcessedHLOD.Contains(ChildHLODActorGuid))
					{
						UE_LOGF(LogWorldPartitionHLODsBuilder, Error, "Child HLOD actor (\"%ls\") missing or out of order in HLOD workload, exiting...", *ChildHLODActorGuid.ToString());
						return false;
					}
				}

				// Skip checking whether external child actors are included if we're not considering them
				if (bShouldConsiderExternalHLODActors)
				{
					for (const FGuid& ExternalChildHLODActorGuid : HLODActorDesc->GetExternalChildHLODActors())
					{
						if (!ProcessedHLOD.Contains(ExternalChildHLODActorGuid))
						{
							UE_LOGF(LogWorldPartitionHLODsBuilder, Error, "External child HLOD actor (\"%ls\") missing or out of order in HLOD workload, exiting...", *ExternalChildHLODActorGuid.ToString());
							return false;
						}
					}
				}
			}

			ProcessedHLOD.Add(HLODActorGuid);
		}
	}

	return true;
}

bool UWorldPartitionHLODsBuilder::GenerateBuildManifest(TMap<FString, TPair<int32, int32>>& FilesToBuilderAndWorldIndexMap) const
{
	// We're generating manifest for Standalone HLOD levels as well (if any), so we want to consider External HLOD actors
	TArray<FHLODWorkload> BuildersWorkload = GetHLODWorkloads(BuilderCount, /*bShouldConsiderExternalHLODActors=*/true);

	// If we're generating manifest for Standalone HLOD levels, each of them needs a separate config file
	TArray<FConfigFile> ConfigFiles;
	bool bHasStandaloneHLOD = WorldPartition ? WorldPartition->HasStandaloneHLOD() : false;
	ConfigFiles.SetNum(bHasStandaloneHLOD ? AdditionalWorldPartitionsForStandaloneHLOD.Num() : 1);
	for (FConfigFile& ConfigFile : ConfigFiles)
	{
		ConfigFile.SetInt64(TEXT("General"), TEXT("BuilderCount"), BuilderCount);
		ConfigFile.SetString(TEXT("General"), TEXT("EngineVersion"), *FEngineVersion::Current().ToString());
	}

	// When processing multiple maps, ensure that the worldload is distributed evenly between builders.
	// Otherwise, maps with a single HLOD would all end up being processed by the first builder, while the others would have no work.
	static int32 BuilderDispatchOffset = 0;

	for(int32 Idx = 0; Idx < BuilderCount; Idx++)
	{
		const int32 WorkloadIndex = Idx;
		const int32 BuilderIndex = (BuilderDispatchOffset + Idx) % BuilderCount;

		if (!BuildersWorkload.IsValidIndex(WorkloadIndex) || BuildersWorkload[WorkloadIndex].PerWorldHLODWorkloads.IsEmpty())
		{
			continue;
		}

		FString SectionName = GetHLODBuilderFolderName(BuilderIndex);

		for (int32 WorldIndex = 0; WorldIndex < BuildersWorkload[WorkloadIndex].PerWorldHLODWorkloads.Num(); WorldIndex++)
		{
			UWorldPartition* CurrentWorldPartition = bHasStandaloneHLOD ? AdditionalWorldPartitionsForStandaloneHLOD[WorldIndex].Get() : WorldPartition;
			for(const FGuid& ActorGuid : BuildersWorkload[WorkloadIndex].PerWorldHLODWorkloads[WorldIndex])
			{
				ConfigFiles[WorldIndex].AddToSection(*SectionName, TEXT("+HLODActorGuid"), ActorGuid.ToString(EGuidFormats::Digits));

				if (CurrentWorldPartition)
				{
					// Track which builder is responsible to handle each actor
					const FWorldPartitionActorDescInstance* ActorDescInstance = CurrentWorldPartition->GetActorDescInstance(ActorGuid);
					if (!ActorDescInstance)
					{
						UE_LOGF(LogWorldPartitionHLODsBuilder, Error, "Invalid actor GUID found while generating the HLOD build manifest, exiting...");
						return false;
					}
					FString ActorPackageFilename = USourceControlHelpers::PackageFilename(ActorDescInstance->GetActorPackage().ToString());
					FilesToBuilderAndWorldIndexMap.Emplace(ActorPackageFilename, TPair<int32, int32>(BuilderIndex, WorldIndex));
				}
			}
		}
	}

	BuilderDispatchOffset++;

	for (int32 Index = 0; Index < ConfigFiles.Num(); Index++)
	{
		FString BuildManifestFile;
		if (bBuildingStandaloneHLOD)
		{
			BuildManifestFile = StandaloneHLODWorkingDirs[Index] / DistributedBuildManifestName;
		}
		else
		{
			BuildManifestFile = BuildManifest;
		}

		ConfigFiles[Index].Dirty = true;

		if (!ConfigFiles[Index].Write(BuildManifestFile))
		{
			UE_LOGF(LogWorldPartitionHLODsBuilder, Error, "Failed to write HLOD build manifest \"%ls\"", *BuildManifestFile);
			return false;
		}
	}

	return true;
}

/*
	Working Dir structure
		/HLODBuilder0
			/Add
				NewFileA
				NewFileB
			/Delete
				DeletedFileA
				DeletedFileB
			/Edit
				EditedFileA
				EditedFileB

		/HLODBuilder1
			...
		/ToSubmit
			...

	Distributed mode
		* Distributed mode is ran into 3 steps
			* Setup (1 job)		
			* Build (N jobs)	
			* Submit (1 job)	
		
		* The Setup step will place files under the "HLODBuilder[0-N]" folder. Those files could be new or modified HLOD actors that will be built in the Build step. The setup step will also place files into the "ToSubmit" folder (deleted HLOD actors for example).
		* Each parallel job in the Build step will retrieve files from the "HLODBuilder[0-N]" folder. They will then proceed to build the HLOD actors as specified in the build manifest file. All built HLOD actor files will then be placed in the /ToSubmit folder.
		* The Submit step will gather all files under /ToSubmit and submit them.
		

		|			Setup			|					Build					  |		   Submit			|
		/Content -----------> /HLODBuilder -----------> /Content -----------> /ToSubmit -----------> /Content
*/

const FName FileAction_Add(TEXT("Add"));
const FName FileAction_Edit(TEXT("Edit"));
const FName FileAction_Delete(TEXT("Delete"));

bool UWorldPartitionHLODsBuilder::CopyFilesToWorkingDir(const FString& TargetDir, const FBuilderModifiedFiles& Files, const FString& WorkingDir, TArray<FString>& BuildProducts)
{
	const FString AbsoluteTargetDir = WorkingDir / TargetDir / TEXT("");
	const FString BaseDir = GetDistributedBuildBaseDir();

	bool bSuccess = true;

	auto CopyFileToWorkingDir = [&](const FString& SourceFilename, const FName FileAction)
	{
		FString SourceFilenameRelativeToProject = SourceFilename;
		FPaths::MakePathRelativeTo(SourceFilenameRelativeToProject, *FPaths::ProjectDir());

		FString TargetFilename = AbsoluteTargetDir / FileAction.ToString() / SourceFilenameRelativeToProject;

		// Store path relative to HLODTemp base directory for portability
		FString RelativeTargetFilename = TargetFilename;
		FPaths::MakePathRelativeTo(RelativeTargetFilename, *BaseDir);
		BuildProducts.Add(RelativeTargetFilename);

		if (FileAction != FileAction_Delete)
		{
			const bool bReplace = true;
			const bool bEvenIfReadOnly = true;
			bool bRet = IFileManager::Get().Copy(*TargetFilename, *SourceFilename, bReplace, bEvenIfReadOnly) == COPY_OK;
			if (!bRet)
			{
				UE_LOGF(LogWorldPartitionHLODsBuilder, Error, "Failed to copy file from \"%ls\" to \"%ls\"", *SourceFilename, *TargetFilename);
				bSuccess = false;
			}
		}
		else
		{
			bool bRet = FFileHelper::SaveStringToFile(TEXT(""), *TargetFilename);
			if (!bRet)
			{
				UE_LOGF(LogWorldPartitionHLODsBuilder, Error, "Failed to create empty file at \"%ls\"", *TargetFilename);
				bSuccess = false;
			}
		}
	};

	// Wait for pending async file writes before copying to working dir
	UPackage::WaitForAsyncFileWrites();

	Algo::ForEach(Files.Get(FBuilderModifiedFiles::EFileOperation::FileAdded), [&](const FString& SourceFilename) { CopyFileToWorkingDir(SourceFilename, FileAction_Add); });
	Algo::ForEach(Files.Get(FBuilderModifiedFiles::EFileOperation::FileEdited), [&](const FString& SourceFilename) { CopyFileToWorkingDir(SourceFilename, FileAction_Edit); });
	Algo::ForEach(Files.Get(FBuilderModifiedFiles::EFileOperation::FileDeleted), [&](const FString& SourceFilename) { CopyFileToWorkingDir(SourceFilename, FileAction_Delete); });
	if (!bSuccess)
	{
		return false;
	}

	// Revert any file changes
	if (ISourceControlModule::Get().IsEnabled())
	{
		bool bRet = USourceControlHelpers::RevertFiles(Files.GetAllFiles());
		if (!bRet)
		{
			UE_LOGF(LogWorldPartitionHLODsBuilder, Error, "Failed to revert modified files: %ls", *USourceControlHelpers::LastErrorMsg().ToString());
			return false;
		}
	}

	// Delete files we added
	for (const FString& FileToDelete : Files.Get(FBuilderModifiedFiles::EFileOperation::FileAdded))
	{
		if (!IFileManager::Get().Delete(*FileToDelete, false, true))
		{
			UE_LOGF(LogWorldPartitionHLODsBuilder, Error, "Error deleting file %ls locally", *FileToDelete);
			return false;
		}
	}

	return true;
}

bool UWorldPartitionHLODsBuilder::CopyFilesFromWorkingDir(const FString& SourceDir)
{
	const FString AbsoluteSourceDir = DistributedBuildWorkingDir / SourceDir / TEXT("");

	auto CopyFromWorkingDir = [](const TMap<FString, FString>& FilesToCopy) -> bool
	{
		for (const auto& Pair : FilesToCopy)
		{
			const bool bReplace = true;
			const bool bEvenIfReadOnly = true;
			bool bRet = IFileManager::Get().Copy(*Pair.Key, *Pair.Value, bReplace, bEvenIfReadOnly) == COPY_OK;
			if (!bRet)
			{
				UE_LOGF(LogWorldPartitionHLODsBuilder, Error, "Failed to copy file from \"%ls\" to \"%ls\"", *Pair.Value, *Pair.Key);
				return false;
			}
		}
		return true;
	};

	TArray<FString> Files;
	IFileManager::Get().FindFilesRecursive(Files, *AbsoluteSourceDir, TEXT("*.*"), true, false);

	TMap<FString, FString>	FilesToAdd;
	TMap<FString, FString>	FilesToEdit;
	TArray<FString>			FilesToDelete;

	bool bRet = true;

	for(const FString& File : Files)
	{
		FString PathRelativeToWorkingDir = File;
		FPaths::MakePathRelativeTo(PathRelativeToWorkingDir, *AbsoluteSourceDir);

		FString FileActionString;
		const int32 SlashIndex = PathRelativeToWorkingDir.Find(TEXT("/"));
		if (SlashIndex != INDEX_NONE)
		{
			FileActionString = PathRelativeToWorkingDir.Mid(0, SlashIndex);
		}

		FPaths::MakePathRelativeTo(PathRelativeToWorkingDir, *(FileActionString / TEXT("")));
		FString FullPathInProjectDirectory = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir() / PathRelativeToWorkingDir);

		FName FileAction(FileActionString);
		if (FileAction == FileAction_Add)
		{
			FilesToAdd.Add(FullPathInProjectDirectory, File);
		}
		else if (FileAction == FileAction_Edit)
		{
			FilesToEdit.Add(FullPathInProjectDirectory, File);
		}
		else if (FileAction == FileAction_Delete)
		{
			FilesToDelete.Add(FullPathInProjectDirectory);
		}
		else
		{
			UE_LOGF(LogWorldPartitionHLODsBuilder, Error, "Unsupported file action %ls for file %ls", *FileActionString, *FullPathInProjectDirectory);
		}
	}

	TArray<FString> ToAdd;
	FilesToAdd.GetKeys(ToAdd);

	TArray<FString> ToEdit;
	FilesToEdit.GetKeys(ToEdit);

	// When resuming a build (after a crash for example) we don't need to perform any file operation as these modification were done in the first run.
	if (!bResumeBuild)
	{
	    // Add
	    if (!FilesToAdd.IsEmpty())
	    {
			bRet = CopyFromWorkingDir(FilesToAdd);
			if (!bRet)
			{
				return false;
			}

			if (ISourceControlModule::Get().IsEnabled())
			{
				bRet = USourceControlHelpers::MarkFilesForAdd(ToAdd);
				if (!bRet)
				{
					UE_LOGF(LogWorldPartitionHLODsBuilder, Error, "Adding files to revision control failed: %ls", *USourceControlHelpers::LastErrorMsg().ToString());
					return false;
				}
			}
	    }
    
	    // Delete
	    if (!FilesToDelete.IsEmpty())
	    {
			if (ISourceControlModule::Get().IsEnabled())
			{
				bRet = USourceControlHelpers::MarkFilesForDelete(FilesToDelete);
				if (!bRet)
				{
					UE_LOGF(LogWorldPartitionHLODsBuilder, Error, "Deleting files from revision control failed: %ls", *USourceControlHelpers::LastErrorMsg().ToString());
					return false;
				}
			}
			else
			{
				for (const FString& FileToDelete : FilesToDelete)
				{
					const bool bRequireExists = false;
					const bool bEvenIfReadOnly = true;
					bRet = IFileManager::Get().Delete(*FileToDelete, bRequireExists, bEvenIfReadOnly);
					if (!bRet)
					{
						UE_LOGF(LogWorldPartitionHLODsBuilder, Error, "Failed to delete file from disk: %ls", *USourceControlHelpers::LastErrorMsg().ToString());
						return false;
					}
				}
			}
	    }
    
	    // Edit
	    if (!FilesToEdit.IsEmpty())
	    {
		    if (ISourceControlModule::Get().IsEnabled())
		    {
				bRet = USourceControlHelpers::CheckOutFiles(ToEdit);
				if (!bRet)
				{
					UE_LOGF(LogWorldPartitionHLODsBuilder, Error, "Checking out files from revision control failed: %ls", *USourceControlHelpers::LastErrorMsg().ToString());
					return false;
				}
			}
		
			bRet = CopyFromWorkingDir(FilesToEdit);
			if (!bRet)
			{
				return false;
			}
	    }
	}

	// Keep track of all modified files
	ModifiedFiles.Append(FBuilderModifiedFiles::EFileOperation::FileAdded, ToAdd);
	ModifiedFiles.Append(FBuilderModifiedFiles::EFileOperation::FileDeleted, FilesToDelete);
	ModifiedFiles.Append(FBuilderModifiedFiles::EFileOperation::FileEdited, ToEdit);

	// Force a rescan of the updated files
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	AssetRegistry.ScanModifiedAssetFiles(ModifiedFiles.GetAllFiles());

	return true;
}

bool UWorldPartitionHLODsBuilder::AddBuildProducts(const TArray<FString>& BuildProducts) const
{
	// Write build products to a file in the project's Intermediate directory
	FString BuildProductsFile = FString::Printf(TEXT("%s%s"), *GetDistributedBuildBaseDir(), *BuildProductsFileName);
	bool bRet = FFileHelper::SaveStringArrayToFile(BuildProducts, *BuildProductsFile, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);
	if (!bRet)
	{
		UE_LOGF(LogWorldPartitionHLODsBuilder, Error, "Error writing build product file %ls", *BuildProductsFile);
	}
	return bRet;
}

struct FScopedLogSuppression
{
	FScopedLogSuppression(FStringView LogCategory)
	{
		LogCommand = TEXT("Log ");
		LogCommand += LogCategory;

		FOutputDeviceNull Null;
		GEngine->Exec(nullptr, *FString(LogCommand + TEXT(" off")), Null);
	}

	~FScopedLogSuppression()
	{
		FOutputDeviceNull Null;
		GEngine->Exec(nullptr, *FString(LogCommand + TEXT(" on")), Null);
	}

private:
	FString LogCommand;
};

EHLODRebuildPolicyDecision UWorldPartitionHLODsBuilder::EvaluateHLODBuildConditions(AWorldPartitionHLOD* HLODActor, const FHLODRebuildPolicyDataSet& InOldData, const FHLODRebuildPolicyDataSet& InNewData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionHLODsBuilder::EvaluateHLODBuildConditions);

	FName HLODLayerStatsName = NAME_None;
	if (ensure(HLODActor->GetSourceActors() && HLODActor->GetSourceActors()->GetHLODLayer()))
	{
		HLODLayerStatsName = HLODActor->GetSourceActors()->GetHLODLayer()->GetFName();
	}

	BuildEvaluationStats::FStats& EvaluationStats = BuildEvaluationStats::StatsByHLODLayer.FindOrAdd(HLODLayerStatsName);

	FScopedDurationTimer TimerEvaluate(EvaluationStats.TimeEvaluate);

	EvaluationStats.Evaluated++;

	EHLODRebuildPolicyDecision RebuildDecision = UHLODRebuildPolicy::Evaluate(HLODActor, InOldData, InNewData);
	EvaluationStats.Built += RebuildDecision == EHLODRebuildPolicyDecision::ApproveRebuild ? 1 : 0;

	// Parent branch comparison can't be performed without SCC
	if (!ISourceControlModule::Get().IsEnabled())
	{
		UE_LOGF(LogWorldPartitionHLODsBuilder, Verbose, "Skipping comparison with parent branch: Source control is disabled");
		return RebuildDecision;
	}

	// Current Branch
	const FString& CurrentBranch = FEngineVersion::Current().GetBranch();
	int32 CurrentBranchIndex = ISourceControlModule::Get().GetProvider().GetStateBranchIndex(CurrentBranch);
	if (CurrentBranchIndex == INDEX_NONE)
	{
		UE_LOGF(LogWorldPartitionHLODsBuilder, Verbose, "Skipping comparison with parent branch: Couldn't retrieve current branch index");
		return RebuildDecision;
	}
	UE_LOGF(LogWorldPartitionHLODsBuilder, VeryVerbose, "Current Branch: %ls", *CurrentBranch);

	// Parent Branch
	int32 ParentBranchIndex = CurrentBranchIndex + 1;
	FString ParentBranch;
	if (!ISourceControlModule::Get().GetProvider().GetStateBranchAtIndex(ParentBranchIndex, ParentBranch))
	{
		UE_LOGF(LogWorldPartitionHLODsBuilder, Verbose, "Skipping comparison with parent branch: Couldn't retrieve parent branch");
		return RebuildDecision;
	}
	UE_LOGF(LogWorldPartitionHLODsBuilder, VeryVerbose, "Parent Branch: %ls", *ParentBranch);

	// Local File Path
	const FString LocalFilePathCurrent = USourceControlHelpers::PackageFilename(HLODActor->GetPackage());
	UE_LOGF(LogWorldPartitionHLODsBuilder, VeryVerbose, "Local File Path: %ls", *LocalFilePathCurrent);

	// Depot File Path (Current Branch)
	const TSharedRef<FWhere, ESPMode::ThreadSafe> WhereOperation = ISourceControlOperation::Create<FWhere>();
	ECommandResult::Type WhereResult = ISourceControlModule::Get().GetProvider().Execute(WhereOperation, LocalFilePathCurrent, EConcurrency::Synchronous);
	if (WhereResult != ECommandResult::Succeeded || WhereOperation->GetFiles().IsEmpty())
	{
		UE_LOGF(LogWorldPartitionHLODsBuilder, Verbose, "Skipping comparison with parent branch: Couldn't retrieve depot file path");
		return RebuildDecision;
	}
	const FString DepotFilePathCurrent = WhereOperation->GetFiles()[0].RemotePath;
	UE_LOGF(LogWorldPartitionHLODsBuilder, VeryVerbose, "Depot File Path (Current): %ls", *DepotFilePathCurrent);

	// Depot File Path (Parent Branch)
	FString DepotFilePathParent = DepotFilePathCurrent;
	if (!DepotFilePathParent.RemoveFromStart(CurrentBranch))
	{
		UE_LOGF(LogWorldPartitionHLODsBuilder, Verbose, "Skipping comparison with parent branch: Couldn't map current depot file path to the parent branch file path");
		return RebuildDecision;
	}
	DepotFilePathParent = ParentBranch + DepotFilePathParent;
	UE_LOGF(LogWorldPartitionHLODsBuilder, VeryVerbose, "Depot File Path (Parent): %ls", *DepotFilePathParent);

	// Retrieve file from parent branch, store to a temp location
	const FString Prefix = TEXT("TempHLOD");
	const FString EmptyExtension = TEXT("");
	const FString TempFolder = FPaths::CreateTempFilename(*FPaths::DiffDir(), *Prefix, *EmptyExtension);
	const FString FileName = FPaths::GetCleanFilename(LocalFilePathCurrent);
	const FString LocalFilePathParent = FPaths::ConvertRelativePathToFull(TempFolder / FileName);
	{
		FScopedDurationTimer TimerSync(EvaluationStats.TimeSyncParent);
		TSharedRef<FDownloadFile, ESPMode::ThreadSafe> DownloadCommand = ISourceControlOperation::Create<FDownloadFile>(TempFolder);
		DownloadCommand->SetEnableErrorLogging(false);
		DownloadCommand->SetEnableInfoLogging(false);
		if (ISourceControlModule::Get().GetProvider().Execute(DownloadCommand, DepotFilePathParent, EConcurrency::Synchronous) != ECommandResult::Succeeded)
		{
			UE_LOGF(LogWorldPartitionHLODsBuilder, Verbose, "Skipping comparison with parent branch: Couldn't download file from parent branch");
			EvaluationStats.NotFoundInParentBranch++;
			return RebuildDecision;
		}
		EvaluationStats.DataSyncedBytes += IFileManager::Get().FileSize(*LocalFilePathParent);
	}

	// Compare against the parent branch
	TOptional<EHLODRebuildPolicyDecision> ParentRebuildDecision;

	{
		// Since we are copying the package from another (parent) branch, there's always the possibility that the package 
		// we are about to load was saved with a newer version of the engine (a paused auto integration for example).
		// This will be gracefully handled by the engine (package will fail to load).
		// Suppress linker errors, as this is not a problem for our use case.
		FScopedLogSuppression VerbosityScope(TEXT("LogLinker"));

		FScopedDurationTimer TimerLoad(EvaluationStats.TimeLoadParent);

		const FPackagePath TempPackagePath = FPackagePath::FromLocalPath(LocalFilePathParent);
		const FPackagePath AssetPath = FPackagePath::FromLocalPath(LocalFilePathCurrent);
		if (UPackage* TempPackage = DiffUtils::LoadPackageForDiff(TempPackagePath, AssetPath))
		{
			// Grab the old asset from that old package
			const AWorldPartitionHLOD* FoundActor = nullptr;
			ForEachObjectWithOuter(TempPackage, [&FoundActor](const UObject* InObject)
			{
				if (!FoundActor)
				{
					FoundActor = Cast<AWorldPartitionHLOD>(InObject);
				}
			});

			if (FoundActor)
			{
				TUniquePtr<FWorldPartitionActorDesc> CurrentActorDesc = HLODActor->CreateActorDesc();
				TUniquePtr<FWorldPartitionActorDesc> ParentActorDesc = FoundActor->CreateActorDesc();
				const FHLODActorDesc& CurrentHLODActorDesc = *(static_cast<FHLODActorDesc*>(CurrentActorDesc.Get()));
				const FHLODActorDesc& ParentHLODActorDesc = *(static_cast<FHLODActorDesc*>(ParentActorDesc.Get()));
				if (CurrentHLODActorDesc.EqualsUnbuilt(ParentHLODActorDesc))
				{
					ParentRebuildDecision = UHLODRebuildPolicy::Evaluate(FoundActor, FoundActor->GetHLODRebuildPolicyDataSet(), InNewData);
				}
			}

			ResetLoaders(TempPackage);
		}
	}

	const bool bOldHLODMatch = RebuildDecision == EHLODRebuildPolicyDecision::RejectRebuild;
	const bool bParentHLODMatch = ParentRebuildDecision.IsSet() && ParentRebuildDecision.GetValue() == EHLODRebuildPolicyDecision::RejectRebuild;
	EvaluationStats.ReusedFromParentBranch += bParentHLODMatch ? 1 : 0;

	// Abort build if old vs new HLOD match, or if the parent vs new match
	const bool bAbortBuild = bOldHLODMatch || bParentHLODMatch;

	// Copy parent if:
	// -> Actor changed, but matches parent (safe to reuse)
	// -> Content match in old vs parent, but file content differs (check MD5)... we want identical data as much as possible to reduce patch sizes between versions
	const bool bCopyParent = bParentHLODMatch && (!bOldHLODMatch || FMD5Hash::HashFile(*LocalFilePathCurrent) != FMD5Hash::HashFile(*LocalFilePathParent));
	if (bCopyParent)
	{
		ParentBranchHLODFileToCopy = LocalFilePathParent;
	}
	else
	{
		// Clean up temp file
		IFileManager::Get().DeleteDirectory(*TempFolder, /*RequireExists=*/false, /*Tree=*/true);
	}

		
	EvaluationStats.Built -= RebuildDecision == EHLODRebuildPolicyDecision::ApproveRebuild && bAbortBuild ? 1 : 0;

	UE_CLOGF(bCopyParent, LogWorldPartitionHLODsBuilder, Display, "Skipping HLOD build - Copying HLOD actor file from parent branch: \"%ls\"", *DepotFilePathParent);
	UE_CLOGF(!bCopyParent && bParentHLODMatch, LogWorldPartitionHLODsBuilder, Display, "Skipping HLOD build - Identical to actor file in parent branch: \"%ls\"", *DepotFilePathParent);
	UE_CLOGF(!bParentHLODMatch && bOldHLODMatch, LogWorldPartitionHLODsBuilder, Display, "Skipping HLOD build - HLOD actor doesn't need to be rebuilt: \"%ls\"", *HLODActor->GetActorLabel());
	
	return bAbortBuild ? EHLODRebuildPolicyDecision::RejectRebuild : EHLODRebuildPolicyDecision::ApproveRebuild;
}

void UWorldPartitionHLODsBuilder::CopyParentBranchHLODFile(UPackage* HLODActorPackage)
{
	ResetLoaders(HLODActorPackage);

	SourceControlHelper->Copy(ParentBranchHLODFileToCopy, USourceControlHelpers::PackageFilename(HLODActorPackage));

	// Clean up temp file
	IFileManager::Get().DeleteDirectory(*FPaths::GetPath(ParentBranchHLODFileToCopy), /*RequireExists=*/false, /*Tree=*/true);

	ParentBranchHLODFileToCopy.Empty();
}

#undef LOCTEXT_NAMESPACE
