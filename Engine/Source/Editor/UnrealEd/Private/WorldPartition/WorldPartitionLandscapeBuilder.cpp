// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionLandscapeBuilder.h"

#include "CoreMinimal.h"
#include "HAL/FileManager.h"
#include "Logging/LogMacros.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "Algo/ForEach.h"
#include "UObject/SavePackage.h"
#include "FileHelpers.h"

#include "EngineUtils.h"
#include "EngineModule.h"
#include "SourceControlHelpers.h"

#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "SourceControlOperations.h"

#include "Editor.h"

#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/WorldPartition.h"

#include "LandscapeSubsystem.h"
#include "LandscapeEditTypes.h"

#include "HAL/IConsoleManager.h"
#include "Engine/MapBuildDataRegistry.h"


DEFINE_LOG_CATEGORY_STATIC(LogWorldPartitionLandscapeBuilder, Log, All);

UWorldPartitionLandscapeBuilder::UWorldPartitionLandscapeBuilder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	// Use default cell size or replace if included in command args
	GetParamValue("IterativeCellSize=", IterativeCellSize);
	UE_LOGF(LogWorldPartitionLandscapeBuilder, Display, "IterativeCellSize: %d", IterativeCellSize);
}

bool UWorldPartitionLandscapeBuilder::RequiresCommandletRendering() const
{
	// The Landscape nanite build needs the renderer to generate some data so we need rendering
	return true;
}

UWorldPartitionBuilder::ELoadingMode UWorldPartitionLandscapeBuilder::GetLoadingMode() const
{
	return ELoadingMode::EntireWorld;
}

bool UWorldPartitionLandscapeBuilder::RunInternal(UWorld* World, const FCellInfo& InCellInfo, FPackageSourceControlHelper& PackageHelper)
{
	if (ULandscapeSubsystem* LandscapeSubsystem = World->GetSubsystem<ULandscapeSubsystem>())
	{
		// Builds grass maps, physical materials, and nanite components
		LandscapeSubsystem->BuildAll(UE::Landscape::EBuildFlags::WriteFinalLog);
		return true;
	}
	else
	{
		UE_LOGF(LogWorldPartitionLandscapeBuilder, Error, "Failed to retrieve ULandscapeSubsystem.");
		return false;
	}
}

bool UWorldPartitionLandscapeBuilder::PostRun(UWorld* World, FPackageSourceControlHelper& PackageHelper, const bool bInRunSuccess)
{
	Super::PostRun(World, PackageHelper, bInRunSuccess);
	if (bInRunSuccess)
	{
		TArray<UPackage*> OutDirtyPackages;
		UEditorLoadingAndSavingUtils::GetDirtyMapPackages(OutDirtyPackages);
		if (UWorldPartitionBuilder::SavePackages(OutDirtyPackages, PackageHelper, /*bErrorsAsWarnings =*/ true))
		{
			OutDirtyPackages.Empty();
			return true;
		}
	}
	return false;
}