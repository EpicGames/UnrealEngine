// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionLandscapeSplineMeshesBuilder.h"

#include "PackageSourceControlHelper.h"
#include "SourceControlHelpers.h"
#include "HAL/PlatformFileManager.h"
#include "UObject/Linker.h"
#include "UObject/SavePackage.h"
#include "UObject/ScriptInterface.h"
#include "UObject/GCObjectScopeGuard.h"
#include "Algo/Transform.h"
#include "Engine/World.h"
#include "Engine/StaticMesh.h"
#include "ActorFolder.h"
#include "GameFramework/Actor.h"
#include "GameFramework/WorldSettings.h"
#include "Components/StaticMeshComponent.h"
#include "Components/SplineMeshComponent.h"
#include "ControlPointMeshComponent.h"
#include "LandscapeSplinesComponent.h"
#include "LandscapeSplineControlPoint.h"
#include "LandscapeSplineMeshesActor.h"
#include "LandscapeSplineActor.h"
#include "LandscapeInfo.h"
#include "Landscape.h"
#include "StaticMeshComponentLODInfo.h"
#include "StaticMeshResources.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "ActorPartition/ActorPartitionSubsystem.h"
#include "WorldPartition/HLOD/HLODLayer.h"
#include "Rendering/ColorVertexBuffer.h"
#include "EngineUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionLandscapeSplineMeshesBuilder)

DEFINE_LOG_CATEGORY_STATIC(LogWorldPartitionLandscapeSplineMeshBuilder, All, All);

static const FName CreatedFromBuilderTag(TEXT("CreatedFromWorldPartitionLandscapeSplineMeshBuilder"));

UWorldPartitionLandscapeSplineMeshesBuilder::UWorldPartitionLandscapeSplineMeshesBuilder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, NewGridSize(0)
{
	GetParamValue("NewGridSize=", NewGridSize);
}

FName UWorldPartitionLandscapeSplineMeshesBuilder::GetSplineCollisionProfileName(const UStaticMeshComponent* InMeshComponent)
{
	FName CollisionProfileName = InMeshComponent->GetCollisionProfileName();

	// Get collision profile name
	if (ILandscapeSplineInterface* LSSplineInterface = Cast<ILandscapeSplineInterface>(InMeshComponent->GetOwner()))
	{
		UObject* ComponentOwner = LSSplineInterface->GetSplinesComponent()->GetOwnerForMeshComponent(InMeshComponent);
		if (ComponentOwner)
		{
			if (ULandscapeSplineControlPoint* ControlPoint = Cast<ULandscapeSplineControlPoint>(ComponentOwner))
			{
				CollisionProfileName = ControlPoint->GetCollisionProfileName();
			}
			else if (ULandscapeSplineSegment* SplineSegment = Cast<ULandscapeSplineSegment>(ComponentOwner))
			{
				CollisionProfileName = SplineSegment->GetCollisionProfileName();
			}
		}
	}

	return CollisionProfileName;
}

int32 UWorldPartitionLandscapeSplineMeshesBuilder::HashStaticMeshComponent(const UStaticMeshComponent* InComponent)
{
	const AActor* Owner = InComponent->GetOwner();

	// Here we use ToCompactString as it internally clamps to 2 decimal digits, or else we get error precision and hash will differ)
	uint32 HashValue = GetTypeHash(InComponent->GetComponentLocation().ToCompactString());
	HashValue = HashCombine(HashValue, GetTypeHash(InComponent->GetComponentScale().ToCompactString()));
	HashValue = HashCombine(HashValue, GetTypeHash(InComponent->GetComponentRotation().GetNormalized().ToCompactString()));
	HashValue = HashCombine(HashValue, GetTypeHash(GetSplineCollisionProfileName(InComponent)));
	HashValue = HashCombine(HashValue, GetTypeHash(InComponent->CastShadow));
	HashValue = HashCombine(HashValue, GetTypeHash(InComponent->GetStaticMesh()));
	HashValue = HashCombine(HashValue, GetTypeHash(InComponent->LDMaxDrawDistance));
	HashValue = HashCombine(HashValue, GetTypeHash(InComponent->TranslucencySortPriority));

	// Hash VT settings
	for (const URuntimeVirtualTexture* RVT : InComponent->RuntimeVirtualTextures)
	{
		HashValue = HashCombine(HashValue, GetTypeHash(RVT));
	}
	HashValue = HashCombine(HashValue, GetTypeHash(InComponent->VirtualTextureRenderPassType));
	HashValue = HashCombine(HashValue, GetTypeHash(InComponent->VirtualTextureLodBias));
	HashValue = HashCombine(HashValue, GetTypeHash(InComponent->VirtualTextureCullMips));

	// Hash vertex colors
	int32 NumLODs = InComponent->GetStaticMesh()->GetNumLODs();
	for (int32 CurrentLOD = 0; CurrentLOD != NumLODs; CurrentLOD++)
	{
		FStaticMeshLODResources& SourceLODModel = InComponent->GetStaticMesh()->GetRenderData()->LODResources[CurrentLOD];
		if (InComponent->LODData.IsValidIndex(CurrentLOD))
		{
			const FStaticMeshComponentLODInfo& SourceLODInfo = InComponent->LODData[CurrentLOD];
			if (SourceLODInfo.OverrideVertexColors != NULL)
			{
				// Copy vertex colors from source to target.
				FColorVertexBuffer* SourceColorBuffer = SourceLODInfo.OverrideVertexColors;

				for (uint32 ColorVertexIndex = 0; ColorVertexIndex < SourceColorBuffer->GetNumVertices(); ColorVertexIndex++)
				{
					HashValue = HashCombine(HashValue, GetTypeHash(SourceColorBuffer->VertexColor(ColorVertexIndex)));
				}
			}
		}
	}

	// Hash used materials
	for (int32 i = 0; i < InComponent->GetNumMaterials(); ++i)
	{
		if (UMaterialInterface* Material = InComponent->GetMaterial(i))
		{
			HashValue = HashCombine(HashValue, GetTypeHash(Material->GetFName()));
		}
	}

	// Hash spline mesh component params
	if (const USplineMeshComponent* SplineMeshComponent = Cast<USplineMeshComponent>(InComponent))
	{
		HashValue = HashCombine(HashValue, GetTypeHash(SplineMeshComponent->GetForwardAxis()));
		HashValue = HashCombine(HashValue, GetTypeHash(SplineMeshComponent->GetStartPosition()));
		HashValue = HashCombine(HashValue, GetTypeHash(SplineMeshComponent->GetStartTangent()));
		HashValue = HashCombine(HashValue, GetTypeHash(SplineMeshComponent->GetEndPosition()));
		HashValue = HashCombine(HashValue, GetTypeHash(SplineMeshComponent->GetEndTangent()));
		HashValue = HashCombine(HashValue, GetTypeHash(SplineMeshComponent->GetStartOffset()));
		HashValue = HashCombine(HashValue, GetTypeHash(SplineMeshComponent->GetEndOffset()));
		HashValue = HashCombine(HashValue, GetTypeHash(SplineMeshComponent->GetStartScale()));
		HashValue = HashCombine(HashValue, GetTypeHash(SplineMeshComponent->GetEndScale()));
		HashValue = HashCombine(HashValue, GetTypeHash(SplineMeshComponent->GetStartRoll()));
		HashValue = HashCombine(HashValue, GetTypeHash(SplineMeshComponent->GetEndRoll()));
		HashValue = HashCombine(HashValue, GetTypeHash(SplineMeshComponent->GetSplineUpDir()));

		HashValue = HashCombine(HashValue, GetTypeHash(SplineMeshComponent->VirtualTextureMainPassMaxDrawDistance));
	}
	// Hash control point mesh component params
	else if (const UControlPointMeshComponent* ControlPointMeshComponent = Cast<UControlPointMeshComponent>(InComponent))
	{
		HashValue = HashCombine(HashValue, GetTypeHash(ControlPointMeshComponent->VirtualTextureMainPassMaxDrawDistance));
	}

	// Hash the owning actor's tags
	if (Owner)
	{
		TArray<FName> SortedTags = Owner->Tags;
		SortedTags.Sort(FNameLexicalLess());
		for (const FName& ActorTag : SortedTags)
		{
			if (!ActorTag.IsEqual(CreatedFromBuilderTag))
			{
				HashValue = HashCombine(HashValue, GetTypeHash(ActorTag));
			}
		}
	}

	// Hash HLOD relevancy
	const bool bOwnerHLODRelevant = !Owner || Owner->bEnableAutoLODGeneration;
	HashValue = HashCombine(HashValue, !!InComponent->bEnableAutoLODGeneration && bOwnerHLODRelevant);

	return HashValue;
}

void UWorldPartitionLandscapeSplineMeshesBuilder::CloneStaticMeshComponent(const UStaticMeshComponent* InSrcMeshComponent, UStaticMeshComponent* DstMeshComponent)
{
	check(InSrcMeshComponent);
	check(DstMeshComponent);

	const AActor* Owner = InSrcMeshComponent->GetOwner();

	// Clone a USplineMeshComponent
	if (const USplineMeshComponent* SplineMeshComponent = Cast<USplineMeshComponent>(InSrcMeshComponent))
	{
		USplineMeshComponent* DstSplineMeshComponent = CastChecked<USplineMeshComponent>(DstMeshComponent);
		DstSplineMeshComponent->bAllowSplineEditingPerInstance = true;
		// Copy spline settings
		DstSplineMeshComponent->SetForwardAxis(SplineMeshComponent->GetForwardAxis(), false);
		DstSplineMeshComponent->SetStartAndEnd(SplineMeshComponent->GetStartPosition(), SplineMeshComponent->GetStartTangent(), SplineMeshComponent->GetEndPosition(), SplineMeshComponent->GetEndTangent(), false);
		DstSplineMeshComponent->SetStartOffset(SplineMeshComponent->GetStartOffset(), false);
		DstSplineMeshComponent->SetEndOffset(SplineMeshComponent->GetEndOffset(), false);
		DstSplineMeshComponent->SetStartScale(SplineMeshComponent->GetStartScale(), false);
		DstSplineMeshComponent->SetEndScale(SplineMeshComponent->GetEndScale(), false);
		DstSplineMeshComponent->SetStartRoll(SplineMeshComponent->GetStartRoll(), false);
		DstSplineMeshComponent->SetEndRoll(SplineMeshComponent->GetEndRoll(), false);
		DstSplineMeshComponent->SetSplineUpDir(SplineMeshComponent->GetSplineUpDir(), false);
		DstSplineMeshComponent->VirtualTextureMainPassMaxDrawDistance = SplineMeshComponent->VirtualTextureMainPassMaxDrawDistance;
	}
	// Clone a UControlPointMeshComponent
	else if (const UControlPointMeshComponent* ControlPointMeshComponent = Cast<UControlPointMeshComponent>(InSrcMeshComponent))
	{
		UControlPointMeshComponent* DstControlPointMeshComponent = CastChecked<UControlPointMeshComponent>(DstMeshComponent);
		DstControlPointMeshComponent->VirtualTextureMainPassMaxDrawDistance = ControlPointMeshComponent->VirtualTextureMainPassMaxDrawDistance;
	}

	// Setup component transform
	DstMeshComponent->SetWorldTransform(InSrcMeshComponent->GetComponentTransform());

	// Copy collision profile
	DstMeshComponent->SetCollisionProfileName(GetSplineCollisionProfileName(InSrcMeshComponent));

	// Shadow
	DstMeshComponent->SetCastShadow(InSrcMeshComponent->CastShadow);

	// Copy mesh
	DstMeshComponent->SetStaticMesh(InSrcMeshComponent->GetStaticMesh());
	if (DstMeshComponent->IsA<USplineMeshComponent>())
	{
		CastChecked<USplineMeshComponent>(DstMeshComponent)->UpdateMesh();
	}

	// Max draw distance
	DstMeshComponent->LDMaxDrawDistance = InSrcMeshComponent->LDMaxDrawDistance;

	// Translucency sort priority
	DstMeshComponent->TranslucencySortPriority = InSrcMeshComponent->TranslucencySortPriority;

	// Copy VT settings
	DstMeshComponent->RuntimeVirtualTextures = InSrcMeshComponent->RuntimeVirtualTextures;
	DstMeshComponent->VirtualTextureRenderPassType = InSrcMeshComponent->VirtualTextureRenderPassType;
	DstMeshComponent->VirtualTextureLodBias = InSrcMeshComponent->VirtualTextureLodBias;
	DstMeshComponent->VirtualTextureCullMips = InSrcMeshComponent->VirtualTextureCullMips;

	// Copy vertex colors
	DstMeshComponent->CopyInstanceVertexColorsIfCompatible(InSrcMeshComponent);

	// Copy assigned materials
	for (int32 i = 0; i < InSrcMeshComponent->GetNumMaterials(); ++i)
	{
		DstMeshComponent->SetMaterial(i, InSrcMeshComponent->GetMaterial(i));
	}

	// Copy the owning actor's tags
	if (Owner)
	{
		DstMeshComponent->ComponentTags.Append(Owner->Tags);
	}

	// HLOD relevant
	const bool bOwnerHLODRelevant = !Owner || Owner->bEnableAutoLODGeneration;
	DstMeshComponent->bEnableAutoLODGeneration = !!InSrcMeshComponent->bEnableAutoLODGeneration && bOwnerHLODRelevant;
}

void UWorldPartitionLandscapeSplineMeshesBuilder::CloneStaticMeshComponentInActor(ALandscapeSplineMeshesActor* InActor, const UStaticMeshComponent* InMeshComponent)
{
	check(InActor);
	check(InMeshComponent->IsA<USplineMeshComponent>() || InMeshComponent->IsA<UControlPointMeshComponent>());
	UStaticMeshComponent* NewMeshComponent = nullptr;
	if (InMeshComponent->IsA<USplineMeshComponent>())
	{
		NewMeshComponent = CastChecked<USplineMeshComponent>(InActor->CreateStaticMeshComponent(USplineMeshComponent::StaticClass()));
	}
	else
	{
		NewMeshComponent = CastChecked<UControlPointMeshComponent>(InActor->CreateStaticMeshComponent(UControlPointMeshComponent::StaticClass()));
	}
	CloneStaticMeshComponent(InMeshComponent, NewMeshComponent);
}

ALandscapeSplineMeshesActor* UWorldPartitionLandscapeSplineMeshesBuilder::GetOrCreatePartitionActorForComponent(UWorld* InWorld, const UStaticMeshComponent* InMeshComponent, const FGuid& InLandscapeGuid)
{
	check(InWorld);
	check(InWorld->IsPartitionedWorld());
	check(InMeshComponent);
	check(InMeshComponent->IsA<USplineMeshComponent>() || InMeshComponent->IsA<UControlPointMeshComponent>());
	UActorPartitionSubsystem* ActorPartitionSubsystem = InWorld->GetSubsystem<UActorPartitionSubsystem>();
	check(ActorPartitionSubsystem);

	FGuid GridGuid = InLandscapeGuid;

	AActor* MeshOwner = InMeshComponent->GetOwner();
	UHLODLayer* HLODLayer = nullptr;

	if (const USplineMeshComponent* SplineMeshComponent = Cast<USplineMeshComponent>(InMeshComponent))
	{
		HLODLayer = SplineMeshComponent->GetHLODLayerForSplinePartitioning();
	}
	else if (const UControlPointMeshComponent* ControlPointMeshComponent = Cast<UControlPointMeshComponent>(InMeshComponent))
	{
		HLODLayer = ControlPointMeshComponent->GetHLODLayerForSplinePartitioning();
	}
	else
	{
		checkNoEntry();
	}

	if (InMeshComponent->IsHLODRelevant() && MeshOwner->IsHLODRelevant() && HLODLayer != nullptr)
	{
		// To get a new unique GUID, combine the original GUID with the HLOD Layer package guid
		GridGuid = FGuid::Combine(GridGuid, HLODLayer->GetPackage()->GetPersistentGuid());
	}

	// Create or find the placement partition actor
	auto OnActorCreated = [&GridGuid, HLODLayer](APartitionActor* CreatedPartitionActor)
	{
		if (ALandscapeSplineMeshesActor* LandscapeSplineMeshesActor = CastChecked<ALandscapeSplineMeshesActor>(CreatedPartitionActor))
		{
			LandscapeSplineMeshesActor->SetGridGuid(GridGuid);
			LandscapeSplineMeshesActor->SetHLODLayer(HLODLayer);
		}
	};

	FActorPartitionGetParams Params(
		ALandscapeSplineMeshesActor::StaticClass(),
		true,
		InWorld->PersistentLevel,
		FVector(FVector2D(InMeshComponent->GetComponentLocation()), 0),
		0,
		GridGuid,
		true,
		OnActorCreated);

	return CastChecked<ALandscapeSplineMeshesActor>(ActorPartitionSubsystem->GetActor(Params));
}

void UWorldPartitionLandscapeSplineMeshesBuilder::FilterStaticMeshComponents(TArray<UStaticMeshComponent*>& InOutComponents)
{
	// Remove unwanted components from the list
	InOutComponents.RemoveAllSwap([](UStaticMeshComponent* Component)
	{
		// We only care about SplineMeshComponents & ControlPointMeshComponents
		if (USplineMeshComponent* SplineMeshComponent = Cast<USplineMeshComponent>(Component))
		{
			return !SplineMeshComponent->IsRelevantForSplinePartitioning();
		}
		else if (UControlPointMeshComponent* ControlPointMeshComponent = Cast<UControlPointMeshComponent>(Component))
		{
			return !ControlPointMeshComponent->IsRelevantForSplinePartitioning();
		}

		return true;
	});
}

bool UWorldPartitionLandscapeSplineMeshesBuilder::RunInternal(UWorld* InWorld, const FCellInfo& InCellInfo, FPackageSourceControlHelper& PackageHelper)
{			
	UWorldPartition* WorldPartition = InWorld->GetWorldPartition();
	if (!WorldPartition)
	{
		UE_LOGF(LogWorldPartitionLandscapeSplineMeshBuilder, Error, "World is not partitioned.");
		return false;
	}

	TSet<UPackage*> PackagesToSave;

	auto AddObjectToSaveIfDirty = [&PackagesToSave](UObject* Object)
	{
		UPackage* Package = Object->GetPackage();
		if (Package->IsDirty())
		{
			PackagesToSave.Add(Package);
		}
	};

	// Update world settings if necessary
	if (NewGridSize != 0 && (InWorld->GetWorldSettings()->LandscapeSplineMeshesGridSize != NewGridSize))
	{
		UE_LOGF(LogWorldPartitionLandscapeSplineMeshBuilder, Display, "Changing Landscape Spline Meshes Grid Size from %d to %d", InWorld->GetWorldSettings()->LandscapeSplineMeshesGridSize, NewGridSize);
		InWorld->GetWorldSettings()->Modify();
		InWorld->GetWorldSettings()->LandscapeSplineMeshesGridSize = NewGridSize;
		AddObjectToSaveIfDirty(InWorld->GetWorldSettings());
	}
		
	FWorldPartitionHelpers::FForEachActorWithLoadingParams ForEachActorWithLoadingParams;
	ForEachActorWithLoadingParams.ActorClasses = { ALandscapeSplineMeshesActor::StaticClass(), ALandscapeSplineActor::StaticClass() };

	TArray<FWorldPartitionReference> ActorReferences;
	TSet<ALandscapeSplineMeshesActor*> PreviousGeneratedActors;

	// Preload all ALandscapeSplineMeshesActor and ALandscapeSplineActor
	FWorldPartitionHelpers::ForEachActorWithLoading(WorldPartition, [&ActorReferences, &PreviousGeneratedActors, WorldPartition](const FWorldPartitionActorDescInstance* ActorDescInstance)
	{
		ActorReferences.Emplace(WorldPartition, ActorDescInstance->GetGuid());
		AActor* Actor = ActorDescInstance->GetActor();
		if (IsValid(Actor))
		{
			if (ALandscapeSplineMeshesActor* LandscapeSplineMeshesActor = Cast<ALandscapeSplineMeshesActor>(Actor))
			{
				if (LandscapeSplineMeshesActor->Tags.Contains(CreatedFromBuilderTag))
				{
					PreviousGeneratedActors.Add(LandscapeSplineMeshesActor);
				}
			}
		}
		return true;
	}, ForEachActorWithLoadingParams);

	TMap<ALandscapeSplineMeshesActor*, TArray<UStaticMeshComponent*>> GeneratedActorComponents;
	TSet<ALandscapeSplineMeshesActor*> ActorsToValidate;
	TSet<ALandscapeSplineMeshesActor*> ActorsToProcess;
	for (TActorIterator<ALandscape> It(InWorld); It; ++It)
	{
		ALandscape* Landscape = *It;
		ULandscapeInfo* LandscapeInfo = IsValid(Landscape) ? Landscape->GetLandscapeInfo() : nullptr;
		if (!LandscapeInfo)
		{
			continue;
		}

		// Reset flag to ensure spline actors/component IsEditorOnly() logic remains valid during the processing below.
		// Otherwise they'll all be considered editor only, which will cause issues such as IsHLODRelevant() returning false.
		bool bLandscapeHadSplineActors = Landscape->GetUseGeneratedLandscapeSplineMeshesActors();
		bool bLandscapeHasSplineActors = false;
		Landscape->SetUseGeneratedLandscapeSplineMeshesActors(false);

		for (TScriptInterface<ILandscapeSplineInterface> SplineOwner : LandscapeInfo->GetSplineActors())
		{
			ALandscapeSplineActor* SplineActor = Cast<ALandscapeSplineActor>(SplineOwner.GetObject());
			if (!SplineActor)
			{
				continue;
			}

			// Only partition splines if they are spatially loaded & not editor only
			if (!SplineActor->ShouldPartitionSpline())
			{
				continue;
			}
						
			TArray<UStaticMeshComponent*> Components;
			SplineActor->GetComponents(Components);

			// For components sources (USplineMeshComponent & UControlPointMeshComponent) to properly return IsEditorOnly, 
			// we need to ensure that old LandscapeSplineActors have their LandscapeActor pointer properly setup
			SplineActor->SetLandscapeActor(Landscape);
			AddObjectToSaveIfDirty(SplineActor);

			// Remove invalid components from the list
			FilterStaticMeshComponents(Components);
			if (Components.IsEmpty())
			{
				continue;
			}

			bLandscapeHasSplineActors = true;

			// Build a list of new actors and existing actors
			for (UStaticMeshComponent* MeshComponent : Components)
			{
				ALandscapeSplineMeshesActor* Actor = GetOrCreatePartitionActorForComponent(InWorld, MeshComponent, LandscapeInfo->LandscapeGuid);
				if (ensure(Actor))
				{
					GeneratedActorComponents.FindOrAdd(Actor).Add(MeshComponent);
					if (PreviousGeneratedActors.Remove(Actor))
					{
						ActorsToValidate.Add(Actor);
					}
					else if (!ActorsToValidate.Contains(Actor))
					{
						ActorsToProcess.Add(Actor);
					}
				}
			}
		}

		if (bLandscapeHasSplineActors)
		{
			Landscape->SetUseGeneratedLandscapeSplineMeshesActors(true);
		}

		if (bLandscapeHasSplineActors != bLandscapeHadSplineActors)
		{
			Landscape->MarkPackageDirty();
			AddObjectToSaveIfDirty(Landscape);
		}
	}

	// Find existing actors that needs to be updated
	for (ALandscapeSplineMeshesActor* ExistingActor : ActorsToValidate)
	{
		TArray<UStaticMeshComponent*> NewComponents = GeneratedActorComponents.FindChecked(ExistingActor);

		// Compute hash for each component that would be part of this actor
		TSet<uint32> NewComponentHashes;
		for (UStaticMeshComponent* Component : NewComponents)
		{
			NewComponentHashes.Add(HashStaticMeshComponent(Component));
		}

		const int32 NewComponentCount = NewComponents.Num();
		if (ALandscapeSplineMeshesActor* LandscapeSplineMeshesActor = Cast<ALandscapeSplineMeshesActor>(ExistingActor))
		{
			bool bDiffers = (LandscapeSplineMeshesActor->GetStaticMeshComponents().Num() != NewComponentCount);
			if (!bDiffers)
			{
				for (const UStaticMeshComponent* Component : LandscapeSplineMeshesActor->GetStaticMeshComponents())
				{
					uint32 HashValue = HashStaticMeshComponent(Component);
					if (NewComponentHashes.Remove(HashValue) != 1)
					{
						bDiffers = true;
						break;
					}
				}
			}
			if (bDiffers)
			{
				LandscapeSplineMeshesActor->ClearStaticMeshComponents();
				ActorsToProcess.Add(ExistingActor);
			}
		}
	}

	static const FName FolderPath(TEXT("Spline Mesh Actors"));

	int32 Progress = 0;
	UE_LOGF(LogWorldPartitionLandscapeSplineMeshBuilder, Display, "Processing Landscape Splines for world %ls", *InWorld->GetName());
	for (ALandscapeSplineMeshesActor* Actor : ActorsToProcess)
	{
		UE_LOGF(LogWorldPartitionLandscapeSplineMeshBuilder, Display, "Processing Splines for %ls [%d/%d]", *Actor->GetName(), ++Progress, ActorsToProcess.Num());

		// Add tag to track actors created from this tool
		Actor->Tags.AddUnique(CreatedFromBuilderTag);
		// Regroup in one folder
		Actor->SetFolderPath(FolderPath);

		TArray<UStaticMeshComponent*> SourceComponents = GeneratedActorComponents.FindChecked(Actor);
		for (UStaticMeshComponent* Component : SourceComponents)
		{
			CloneStaticMeshComponentInActor(Actor, Component);
		}

		check(Actor->GetPackage()->IsDirty());
		AddObjectToSaveIfDirty(Actor);
	}

	if (UActorFolder* ActorFolder = InWorld->PersistentLevel->GetActorFolder(FolderPath))
	{
		AddObjectToSaveIfDirty(ActorFolder);
	}

	// Destroy actors that are no longer needed
	TArray<UPackage*> PackagesToDelete;
	for (ALandscapeSplineMeshesActor* Actor : PreviousGeneratedActors)
	{
		if (UPackage* Package = Actor->GetExternalPackage())
		{
			PackagesToDelete.Add(Package);

			// Releases file handles so packages can be deleted
			ResetLoaders(Package);
		}
		InWorld->DestroyActor(Actor);
	}

	if (IsRunningCommandlet())
	{
		if (!UWorldPartitionBuilder::SavePackages(PackagesToSave.Array(), PackageHelper))
		{
			return false;
		}

		if (!UWorldPartitionBuilder::DeletePackages(PackagesToDelete, PackageHelper))
		{
			return false;
		}

		TArray<FString> ModifiedFiles;
		auto GetPackageFilename = [](const UPackage* InPackage) { return USourceControlHelpers::PackageFilename(InPackage); };
		Algo::Transform(PackagesToDelete, ModifiedFiles, GetPackageFilename);
		Algo::Transform(PackagesToSave, ModifiedFiles, GetPackageFilename);

		const FString ChangeDescription = FString::Printf(TEXT("Rebuilt landscape splines for %s"), *InWorld->GetName());
		if (!OnFilesModified(ModifiedFiles, ChangeDescription))
		{
			return false;
		}
	}

	return true;
}

// Version of builder
bool UWorldPartitionLandscapeSplineMeshesBuilder::RunOnInitializedWorld(UWorld* InWorld)
{
	if (!InWorld || !InWorld->bIsWorldInitialized)
	{
		UE_LOGF(LogWorldPartitionLandscapeSplineMeshBuilder, Error, "World must be initialized.");
		return false;
	}

	UWorldPartitionLandscapeSplineMeshesBuilder* Builder = NewObject<UWorldPartitionLandscapeSplineMeshesBuilder>(GetTransientPackage(), UWorldPartitionLandscapeSplineMeshesBuilder::StaticClass());
	if (!Builder)
	{
		UE_LOGF(LogWorldPartitionLandscapeSplineMeshBuilder, Error, "Failed to create WorldPartitionLandscapeSplineMeshesBuilder.");
		return false;
	}
	
	FPackageSourceControlHelper SCCHelper;
	bool bResult = false;

	{
		FGCObjectScopeGuard BuilderGuard(Builder);
		bResult = Builder->Run(InWorld, SCCHelper);
	}

	return bResult;
}
