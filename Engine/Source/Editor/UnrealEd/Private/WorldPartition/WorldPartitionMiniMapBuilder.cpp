// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionMiniMapBuilder.h"

#include "CoreMinimal.h"
#include "Logging/LogMacros.h"
#include "Misc/CommandLine.h"

#include "AssetCompilingManager.h"
#include "Engine/Texture2D.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "Misc/FileHelper.h"
#include "Factories/TextureFactory.h"
#include "SourceControlHelpers.h"
#include "UObject/SavePackage.h"
#include "UObject/StrongObjectPtr.h"

#include "WorldPartition/ActorDescContainer.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/HLOD/HLODActor.h"
#include "WorldPartition/HLOD/HLODActorDesc.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHandle.h"
#include "WorldPartition/WorldPartitionMiniMap.h"
#include "WorldPartition/WorldPartitionMiniMapHelper.h"
#include "WorldPartition/WorldPartitionMiniMapVolume.h"
#include "LevelInstance/LevelInstanceSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionMiniMapBuilder)

DEFINE_LOG_CATEGORY_STATIC(LogWorldPartitionMiniMapBuilder, All, All);

UWorldPartitionMiniMapBuilder::FOnWarmupTickDelegate UWorldPartitionMiniMapBuilder::OnWarmupTick;

UWorldPartitionMiniMapBuilder::UWorldPartitionMiniMapBuilder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UWorldPartitionMiniMapBuilder::PreRun(UWorld* World, FPackageSourceControlHelper& PackageHelper)
{
	// Find or create the minimap actor
	if (WorldMiniMap == nullptr)
	{
		WorldMiniMap = FWorldPartitionMiniMapHelper::GetWorldPartitionMiniMap(World, true);
		IterativeCellSize = WorldMiniMap->BuilderCellSize;
	}

	if (!WorldMiniMap)
	{
		UE_LOGF(LogWorldPartitionMiniMapBuilder, Error, "Failed to create Minimap. WorldPartitionMiniMap actor not found in the persistent level.");
		return false;
	}

	// Dump bitmaps for debugging purpose
	bDebugCapture = HasParam("DebugCapture");

	// World bounds to process
	IterativeWorldBounds = WorldMiniMap->GetMiniMapWorldBounds();

	//DirectionalLightComponent will return a Bound of size WORLD_MAX which result in a huge non-spatial Bound. The non-spatial bound is returned by GetMiniMapWorldBounds under the following conditions:
	// 1. The WP level has streaming disabled
	// 2. There is no AWorldPartitionMiniMapVolume in the level
	// 3. The level contains an Actor with a DirectionalLightComponent as its only component and root. 
	const FVector WorldSize = IterativeWorldBounds.GetSize();
	if (WorldSize.X >= (WORLD_MAX - 1) || WorldSize.Y >= (WORLD_MAX - 1) || WorldSize.Z >= (WORLD_MAX - 1))
	{
		UE_LOG(LogWorldPartitionMiniMapBuilder, Warning, TEXT("The World BoundingBox is huge %s. Consider adding a AWorldPartitionMiniMapVolume to the level and set it to cover the surface expected to be part of the minimap."), *IterativeWorldBounds.ToString());
	}

	// Compute minimap resolution
	WorldMiniMap->GetMiniMapResolution(MinimapImageSizeX, MinimapImageSizeY, WorldUnitsPerPixel);

	// Create minimap texture
	{
		TStrongObjectPtr<UTextureFactory> Factory(NewObject<UTextureFactory>());
		WorldMiniMap->MiniMapTexture = Factory->CreateTexture2D(WorldMiniMap, TEXT("MinimapTexture"), RF_TextExportTransient);
		WorldMiniMap->MiniMapTexture->PreEditChange(nullptr);
		
		WorldMiniMap->MiniMapTexture->Source.Init(MinimapImageSizeX, MinimapImageSizeY, 1, 1, TSF_BGRA8);
		WorldMiniMap->MiniMapTexture->MipGenSettings = TMGS_SimpleAverage;
		WorldMiniMap->MiniMapWorldBounds = IterativeWorldBounds;
	}

	// Compute world to minimap transform
	{
		WorldToMinimap = FReversedZOrthoMatrix(IterativeWorldBounds.Min.X, IterativeWorldBounds.Max.X, IterativeWorldBounds.Min.Y, IterativeWorldBounds.Max.Y, 1.0f, 0.0f);

		FVector3d Translation(IterativeWorldBounds.Max.X / IterativeWorldBounds.GetSize().X, IterativeWorldBounds.Max.Y / IterativeWorldBounds.GetSize().Y, 0);
		FVector3d Scaling(MinimapImageSizeX, MinimapImageSizeY, 1);

		WorldToMinimap *= FTranslationMatrix(Translation);
		WorldToMinimap *= FScaleMatrix(Scaling);
	}

	IncludedDataLayers = WorldMiniMap->IncludedDataLayers;
	ExcludedDataLayers = WorldMiniMap->ExcludedDataLayers;

	return true;
}

bool UWorldPartitionMiniMapBuilder::RunInternal(UWorld* World, const FCellInfo& InCellInfo, FPackageSourceControlHelper& PackageHelper)
{
	check(World != nullptr);

	// Clamp input bounds
	const FBox ClampedBounds = InCellInfo.Bounds.Overlap(IterativeWorldBounds);

	// World X,Y to minimap X,Y
	const FVector3d MinimapMin = WorldToMinimap.TransformPosition(ClampedBounds.Min);
	const FVector3d MinimapMax = WorldToMinimap.TransformPosition(ClampedBounds.Max);

	int32 XMin = static_cast<int32>(FMath::Max(FMath::Floor(MinimapMin.X), 0));
	int32 YMin = static_cast<int32>(FMath::Max(FMath::Floor(MinimapMin.Y), 0));
	int32 XMax = static_cast<int32>(FMath::Max(FMath::Floor(MinimapMax.X), 0));
	int32 YMax = static_cast<int32>(FMath::Max(FMath::Floor(MinimapMax.Y), 0));

	const FIntVector2 DstMin(XMin, YMin);
	const FIntVector2 DstMax(XMax, YMax);

	const uint32 CaptureWidthPixels = DstMax.X - DstMin.X;
	const uint32 CaptureHeightPixels = DstMax.Y - DstMin.Y;

	// Capture a tile if the region to capture is not empty
	if (CaptureWidthPixels > 0 && CaptureHeightPixels > 0)
	{
		FString TextureName = FString::Format(TEXT("MinimapTile_{0}_{1}_{2}"), { InCellInfo.Location.X, InCellInfo.Location.Y, InCellInfo.Location.Z });

		UTexture2D* TileTexture = NewObject<UTexture2D>(GetTransientPackage(), FName(TextureName), RF_Transient);
		TileTexture->Source.Init(CaptureWidthPixels, CaptureHeightPixels, 1, 1, TSF_BGRA8);
		TileTexture->PowerOfTwoMode = ETexturePowerOfTwoSetting::PadToPowerOfTwo;

		FWorldPartitionMiniMapHelper::CaptureBoundsMiniMapToTexture(World, GetTransientPackage(), CaptureWidthPixels, CaptureHeightPixels, TileTexture, TextureName, ClampedBounds, WorldMiniMap->CaptureSource, WorldMiniMap->CaptureWarmupFrames,
			[World]()
			{
				OnWarmupTick.Broadcast(World);
			});

		// Copy captured image to VT minimap
		const uint32 BPP = TileTexture->Source.GetBytesPerPixel();
		const uint32 CopyWidthBytes = CaptureWidthPixels * BPP;

		const uint8* SrcDataPtr = TileTexture->Source.LockMipReadOnly(0);
		check(SrcDataPtr);

		// PreEditChange was called before in PreRun
		uint8* const MiniMapDstPtr = WorldMiniMap->MiniMapTexture->Source.LockMip(0);
		check(MiniMapDstPtr);

		const uint32 DstDataStrideBytes = WorldMiniMap->MiniMapTexture->Source.GetSizeX() * BPP;
		uint8* const DstDataPtr = MiniMapDstPtr + (DstMin.Y * DstDataStrideBytes) + (DstMin.X * BPP);
		check(DstDataPtr);

		for (uint32 RowIdx = 0; RowIdx < CaptureHeightPixels; ++RowIdx)
		{
			uint8* DstCopy = DstDataPtr + DstDataStrideBytes * RowIdx;
			const uint8* SrcCopy = SrcDataPtr + CopyWidthBytes * RowIdx;
			
			check(DstCopy >= DstDataPtr);
			check(DstCopy + CopyWidthBytes <= DstDataPtr + WorldMiniMap->MiniMapTexture->Source.CalcMipSize(0));

			check(SrcCopy >= SrcDataPtr);
			check(SrcCopy + CopyWidthBytes <= SrcDataPtr + TileTexture->Source.CalcMipSize(0));

			FMemory::Memcpy(DstCopy, SrcCopy, CopyWidthBytes);
		}

		// Write tile bitmap for debugging purpose
		if (bDebugCapture)
		{
			const FString DirectoryPath = FPaths::ConvertRelativePathToFull(FPaths::ProjectIntermediateDir() + TEXT("Minimap"));
			FString MinimapDebugImagePath = DirectoryPath / World->GetName() + "_" + TextureName + ".bmp";
			FFileHelper::CreateBitmap(*MinimapDebugImagePath, TileTexture->Source.GetSizeX(), TileTexture->Source.GetSizeY(), (FColor*)SrcDataPtr);
		}

		WorldMiniMap->MiniMapTexture->Source.UnlockMip(0);
		TileTexture->Source.UnlockMip(0);
	}

	return true;
}

bool UWorldPartitionMiniMapBuilder::PostRun(UWorld* World, FPackageSourceControlHelper& PackageHelper, const bool bInRunSuccess)
{
	if (!bInRunSuccess)
	{
		return false;
	}

	// Make sure all assets and textures are ready
	FAssetCompilingManager::Get().FinishAllCompilation();

	// Finalize texture
	{
		// Write minimap bitmap for debugging purpose
		if (bDebugCapture)
		{
			const FString DirectoryPath = FPaths::ConvertRelativePathToFull(FPaths::ProjectIntermediateDir() + TEXT("Minimap"));
			FString MinimapDebugImagePath = DirectoryPath / World->GetName() + "-Minimap.bmp";

			const void* MiniMapSourcePtr = WorldMiniMap->MiniMapTexture->Source.LockMipReadOnly(0);
			FFileHelper::CreateBitmap(*MinimapDebugImagePath, WorldMiniMap->MiniMapTexture->Source.GetSizeX(), WorldMiniMap->MiniMapTexture->Source.GetSizeY(), (const FColor*)MiniMapSourcePtr);
			WorldMiniMap->MiniMapTexture->Source.UnlockMip(0);
		}

		WorldMiniMap->MiniMapTexture->PowerOfTwoMode = ETexturePowerOfTwoSetting::PadToPowerOfTwo;	// Required for VTs
		WorldMiniMap->MiniMapTexture->AdjustMinAlpha = 1.f;
		WorldMiniMap->MiniMapTexture->LODGroup = TEXTUREGROUP_UI;
		WorldMiniMap->MiniMapTexture->VirtualTextureStreaming = true;
		//WorldMiniMap->MiniMapTexture->UpdateResource(); // @@?? <- UpdateResource before PostEditChange looks wrong
		WorldMiniMap->MiniMapTexture->PostEditChange();
	}

	// Compute relevant UV space for the minimap
	{
		FVector2D TexturePOW2ScaleFactor = FVector2D((float)MinimapImageSizeX / FMath::RoundUpToPowerOfTwo(MinimapImageSizeX),
			(float)MinimapImageSizeY / FMath::RoundUpToPowerOfTwo(MinimapImageSizeY));

		WorldMiniMap->UVOffset.Min = FVector2d(0, 0);
		WorldMiniMap->UVOffset.Max = TexturePOW2ScaleFactor;
		WorldMiniMap->UVOffset.bIsValid = true;
	}

	// Make sure the minimap texture is ready before saving
	FAssetCompilingManager::Get().FinishAllCompilation();

	// Save MiniMap Package
	UPackage* WorldMiniMapExternalPackage = WorldMiniMap->GetExternalPackage();
	FString PackageFileName = SourceControlHelpers::PackageFilename(WorldMiniMapExternalPackage);

	if (!PackageHelper.Checkout(WorldMiniMapExternalPackage))
	{
		UE_LOGF(LogWorldPartitionMiniMapBuilder, Error, "Error checking out package %ls.", *WorldMiniMapExternalPackage->GetName());
		return false;
	}

	FSavePackageArgs SaveArgs;
	SaveArgs.TopLevelFlags = RF_Standalone;
	if (!UPackage::SavePackage(WorldMiniMapExternalPackage, nullptr, *PackageFileName, SaveArgs))
	{
		UE_LOGF(LogWorldPartitionMiniMapBuilder, Error, "Error saving package %ls.", *WorldMiniMapExternalPackage->GetName());
		return false;
	}

	if (!PackageHelper.AddToSourceControl(WorldMiniMapExternalPackage))
	{
		UE_LOGF(LogWorldPartitionMiniMapBuilder, Error, "Error adding package %ls to revision control.", *WorldMiniMapExternalPackage->GetName());
		return false;
	}

	TArray<FString> ModifiedFiles = { PackageFileName };

	// Make sure to delete unneeded minimap actors in case there is more than one in the map
	for (TActorIterator<AWorldPartitionMiniMap> It(World); It; ++It)
	{
		if (*It != WorldMiniMap)
		{
			if (UPackage* ExternalPackage = It->GetExternalPackage())
			{
				PackageHelper.Delete(ExternalPackage);
				ModifiedFiles.Add(SourceControlHelpers::PackageFilename(ExternalPackage));
			}
		}
	}

	const FString ChangeDescription = FString::Printf(TEXT("Rebuilt minimap for %s"), *World->GetName());
	return OnFilesModified(ModifiedFiles, ChangeDescription);
}
