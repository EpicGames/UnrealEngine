// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionNavigationDataBuilder.h"

#include "CoreMinimal.h"
#include "EngineUtils.h"
#include "FileHelpers.h"
#include "StaticMeshCompiler.h"
#include "ActorPartition/ActorPartitionSubsystem.h"
#include "Logging/LogMacros.h"
#include "Misc/CommandLine.h"
#include "UObject/SavePackage.h"
#include "Commandlets/Commandlet.h"
#include "AI/NavigationSystemBase.h"

#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "WorldPartition/DataLayer/DataLayerAsset.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/NavigationData/NavigationDataChunkActor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionNavigationDataBuilder)

DEFINE_LOG_CATEGORY_STATIC(LogWorldPartitionNavigationDataBuilder, Log, All);

UWorldPartitionNavigationDataBuilder::UWorldPartitionNavigationDataBuilder(FVTableHelper& Helper)
	: Super(Helper)
{
}

UWorldPartitionNavigationDataBuilder::UWorldPartitionNavigationDataBuilder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UWorldPartitionNavigationDataBuilder::PreRun(UWorld* World, FPackageSourceControlHelper& PackageHelper)
{
	// Set runtime data layer to be included in the base navmesh generation.
	UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(World);
	for (const TObjectPtr<UDataLayerAsset>& DataLayer : World->GetWorldSettings()->BaseNavmeshDataLayers)
	{
		if (DataLayer != nullptr)
		{
			const UDataLayerInstance* DataLayerInstance = DataLayerManager->GetDataLayerInstance(DataLayer);
			if (DataLayerInstance == nullptr)
			{
				UE_LOGF(LogWorldPartitionNavigationDataBuilder, Error, "Missing UDataLayerInstance for %ls.", *DataLayer->GetName());
			}
			else if (DataLayerInstance->IsRuntime())
			{
				IncludedDataLayers.Add(const_cast<UDataLayerAsset*>(DataLayerInstance->GetAsset()));
			}
		}
	}
	
	const TSubclassOf<APartitionActor>& NavigationDataActorClass = ANavigationDataChunkActor::StaticClass();
	uint32 GridSize = NavigationDataActorClass->GetDefaultObject<APartitionActor>()->GetDefaultGridSize(World);
	GridSize = FMath::Max(GridSize, 1u);

	// Size of loaded cell. Set as big as your hardware can afford.
	const uint32 LoadingCellSizeSetting = World->GetWorldSettings()->NavigationDataBuilderLoadingCellSize;
	IterativeCellSize = GridSize * FMath::Max(LoadingCellSizeSetting / GridSize, 1u);
	
	// Extra padding around loaded cell.
	// Make sure navigation is added and initialized before fetching the build overlap
	FNavigationSystem::AddNavigationSystemToWorld(*World, FNavigationSystemRunMode::EditorWorldPartitionBuildMode);
	IterativeCellOverlapSize = FMath::CeilToInt32(FNavigationSystem::GetWorldPartitionNavigationDataBuilderOverlap(*World));

	bCleanBuilderPackages = HasParam("CleanPackages");

	UE_LOGF(LogWorldPartitionNavigationDataBuilder, Log, "Starting NavigationDataBuilder");
	UE_LOGF(LogWorldPartitionNavigationDataBuilder, Log, "   ANavigationDataChunkActor GridSize: %8i", GridSize);
	UE_LOGF(LogWorldPartitionNavigationDataBuilder, Log, "   IterativeCellSize:                  %8i (%ix%i navigation datachunk partition actor per loaded cell)", IterativeCellSize, IterativeCellSize/GridSize, IterativeCellSize/GridSize);
	UE_LOGF(LogWorldPartitionNavigationDataBuilder, Log, "   IterativeCellOverlapSize:           %8i", IterativeCellOverlapSize);
	
	return true;
}

bool UWorldPartitionNavigationDataBuilder::RunInternal(UWorld* World, const FCellInfo& InCellInfo, FPackageSourceControlHelper& PackageHelper)
{
	UE_LOGF(LogWorldPartitionNavigationDataBuilder, Verbose, " ");
	UE_LOGF(LogWorldPartitionNavigationDataBuilder, Verbose, "============================================================================================================");
	UE_LOGF(LogWorldPartitionNavigationDataBuilder, Verbose, "RunInternal");
	UE_LOGF(LogWorldPartitionNavigationDataBuilder, Verbose, "   Bounds %ls .", *InCellInfo.Bounds.ToString());
	
	UWorldPartitionSubsystem* WorldPartitionSubsystem = World->GetSubsystem<UWorldPartitionSubsystem>();
	check(WorldPartitionSubsystem);

	UWorldPartition* WorldPartition = World->GetWorldPartition();
	check(WorldPartition);
	
	TSet<UPackage*> NavigationDataChunkActorPackages;

	// Check if we are just in cleaning mode.
	if (bCleanBuilderPackages)
	{
		TSet<UPackage*> PackagesToClean;
		for (TActorIterator<ANavigationDataChunkActor> ItActor(World); ItActor; ++ItActor)
		{
			UPackage* Package = ItActor->GetPackage();
			check(!PackagesToClean.Find(Package));

			if (Package)
			{
				const FString Filename = SourceControlHelpers::PackageFilename(Package->GetName());
				if(IPlatformFile::GetPlatformPhysical().FileExists(*Filename))
				{
					PackagesToClean.Add(Package);
				}
				else
				{
					UE_LOGF(LogWorldPartitionNavigationDataBuilder, Verbose, "   Skippping package file already deleted: %ls", *Package->GetName());
				}
			}
		}

		UE_LOGF(LogWorldPartitionNavigationDataBuilder, Verbose, "   Number of packages to clear: %i", PackagesToClean.Num());

		// Just delete all ANavigationDataChunkActor packages
		if (!DeletePackages(PackageHelper, PackagesToClean.Array()))
		{
			UE_LOGF(LogWorldPartitionNavigationDataBuilder, Error, "Error deleting packages.");
		}

		// If we had packages to delete we need to notify the delete after the save. Else WP might try to load the deleted descriptors on the next iteration.
		// Note: this notification is expected to be done by the SavePackages()
		for (UPackage* Package : PackagesToClean)
		{
			WorldPartition->OnPackageDeleted(Package);
		}

		return true;
	}

	// Gather all packages before any navigation data chunk actors are deleted
	for (TActorIterator<ANavigationDataChunkActor> ItActor(World); ItActor; ++ItActor)
	{
		NavigationDataChunkActorPackages.Add(ItActor->GetPackage());
	}

	// Destroy any existing navigation data chunk actors within bounds we are generating, we will make new ones.
	int32 Count = 0;
	const FBox GeneratingBounds = InCellInfo.Bounds.ExpandBy(-IterativeCellOverlapSize);
	UE_LOGF(LogWorldPartitionNavigationDataBuilder, Verbose, "   GeneratingBounds %ls", *GeneratingBounds.ToString());
	
	for (TActorIterator<ANavigationDataChunkActor> It(World); It; ++It)
	{
		Count++;
		
		ANavigationDataChunkActor* Actor = *It;
		const FVector Location = Actor->GetActorLocation();
		
		auto IsInside2D = [](const FBox Bounds, const FVector& In) -> bool
		{
			return ((In.X >= Bounds.Min.X) && (In.X < Bounds.Max.X) && (In.Y >= Bounds.Min.Y) && (In.Y < Bounds.Max.Y));
		};

		UE_LOGF(LogWorldPartitionNavigationDataBuilder, Verbose, "   Location %ls %ls (%ls %ls)",
			*Location.ToCompactString(), IsInside2D(GeneratingBounds, Location) ? TEXT("inside") : TEXT("outside"), *Actor->GetName(), *Actor->GetPackage()->GetName());
		
		if (IsInside2D(GeneratingBounds, Location))
		{
			UE_LOGF(LogWorldPartitionNavigationDataBuilder, Verbose, "   Destroy actor %ls in package %ls.", *Actor->GetName(), *Actor->GetPackage()->GetName());
			const bool bDestroyed = World->DestroyActor(Actor);
			if (!bDestroyed)
			{
				UE_LOGF(LogWorldPartitionNavigationDataBuilder, Warning, "      Could not be destroy %ls.", *Actor->GetName());
			}
		}
	}
	UE_LOGF(LogWorldPartitionNavigationDataBuilder, Verbose, "   Number of ANavigationDataChunkActor: %i", Count);

	// Make sure static meshes have compiled before generating navigation data
	FStaticMeshCompilingManager::Get().FinishAllCompilation();
	
	// Rebuild ANavigationDataChunkActor in loaded bounds
	GenerateNavigationData(WorldPartition, InCellInfo.Bounds, GeneratingBounds);

	// Gather all packages again to include newly created ANavigationDataChunkActor actors
	for (TActorIterator<ANavigationDataChunkActor> ItActor(World); ItActor; ++ItActor)
	{
		NavigationDataChunkActorPackages.Add(ItActor->GetPackage());

		// Log
		FString String;
		if (ItActor->GetPackage())
		{
			String += ItActor->GetPackage()->GetName();
			String += UPackage::IsEmptyPackage(ItActor->GetPackage()) ? " empty" : " ";
			String += ItActor->GetPackage()->IsDirty() ? " dirty" : " ";
		}
		UE_LOGF(LogWorldPartitionNavigationDataBuilder, Verbose, "   Adding package %ls (from actor %ls).", *String, *ItActor->GetName());
	}

	TArray<UPackage*> PackagesToSave;
	
	TArray<UPackage*> PackagesToAdd;
	TArray<UPackage*> PackagesToDelete;

	for (UPackage* ActorPackage : NavigationDataChunkActorPackages)
	{
		// Only change package that have been dirtied
		if (ActorPackage && ActorPackage->IsDirty())
		{
			if (UPackage::IsEmptyPackage(ActorPackage))
			{
				const FString Filename = SourceControlHelpers::PackageFilename(ActorPackage->GetName());
				if(IPlatformFile::GetPlatformPhysical().FileExists(*Filename))
				{
					PackagesToDelete.Add(ActorPackage);
				}
				else
				{
					UE_LOGF(LogWorldPartitionNavigationDataBuilder, Verbose, "   Skippping package file already deleted: %ls", *ActorPackage->GetName());
				}
			}
			else
			{
				PackagesToAdd.Add(ActorPackage);
				PackagesToSave.Add(ActorPackage);
			}
		}
	}

	// Delete packages
	if (!DeletePackages(PackageHelper, PackagesToDelete))
	{
		UE_LOGF(LogWorldPartitionNavigationDataBuilder, Error, "Error deleting packages.");
		return true;
	}

	// Save packages
	if (!PackagesToSave.IsEmpty())
	{
		{
			// Checkout or remove read-only for packages to add
			TRACE_CPUPROFILER_EVENT_SCOPE(CheckoutPackages);
			UE_LOGF(LogWorldPartitionNavigationDataBuilder, Log, "Checking out %d packages.", PackagesToAdd.Num());

			if (PackageHelper.UseSourceControl())
			{
				FEditorFileUtils::CheckoutPackages(PackagesToAdd, /*OutPackagesCheckedOut*/nullptr, /*bErrorIfAlreadyCheckedOut*/false);
			}
			else
			{
				// Remove read-only
				for (const UPackage* Package : PackagesToAdd)
				{
					const FString PackageFilename = SourceControlHelpers::PackageFilename(Package);
					if (IPlatformFile::GetPlatformPhysical().FileExists(*PackageFilename))
					{
						if (!IPlatformFile::GetPlatformPhysical().SetReadOnly(*PackageFilename, /*bNewReadOnlyValue*/false))
						{
							UE_LOGF(LogWorldPartitionNavigationDataBuilder, Error, "Error setting %ls writable", *PackageFilename);
							return true;
						}
					}
				}
			}
		}

		if (!SavePackages(PackagesToSave))
		{
			return true;
		}

		// If we had packages to delete we need to notify the delete after the save. Else WP might try to load the deleted descriptors on the next iteration.
		// Note: this notification is expected to be done by the SavePackages()
		for (UPackage* Package : PackagesToDelete)
		{
			WorldPartition->OnPackageDeleted(Package);
		}
		
		if (PackageHelper.UseSourceControl())
		{
			{
				// Add new packages to source control
				TRACE_CPUPROFILER_EVENT_SCOPE(AddingToSourceControl);
				UE_LOGF(LogWorldPartitionNavigationDataBuilder, Log, "Adding packages to revision control.");

				if (!PackageHelper.AddToSourceControl(PackagesToAdd))
				{
					UE_LOGF(LogWorldPartitionNavigationDataBuilder, Error, "Error adding packages.");
					return true;
				}
			}

			// Calculate AddedPackagesToSubmit, also adding an entry to AddedPackagesToSubmitMap as when we process later WP cells they may have PackagesToDelete
			// that have already been added to AddedPackagesToSubmit (which we need to remove) and we want to avoid lots of string compares.
			AddedPackagesToSubmit.Reserve(AddedPackagesToSubmit.Num() + PackagesToAdd.Num());
			AddedPackagesToSubmitMap.Reserve(AddedPackagesToSubmitMap.Num() + PackagesToAdd.Num());

			for (const UPackage* Package : PackagesToAdd)
			{
				FString PackageFilename = SourceControlHelpers::PackageFilename(Package);
			
				checkSlow(AddedPackagesToSubmit.Find(PackageFilename) == INDEX_NONE);
				checkSlow(DeletedPackagesToSubmit.Find(PackageFilename) == INDEX_NONE);

				const int32 Idx = AddedPackagesToSubmit.Add(PackageFilename);
				AddedPackagesToSubmitMap.Add(MoveTemp(PackageFilename), Idx);
			}
		}
	}

	if (PackageHelper.UseSourceControl())
	{
		DeletedPackagesToSubmit.Reserve(DeletedPackagesToSubmit.Num() + PackagesToDelete.Num());

		for (const UPackage* Package : PackagesToDelete)
		{
			const FString PackageFilename = SourceControlHelpers::PackageFilename(Package);

			checkSlow(DeletedPackagesToSubmit.Find(PackageFilename) == INDEX_NONE);

			const int32* AddedIdx = AddedPackagesToSubmitMap.Find(PackageFilename);
			
			if (AddedIdx != nullptr)
			{
				// Dont remove the entry completely or the Indices in AddedPackagesToSubmitMap will point to the wrong place.
				AddedPackagesToSubmit[*AddedIdx] = FString(TEXT(""));
				const int32 NumRemoved = AddedPackagesToSubmitMap.Remove(PackageFilename);
				ensure(NumRemoved == 1);
			}
			else
			{
				DeletedPackagesToSubmit.Add(PackageFilename);
			}
		}
	}

	UPackage::WaitForAsyncFileWrites();

	return true;
}

bool UWorldPartitionNavigationDataBuilder::PostRun(UWorld* World, FPackageSourceControlHelper& PackageHelper, const bool bInRunSuccess)
{
	Super::PostRun(World, PackageHelper, bInRunSuccess);
	
	const FString ChangeDescription = FString::Printf(TEXT("Rebuilt navigation data for %s"), *World->GetName());
	TArray<FString> PackagesToSubmit;

	PackagesToSubmit.Reserve(AddedPackagesToSubmit.Num() + DeletedPackagesToSubmit.Num());

	// DeletedPackages may never have been added to source control so don't attempt to check them in!
	PackageHelper.GetMarkedForDeleteFiles(DeletedPackagesToSubmit, PackagesToSubmit);

	for (const FString& Filename : AddedPackagesToSubmit)
	{
		// Empty Filenames indicate the file was added then deleted / reverted.
		if (Filename != FString(TEXT("")))
		{
			PackagesToSubmit.Add(Filename);
		}
	}

	DeletedPackagesToSubmit.Empty();
	AddedPackagesToSubmit.Empty();
	AddedPackagesToSubmitMap.Empty();

	if (!OnFilesModified(PackagesToSubmit, ChangeDescription))
	{
		UE_LOGF(LogWorldPartitionNavigationDataBuilder, Error, "Error auto-submitting packages.");
		return true;
	}

	return true;
}

FName GetCellName(const UWorldPartition* WorldPartition, const UActorPartitionSubsystem::FCellCoord& InCellCoord)
{
	const FString PackageName = FPackageName::GetShortName(WorldPartition->GetPackage());
	const FString PackageNameNoPIEPrefix = UWorld::RemovePIEPrefix(PackageName);
	return FName(*FString::Printf(TEXT("%s_%lld_%lld"), *PackageNameNoPIEPrefix, InCellCoord.X, InCellCoord.Y));
}

bool UWorldPartitionNavigationDataBuilder::GenerateNavigationData(UWorldPartition* WorldPartition, const FBox& LoadedBounds, const FBox& GeneratingBounds) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionNavigationDataBuilder::GenerateNavigationData);

	static int32 CallCount = 0;
	CallCount++;
	UE_LOGF(LogWorldPartitionNavigationDataBuilder, Log, "Iteration %i. GenerateNavigationData for LoadedBounds %ls", CallCount, *LoadedBounds.ToString());

	UWorld* World = WorldPartition->GetWorld();

	// Generate navmesh
	// Make sure navigation is added and initialized in EditorWorldPartitionBuildMode
	FNavigationSystem::AddNavigationSystemToWorld(*World, FNavigationSystemRunMode::EditorWorldPartitionBuildMode);

	UNavigationSystemBase* NavSystem = World->GetNavigationSystem();
	if (NavSystem == nullptr)
	{
		UE_LOGF(LogWorldPartitionNavigationDataBuilder, Verbose, "No navigation system to generate navigation data.");
		return false;
	}

	// First check if there is any intersection with the nav bounds.
	const FBox NavBounds = NavSystem->GetNavigableWorldBounds();
	if (!LoadedBounds.Intersect(NavBounds))
	{
		// No intersections, early out
		UE_LOGF(LogWorldPartitionNavigationDataBuilder, Log, "GenerateNavigationData finished (no intersection with nav bounds).");
		return true;
	}

	// Invoke navigation data generator
	NavSystem->SetBuildBounds(LoadedBounds);
	FNavigationSystem::Build(*World);

	// Compute navdata bounds from tiles
	const FBox NavDataBounds = NavSystem->ComputeNavDataBounds();
	if (!LoadedBounds.Intersect(NavDataBounds))
	{
		// No intersections, early out
		UE_LOGF(LogWorldPartitionNavigationDataBuilder, Log, "GenerateNavigationData finished (no intersection with generated navigation data).");
		return true;
	}

	// For each cell, gather navmesh and generate a datachunk actor
	int32 ActorCount = 0;

	// A DataChunkActor will be generated for each tile touching the generating bounds.
	const TSubclassOf<APartitionActor>& NavigationDataActorClass = ANavigationDataChunkActor::StaticClass();

	int32 XMin = static_cast<int32>(GeneratingBounds.Min.X);
	int32 YMin = static_cast<int32>(GeneratingBounds.Min.Y);
	int32 XMax = static_cast<int32>(GeneratingBounds.Max.X);
	int32 YMax = static_cast<int32>(GeneratingBounds.Max.Y);

	const uint32 GridSize = FMath::Max(NavigationDataActorClass->GetDefaultObject<APartitionActor>()->GetDefaultGridSize(World), 1u);

	const FIntRect GeneratingBounds2D(XMin, YMin, XMax, YMax);
	FActorPartitionGridHelper::ForEachIntersectingCell(NavigationDataActorClass, GeneratingBounds2D, World->PersistentLevel,
		[&WorldPartition, &ActorCount, World, &NavDataBounds, GridSize, this](const UActorPartitionSubsystem::FCellCoord& InCellCoord, const FIntRect& InCellBounds)->bool
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MakeNavigationDataChunkActorForGridCell);
		
			const FBox2D CellBounds(FVector2D(InCellBounds.Min), FVector2D(InCellBounds.Max));
			
			if (CellBounds.GetExtent().X < 1.f || CellBounds.GetExtent().Y < 1.f)
			{
				// Safety, since we reduce by 1.f below.
				UE_LOGF(LogWorldPartitionNavigationDataBuilder, Warning, "%ls: grid cell too small.", ANSI_TO_TCHAR(__FUNCTION__));
				return false;
			}

			constexpr float HalfHeight = HALF_WORLD_MAX;
			const FBox QueryBounds(FVector(CellBounds.Min.X, CellBounds.Min.Y, -HalfHeight), FVector(CellBounds.Max.X, CellBounds.Max.Y, HalfHeight));

			if (!NavDataBounds.IsValid || !NavDataBounds.Intersect(QueryBounds))
			{
				// Skip if there is no navdata for this cell
				return true;
			}

			//@todo_ow: Properly handle data layers
			FActorSpawnParameters SpawnParams;
			SpawnParams.bDeferConstruction = true;
			SpawnParams.bCreateActorPackage = true;
			ANavigationDataChunkActor* DataChunkActor = World->SpawnActor<ANavigationDataChunkActor>(SpawnParams);
			ActorCount++;

			DataChunkActor->SetGridSize(GridSize);
			
			const FVector2D CellCenter = CellBounds.GetCenter();
			DataChunkActor->SetActorLocation(FVector(CellCenter.X, CellCenter.Y, 0.f));

			FBox TilesBounds(EForceInit::ForceInit);
			DataChunkActor->CollectNavData(QueryBounds, TilesBounds);

			FBox ChunkActorBounds(FVector(QueryBounds.Min.X, QueryBounds.Min.Y, TilesBounds.Min.Z), FVector(QueryBounds.Max.X, QueryBounds.Max.Y, TilesBounds.Max.Z));
			ChunkActorBounds = ChunkActorBounds.ExpandBy(FVector(-1.f, -1.f, 1.f)); //reduce XY by 1cm to avoid precision issues causing potential overflow on neighboring cell, add 1cm in Z to have a minimum of volume.
			UE_LOGF(LogWorldPartitionNavigationDataBuilder, VeryVerbose, "Setting ChunkActorBounds to %ls", *ChunkActorBounds.ToString());
			DataChunkActor->SetDataChunkActorBounds(ChunkActorBounds);

			const FName CellName = GetCellName(WorldPartition, InCellCoord);
			DataChunkActor->SetActorLabel(FString::Printf(TEXT("NavDataChunkActor_%s"), *CellName.ToString()));

			UE_LOGF(LogWorldPartitionNavigationDataBuilder, Verbose, "%i) %ls added.", ActorCount, *DataChunkActor->GetName());

			return true;
		});

	UE_LOGF(LogWorldPartitionNavigationDataBuilder, Log, "GenerateNavigationData finished (%i actors added).", ActorCount);
	return true;
}

bool UWorldPartitionNavigationDataBuilder::SavePackages(const TArray<UPackage*>& PackagesToSave) const
{
	// Save packages
	TRACE_CPUPROFILER_EVENT_SCOPE(SavingPackages);
	UE_LOGF(LogWorldPartitionNavigationDataBuilder, Log, "Saving %d packages.", PackagesToSave.Num());

	for (UPackage* Package : PackagesToSave)
	{
		UE_LOGF(LogWorldPartitionNavigationDataBuilder, Verbose, "   Saving package  %ls.", *Package->GetName());
		FString PackageFileName = SourceControlHelpers::PackageFilename(Package);
		FSavePackageArgs SaveArgs;
		SaveArgs.TopLevelFlags = RF_Standalone;
		SaveArgs.SaveFlags = SAVE_Async;
		if (!UPackage::SavePackage(Package, nullptr, *PackageFileName, SaveArgs))
		{
			UE_LOGF(LogWorldPartitionNavigationDataBuilder, Error, "   Error saving package %ls.", *Package->GetName());
			return false;
		}
	}

	UPackage::WaitForAsyncFileWrites();

	return true;
}

bool UWorldPartitionNavigationDataBuilder::DeletePackages(const FPackageSourceControlHelper& PackageHelper, const TArray<UPackage*>& PackagesToDelete) const
{
	if (!PackagesToDelete.IsEmpty())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DeletePackages);
		
		UE_LOGF(LogWorldPartitionNavigationDataBuilder, Log, "Deleting %d packages.", PackagesToDelete.Num());
		for (const UPackage* Package : PackagesToDelete)
		{
			UE_LOGF(LogWorldPartitionNavigationDataBuilder, Verbose, "   Deleting package  %ls.", *Package->GetName());
		}
		
		if (!PackageHelper.Delete(PackagesToDelete))
		{
			return false;
		}
	}

	return true;
}
