// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionRenameDuplicateBuilder.h"
#include "WorldPartition/WorldPartitionStreamingGeneration.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "ReferenceCluster.h"
#include "PackageSourceControlHelper.h"
#include "SourceControlHelpers.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "EditorWorldUtils.h"
#include "Misc/CommandLine.h"
#include "UObject/SavePackage.h"
#include "UObject/Linker.h"
#include "UObject/MetaData.h"
#include "UObject/ObjectRedirector.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "HAL/FileManager.h"
#include "WorldPartition/ActorDescContainerInstanceCollection.h"
#include "WorldPartition/ActorDescContainerInstance.h"
#include "WorldPartition/ActorDescContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionRenameDuplicateBuilder)

DEFINE_LOG_CATEGORY_STATIC(LogWorldPartitionRenameDuplicateBuilder, All, All);

class FReplaceObjectRefsArchive : public FArchiveUObject
{
public:
	FReplaceObjectRefsArchive(UObject* InRoot, const TMap<TObjectPtr<UObject>,TObjectPtr<UObject>>& InObjectsToReplace)
		: Root(InRoot)
		, ObjectsToReplace(InObjectsToReplace)
	{
		// Don't gather transient actor references
		SetIsPersistent(true);

		// Don't trigger serialization of compilable assets
		SetShouldSkipCompilingAssets(true);

		ArIgnoreOuterRef = true;
		ArIsObjectReferenceCollector = true;
		ArShouldSkipBulkData = true;
		ArIsModifyingWeakAndStrongReferences = true;

		SubObjects.Add(Root);

		Root->Serialize(*this);
	}

	virtual FArchive& operator<<(UObject*& Obj) override
	{
		if (Obj && !Obj->IsTemplate() && !Obj->HasAnyFlags(RF_Transient))
		{
			bool bWasAlreadyInSet;
			SubObjects.Add(Obj, &bWasAlreadyInSet);

			if (!bWasAlreadyInSet)
			{
				if (const TObjectPtr<UObject>* ReplacementObject = ObjectsToReplace.Find(Obj))
				{
					Obj = *ReplacementObject;
				}
				else if (Obj->IsInOuter(Root))
				{
					Obj->Serialize(*this);
				}
			}
		}
		return *this;
	}

private:
	UObject* Root;
	const TMap<TObjectPtr<UObject>, TObjectPtr<UObject>>& ObjectsToReplace;
	TSet<UObject*> SubObjects;
};

static bool DeleteExistingMapPackages(const FString& ExistingPackageName, FPackageSourceControlHelper& PackageHelper)
{
	UE_SCOPED_TIMER(TEXT("Delete existing destination packages"), LogWorldPartitionRenameDuplicateBuilder, Display);
	TArray<FString> PackagesToDelete;

	FString ExistingMapPackageFilePath;	
	if (FPackageName::TryConvertLongPackageNameToFilename(ExistingPackageName, ExistingMapPackageFilePath, FPackageName::GetMapPackageExtension()))
	{
		if (IFileManager::Get().FileExists(*ExistingMapPackageFilePath))
		{
			PackagesToDelete.Add(ExistingMapPackageFilePath);
		}

		FString BuildDataPackageName = ExistingPackageName + TEXT("_BuiltData");
		FString ExistingBuildDataPackageFilePath = FPackageName::LongPackageNameToFilename(BuildDataPackageName, FPackageName::GetAssetPackageExtension());
		if (IFileManager::Get().FileExists(*ExistingBuildDataPackageFilePath))
		{
			PackagesToDelete.Add(ExistingBuildDataPackageFilePath);
		}

		// Search for external object packages
		const TArray<FString> ExternalPackagesPaths = ULevel::GetExternalObjectsPaths(ExistingPackageName);
		for (const FString& ExternalPackagesPath : ExternalPackagesPaths)
		{
			FString ExternalPackagesFilePath = FPackageName::LongPackageNameToFilename(ExternalPackagesPath);
			if (IFileManager::Get().DirectoryExists(*ExternalPackagesFilePath))
			{
				const bool bSuccess = IFileManager::Get().IterateDirectoryRecursively(*ExternalPackagesFilePath, [&PackagesToDelete](const TCHAR* FilenameOrDirectory, bool bIsDirectory)
				{
					if (!bIsDirectory)
					{
						FString Filename(FilenameOrDirectory);
						if (Filename.EndsWith(FPackageName::GetAssetPackageExtension()))
						{
							PackagesToDelete.Add(Filename);
						}
					}
					// Continue Directory Iteration
					return true;
				});

				if (!bSuccess)
				{
					UE_LOGF(LogWorldPartitionRenameDuplicateBuilder, Error, "Failed to iterate existing external actors path: %ls", *ExternalPackagesPath);
					return false;
				}
			}
		}

		return UWorldPartitionBuilder::DeletePackages(PackagesToDelete, PackageHelper);
	}

	return true;
}

UWorldPartitionRenameDuplicateBuilder::UWorldPartitionRenameDuplicateBuilder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	GetParamValue("NewPackage=", NewPackageName);
	bRename = HasParam("Rename");
}

bool UWorldPartitionRenameDuplicateBuilder::RunInternal(UWorld* World, const FCellInfo& CellInfo, FPackageSourceControlHelper& PackageHelper)
{
	UWorldPartition* WorldPartition = World->GetWorldPartition();
	if (!WorldPartition)
	{
		UE_LOGF(LogWorldPartitionRenameDuplicateBuilder, Error, "Failed to retrieve WorldPartition.");
		return false;
	}

	if (WorldPartition->GetActorDescContainerCount() > 1)
	{
		UE_LOGF(LogWorldPartitionRenameDuplicateBuilder, Warning, "Actors coming from containers outside of %ls will not be duplicated", *WorldPartition->GetActorDescContainerInstance()->GetExternalActorPath());
	}

	// Build actor clusters
	TArray<TPair<FGuid, TArray<FGuid>>> ActorsWithRefs;
	for (FActorDescContainerInstanceCollection::TIterator<> Iterator(WorldPartition); Iterator; ++Iterator)
	{
		FWorldPartitionActorDescInstance* ActorDescInstance = WorldPartition->GetActorDescInstance(Iterator->GetGuid());

		TArray<FGuid> ValidReferences;
		const TArray<FGuid>& References = Iterator->GetReferences();
		for (const FGuid& Reference : References)
		{
			if (WorldPartition->GetActorDescInstance(Reference))
			{
				ValidReferences.Add(Reference);
			}
			else
			{
				UE_LOGF(LogWorldPartitionRenameDuplicateBuilder, Warning, "Invalid reference %ls of actor %ls skipped", *Reference.ToString(EGuidFormats::DigitsWithHyphensInBraces), *ActorDescInstance->GetActorPackage().ToString());
			}
		}

		ActorsWithRefs.Emplace(Iterator->GetGuid(), ValidReferences);
	}

	TArray<TArray<FGuid>> ActorClusters = GenerateObjectsClusters(ActorsWithRefs);

	UPackage* OriginalPackage = World->GetPackage();
	OriginalWorldName = World->GetName();
	OriginalPackageName = OriginalPackage->GetName();
	
	TSet<FString> PackagesToDelete;
	if (bRename)
	{
		if (World->PersistentLevel->MapBuildData)
		{
			FString BuildDataPackageName = OriginalPackageName + TEXT("_BuiltData");
			PackagesToDelete.Add(BuildDataPackageName);
		}

		for (UPackage* ExternalPackage : OriginalPackage->GetExternalPackages())
		{
			PackagesToDelete.Add(ExternalPackage->GetName());
			ResetLoaders(ExternalPackage);
		}
	}
	
	const FString NewWorldName = FPackageName::GetLongPackageAssetName(NewPackageName);

	// Delete destination if it exists
	if (!DeleteExistingMapPackages(NewPackageName, PackageHelper))
	{
		UE_LOGF(LogWorldPartitionRenameDuplicateBuilder, Error, "Failed to delete existing destination package.");
		return false;
	}
				
	UPackage* NewPackage = CreatePackage(*NewPackageName);
	if (!NewPackage)
	{
		UE_LOGF(LogWorldPartitionRenameDuplicateBuilder, Error, "Failed to create destination package.");
		return false;
	}

	TMap<FGuid, FGuid> DuplicatedActorGuids;
	TArray<UPackage*> DuplicatedPackagesToSave;
	UWorld* NewWorld = nullptr;
	{
		UE_SCOPED_TIMER(TEXT("Duplicating world"), LogWorldPartitionRenameDuplicateBuilder, Display);
		FObjectDuplicationParameters DuplicationParameters(World, NewPackage);
		DuplicationParameters.DuplicateMode = EDuplicateMode::World;

		DuplicatedObjects.Empty();
		TMap<UObject*, UObject*> DuplicatedObjectPtrs;
		DuplicationParameters.CreatedObjects = &DuplicatedObjectPtrs;
		
		NewWorld = Cast<UWorld>(StaticDuplicateObjectEx(DuplicationParameters));
		check(NewWorld);
	
		// Copy Object pointers to Property so that GC doesn't try and collect any of them
		for (const auto& Pair : DuplicatedObjectPtrs)
		{
			DuplicatedObjects.Add(Pair.Key, Pair.Value);

			// Keep list of duplicated actor guids to skip processing them
			if (Pair.Value->IsPackageExternal())
			{
				if (AActor* SourceActor = Cast<AActor>(Pair.Key))
				{
					AActor* DuplicatedActor = CastChecked<AActor>(Pair.Value);

					DuplicatedActorGuids.Add(SourceActor->GetActorGuid(), DuplicatedActor->GetActorGuid());
				}
			}
		}

		DuplicatedPackagesToSave.Append(NewWorld->GetPackage()->GetExternalPackages());
		DuplicatedPackagesToSave.Add(NewWorld->GetPackage());
	}
		
	// World Scope
	{
		UWorld::InitializationValues IVS;
		IVS.RequiresHitProxies(false);
		IVS.ShouldSimulatePhysics(false);
		IVS.EnableTraceCollision(false);
		IVS.CreateNavigation(false);
		IVS.CreateAISystem(false);
		IVS.AllowAudioPlayback(false);
		IVS.CreatePhysicsScene(true);
		FScopedEditorWorld NewEditorWorld(NewWorld, IVS);

		// Fixup SoftPath archive
		FSoftObjectPathFixupArchive SoftObjectPathFixupArchive(OriginalPackageName + TEXT(".") + OriginalWorldName, NewPackageName + TEXT(".") + NewWorldName);

		{
			UE_SCOPED_TIMER(TEXT("Saving actors"), LogWorldPartitionRenameDuplicateBuilder, Display);
								
			auto ProcessLoadedActors = [&](TArray<FWorldPartitionReference>& ActorReferences) -> bool
			{
				if (!ActorReferences.Num())
				{
					return true;
				}

				TArray<UPackage*> ActorPackages;
				
				for (const FWorldPartitionReference& ActorReference : ActorReferences)
				{
					AActor* Actor = ActorReference.GetActor();
					UPackage* PreviousActorPackage = Actor->GetExternalPackage();

					// Rename Actor first so new package gets created
					Actor->Rename(nullptr, NewWorld->PersistentLevel, REN_NonTransactional | REN_DontCreateRedirectors | REN_DoNotDirty | REN_AllowPackageLinkerMismatch);

					TArray<UObject*> DependantObjects;
					ForEachObjectWithPackage(PreviousActorPackage, [&DependantObjects](UObject* Object)
					{
						DependantObjects.Add(Object);
						return true;
					}, EGetObjectsFlags::None);

					// Move dependant objects into the new actor package
					for (UObject* DependantObject : DependantObjects) //-V1078
					{
						DependantObject->Rename(nullptr, Actor->GetExternalPackage(), REN_NonTransactional | REN_DontCreateRedirectors | REN_DoNotDirty | REN_AllowPackageLinkerMismatch);
					}

					// Releases file handle so it can be deleted
					if (bRename)
					{
						ResetLoaders(PreviousActorPackage);
					}

					// Patch SoftObject Paths
					SoftObjectPathFixupArchive.Fixup(Actor);
					// Patch Duplicated Object Refs
					FReplaceObjectRefsArchive(Actor, DuplicatedObjects);

					ActorPackages.Add(Actor->GetPackage());
				}

				UE_LOGF(LogWorldPartitionRenameDuplicateBuilder, Display, "Saving %d actor(s)", ActorPackages.Num());
				if (!UWorldPartitionBuilder::SavePackages(ActorPackages, PackageHelper))
				{
					UE_LOGF(LogWorldPartitionRenameDuplicateBuilder, Error, "Failed to save actor packages:");
					for (UPackage* ActorPackage : ActorPackages)
					{
						UE_LOGF(LogWorldPartitionRenameDuplicateBuilder, Error, "    Package: %ls", *ActorPackage->GetName());
					}
					return false;
				}
				ActorPackages.Empty();

				// Rename Actor(s) back into their original Outer so that they stay valid until the next GC. 
				// This is to prevent failures when some non-serialized references get taken by loaded actors this makes sure those references will resolve. (example that no longer exists: Landscape SplineHandles)
				for (FWorldPartitionReference ActorReference : ActorReferences)
				{
					AActor* Actor = ActorReference.GetActor();
					UPackage* NewActorPackage = Actor->GetExternalPackage();
					check(Actor);
					Actor->Rename(nullptr, World->PersistentLevel, REN_NonTransactional | REN_DontCreateRedirectors | REN_DoNotDirty | REN_AllowPackageLinkerMismatch);

					TArray<UObject*> DependantObjects;
					ForEachObjectWithPackage(NewActorPackage, [&DependantObjects](UObject* Object)
					{
						DependantObjects.Add(Object);
						return true;
					}, EGetObjectsFlags::None);

					// Move back dependant objects into the previous actor package
					for (UObject* DependantObject : DependantObjects) //-V1078
					{
						DependantObject->Rename(nullptr, Actor->GetExternalPackage(), REN_NonTransactional | REN_DontCreateRedirectors | REN_DoNotDirty | REN_AllowPackageLinkerMismatch);
					}
				}
				ActorReferences.Empty();

				return true;
			};

			TArray<FWorldPartitionReference> ActorReferences;
			for (const TArray<FGuid>& ActorCluster : ActorClusters)
			{
				UE_LOGF(LogWorldPartitionRenameDuplicateBuilder, Display, "Processing cluster with %d actor(s)", ActorCluster.Num());
				for (const FGuid& ActorGuid : ActorCluster)
				{
					// Duplicated actors don't need to be processed
					if (!DuplicatedActorGuids.Contains(ActorGuid))
					{
						FWorldPartitionReference ActorReference(WorldPartition, ActorGuid);
						check(ActorReference.IsValid());
						AActor* Actor = ActorReference.GetActor();
						check(Actor);
						ActorReferences.Add(ActorReference);
					}

					// If we are renaming add package to delete
					if (bRename)
					{
						const FWorldPartitionActorDescInstance* ActorDescInstance = WorldPartition->GetActorDescInstance(ActorGuid);
						check(ActorDescInstance);

						PackagesToDelete.Add(ActorDescInstance->GetActorPackage().ToString());
					}
				}
						
				if (FWorldPartitionHelpers::ShouldCollectGarbage())
				{
					if (!ProcessLoadedActors(ActorReferences))
					{
						return false;
					}
					FWorldPartitionHelpers::DoCollectGarbage();
				}
			}

			// Call one last time
			if (!ProcessLoadedActors(ActorReferences))
			{
				return false;
			}
		}

		{
			// Save all duplicated packages
			UE_SCOPED_TIMER(TEXT("Saving new map packages"), LogWorldPartitionRenameDuplicateBuilder, Display);
			if (!UWorldPartitionBuilder::SavePackages(DuplicatedPackagesToSave, PackageHelper))
			{
				return false;
			}
		}
				

		{
			// Validate results
			UE_SCOPED_TIMER(TEXT("Validating actors"), LogWorldPartitionRenameDuplicateBuilder, Display);
			for (FActorDescContainerInstanceCollection::TConstIterator<> Iterator(WorldPartition); Iterator; ++Iterator)
			{
				const FWorldPartitionActorDescInstance* SourceActorDescInstance = *Iterator;
				FGuid* DuplicatedGuid = DuplicatedActorGuids.Find(SourceActorDescInstance->GetGuid());
				const FWorldPartitionActorDescInstance* NewActorDescInstance = NewWorld->GetWorldPartition()->GetActorDescInstance(DuplicatedGuid ? *DuplicatedGuid : SourceActorDescInstance->GetGuid());
				if (!NewActorDescInstance)
				{
					UE_LOGF(LogWorldPartitionRenameDuplicateBuilder, Warning, "Failed to find source actor for Actor: %ls", *SourceActorDescInstance->GetActorSoftPath().ToString());
				}
				else
				{
					if (NewActorDescInstance->GetReferences().Num() != SourceActorDescInstance->GetReferences().Num())
					{
						UE_LOGF(LogWorldPartitionRenameDuplicateBuilder, Warning, "Actor: %ls and Source Actor: %ls have mismatching reference count", *NewActorDescInstance->GetActorSoftPath().ToString(), *SourceActorDescInstance->GetActorSoftPath().ToString());
					}
					else
					{
						for (const FGuid& ReferenceGuid : SourceActorDescInstance->GetReferences())
						{
							FGuid* DuplicateReferenceGuid = DuplicatedActorGuids.Find(ReferenceGuid);
							if (!NewActorDescInstance->GetReferences().Contains(DuplicateReferenceGuid ? *DuplicateReferenceGuid : ReferenceGuid))
							{
								UE_LOGF(LogWorldPartitionRenameDuplicateBuilder, Warning, "Actor: %ls and Source Actor: %ls have mismatching reference", *NewActorDescInstance->GetActorSoftPath().ToString(), *SourceActorDescInstance->GetActorSoftPath().ToString());
							}
						}
					}
				}
			}
		}


		DuplicatedObjects.Empty();
	}
		
	if (PackagesToDelete.Num() > 0)
	{
		UE_SCOPED_TIMER(TEXT("Delete source packages (-rename switch)"), LogWorldPartitionRenameDuplicateBuilder, Display);
		
		UE_LOGF(LogWorldPartitionRenameDuplicateBuilder, Display, "Deleting %d packages", PackagesToDelete.Num());
		if (!PackageHelper.Delete(PackagesToDelete.Array()))
		{
			UE_LOGF(LogWorldPartitionRenameDuplicateBuilder, Error, "Failed to delete source packages:");
			for (const FString& PackageToDelete : PackagesToDelete)
			{
				UE_LOGF(LogWorldPartitionRenameDuplicateBuilder, Error, "    Package: %ls", *PackageToDelete);
			}
			return false;
		}
	}
			
	return true;
}

bool UWorldPartitionRenameDuplicateBuilder::PostWorldTeardown(FPackageSourceControlHelper& PackageHelper)
{
	if (!UWorldPartitionBuilder::PostWorldTeardown(PackageHelper))
	{
		return false;
	}

	// Create redirector
	if (bRename)
	{
		// Make sure to release handle on original package if a redirector needs to be saved
		UPackage* OriginalPackage = FindPackage(nullptr, *OriginalPackageName);
		ResetLoaders(OriginalPackage);
		FWorldPartitionHelpers::DoCollectGarbage();
		check(!FindPackage(nullptr, *OriginalPackageName));

		UPackage* RedirectorPackage = CreatePackage(*OriginalPackageName);
		RedirectorPackage->ThisContainsMap();

		UObjectRedirector* Redirector = NewObject<UObjectRedirector>(RedirectorPackage, *OriginalWorldName, RF_Standalone | RF_Public);
		FSoftObjectPath RedirectorPath(Redirector);

		UPackage* NewWorldPackage = LoadPackage(nullptr, *NewPackageName, LOAD_None);
		Redirector->DestinationObject = UWorld::FindWorldInPackage(NewWorldPackage);
		check(Redirector->DestinationObject);
		RedirectorPackage->MarkAsFullyLoaded();

		// Saving the NewPackage will save the duplicated external packages
		UE_SCOPED_TIMER(TEXT("Saving new redirector"), LogWorldPartitionRenameDuplicateBuilder, Display);
		if (!UWorldPartitionBuilder::SavePackages({ RedirectorPackage }, PackageHelper))
		{
			UE_LOGF(LogWorldPartitionRenameDuplicateBuilder, Error, "Failed to save redirector package: %ls", *RedirectorPackage->GetName());
			return false;
		}

		// Validate Redirector
		ResetLoaders(RedirectorPackage);
		ForEachObjectWithPackage(RedirectorPackage, [](UObject* Object)
		{
			Object->ClearFlags(RF_Standalone);
			return true;
		}, EGetObjectsFlags::None);
		FWorldPartitionHelpers::DoCollectGarbage();
		check(!FindPackage(nullptr, *OriginalPackageName));

		UWorld* RedirectedWorld = CastChecked<UWorld>(RedirectorPath.TryLoad());
		if (!RedirectedWorld)
		{
			UE_LOGF(LogWorldPartitionRenameDuplicateBuilder, Error, "Failed to validate redirector package: %ls", *RedirectorPackage->GetName());
			return false;
		}
	}

	return true;
}
