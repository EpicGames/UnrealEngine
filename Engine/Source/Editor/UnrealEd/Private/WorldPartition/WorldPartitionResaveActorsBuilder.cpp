// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionResaveActorsBuilder.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/WorldPartitionClassDescRegistry.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/ActorDescContainerSubsystem.h"
#include "PackageSourceControlHelper.h"
#include "SourceControlHelpers.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "Commandlets/Commandlet.h"
#include "Misc/CommandLine.h"
#include "UObject/MetaData.h"
#include "UObject/SavePackage.h"
#include "ReferenceCluster.h"
#include "ActorFolder.h"
#include "Misc/FileHelper.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "AssetToolsModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionResaveActorsBuilder)

DEFINE_LOG_CATEGORY_STATIC(LogWorldPartitionResaveActorsBuilder, All, All);

UWorldPartitionResaveActorsBuilder::UWorldPartitionResaveActorsBuilder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}

bool UWorldPartitionResaveActorsBuilder::PreRun(UWorld* World, FPackageSourceControlHelper& PackageHelper)
{
	TArray<FString> Tokens, Switches;
	UCommandlet::ParseCommandLine(*GetBuilderArgs(), Tokens, Switches);
	
	if (!UCommandlet::ApplyCommandLineSwitches(this, Switches))
	{
		return false;
	}

	if (bSwitchActorPackagingSchemeToReduced)
	{
		if (!ActorClassName.IsEmpty())
		{
			UE_LOGF(LogWorldPartitionResaveActorsBuilder, Error, "SwitchActorPackagingSchemeToReduced is not compatible with ActorClassName");
			return false;
		}
		else if (!ActorClassesFromFile.IsEmpty())
		{
			UE_LOGF(LogWorldPartitionResaveActorsBuilder, Error, "SwitchActorPackagingSchemeToReduced is not compatible with ActorClassesFromFile");
			return false;
		}
		else if (!ActorTags.IsEmpty())
		{
			UE_LOGF(LogWorldPartitionResaveActorsBuilder, Error, "SwitchActorPackagingSchemeToReduced is not compatible with ActorTags");
			return false;
		}
		else if (!ActorProperties.IsEmpty())
		{
			UE_LOGF(LogWorldPartitionResaveActorsBuilder, Error, "SwitchActorPackagingSchemeToReduced is not compatible with ActorProperties");
			return false;
		}
		else if (bResaveDirtyActorDescsOnly)
		{
			UE_LOGF(LogWorldPartitionResaveActorsBuilder, Error, "SwitchActorPackagingSchemeToReduced is not compatible with ResaveDirtyActorDescsOnly");
			return false;
		}
	}

	if (bEnableActorFolders)
	{
		if (bResaveDirtyActorDescsOnly)
		{
			UE_LOGF(LogWorldPartitionResaveActorsBuilder, Error, "EnableActorFolders is not compatible with ResaveDirtyActorDescsOnly");
			return false;
		}
	}

	return true;
}

TArray<TSubclassOf<AActor>> UWorldPartitionResaveActorsBuilder::GetActorClassesFilter()
{
	TArray<TSubclassOf<AActor>> ActorClasses;

	auto GetActorClassFromName = [](const FString& ClassName)
	{
		// Look for native classes
		UClass* ActorClass = UClass::TryFindTypeSlow<UClass>(ClassName);

		if (!ActorClass)
		{
			// Look for a fully qualified BP class
			ActorClass = LoadClass<AActor>(nullptr, *ClassName, nullptr, LOAD_NoWarn, nullptr);

			if (!ActorClass)
			{
				// Look for a package BP
				if (FPackageName::DoesPackageExist(ClassName))
				{
					ActorClass = LoadClass<AActor>(nullptr, *FString::Printf(TEXT("%s.%s_C"), *ClassName, *FPackageName::GetLongPackageAssetName(ClassName)), nullptr, LOAD_NoWarn, nullptr);
				}
			}
		}

		return ActorClass;
	};

	if (!ActorClassName.IsEmpty())
	{
		check(!bSwitchActorPackagingSchemeToReduced);

		UClass* ActorClass = GetActorClassFromName(ActorClassName);
		if (ActorClass)
		{
			ActorClasses.Add(ActorClass);
		}
		else
		{
			UE_LOGF(LogWorldPartitionResaveActorsBuilder, Error, "Failed to find Actor Class: %ls.", *ActorClassName);
		}
	}
	else if (!ActorClassesFromFile.IsEmpty())
	{
		TArray<FString> ClassesNames;

		bool bReadFile = FFileHelper::LoadFileToStringArray(ClassesNames, *ActorClassesFromFile);
		if (bReadFile)
		{
			for (const FString& ClassName : ClassesNames)
			{
				UClass* ActorClass = GetActorClassFromName(ClassName);
				if (ActorClass)
				{
					ActorClasses.Add(ActorClass);
				}
				else
				{
					UE_LOGF(LogWorldPartitionResaveActorsBuilder, Warning, "Failed to find Actor Class: %ls.", *ClassName);
				}
			}

			if (ActorClasses.IsEmpty())
			{
				UE_LOGF(LogWorldPartitionResaveActorsBuilder, Error, "Failed to find any valid actor classes in file: %ls.", *ActorClassesFromFile);
			}			
		}
		else
		{
			UE_LOGF(LogWorldPartitionResaveActorsBuilder, Error, "Failed to open/process actor classes in file: %ls.", *ActorClassesFromFile);
		}
	}
	else 
	{
		// No class filtering specified... all actor classes are valid
		ActorClasses.Add(AActor::StaticClass());
	}

	return ActorClasses;
}

bool UWorldPartitionResaveActorsBuilder::RunInternal(UWorld* World, const FCellInfo& InCellInfo, FPackageSourceControlHelper& PackageHelper)
{
	UPackage* WorldPackage = World->GetPackage();

	// Actor Class Filter
	TArray<TSubclassOf<AActor>> ActorClasses = GetActorClassesFilter();
	if (ActorClasses.IsEmpty())
	{
		return false;
	}

	TArray<FString> PackagesToDelete;
	TArray<FString> DirtyActorDescsOld;
	TArray<FString> DirtyActorDescsNew;

	UWorldPartition* WorldPartition = World->GetWorldPartition();
	if (!WorldPartition)
	{
		if (!World->PersistentLevel->bUseExternalActors)
		{
			UE_LOGF(LogWorldPartitionResaveActorsBuilder, Error, "Cannot work on maps not using one file per actor.");
			return false;
		}

		if (bSwitchActorPackagingSchemeToReduced)
		{
			UE_LOGF(LogWorldPartitionResaveActorsBuilder, Error, "Cannot switch actor packaging scheme on non-partitioned worlds.");
			return false;
		}

		if (bEnableActorFolders)
		{
			UE_LOGF(LogWorldPartitionResaveActorsBuilder, Error, "Cannot enable actor folders non-partitioned worlds.");
			return false;
		}

		if (UActorDescContainer* LevelContainer = UActorDescContainerSubsystem::GetChecked().RegisterContainer(UActorDescContainer::FInitializeParams{ World->GetPackage()->GetLoadedPath().GetPackageFName() }))
		{
			TArray<UPackage*> PackagesToSave;
			for (FActorDescList::TIterator<> ActorDescIterator(LevelContainer); ActorDescIterator; ++ActorDescIterator)
			{
				if (AActor* Actor = FindObject<AActor>(World->PersistentLevel, *ActorDescIterator->GetActorName().ToString()))
				{
					UPackage* Package = Actor->GetExternalPackage();
					check(Package);

					if (bResaveDirtyActorDescsOnly)
					{
						TUniquePtr<FWorldPartitionActorDesc> NewActorDesc = Actor->CreateActorDesc();

						if (!ActorDescIterator->IsResaveNeeded() && !ActorDescIterator->ShouldResave(NewActorDesc.Get()))
						{
							continue;
						}

						if (bDiffDirtyActorDescs)
						{
							DirtyActorDescsOld.Add(ActorDescIterator->ToString(FWorldPartitionActorDesc::EToStringMode::ForDiff));
							DirtyActorDescsNew.Add(NewActorDesc->ToString(FWorldPartitionActorDesc::EToStringMode::ForDiff));
						}

						UE_LOGF(LogWorldPartitionResaveActorsBuilder, Log, "Package %ls needs to be resaved.", *Package->GetName());
					}

					if (!bReportOnly)
					{
						PackagesToSave.Add(Package);
					}
				}
			}

			UActorDescContainerSubsystem::GetChecked().UnregisterContainer(LevelContainer);

			if (!UWorldPartitionBuilder::SavePackages(PackagesToSave, PackageHelper))
			{
				return false;
			}
		}
	}
	else
	{
		if (bSwitchActorPackagingSchemeToReduced)
		{
			if (World->PersistentLevel->GetActorPackagingScheme() == EActorPackagingScheme::Reduced)
			{
				UE_LOGF(LogWorldPartitionResaveActorsBuilder, Error, "World is already using the reduced actor packaging scheme.");
				return false;
			}

			World->PersistentLevel->ActorPackagingScheme = EActorPackagingScheme::Reduced;
		}

		if (bEnableActorFolders)
		{
			if (World->PersistentLevel->IsUsingActorFolders())
			{
				UE_LOGF(LogWorldPartitionResaveActorsBuilder, Error, "World is already using actor folder objects.");
				return false;
			}

			World->PersistentLevel->SetUseActorFolders(true);
			World->PersistentLevel->bFixupActorFoldersAtLoad = false;
		}

		if (bSwitchActorPackagingSchemeToReduced)
		{
			World->PersistentLevel->ActorPackagingScheme = EActorPackagingScheme::Reduced;

			// Build clusters
			TArray<TPair<FGuid, TArray<FGuid>>> ActorsWithRefs;
			for (FActorDescContainerInstanceCollection::TIterator<> Iterator(WorldPartition); Iterator; ++Iterator)
			{
				ActorsWithRefs.Emplace(Iterator->GetGuid(), Iterator->GetReferences());
			}

			TArray<TArray<FGuid>> Clusters = GenerateObjectsClusters(ActorsWithRefs);

			TSet<FWorldPartitionReference> LoadedReferences;
			TArray<UPackage*> PackagesToSave;
			for (const TArray<FGuid>& Cluster : Clusters)
			{
				// Load actor clusters
				TArray<FWorldPartitionReference> ActorReferences;
				ActorReferences.Reserve(Cluster.Num());
				for (const FGuid& ActorGuid : Cluster)
				{
					ActorReferences.Emplace(WorldPartition, ActorGuid);
				}
				LoadedReferences.Append(ActorReferences);

				// Change packaging of all actors in the current cluster
				for (FWorldPartitionReference& ActorReference : ActorReferences)
				{
					const FWorldPartitionActorDescInstance* ActorDescInstance = *ActorReference;
					AActor* Actor = ActorReference.GetActor();

					if (!Actor)
					{
						PackagesToDelete.Add(ActorDescInstance->GetActorPackage().ToString());
						continue;
					}
					else
					{
						UPackage* Package = Actor->GetExternalPackage();
						check(Package);

						if (!bReportOnly)
						{
							// Always mark this package for deletion, as it will contain a temporary redirector to fixup references
							PackagesToDelete.Add(ActorDescInstance->GetActorPackage().ToString());

							// Move actor back into world's package
							Actor->SetPackageExternal(false);

							// Gather dependant objects that also needs to be moved along with the actor
							TArray<UObject*> DependantObjects;
							ForEachObjectWithPackage(Package, [&DependantObjects](UObject* Object)
							{
								DependantObjects.Add(Object);
								return true;
							}, EGetObjectsFlags::None);

							// Move dependant objects into the new world package temporarily
							for (UObject* DependantObject : DependantObjects)
							{
								DependantObject->Rename(nullptr, WorldPackage, REN_NonTransactional | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
							}

							// Move actor in its new package
							Actor->SetPackageExternal(true);

							// Also move dependant objects into the new package
							UPackage* NewActorPackage = Actor->GetExternalPackage();
							for (UObject* DependantObject : DependantObjects)
							{
								DependantObject->Rename(nullptr, NewActorPackage, REN_NonTransactional | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
							}

							PackagesToSave.Add(NewActorPackage);
						}
					}
				}

				if (FWorldPartitionHelpers::ShouldCollectGarbage())
				{
					if (!UWorldPartitionBuilder::SavePackages(PackagesToSave, PackageHelper))
					{
						return false;
					}

					PackagesToSave.Empty();
					LoadedReferences.Empty();
					FWorldPartitionHelpers::DoCollectGarbage();
				}
			}

			// Save last batch
			if (!UWorldPartitionBuilder::SavePackages(PackagesToSave, PackageHelper))
			{
				return false;
			}
			PackagesToSave.Empty();
			LoadedReferences.Empty();
			FWorldPartitionHelpers::DoCollectGarbage();
		}
		else
		{
			if (bResaveBlueprints)
			{
				// Resave blueprints top-down from the class hierarchy
				FWorldPartitionClassDescRegistry& ClassDescRegistry = FWorldPartitionClassDescRegistry::Get();

				TArray<FTopLevelAssetPath> ClassesToResave;
				TMap<FTopLevelAssetPath, TArray<FTopLevelAssetPath>> ClassHierarchy;
				for (auto& [ClassPath, ParentClassPath] : ClassDescRegistry.GetParentClassMap())
				{
					ClassHierarchy.FindOrAdd(ParentClassPath).Add(ClassPath);

					if (!ClassPath.ToString().StartsWith(TEXT("/Script/")) && ParentClassPath.ToString().StartsWith(TEXT("/Script/")))
					{
						ClassesToResave.Add(ClassPath);
					}
				}

				TArray<UPackage*> PackagesToSave;
				while (!ClassesToResave.IsEmpty())
				{
					for (const FTopLevelAssetPath& ClassToResave : ClassesToResave)
					{
						UPackage* ClassPackage = LoadPackage(nullptr, *ClassToResave.GetPackageName().ToString(), LOAD_None);

						if (!ClassPackage)
						{
							UE_LOGF(LogWorldPartitionResaveActorsBuilder, Error, "Failed to load package for class '%ls'", *ClassToResave.ToString());
							return false;
						}

						PackagesToSave.Add(ClassPackage);

						if (FWorldPartitionHelpers::HasExceededMaxMemory())
						{
							UWorldPartitionBuilder::SavePackages(PackagesToSave, PackageHelper, true);
							FWorldPartitionHelpers::DoCollectGarbage();
							PackagesToSave.Empty();
						}
					}

					const TArray<FTopLevelAssetPath> ParentClasses = MoveTemp(ClassesToResave);

					for (const FTopLevelAssetPath& ParentClass : ParentClasses)
					{
						ClassesToResave += ClassHierarchy.FindRef(ParentClass);
					}
				}

				UWorldPartitionBuilder::SavePackages(PackagesToSave, PackageHelper);
				FWorldPartitionHelpers::DoCollectGarbage();
			}

			TArray<UPackage*> PackagesToSave;

			FWorldPartitionHelpers::FForEachActorWithLoadingParams ForEachActorWithLoadingParams;

			ForEachActorWithLoadingParams.ActorClasses = ActorClasses;

			ForEachActorWithLoadingParams.FilterActorDesc = [this](const FWorldPartitionActorDesc* ActorDesc) -> bool
			{
				for (const FName& ActorTag : ActorDesc->GetTags())
				{
					if (ActorTags.Contains(ActorTag))
					{
						return true;
					}
				}
			
				for (auto [PropertyName, PropertyValue] : ActorProperties)
				{
					FName Value;
					if (ActorDesc->GetProperty(PropertyName, &Value))
					{
						if (Value == PropertyValue)
						{
							return true;
						}
					}
				}

				return ActorTags.IsEmpty() && ActorProperties.IsEmpty();
			};

			ForEachActorWithLoadingParams.OnPreGarbageCollect = [&PackagesToSave, &PackageHelper]()
			{
				UWorldPartitionBuilder::SavePackages(PackagesToSave, PackageHelper, true);
				PackagesToSave.Empty();
			};

			FWorldPartitionHelpers::ForEachActorWithLoading(WorldPartition, [this, &PackagesToDelete, &PackagesToSave, &DirtyActorDescsOld, &DirtyActorDescsNew](const FWorldPartitionActorDescInstance* ActorDescInstance)
			{
				AActor* Actor = ActorDescInstance->GetActor();
			
				if (!Actor)
				{
					PackagesToDelete.Add(ActorDescInstance->GetActorPackage().ToString());
				}
				else
				{
					if (bEnableActorFolders)
					{
						if (!Actor->CreateOrUpdateActorFolder())
						{
							UE_LOGF(LogWorldPartitionResaveActorsBuilder, Error, "Failed to create actor folder for actor %ls.", *Actor->GetName());
							return true;
						}
					}

					UPackage* Package = Actor->GetExternalPackage();
					check(Package);

					if (bResaveDirtyActorDescsOnly)
					{
						const FWorldPartitionActorDesc* ActorDesc = ActorDescInstance->GetActorDesc();
						TUniquePtr<FWorldPartitionActorDesc> NewActorDesc = Actor->CreateActorDesc();

						if (!ActorDesc->IsResaveNeeded() && !ActorDesc->ShouldResave(NewActorDesc.Get()))
						{
							return true;
						}

						if (bDiffDirtyActorDescs)
						{
							DirtyActorDescsOld.Add(ActorDesc->ToString(FWorldPartitionActorDesc::EToStringMode::ForDiff));
							DirtyActorDescsNew.Add(NewActorDesc->ToString(FWorldPartitionActorDesc::EToStringMode::ForDiff));
						}

						UE_LOGF(LogWorldPartitionResaveActorsBuilder, Log, "Package %ls needs to be resaved.", *Package->GetName());
					}

					if (!bReportOnly)
					{
						PackagesToSave.Add(Package);
					}
				}

				return true;
			}, ForEachActorWithLoadingParams);
		}
	}

	if (bDiffDirtyActorDescs && DirtyActorDescsOld.Num())
	{
		auto WriteTempFile = [](const TArray<FString>& Lines, FString& TempFileName)
		{
			TempFileName = FPaths::CreateTempFilename(*FPaths::DiffDir(), TEXT("ActorsDescsDiff"));
			return FFileHelper::SaveStringArrayToFile(Lines, *TempFileName);
		};

		FString TempFileOld;
		FString TempFileNew;
		bool OldResult = WriteTempFile(DirtyActorDescsOld, TempFileOld);
		bool NewResult = WriteTempFile(DirtyActorDescsNew, TempFileNew);

		if (OldResult && NewResult)
		{
			FString DiffCommand = GetDefault<UEditorLoadingSavingSettings>()->TextDiffToolPath.FilePath;

			FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
			AssetToolsModule.Get().CreateDiffProcess(DiffCommand, TempFileOld, TempFileNew);
		}
		else
		{
			UE_LOGF(LogWorldPartitionResaveActorsBuilder, Error, "Failed to write actors desc diff files.");
		}
	}

	if (!bReportOnly)
	{
		if (!UWorldPartitionBuilder::DeletePackages(PackagesToDelete, PackageHelper, !bSwitchActorPackagingSchemeToReduced) && bSwitchActorPackagingSchemeToReduced)
		{
			return false;
		}

		if (bEnableActorFolders)
		{
			TArray<UPackage*> FolderPackages;
			World->PersistentLevel->ForEachActorFolder([&FolderPackages](UActorFolder* ActorFolder)
			{
				UPackage* NewActorFolderPackage = ActorFolder->GetExternalPackage();
				check(NewActorFolderPackage);
				FolderPackages.Add(NewActorFolderPackage);
				return true;
			});

			if (!UWorldPartitionBuilder::SavePackages(FolderPackages, PackageHelper))
			{
				return false;
			}
		}

		const bool bNeedWorldResave = bSwitchActorPackagingSchemeToReduced || bEnableActorFolders;
		if (bNeedWorldResave)
		{
			return UWorldPartitionBuilder::SavePackages({ WorldPackage }, PackageHelper);
		}
	}

	return true;
}
