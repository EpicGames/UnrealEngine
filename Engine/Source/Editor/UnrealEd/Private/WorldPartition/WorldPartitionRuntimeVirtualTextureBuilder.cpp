// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionRuntimeVirtualTextureBuilder.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/WorldPartitionHandle.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "Components/RuntimeVirtualTextureComponent.h"
#include "Components/PrimitiveComponent.h"
#include "VT/VirtualTextureBuilder.h"
#include "VirtualTexturingEditorModule.h"
#include "AssetCompilingManager.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectIterator.h"
#include "ComponentRecreateRenderStateContext.h"
#include "Materials/MaterialInstance.h"
#include "Materials/Material.h"
#include "ShaderCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionRuntimeVirtualTextureBuilder)

DEFINE_LOG_CATEGORY_STATIC(LogWorldPartitionRuntimeVirtualTextureBuilder, All, All);

UWorldPartitionRuntimeVirtualTextureBuilder::UWorldPartitionRuntimeVirtualTextureBuilder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UWorldPartitionRuntimeVirtualTextureBuilder::LoadRuntimeVirtualTextureActors(UWorldPartition* WorldPartition, FWorldPartitionHelpers::FForEachActorWithLoadingResult& Result)
{
	check(WorldPartition);

	FWorldPartitionHelpers::FForEachActorWithLoadingParams ForEachActorWithLoadingParams;
	ForEachActorWithLoadingParams.bKeepReferences = true;
	ForEachActorWithLoadingParams.FilterActorDesc = [](const FWorldPartitionActorDesc* ActorDesc) -> bool { return ActorDesc->HasProperty(UPrimitiveComponent::RVTActorDescProperty); };
		
	// @todo: in order to scale the RVTs should be generated with tiling so that we don't need to load all actors writing to RVTs at once.
	FWorldPartitionHelpers::ForEachActorWithLoading(WorldPartition, [](const FWorldPartitionActorDescInstance*) { return true; }, ForEachActorWithLoadingParams, Result);

	// Make sure all assets are finished compiling
	FAssetCompilingManager::Get().FinishAllCompilation();
}

bool UWorldPartitionRuntimeVirtualTextureBuilder::RunInternal(UWorld* World, const FCellInfo& InCellInfo, FPackageSourceControlHelper& PackageHelper)
{
	IVirtualTexturingEditorModule& VTModule = FModuleManager::Get().LoadModuleChecked<IVirtualTexturingEditorModule>("VirtualTexturingEditor");
	
	FWorldPartitionHelpers::FForEachActorWithLoadingResult ForEachActorWithLoadingResult;
	if (UWorldPartition* WorldPartition = World->GetWorldPartition())
	{
		LoadRuntimeVirtualTextureActors(WorldPartition, ForEachActorWithLoadingResult);
	}
	
	TArray<URuntimeVirtualTextureComponent*> Components = VTModule.GatherRuntimeVirtualTextureComponents(World);
	FBuildAllStreamedMipsResult Result = VTModule.BuildAllStreamedMips({ .World = World, .Components = Components, .bRestoreFeatureLevelAfterBuilding = false });

	// Wait for VT Textures to be ready before saving
	FAssetCompilingManager::Get().FinishAllCompilation();

	if (!Result.bSuccess)
	{
		return false;
	}

	if (!SavePackages(Result.ModifiedPackages.Array(), PackageHelper, false))
	{
		return false;
	}

	TArray<FString> FilesToSubmit;
	FilesToSubmit.Append(SourceControlHelpers::PackageFilenames(Result.ModifiedPackages.Array()));

	const FString ChangeDescription = FString::Printf(TEXT("Built RVT for world '%s'"), *World->GetName());
	return OnFilesModified(FilesToSubmit, ChangeDescription);
}
