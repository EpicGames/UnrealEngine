// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPersistentFolders.h"
#include "EditorActorFolders.h"
#include "ActorFolder.h"
#include "Engine/World.h"
#include "Engine/Level.h"
#include "UObject/UObjectHash.h"
#include "ExternalPackageHelper.h"
#include "LevelInstance/LevelInstanceInterface.h"

#define LOCTEXT_NAMESPACE "UnrealEd.WorldPersistentFolders"

FWorldPersistentFolders::FWorldPersistentFolders(UWorldFolders& InWorldFolders)
	: Super(InWorldFolders)
{
}

bool FWorldPersistentFolders::ContainsFolder(const FFolder& InFolder) const
{
	if (Super::ContainsFolder(InFolder))
	{
		return true;
	}

	return !!GetActorFolder(InFolder);
}

bool FWorldPersistentFolders::AddFolder(const FFolder& InFolder)
{
	if (!InFolder.IsNone())
	{
		const UActorFolder* ActorFolder = GetActorFolder(InFolder);
		if (!IsValid(ActorFolder))
		{
			ActorFolder = CreateActorFolder(InFolder);
			return IsValid(ActorFolder);
		}
	}
	return false;
}

bool FWorldPersistentFolders::RemoveFolder(const FFolder& InFolder, bool bShouldDeleteFolder)
{
	UActorFolder* ActorFolder = InFolder.GetActorFolder();
	if (!IsValid(ActorFolder))
	{
		return false;
	}

	if (bShouldDeleteFolder)
	{
		const FFolder::FRootObject& RootObject = InFolder.GetRootObject();
		ULevel* Level = GetRootObjectContainer(InFolder, GetWorld());
		check(Level);
		check(Level == ActorFolder->GetOuterULevel());

		ModifyFolderAndDetectChanges(Level, RootObject, [&ActorFolder]()
		{
			ActorFolder->MarkAsDeleted();
		});
	}
	
	return true;
}

bool FWorldPersistentFolders::RenameFolder(const FFolder& InOldFolder, const FFolder& InNewFolder)
{
	check(InOldFolder.GetRootObject() == InNewFolder.GetRootObject());
	const FFolder::FRootObject& RootObject = InOldFolder.GetRootObject();

	UActorFolder* ActorFolder = InOldFolder.GetActorFolder();
	check(IsValid(ActorFolder));
	UActorFolder* FoundFolder = InNewFolder.GetActorFolder();
	check(!IsValid(FoundFolder) || !FoundFolder->GetPath().IsEqual(InNewFolder.GetPath(), ENameCase::CaseSensitive));


	ULevel* Level = GetRootObjectContainer(InOldFolder, GetWorld());
	check(Level);
	check(Level == ActorFolder->GetOuterULevel());

	ModifyFolderAndDetectChanges(Level, RootObject, [this, &ActorFolder, &InNewFolder]()
	{
		UActorFolder* ParentActorFolder = InNewFolder.GetParent().GetActorFolder();
		ActorFolder->SetParent(ParentActorFolder);
		const FString FolderLabel = InNewFolder.GetLeafName().ToString();
		ActorFolder->SetLabel(FolderLabel);
		check(ActorFolder->GetPath().IsEqual(InNewFolder.GetPath(), ENameCase::CaseSensitive));
	});

	return true;
}

void FWorldPersistentFolders::ModifyFolderAndDetectChanges(ULevel* InLevel, const FFolder::FRootObject& InRootObject, TFunctionRef<void()> InOperation)
{
	TMap<UActorFolder*, FName> OldFolderToPath;
	InLevel->ForEachActorFolder([&OldFolderToPath](UActorFolder* FolderIt)
	{
		OldFolderToPath.Add(FolderIt, FolderIt->GetPath());
		return true;
	}, /*bSkipDeleted*/ true);

	InOperation();

	TArray<TPair<FName, FName>> ChangedFolders;
	InLevel->ForEachActorFolder([&OldFolderToPath, &ChangedFolders](UActorFolder* FolderIt)
	{
		FName NewPath = FolderIt->GetPath();
		FName* OldPath = OldFolderToPath.Find(FolderIt);
		if (OldPath && !OldPath->IsEqual(NewPath, ENameCase::CaseSensitive))
		{
			ChangedFolders.Emplace(*OldPath, NewPath);
		}
		return true;
	}, /*bSkipDeleted*/ true);
	
	for (const TPair<FName, FName>& ChangedFolder : ChangedFolders)
	{
		// Update FoldersProperties
		static FActorFolderProps DefaultFolderProperties;
		FFolder OldFolder(InRootObject, ChangedFolder.Key);
		FFolder NewFolder(InRootObject, ChangedFolder.Value);
		FActorFolderProps* OldFolderProperties = Owner.FoldersProperties.Find(OldFolder);
		FActorFolderProps FolderProperties = OldFolderProperties ? *OldFolderProperties : DefaultFolderProperties;
		Owner.ReAddFolder_Internal(OldFolder, NewFolder, FolderProperties);
			
		Owner.BroadcastOnActorFolderMoved(OldFolder, NewFolder);
	}
}

UActorFolder* FWorldPersistentFolders::GetActorFolder(const FFolder& InFolder, UWorld* InWorld, bool bInAllowCreate)
{
	if (InFolder.IsNone())
	{
		return nullptr;
	}

	UActorFolder* ActorFolder = InFolder.GetActorFolder();
	return (!ActorFolder && bInAllowCreate) ? CreateActorFolder(InFolder, InWorld) : ActorFolder;
}

ULevel* FWorldPersistentFolders::GetRootObjectContainer(const FFolder& InFolder, UWorld* InWorld)
{
	if (InFolder.IsRootObjectValid())
	{
		return InFolder.GetRootObjectAssociatedLevel();
	}
	else if (InWorld)
	{
		return InWorld->PersistentLevel;
	}
	return nullptr;
}

UActorFolder* FWorldPersistentFolders::CreateActorFolder(const FFolder& InFolder, UWorld* InWorld)
{
	UActorFolder* ActorFolder = nullptr;

	if (!InFolder.IsNone())
	{
		// Create upper chain as well
		const FFolder ParentFolder = InFolder.GetParent();
		UActorFolder* ActorParentFolder = CreateActorFolder(ParentFolder, InWorld);

		ActorFolder = GetActorFolder(InFolder, InWorld);
		if (!ActorFolder)
		{
			ULevel* Level = GetRootObjectContainer(InFolder, InWorld);
			if (ensure(IsValid(Level)))
			{
				check(Level->IsUsingActorFolders());
				const FString FolderLabel = InFolder.GetLeafName().ToString();
				ActorFolder = UActorFolder::Create(Level, FolderLabel, ActorParentFolder);
			}
		}
	}

	return ActorFolder;
}

UActorFolder* FWorldPersistentFolders::GetActorFolder(const FFolder& InFolder) const
{
	return GetActorFolder(InFolder, GetWorld());
}

UActorFolder* FWorldPersistentFolders::CreateActorFolder(const FFolder& InFolder)
{
	return CreateActorFolder(InFolder, GetWorld());
}

#undef LOCTEXT_NAMESPACE