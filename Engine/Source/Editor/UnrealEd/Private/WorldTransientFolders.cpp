// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "WorldTransientFolders.h"
#include "WorldFolders.h"
#include "EngineUtils.h"

#define LOCTEXT_NAMESPACE "UnrealEd.WorldTransientFolders"

/** Convert an old path to a new path, replacing an ancestor branch with something else */
static FName OldPathToNewPath(const FString& InOldBranch, const FString& InNewBranch, const FString& PathToMove)
{
	return FName(*(InNewBranch + PathToMove.RightChop(InOldBranch.Len())));
}

bool FWorldTransientFolders::RenameFolder(const FFolder& OldPath, const FFolder& NewPath)
{
	UWorld* WorldPtr = Owner.World.Get();
	check(WorldPtr);

	check(OldPath.GetRootObject() == NewPath.GetRootObject());
	const FFolder::FRootObject& RootObject = OldPath.GetRootObject();
	const FString OldPathString = OldPath.ToString();
	const FString NewPathString = NewPath.ToString();

	TSet<FFolder> RenamedFolders;
	bool RenamedFolder = false;

	// Move any folders we currently hold - old ones will be deleted later
	auto FoldersPropertiesCopy = Owner.FoldersProperties;
	for (const auto& Pair : FoldersPropertiesCopy)
	{
		const FFolder& Path = Pair.Key;
		const FString FolderPath = Path.ToString();

		if (OldPath == Path || Path.IsChildOf(OldPath))
		{
			const FFolder NewFolder = FFolder(RootObject, OldPathToNewPath(OldPathString, NewPathString, FolderPath));

			// Needs to be done this way otherwise case insensitive comparison is used.
			bool ContainsFolder = false;
			for (const auto& FolderPair : Owner.FoldersProperties)
			{
				if (FolderPair.Key.GetRootObject() == NewFolder.GetRootObject() && FolderPair.Key.GetPath().IsEqual(NewFolder.GetPath(), ENameCase::CaseSensitive))
				{
					ContainsFolder = true;
					break;
				}
			}

			if (!ContainsFolder)
			{
				// Use the existing properties for the folder if we have them
				if (FActorFolderProps* ExistingProperties = Owner.FoldersProperties.Find(Path))
				{
					Owner.AddFolder_Internal(NewFolder, *ExistingProperties);
				}
				else
				{
					// Otherwise use default properties
					Owner.AddFolder_Internal(NewFolder);
				}
				Owner.BroadcastOnActorFolderMoved(Path, NewFolder);
				Owner.BroadcastOnActorFolderCreated(NewFolder);
			}

			// case insensitive compare as we don't want to remove the folder if it has the same name
			if (Path != NewFolder)
			{
				RenamedFolders.Add(Path);
			}

			RenamedFolder = true;
		}
	}

	// Now that we have folders created, move any actors that ultimately reside in that folder too
	for (auto ActorIt = FActorIterator(WorldPtr); ActorIt; ++ActorIt)
	{
		// Skip actors not part of the same root
		if (ActorIt->GetFolderRootObject() != RootObject)
		{
			continue;
		}

		// copy, otherwise it returns the new value when set later
		const FFolder OldActorPath = ActorIt->GetFolder();
		if (OldActorPath.IsNone())
		{
			continue;
		}

		if (OldActorPath == OldPath || OldActorPath.IsChildOf(OldPath))
		{
			ActorIt->SetFolderPath_Recursively(OldPathToNewPath(OldPathString, NewPathString, OldActorPath.ToString()));
			const FFolder NewActorPath = ActorIt->GetFolder();

			// case insensitive compare as we don't want to remove the folder if it has the same name
			if (OldActorPath != NewActorPath)
			{
				RenamedFolders.Add(OldActorPath);
			}

			RenamedFolder = true;
		}
	}

	// Cleanup any old folders
	for (const FFolder& Folder : RenamedFolders)
	{
		Owner.RemoveFolder_Internal(Folder);
		Owner.BroadcastOnActorFolderDeleted(Folder);
	}

	return RenamedFolder;
}

#undef LOCTEXT_NAMESPACE