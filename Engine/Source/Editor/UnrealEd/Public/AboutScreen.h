// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Margin.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class ITableRow;
class SButton;
class STableViewBase;
struct FSlateBrush;

/**
 * About screen contents widget                   
 */
class SAboutScreen : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SAboutScreen )
		{}
	SLATE_END_ARGS()

	/**
	 * Constructs the about screen widgets                   
	 */
	UNREALED_API void Construct(const FArguments& InArgs);

private:

	struct FLineDefinition
	{
	public:
		FText Text;
		int32 FontSize;
		FLinearColor TextColor;
		FMargin Margin;

		FLineDefinition( const FText& InText )
			: Text( InText )
			, FontSize( 9 )
			, TextColor( FLinearColor(0.5f, 0.5f, 0.5f) )
			, Margin( FMargin(6.f, 0.f, 0.f, 0.f) )
		{

		}

		FLineDefinition( const FText& InText, int32 InFontSize, const FLinearColor& InTextColor, const FMargin& InMargin )
			: Text( InText )
			, FontSize( InFontSize )
			, TextColor( InTextColor )
			, Margin( InMargin )
		{

		}
	};

	TArray< TSharedRef< FLineDefinition > > AboutLines;
	TSharedPtr<SButton> UEButton;
	TSharedPtr<SButton> EpicGamesButton;

	/** 
	 * Makes the widget for the checkbox items in the list view 
	 */
	TSharedRef<ITableRow> MakeAboutTextItemWidget(TSharedRef<FLineDefinition> Item, const TSharedRef<STableViewBase>& OwnerTable);

	FReply OnUEButtonClicked();
	FReply OnEpicGamesButtonClicked();
	FReply OnClose();
};

