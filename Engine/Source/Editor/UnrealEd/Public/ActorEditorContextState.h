// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SubclassOf.h"
#include "ActorEditorContextState.generated.h"

UCLASS(MinimalAPI, Within = ActorEditorContextStateCollection)
class UActorEditorContextClientState : public UObject
{
	GENERATED_BODY()
};

UCLASS(MinimalAPI)
class UActorEditorContextStateCollection : public UObject
{
	GENERATED_BODY()

public:
	template <class TState>
	const TState* GetState() const
	{
		return Cast<TState>(ClientStates.FindRef(TState::StaticClass()));
	}

	void AddState(UActorEditorContextClientState* InState)
	{
		ClientStates.Emplace(InState->GetClass(), InState);
	}

	bool IsEmpty() const
	{
		return ClientStates.IsEmpty();
	}

private:
	void Reset()
	{
		ClientStates.Reset();
	}

	UPROPERTY(VisibleAnywhere, Instanced, Category = States, meta = (DisplayName = "Context"))
	TMap<TSubclassOf<UActorEditorContextClientState>, TObjectPtr<UActorEditorContextClientState>> ClientStates;

	friend class UActorEditorContextSubsystem;
};
