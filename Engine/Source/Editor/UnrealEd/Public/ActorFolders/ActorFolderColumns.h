// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Folder.h"

#include "ActorFolderColumns.generated.h"

/**
 * Column that stores a constructed FFolder
 */
UE_EXPERIMENTAL(5.8, "The TEDS+Folders integration is experimental and the API subject to change.")
USTRUCT(meta = (DisplayName = "FFolder compatibility"))
struct FFolderCompatibilityColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	FFolder Folder;
};

/**
 * Event column that is added to a folder row for one frame when it is duplicated or pasted, to redirect to the new folder
 */
UE_EXPERIMENTAL(5.8, "The TEDS+Folders integration is experimental and the API subject to change.")
USTRUCT(meta = (DisplayName = "FFolder duplicate event"))
struct FFolderDuplicateEventColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	FFolder DuplicatedFolder;
};

/**
 * Column that stores the FFolder::FRootObject of the row's owning level / level instance / world.
 */
UE_EXPERIMENTAL(5.8, "The TEDS+Folders integration is experimental and the API subject to change.")
USTRUCT(meta = (DisplayName = "FFolder root object"))
struct FRootObjectColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	FFolder::FRootObject RootObject = FFolder::GetInvalidRootObject();
};

/**
 * Tag placed on folder rows whose visibility needs recomputing. Removed once the folder has been
 * processed by the PostOrder visibility walk.
 */
UE_EXPERIMENTAL(5.8, "The TEDS+Folders integration is experimental and the API subject to change.")
USTRUCT()
struct FFolderVisibilityDirtyTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};