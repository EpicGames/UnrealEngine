// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Folder.h"

#define UE_API UNREALED_API

// Utility functions for the actor folder integration with TEDS
namespace UE::Editor::DataStorage::ActorFolders
{
	// Get the name of the table that folder rows are stored in
	UE_EXPERIMENTAL(5.8, "The TEDS+Folders integration is experimental and the API subject to change.")
	UE_API FName GetActorFolderTableName();

	// Register the given folder in TEDS
	UE_EXPERIMENTAL(5.8, "The TEDS+Folders integration is experimental and the API subject to change.")
	UE_API RowHandle RegisterFolderInTeds(ICoreProvider* Storage, const FFolder& Folder, UWorld* World, bool bRecursivelyAddParents = false);

	// Register the given folder in TEDS from a query callback
	UE_EXPERIMENTAL(5.8, "The TEDS+Folders integration is experimental and the API subject to change.")
	UE_API void RegisterFolderInTeds(IQueryContext& Context, ICoreProvider* Storage, const FFolder& Folder, TWeakObjectPtr<UWorld> World, bool bRecursivelyAddParents = false);

	// Set the default columns (+ data) for a folder row
	UE_EXPERIMENTAL(5.8, "The TEDS+Folders integration is experimental and the API subject to change.")
	UE_API void SetFolderColumns(ICoreProvider* Storage, RowHandle Row, UWorld* World, const FFolder& Folder);

	// Unregister a folder from TEDS by looking up the folder's row
	UE_EXPERIMENTAL(5.8, "The TEDS+Folders integration is experimental and the API subject to change.")
	UE_API void UnregisterFolderFromTeds(ICoreProvider* Storage, const FFolder& Folder);
	
	// Unregister a folder from TEDS from a query callback
	UE_EXPERIMENTAL(5.8, "The TEDS+Folders integration is experimental and the API subject to change.")
	UE_API void UnregisterFolderFromTeds(IQueryContext& Context, ICoreProvider* Storage, const FFolder& Folder);

	// Look up the row handle for a folder using the direct API
	UE_EXPERIMENTAL(5.8, "The TEDS+Folders integration is experimental and the API subject to change.")
	UE_API RowHandle LookupMappedRow(const ICoreProvider* Storage, const FFolder& Folder);

	// Look up the row handle for a folder from inside a query callback
	UE_EXPERIMENTAL(5.8, "The TEDS+Folders integration is experimental and the API subject to change.")
	UE_API RowHandle LookupMappedRow(const IQueryContext& Context, const FFolder& Folder);

	// Remap a folder if the index has changed due to data changes
	UE_EXPERIMENTAL(5.8, "The TEDS+Folders integration is experimental and the API subject to change.")
	UE_API RowHandle RemapFolder(ICoreProvider* Storage, const FFolder& InOldFolder, const FFolder& InNewFolder, UWorld* World);

	// For a given ChildRow, add a parent folder that has not been registered in TEDS yet.
	UE_EXPERIMENTAL(5.8, "The TEDS+Folders integration is experimental and the API subject to change.")
	UE_API void SetUnresolvedFolderParent(IQueryContext& Context, RowHandle ChildRow, const FFolder& ParentFolder);
	
	// Get the mapping domain for a given folder
	UE_EXPERIMENTAL(5.8, "The TEDS+Folders integration is experimental and the API subject to change.")
	UE_API FName GetFolderMappingDomain(const FFolder& Folder);
	
	// Get the mapping key for a given folder
	UE_EXPERIMENTAL(5.8, "The TEDS+Folders integration is experimental and the API subject to change.")
	UE_API FMapKey GetFolderIndex(const FFolder& Folder);
	
	// When an actor is pasted/duplicated, fixup the parent folder if it was part of the operation too
	UE_EXPERIMENTAL(5.8, "The TEDS+Folders integration is experimental and the API subject to change.")
	UE_API void FixupActorPostDuplicate(AActor* Actor);

	// If InFolder is currently selected in the level editor's actor selection set, deselect it and return true.
	UE_EXPERIMENTAL(5.8, "The TEDS+Folders integration is experimental and the API subject to change.")
	UE_API bool DeselectFolderIfSelected(ICoreProvider* Storage, const FFolder& InFolder);

	// Select InFolder in the level editor's actor selection set (if the TEDS row is available).
	UE_EXPERIMENTAL(5.8, "The TEDS+Folders integration is experimental and the API subject to change.")
	UE_API void SelectFolder(ICoreProvider* Storage, const FFolder& InFolder);
	
	// Helper to get the mapping domain and key for the first viable parent for an actor in the editor
	UE_EXPERIMENTAL(5.8, "The TEDS+Folders integration is experimental and the API subject to change.")
	UE_API bool GetEditorParentForActor(const AActor* InActor, FName& OutParentMappingDomain, FMapKey& OutParentMappingKey);

}

#undef UE_API