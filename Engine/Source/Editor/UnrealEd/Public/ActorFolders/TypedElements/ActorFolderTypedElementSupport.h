// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Folder.h"
#include "DataStorage/Handles.h"
#include "Elements/Framework/TypedElementData.h" // UE_DECLARE_TYPED_ELEMENT_DATA_RTTI
#include "Elements/Framework/TypedElementHandle.h"
#include "UObject/NameTypes.h"

#define UE_API UNREALED_API

namespace UE::Editor::ActorFolders
{
	// Name associated with the element, which can be used to register interfaces
	const FName NAME_ActorFolder(TEXT("ActorFolderTypedElementName"));
	
	struct FActorFolderElementData
	{
		UE_DECLARE_TYPED_ELEMENT_DATA_RTTI(FActorFolderElementData);
		
		// Folders in TEv1 use their state in TEDS as the source of truth, so we just need to store the row handle
		DataStorage::RowHandle FolderRow = DataStorage::InvalidRowHandle;
	};
	
	/**
	 * Gets a typed element handle for the folder from a global store.
	 */ 
	UE_EXPERIMENTAL(5.8, "The TEv1+Folders integration is experimental and the API subject to change.")
	UE_API FTypedElementHandle AcquireTypedElementHandle(DataStorage::RowHandle FolderRow, const bool bAllowCreate);

	/**
	 * Destroys a typed element handle for the folder in the global store.
	 */ 
	UE_EXPERIMENTAL(5.8, "The TEv1+Folders integration is experimental and the API subject to change.")
	UE_API void DestroyTypedElementHandle(DataStorage::RowHandle FolderRow);

	/**
	 * If the given typed element handle represents a folder, get the row handle.
	 */ 
	UE_EXPERIMENTAL(5.8, "The TEv1+Folders integration is experimental and the API subject to change.")
	UE_API DataStorage::RowHandle GetFolderRow(const FTypedElementHandle& InElementHandle);
	
	/**
	 * If the given typed element handle represents a folder, get the underlying FFolder.
	 */ 
	UE_EXPERIMENTAL(5.8, "The TEv1+Folders integration is experimental and the API subject to change.")
	UE_API FFolder GetFolder(const FTypedElementHandle& InElementHandle);
}

#undef UE_API