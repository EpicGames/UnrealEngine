// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ActorGroupingUtils.generated.h"

/**
 * Helper class for grouping actors in the level editor
 */
UCLASS(transient, MinimalAPI)
class UActorGroupingUtils : public UObject
{
	GENERATED_BODY()
public:
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Actor Grouping")
	static bool IsGroupingActive() { return bGroupingActive; }

	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Actor Grouping")
	static UNREALED_API void SetGroupingActive(bool bInGroupingActive);

	/**
	 * Convenience method for accessing grouping utils in a blueprint or script
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Actor Grouping", DisplayName="Get Actor Grouping Utils")
	static UNREALED_API UActorGroupingUtils* Get();

	/**
	 * Creates a new group from the current selection removing the actors from any existing groups they are already in
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Actor Grouping")
	UNREALED_API virtual AGroupActor* GroupSelected();

	/**
	 * Creates a new group from the provided list of actors removing the actors from any existing groups they are already in
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Actor Grouping")
	UNREALED_API virtual AGroupActor* GroupActors(const TArray<AActor*>& ActorsToGroup);

	/**
	 * Disbands any groups in the current selection, does not attempt to maintain any hierarchy
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Actor Grouping")
	UNREALED_API virtual void UngroupSelected();

	/**
	 * Disbands any groups that the provided actors belong to, does not attempt to maintain any hierarchy
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Actor Grouping")
	UNREALED_API virtual void UngroupActors(const TArray<AActor*>& ActorsToUngroup);

	/**
	* Locks any groups in the current selection
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Actor Grouping")
	UNREALED_API virtual void LockSelectedGroups();

	/**
	 * Unlocks any groups in the current selection
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Actor Grouping")
	UNREALED_API virtual void UnlockSelectedGroups();

	/**
	 * Activates "Add to Group" mode which allows the user to select a group to append current selection
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Actor Grouping")
	UNREALED_API virtual void AddSelectedToGroup();

	/**
	 * Removes any groups or actors in the current selection from their immediate parent.
	 * If all actors/subgroups are removed, the parent group will be destroyed.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Actor Grouping")
	UNREALED_API virtual void RemoveSelectedFromGroup();

	/**
	 * Check if the provided list of actors can be grouped together
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Actor Grouping")
	UNREALED_API bool CanGroupActors(const TArray<AActor*>& ActorsToGroup) const;

	/*
	 * Check if the currently selected actors can be grouped together
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Actor Grouping")
	UNREALED_API bool CanGroupSelectedActors() const;

	/**
	 * External delegates that are run to check if a set of actors can be grouped together
	 */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FCanGroupActors, const TArray<AActor*>&);
	
	/*
	 * Add a filter that is checked to see if a set of actors can be grouped together
	 */
	UNREALED_API void AddCanGroupActorsDelegate(const FName& Owner, const FCanGroupActors& InGroupActorsDelegate);

	/*
	 * Remove the filter checked before grouping actors registered by the provided owner
	 */
	UNREALED_API void RemoveCanGroupActorsDelegate(const FName& Owner);

protected:
	static UNREALED_API bool bGroupingActive;
	static UNREALED_API FSoftClassPath ClassToUse;
	static UNREALED_API TMap<FName, FCanGroupActors> CanGroupActorsDelegates;
};


