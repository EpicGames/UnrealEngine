// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "Modules/ModuleManager.h"
#include "Misc/PackageName.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWindow.h"
#include "Animation/Skeleton.h"
#include "Animation/AnimationAsset.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "Engine/SkeletalMesh.h"
#include "UObject/SoftObjectPtr.h"

class FMenuBuilder;
class UAnimBlueprint;
class UAnimBoneCompressionSettings;
class UAnimSequence;
class UEdGraph;
class UPoseWatch;
class UEdGraphNode;
class UAnimBlueprintGeneratedClass;
class UAnimGraphNode_Base;

/** dialog to prompt users to decide an animation asset name */
class SCreateAnimationAssetDlg : public SWindow
{
public:
	SLATE_BEGIN_ARGS(SCreateAnimationAssetDlg)
		: _PrioritizeLastPathUsed(false)
	{
	}
		SLATE_ARGUMENT(FText, DefaultAssetPath)
		SLATE_ARGUMENT(bool, PrioritizeLastPathUsed)
	SLATE_END_ARGS()

		SCreateAnimationAssetDlg()
		: UserResponse(EAppReturnType::Cancel)
	{
	}

	void Construct(const FArguments& InArgs);

public:
	/** Displays the dialog in a blocking fashion */
	EAppReturnType::Type ShowModal();

	/** Gets the resulting asset path */
	FString GetAssetPath();

	/** Gets the resulting asset name */
	FString GetAssetName();

	/** Gets the resulting full asset path (path+'/'+name) */
	FString GetFullAssetPath();

protected:
	void OnPathChange(const FString& NewPath);
	void OnNameChange(const FText& NewName, ETextCommit::Type CommitInfo);
	FReply OnButtonClick(EAppReturnType::Type ButtonID);

	bool ValidatePackage();

	EAppReturnType::Type UserResponse;
	FText AssetPath;
	FText AssetName;

	static FText LastUsedAssetPath;
};

/** A struct containing the settings to control the SAnimationCompressionSelectionDialog creation. */
struct FAnimationCompressionSelectionDialogConfig
{
	FText DialogTitleOverride;
	FVector2D WindowSizeOverride;

	UAnimBoneCompressionSettings* DefaultSelectedAsset;

	FAnimationCompressionSelectionDialogConfig()
		: WindowSizeOverride(ForceInitToZero)
		, DefaultSelectedAsset(nullptr)
	{}
};

/** Dialog to prompt user to select an animation compression settings asset. */
class SAnimationCompressionSelectionDialog : public SCompoundWidget
{
public:
	/** Called from the Dialog when an asset has been selected. */
	DECLARE_DELEGATE_OneParam(FOnAssetSelected, const FAssetData& /*SelectedAsset*/);

	SLATE_BEGIN_ARGS(SAnimationCompressionSelectionDialog) {}

	SLATE_END_ARGS()

	UNREALED_API SAnimationCompressionSelectionDialog();
	UNREALED_API virtual ~SAnimationCompressionSelectionDialog();

	UNREALED_API virtual void Construct(const FArguments& InArgs, const FAnimationCompressionSelectionDialogConfig& InConfig);

	/** Sets the delegate handler for when an open operation is committed */
	UNREALED_API void SetOnAssetSelected(const FOnAssetSelected& InHandler);

private:
	void DoSelectAsset(const FAssetData& SelectedAsset);
	FReply OnConfirmClicked();
	FReply OnCancelClicked();
	void CloseDialog();
	void OnAssetSelected(const FAssetData& AssetData);
	void OnAssetsActivated(const TArray<FAssetData>& SelectedAssets, EAssetTypeActivationMethod::Type ActivationType);
	bool IsConfirmButtonEnabled() const;

	/** Asset Picker used by the dialog */
	TSharedPtr<SWidget> AssetPicker;

	/** The assets that are currently selected in the asset picker */
	TArray<FAssetData> CurrentlySelectedAssets;

	/** Used to specify that a valid asset was chosen */
	bool bValidAssetChosen;

	/** Fired when assets are chosen for open. Only fired in open dialogs. */
	FOnAssetSelected OnAssetSelectedHandler;

	/** Used to get the currently selected assets */
	FGetCurrentSelectionDelegate GetCurrentSelectionDelegate;
};

/** Defines FCanExecuteAction delegate interface. Returns false to force the caller to delete the just created assets*/
DECLARE_DELEGATE_RetVal_OneParam(bool, FAnimAssetCreated, TArray<class UObject*>);

//Animation editor utility functions
namespace AnimationEditorUtils
{
	UNREALED_API FAssetData CreateModalAnimationCompressionSelectionDialog(const FAnimationCompressionSelectionDialogConfig& InConfig);

	UNREALED_API void CreateAnimationAssets(const TArray<TSoftObjectPtr<UObject>>& SkeletonsOrSkeletalMeshes, TSubclassOf<UAnimationAsset> AssetClass, const FString& InPrefix, FAnimAssetCreated AssetCreated, UObject* NameBaseObject = nullptr, bool bDoNotShowNameDialog = false, bool bAllowReplaceExisting = false);
	
	UNREALED_API void CreateNewAnimBlueprint(TArray<TWeakObjectPtr<UObject>> SkeletonsOrSkeletalMeshes, FAnimAssetCreated AssetCreated, bool bInContentBrowser);
	UNREALED_API void CreateNewAnimBlueprint(TArray<TSoftObjectPtr<UObject>> SkeletonsOrSkeletalMeshes, FAnimAssetCreated AssetCreated, bool bInContentBrowser);
	UNREALED_API void FillCreateAssetMenu(FMenuBuilder& MenuBuilder, const TArray<TSoftObjectPtr<UObject>>& SkeletonsOrSkeletalMeshes, FAnimAssetCreated AssetCreated, bool bInContentBrowser=true);
	UNREALED_API void CreateUniqueAssetName(const FString& InBasePackageName, const FString& InSuffix, FString& OutPackageName, FString& OutAssetName);

	/** Applies the animation compression codecs to the sequence list with optional override settings */
	UNREALED_API bool ApplyCompressionAlgorithm(TArray<UAnimSequence*>& AnimSequencePtrs, UAnimBoneCompressionSettings* OverrideSettings);

	// template version of simple creating animation asset
	template< class T >
	T* CreateAnimationAsset(UObject* SkeletonOrSkeletalMesh, const FString& AssetPath, const FString& InPrefix)
	{
		USkeletalMesh* SkeletalMesh = nullptr;
		USkeleton* Skeleton = Cast<USkeleton>(SkeletonOrSkeletalMesh);
		if (Skeleton == nullptr)
		{
			SkeletalMesh = CastChecked<USkeletalMesh>(SkeletonOrSkeletalMesh);
			Skeleton = SkeletalMesh->GetSkeleton();
		}

		if (Skeleton)
		{
			FString Name;
			FString PackageName;
			// Determine an appropriate name
			CreateUniqueAssetName(AssetPath, InPrefix, PackageName, Name);

			// Create the asset, and assign its skeleton
			FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
			T* NewAsset = Cast<T>(AssetToolsModule.Get().CreateAsset(Name, FPackageName::GetLongPackagePath(PackageName), T::StaticClass(), NULL));

			if (NewAsset)
			{
				NewAsset->SetSkeleton(Skeleton);
				if (SkeletalMesh)
				{
					NewAsset->SetPreviewMesh(SkeletalMesh);
				}
				NewAsset->MarkPackageDirty();
			}

			return NewAsset;
		}

		return nullptr;
	}
	
	// The following functions are used to fix subgraph arrays for assets
	UNREALED_API void RegenerateSubGraphArrays(UAnimBlueprint* Blueprint);
	void RegenerateGraphSubGraphs(UAnimBlueprint* OwningBlueprint, UEdGraph* GraphToFix);
	void RemoveDuplicateSubGraphs(UEdGraph* GraphToClean);
	UNREALED_API void FindChildGraphsFromNodes(UEdGraph* GraphToSearch, TArray<UEdGraph*>& ChildGraphs);

	// Is the supplied UEdGraph an Animation Graph
	UNREALED_API bool IsAnimGraph(UEdGraph* Graph);

	int32 GetPoseWatchNodeLinkID(UPoseWatch* PoseWatch, OUT UAnimBlueprintGeneratedClass*& AnimBPGenClass);
	UNREALED_API void SetPoseWatch(UPoseWatch* PoseWatch, UAnimBlueprint* AnimBlueprintIfKnown = nullptr);
	UNREALED_API UPoseWatch* FindPoseWatchForNode(const UEdGraphNode* Node, UAnimBlueprint* AnimBlueprintIfKnown=nullptr);
	UNREALED_API UPoseWatch* MakePoseWatchForNode(UAnimBlueprint* AnimBlueprint, UEdGraphNode* Node);
	UNREALED_API void RemovePoseWatch(UPoseWatch* PoseWatch, UAnimBlueprint* AnimBlueprintIfKnown=nullptr);
	UNREALED_API void RemovePoseWatchFromNode(UEdGraphNode* Node, UAnimBlueprint* AnimBlueprint);
	UNREALED_API void RemovePoseWatchesFromGraph(UAnimBlueprint* AnimBlueprint, class UEdGraph* Graph);

	// Delegate fired when a pose watch is added or removed
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPoseWatchesChanged, UAnimBlueprint* /*InAnimBlueprint*/, UEdGraphNode* /*InNode*/);
	UNREALED_API FOnPoseWatchesChanged& OnPoseWatchesChanged();

	UNREALED_API void SetupDebugLinkedAnimInstances(UAnimBlueprint* InAnimBlueprint, UObject* InRootObjectBeingDebugged);

	//////////////////////////////////////////////////////////////////////////////////////////

	template <typename TFactory, typename T>
	void ExecuteNewAnimAsset(TArray<TSoftObjectPtr<UObject>> SkeletonsOrSkeletalMeshes, const FString InSuffix, FAnimAssetCreated AssetCreated, bool bInContentBrowser, bool bAllowReplaceExisting)
	{
		if(bInContentBrowser && SkeletonsOrSkeletalMeshes.Num() == 1)
		{
			USkeletalMesh* SkeletalMesh = nullptr;
			USkeleton* Skeleton = Cast<USkeleton>(SkeletonsOrSkeletalMeshes[0].LoadSynchronous());
			if (Skeleton == nullptr)
			{
				SkeletalMesh = CastChecked<USkeletalMesh>(SkeletonsOrSkeletalMeshes[0].LoadSynchronous());
				Skeleton = SkeletalMesh->GetSkeleton();
			}

			if(Skeleton)
			{
				// Determine an appropriate name for inline-rename
				FString Name;
				FString PackageName;
				CreateUniqueAssetName(Skeleton->GetOutermost()->GetName(), InSuffix, PackageName, Name);

				TFactory* Factory = NewObject<TFactory>();
				Factory->TargetSkeleton = Skeleton;
				Factory->PreviewSkeletalMesh = SkeletalMesh;

				FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
				ContentBrowserModule.Get().CreateNewAsset(Name, FPackageName::GetLongPackagePath(PackageName), T::StaticClass(), Factory);

				if(AssetCreated.IsBound())
				{
					// @TODO: this doesn't work
					//FString LongPackagePath = FPackageName::GetLongPackagePath(PackageName);
					UObject* 	Parent = FindPackage(NULL, *PackageName);
					UObject* NewAsset = FindObject<UObject>(Parent, *Name, EFindObjectFlags::None);
					if(NewAsset)
					{
						TArray<UObject*> NewAssets;
						NewAssets.Add(NewAsset);
						if (!AssetCreated.Execute(NewAssets))
						{
							//Destroy the assets we just create
							for (UObject* ObjectToDelete : NewAssets)
							{
								ObjectToDelete->ClearFlags(RF_Standalone | RF_Public);
								ObjectToDelete->RemoveFromRoot();
								ObjectToDelete->MarkAsGarbage();
							}
							CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
						}
					}
				}
			}
		}
		else
		{
			UObject* NameBaseObject = nullptr;
			const bool bDoNotShowNameDialog = false;

			CreateAnimationAssets(SkeletonsOrSkeletalMeshes, T::StaticClass(), InSuffix, AssetCreated, NameBaseObject, bDoNotShowNameDialog, bAllowReplaceExisting);
		}
	}
} // namespace AnimationEditorUtils
