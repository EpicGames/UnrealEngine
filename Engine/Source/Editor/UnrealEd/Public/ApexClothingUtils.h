// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"
UE_DEPRECATED_HEADER(5.7, "Apex clothing is no longer supported, remove this file from your header inclusions.")

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "CoreMinimal.h"
#include "EngineDefines.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7

class USkeletalMesh;

namespace ApexClothingUtils
{
	UE_DEPRECATED(5.7, "Apex clothing is no longer supported, this implementation will be removed.")
	UNREALED_API void RemoveAssetFromSkeletalMesh(USkeletalMesh* SkelMesh, uint32 AssetIndex, bool bReleaseAsset = true, bool bRecreateSkelMeshComponent = false);
}
