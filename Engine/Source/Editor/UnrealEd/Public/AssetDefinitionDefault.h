// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetDefinition.h"

#include "AssetDefinitionDefault.generated.h"

UCLASS(Abstract, MinimalAPI)
class UAssetDefinitionDefault : public UAssetDefinition
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	UNREALED_API virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	UNREALED_API virtual EAssetCommandResult PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const override;
	// UAssetDefinition End
};

namespace UE::Editor
{
	/**
	 * This function will find or set a new thumbnail info on the object if it either does not exist or the types don't
	 * match.  This function has some requirements, such as the property MUST minimally exist as such,
	 * 
	 * UPROPERTY(Instanced)
	 * TObjectPtr<class UThumbnailInfo> ThumbnailInfo;
	 * 
	 * If your class needs the thumbnail to be set using a Setter/Getter, that's ok, you can have 
	 * Getter=GetThumbnailInfo, Setter=SetThumbnailInfo, in the UProperty declaration to take advantage of native
	 * setters and getters.
	 */
	UNREALED_API UThumbnailInfo* FindOrCreateThumbnailInfo(UObject* AssetObject, TSubclassOf<UThumbnailInfo> ThumbnailClass);


	UNREALED_API bool TrySetExistingThumbnailInfo(UObject* InAssetObject, UThumbnailInfo* InThumbnailInfo);

	template<typename ThumbnailClass>
	ThumbnailClass* FindOrCreateThumbnailInfo(UObject* AssetObject)
	{
		return Cast<ThumbnailClass>(FindOrCreateThumbnailInfo(AssetObject, ThumbnailClass::StaticClass()));
	}
}
