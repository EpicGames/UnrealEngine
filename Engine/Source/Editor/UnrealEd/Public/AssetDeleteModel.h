// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

struct FAssetData;

/**
 * The pending deleted object.
 */
struct FPendingDelete : TSharedFromThis<FPendingDelete>
{
public:
	FPendingDelete( UObject* InObject );

	/** Checks for references on disk and in memory for this object filling out all information. */
	void CheckForReferences();

	/** Gets the object being deleted. */
	UObject* GetObject() { return Object; }

	/** Checks if the object is contained in the pending delete object hierarchy */
	bool IsObjectContained(const UObject* InObject) const;

	/** Checks if the package is contained in the asset package being deleted */
	bool IsAssetContained(const FName& PackageName) const;

	/** Is the pending deleted object referenced in memory by something other than the undo stack; INCLUDES PENDING DELETES */
	bool IsReferencedInMemoryByNonUndo() const { return bIsReferencedInMemoryByNonUndo; }

	/** Is the pending deleted object referenced in memory by the undo stack; INCLUDES PENDING DELETES */
	bool IsReferencedInMemoryByUndo() const { return bIsReferencedInMemoryByUndo; }

	/** Returns if the pending delete is internal, and need not be shown to the user. */
	bool IsInternal() const { return bIsInternal; }

	/** Sets if the pending delete is internal, and need not be shown to the user. */
	void IsInternal(bool Value) { bIsInternal = Value; }

	/** Support comparing for unique insertions */
	bool operator == ( const FPendingDelete& Other ) const;

	/** The on disk references to this object */
	TArray<FName> DiskReferences;

	/** In memory references to this object (*excluding* the undo buffer) */
	FReferencerInformationList MemoryReferences;

	/** The remaining disk references; EXCLUDES PENDING DELETES */
	int32 RemainingDiskReferences;
	/** The remaining memory references; EXCLUDES PENDING DELETES */
	int32 RemainingMemoryReferences;

private:
	/** The object to delete */
	UObject* Object;

	/** Internal objects being deleted that we need to make sure aren't counted as memory references. */
	TArray<UObject*> InternalObjects;

	/** A flag indicating that references have been checked, so don't check again. */
	bool bReferencesChecked;

	/** flag indicating if this object is referenced in memory by the engine (excluding the undo buffer). */
	bool bIsReferencedInMemoryByNonUndo;

	/** flag indicating if this object is referenced in memory by the undo stack. */
	bool bIsReferencedInMemoryByUndo;

	/**
	 * flag to control the visibility of this pending deleted object.  Some internal objects,
	 * like blueprint generated classes and skeleton classes need to be added to the list of pending
	 * deletes but users don't need to see them.
	 */
	bool bIsInternal;
};

/**
 * The model behind a delete operation, which is an asynchronous process because of all the checks
 * that must be performed against the GC for UObjects, and looking up references for assets through
 * the asset registry.
 */
class FAssetDeleteModel
{
public:
	/** States used to manage the async deletion process. */
	enum EState
	{
		// Waiting to start scanning
		Waiting = 0,
		// Begin scanning for references
		StartScanning,
		// Scan for references to the pending deleted assets
		Scanning,
		// check compatibility for replacing references
		UpdateActions,
		// Finished
		Finished,
	};

public:

	/** Constructor */
	UNREALED_API FAssetDeleteModel( const TArray<UObject*>& InObjectsToDelete );

	/** Destructor */
	UNREALED_API ~FAssetDeleteModel();

	/** Add an object to the list of pending deleted assets, this will invalidate the scanning state. */
	UNREALED_API void AddObjectToDelete(UObject* InObject);

	/** Returns the pending deleted assets. */
	const TArray< TSharedPtr< FPendingDelete > >* GetPendingDeletedAssets() const { return &PendingDeletes; };

	/** Returns a map of currently discovered source content files, and the number of times they are referenced by non-deleted assets */
	const TMap< FString, int32 >& GetPendingDeletedSourceFileCounts() const { return SourceFileToAssetCount; };

	/** Returns the current state of the deletion process */
	EState GetState() const { return State; }

	/** Gets the packages of the assets on disk that reference the pending deleted objects;  won't be accurate until the scanning process completes. */
	const TSet< FName >& GetAssetReferences() const { return OnDiskReferences; };

	/** Ticks the delete model which does a little work before returning so that we don't completely block when deleting a lot of things. */
	UNREALED_API void Tick( const float InDeltaTime );

	/** Returns true if the object is one of the pending deleted assets. */
	UNREALED_API bool IsObjectInPendingDeletes( const UObject* InObject ) const;

	/** Returns true if the package is one of the pending deleted assets. */
	UNREALED_API bool IsAssetInPendingDeletes( const FName& PackageName ) const;

	/** Deletes any source content files referenced by the assets */
	UNREALED_API void DeleteSourceContentFiles();

	/** Returns true if it is valid to delete the current objects with no problems. */
	UNREALED_API bool CanDelete() const;

	/** Performs the delete if it's possible. */
	UNREALED_API bool DoDelete();

	/** Returns true if it is valid to force the delete of the current assets. */
	UNREALED_API bool CanForceDelete() const;

	/** Performs a force delete on the pending deleted assets if possible. */
	UNREALED_API bool DoForceDelete();

	/** Returns true if it's valid to replace the references of the pending deleted objects. */
	UNREALED_API bool CanReplaceReferences() const;

	/** Returns true if it is valid to force the delete of the current assets with the provided asset. */
	UNREALED_API bool CanReplaceReferencesWith( const FAssetData& InAssetData ) const;

	/** Performs the replace references action if possible with the provided asset. */
	UNREALED_API bool DoReplaceReferences( const FAssetData& ReplaceReferencesWith );

	/** Gets the 0..1 progress of the scanning. */
	UNREALED_API float GetProgress() const;

	/** Gets the current text to display for the current progress of the scanning. */
	UNREALED_API FText GetProgressText() const;

	/** Is any of the pending deleted assets being referenced in memory. */
	bool IsAnythingReferencedInMemoryByNonUndo() const { return bIsAnythingReferencedInMemoryByNonUndo; }

	/** Is any of the pending deleted assets being referenced in the undo stack. */
	bool IsAnythingReferencedInMemoryByUndo() const { return bIsAnythingReferencedInMemoryByUndo; }

	/** Check whether we have any source files residing under monitored, mounted paths to delete */
	UNREALED_API bool HasAnySourceContentFilesToDelete() const;

	/** Goes to the next actor in the loaded level if it is available */
	UNREALED_API bool GoToNextReferenceInLevel() const;

	/** Gets the number of objects successfully deleted. */
	UNREALED_API int32 GetDeletedObjectCount() const;
	
	/** Fires whenever the state changes. */
	DECLARE_EVENT_OneParam( FAssetDeleteModel, FOnStateChanged, EState /*NewState*/ );
	FOnStateChanged& OnStateChanged()
	{
		return StateChanged;
	}

private:
	UNREALED_API void PrepareToDelete(UObject* InObject);

	/** Sets the current state of the model. */
	UNREALED_API void SetState( EState NewState );

	/** Computes the value that should be used for CanReplaceReferences */
	UNREALED_API bool ComputeCanReplaceReferences();

	/** Discover source file references for the specified object */
	UNREALED_API void DiscoverSourceFileReferences(FPendingDelete& PendingDelete);

private:

	/** Holds an event delegate that is executed when the state changes */
	FOnStateChanged StateChanged;

	/** The assets being deleted */
	TArray< TSharedPtr< FPendingDelete > > PendingDeletes;

	/** A running count of source content filename -> number of assets referencing. For files that are no longer referenced, the count will be 0. */
	TMap<FString, int32> SourceFileToAssetCount;

	/** On disk references to the currently to be deleted objects */
	TSet< FName > OnDiskReferences;

	/** The internal progress/state of the delete model which can take several frames to recalculate deletion validity */
	EState State;

	/** Pending objects can replaced so the 'Replace References' option should be available */
	bool bPendingObjectsCanBeReplaced;

	/** Is any of the pending deleted assets being referenced in memory. */
	bool bIsAnythingReferencedInMemoryByNonUndo;

	/** Is any of the pending deleted assets being referenced in the undo stack. */
	bool bIsAnythingReferencedInMemoryByUndo;

	/** A tick-to-tick state tracking variable so we know what pending deleted object we checked last. */
	int32 PendingDeleteIndex;

	/** The number of objects successfully deleted. */
	int32 ObjectsDeleted;
};
