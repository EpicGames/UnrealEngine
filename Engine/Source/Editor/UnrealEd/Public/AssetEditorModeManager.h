// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorModeManager.h"

class FPreviewScene;
class UWorld;

//////////////////////////////////////////////////////////////////////////
// FAssetEditorModeManager

class FAssetEditorModeManager : public FEditorModeTools
{
public:
	UNREALED_API FAssetEditorModeManager();
	UNREALED_API virtual ~FAssetEditorModeManager() override;

	// FEditorModeTools interface
	UNREALED_API virtual USelection* GetSelectedActors() const override;
	UNREALED_API virtual USelection* GetSelectedObjects() const override;
	UNREALED_API virtual USelection* GetSelectedComponents() const override;
	UNREALED_API virtual UWorld* GetWorld() const override;
	// End of FEditorModeTools interface

	UNREALED_API virtual void SetPreviewScene(FPreviewScene* NewPreviewScene);
	UNREALED_API FPreviewScene* GetPreviewScene() const;

protected:
	USelection* ActorSet = nullptr;
	USelection* ObjectSet = nullptr;
	USelection* ComponentSet = nullptr;
	FPreviewScene* PreviewScene = nullptr;
	TWeakObjectPtr<UWorld> PreviewSceneWorld = nullptr;
};
