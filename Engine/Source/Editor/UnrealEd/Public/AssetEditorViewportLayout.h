// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Stats/Stats.h"
#include "Misc/Attribute.h"
#include "Animation/CurveSequence.h"
#include "Layout/Visibility.h"
#include "Widgets/SWidget.h"
#include "Editor/UnrealEdTypes.h"
#include "Application/ThrottleManager.h"
#include "EditorViewportLayout.h"
#include "TickableEditorObject.h"
#include "SEditorViewport.h"

class FAssetEditorViewportLayout;
class SAssetEditorViewportsOverlay;
class SWindow;
class SAssetEditorViewport;
class FEditorViewportTabContent;
class FViewportTabContent;

/** Arguments for constructing a viewport */
struct FAssetEditorViewportConstructionArgs
{
	FAssetEditorViewportConstructionArgs()
		: ViewportType(LVT_Perspective)
		, bRealtime(false)
	{}

	/** The viewport's parent layout */
	TSharedPtr<FAssetEditorViewportLayout> ParentLayout;
	/** The viewport's desired type */
	ELevelViewportType ViewportType;
	/** Whether the viewport should default to realtime */
	bool bRealtime;
	/** A config key for loading/saving settings for the viewport */
	FName ConfigKey;
	/** Widget enabled attribute */
	TAttribute<bool> IsEnabled;
};
using AssetEditorViewportFactoryFunction = TFunction<TSharedRef<SAssetEditorViewport>(const FAssetEditorViewportConstructionArgs&)>;


namespace EditorViewportConfigurationNames
{
	static const FName TwoPanesHoriz("TwoPanesHoriz");
	static const FName TwoPanesVert("TwoPanesVert");
	static const FName ThreePanesLeft("ThreePanesLeft");
	static const FName ThreePanesRight("ThreePanesRight");
	static const FName ThreePanesTop("ThreePanesTop");
	static const FName ThreePanesBottom("ThreePanesBottom");
	static const FName FourPanesLeft("FourPanesLeft");
	static const FName FourPanesRight("FourPanesRight");
	static const FName FourPanesTop("FourPanesTop");
	static const FName FourPanesBottom("FourPanesBottom");
	static const FName FourPanes2x2("FourPanes2x2");
	static const FName OnePane("OnePane");
}

/**
* Overlay wrapper class so that we can cache the size of the widget
* It will also store the ViewportLayout data because that data can't be stored
* per app; it must be stored per viewport overlay in case the app that made it closes.
*/
class SAssetEditorViewportsOverlay : public SCompoundWidget
{

public:

	SLATE_BEGIN_ARGS(SAssetEditorViewportsOverlay) {}
		SLATE_DEFAULT_SLOT(FArguments, Content)
		SLATE_ARGUMENT(TSharedPtr<FViewportTabContent>, ViewportTab)
	SLATE_END_ARGS()

		UNREALED_API void Construct(const FArguments& InArgs);

	/** Default constructor */
	SAssetEditorViewportsOverlay()
		: CachedSize(FVector2D::ZeroVector)
	{}

	/** Overridden from SWidget */
	UNREALED_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/** Wraps SOverlay::AddSlot() */
	UNREALED_API SOverlay::FScopedWidgetSlotArguments AddSlot();

	/** Wraps SOverlay::RemoveSlot() */
	UNREALED_API void RemoveSlot();

	/**
	* Returns the cached size of this viewport overlay
	*
	* @return	The size that was cached
	*/
	UNREALED_API const FVector2D& GetCachedSize() const;

	/** Gets the  Viewport Tab that created this overlay */
	UNREALED_API TSharedPtr<FViewportTabContent> GetViewportTab() const;

private:

	/** Reference to the owning  viewport tab */
	TSharedPtr<FViewportTabContent> ViewportTab;

	/** The overlay widget we're containing */
	TSharedPtr< SOverlay > OverlayWidget;

	/** Cache our size, so that we can use this when animating a viewport maximize/restore */
	FVector2D CachedSize;
};

class FAssetEditorViewportPaneLayout
{
public:
	virtual ~FAssetEditorViewportPaneLayout() = default;

	/**
	 * Sets up the layout based on the specific layout configuration implementation.
	 *
	 * @param SpecificLayoutString		The layout string loaded from a file.
	 * @return The base widget representing the layout.  This is commonly a splitter.
	 */
	virtual TSharedRef<SWidget> MakeViewportLayout(TSharedPtr<FAssetEditorViewportLayout> InParentLayout, const FString& LayoutString) = 0;

	/**
	 * Returns the typename of this layout
	 * Generally one of EditorViewportConfigurationNames
	 */
	virtual const FName& GetLayoutTypeName() const = 0;

	UNREALED_API void LoadConfig(const FString& LayoutString, TFunction<void(const FString&, const FName)> LoadAdditionalLayoutInfoCallback = nullptr);

	UNREALED_API void SaveConfig(const FString& LayoutString, TFunction<void(const FString&)> SaveAdditionalLayoutInfoCallback = nullptr) const;

	virtual void ReplaceWidget(TSharedRef<SWidget> OriginalWidget, TSharedRef<SWidget> ReplacementWidget) = 0;

protected:
	UNREALED_API FString GetTypeSpecificLayoutString(const FString& LayoutString) const;
	virtual void SaveLayoutString(const FString& LayoutString) const {}
	virtual void LoadLayoutString(const FString& LayoutString) {}

	FName PerspectiveViewportConfigKey;
};

/**
 * Base class for viewport layout configurations
 * Handles maximizing and restoring well as visibility of specific viewports.
 */
class FAssetEditorViewportLayout : public TSharedFromThis<FAssetEditorViewportLayout>, public FEditorViewportLayout, public FTickableEditorObject
{
public:
	/**
	 * Constructor
	 */
	UNREALED_API FAssetEditorViewportLayout();

	/**
	 * Destructor
	 */
	UNREALED_API virtual ~FAssetEditorViewportLayout();

	/** Create an instance of a custom viewport from the specified viewport type name */
	UNREALED_API virtual TSharedRef<SWidget> FactoryViewport(FName InTypeName, const FAssetEditorViewportConstructionArgs& ConstructionArgs);

	/** FTickableEditorObject interface */
	virtual void Tick(float DeltaTime) override {}
	virtual bool IsTickable() const override { return false; }
 	virtual TStatId GetStatId() const override { return TStatId(); }

	UNREALED_API virtual void FactoryPaneConfigurationFromTypeName(const FName& InLayoutConfigTypeName) override;
	UNREALED_API virtual const FName GetActivePaneConfigurationTypeName() const override;

	/**
	 * Builds a viewport layout and returns the widget containing the layout
	 * 
	 * @param InParentDockTab		The parent dock tab widget of this viewport configuration
	 * @param InParentTab			The parent tab object
	 * @param LayoutString			The layout string loaded from file to custom build the layout with
	 */
 	UNREALED_API virtual TSharedRef<SWidget> BuildViewportLayout(TSharedPtr<SDockTab> InParentDockTab, TSharedPtr<FEditorViewportTabContent> InParentTab, const FString& LayoutString );

	/** Returns the parent tab content object */
	TWeakPtr< FEditorViewportTabContent > GetParentTabContent() const { return ParentTabContent; }

	/**
	 * Sets up the layout based on the specific layout configuration implementation.
	 *
	 * @param LayoutString		The layout string loaded from a file.
	 * @return The base widget representing the layout.  This is commonly a splitter.
	 */
	UE_DEPRECATED(5.0, "This functionality has moved to the layout configurations. Use BuildViewportLayout.")
	UNREALED_API virtual TSharedRef<SWidget> MakeViewportLayout(const FString& LayoutString) final;

	UNREALED_API virtual void LoadConfig(const FString& LayoutString);
	UNREALED_API virtual void SaveConfig(const FString& LayoutString) const;

protected:
	/**
	 * Delegate called to get the visibility of the non-maximized viewports
	 * The non-maximized viewports are not visible if there is a maximized viewport on top of them
	 *
	 * @param EVisibility::Visible when visible, EVisibility::Collapsed otherwise
	 */
	UNREALED_API virtual EVisibility OnGetNonMaximizedVisibility() const;

	/** The overlay widget that handles what viewports should be on top (non-maximized or maximized) */
	TWeakPtr< SAssetEditorViewportsOverlay > ViewportsOverlayPtr;

	/** The parent tab content object where this layout resides */
	TWeakPtr< FEditorViewportTabContent > ParentTabContent;

	/** The parent tab where this layout resides */
	TWeakPtr< SDockTab > ParentTab;

	TSharedPtr<FAssetEditorViewportPaneLayout> LayoutConfiguration;
};
