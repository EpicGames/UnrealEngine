// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Logging/LogMacros.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "InterchangeManager.h"
#include "AssetImportTask.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogAssetImportTask, Log, All);

class UFactory;

/**
 * Contains data for a group of assets to import
 */ 
UCLASS(Transient, BlueprintType, MinimalAPI)
class UAssetImportTask : public UObject
{
	GENERATED_BODY()

public:
	UNREALED_API UAssetImportTask();

// Task Options
public:
	/** Filename to import */
	UPROPERTY(BlueprintReadWrite, Category = "Asset Import Task")
	FString Filename;

	/** Path where asset will be imported to */
	UPROPERTY(BlueprintReadWrite, Category = "Asset Import Task")
	FString DestinationPath;

	/** Optional custom name to import as (if you are using interchange the name must be set in a pipeline and this field will be ignored)*/
	UPROPERTY(BlueprintReadWrite, Category = "Asset Import Task")
	FString DestinationName;

	/** Overwrite existing assets */
	UPROPERTY(BlueprintReadWrite, Category = "Asset Import Task")
	bool bReplaceExisting;

	/** Replace existing settings when overwriting existing assets  */
	UPROPERTY(BlueprintReadWrite, Category = "Asset Import Task")
	bool bReplaceExistingSettings;

	/** Avoid dialogs */
	UPROPERTY(BlueprintReadWrite, Category = "Asset Import Task")
	bool bAutomated;

	/** Save after importing */
	UPROPERTY(BlueprintReadWrite, Category = "Asset Import Task")
	bool bSave;

	/** Perform the import asynchronously for file formats where async import is available */
	UPROPERTY(BlueprintReadWrite, Category = "Asset Import Task")
	bool bAsync;

	/** Optional factory to use */
	UPROPERTY(BlueprintReadWrite, Category = "Asset Import Task")
	TObjectPtr<UFactory> Factory;

	/** Import options specific to the type of asset */
	UPROPERTY(BlueprintReadWrite, Category = "Asset Import Task")
	TObjectPtr<UObject> Options;

// Task Results
public:
	/**
	 * Get the list of imported objects.
	 * Note that if the import was asynchronous, this will block until the results are ready.
	 * To test whether asynchronous results are ready or not, use IsAsyncImportComplete().
	 */
	UFUNCTION(BlueprintCallable, Category = "Asset Import Task")
	UNREALED_API const TArray<UObject*>& GetObjects() const;

	/**
	 * Query whether this asynchronous import task is complete, and the results are ready to read.
	 * This will always return true in the case of a blocking import.
	 */
	UFUNCTION(BlueprintCallable, Category = "Asset Import Task")
	UNREALED_API bool IsAsyncImportComplete() const;

	/** Paths to objects created or updated after import */
	UPROPERTY(BlueprintReadWrite, Category = "Asset Import Task")
	TArray<FString> ImportedObjectPaths;

	/** Imported objects */
	// Note: after deprecation this will become a private member
	UE_DEPRECATED(5.1, "Please do not access this member directly; use GetObjects() instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Please use the GetObjects function instead."))
	TArray<TObjectPtr<UObject>> Result;

	/** Async results */
	UE::Interchange::FAssetImportResultPtr AsyncResults;
};

