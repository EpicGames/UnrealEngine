// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FAssetNotifications
{
public:
	/** Inform about change in skeleton asset that requires saving*/
	UNREALED_API static void SkeletonNeedsToBeSaved(const class USkeleton* Skeleton);
	UNREALED_API static void CannotEditCookedAsset(const class UObject* Asset);
};
