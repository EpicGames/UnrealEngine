// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"

struct FPendingPrivateAsset : TSharedFromThis<FPendingPrivateAsset>
{
public:

	FPendingPrivateAsset(UObject* InObject, const EAssetAccessSpecifier InTargetAssetAccessSpecifier, UPackage* InOwningPackage = nullptr);

	/** Checks if a given reference does not share the same mount point as the object. Which would be illegal if the object is made private */
	bool IsReferenceIllegal(const FName& InReference);

	/** Checks for and records references on disk and in memory for this object that would be considered illegal if the object became private */
	void CheckForIllegalReferences();

	/** Gets the object being privatized */
	UObject* GetObject() { return Object; }

	/** Gets the objects owning package */
	UPackage* GetOwningPackage() { return OwningPackage; }

	/** Is the pending object referenced in memory by the undo stack */
	bool IsReferencedInMemoryByNonUndo() { return bIsReferencedInMemoryByNonUndo; }

	/** Is the pending private object referenced in memory by something other than the undo stack */
	bool IsReferencedInMemoryByUndo() { return bIsReferencedInMemoryByUndo; }

	/** Performs the privatization action */
	void DoPrivatize();

	/** The on disk references to this object that would be illegal if the object were made private */
	TArray<FName> IllegalDiskReferences;

	/** The in memory references to this object that would be illegal if the object were made private (*excluding the undo buffer) */
	FReferencerInformationList IllegalMemoryReferences;

private:

	/** The object to make private */
	UObject* Object;

	/** The owning package of the object*/
	UPackage* OwningPackage;
	/** The cached mount pount name of the owning package */
	FName OwningPackageMountPointName;

	/** Asset access specifier to apply */
	EAssetAccessSpecifier TargetAssetAccessSpecifier;

	/** Flag indicating if this object is referenced in memory by the engine (excluding the undo buffer) */
	bool bIsReferencedInMemoryByNonUndo;

	/** Flag indicating if this object is referenced by the undo stack */
	bool bIsReferencedInMemoryByUndo;

	/** Flag indicating that references have been checked, so don't check again. */
	bool bReferencesChecked;
};

class FAssetPrivatizeModel
{
public:
	enum EState
	{
		// Waiting to start scanning
		Waiting = 0,
		// Begin scanning for references
		StartScanning,
		// Scan for references to the pending private assets
		Scanning,
		// Finished scanning
		Finished,
	};

	UNREALED_API FAssetPrivatizeModel(const TArray<UObject*>& InObjectsToPrivatize, const EAssetAccessSpecifier AssetAccessSpecifier);

	/** Add an object to the list of pending private assets, this will invalidate the scanning state */
	UNREALED_API void AddObjectToPrivatize(UObject* InObject, UPackage* InOwningPackage);

	/** Returns the pending private assets */
	const TArray<TSharedPtr<FPendingPrivateAsset>>* GetPendingPrivateAssets() const { return &PendingPrivateAssets; }

	/** Gets the packages of the assets on disk that would illegally reference the pending private objects if they were made private; won't be accurate until the scanning process completes. */
	const TSet<FName>& GetIllegalAssetReferences() const { return IllegalOnDiskReferences; }

	/** Ticks the privatize model which does a timeslice of work before returning */
	UNREALED_API void Tick(const float InDeltaTime);

	/** Returns the current state of the scanning process */
	EState GetState() { return State; }

	/** Gets the 0..1 progress of the scan */
	UNREALED_API float GetProgress() const;

	/** Gets the current text to display for the current progress of the scan */
	UNREALED_API FText GetProgressText() const;

	/** Are any of the pending assets being referenced in memory. */
	bool IsAnythingReferencedInMemoryByNonUndo() const { return bIsAnythingReferencedInMemoryByNonUndo; }

	/** Are any of the pending assets being referenced in the undo stack. */
	bool IsAnythingReferencedInMemoryByUndo() const { return bIsAnythingReferencedInMemoryByUndo; }

	/** Gets the number of objects successfully made private */
	int32 GetObjectsPrivatizedCount() const { return ObjectsPrivatized; }

	/** Get asset access specifier to apply */
	EAssetAccessSpecifier GetTargetAssetAccessSpecifier() const { return TargetAssetAccessSpecifier; }

	/** Returns true if it is valid to mark private the pending objects with no problems (there are no illegal references detected) */
	UNREALED_API bool CanPrivatize();

	/** Performs the privatization action if possible */
	UNREALED_API bool DoPrivatize();

	/** Returns true if we can't cleanly mark the pending objects as private (there are illegal references detected) */
	UNREALED_API bool CanForcePrivatize();

	/** Marks the pending private objects as private and nulls out all detected illegal references */
	UNREALED_API bool DoForcePrivatize();

	/** Fires whenever the state changes */
	DECLARE_EVENT_OneParam(FAssetPrivatizeModel, FOnStateChanged, EState /*NewState*/);
	FOnStateChanged& OnStateChanged()
	{
		return StateChanged;
	}

private:

	/** Sets the current state of the model */
	UNREALED_API void SetState(EState NewState);

	/** Helper that does the reference detection during scanning */
	UNREALED_API void ScanForReferences();

private:

	/** Holds an event delegate that is executed when the state changes */
	FOnStateChanged StateChanged;

	/** The assets being marked private */
	TArray<TSharedPtr<FPendingPrivateAsset>> PendingPrivateAssets;

	/** Are any of the pending private assets being referenced in memory */
	bool bIsAnythingReferencedInMemoryByNonUndo;

	/** Are any of the pending private assets being referenced in the undo stack */
	bool bIsAnythingReferencedInMemoryByUndo;

	/** A tick-to-tick state tracking variable so we know what pending private object we checked last */
	int32 PendingPrivateIndex;

	/** On disk references to the current pending private objects that would be illegal if they are made private */
	TSet<FName> IllegalOnDiskReferences;

	/** The internal progress/state of the privatize model which can take several frames to recalculate validity */
	EState State;

	/** The maximum timeslice per tick we are willing to scan for*/
	const double MaxTickSeconds = 0.100;

	/** The number of objects successfully marked private */
	int32 ObjectsPrivatized;

	/** Asset access specifier to apply */
	EAssetAccessSpecifier TargetAssetAccessSpecifier = EAssetAccessSpecifier::Private;
};
