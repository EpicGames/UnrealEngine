// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetRegistry/AssetData.h"
#include "Input/Reply.h"

class AActor;
struct FTypedElementHandle;
class IAssetFactoryInterface;
class UActorFactory;
class ULevel;
class UMaterialInterface;

// forward decl.
class UActorFactory;

struct AssetMarshalDefs
{
	static const TCHAR AssetDelimiter = TEXT('|');
	static const TCHAR NameTypeDelimiter = TEXT(' '); 
	static const TCHAR* FormatName() { return TEXT("UnrealEd/Assets"); }
};

namespace AssetUtil
{
	/** Extract the names of the assets which are being dragged*/
	UNREALED_API TArray<FAssetData>	ExtractAssetDataFromDrag( const FDragDropEvent& DragDropEvent );

	UNREALED_API TArray<FAssetData> ExtractAssetDataFromDrag(const TSharedPtr<const FDragDropOperation>& Operation);

	/**
	 *  Given an array of asset names, loads the assets into an array of objects
	 *
	 * @param AssetNames		(In) Array of Asset names to get UObjects for
	 * @param bAllWereLoaded	(Out) Optional param which indicates if all assets were loaded
	 */
	UNREALED_API TArray<UObject*>	GetObjects(const TArray<FString>& AssetNames, bool* bAllWereLoaded = NULL);

	/** Given an asset name, finds the object for the asset*/
	UNREALED_API UObject*			GetObject(const FString& AssetName);
	
	/** Does drag event contain any asset types which have components that support drag/drop?*/
	UNREALED_API FReply 			CanHandleAssetDrag( const FDragDropEvent &DragDropEvent );
}

/**
 * Generic information about the level selection set (actors or surfaces)
 */
struct FSelectedActorInfo
{
	/** String representing the selected class */
	FString SelectionStr;
	/** The selected class type */
	UClass* SelectionClass;
	/** The level that is shared between all actors, or NULL if selected actors aren't in the same level */
	ULevel* SharedLevel;
	/** The world that is shared between all actors, or NULL if selected actors aren't in the same level */
	UWorld* SharedWorld;
	/** How many are selected */
	uint32 NumSelected;
	/** How many nav points are selected */
	uint32 NumNavPoints;
	/** The number of selected actors that arent in a group */
	uint32 NumSelectedUngroupedActors;
	/** Number of properties of selected actors that are not yet propagated from the simulation world to the editor world */
	uint32 NumSimulationChanges;
	/** We have at least one actor that is attached to something */
	uint32 bHaveAttachedActor:1;
	/** Are all the selected actors the same type */
	uint32 bAllSelectedActorsOfSameType:1;
	/** Are all the selected actors brushes */
	uint32 bAllSelectedAreBrushes:1;
	/** true if a locked group is selected */
	uint32 bHaveSelectedLockedGroup:1;
	/** true if an unlocked group is selected */
	uint32 bHaveSelectedUnlockedGroup:1;
	/** true if a sub-group is selected */
	uint32 bHaveSelectedSubGroup:1;
	/** true if all selected actors belong to the same level */
	uint32 bSelectedActorsBelongToSameLevel:1;
	/** true if all selected actors belong to the current level */
	uint32 bAllSelectedActorsBelongToCurrentLevel:1;
	/** true if all selected actors belong to the same world */
	uint32 bAllSelectedActorsBelongToSameWorld:1;
	/** true if all selected actors have collision models */
	uint32 bAllSelectedStaticMeshesHaveCollisionModels:1;
	/** true if a brush is in the selection */
	uint32 bHaveBrush:1;
	/** true if there is a bsp brush in the selection */
	uint32 bHaveBSPBrush:1;
	/** true if a volume brush is in the selection */
	uint32 bHaveVolume:1;
	/** true if a builder brush is in the selection */
	uint32 bHaveBuilderBrush:1;
	/** true if an actor in the selection has a static mesh component */
	uint32 bHaveStaticMeshComponent:1;
	/** true if an actor in the selection is a static mesh*/
	uint32 bHaveStaticMesh:1;
	/** true if an actor in the selection is a light */
	uint32 bHaveLight:1;
	/** true if an actor in the selection is a pawn */
	uint32 bHavePawn:1;
	/** true if an actor in the selection is a skeletal mesh */
	uint32 bHaveSkeletalMesh:1;
	/** true if an actor in the selection is an emitter */
	uint32 bHaveEmitter:1;
	/** true if an actor in the selection is hidden */
	uint32 bHaveHidden:1;
	/** true if a landscape is in the selection */
	uint32 bHaveLandscape:1;
	/** true if an experimental actor (or actor containing such a component) is selected */
	uint32 bHaveExperimentalClass:1;
	/** true if an early access actor (or actor containing such a component) is selected */
	uint32 bHaveEarlyAccessClass:1;
	/** true if an actor in the selection has an override for the asset it browses to in the Content Browser*/
	uint32 bHaveBrowseOverride:1;

	FSelectedActorInfo()
		: SelectionClass(NULL)
		, SharedLevel(NULL)
		, SharedWorld(NULL)
		, NumSelected(0)
		, NumNavPoints(0)
		, NumSelectedUngroupedActors(0)
		, NumSimulationChanges(0)
		, bHaveAttachedActor(false)
		, bAllSelectedActorsOfSameType(true)
		, bAllSelectedAreBrushes(false)
		, bHaveSelectedLockedGroup(false)
		, bHaveSelectedUnlockedGroup(false)
		, bHaveSelectedSubGroup(false)
		, bSelectedActorsBelongToSameLevel(true)
		, bAllSelectedActorsBelongToCurrentLevel(true)
		, bAllSelectedActorsBelongToSameWorld(true)
		, bAllSelectedStaticMeshesHaveCollisionModels(true)
		, bHaveBrush(false)
		, bHaveBSPBrush(false)
		, bHaveVolume(false)
		, bHaveBuilderBrush(false)
		, bHaveStaticMeshComponent(false)
		, bHaveStaticMesh(false)
		, bHaveLight(false)
		, bHavePawn(false)
		, bHaveSkeletalMesh(false)
		, bHaveEmitter(false)
		, bHaveHidden(false)
		, bHaveLandscape(false)
		, bHaveExperimentalClass(false)
		, bHaveEarlyAccessClass(false)
		, bHaveBrowseOverride(false)
	{
	}

	/**
	 * @return true if the selection set has an actor that can be converted to a different actor
	 */
	bool HasConvertableAsset() const
	{
		return NumSelected && !bHaveBuilderBrush;
	}

};

namespace AssetSelectionUtils
{
	/**
	 * Checks if a class type can be placed in a level
	 * 
	 * @param Class	The class to check
	 * @return true if the class is placeable
	 */
	UNREALED_API bool IsClassPlaceable(const UClass* Class);
	/**
	 * Checks if a Blueprint based on a class type can be placed in a level
	 *
	 * @param Class	The class to check
	 * @return true if the Blueprint based on this class is placeable
	 */
	UNREALED_API bool IsChildBlueprintPlaceable(const UClass* Class);
	/**
	 * Gets the selected assets in the content browser
	 *
	 * @param SelectedLoadedAssets	(Out) The list of assets which are loaded
	 * @param SelectedUnloadedAssets	(Out) The ist of assets which are selected but are unloaded
	 */
	UNREALED_API void GetSelectedAssets( TArray<FAssetData>& OutSelectedAssets );

	/**
	 * Gets generic info about the selected actors or surfaces in the world
	 *
	 * @param SelectedActors	The selected actors to build info about
	 * @return Information about the selection
	 */
	UNREALED_API FSelectedActorInfo BuildSelectedActorInfo( const TArray<AActor*>& SelectedActors );

	/**
	 * A wrapper for the BuildSelectedActorInfo function that passes the SelectedActors array form GEditor
	 *
	 * @return Information about the selection
	 */
	UNREALED_API FSelectedActorInfo GetSelectedActorInfo();

	/**
	 * @return the number of selected bsp surfaces
	 */
	UNREALED_API int32 GetNumSelectedSurfaces( UWorld* InWorld );

	/**
	 * @return whether any surface is selected
	 */
	UNREALED_API bool IsAnySurfaceSelected( UWorld* InWorld );

	/**
	 * @return true if the builder brush is in the list of selected actors; false, otherwise. 
	 */
	UNREALED_API bool IsBuilderBrushSelected();
};

namespace ActorPlacementUtils
{
	UNREALED_API bool IsLevelValidForActorPlacement(ULevel* Level, TArray<FTransform>& InActorTransforms);
}

class FActorFactoryAssetProxy
{
public:

	/** 
	 * Information about an add actor menu item
	 */
	struct FMenuItem
	{
		/** Actor factory used to spawn the actor when the menu item is clicked */
		UActorFactory* FactoryToUse;

		/** The asset data to use with the factory */
		FAssetData AssetData;

		FMenuItem( UActorFactory* InFactoryToUse, const FAssetData& InAssetData )
			: FactoryToUse( InFactoryToUse )
			, AssetData( InAssetData )
		{}
	};

	/**
	 * Builds a list of strings for populating the actor factory context menu items.  This menu is shown when
	 * the user right-clicks in a level viewport.
	 *
	 * @param	AssetData					The asset the factories should consider when building the menus.
	 * @param	OutMenuItems				receives the list of menu items to use for populating an actor factory menu.
	 * @param	ExcludeStandAloneFactories	if true, only factories that can create actors with the currently selected assets will be added
	 */
	static UNREALED_API void GenerateActorFactoryMenuItems( const FAssetData& AssetData, TArray<FMenuItem>* OutMenuItems, bool ExcludeStandAloneFactories );

	/**
	 * Find the appropriate actor factory for an asset by type.
	 *
	 * @param	AssetData			contains information about an asset that to get a factory for
	 * @param	bRequireValidObject	indicates whether a valid asset object is required.  specify false to allow the asset
	 *								class's CDO to be used in place of the asset if no asset is part of the drag-n-drop
	 *
	 * @return	the factory that is responsible for creating actors for the specified asset type.
	 */
	static UNREALED_API UActorFactory* GetFactoryForAsset( const FAssetData& DropData, bool bRequireValidObject=false );

	/**
	 * Find the appropriate actor factory for an asset.
	 *
	 * @param	AssetObj	The asset that to find the appropriate actor factory for
	 *
	 * @return	The factory that is responsible for creating actors for the specified asset
	 */
	static UNREALED_API UActorFactory* GetFactoryForAssetObject( UObject* AssetObj );

	//~ TODO: UE_DEPRECATED(5.4, "Use UE::AssetPlacementUtil::PlaceAssetInCurrentLevel instead")
	/**
	 * Places an actor instance using the factory appropriate for the type of asset
	 *
	 * @param	AssetObj						the asset that is contained in the d&d operation
	 * @param	ObjectFlags						The object flags to place on the actor when it is spawned
	 * @param	FactoryToUse					optional actor factory to use to create the actor; if not specified,
	 *											the highest priority factory that is valid will be used
	 *
	 * @return	the actor that was created by the factory, or NULL if there aren't any factories for this asset (or
	 *			the actor couldn't be created for some other reason)
	 */
	static UNREALED_API AActor* AddActorForAsset( UObject* AssetObj, bool SelectActor = true, EObjectFlags ObjectFlags = RF_Transactional, UActorFactory* FactoryToUse = NULL, const FName Name = NAME_None );

	//~ TODO: UE_DEPRECATED(5.4)
	/**
	 * Places an actor instance using the factory appropriate for the type of asset using the current object selection as the asset
	 *
	 * @param	ActorClass						The type of actor to create
	 * @param	ActorLocation					specify null to position the actor at the mouse location, 
												otherwise it will be placed at the origin.
	 * @param	ObjectFlags						The object flags to place on the actor when it is spawned
	 * @param	FactoryToUse					optional actor factory to use to create the actor; if not specified,
	 *											the highest priority factory that is valid will be used
	 *
	 * @return	the actor that was created by the factory, or NULL if there aren't any factories for this asset (or
	 *			the actor couldn't be created for some other reason)
	 */
	static UNREALED_API AActor* AddActorFromSelection( UClass* ActorClass, const FVector* ActorLocation=NULL, bool SelectActor = true, EObjectFlags ObjectFlags = RF_Transactional, UActorFactory* ActorFactory = NULL, const FName Name = NAME_None );

	/**
	 * Determines if the provided actor is capable of having a material applied to it.
	 *
	 * @param	TargetActor	Actor to check for the validity of material application
	 *
	 * @return	true if the actor is valid for material application; false otherwise
	 */
	static UNREALED_API bool IsActorValidForMaterialApplication( AActor* TargetActor );

	/**
	 * Attempts to apply the material to the specified actor.
	 *
	 * @param	TargetActor		the actor to apply the material to
	 * @param	MaterialToApply	the material to apply to the actor
	 * @param   OptionalMaterialSlot the material slot to apply it to.
	 *
	 * @return	true if the material was successfully applied to the actor
	 */
	static UNREALED_API bool ApplyMaterialToActor( AActor* TargetActor, UMaterialInterface* MaterialToApply, int32 OptionalMaterialSlot = -1 );

private:
	/**
	 * Constructor
	 *
	 * Private, as this class is [currently] not intended to be instantiated
	 */
	FActorFactoryAssetProxy()
	{
	}
};

namespace UE::AssetPlacementUtil
{
	struct FExtraPlaceAssetOptions
	{
		bool bSelectOutput = true;
		EObjectFlags ObjectFlags = RF_Transactional;
		TScriptInterface<IAssetFactoryInterface> FactoryToUse = NULL;
		FName Name = NAME_None;
	};

	/**
	 * Places an asset instance using the factory appropriate for the type of asset
	 *
	 * @param	AssetObj						the asset that is contained in the d&d operation
	 *
	 * @return	the object that was created by the factory, or NULL if there aren't any factories for this asset (or
	 *			the object couldn't be created for some other reason)
	 */
	UNREALED_API TArray<FTypedElementHandle> PlaceAssetInCurrentLevel(UObject* AssetObj, const FExtraPlaceAssetOptions& ExtraParams);
}
