// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "Misc/Attribute.h"
#include "AssetRegistry/AssetData.h"
#include "Rendering/RenderingCommon.h"
#include "Widgets/SWidget.h"
#include "Widgets/SNullWidget.h"
#include "TickableEditorObject.h"
#include "ObjectTools.h"

class AActor;
class FAssetThumbnailPool;
class FSlateShaderResource;
class FSlateTexture2DRHIRef;
class FSlateTextureRenderTarget2DResource;
class IAssetSystemInfoProvider;
class IToolTip;
struct FPropertyChangedEvent;

namespace EThumbnailLabel
{
	enum Type
	{
		ClassName,
		AssetName,
		NoLabel
	};
};

enum class EThumbnailSize : uint8
{
	Tiny = 0,
	Small,
	Medium,
	Large,
	XLarge,
	Huge,

	// Not a size
	MAX
};

/** The edge of the thumbnail along which to display the color strip */
enum class EThumbnailColorStripOrientation : uint8
{
	/** Display the color strip as a horizontal line along the bottom edge */
	HorizontalBottomEdge,
	/** Display the color strip as a vertical line along the right edge */
	VerticalRightEdge,
};

/** A struct containing details about how the asset thumbnail should behave */
struct FAssetThumbnailConfig
{
	FAssetThumbnailConfig()
		: bAllowFadeIn( false )
		, bForceGenericThumbnail( false )
		, bAllowHintText( true )
		, bAllowRealTimeOnHovered( true )
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		, bAllowAssetSpecificThumbnailOverlay( false )
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		, AllowAssetSpecificThumbnailOverlay( false )
		, AllowAssetSpecificThumbnailOverlayIndicator()
		, ClassThumbnailBrushOverride( NAME_None )
		, ThumbnailLabel( EThumbnailLabel::ClassName )
		, HighlightedText( FText::GetEmpty() )
		, HintColorAndOpacity( FLinearColor( 0.0f, 0.0f, 0.0f, 0.0f ) )
		, AssetTypeColorOverride()
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		, Padding(0)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		, BorderPadding(2.f)
		, AssetSystemInfoProvider(nullptr)
		, AdditionalTooltipInSmallView(SNullWidget::NullWidget)
		, AllowAssetStatusThumbnailOverlay( false )
		, ShowAssetColor(true)
		, AssetBorderImageOverride()
		, ShowAssetBorder(false)
		, bCanDisplayEditModePrimitiveTools(false)
		, IsEditModeVisible()
		, AlwaysExpandTooltip(false)
	{
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FAssetThumbnailConfig(const FAssetThumbnailConfig&) = default;
	FAssetThumbnailConfig& operator=(const FAssetThumbnailConfig&) = default;
	FAssetThumbnailConfig(FAssetThumbnailConfig&&) = default;
	FAssetThumbnailConfig& operator=(FAssetThumbnailConfig&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	bool bAllowFadeIn;
	bool bForceGenericThumbnail;
	bool bAllowHintText;
	bool bAllowRealTimeOnHovered;
	UE_DEPRECATED(5.6, "Use the TAttribute version to have the visibility dynamic")
	bool bAllowAssetSpecificThumbnailOverlay;
	/** Whether to show the asset play button or not */
	TAttribute< bool > AllowAssetSpecificThumbnailOverlay;
	/** Whether to show the play indicator or not, by default it will match the AllowAssetSpecificThumbnailOverlay one */
	TAttribute< bool > AllowAssetSpecificThumbnailOverlayIndicator;
	FName ClassThumbnailBrushOverride;
	EThumbnailLabel::Type ThumbnailLabel;
	TAttribute< FText > HighlightedText;
	TAttribute< FLinearColor > HintColorAndOpacity;
	TOptional< FLinearColor > AssetTypeColorOverride;
	UE_DEPRECATED(5.6, "Use the TAttribute version \"BorderPadding\" to have the padding dynamic")
	FMargin Padding;
	/** Border padding for the thumbnail should match the thickness of the OverrideBorder, the default border size and padding is 2px */
	TAttribute<FMargin> BorderPadding;
	TAttribute<int32> GenericThumbnailSize = 64;
	EThumbnailColorStripOrientation ColorStripOrientation = EThumbnailColorStripOrientation::HorizontalBottomEdge;
	TSharedPtr<IAssetSystemInfoProvider> AssetSystemInfoProvider;
	/** Additional tooltip placed between AssetType and Status */
	TAttribute<TSharedPtr<SWidget>> AdditionalTooltipInSmallView;
	/** Whether to show the asset status or not */
	TAttribute< bool > AllowAssetStatusThumbnailOverlay;
	/** Whether to show the asset color or not */
	TAttribute< bool > ShowAssetColor;
	/** Border override brush, will be used if set, otherwise if ShowAssetBorder is true the default border will be used, do not assign both if no border is wanted */
	TAttribute<const FSlateBrush*> AssetBorderImageOverride;
	/** If true will create a default border for the Thumbnail, overridden by AssetBorderImageOverride, if this is false and no override is given no border will be created */
	bool ShowAssetBorder;
	bool bCanDisplayEditModePrimitiveTools;
	TAttribute<EVisibility> IsEditModeVisible;
	TAttribute<bool> AlwaysExpandTooltip;
};


/**
 * Interface for rendering a thumbnail in a slate viewport                   
 */
class FAssetThumbnail
	: public ISlateViewport
	, public TSharedFromThis<FAssetThumbnail>
{
public:
	/**
	 * @param InAsset	The asset to display a thumbnail for
	 * @param InWidth		The width that the thumbnail should be
	 * @param InHeight	The height that the thumbnail should be
	 * @param InThumbnailPool	The thumbnail pool to request textures from
	 */
	UNREALED_API FAssetThumbnail( UObject* InAsset, uint32 InWidth, uint32 InHeight, const TSharedPtr<class FAssetThumbnailPool>& InThumbnailPool );
	UNREALED_API FAssetThumbnail( const FAssetData& InAsset, uint32 InWidth, uint32 InHeight, const TSharedPtr<class FAssetThumbnailPool>& InThumbnailPool );
	UNREALED_API ~FAssetThumbnail();

	/**
	 * @return	The size of the viewport (thumbnail size)                   
	 */
	virtual FIntPoint GetSize() const override;

	/**
	 * @return The texture used to display the viewports content                  
	 */
	virtual FSlateShaderResource* GetViewportRenderTargetTexture() const override;

	/**
	 * Returns true if the viewport should be vsynced.
	 */
	virtual bool RequiresVsync() const override { return false; }

	/**
	 * @return The object we are rendering the thumbnail for                
	 */
	UNREALED_API UObject* GetAsset() const;

	/**
	 * @return The asset data for the object we are rendering the thumbnail for
	 */
	UNREALED_API const FAssetData& GetAssetData() const;

	/**
	 * Sets the asset to render the thumbnail for 
	 *
	 * @param InAsset	The new asset
	 */
	UNREALED_API void SetAsset( const UObject* InAsset );

	/**
	 * Sets the asset to render the thumbnail for
	 *
	 * @param InAssetData	Asset data containing the the new asset to render
	 */
	UNREALED_API void SetAsset( const FAssetData& InAssetData );

	/**
	 * @return A slate widget representing this thumbnail
	 */
	UNREALED_API TSharedRef<SWidget> MakeThumbnailWidget( const FAssetThumbnailConfig& InConfig = FAssetThumbnailConfig() );

	/** Re-renders this thumbnail */
	UNREALED_API void RefreshThumbnail();

	/** Updates if this thumbnail should be realtime rendered via the pool */
	UNREALED_API void SetRealTime( bool bRealTime );

	DECLARE_EVENT(FAssetThumbnail, FOnAssetDataChanged);
	FOnAssetDataChanged& OnAssetDataChanged() { return AssetDataChangedEvent; }

private:
	/** Thumbnail pool for rendering the thumbnail */
	TWeakPtr<FAssetThumbnailPool> ThumbnailPool;
	/** Triggered when the asset data changes */
	FOnAssetDataChanged AssetDataChangedEvent;
	/** The asset data for the object we are rendering the thumbnail for */
	FAssetData AssetData;
	/** Width of the thumbnail */
	uint32 Width;
	/** Height of the thumbnail */
	uint32 Height;
};

/**
 * Utility class for keeping track of, rendering, and recycling thumbnails rendered in Slate              
 */
class FAssetThumbnailPool : public FTickableEditorObject
{
public:
	UNREALED_API static FName CustomThumbnailTagName;

	/**
	 * Constructor 
	 *
	 * @param InNumInPool						The number of thumbnails allowed in the pool
	 * @param InMaxFrameTimeAllowance			The maximum number of seconds per tick to spend rendering thumbnails
	 * @param InMaxRealTimeThumbnailsPerFrame	The maximum number of real-time thumbnails to render per tick
	 */
	UNREALED_API explicit FAssetThumbnailPool( uint32 InNumInPool, double InMaxFrameTimeAllowance = 0.005, uint32 InMaxRealTimeThumbnailsPerFrame = 3 );
	/**
	 * If you receive a compilation error pointing you here, this is due to a parameter that was removed in 5.5, but was not fully deprecated.
	 * In the case your call-site was using the removed parameter @param InAreRealTimeThumbnailsAllowed	Attribute that determines if thumbnails should be rendered in real-time
	 * an implicit cast could occur which would result in unexpected thumbnail pool behavior by setting the tick to implicit cast of a boolean parameter
	 */
	template <class T> UE_DEPRECATED(5.7, "If your function is using a bool as parameter 2, consider removing it and see comments in header file.")
	FAssetThumbnailPool( uint32 InNumInPool, T InMaxFrameTimeAllowance, uint32 InMaxRealTimeThumbnailsPerFrame = 3 ) = delete;

	/** Destructor to free all remaining resources */
	UNREALED_API ~FAssetThumbnailPool();

	//~ Begin FTickableObject Interface
	UNREALED_API virtual TStatId GetStatId() const override;

	/** Checks if any new thumbnails are queued */
	UNREALED_API virtual bool IsTickable() const override;

	/** Ticks the pool, rendering new thumbnails as needed */
	UNREALED_API virtual void Tick( float DeltaTime ) override;

	//~ End FTickableObject Interface

	/**
	 * Accesses the texture for an object.  If a thumbnail was recently rendered this function simply returns the thumbnail.  If it was not, it requests a new one be generated
	 * No assumptions should be made about whether or not it was rendered
	 *
	 * @param Asset The asset to get the thumbnail for
	 * @param Width	The width of the thumbnail
	 * @param Height The height of the thumbnail
	 * @return The thumbnail for the asset or NULL if one could not be produced
	 */
	FSlateTexture2DRHIRef* AccessTexture( const FAssetData& AssetData, uint32 Width, uint32 Height );

	/**
	 * Adds a referencer to keep textures around as long as they are needed
	 */
	void AddReferencer( const FAssetThumbnail& AssetThumbnail );

	/**
	 * Removes a referencer to clean up textures that are no longer needed
	 */
	void RemoveReferencer( const FAssetThumbnail& AssetThumbnail );

	/** Returns true if the thumbnail for the specified asset in the specified size is in the stack of thumbnails to render */
	bool IsInRenderStack( const TSharedPtr<FAssetThumbnail>& Thumbnail ) const;

	/** Returns true if the thumbnail for the specified asset in the specified size has been rendered */
	bool IsRendered( const TSharedPtr<FAssetThumbnail>& Thumbnail ) const;

	/** Brings all items in ThumbnailsToPrioritize to the front of the render stack if they are actually in the stack */
	UNREALED_API void PrioritizeThumbnails( const TArray< TSharedPtr<FAssetThumbnail> >& ThumbnailsToPrioritize, uint32 Width, uint32 Height );

	/** Register/Unregister a callback for when thumbnails are rendered */
	DECLARE_EVENT_OneParam( FAssetThumbnailPool, FThumbnailRendered, const FAssetData& );
	FThumbnailRendered& OnThumbnailRendered() { return ThumbnailRenderedEvent; }

	/** Register/Unregister a callback for when thumbnails fail to render */
	DECLARE_EVENT_OneParam( FAssetThumbnailPool, FThumbnailRenderFailed, const FAssetData& );
	FThumbnailRenderFailed& OnThumbnailRenderFailed() { return ThumbnailRenderFailedEvent; }

	/** Re-renders the specified thumbnail */
	UNREALED_API void RefreshThumbnail( const TSharedPtr<FAssetThumbnail>& ThumbnailToRefresh );

	/** Enables/disables realtime thumbnail behavior */
	UNREALED_API void SetRealTimeThumbnail(const TSharedPtr<FAssetThumbnail>& Thumbnail, bool bRealTimeThumbnail);

private:
	struct FThumbId;

	/**
	 * Releases all rendering resources held by the pool
	 */
	void ReleaseResources();

	/**
	 * Frees the rendering resources and clears a slot in the pool for an asset thumbnail at the specified width and height
	 *
	 * @param ObjectPath	The path to the asset whose thumbnail should be free
	 * @param Width 		The width of the thumbnail to free
	 * @param Height		The height of the thumbnail to free
	 */
	void FreeThumbnail( const FSoftObjectPath& ObjectPath, uint32 Width, uint32 Height );

	/** Adds the thumbnails associated with the object found at ObjectPath to the render stack */
	void RefreshThumbnailsFor( const FSoftObjectPath& ObjectPath );
	/** Adds the thumbnails associated with the ThumbId to the render stack */
	void RefreshThumbnailFor( const FThumbId ThumbId );

	/** Handler for when an asset is loaded */
	void OnAssetLoaded( UObject* Asset );

	/** Handler for when a thumbnail gets flagged as dirty. Used to refresh the thumbnail. */
	void OnThumbnailDirtied( const FSoftObjectPath& ObjectPath );

	/** Copies any completed object thumbnails to the local object thumbnail cache */
	void ProcessPendingObjectThumbnails();

	/** Requests a refresh for recently loaded assets for which we have a thumbnail. Done outside of the OnAssetLoaded callback so as to minimize loading disruptions. */
	void ProcessRecentlyLoadedAssets();

private:
	/** Information about a thumbnail */
	struct FThumbnailInfo
	{
		/** The object whose thumbnail is rendered */
		FAssetData AssetData;
		/** Rendering resource for slate */
		FSlateTexture2DRHIRef* ThumbnailTexture = nullptr;
		/** Render target for slate */
		FSlateTextureRenderTarget2DResource* ThumbnailRenderTarget = nullptr;
		/** The time since last access */
		double LastAccessTime = 0.0;
		/** The time since last update */
		double LastUpdateTime = 0.0;
		/** Width of the thumbnail */
		const uint32 Width;
		/** Height of the thumbnail */
		const uint32 Height;

		FThumbnailInfo(uint32 InWidth, uint32 InHeight) : Width(InWidth), Height(InHeight) {}
		~FThumbnailInfo();
	};
	/**
	 * Assign a thumbnail from its render target and re-render it if necessary.
	 *
	 * @param ThumbnailInfo The thumbnail info to assign a texture to
	 * @param bOutRetryOnFailure If the asset we want to load the thumbnail is compiling, this flag will be set to true, it wont be touch in other cases.
	 * @param CustomAssetToRender The asset to render when generating the texture
	 *
	 * @return true if the thumbnail was assigned to a valid texture
	 */
	bool LoadThumbnail(TSharedRef<FThumbnailInfo> ThumbnailInfo, bool& bOutRetryOnFailure, const FAssetData& CustomAssetToRender = FAssetData());
	bool LoadThumbnailForLoadedAsset(const TSharedRef<FThumbnailInfo>& ThumbnailInfo, bool& bOutRetryOnFailure, const FAssetData& CustomAssetToRender);
	bool LoadThumbnailForUnloadedAsset(const TSharedRef<FThumbnailInfo>& ThumbnailInfo, const FAssetData& CustomAssetToRender);
	
	/** Tries to load the thumbnail from an object thumbnail cached locally or on the asset package. */
	bool LoadThumbnailFromObjectThumbnailInCache(const TSharedRef<FThumbnailInfo>& ThumbnailInfo, const FAssetData& CustomAssetToRender);

	/** Tries to load the thumbnail from the package on disk. */
	bool LoadThumbnailFromObjectThumbnailOnDisk(const TSharedRef<FThumbnailInfo>& ThumbnailInfo, const FAssetData& CustomAssetToRender);

	/** Tries to render a fresh thumbnail if the thumbnail render frequency allows it. */
	bool MakeFreshThumbnailFromRender(const TSharedRef<FThumbnailInfo>& ThumbnailInfo, bool& bOutRetryOnFailure, const FAssetData& CustomAssetToRender);

	/** Copies the image data from the object thumbnail to the slate texture in the thumbnail info. */
	bool CopyObjectThumbnailImageDataToSlateTextureData(const TSharedRef<FThumbnailInfo>& ThumbnailInfo, const FObjectThumbnail& ObjectThumbnail);

	struct FThumbnailInfo_RenderThread
	{
		/** Rendering resource for slate */
		FSlateTexture2DRHIRef* ThumbnailTexture;
		/** Render target for slate */
		FSlateTextureRenderTarget2DResource* ThumbnailRenderTarget;
		/** Width of the thumbnail */
		uint32 Width;
		/** Height of the thumbnail */
		uint32 Height;

		FThumbnailInfo_RenderThread(const FThumbnailInfo& Info)
			: ThumbnailTexture(Info.ThumbnailTexture)
			, ThumbnailRenderTarget(Info.ThumbnailRenderTarget)
			, Width(Info.Width)
			, Height(Info.Height)
		{}
	};
	
	/** Key for looking up thumbnails in a map */
	struct FThumbId
	{
		FSoftObjectPath ObjectPath;
		uint32 Width;
		uint32 Height;

		FThumbId( FSoftObjectPath InObjectPath, uint32 InWidth, uint32 InHeight )
			: ObjectPath( MoveTemp(InObjectPath) )
			, Width( InWidth )
			, Height( InHeight )
		{}

		bool operator==( const FThumbId& Other ) const
		{
			return ObjectPath == Other.ObjectPath && Width == Other.Width && Height == Other.Height;
		}

		friend uint32 GetTypeHash( const FThumbId& Id )
		{
			return GetTypeHash( Id.ObjectPath ) ^ GetTypeHash( Id.Width ) ^ GetTypeHash( Id.Height );
		}
	};
	/** The delegate to execute when a thumbnail is rendered */
	FThumbnailRendered ThumbnailRenderedEvent;

	/** The delegate to execute when a thumbnail failed to render */
	FThumbnailRenderFailed ThumbnailRenderFailedEvent;

	/** A mapping of objects to their thumbnails */
	TMap< FThumbId, TSharedRef<FThumbnailInfo> > ThumbnailToTextureMap;

	/** List of thumbnails to render when possible */
	TArray< TSharedRef<FThumbnailInfo> > ThumbnailsToRenderStack;

	/** List of thumbnails that can be rendered in real-time */
	TArray< TSharedRef<FThumbnailInfo> > RealTimeThumbnails;

	/** List of real-time thumbnails that are queued to be rendered */
	TArray< TSharedRef<FThumbnailInfo> > RealTimeThumbnailsToRender;

	/** List of free thumbnails that can be reused */
	TArray< TSharedRef<FThumbnailInfo> > FreeThumbnails;

	/** Cached object thumbnails that we rendered. Ignored when a thumbnail refresh is explicitly requested or thumbnails were marked as dirty for the asset */
	TMap< FSoftObjectPath, FObjectThumbnail > LocalObjectThumbnailsCache;

	/** Object thumbnails that are waiting for their textures to be rendered */
	TMap< FSoftObjectPath, ThumbnailTools::FAsyncObjectThumbnail > PendingObjectThumbnails;

	/** A mapping of objects to the number of referencers */
	TMap< FThumbId, int32 > RefCountMap;

	/** A list of object paths for recently loaded assets whose thumbnails need to be refreshed. */
	TArray<FSoftObjectPath> RecentlyLoadedAssets;

	/** Max number of thumbnails in the pool */
	uint32 NumInPool;

	/** Max number of dynamic thumbnails to update per frame */
	uint32 MaxRealTimeThumbnailsPerFrame;

	/** Max number of seconds per tick to spend rendering thumbnails */
	double MaxFrameTimeAllowance;
};
