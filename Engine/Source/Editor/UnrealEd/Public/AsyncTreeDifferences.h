// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DiffUtils.h"
#include "Engine/EngineTypes.h"
#include "Misc/Attribute.h"
#include "Templates/UniquePtr.h"

enum class ETreeDiffResult
{
	Invalid,
	MissingFromTree1,
	MissingFromTree2,
	DifferentValues,
	Identical
};

enum class ETreeTraverseOrder
{
	PreOrder, // parent then children
	PostOrder, // children then parent
	
	ReversePreOrder, // parent then children in reverse order
	ReversePostOrder // children in reverse order then parent
};

enum class ETreeTraverseControl
{
	Continue, // continue traversing
	Break, // stop

	// PreOrder traversal only:
	SkipChildren, // don't iterate the children of this node
};

/// To use TAsyncTreeDifferences, define the following template specializations for your type
////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//namespace TreeDiffSpecification
//{
//	/**
//	 * determine whether the values stored in two nodes are equal.
//	 * @param TreeNodeA node from the first user provided tree (guaranteed not to be null)
//	 * @param TreeNodeB node from the second user provided tree (guaranteed not to be null)
//	 * @param OutDifferingProperties optional output list of property paths for the properties that differs between the two nodes
//	 */
//	template<typename InNodeType>
//	bool AreValuesEqual(const InNodeType& TreeNodeA, const InNodeType& TreeNodeB, TArray<FPropertySoftPath>* OutDifferingProperties = nullptr);
//
//	/**
//	 * determine whether two nodes occupy the same space in their trees
//	 * for example if you have a tree key/value pairs, AreMatching should compare the keys while AreValuesEqual
//	 * should compare the values
//	 * @param TreeNodeA node from the first user provided tree (guaranteed not to be null)
//	 * @param TreeNodeB node from the second user provided tree (guaranteed not to be null)
//	 * @param OutDifferingProperties optional output list of property paths for the properties that differs between the two nodes
//	 */
//	template<typename InNodeType>
//	bool AreMatching(const InNodeType& TreeNodeA, const InNodeType& TreeNodeB, TArray<FPropertySoftPath>* OutDifferingProperties = nullptr);
//
//	/**
//	 * retrieves an array of children nodes from the parent node
//	 * @param InParent node from one of the two user provided trees (guaranteed not to be null)
//	 * @param[out] OutChildren to be filled with the children of parent
//	 */
//	template<typename InNodeType>
//	void GetChildren(const InNodeType& InParent, TArray<InNodeType>& OutChildren);
//
//	/**
//	 * return true for nodes that match using AreValuesEqual first, and pair up by position second
//	 * this is useful for arrays since we often want to keep elements with the same data paired while diffing other elements in order
//	 * @param TreeNode node from one of the two user provided trees (guaranteed not to be null)
//	*/
//	template<typename InNodeType>
//	bool ShouldMatchByValue(const InNodeType& TreeNode);
//
//}

template<typename InNodeType>
class TTreeDiffSpecification
{
public:
	virtual ~TTreeDiffSpecification() = default;
	
	/**
	 * determine whether the values stored in two nodes are equal.
	 * @param TreeNodeA node from the first user provided tree (guaranteed not to be null)
	 * @param TreeNodeB node from the second user provided tree (guaranteed not to be null)
	 * @param OutDifferingProperties optional output list of property paths for the properties that differs between the two nodes
	 */
	bool AreValuesEqual(const InNodeType& TreeNodeA, const InNodeType& TreeNodeB) const
	{
		return TreeNodeA == TreeNodeB;
	}
	UE_DEPRECATED(5.6, "Output parameters wasn't generic and is no longer used.")
	bool AreValuesEqual(const InNodeType& TreeNodeA, const InNodeType& TreeNodeB, TArray<FPropertySoftPath>* OutDifferingProperties) const
	{
		return AreValuesEqual(TreeNodeA, TreeNodeB);
	}

	/**
	 * determine whether two nodes occupy the same space in their trees
	 * for example if you have a tree key/value pairs, AreMatching should compare the keys while AreValuesEqual
	 * should compare the values
	 * @param TreeNodeA node from the first user provided tree (guaranteed not to be null)
	 * @param TreeNodeB node from the second user provided tree (guaranteed not to be null)
	 * @param OutDifferingProperties optional output list of property paths for the properties that differs between the two nodes
	 */
	bool AreMatching(const InNodeType& TreeNodeA, const InNodeType& TreeNodeB) const
	{
		return TreeNodeA == TreeNodeB;
	}

	UE_DEPRECATED(5.6, "Output parameters wasn't generic and is no longer used.")
	bool AreMatching(const InNodeType& TreeNodeA, const InNodeType& TreeNodeB, TArray<FPropertySoftPath>* OutDifferingProperties) const
	{
		return TreeNodeA == TreeNodeB;
	}

	/**
	 * retrieves an array of children nodes from the parent node
	 * @param InParent node from one of the two user provided trees (guaranteed not to be null)
	 * @param[out] OutChildren to be filled with the children of parent
	 */
	void GetChildren(const InNodeType& InParent, TArray<InNodeType>& OutChildren) const
	{}

	/**
	 * return true for nodes that match using AreValuesEqual first, and pair up by position second
	 * this is useful for arrays since we often want to keep elements with the same data paired while diffing other elements in order
	 * @param TreeNode node from one of the two user provided trees (guaranteed not to be null)
	*/
	bool ShouldMatchByValue(const InNodeType& TreeNode) const
	{
		return true;
	}

	/**
	 * return true if TreeNode is considered equal when all it's children are equal.
	 * This avoids an unnecessary call to AreValuesEqual
	*/
	bool ShouldInheritEqualFromChildren(const InNodeType& TreeNodeA, const InNodeType& TreeNodeB) const
	{
		return false;
	}
};


// the TAsyncTreeDifferences structure's goal is to build and maintain an updated tree made of these nodes.
// where TDiffNode::ValueA and TDiffNode::ValueB are matching nodes from two diffed trees. Note that these values can
// be both "matching" and not equal which would set TDiffNode::DiffResult to DifferentValues
template<typename InNodeType>
struct TDiffNode
{
	// because we've got a bit of node-type inception going on, we'll refer to the nodes in the user provided trees as
	// values rather than nodes. (but it's still sometimes helpful to remember that they are nodes in trees)
	using ValueType = InNodeType;
	
	ValueType ValueA = NullValue;
	ValueType ValueB = NullValue;
	
	TArray<TUniquePtr<TDiffNode>> Children = {};
	const TDiffNode* Parent = nullptr;
	ETreeDiffResult DiffResult = ETreeDiffResult::Invalid;

	TDiffNode() = default;
	TDiffNode(TDiffNode &&Other) = default;
	
	TDiffNode(const ValueType& InValueA, const ValueType& InValueB, const TDiffNode* InParent)
		: ValueA(InValueA), ValueB(InValueB), Children(), Parent(InParent)
	{}

	void SetDiffType(TUniquePtr<TTreeDiffSpecification<ValueType>>& Specification);

	bool operator==(const TDiffNode& Other) const;
	
	static inline ValueType NullValue{}; // assume default constructor to signify a "null" node
};

template<typename InNodeType>
class TAsyncTreeDifferences
{
public:
	using ValueType = InNodeType;
	using DiffNodeType = TDiffNode<ValueType>;
	
	TAsyncTreeDifferences(const TAttribute<TArray<ValueType>>& InRootValuesA, const TAttribute<TArray<ValueType>>& InRootValuesB);
	
	// update the differences list over time
	void Tick(float MaxAllottedTimeMs = 1.f);

	// process all tasks synchronously until all data is up to date. (slow)
	void FlushQueue();

	// calls Method(DiffNode) on each diff node in the tree.
	void ForEach(ETreeTraverseOrder TraversalOrder, const TFunction<ETreeTraverseControl(const TUniquePtr<DiffNodeType>&)>& Method) const;

	template<typename SpecificationType, typename... TArgs>
	void SetDiffSpecification(TArgs... Args);

	// head of the main diff tree (note: not for user modification. Tick() will overwrite user changes)
	TUniquePtr<DiffNodeType> Head;

	// every time the update queue is completed this is called (max of once per Tick)
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnQueueFlushed, const TAsyncTreeDifferences&);
	FOnQueueFlushed OnQueueFlushed;
private:

	// process one element from the top of the queue
	void ProcessTopOfQueue();
	
	void QueueParallelNodeLists(const TArray<ValueType>& ValuesA, const TArray<ValueType>& ValuesB, DiffNodeType* ParentNode);
	
	TArray<TArray<int32>> CalculateLCSTableForMatchingValues(const TArray<ValueType>& ValuesA, const TArray<ValueType>& ValuesB);
	static TArray<TArray<int32>> CalculateLCSTableForDiffNodes(const TArray<DiffNodeType>& FoundNodes, const TArray<TUniquePtr<DiffNodeType>>& ExpectedNodes);

	bool AreMatching(const ValueType& ValueA, const ValueType& ValueB);
	
	static bool PreOrderRecursive(const TUniquePtr<DiffNodeType>& Node, const TFunction<ETreeTraverseControl(const TUniquePtr<DiffNodeType>&)>& Method);
	static bool PostOrderRecursive(const TUniquePtr<DiffNodeType>& Node, const TFunction<ETreeTraverseControl(const TUniquePtr<DiffNodeType>&)>& Method);
	static bool ReversePreOrderRecursive(const TUniquePtr<DiffNodeType>& Node, const TFunction<ETreeTraverseControl(const TUniquePtr<DiffNodeType>&)>& Method);
	static bool ReversePostOrderRecursive(const TUniquePtr<DiffNodeType>& Node, const TFunction<ETreeTraverseControl(const TUniquePtr<DiffNodeType>&)>& Method);

	TArray<DiffNodeType*> UpdateQueue;
	TAttribute<TArray<ValueType>> RootValuesA;
	TAttribute<TArray<ValueType>> RootValuesB;
	TUniquePtr<TTreeDiffSpecification<ValueType>> Specification;
};

template <typename InNodeType>
void TDiffNode<InNodeType>::SetDiffType(TUniquePtr<TTreeDiffSpecification<ValueType>>& Specification)
{
	if (ValueA != NullValue && ValueB != NullValue)
	{
		if (Specification->ShouldInheritEqualFromChildren(ValueA, ValueB) && !Children.IsEmpty())
		{
			// iterate children. Parent is identical iff all of it's children are identical
			for (const TUniquePtr<TDiffNode>& Child : Children)
			{
				if (Child->DiffResult != ETreeDiffResult::Identical)
				{
					DiffResult = ETreeDiffResult::DifferentValues;
					return;
				}
			}
			DiffResult = ETreeDiffResult::Identical;
			return;
		}
		else
		{
			if (Specification->AreValuesEqual(ValueA, ValueB))
			{
				DiffResult = ETreeDiffResult::Identical;
				return;
			}
			DiffResult = ETreeDiffResult::DifferentValues;
			return;
		}
	}
	if (ValueA != NullValue && ValueB == NullValue)
	{
		DiffResult = ETreeDiffResult::MissingFromTree2;
		return;
	}
	if (ValueA == NullValue && ValueB != NullValue)
	{
		DiffResult = ETreeDiffResult::MissingFromTree1;
		return;
	}
	
	check(false);
	DiffResult = ETreeDiffResult::Invalid;
}

template <typename InNodeType>
bool TDiffNode<InNodeType>::operator==(const TDiffNode& Other) const
{
	return Other.ValueA == ValueA && Other.ValueB == ValueB;
}

template <typename InNodeType>
TAsyncTreeDifferences<InNodeType>::TAsyncTreeDifferences(const TAttribute<TArray<ValueType>>& InRootValuesA, const TAttribute<TArray<ValueType>>& InRootValuesB)
	: Head(MakeUnique<DiffNodeType>())
	, RootValuesA(InRootValuesA)
	, RootValuesB(InRootValuesB)
	, Specification(MakeUnique<TTreeDiffSpecification<ValueType>>())
{
	QueueParallelNodeLists(RootValuesA.Get(), RootValuesB.Get(), Head.Get());
}

template <typename InNodeType>
void TAsyncTreeDifferences<InNodeType>::Tick(float MaxAllottedTimeMs)
{
	if (UpdateQueue.IsEmpty())
	{
		QueueParallelNodeLists(RootValuesA.Get(), RootValuesB.Get(), Head.Get());
	}
	
	const double StartTime = FPlatformTime::Seconds();
	while(!UpdateQueue.IsEmpty() && FPlatformTime::Seconds() - StartTime < MaxAllottedTimeMs)
	{
		ProcessTopOfQueue();
	}
	
	if (UpdateQueue.IsEmpty())
	{
		OnQueueFlushed.Broadcast(*this);
	}
}

template <typename InNodeType>
void TAsyncTreeDifferences<InNodeType>::FlushQueue()
{
	while(!UpdateQueue.IsEmpty())
	{
		ProcessTopOfQueue();
	}
	OnQueueFlushed.Broadcast(*this);
}

template <typename InNodeType>
void TAsyncTreeDifferences<InNodeType>::ForEach(ETreeTraverseOrder TraversalOrder, const TFunction<ETreeTraverseControl(const TUniquePtr<DiffNodeType>&)>& Method) const
{
	switch(TraversalOrder)
	{
	case ETreeTraverseOrder::PreOrder:
		PreOrderRecursive(Head, Method);
		break;
	case ETreeTraverseOrder::PostOrder:
		PostOrderRecursive(Head, Method);
		break;
	case ETreeTraverseOrder::ReversePreOrder:
		ReversePreOrderRecursive(Head, Method);
		break;
	case ETreeTraverseOrder::ReversePostOrder:
		ReversePostOrderRecursive(Head, Method);
		break;
	default: check(false);
	}
}

template <typename InNodeType>
template <typename SpecificationType, typename... TArgs>
void TAsyncTreeDifferences<InNodeType>::SetDiffSpecification(TArgs... Args)
{
	Specification = MakeUnique<SpecificationType>(Args...);
}

template <typename InNodeType>
bool TAsyncTreeDifferences<InNodeType>::PreOrderRecursive(const TUniquePtr<DiffNodeType>& Node, const TFunction<ETreeTraverseControl(const TUniquePtr<DiffNodeType>&)>& Method)
{
	if (Node->DiffResult != ETreeDiffResult::Invalid)
	{
		switch (Method(Node))
		{
			case ETreeTraverseControl::Break: return false;
			case ETreeTraverseControl::SkipChildren: return true;
		}
	}
	
	for (const TUniquePtr<DiffNodeType>& Child : Node->Children)
	{
		if (!PreOrderRecursive(Child, Method))
		{
			return false;
		}
	}

	return true;
}

template <typename InNodeType>
bool TAsyncTreeDifferences<InNodeType>::PostOrderRecursive(const TUniquePtr<DiffNodeType>& Node, const TFunction<ETreeTraverseControl(const TUniquePtr<DiffNodeType>&)>& Method)
{
	for (const TUniquePtr<DiffNodeType>& Child : Node->Children)
	{
		if (!PostOrderRecursive(Child, Method))
		{
			return false;
		}
	}
	
	if (Node->DiffResult != ETreeDiffResult::Invalid)
	{
		switch (Method(Node))
		{
			case ETreeTraverseControl::Break: return false;
		}
	}

	return true;
}

template <typename InNodeType>
bool TAsyncTreeDifferences<InNodeType>::ReversePreOrderRecursive(const TUniquePtr<DiffNodeType>& Node, const TFunction<ETreeTraverseControl(const TUniquePtr<DiffNodeType>&)>& Method)
{
	if (Node->DiffResult != ETreeDiffResult::Invalid)
	{
		switch (Method(Node))
		{
			case ETreeTraverseControl::Break: return false;
			case ETreeTraverseControl::SkipChildren: return true;
		}
	}
	
	for (int32 I = Node->Children.Num() - 1; I >= 0; --I)
	{
		const TUniquePtr<DiffNodeType>& Child = Node->Children[I];
		if (!ReversePreOrderRecursive(Child, Method))
		{
			return false;
		}
	}

	return true;
}

template <typename InNodeType>
bool TAsyncTreeDifferences<InNodeType>::ReversePostOrderRecursive(const TUniquePtr<DiffNodeType>& Node, const TFunction<ETreeTraverseControl(const TUniquePtr<DiffNodeType>&)>& Method)
{
	for (int32 I = Node->Children.Num() - 1; I >= 0; --I)
	{
		const TUniquePtr<DiffNodeType>& Child = Node->Children[I];
		if (!ReversePostOrderRecursive(Child, Method))
		{
			return false;
		}
	}
	
	if (Node->DiffResult != ETreeDiffResult::Invalid)
	{
		switch (Method(Node))
		{
			case ETreeTraverseControl::Break: return false;
		}
	}

	return true;
}

template <typename InNodeType>
void TAsyncTreeDifferences<InNodeType>::ProcessTopOfQueue()
{
	DiffNodeType* DiffNode = UpdateQueue.Pop();
	
	TArray<ValueType> ChildrenA;
	TArray<ValueType> ChildrenB;
	if (DiffNode->ValueA != DiffNodeType::NullValue)
	{
		Specification->GetChildren(DiffNode->ValueA, ChildrenA);
	}
	if (DiffNode->ValueB != DiffNodeType::NullValue)
	{
		Specification->GetChildren(DiffNode->ValueB, ChildrenB);
	}
	
	if (!ChildrenA.IsEmpty() || !ChildrenB.IsEmpty())
	{
		QueueParallelNodeLists(ChildrenA, ChildrenB, DiffNode);
	}
}


// this is where the magic happens :)
template <typename InNodeType>
void TAsyncTreeDifferences<InNodeType>::QueueParallelNodeLists(const TArray<ValueType>& ValuesA, const TArray<ValueType>& ValuesB, DiffNodeType* ParentNode)
{
	// calculate the diff nodes that should be children of ParentNode
	TArray<DiffNodeType> FoundChildren;
	{
		// reversing data before calculating LCS puts "add" diff results towards the end which is a friendlier format
		// (don't worry we'll return the data to it's original order)
		Algo::Reverse(const_cast<TArray<ValueType>&>(ValuesA));
		Algo::Reverse(const_cast<TArray<ValueType>&>(ValuesB));
		
		// Find the Longest Common Subsequence between node lists
        const TArray<TArray<int32>> LCS = CalculateLCSTableForMatchingValues(ValuesA, ValuesB);
        
        // Using an LCS Table, we can determine which tree nodes match one another in each tree and queue them up to update together
        int32 IndexA = ValuesA.Num();
        int32 IndexB = ValuesB.Num();
        while (IndexA > 0 || IndexB > 0)
        {
        	if (IndexA == 0)
        	{
        		// ValueB doesn't match. push it on it's own
        		FoundChildren.Add(DiffNodeType(DiffNodeType::NullValue, ValuesB[--IndexB], ParentNode));
        	}
        	else if (IndexB == 0)
        	{
        		// ValueA doesn't match. push it on it's own
        		FoundChildren.Add(DiffNodeType(ValuesA[--IndexA], DiffNodeType::NullValue, ParentNode));
        	}
        	else if (AreMatching(ValuesA[IndexA - 1], ValuesB[IndexB - 1]))
        	{
        		// found a match between both nodes
        		FoundChildren.Add(DiffNodeType(ValuesA[--IndexA], ValuesB[--IndexB], ParentNode));
        	}
        	else if (LCS[IndexA - 1][IndexB] <= LCS[IndexA][IndexB - 1])
        	{
        		// ValueB doesn't match. push it on it's own
        		FoundChildren.Add(DiffNodeType(DiffNodeType::NullValue, ValuesB[--IndexB], ParentNode));
        	}
        	else
        	{
        		// ValueA doesn't match. push it on it's own
        		FoundChildren.Add(DiffNodeType(ValuesA[--IndexA], DiffNodeType::NullValue, ParentNode));
        	}
        }

		// undo the temp reverse from above
		Algo::Reverse(const_cast<TArray<ValueType>&>(ValuesA));
		Algo::Reverse(const_cast<TArray<ValueType>&>(ValuesB));
	}
	
	// compress consecutive value matched entries
	TArray<DiffNodeType> AllFoundChildren = MoveTemp(FoundChildren);
	FoundChildren = TArray<DiffNodeType>();

	int32 CompressIndex = INDEX_NONE;
	int32 CompressCount = 0;
	for (DiffNodeType& FoundChild : AllFoundChildren)
	{
		if (FoundChild.ValueA == DiffNodeType::NullValue &&
			Specification->ShouldMatchByValue(FoundChild.ValueB))
		{
			if (CompressCount == 0)
			{
				CompressIndex = FoundChildren.Num();
			}
			++CompressCount;
			FoundChildren.Add(MoveTemp(FoundChild));
			continue;
		}
		if (CompressCount > 0 &&
            FoundChild.ValueB == DiffNodeType::NullValue &&
			Specification->ShouldMatchByValue(FoundChild.ValueA))
		{
			FoundChildren[CompressIndex].ValueA = FoundChild.ValueA;
			++CompressIndex;
			--CompressCount;
			continue;
		}
		
		FoundChildren.Add(MoveTemp(FoundChild));
		CompressIndex = INDEX_NONE;
		CompressCount = 0;
	}
	
	// compare/replace results with what's *actually* in the parent node to avoid modifying nodes that didn't change
	{
		TArray<TUniquePtr<DiffNodeType>> ExpectedChildren = MoveTemp(ParentNode->Children);
		ParentNode->Children = TArray<TUniquePtr<DiffNodeType>>();
		const TArray<TArray<int32>> LCS = CalculateLCSTableForDiffNodes(FoundChildren, ExpectedChildren);
		int32 FoundChildrenIndex = FoundChildren.Num();
		int32 ExpectedChildrenIndex = ExpectedChildren.Num();
		while (FoundChildrenIndex > 0)
		{
			if (ExpectedChildrenIndex == 0)
			{
				// a new child was found. move it into the children
				ParentNode->Children.Add(MakeUnique<DiffNodeType>(MoveTemp(FoundChildren[--FoundChildrenIndex])));
			}
			else if (FoundChildren[FoundChildrenIndex - 1] == *ExpectedChildren[ExpectedChildrenIndex - 1])
			{
				// found a match between both nodes. Preserve the old data by copying it into the new child array
				ParentNode->Children.Add(MoveTemp(ExpectedChildren[--ExpectedChildrenIndex]));
				--FoundChildrenIndex;
			}
			else if (LCS[FoundChildrenIndex - 1][ExpectedChildrenIndex] <= LCS[FoundChildrenIndex][ExpectedChildrenIndex - 1])
			{
				// a child was removed. ignore the old data
				--ExpectedChildrenIndex;
			}
			else
			{
				// a new child was found. move it into the children
				ParentNode->Children.Add(MakeUnique<DiffNodeType>(MoveTemp(FoundChildren[--FoundChildrenIndex])));
			}
		}
	}
	Algo::Reverse(ParentNode->Children);

	// diff the values of all the children
	for (const TUniquePtr<DiffNodeType>& Child : ParentNode->Children)
	{
		Child->SetDiffType(Specification);
	}

	// queue all the children to update
	for (const TUniquePtr<DiffNodeType>& Child : ParentNode->Children)
	{
		UpdateQueue.Add(Child.Get());
	}
}

template <typename InNodeType>
TArray<TArray<int32>> TAsyncTreeDifferences<InNodeType>::CalculateLCSTableForMatchingValues(
	const TArray<ValueType>& ValuesA, const TArray<ValueType>& ValuesB)
{
	TArray<TArray<int32>> LCS;
	LCS.SetNum(ValuesA.Num() + 1);
	for (int32 I = 0; I <= ValuesA.Num(); I++)
	{
		LCS[I].SetNum(ValuesB.Num() + 1);
		if (I == 0)
		{
			continue;
		}
	
		for (int32 J = 1; J <= ValuesB.Num(); J++)
		{
			if (AreMatching(ValuesA[I - 1], ValuesB[J - 1]))
			{
				LCS[I][J] = LCS[I - 1][J - 1] + 1;
			}
			else
			{
				LCS[I][J] = FMath::Max(LCS[I - 1][J], LCS[I][J - 1]);
			}
		}
	}
	return LCS;
}

template <typename InNodeType>
TArray<TArray<int32>> TAsyncTreeDifferences<InNodeType>::CalculateLCSTableForDiffNodes(const TArray<DiffNodeType>& FoundNodes,
	const TArray<TUniquePtr<DiffNodeType>>& ExpectedNodes)
{
	TArray<TArray<int32>> LCS;
	LCS.SetNum(FoundNodes.Num() + 1);
	for (int32 I = 0; I <= FoundNodes.Num(); I++)
	{
		LCS[I].SetNum(ExpectedNodes.Num() + 1);
		if (I == 0)
		{
			continue;
		}
	
		for (int32 J = 1; J <= ExpectedNodes.Num(); J++)
		{
			if (FoundNodes[I - 1] == *ExpectedNodes[J - 1])
			{
				LCS[I][J] = LCS[I - 1][J - 1] + 1;
			}
			else
			{
				LCS[I][J] = FMath::Max(LCS[I - 1][J], LCS[I][J - 1]);
			}
		}
	}
	return LCS;
}

template <typename InNodeType>
bool TAsyncTreeDifferences<InNodeType>::AreMatching(const ValueType& ValueA, const ValueType& ValueB)
{
	const bool bMatchValueAByValue = Specification->ShouldMatchByValue(ValueA);
	const bool bMatchValueBByValue = Specification->ShouldMatchByValue(ValueB);
	if (bMatchValueAByValue && bMatchValueBByValue)
	{
		return Specification->AreValuesEqual(ValueA, ValueB);
	}
	if (!bMatchValueAByValue && !bMatchValueBByValue)
	{
		return Specification->AreMatching(ValueA, ValueB);
	}

	// a node that should match by value will never match a node that shouldn't match by value
	return false;
}

