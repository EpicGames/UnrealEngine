// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "EditorFramework/AssetImportData.h"
#include "Misc/Optional.h"
#include "Tasks/Pipe.h"
#include "UObject/Object.h"

class FName;
class IAssetRegistry;
class UAssetImportData;
struct FAssetData;
struct FAssetImportInfo;


/** Class responsible for maintaing a cache of clean source file names (bla.txt) to asset data */
class FAssetSourceFilenameCache
{
public:
	UNREALED_API FAssetSourceFilenameCache();

	/** Singleton access */
	static UNREALED_API FAssetSourceFilenameCache& Get();

	/** Helper functions to extract asset import information from asset registry tags */
	static UNREALED_API TOptional<FAssetImportInfo> ExtractAssetImportInfo(const FAssetData& AssetData);

	/** Retrieve a list of assets that were imported from the specified filename */
	UNREALED_API TArray<FAssetData> GetAssetsPertainingToFile(const IAssetRegistry& Registry, const FString& AbsoluteFilename) const;

	/** Shutdown this instance of the cache */
	UNREALED_API void Shutdown();

	/** Event for when an asset has been renamed, and has been updated in our source file cache */
	DECLARE_EVENT_TwoParams( FAssetSourceFilenameCache, FAssetRenamedEvent, const FAssetData&, const FString& );
	FAssetRenamedEvent& OnAssetRenamed() { return AssetRenamedEvent; }

private:
	/** Delegate bindings that keep the cache up-to-date */
	UNREALED_API void HandleOnAssetsAdded(TConstArrayView<FAssetData> Assets);
	UNREALED_API void HandleOnAssetsRemoved(TConstArrayView<FAssetData> AssetData);
	UNREALED_API void HandleOnAssetRenamed(const FAssetData& AssetData, const FString& OldPath);
	UNREALED_API void HandleOnAssetUpdated(const FAssetImportInfo& OldData, const UAssetImportData* ImportData);
	
	/** Event that is triggered when an asset has been renamed, and we've updated our cache */
	FAssetRenamedEvent AssetRenamedEvent;

	// Pipe to serialize access to cache 
	mutable UE::Tasks::FPipe CachePipe;	

	/** Map of clean filenames (no leading path information) to object paths that were imported with that file */
	TMap<FString, TSet<FSoftObjectPath>> SourceFileToObjectPathCache;
};
