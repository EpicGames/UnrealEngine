// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "AutoReimportManager.generated.h"

class FAutoReimportManager;
struct FAutoReimportDirectoryConfig;

/** Struct representing a path on disk, and its virtual mount point */
struct FPathAndMountPoint
{
	FPathAndMountPoint() {}
	FPathAndMountPoint(FString InPath, FString InMountPoint) : Path(MoveTemp(InPath)), MountPoint(MoveTemp(InMountPoint)) {}

	/** The directory on disk. Absolute. */
	FString Path;

	/** The mount point, if any to which this directory relates */
	FString MountPoint;
};
	
/* Deals with auto reimporting of objects when the objects file on disk is modified*/
UCLASS(config=Editor, transient, MinimalAPI)
class UAutoReimportManager : public UObject
{
	GENERATED_UCLASS_BODY()
public:

	UNREALED_API ~UAutoReimportManager();

	/** Initialize during engine startup */
	UNREALED_API void Initialize();

	/** Get a list of absolute directories that we are monitoring */
	UNREALED_API TArray<FPathAndMountPoint> GetMonitoredDirectories() const;

	/** Report an external change to the manager, such that a subsequent equal change reported by the os be ignored */
	UNREALED_API void IgnoreNewFile(const FString& Filename);
	UNREALED_API void IgnoreFileModification(const FString& Filename);
	UNREALED_API void IgnoreMovedFile(const FString& SrcFilename, const FString& DstFilename);
	UNREALED_API void IgnoreDeletedFile(const FString& Filename);

	/** Add a directory to watch at runtime. Directories added this way are not saved in the settings. Returns a handle that can be passed to RemoveRuntimeMonitoredDirectory.*/
	UNREALED_API int32 AddRuntimeMonitoredDirectory(const FAutoReimportDirectoryConfig& Directory);

	/** Remove runtime directory with the given handle. */
	UNREALED_API void RemoveRuntimeMonitoredDirectory(const int32 Handle);

private:

	/** UObject Interface */
	UNREALED_API virtual void BeginDestroy() override;

	/** Private implementation of the reimport manager */
	TSharedPtr<class FAutoReimportManager> Implementation;
};
