// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Logging/LogMacros.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "AutomatedAssetImportData.generated.h"

class FJsonObject;
class UFactory;

DECLARE_LOG_CATEGORY_EXTERN(LogAutomatedImport, Log, All);

/**
 * Contains data for a group of assets to import
 */ 
UCLASS(Transient, BlueprintType, MinimalAPI)
class UAutomatedAssetImportData : public UObject
{
	GENERATED_BODY()

public:
	UNREALED_API UAutomatedAssetImportData();

	/** @return true if this group contains enough valid data to import*/
	UNREALED_API bool IsValid() const;

	/** Initalizes the group */
	UNREALED_API void Initialize(TSharedPtr<FJsonObject> InImportGroupJsonData);

	/** @return the display name of the group */
	UNREALED_API FString GetDisplayName() const; 
public:
	/** Display name of the group. This is for logging purposes only. */
	UPROPERTY(BlueprintReadWrite, Category="Asset Import Data")
	FString GroupName;

	/** Filenames to import */
	UPROPERTY(BlueprintReadWrite, Category = "Asset Import Data")
	TArray<FString> Filenames;

	/** Content path in the projects content directory where assets will be imported */
	UPROPERTY(BlueprintReadWrite, Category = "Asset Import Data")
	FString DestinationPath;

	/** Name of the factory to use when importing these assets. If not specified the factory type will be auto detected */
	UPROPERTY(BlueprintReadWrite, Category = "Asset Import Data")
	FString FactoryName;

	/** Whether or not to replace existing assets */
	UPROPERTY(BlueprintReadWrite, Category = "Asset Import Data")
	bool bReplaceExisting;

	/** Whether or not to skip importing over read only assets that could not be checked out */
	UPROPERTY(BlueprintReadWrite, Category = "Asset Import Data")
	bool bSkipReadOnly;

	/** Pointer to the factory currently being used */
	UPROPERTY(BlueprintReadWrite, Category = "Asset Import Data")
	TObjectPtr<UFactory> Factory;

	/** Full path to level to load before importing this group (only matters if importing assets into a level) */
	UPROPERTY(BlueprintReadWrite, Category = "Asset Import Data")
	FString LevelToLoad;

	/** Json data to be read when importing this group */
	TSharedPtr<FJsonObject> ImportGroupJsonData;

};
