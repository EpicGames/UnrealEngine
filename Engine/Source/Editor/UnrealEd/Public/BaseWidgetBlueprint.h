// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Blueprint/UserWidgetBlueprint.h"
#include "BaseWidgetBlueprint.generated.h"

UCLASS(Abstract, MinimalAPI)
class UBaseWidgetBlueprint : public UUserWidgetBlueprint
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITORONLY_DATA
	/** A tree of the widget templates to be created */
	UPROPERTY()
	TObjectPtr<class UWidgetTree> WidgetTree;
#endif

	UNREALED_API virtual void PostLoad() override;

	/**
	* Returns collection of widgets that represent the 'source' (user edited) widgets for this
	* blueprint - avoids calling virtual functions on instances and is therefore safe to use
	* throughout compilation.
	*/
	UNREALED_API TArray<class UWidget*> GetAllSourceWidgets();
	UNREALED_API TArray<const class UWidget*> GetAllSourceWidgets() const;

	/** Identical to GetAllSourceWidgets, but as an algorithm */
	UNREALED_API void ForEachSourceWidget(TFunctionRef<void(class UWidget*)> Fn);
	UNREALED_API void ForEachSourceWidget(TFunctionRef<void(class UWidget*)> Fn) const;

private:
	void ForEachSourceWidgetImpl(TFunctionRef<void(class UWidget*)> Fn) const;
};
