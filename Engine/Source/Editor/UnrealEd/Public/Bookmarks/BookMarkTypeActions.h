// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Bookmarks/IBookmarkTypeActions.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"

#include "Engine/BookMark.h"
#include "Engine/LevelStreaming.h"
#include "Engine/World.h"
#include "LevelEditorViewport.h"

class FBookMarkTypeActions : public IBookmarkTypeActions
{
public:

	virtual TSubclassOf<class UBookmarkBase> GetBookmarkClass() override
	{
		return UBookMark::StaticClass();
	}

	virtual void InitFromViewport(UBookmarkBase* InBookmark, FEditorViewportClient& InViewportClient) override
	{
		if (UBookMark* Bookmark = Cast<UBookMark>(InBookmark))
		{
			if (UWorld * const World = InViewportClient.GetWorld())
			{
				Bookmark->Modify();

				// Use the rotation from the first perspective viewport can find.
				FRotator Rotation(0, 0, 0);
				if (!InViewportClient.IsOrtho())
				{
					Rotation = InViewportClient.GetViewRotation();
				}

				Bookmark->Location = InViewportClient.GetViewLocation();
				Bookmark->Rotation = Rotation;

				// Keep a record of which levels were hidden so that we can restore these with the bookmark
				Bookmark->HiddenLevels.Empty();
				for (ULevelStreaming* StreamingLevel : World->GetStreamingLevels())
				{
					if (StreamingLevel && !StreamingLevel->GetShouldBeVisibleInEditor())
					{
						Bookmark->HiddenLevels.Add(StreamingLevel->GetFullName());
					}
				}
			}
		}
	}

	virtual void JumpToBookmark(UBookmarkBase* InBookmark, const TSharedPtr<struct FBookmarkBaseJumpToSettings> InSettings, FEditorViewportClient& InViewportClient) override
	{
		if (UBookMark* Bookmark = Cast<UBookMark>(InBookmark))
		{
			// Set all level editing cameras to this bookmark
			for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
			{
				ApplyBookmarkToViewportClient(Bookmark, LevelVC);
			}
		}
	}

protected:

	void ApplyBookmarkToViewportClient(UBookMark* InBookmark, FEditorViewportClient* InViewportClient)
	{
		InViewportClient->SetViewLocation(InBookmark->Location);
		if (!InViewportClient->IsOrtho())
		{
			InViewportClient->SetViewRotation(InBookmark->Rotation);
		}
		InViewportClient->Invalidate();

		FEditorDelegates::OnEditorCameraMoved.Broadcast(InBookmark->Location, InBookmark->Rotation, InViewportClient->ViewportType, InViewportClient->ViewIndex);
	}
};
