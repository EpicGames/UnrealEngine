// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class IBookmarkTypeActions;
class FEditorViewportClient;
class UBookMark;

/**
 * Provides a way to temporarily bookmark and restore all viewports
 * This allows camera locations and rotations to be preserved across a map reload
 */
class FBookmarkScoped
{
public:

	/** Constructor; bookmark all viewports */
	UNREALED_API FBookmarkScoped();

	/** Destructor; restore all viewports */
	UNREALED_API ~FBookmarkScoped();

private:

	/** Action which allows a single viewport to jump to a single bookmark */
	TSharedPtr<IBookmarkTypeActions> Actions;

	/** Bookmarks for each viewport */
	TMap<FEditorViewportClient*, UBookMark*> BookMarks;
};
