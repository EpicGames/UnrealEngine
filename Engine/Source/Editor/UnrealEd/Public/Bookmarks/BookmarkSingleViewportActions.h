// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Bookmarks/BookMarkTypeActions.h"

/**
 * A bookmark action which jumps a single viewport to a single bookmark
 */
class FBookmarkSingleViewportActions : public FBookMarkTypeActions
{
public:

	virtual void JumpToBookmark(UBookmarkBase* InBookmark, const TSharedPtr<struct FBookmarkBaseJumpToSettings> InSettings, FEditorViewportClient& InViewportClient) override
	{
		if (UBookMark* Bookmark = Cast<UBookMark>(InBookmark))
		{
			ApplyBookmarkToViewportClient(Bookmark, &InViewportClient);
		}
	}
};
