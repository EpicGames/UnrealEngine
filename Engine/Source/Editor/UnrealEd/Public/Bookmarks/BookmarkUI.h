// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Textures/SlateIcon.h"
#include "Internationalization/Internationalization.h"

struct FBookmarkUI
{
	static FORCEINLINE FSlateIcon GetDefaultIcon()
	{
		return FSlateIcon();
	}

	static FORCEINLINE FName GetJumpToCommandName(const int32 BookmarkIndex)
	{
		return FName(*FString::Printf(TEXT("JumpToBookmark%i"), BookmarkIndex));
	}

	static FORCEINLINE FText GetJumpToTooltip(const int32 BookmarkIndex)
	{
		return FText::Format(NSLOCTEXT("Bookmarks", "JumpToBookmark_ToolTip", "Moves the viewport to the location and orientation stored at bookmark {0}"), FText::AsNumber(BookmarkIndex));
	}

	static FORCEINLINE FText GetJumpToLabel(const int32 BookmarkIndex)
	{
		return FText::Format(NSLOCTEXT("Bookmarks", "JumpToBookmark", "Jump to Bookmark {0}"), FText::AsNumber(BookmarkIndex));
	}

	static FORCEINLINE FText GetPlainLabel(const int32 BookmarkIndex)
	{
		return FText::Format(NSLOCTEXT("Bookmarks", "Bookmark", "Bookmark {0}"), FText::AsNumber(BookmarkIndex));
	}

	static FORCEINLINE FName GetSetCommandName(const int32 BookmarkIndex)
	{
		return FName(*FString::Printf(TEXT("SetBookmark%i"), BookmarkIndex));
	}

	static FORCEINLINE FText GetSetTooltip(const int32 BookmarkIndex)
	{
		return FText::Format(NSLOCTEXT("Bookmarks", "SetBookmark_ToolTip", "Stores the viewports location and orientation in bookmark {0}"), FText::AsNumber(BookmarkIndex));
	}

	static FORCEINLINE FText GetSetLabel(const int32 BookmarkIndex)
	{
		return FText::Format(NSLOCTEXT("Bookmarks", "SetBookmark", "Set Bookmark {0}"), FText::AsNumber(BookmarkIndex));
	}

	static FORCEINLINE FName GetClearCommandName(const int32 BookmarkIndex)
	{
		return FName(*FString::Printf(TEXT("ClearBookmark%i"), BookmarkIndex));
	}

	static FORCEINLINE FText GetClearTooltip(const int32 BookmarkIndex)
	{
		return FText::Format(NSLOCTEXT("Bookmarks", "ClearBookmark_ToolTip", "Clears the viewports location and orientation in bookmark {0}"), FText::AsNumber(BookmarkIndex));
	}

	static FORCEINLINE FText GetClearLabel(const int32 BookmarkIndex)
	{
		return FText::Format(NSLOCTEXT("Bookmarks", "ClearBookmark", "Clear Bookmark {0}"), FText::AsNumber(BookmarkIndex));
	}
};