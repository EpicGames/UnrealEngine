// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"

/**
 * Actions that can be applied to or completed with bookmarks.
 * Must be registered, see IBookmarkTypeTools.
 */
class IBookmarkTypeActions
{
public:

	virtual ~IBookmarkTypeActions() = default;

	/**
	 * Gets the class of bookmarks with which these actions can be used.
	 * Must match exactly with the desired bookmark class.
	 */
	virtual TSubclassOf<class UBookmarkBase> GetBookmarkClass() = 0;

	/**
	 * Called to initialize the given bookmark from the given Viewport.
	 *
	 * @param InBookmark		The bookmark to initialize.
	 * @param InViewportClient	The client that can be used for initialization.
	 */
	virtual void InitFromViewport(class UBookmarkBase* InBookmark, class FEditorViewportClient& InViewportClient) = 0;

	/**
	 * Called to jump to the given bookmark.
	 *
	 * @param InBookmark		The bookmark that should be jumped to / activated.
	 * @param InSettings		Optional settings, dependent on the bookmark class.
	 * @param InViewportClient	The viewport that will be updated.
	 */
	virtual void JumpToBookmark(class UBookmarkBase* Bookmark, const TSharedPtr<struct FBookmarkBaseJumpToSettings> InSettings, class FEditorViewportClient& InViewportClient) = 0;
};
