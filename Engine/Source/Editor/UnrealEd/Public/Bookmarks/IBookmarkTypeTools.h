// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UnrealTemplate.h"
#include "Templates/SubclassOf.h"

/**
 * Provides convenience methods for interacting with Bookmarks.
 */
class IBookmarkTypeTools : public FNoncopyable
{
protected:

	virtual ~IBookmarkTypeTools() = default;

public:

	static UNREALED_API IBookmarkTypeTools& Get();

	/**
	 * Gets the current maximum number of bookmarks allowed.
	 *
	 * @param InViewportClient	Level editor viewport client used to reference the world which owns the bookmarks.
	 */
	virtual const uint32 GetMaxNumberOfBookmarks(const class FEditorViewportClient* InViewportClient) const = 0;

	/**
	 * Checks to see if a bookmark exists at a given index
	 * 
	 * @param InIndex			Index of the bookmark to set.
	 * @param InViewportClient	Level editor viewport client used to reference the world which owns the bookmark.
	 */
	virtual const bool CheckBookmark(const uint32 InIndex, const class FEditorViewportClient* InViewportClient) const = 0;

	/**
	 * Sets the specified bookmark based on the given viewport, allocating it if necessary.
	 * 
	 * @param InIndex			Index of the bookmark to set.
	 * @param InViewportClient	Level editor viewport client used to reference the world which owns the bookmark.
	 */
	virtual void CreateOrSetBookmark(const uint32 InIndex, class FEditorViewportClient* InViewportClient) const = 0;

	/**
	 * Compacts the available bookmarks into mapped spaces.
	 * Does nothing if all mapped spaces are already filled, or if no bookmarks exist that are not mapped.
	 *
	 * @param InViewportClient	Level editor viewport client used to reference the world which owns the bookmarks.
	 */
	virtual void CompactBookmarks(class FEditorViewportClient* InViewportClient) const = 0;

	/**
	 * Jumps to a bookmark from the list.
	 * 
	 * @param InIndex			Index of the bookmark to set.
	 * @param InSettings		Settings to used when jumpting to the bookmark.
	 * @param InViewportClient	Level editor viewport client used to reference the world which owns the bookmark.
	 */
	virtual void JumpToBookmark(const uint32 InIndex, const TSharedPtr<struct FBookmarkBaseJumpToSettings> InSettings, class FEditorViewportClient* InViewportClient) const = 0;

	/**
	 * Clears a bookmark from the list.
	 * 
	 * @param InIndex			Index of the bookmark to clear.
	 * @param InViewportClient	Level editor viewport client used to reference the world which owns the bookmark.
	 */
	virtual void ClearBookmark(const uint32 InIndex, class FEditorViewportClient* InViewportClient) const = 0;

	/**
	 * Clears all book marks
	 * 
	 * @param InViewportClient	Level editor viewport client used to reference the world which owns the bookmarks.
	 */
	virtual void ClearAllBookmarks(class FEditorViewportClient* InViewportClient) const = 0;

	/**
	 * Gets the currently configured Bookmark class.
	 */
	virtual const TSubclassOf<class UBookmarkBase> GetBookmarkClass(const class FEditorViewportClient* InViewportClient) const = 0;

	/**
	 * Registers the given Bookmark Type Actions so they can be used by the editor.
	 *
	 * @param InActions			The Actions to register.		
	 */
	virtual void RegisterBookmarkTypeActions(const TSharedRef<class IBookmarkTypeActions>& InActions) = 0;

	/**
	 * Unregisters the given Bookmark Type Actions so they are no longer considered by the editor.
	 *
	 * @param InActions			The Actions to unregister.		
	 */
	virtual void UnregisterBookmarkTypeActions(const TSharedRef<class IBookmarkTypeActions>& InActions) = 0;

	/**
	 * Upgrades all bookmarks, ensuring they are of appropriate class. 
	 * Note, this is currently not used.
	 *
	 * @param InViewportClient	Level editor viewport client used to reference the world which owns the bookmarks.
	 * @param InWorldSettings	World Settings object that is requesting the upgrade.
	 */
	virtual void UpgradeBookmarks(class FEditorViewportClient* InViewportClient, class AWorldSettings* InWorldSettings) const = 0;
};
