// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * While in scope, sets the cursor to the busy (hourglass) cursor for all windows.
 */
class FScopedBusyCursor
{
public:
	UNREALED_API FScopedBusyCursor();
	UNREALED_API ~FScopedBusyCursor();
};
