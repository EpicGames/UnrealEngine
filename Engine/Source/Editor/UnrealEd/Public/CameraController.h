// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CameraController.h: Implements controls for a camera with pseudo-physics
=============================================================================*/

#pragma once

#include "CoreMinimal.h"

/**
 * FCameraControllerUserImpulseData
 *
 * Wrapper structure for all of the various user input parameters for camera movement
 */
class FCameraControllerUserImpulseData
{

public:

	/** Scalar user input for moving forwards (positive) or backwards (negative) */
	float MoveForwardBackwardImpulse;

	/** Scalar user input for moving right (positive) or left (negative) */
	float MoveRightLeftImpulse;

	/** Scalar user input for moving up (positive) or down (negative) in world space */
	float MoveWorldUpDownImpulse;

	/** Scalar user input for moving up (positive) or down (negative) in local space */
	float MoveLocalUpDownImpulse;

	/** Scalar user input for turning right (positive) or left (negative) */
	float RotateYawImpulse;

	/** Scalar user input for pitching up (positive) or down (negative) */
	float RotatePitchImpulse;

	/** Scalar user input for rolling clockwise (positive) or counter-clockwise (negative) */
	float RotateRollImpulse;

	/** Velocity modifier for turning right (positive) or left (negative) */
	float RotateYawVelocityModifier;

	/** Velocity modifier for pitching up (positive) or down (negative) */
	float RotatePitchVelocityModifier;

	/** Velocity modifier for rolling clockwise (positive) or counter-clockwise (negative) */
	float RotateRollVelocityModifier;

	/** Scalar user input for increasing FOV (zoom out, positive) or decreasing FOV (zoom in, negative) */
	float ZoomOutInImpulse;


	/** Constructor */
	FCameraControllerUserImpulseData()
		: MoveForwardBackwardImpulse( 0.0f ),
		  MoveRightLeftImpulse( 0.0f ),
		  MoveWorldUpDownImpulse( 0.0f ),
		  MoveLocalUpDownImpulse( 0.0f ),
		  RotateYawImpulse( 0.0f ),
		  RotatePitchImpulse( 0.0f ),
		  RotateRollImpulse( 0.0f ),
		  RotateYawVelocityModifier( 0.0f ),
		  RotatePitchVelocityModifier( 0.0f ),
		  RotateRollVelocityModifier( 0.0f ),
		  ZoomOutInImpulse( 0.0f )
	{
	}
	
	bool HasMovement() const
	{
		return RotateYawVelocityModifier != 0.0f ||
			RotatePitchVelocityModifier != 0.0f ||
			RotateRollVelocityModifier != 0.0f ||
			MoveForwardBackwardImpulse != 0.0f ||
			MoveRightLeftImpulse != 0.0f ||
			MoveWorldUpDownImpulse != 0.0f ||
			MoveLocalUpDownImpulse != 0.0f ||
			ZoomOutInImpulse != 0.0f ||
			HasRotationalImpulse();
	}
	
	bool HasRotationalImpulse() const
	{
		return RotateYawImpulse != 0.0f ||
			RotatePitchImpulse != 0.0f ||
			RotateRollImpulse != 0.0f;
	}
};



/**
 * FCameraControllerConfig
 *
 * Configuration data for the camera controller object
 */
class FCameraControllerConfig
{

public:

	/**
	 * General configuration
	 */

	/** Impulses below this amount will be ignored */
	float ImpulseDeadZoneAmount;


	/**
	 * Movement configuration
	 */

	/** True if camera movement (forward/backward/left/right) should use a physics model with velocity */
	bool bUsePhysicsBasedMovement;

	/** Movement acceleration rate in units per second per second */
	float MovementAccelerationRate;

	/** Movement velocity damping amount in 'velocities' per second */
	float MovementVelocityDampingAmount;

	/** Maximum movement speed in units per second */
	float MaximumMovementSpeed;


	/**
	 * Rotation configuration
	 */

	/** True if camera rotation (yaw/pitch/roll) should use a physics model with velocity */
	bool bUsePhysicsBasedRotation;

	/**Allows xbox controller to temporarily force rotational physics on*/
	bool bForceRotationalPhysics;

	/** Rotation acceleration rate in degrees per second per second */
	float RotationAccelerationRate;

	/** Rotation velocity damping amount in 'velocities' per second */
	float RotationVelocityDampingAmount;

	/** Maximum rotation speed in degrees per second */
	float MaximumRotationSpeed;

	/** Minimum allowed camera pitch rotation in degrees */
	float MinimumAllowedPitchRotation;

	/** Maximum allowed camera pitch rotation in degrees */
	float MaximumAllowedPitchRotation;


	/**
	 * FOV zooming configuration
	 */

	/** True if FOV should snap back when flight controls are released */
	bool bEnableFOVRecoil;

	/** True if FOV zooming should use a physics model with velocity */
	bool bUsePhysicsBasedFOV;

	/** FOV acceleration rate in degrees per second per second */
	float FOVAccelerationRate;

	/** FOV velocity damping amount in 'velocities' per second */
	float FOVVelocityDampingAmount;

	/** Maximum FOV change speed in degrees per second */
	float MaximumFOVSpeed;

	/** Minimum allowed camera FOV in degrees */
	float MinimumAllowedFOV;

	/** Maximum allowed camera FOV in degrees */
	float MaximumAllowedFOV;

	/**Multiplier for translation movement*/
	float TranslationMultiplier;
	/**Multiplier for rotation movement*/
	float RotationMultiplier;
	/**Multiplier for zoom movement*/
	float ZoomMultiplier;
	/**Camera Trim (pitch offset) */
	float PitchTrim;

	/**Invert the impulses on the x axis*/
	bool bInvertX;
	/**Invert the impulses on the y axis*/
	bool bInvertY;
	/**Whether the camera is planar or free flying*/
	bool bPlanarCamera;

	/**True if we wish to constrain the pitch to a min/max angle*/
	bool bLockedPitch;

	/** Constructor */
	FCameraControllerConfig()
		: ImpulseDeadZoneAmount( 0.2f ),
		  bUsePhysicsBasedMovement( true ),	  
		  MovementAccelerationRate( 20000.0f ),
		  MovementVelocityDampingAmount( 10.0f ),
		  MaximumMovementSpeed( MAX_FLT ),
		  bUsePhysicsBasedRotation( false ),
		  bForceRotationalPhysics( false ),
		  RotationAccelerationRate( 1600.0f ),
		  RotationVelocityDampingAmount( 12.0f ),
		  MaximumRotationSpeed( MAX_FLT ),
		  MinimumAllowedPitchRotation( -90.0f ),
		  MaximumAllowedPitchRotation( 90.0f ),
		  bEnableFOVRecoil( true ),
		  bUsePhysicsBasedFOV( true ),
		  FOVAccelerationRate( 1200.0f ),
		  FOVVelocityDampingAmount( 10.0f ),
		  MaximumFOVSpeed( MAX_FLT ),
		  MinimumAllowedFOV( 5.0f ),
		  MaximumAllowedFOV( 170.0f ),
		  TranslationMultiplier(1.0f),
		  RotationMultiplier(1.0f),
		  ZoomMultiplier(1.0f),
		  PitchTrim(0.0f),
		  bInvertX(false),
		  bInvertY(false),
		  bPlanarCamera(false),
		  bLockedPitch(true)
	{
	}

};



/**
 * FEditorCameraController
 *
 * An interactive camera movement system.  Supports simple physics-based animation.
 */
class FEditorCameraController
{

public:

	/** Constructor */
	UNREALED_API FEditorCameraController();

	virtual ~FEditorCameraController() = default;

	/** Sets the configuration for this camera controller */
	void SetConfig( const FCameraControllerConfig& InConfig )
	{
		Config = InConfig;
	}


	/** Returns the configuration of this camera controller */
	FCameraControllerConfig& GetConfig()
	{
		return Config;
	}


	/** Access the configuration for this camera.  Making changes is allowed. */
	FCameraControllerConfig& AccessConfig()
	{
		return Config;
	}


	/**
	 * Updates the position and orientation of the camera as well as other state (like velocity.)  Should be
	 * called every frame.
	 *
	 * @param	UserImpulseData			Input data from the user this frame
	 * @param	DeltaTime				Time interval since last update
	 * @param	bAllowRecoilIfNoImpulse	True if we should recoil FOV if needed
	 * @param	MovementSpeedScale		Scales the speed of movement
	 * @param	InOutCameraPosition		[in, out] Camera position
	 * @param	InOutCameraEuler		[in, out] Camera orientation
	 * @param	InOutCameraFOV			[in, out] Camera field of view
	 */
	UNREALED_API virtual void UpdateSimulation(
		const FCameraControllerUserImpulseData& UserImpulseData,
		const float DeltaTime,
		const bool bAllowRecoilIfNoImpulse,
		const float MovementSpeedScale,
		FVector& InOutCameraPosition,
		FVector& InOutCameraEuler,
		float& InOutCameraFOV );

	/** @return True if this camera currently has rotational velocity. */
	bool IsRotating (void) const;

	/** Reset the different camera velocity. */
	UNREALED_API void ResetVelocity();

private:

	/**
	 * Applies the dead zone setting to the incoming user impulse data
	 *
	 * @param	InUserImpulse	Input user impulse data
	 * @param	OutUserImpulse	[out] Output user impulse data with dead zone applied
	 * @param	bOutAnyImpulseData	[out] True if there was any user impulse this frame
	 */
	void ApplyImpulseDeadZone( const FCameraControllerUserImpulseData& InUserImpulse,
							   FCameraControllerUserImpulseData& OutUserImpulse,
							   bool& bOutAnyImpulseData );

	/**
	 * Updates the camera position.  Called every frame by UpdateSimulation.
	 *
	 * @param	UserImpulse				User impulse data for the current frame
	 * @param	DeltaTime				Time interval
	 * @param	MovementSpeedScale		Additional movement accel/speed scale
	 * @param	CameraEuler				Current camera rotation
	 * @param	InOutCameraPosition		[in, out] Camera position
	 */
	void UpdatePosition( const FCameraControllerUserImpulseData& UserImpulse, const float DeltaTime, const float MovementSpeedScale, const FVector& CameraEuler, FVector& InOutCameraPosition );


	/**
	 * Update the field of view.  Called every frame by UpdateSimulation.
	 *
	 * @param	UserImpulse		User impulse data for this frame
	 * @param	DeltaTime		Time interval
	 * @param	InOutCameraFOV	[in, out] Camera field of view
	 */
	void UpdateFOV( const FCameraControllerUserImpulseData& UserImpulse, const float DeltaTime, float& InOutCameraFOV );


	/**
	 * Applies FOV recoil (if appropriate)
	 *
	 * @param	DeltaTime					Time interval
	 * @param	bAllowRecoilIfNoImpulse		Whether recoil should be allowed if there wasn't any user impulse
	 * @param	bAnyUserImpulse				True if there was user impulse data this iteration
	 * @param	InOutCameraFOV				[in, out] Camera field of view
	 */
	void ApplyRecoil( const float DeltaTime, const bool bAllowRecoilIfNoImpulse, bool bAnyUserImpulse, float& InOutCameraFOV );


	/**
	 * Updates the camera rotation.  Called every frame by UpdateSimulation.
	 *
	 * @param	UserImpulse			User impulse data for this frame
	 * @param	DeltaTime			Time interval
	 * @param	InOutCameraEuler	[in, out] Camera rotation
	 */
	void UpdateRotation( const FCameraControllerUserImpulseData& UserImpulse, const float DeltaTime, FVector &InOutCameraEuler );



private:

	/** Configuration */
	FCameraControllerConfig Config;

	/** World space movement velocity */
	FVector MovementVelocity;

	/** FOV velocity in degrees per second */
	float FOVVelocity;

	/** Rotation velocity euler (yaw, pitch and roll) in degrees per second */
	FVector RotationVelocityEuler;

	/** Cached FOV angle, for recoiling back to the original FOV */
	float OriginalFOVForRecoil;

};
