// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Change.h"
#include "Engine/Engine.h"
#include "Templates/PointerIsConvertibleFromTo.h"
#include "Misc/ITransaction.h"

#if WITH_EDITOR	

namespace UE { 

/** Helper structure used for inserting FChange based undo/redo operations inside of the GUndo (ITransaction) instance.
	The operations are targeted a specific UObject, which is used as the 'transactional' object. The transactions themselves 
	are initiated using the ITransactor API using GEngine as the instance.
*/
struct FChangeTransactor
{
	FChangeTransactor() : TransactionObject(nullptr), PendingTransactionIndex(INDEX_NONE), TransactedChanges(0) {}
	FChangeTransactor(UObject* InTransactionObject) : TransactionObject(InTransactionObject), PendingTransactionIndex(INDEX_NONE), TransactedChanges(0) {}
	~FChangeTransactor() = default;
	
	/**
	* Set the UObject instance for which to transact FChange's
	*
	* @param	InTransactionObject		UObject instance to target
	*/
	void SetTransactionObject(UObject* InTransactionObject)
	{
		checkf(!IsTransactionPending(), TEXT("A transaction is still pending"));
		TransactionObject = InTransactionObject;
	}
	
	/**
	* Opens a new transaction with the provided description
	*
	* @param	TransactionDescription		Description of the transaction taking place
	*/
	void OpenTransaction(const FText& TransactionDescription)
	{
		CheckTransactionObject();
		checkf(!IsTransactionPending(), TEXT("A transaction is already pending"));
		
		if (CanTransactChanges())
		{
			CompoundChangeData = MakeUnique<FCompoundChangeInput>();
			PendingTransactionIndex = GEngine->BeginTransaction(TEXT("FChangeTransactor"), TransactionDescription, TransactionObject.Get());
		}
	}

	/**
	* Closes the currently pending transaction, inserting a FCompoundChange object, containing any FChange's transacted during
	* the transaction, into GUndo alongside of the currently targeted UObject instance 
	*/
	void CloseTransaction()
	{
		CheckTransactionObject();

		checkf(IsTransactionPending(), TEXT("No transaction was previously opened"));
				
		if (CompoundChangeData->Subchanges.Num() > 0)
		{
			if (GUndo)
			{
				TUniquePtr<FCompoundChange> CompoundAction = MakeUnique<FCompoundChange>(MoveTemp(*CompoundChangeData));
				check(GUndo != nullptr);
				GUndo->StoreUndo(TransactionObject.Get(), MoveTemp(CompoundAction));
			}
		}
	
		if (PendingTransactionIndex != -1)
		{
			GEngine->EndTransaction();
		}

		PendingTransactionIndex = -1;
		CompoundChangeData.Reset();
		TransactedChanges = 0;
	}

	/*
	* @return	Whether or not a transaction is currently pending
	*/
	bool IsTransactionPending()
	{
		return PendingTransactionIndex >= 0 && CompoundChangeData.IsValid();
	}

	/**
	* Inserts a FChange instance into the compound change data 
	*
	* @param	Args	Template arguments for T's constructor
	*/
	template<typename T, typename... TArgs>
	void AddTransactionChange(TArgs&&... Args)
	{
		static_assert(TPointerIsConvertibleFromTo</*typename*/ T, FChange>::Value, "Invalid type T, not derived from FChange");
		const bool bSwapBasedChange = TPointerIsConvertibleFromTo</*typename*/ T, FSwapChange>::Value;

		CheckTransactionObject();

		if (CanTransactChanges())
		{
			checkf(IsTransactionPending(), TEXT("No transaction was previously opened"));

			TUniquePtr<T> Change = MakeUnique<T>(Forward<TArgs>(Args)...);
		
			if (bSwapBasedChange && CompoundChangeData.IsValid())
			{
				CompoundChangeData->Subchanges.Add(MoveTemp(Change));
			}
			else if (GUndo)
			{
				checkf(CompoundChangeData->Subchanges.Num() == 0, TEXT("Cannot mix FSwapChange and FCommandChange based transactions, the Swap-based changes will be compounded - causing the ordering to be invalid"));
				GUndo->StoreUndo(TransactionObject.Get(), MoveTemp(Change));
				++TransactedChanges;
			}
		}
	}
	
	static bool CanTransactChanges()
	{
		return GEngine && GEngine->CanTransact() && !GIsTransacting;
	};
private:	
	void CheckTransactionObject()
	{
		checkf(TransactionObject.IsValid(), TEXT("No valid transaction object"));
	}

private:
	/** Weak reference to the target object */
	TWeakObjectPtr<UObject> TransactionObject;
	
	/** Pending transaction index and compound change object */
	int32 PendingTransactionIndex;
	TUniquePtr<FCompoundChangeInput> CompoundChangeData;

	/** Number of non-FSwapChange based changes which have been transacted */
	int32 TransactedChanges;
};

struct FScopedCompoundTransaction
{
	FScopedCompoundTransaction(FChangeTransactor& InTransactor, const FText& InDescription) : Transactor(InTransactor), bCreated(false)
	{
		if (FChangeTransactor::CanTransactChanges() && !Transactor.IsTransactionPending())
		{
			Transactor.OpenTransaction(InDescription);
			bCreated = true;
		}
	}

	~FScopedCompoundTransaction()
	{
		if (bCreated)
		{
			Transactor.CloseTransaction();
		}
	}

	FChangeTransactor& Transactor;
	bool bCreated;
};

}
#endif // WITH_EDITOR
