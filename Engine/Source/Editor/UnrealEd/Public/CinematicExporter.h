// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"

class ABrush;
class ACameraActor;
class ALight;
class UModel;
class UStaticMeshComponent;
class USkeletalMeshComponent;

namespace fbxsdk
{
	class FbxNode;
}

/** Adapter interface which allows finding the corresponding actor node name to act on sequencer. */
class INodeNameAdapter
{
public:
	virtual ~INodeNameAdapter() = default;
	virtual FString GetActorNodeName(const AActor* InActor) { return InActor->GetName(); }
	virtual void AddFbxNode(UObject* InObject, fbxsdk::FbxNode* InFbxNode) {}
	virtual fbxsdk::FbxNode* GetFbxNode(UObject* InObject) { return nullptr; }
};

/**
 * Base cinematic exporter class.
 * Except for CImporter, consider the other classes as private.
 */
class FCinematicExporter
{
public:
	virtual ~FCinematicExporter() = default;

	/**
	* Load the export option from the last save state and show the dialog if bShowOptionDialog is true.
	* FullPath is the export file path we display in the dialog.
	* If the user cancels the dialog, the OutOperationCanceled will be true.
	* bOutExportAll will be true if the user wants to use the same option for all other assets they want to export.
	*
	* The function is saving the dialog state in a user ini file and reload it from there. It is not changing the CDO.
	*/
	virtual void FillExportOptions(bool bBatchMode, bool bShowOptionDialog, const FString& FullPath, bool& OutOperationCanceled, bool& bOutExportAll) = 0;

	/**
	 * Creates and readies an empty document for export.
	 */
	virtual void CreateDocument() = 0;
	
	void SetTransformBaking(bool bBakeTransforms)
	{
		bBakeKeys = bBakeTransforms;
	}

	void SetKeepHierarchy(bool bInKeepHierarchy)
	{
		bKeepHierarchy = bInKeepHierarchy;
	}

	/**
	 * Exports the basic scene information to a file.
	 */
	virtual void ExportLevelMesh( ULevel* Level, bool bSelectedOnly, INodeNameAdapter& NodeNameAdapter, bool bSaveAnimSeq = true) = 0;

	/**
	 * Exports the light-specific information for a light actor.
	 */
	virtual void ExportLight( ALight* Actor, INodeNameAdapter& NodeNameAdapter ) = 0;

	/**
	 * Exports the camera-specific information for a camera actor.
	 */
	virtual void ExportCamera( ACameraActor* Actor, bool bExportComponents, INodeNameAdapter& NodeNameAdapter ) = 0;

	/**
	 * Exports the mesh and the actor information for a brush actor.
	 */
	virtual void ExportBrush(ABrush* Actor, UModel* Model, bool bConvertToStaticMesh, INodeNameAdapter& NodeNameAdapter ) = 0;

	/**
	 * Exports the mesh and the actor information for a static mesh actor.
	 */
	virtual void ExportStaticMesh( AActor* Actor, UStaticMeshComponent* StaticMeshComponent, INodeNameAdapter& NodeNameAdapter ) = 0;

	/**
	 * Writes the file to disk and releases it.
	 */
	virtual void WriteToFile(const TCHAR* Filename) = 0;

	/**
	 * Closes the file, releasing its memory.
	 */
	virtual void CloseDocument() = 0;
		
protected:

	/** When true, a key will exported per frame at the set frames-per-second (FPS). */
	bool bBakeKeys;
	/** When true, we'll export with hierarchical relation of attachment with relative transform */
	bool bKeepHierarchy;
};
