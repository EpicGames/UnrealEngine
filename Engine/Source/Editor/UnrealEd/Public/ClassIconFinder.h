// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Styling/SlateIconFinder.h"
#include "Textures/SlateIcon.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtrTemplates.h"

class AActor;
class ISlateStyle;
class UBlueprint;
class UClass;
struct FAssetData;
struct FSlateBrush;

class FClassIconFinder
{
public:

	/** Find the best fitting small icon to use for the supplied actor array */
	UNREALED_API static const FSlateBrush* FindIconForActors(const TArray< TWeakObjectPtr<AActor> >& InActors, UClass*& CommonBaseClass);

	/** Find the small icon to use for the supplied actor */
	UNREALED_API static const FSlateBrush* FindIconForActor( const TWeakObjectPtr<const AActor>& InActor );
	
	/** Find the small icon to use for the supplied actor */
	UNREALED_API static FSlateIcon FindSlateIconForActor( const TWeakObjectPtr<const AActor>& InActor );

	/** Utility function to convert a Blueprint into the most suitable class possible for use by the icon finder */
	UNREALED_API static const UClass* GetIconClassForBlueprint(const UBlueprint* InBlueprint);

	/** Utility function to convert an asset into the most suitable class possible for use by the icon finder */
	UNREALED_API static const UClass* GetIconClassForAssetData(const FAssetData& InAssetData, bool* bOutIsClassType = nullptr);

	/** Find the large thumbnail name to use for the supplied class */
	static const FSlateBrush* FindThumbnailForClass(const UClass* InClass, const FName& InDefaultName = FName() )
	{
		return FSlateIconFinder::FindCustomIconBrushForClass(InClass, TEXT("ClassThumbnail"), InDefaultName);
	}

private:
	UNREALED_API static const FSlateBrush* LookupBrush(FName IconName);
};
