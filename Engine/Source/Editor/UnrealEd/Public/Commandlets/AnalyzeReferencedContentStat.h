// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SceneTypes.h"

class USoundWave;
class USoundCue;
class AStaticMeshActor;
class UAnimSequence;
class UParticleSystem;
class UTexture;
class UModel;
class USkeletalMesh;
class UStaticMesh;
enum AnimationCompressionFormat : int;

class FAnalyzeReferencedContentStat
{
public:
	//@todo. If you add new object types, make sure to update this enumeration
	//		 as well as the optional command line.
	enum EIgnoreObjectFlags
	{
		IGNORE_StaticMesh				= 0x00000001,
		IGNORE_StaticMeshComponent		= 0x00000002,
		IGNORE_StaticMeshActor			= 0x00000004,
		IGNORE_Texture					= 0x00000008,
		IGNORE_Particle					= 0x00000020,
		IGNORE_Anim						= 0x00000040, // This includes all animsets/animsequences
		IGNORE_LightingOptimization		= 0x00000080,
		IGNORE_SoundCue					= 0x00000100,
		IGNORE_Brush					= 0x00000200,
		IGNORE_Level					= 0x00000400,
		IGNORE_SkeletalMesh				= 0x00000800,
		IGNORE_SkeletalMeshComponent	= 0x00001000, 
		IGNORE_Primitive				= 0x00002000,
		IGNORE_SoundWave			= 0x00004000,
	};

	int32 IgnoreObjects;

	// this holds all of the common data for our structs
	typedef TMap<FString,uint32> PerLevelDataMap;
	struct FAssetStatsBase
	{
		/** Mapping from LevelName to the number of instances of this type in that level */
		PerLevelDataMap LevelNameToInstanceCount;

		/** Maps this static mesh was used in.								*/
		TArray<FString> MapsUsedIn;

		/** @return true if this asset type should be logged */
		bool ShouldLogStat() const
		{
			return true;
		}

		/**
		* This function fills up MapsUsedIn and LevelNameToInstanceCount if bAddPerLevelDataMap is true. 
		*
		* @param	LevelPackage	Level Package this object belongs to
		* @param	bAddPerLevelDataMap	Set this to be true if you'd like to collect this stat per level (in the Level folder)
		* 
		*/
		void AddLevelInfo( UPackage* LevelPackage, bool bAddPerLevelDataMap = false );
	};

	/**
	* Encapsulates gathered stats for a particular UStaticMesh object.
	*/
	struct FStaticMeshStats : public FAssetStatsBase
	{
		/** Constructor, initializing all members. */
		FStaticMeshStats( UStaticMesh* StaticMesh );

		/**
		* Stringifies gathered stats in CSV format.
		*
		* @return comma separated list of stats
		*/
		FString ToCSV() const;
		/** This takes a LevelName and then looks for the number of Instances of this StatMesh used within that level **/
		FString ToCSV( const FString& LevelName ) const;
		/** @return true if this asset type should be logged */
		bool ShouldLogStat() const
		{
			return true;
		}

		/**
		* Returns a header row for CSV
		*
		* @return comma separated header row
		*/
		static FString GetCSVHeaderRow();

		/**
		*	This is for summary
		*	@return command separated summary CSV header row
		*/
		static FString GetSummaryCSVHeaderRow();

		/**
		*	This is for summary
		*	@return command separated summary data row
		*/
		static FString ToSummaryCSV(const FString& LevelName, const TMap<FString,FStaticMeshStats>& StatsData);

		/** Resource type.																*/
		FString ResourceType;
		/** Resource name.																*/
		FString ResourceName;
		/** Number of static mesh instances overall.									*/
		int32	NumInstances;
		/** Triangle count of mesh.														*/
		int32	NumTriangles;
		/** Section count of mesh.														*/
		int32 NumSections;
		/** Number of convex hulls in the collision geometry of mesh.					*/
		int32 NumConvexPrimitives;
		/** Does this static mesh use simple collision */
		int32 bUsesSimpleRigidBodyCollision;
		/** Number of sections that have collision enabled                              */
		int32 NumElementsWithCollision;
		/** Whether resource is referenced by script.									*/
		bool bIsReferencedByScript;
		/** Whether resource is referenced by particle system                           */
		bool bIsReferencedByParticles;
		/** Resource size of static mesh.												*/
		int32	ResourceSize;
		/** Is this mesh scaled non-uniformly in a level								*/
		bool bIsMeshNonUniformlyScaled;
		/** Does this mesh have box collision that should be converted					*/
		bool bShouldConvertBoxColl;
		/** Array of different scales that this mesh is used at							*/
		TArray<FVector> UsedAtScales;
	};

	/**
	* Encapsulates gathered stats for a particular USkeletalMesh object.
	*/
	struct FSkeletalMeshStats : public FAssetStatsBase
	{
		/** Constructor, initializing all members. */
		FSkeletalMeshStats( USkeletalMesh* SkeletalMesh );

		/**
		* Stringifies gathered stats in CSV format.
		*
		* @return comma separated list of stats
		*/
		FString ToCSV() const;
		/** This takes a LevelName and then looks for the number of Instances of this StatMesh used within that level **/
		FString ToCSV( const FString& LevelName ) const;
		/** @return true if this asset type should be logged */
		bool ShouldLogStat() const
		{
			return true;
		}

		/**
		* Returns a header row for CSV
		*
		* @return comma separated header row
		*/
		static FString GetCSVHeaderRow();

		/**
		*	This is for summary
		*	@return command separated summary CSV header row
		*/
		static FString GetSummaryCSVHeaderRow();

		/**
		*	This is for summary
		*	@return command separated summary data row
		*/
		static FString ToSummaryCSV(const FString& LevelName, const TMap<FString,FSkeletalMeshStats>& StatsData);

		/** Resource type.																*/
		FString ResourceType;
		/** Resource name.																*/
		FString ResourceName;
		/** Number of Skeletal mesh instances overall.									*/
		int32	NumInstances;
		/** Triangle count of mesh.														*/
		int32	NumTriangles;
		/** Vertex count of mesh.														*/
		int32	NumVertices;
		/** Vertex buffer size of Skeletal mesh.												*/
		int32	VertexMemorySize;
		/** Index buffer size of Skeletal mesh.												*/
		int32	IndexMemorySize;
		/** Rigid vertex count of mesh.													*/
		int32	NumRigidVertices;
		/** Soft vertex count of mesh.													*/
		int32	NumSoftVertices;
		/** Section count of mesh.														*/
		int32 NumSections;
		/** Chunk count of mesh.														*/
		int32 NumChunks;
		/** Max bone influences of mesh.												*/
		int32 MaxBoneInfluences;
		/** Active bone index count of mesh.											*/
		int32 NumActiveBoneIndices;
		/** Required bone count of mesh.												*/
		int32 NumRequiredBones;
		/** Number of materials applied to the mesh.									*/
		int32 NumMaterials;
		/** Whether resource is referenced by script.									*/
		bool bIsReferencedByScript;
		/** Whether resource is referenced by particle system                           */
		bool bIsReferencedByParticles;
		/** Resource size of Skeletal mesh.												*/
		int32	ResourceSize;
	};

	/**
	* Encapsulates gathered stats for a particular UModelComponent, UTerrainComponent object.
	*/
	struct FPrimitiveStats : public FAssetStatsBase
	{
		/** Constructor, initializing all members. */
		FPrimitiveStats( UModel* Model );
		FPrimitiveStats( USkeletalMesh * SkeletalMesh, FAnalyzeReferencedContentStat::FSkeletalMeshStats * SkeletalMeshStats );
		FPrimitiveStats( UStaticMesh* SkeletalMesh, FAnalyzeReferencedContentStat::FStaticMeshStats * StaticMeshStats );

		/**
		* Stringifies gathered stats in CSV format.
		*
		* @return comma separated list of stats
		*/
		FString ToCSV() const;
		/** This takes a LevelName and then looks for the number of Instances of this StatMesh used within that level **/
		FString ToCSV( const FString& LevelName ) const;
		/** @return true if this asset type should be logged */
		bool ShouldLogStat() const
		{
			return true;
		}

		/**
		* Returns a header row for CSV
		*
		* @return comma separated header row
		*/
		static FString GetCSVHeaderRow();

		/**
		*	This is for summary
		*	@return command separated summary CSV header row
		*/
		static FString GetSummaryCSVHeaderRow();

		/**
		*	This is for summary
		*	@return command separated summary data row
		*/
		static FString ToSummaryCSV(const FString& LevelName, const TMap<FString,FPrimitiveStats>& StatsData);

		/** Resource type.																*/
		FString ResourceType;
		/** Resource name.																*/
		FString ResourceName;
		/** Number of static mesh instances overall.									*/
		int32	NumInstances;
		/** Triangle count of mesh.														*/
		int32	NumTriangles;
		/** Section count of mesh.														*/
		int32 NumSections;
		/** Resource size of static mesh.												*/
		int32	ResourceSize;
	};

	/**
	* Encapsulates gathered stats for a particular UTexture object.
	*/
	struct FTextureStats : public FAssetStatsBase
	{
		/** Constructor, initializing all members */
		FTextureStats( UTexture* Texture );

		/**
		* Stringifies gathered stats in CSV format.
		*
		* @return comma separated list of stats
		*/
		FString ToCSV() const;
		/** @return true if this asset type should be logged */
		bool ShouldLogStat() const
		{
			return true;
		}

		/**
		* Returns a header row for CSV
		*
		* @return comma separated header row
		*/
		static FString GetCSVHeaderRow();

		/**
		*	This is for summary
		*	@return command separated summary CSV header row
		*/
		static FString GetSummaryCSVHeaderRow();

		/**
		*	This is for summary
		*	@return command separated summary data row
		*/
		static FString ToSummaryCSV(const FString& LevelName, const TMap<FString,FTextureStats>& StatsData);

		/** Resource type.																*/
		FString ResourceType;
		/** Resource name.																*/
		FString ResourceName;
		/** Map of materials this textures is being used by.							*/
		TMap<FString,int32> MaterialsUsedBy;
		/** Whether resource is referenced by script.									*/
		bool bIsReferencedByScript;
		/** Resource size of texture.													*/
		int32	ResourceSize;
		/** LOD bias.																	*/
		int32	LODBias;
		/** LOD group.																	*/
		int32 LODGroup;
		/** Texture pixel format.														*/
		FString Format;
	};

	/**
	* Encapsulates gathered stats for a particular UParticleSystem object
	*/
	struct FParticleStats : public FAssetStatsBase
	{
		/** Constructor, initializing all members */
		FParticleStats( UParticleSystem* ParticleSystem );

		/**
		* Stringifies gathered stats in CSV format.
		*
		* @return comma separated list of stats
		*/
		FString ToCSV() const;
		/** @return true if this asset type should be logged */
		bool ShouldLogStat() const;

		/**
		* Returns a header row for CSV
		*
		* @return comma separated header row
		*/
		static FString GetCSVHeaderRow();

		/**
		*	This is for summary
		*	@return command separated summary CSV header row
		*/
		static FString GetSummaryCSVHeaderRow();

		/**
		*	This is for summary
		*	@return command separated summary data row
		*/
		static FString ToSummaryCSV(const FString& LevelName, const TMap<FString,FParticleStats>& StatsData);

		/** Resource type.																*/
		FString ResourceType;
		/** Resource name.																*/
		FString ResourceName;
		/** Whether resource is referenced by script.									*/
		bool bIsReferencedByScript;	
		/** Number of emitters in this system.											*/
		int32 NumEmitters;
		/** Combined number of modules in all emitters used.							*/
		int32 NumModules;
		/** Combined number of peak particles in system.								*/
		int32 NumPeakActiveParticles;
		/** Combined number of collision modules across emitters                        */
		int32 NumEmittersUsingCollision;
		/** Combined number of emitters that have active physics                        */
		int32 NumEmittersUsingPhysics;
		/** Maximum number of particles drawn per frame                                 */
		int32 MaxNumDrawnPerFrame;
		/** Ratio of particles simulated to particles drawn                             */
		float PeakActiveToMaxDrawnRatio;
		/** This is the size in bytes that this Particle System will use                */
		int32 NumBytesUsed;
		/** If any modules have mesh emitters that have DoCollision == true wich is more than likely bad and perf costing */
		bool bMeshEmitterHasDoCollisions;
		/** If any modules have mesh emitters that have DoCollision == true wich is more than likely bad and perf costing */
		bool bMeshEmitterHasCastShadows;
		/** If the particle system has warm up time greater than N seconds**/
		float WarmUpTime;
	};

	/**
	* Encapsulates gathered textures-->particle systems information for all particle systems
	*/
	struct FTextureToParticleSystemStats : public FAssetStatsBase
	{
		FTextureToParticleSystemStats(UTexture* InTexture);
		void AddParticleSystem(UParticleSystem* InParticleSystem);

		/**
		* Stringifies gathered stats in CSV format.
		*
		* @return comma separated list of stats
		*/
		FString ToCSV() const;
		/** @return true if this asset type should be logged */
		bool ShouldLogStat() const
		{
			return true;
		}

		/**
		* Returns a header row for CSV
		*
		* @return comma separated header row
		*/
		static FString GetCSVHeaderRow();

		const int32 GetParticleSystemsContainedInCount() const
		{
			return ParticleSystemsContainedIn.Num();
		}

		FString GetParticleSystemContainedIn(int32 Index) const
		{
			if ((Index >= 0) && (Index < ParticleSystemsContainedIn.Num()))
			{
				return ParticleSystemsContainedIn[Index];
			}

			return TEXT("*** INVALID ***");
		}

	protected:
		/** Texture name.										*/
		FString TextureName;
		/** Texture size.										*/
		FString TextureSize;
		/** Texture pixel format.								*/
		FString Format;
		TArray<FString> ParticleSystemsContainedIn;
	};

	struct FAnimSequenceStats : public FAssetStatsBase
	{
		/** Constructor, initializing all members */
		FAnimSequenceStats( UAnimSequence* Sequence );

		/**
		* Stringifies gathered stats in CSV format.
		*
		* @return comma separated list of stats
		*/
		FString ToCSV() const;

		/** This takes a LevelName and then looks for the number of Instances of this AnimStat used within that level **/
		FString ToCSV( const FString& LevelName ) const;

		/** @return true if this asset type should be logged */
		bool ShouldLogStat() const
		{
			return true;
		}

		/**
		* Returns a header row for CSV
		*
		* @return comma separated header row
		*/
		static FString GetCSVHeaderRow();

		/**
		*	This is for summary
		*	@return command separated summary CSV header row
		*/
		static FString GetSummaryCSVHeaderRow();

		/**
		*	This is for summary
		*	@return command separated summary data row
		*/
		static FString ToSummaryCSV(const FString& LevelName, const TMap<FString,FAnimSequenceStats>& StatsData);

		/** Resource type.																*/
		FString ResourceType;
		/** Resource name.																*/
		FString ResourceName;
		/** Animset name.																*/
		FString AnimSetName;
		/** Animation Tag.																*/
		FString AnimTag;
		/** Whether resource is referenced by script.									*/
		bool bIsReferencedByScript;
		/** Whether resource is forced to be uncompressed by human action               */
		bool bMarkedAsDoNotOverrideCompression;
		/** Type of compression used on this animation.									*/
		AnimationCompressionFormat TranslationFormat;
		/** Type of compression used on this animation.									*/
		AnimationCompressionFormat RotationFormat;
		/** Type of compression used on this animation.									*/
		AnimationCompressionFormat ScaleFormat;
		/** Name of compression algo class used. */
		FString CompressionScheme;
		/** Size in bytes of this animation. */
		int32	AnimationResourceSize;
		/** Percentage(0-100%) of compress ratio*/
		int32 CompressionRatio;
		/** Total Tracks in this animation. */
		int32	TotalTracks;
		/** Total Tracks with no animated translation. */
		int32 NumTransTracksWithOneKey;
		/** Total Tracks with no animated rotation. */
		int32 NumRotTracksWithOneKey;
		/** Total Tracks with no animated scale. */
		int32 NumScaleTracksWithOneKey;
		/** Size in bytes of this animation's track table. */
		int32	TrackTableSize;
		/** total translation keys. */
		int32 TotalNumTransKeys;
		/** total rotation keys. */
		int32 TotalNumRotKeys;
		/** total scale keys. */
		int32 TotalNumScaleKeys;
		/** Average size of a single translation key, in bytes. */
		float TranslationKeySize;
		/** Average size of a single rotation key, in bytes. */
		float RotationKeySize;
		/** Average size of a single scale key, in bytes. */
		float ScaleKeySize;
		/** Size of the overhead that isn't directly key data (includes data that is required to reconstruct keys, so it's not 'waste'), in bytes */
		int32 OverheadSize;
		/** Total Frames in this animation. */
		int32	TotalFrames;

		enum EAnimReferenceType
		{
			ART_SkeletalMeshComponent, // Regular SkeletalMeshComponent - mostly from script
			ART_Crowd, // From Crowd spawner, expected to be none or very small
		};

		/** Reference Type **/
		EAnimReferenceType ReferenceType;

	};

	struct FLightingOptimizationStats : public FAssetStatsBase
	{
		/** Constructor, initializing all members */
		FLightingOptimizationStats( AStaticMeshActor* StaticMeshActor );

		/**
		* Stringifies gathered stats in CSV format.
		*
		* @return comma separated list of stats
		*/
		FString ToCSV() const;
		/** @return true if this asset type should be logged */
		bool ShouldLogStat() const
		{
			return true;
		}

		/**
		* Returns a header row for CSV
		*
		* @return comma separated header row
		*/
		static FString GetCSVHeaderRow();

		/** Assuming DXT1 lightmaps...
		*   4 bits/pixel * width * height = Highest MIP Level * 1.333 MIP Factor for approx usage for a full mip chain
		*   Either 1 or 3 textures if we're doing simple or directional (3-axis) lightmap
		*   Most lightmaps require a second UV channel which is probably an extra 4 bytes (2 floats compressed to SHORT) 
		*/
		static int32 CalculateLightmapLightingBytesUsed(int32 Width, int32 Height, int32 NumVertices, int32 UVChannelIndex);

		/** 
		*	For a given list of parameters, compute a full spread of potential savings values using vertex light, or 256, 128, 64, 32 pixel square light maps
		*  @param LMType - Current type of lighting being used
		*  @param NumVertices - Number of vertices in the given mesh
		*  @param Width - Width of current lightmap
		*  @param Height - Height of current lightmap
		*  @param TexCoordIndex - channel index of the uvs currently used for lightmaps
		*  @param LOI - A struct to be filled in by the function with the potential savings
		*/
		static void CalculateLightingOptimizationInfo(ELightMapInteractionType LMType, int32 NumVertices, int32 Width, int32 Height, int32 TexCoordIndex, FLightingOptimizationStats& LOStats);

		static const int32 NumLightmapTextureSizes = 4;
		static const int32 LightMapSizes[NumLightmapTextureSizes];

		/** Name of the Level this StaticMeshActor is on */
		FString						LevelName;
		/** Name of the StaticMeshActor this optimization is for */
		FString						ActorName;
		/** Name of the StaticMesh belonging to the above StaticMeshActor */
		FString						SMName;
		/** Current type of lighting scheme used */
		ELightMapInteractionType    IsType;        
		/** Texture size of the current lighting scheme, if texture, 0 otherwise */
		int32                         TextureSize;   
		/** Amount of memory used by the current lighting scheme */
		int32							CurrentBytesUsed;
		/** Amount of memory savings for each lighting scheme (256,128,64,32 pixel lightmaps + vertex lighting) */
		int32							BytesSaved[NumLightmapTextureSizes + 1];
	};

	/**
	* Encapsulates gathered stats for a particular USoundCue object.
	*/
	struct FSoundCueStats : public FAssetStatsBase
	{
		/** Constructor, initializing all members. */
		FSoundCueStats( USoundCue* SoundCue );

		/**
		* Stringifies gathered stats in CSV format.
		*
		* @return comma separated list of stats
		*/
		FString ToCSV() const;
		/** This takes a LevelName and then looks for the number of Instances of this StatMesh used within that level **/
		FString ToCSV( const FString& LevelName ) const;
		/** @return true if this asset type should be logged */
		bool ShouldLogStat() const
		{
			return true;
		}

		/**
		* Returns a header row for CSV
		*
		* @return comma separated header row
		*/
		static FString GetCSVHeaderRow();

		/**
		*	This is for summary
		*	@return command separated summary CSV header row
		*/
		static FString GetSummaryCSVHeaderRow();

		/**
		*	This is for summary
		*	@return command separated summary data row
		*/
		static FString ToSummaryCSV(const FString& LevelName, const TMap<FString,FSoundCueStats>& StatsData);

		/** Resource type.																*/
		FString ResourceType;
		/** Resource name.																*/
		FString ResourceName;
		/** Whether resource is referenced by script.									*/
		bool bIsReferencedByScript;
		/** Resource size of static mesh.												*/
		int32	ResourceSize;
	};


	/**
	* Encapsulates gathered stats for a particular USoundCue object.
	*/
	struct FSoundWaveStats : public FAssetStatsBase
	{
		/** Constructor, initializing all members. */
		FSoundWaveStats( USoundWave* SoundWave );

		/**
		* Stringifies gathered stats in CSV format.
		*
		* @return comma separated list of stats
		*/
		FString ToCSV() const;
		/** This takes a LevelName and then looks for the number of Instances of this StatMesh used within that level **/
		FString ToCSV( const FString& LevelName ) const;
		/** @return true if this asset type should be logged */
		bool ShouldLogStat() const
		{
			return true;
		}

		/**
		* Returns a header row for CSV
		*
		* @return comma separated header row
		*/
		static FString GetCSVHeaderRow();

		/**
		*	This is for summary
		*	@return command separated summary CSV header row
		*/
		static FString GetSummaryCSVHeaderRow();

		/**
		*	This is for summary
		*	@return command separated summary data row
		*/
		static FString ToSummaryCSV(const FString& LevelName, const TMap<FString,FSoundWaveStats>& StatsData);

		/** Resource type.																*/
		FString ResourceType;
		/** Resource name.																*/
		FString ResourceName;
		/** Whether resource is referenced by script.									*/
		bool bIsReferencedByScript;
		/** Resource size of static mesh.												*/
		int32	ResourceSize;
	};

	//@todo: this code is in dire need of refactoring

	/**
	* Retrieves/ creates texture stats associated with passed in texture.
	*
	* @warning: returns pointer into TMap, only valid till next time Set is called
	*
	* @param	Texture		Texture to retrieve/ create texture stats for
	* @return	pointer to texture stats associated with texture
	*/
	FTextureStats* GetTextureStats( UTexture* Texture );

	/**
	* Retrieves/ creates static mesh stats associated with passed in static mesh.
	*
	* @warning: returns pointer into TMap, only valid till next time Set is called
	*
	* @param	StaticMesh	Static mesh to retrieve/ create static mesh stats for
	* @return	pointer to static mesh stats associated with static mesh
	*/
	FStaticMeshStats* GetStaticMeshStats( UStaticMesh* StaticMesh, UPackage* LevelPackage );

	/**
	* Retrieves/ creates static mesh stats associated with passed in primitive stats.
	*
	* @warning: returns pointer into TMap, only valid till next time Set is called
	*
	* @param	Object	Object to retrieve/ create primitive stats for
	* @return	pointer to primitive stats associated with object (ModelComponent, TerrainComponent)
	*/
	FPrimitiveStats* GetPrimitiveStats( UObject* Object, UPackage* LevelPackage );

	/**
	* Retrieves/ creates skeletal mesh stats associated with passed in skeletal mesh.
	*
	* @warning: returns pointer into TMap, only valid till next time Set is called
	*
	* @param	SkeletalMesh	Skeletal mesh to retrieve/ create skeletal mesh stats for
	* @return	pointer to skeletal mesh stats associated with skeletal mesh
	*/
	FSkeletalMeshStats* GetSkeletalMeshStats( USkeletalMesh* SkeletalMesh, UPackage* LevelPackage );

	/**
	* Retrieves/ creates particle stats associated with passed in particle system.
	*
	* @warning: returns pointer into TMap, only valid till next time Set is called
	*
	* @param	ParticleSystem	Particle system to retrieve/ create static mesh stats for
	* @return	pointer to particle system stats associated with static mesh
	*/
	FParticleStats* GetParticleStats( UParticleSystem* ParticleSystem );

	/**
	* Retrieves/creates texture in particle system stats associated with the passed in texture.
	*
	* @warning: returns pointer into TMap, only valid till next time Set is called
	*
	* @param	InTexture	The texture to retrieve/create stats for
	* @return	pointer to textureinparticlesystem stats
	*/
	FTextureToParticleSystemStats* GetTextureToParticleSystemStats(UTexture* InTexture);

	/**
	* Retrieves/ creates animation sequence stats associated with passed in animation sequence.
	*
	* @warning: returns pointer into TMap, only valid till next time Set is called
	*
	* @param	AnimSequence	Anim sequence to retrieve/ create anim sequence stats for
	* @return	pointer to particle system stats associated with anim sequence
	*/
	FAnimSequenceStats* GetAnimSequenceStats( UAnimSequence* AnimSequence );


	/**
	* Retrieves/ creates lighting optimization stats associated with passed in static mesh actor.
	*
	* @warning: returns pointer into TMap, only valid till next time Set is called
	*
	* @param	ActorComponent	Actor component to calculate potential light map savings stats for
	* @return	pointer to lighting optimization stats associated with this actor component
	*/
	FLightingOptimizationStats* GetLightingOptimizationStats( AStaticMeshActor* ActorComponent );

	/**
	* Retrieves/ creates sound cue stats associated with passed in sound cue.
	*
	* @warning: returns pointer into TMap, only valid till next time Set is called
	*
	* @param	SoundCue	Sound cue  to retrieve/ create sound cue  stats for
	* @return				pointer to sound cue  stats associated with sound cue  
	*/
	FSoundCueStats* GetSoundCueStats( USoundCue* SoundCue, UPackage* LevelPackage );

	/**
	* Retrieves/ creates sound cue stats associated with passed in sound cue.
	*
	* @warning: returns pointer into TMap, only valid till next time Set is called
	*
	* @param	SoundCue	Sound cue  to retrieve/ create sound cue  stats for
	* @return				pointer to sound cue  stats associated with sound cue  
	*/
	FSoundWaveStats* GetSoundWaveStats( USoundWave* SoundWave, UPackage* LevelPackage );

	/** Mapping from a fully qualified resource string (including type) to static mesh stats info.								*/
	TMap<FString,FStaticMeshStats> ResourceNameToStaticMeshStats;
	/** Mapping from a fully qualified resource string (including type) to other primitive stats info - excluding staticmeshes/skeletalmeshes.	*/
	TMap<FString,FPrimitiveStats> ResourceNameToPrimitiveStats;
	/** Mapping from a fully qualified resource string (including type) to skeletal mesh stats info.								*/
	TMap<FString,FSkeletalMeshStats> ResourceNameToSkeletalMeshStats;
	/** Mapping from a fully qualified resource string (including type) to texture stats info.									*/
	TMap<FString,FTextureStats> ResourceNameToTextureStats;
	/** Mapping from a fully qualified resource string (including type) to particle stats info.									*/
	TMap<FString,FParticleStats> ResourceNameToParticleStats;
	/** Mapping from a full qualified resource string (including type) to texutreToParticleSystem stats info					*/
	TMap<FString,FTextureToParticleSystemStats> ResourceNameToTextureToParticleSystemStats;
	/** Mapping from a fully qualified resource string (including type) to anim stats info.										*/
	TMap<FString,FAnimSequenceStats> ResourceNameToAnimStats;																
	/** Mapping from a fully qualified resource string (including type) to lighting optimization stats info.					*/
	TMap<FString,FLightingOptimizationStats> ResourceNameToLightingStats;
	/** Mapping from a fully qualified resource string (including type) to sound cue stats info.								*/
	TMap<FString,FSoundCueStats> ResourceNameToSoundCueStats;
	/** Mapping from a fully qualified resource string (including type) to sound cue stats info.								*/
	TMap<FString,FSoundWaveStats> ResourceNameToSoundWaveStats;

	void	SetIgnoreObjectFlag( int32 IgnoreObjectFlag ) { IgnoreObjects = IgnoreObjectFlag; }
	int32		GetIgnoreObjectFlag()	{ return IgnoreObjects;		}
	bool	InIgnoreObjectFlag( int32 IgnoreObjectFlag ) { return !!(IgnoreObjects & IgnoreObjectFlag); }

	void WriteOutAllAvailableStatData( const FString& CSVDirectory );

	/** This will write out the specified Stats to the AnalyzeReferencedContentCSVs dir **/
	template< typename STAT_TYPE >
	static void WriteOutCSVs( const TMap<FString,STAT_TYPE>& StatsData, const FString& CSVDirectory, const FString& StatsName );

	/** This will write out the summary Stats to the AnalyzeReferencedContentCSVs dir **/
	template< typename STAT_TYPE >
	static void WriteOutSummaryCSVs( const TMap<FString,STAT_TYPE>& StatsData, const FString& CSVDirectory, const FString& StatsName );

	template< typename STAT_TYPE >
	static void WriteOutCSVsPerLevel( const TMap<FString,STAT_TYPE>& StatsData, const FString& CSVDirectory, const FString& StatsName );

	template< typename STAT_TYPE >
	static int32 GetTotalCountPerLevel( const TMap<FString,STAT_TYPE>& StatsData, const FString& LevelName );

	void WriteOutSummary(const FString& CSVDirectory);

	// include only map list that has been loaded
	TArray<FString> MapFileList;
};
