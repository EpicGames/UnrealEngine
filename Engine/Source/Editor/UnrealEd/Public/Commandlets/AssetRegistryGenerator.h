// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryState.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "Cooker/CookArtifact.h"
#include "Cooker/MPCollector.h"
#include "Misc/AssetRegistryInterface.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UObjectHash.h"

class IAssetRegistry;
class ITargetPlatform;
class IChunkDataGenerator;
class UChunkDependencyInfo;
class UCookOnTheFlyServer;
struct FChunkDependencyTreeNode;
struct FSoftObjectPath;
namespace UE::Cook { class FAssetRegistryMPCollector; }
namespace UE::Cook { class FAssetRegistryPackageMessage; }
namespace UE::Cook { class FAssetRegistryReporterRemote; }
namespace UE::Cook { class FCookSandbox; }
namespace UE::Cook { class FCookWorkerClient; }
namespace UE::Cook { struct FPackageData; }

/**
 * Helper class for generating streaming install manifests
 */
class FAssetRegistryGenerator
{
public:
	/**
	 * Constructor
	 */
	FAssetRegistryGenerator(const ITargetPlatform* InPlatform);

	/**
	 * Destructor
	 */
	~FAssetRegistryGenerator();

	void Initialize();
	/**
	 * Called at the beginning of CookOnTheFly to bootstrap the targetplatform's AssetRegistry by copying it from the
	 * editor's AssetRegistry.
	 */
	void InitializeAssetRegistryForCookOnTheFly(const FAssetRegistrySerializationOptions& DevelopmentSaveOptions);

	const ITargetPlatform* GetTargetPlatform() const { return TargetPlatform; }

	/**
	 * Options when computing the differences between current and previous state.
	 */
	struct FComputeDifferenceOptions
	{
		/** if true, modified packages are recursed to X in X->Y->Z chains. Otherwise, only Y and Z are seen as modified */
		bool bRecurseModifications;
		/** if true, modified script / c++ packages are recursed, if false only asset references are recursed */
		bool bRecurseScriptModifications;
		/**
		 * If true, use the AllowList and DenyList for classes on a package's used classes to decide in
		 * legacyiterative cooks whether the package can be skipped if unmodified.
		 */
		bool bLegacyIterativeUseClassFilters;
	};

	/** Info about a GeneratorPackage (see ICookPackageSplitter) loaded from previous incremental cooks. */
	struct FGeneratorPackageInfo
	{
		TMap<FName, FAssetPackageData> Generated;
	};

	enum EDifference
	{
		// Cooked packages have files that can be loaded at runtime.
		IdenticalCooked,
		ModifiedCooked,
		RemovedCooked,

		// Uncooked packages either were skipped by the platform (including editor-only packages) or had an error during Save
		IdenticalUncooked,
		ModifiedUncooked,
		RemovedUncooked,

		// NeverCookPlaceholders are packages that were marked NeverCook, so we did not attempt to save them, but they
		// were also marked as necessary for dependency testing, so we included them in the AssetRegistry
		IdenticalNeverCookPlaceholder,
		ModifiedNeverCookPlaceholder,
		RemovedNeverCookPlaceholder,

		// Scripts have entries in the AssetRegistry for dependency testing, but are embedded in the binary and do not
		// have their own files
		IdenticalScript,
		ModifiedScript,
		RemovedScript,
	};
	/**
	 * Differences between the current and the previous state.
	 */
	struct FAssetRegistryDifference
	{
		/** Collection of all non-generated packages contained in the previous cook, and their difference category. */
		TMap<FName, EDifference> Packages;
		/**
		 * Collection of Generator packages and the packages they generated. The keys of the map are Generator
		 * packages, and these packages exist in this->Packages. But the values include a list of Generated packages,
		 * which are not present in Packages. Generated packages cannot be given a difference category until the
		 * Generator runs, or if the Generator is unmodified.
		 */
		TMap<FName, FGeneratorPackageInfo> GeneratorPackages;
	};

	/** Sets asset registry from a previous run that is used for incremental cooking. */
	void SetPreviousAssetRegistry(TUniquePtr<FAssetRegistryState>&& PreviousState);

	/**
	 * Computes differences between the current asset registry state and the provided previous state.
	 *
	 * @param Options options to use when computing the differences
	 * @param PreviousAssetPackageDataMap previously cooked asset package data
	 * @param OutDifference the differences between the current and the previous state
	 */
	void ComputePackageDifferences(const FComputeDifferenceOptions& Options, const FAssetRegistryState& PreviousState, 
		FAssetRegistryDifference& OutDifference);

	/** Reads Generator packages and other cooking data from the previous incrementalcook's DevelopmentAssetRegistry. */
	void ReportGeneratorPackages(const FAssetRegistryState& PreviousState,
		TMap<FName, FGeneratorPackageInfo>& OutGeneratorPackages, int32& OutNumNeverCookPlaceHolderPackages);

	/**
	 * Given the list of packages still in the oplog, remove any packages from the AssetRegistryState that are no longer
	 * in the oplog (they were not referenced in the current session and were pruned for being too old or invalid).
	 */
	void RemoveStalePackages(const TMap<FName, UE::Cook::Artifact::FOplogPackageData>& OplogPackages);

	/**
	 * Add or update data in the generated AssetRegistry for non-package data such as management references from
	 * PrimaryAssets and /Script PackageDatas.
	 */
	void AddGlobalsToCookedAssetRegistry(UCookOnTheFlyServer& COTFS);

	/**
	 * FinalizeChunkIDs 
	 * Create the list of packages to store in each Chunk; each Chunk corresponds to a pak file, or group of pak files if the chunk is split.
	 * Each package may be in multiple chunks. The selection of chunks for each package is based on
	 * ChunkIds explicitly assigned to the package in the editor (UPackage::GetChunkIds()) and
	 * on assignment rules defined by the AssetManager
	 *
	 * @param CookedPackages list of packages which were cooked
	 * @param DevelopmentOnlyPackages list of packages that were specifically not cooked, but to add to the development asset registry
	 * @param InSandboxFile sandbox to load/save data
	 * @param bGenerateStreamingInstallManifest should we build a streaming install manifest
	 *        If false, no manifest is written and all packages are implicitly assigned to chunk 0 by the automation tool.
	 *        If true, packages are assigned to chunks based on settings (possibly all in chunk 0 if settings are empty)
	 *        and a manifest of packagenames is written for each chunk
	 * @param StartupPackages The packages the cooker recorded as being loaded at startup.
	 *        These packages have special chunk rules.
	 */
	void FinalizeChunkIDs(const TSet<FName>& CookedPackages, const TSet<FName>& DevelopmentOnlyPackages,
		UE::Cook::FCookSandbox& InSandboxFile, bool bGenerateStreamingInstallManifest,
		const TSet<FName>& StartupPackages, const FAssetRegistrySerializationOptions& DevelopmentSaveOptions);

	/**
	 * Register a chunk data generator with this generator.
	 * @note Should be called prior to SaveManifests.
	 */
	void RegisterChunkDataGenerator(TSharedRef<IChunkDataGenerator> InChunkDataGenerator);

	/**
	* PreSave
	* Notify generator that we are about to save the registry and chunk manifests
	*/
	void PreSave(const TSet<FName>& InCookedPackages);

	/**
	* PostSave
	* Notify generator that we are finished saving registry
	*/
	void PostSave();

	/**
	 * ContainsMap
	 * Does this package contain a map file (determined by finding if this package contains a UWorld / ULevel object)
	 *
	 * @param PackageName long package name of the package we want to determine if contains a map 
	 * @return return if the package contains a UWorld / ULevel object (contains a map)
	 */
	bool ContainsMap(const FName& PackageName) const;

	/** 
	 * Returns editable version of the asset package state being generated 
	 */
	FAssetPackageData* GetAssetPackageData(const FName& PackageName);

	/**
	 * Deletes temporary manifest directories.
	 */
	bool CleanManifestDirectories(UE::Cook::FCookSandbox& InSandboxFile, UCookOnTheFlyServer& COTFS);

	/**
	 * Saves all generated manifests for each target platform.
	 * 
	 * @param InOverrideChunkSize the ChunkSize used during chunk division.
	 *        If greater than 0, this overrides the default chunksize derived from the platform.
	 * @param InManifestSubDir If non-null, the manifests are written into this subpath
	 *        of the usual location.
	 */
	bool SaveManifests(UE::Cook::FCookSandbox& InSandboxFile, UCookOnTheFlyServer& COTFS,
		int64 InOverrideChunkSize = 0, const TCHAR* InManifestSubDir = nullptr);

	/**
	* Saves generated asset registry data for each platform.
	*/
	bool SaveAssetRegistry(const FString& SandboxPath, bool bSerializeDevelopmentAssetRegistry,
		bool bForceNoFilterAssets, uint64& OutDevelopmentAssetRegistryHash, const TSet<FName>& CookedPackages,
		const FAssetRegistrySerializationOptions& DevelopmentSaveOptions);

	/** 
	 * Writes out CookerOpenOrder.log file 
	 */
	bool WriteCookerOpenOrder(UE::Cook::FCookSandbox& InSandboxFile);

	/**
	 * Follows an assets dependency chain to build up a list of package names in the same order as the runtime would attempt to load them
	 * 
	 * @param InPackageName - The name of the package containing the asset to (potentially) add to the file order
	 * @param OutFileOrder - Output array which collects the package names, maintaining order
	 * @param OutEncounteredArray - Temporary collection of package names we've seen. Similar to OutFileOrder but updated BEFORE following dependencies so as to avoid circular references
	 * @param InPackageNameSet - The source package name list. Used to distinguish between dependencies on other packages and internal objects
	 * @param InTopLevelAssets - Names of packages containing top level assets such as maps
	 */
	void AddAssetToFileOrderRecursive(const FName& InPackageName, TArray<FName>& OutFileOrder, TSet<FName>& OutEncounteredNames, const TSet<FName>& InPackageNameSet, const TSet<FName>& InTopLevelAssets);

	/**
	 * Get pakchunk file index from ChunkID
	 *
	 * @param ChunkID
	 * @return Index of target pakchunk file
	 */
	int32 GetPakchunkIndex(int32 ChunkId) const;

	/**
	 * Attempts to update the metadata for a package in an asset registry generator.
	 * This is only called for CookByTheBook.
	 *
	 * @param Package The package to update info on
	 * @param SavePackageResult The metadata to associate with the given package name
	 * @param bIncludeOnlyDiskAssets Include only disk assets or else also enumerate memory assets
	 */
	void UpdateAssetRegistryData(FName PackageName, const UPackage* Package,
		UE::Cook::ECookResult CookResult, FSavePackageResultStruct* SavePackageResult,
		TOptional<TArray<FAssetData>>&& AssetDatasFromSave,
		TOptional<FAssetPackageData>&& OverrideAssetPackageData,
		TOptional<TArray<FAssetDependency>>&& OverridePackageDependencies,
		UCookOnTheFlyServer& COTFS);
	void UpdateAssetRegistryData(UE::Cook::FMPCollectorServerMessageContext& Context,
		UE::Cook::FAssetRegistryPackageMessage&& Message, UCookOnTheFlyServer& COTFS);

	/**
	 * Check config to see whether chunk assignments use the AssetManager. If so, run the once-per-process construction
	 * of ManageReferences and store them in the global AssetRegistry.
	 */
	static void UpdateAssetManagerDatabase();

	/** Sets the generated pak chunk to be named the given OverrideName rather than the chunkId. This does not update the internal asset registry number only the generated pak chunk file name. */
	static UNREALED_API void AddChunkIdStringOverride(const int32 ChunkId, const FString& OverrideName);

	/** Removes the String Override for the given ChunkId. Returns true if an override was removed. */
	static UNREALED_API bool ClearChunkIdOverride(const int32 ChunkId);

private:
	class FGetShortestReferenceChain;

	/**
	 * Updates all asset package flags in the specified package
	 *
	 * @param PackageName The name of the package
	 * @param PackageFlags Package flags to set
	 * @return True if any assets exists in the package
	 */
	bool UpdateAssetPackageFlags(const FName& PackageName, const uint32 PackageFlags);

	static UE::Cook::FAssetRegistryPackageMessage RecordUpdateAssetRegistryData(FName PackageName,
		const ITargetPlatform* TargetPlatform, UE::Cook::ECookResult CookResult,
		FSavePackageResultStruct* SavePackageResult,
		TOptional<TArray<FAssetData>>&& AssetDatasFromSave,
		TOptional<FAssetPackageData>&& OverrideAssetPackageData,
		TOptional<TArray<FAssetDependency>>&& OverridePackageDependencies);
	void StoreUpdateAssetRegistryData(FName PackageName, UE::Cook::FAssetRegistryPackageMessage&& Message);
	/** Delete Assets we copied from the global AssetRegistry that no longer exist in the cook save of the package. */
	void RemovePackageAssetsThatDoNotExistInCookedPackage(FName PackageName,
		const TArray<FAssetData>& AssetDatasFromSave, bool bCookSucceeded);
	void SetOverridePackageDependencies(FName PackageName, TConstArrayView<FAssetDependency> OverridePackageDependencies);

	bool ComputePackageDifferences_IsPackageFileUnchanged(const FComputeDifferenceOptions& Options, FName PackageName,
		const FAssetPackageData& CurrentPackageData, const FAssetPackageData& PreviousPackageData);
	/**
	 * When copying AssetPackageData from the previous saved development assetregistry into the
	 * FGeneratorPackageInfo returned from ComputePackageDifferences that are used for incremental cook,
	 * we only need a subset of the data; we skip copying the portions we don't need to improve performance.
	 */
	static FAssetPackageData CopyAssetPackageDataForIncrementalCook(const FAssetPackageData& Source);

	/** State of the asset registry that is being built for this platform */
	FAssetRegistryState State;
	
	/** Map of Package name to Sandbox Paths */
	typedef TMap<FName, FString> FChunkPackageSet;
	/** Holds a reference to asset registry */
	IAssetRegistry& AssetRegistry;
	/** Platform to generate the manifest for */
	const ITargetPlatform* TargetPlatform;
	/** List of all asset packages that were created while loading the last package in the cooker. */
	TSet<FName> AssetsLoadedWithLastPackage;
	/** Lookup for the ChunkIDs that were explicitly set by the user in the editor */
	TMap<FName, TArray<int32>> ExplicitChunkIDs;
	/** Set of packages containing a map */
	TSet<FName> PackagesContainingMaps;
	/** Should the chunks be generated or only asset registry */
	bool bGenerateChunks;
	/** Highest chunk id, being used for geneating dependency tree */
	int32 HighestChunkId;
	/** Array of Maps with chunks<->packages assignments */
	TArray<TUniquePtr<FChunkPackageSet>> ChunkManifests;
	/** Map of packages that has not been assigned to chunks */
	FChunkPackageSet UnassignedPackageSet;
	/** Map of all cooked Packages */
	FChunkPackageSet AllCookedPackageSet;
	/** Array of Maps with chunks<->packages assignments. This version contains all dependent packages */
	TArray<TUniquePtr<FChunkPackageSet>> FinalChunkManifests;
	/** Additional data generators used when creating chunks */
	TArray<TSharedRef<IChunkDataGenerator>> ChunkDataGenerators;
	/** Source of the config-driven parent-child relationships between chunks. */
	UChunkDependencyInfo& DependencyInfo;

	/** Required flags a dependency must have if it is to be followed when adding package dependencies to chunks.*/
	UE::AssetRegistry::EDependencyQuery DependencyQuery;

	/** Mapping from chunk id to pakchunk file index. If not defined, Pakchunk index will be the same as chunk id by default */
	TMap<int32, int32> ChunkIdPakchunkIndexMapping;

	/**
	 * Gets the chunkId -> String map from the asset manager
	 * Wrapper function to confirm and error when there's an invalid characters used.
	 */
	TMap<int32, FString> GetPakChunkIdToStringOverrideMap();

	/**
	 * Returns the PakChunkName for the given chunk id.
	 * When a PakChunkId is overridden name will be wrapped by the Delimiter - so that there is an agreed format for regexing on later when loading these.
	 * Paks will fall into the format of:
	 * PakChunk0.pak
	 * PakChunk-MyName-.pak
	 */
	FString GetPakChunkNameForId(const TMap<int32, FString>& ChunkIdToStringOveride, const int32 PakChunkId);
	FString GetPakChunkNameForId(const int32 PakChunkId);

	/**
	 * Updates AssetData with TagsAndValues corresponding to any collections 
	 * flagged for inclusion as asset registry tags.
	 */
	void UpdateCollectionAssetData();

	/**
	 * Adds a package to chunk manifest
	 * 
	 * @param The sandbox filepath of the package
	 * @param The package name
	 * @param The ID of the chunk to assign it to
	 */
	void AddPackageToManifest(const FString& PackageSandboxPath, FName PackageName, int32 ChunkId);

	/**
	* Remove a package from a chunk manifest. Does nothing if package doesn't exist in chunk.
	*
	* @param The package name
	* @param The ID of the chunk to assign it to
	*/
	void RemovePackageFromManifest(FName PackageName, int32 ChunkId);

	/**
	 * Walks the dependency graph of assets and assigns packages to correct chunks.
	 * 
	 * @param the InSandboxFile used during cook
	 */
	void FixupPackageDependenciesForChunks(UE::Cook::FCookSandbox& InSandboxFile);

	/**
	 * Attaches encryption key guids into the registry data for encrypted primary assets
	 */
	void InjectEncryptionData(FAssetRegistryState& TargetState);

	void AddPackageToChunk(FChunkPackageSet& ThisPackageSet, FName InPkgName,
		const FString& InSandboxFile, int32 PakchunkIndex, UE::Cook::FCookSandbox& SandboxPlatformFile);

	/** The output path for this generator's platform where chunk manifests are written for use during staging. */
	FString GetChunkManifestDirectory(UCookOnTheFlyServer& COTFS) const;

	/** Returns the config-driven max size of a chunk for the given platform, or -1 for no limit. */
	int64 GetMaxChunkSizePerPlatform( const ITargetPlatform* Platform ) const;

	/** Returns an array of chunks ID for a package name that have been assigned during the cook process. */
	TArray<int32> GetExistingPackageChunkAssignments(FName PackageFName, const TSet<FName>& StartupPackages);

	/**
	 * Get chunks IDs for a package that were assigned to the package in the editor from AssetFileContextMenu.
	 * These explicit chunkids are unioned with the chunkids calculated by the AssetManager.
	 */
	TArray<int32> GetExplicitChunkIDs(const FName& PackageFName);

	/** Calculate the final ChunkIds used by the package and store the package in the manifest for each of those chunks. */
	void CalculateChunkIdsAndAssignToManifest(const FName& PackageFName, const FString& PackagePathName,
		const FString& SandboxFilename, const FString& LastLoadedMapName, UE::Cook::FCookSandbox& InSandboxFile,
		const TSet<FName>& StartupPackages);

	/** Returns true if the specific platform desires multiple chunks suitable for streaming install */
	bool ShouldPlatformGenerateStreamingInstallManifest(const ITargetPlatform* Platform) const;

	/** Generates and saves streaming install chunk manifest */
	bool GenerateStreamingInstallManifest(int64 InOverrideChunkSize, const TCHAR* InManifestSubDir,
		UE::Cook::FCookSandbox& InSandboxFile, UCookOnTheFlyServer& COTFS);

	/** Gather a list of dependencies required by to completely load this package */
	bool GatherAllPackageDependencies(FName PackageName, TArray<FName>& DependentPackageNames);

	/** Gather the list of dependencies that link the source to the target.  Output array includes the target */
	bool GetPackageDependencyChain(FName SourcePackage, FName TargetPackage, TSet<FName>& VisitedPackages, TArray<FName>& OutDependencyChain);

	/** Get an array of Packages this package will import */
	bool GetPackageDependencies(FName PackageName, TArray<FName>& DependentPackageNames, UE::AssetRegistry::EDependencyQuery InDependencyQuery);

	/** Save a CSV dump of chunk asset information, if bWriteIndividualFiles is true it writes a CSV per chunk in addition to AllChunksInfo */
	bool GenerateAssetChunkInformationCSV(const FString& OutputPath, bool bWriteIndividualFiles = false);

	/** Recursively remove redundant packages from child chunks based on the chunk dependency tree. */
	void SubtractParentChunkPackagesFromChildChunks(const FChunkDependencyTreeNode& Node,
		const TSet<FName>& CumulativeParentPackages, TArray<TArray<FName>>& OutPackagesMovedBetweenChunks);

	/** Helper function to verify Chunk asset assignment is valid */
	bool CheckChunkAssetsAreNotInChild(const FChunkDependencyTreeNode& Node);

	/** Initialize ChunkIdPakchunkIndexMapping and PakchunkIndexChunkIdMapping. */
	void InitializeChunkIdPakchunkIndexMapping();

	/**
	 * If InState records PackageName is generated, return the name of the Generator and the property flags of the Generator's
	 * AssetRegistry dependency upon the generated package, otherwise return NAME_None and EDependencyProperty::None.
	 */
	static FName GetGeneratorPackage(FName PackageName, const FAssetRegistryState& InState,
		UE::AssetRegistry::EDependencyProperty& OutDependencyPropertyOfGenerator);

	friend class UE::Cook::FAssetRegistryReporterRemote;
};

namespace UE::Cook
{

class IAssetRegistryReporter
{
public:
	virtual ~IAssetRegistryReporter() = default;

	virtual void UpdateAssetRegistryData(FName PackageName, const UPackage* Package, UE::Cook::ECookResult CookResult,
		FSavePackageResultStruct* SavePackageResult,
		TOptional<TArray<FAssetData>>&& AssetDatasFromSave, TOptional<FAssetPackageData>&& OverrideAssetPackageData,
		TOptional<TArray<FAssetDependency>>&& OverridePackageDependencies,
		UCookOnTheFlyServer& COTFS) = 0;

};

class FAssetRegistryReporterLocal : public IAssetRegistryReporter
{
public:
	FAssetRegistryReporterLocal(FAssetRegistryGenerator& InGenerator)
		: Generator(InGenerator)
	{
	}

	virtual void UpdateAssetRegistryData(FName PackageName, const UPackage* Package, UE::Cook::ECookResult CookResult,
		FSavePackageResultStruct* SavePackageResult,
		TOptional<TArray<FAssetData>>&& AssetDatasFromSave, TOptional<FAssetPackageData>&& OverrideAssetPackageData,
		TOptional<TArray<FAssetDependency>>&& OverridePackageDependencies, UCookOnTheFlyServer& COTFS) override
	{
		Generator.UpdateAssetRegistryData(PackageName, Package, CookResult, SavePackageResult,
			MoveTemp(AssetDatasFromSave), MoveTemp(OverrideAssetPackageData), MoveTemp(OverridePackageDependencies),
			COTFS);
	}

private:
	FAssetRegistryGenerator& Generator;
};

class FAssetRegistryReporterRemote : public IAssetRegistryReporter
{
public:
	FAssetRegistryReporterRemote(FCookWorkerClient& InClient, const ITargetPlatform* InTargetPlatform);

	virtual void UpdateAssetRegistryData(FName PackageName, const UPackage* Package, UE::Cook::ECookResult CookResult,
		FSavePackageResultStruct* SavePackageResult,
		TOptional<TArray<FAssetData>>&& AssetDatasFromSave, TOptional<FAssetPackageData>&& OverrideAssetPackageData,
		TOptional<TArray<FAssetDependency>>&& OverridePackageDependencies, UCookOnTheFlyServer& COTFS) override;

private:
	FCookWorkerClient& Client;
	const ITargetPlatform* TargetPlatform = nullptr;
	TMap<FName, FCbObject> PackageUpdateMessages;

	friend FAssetRegistryMPCollector;
};

class FAssetRegistryPackageMessage : public IMPCollectorMessage
{
public:
	virtual void Write(FCbWriter& Writer) const override;
	virtual bool TryRead(FCbObjectView Object) override;
	virtual FGuid GetMessageType() const override { return MessageType; }
	virtual const TCHAR* GetDebugName() const override { return TEXT("AssetRegistryPackageMessage"); }

	FName PackageName;
	const ITargetPlatform* TargetPlatform;
	TArray<FAssetData> AssetDatas;
	TOptional<FAssetPackageData> OverrideAssetPackageData;
	TOptional<TArray<FAssetDependency>> OverridePackageDependencies;
	uint32 PackageFlags = 0;
	int64 DiskSize = -1;

public:
	static FGuid MessageType;
};

class FAssetRegistryMPCollector : public UE::Cook::IMPCollector
{
public:
	FAssetRegistryMPCollector(UCookOnTheFlyServer& InCOTFS);

	virtual FGuid GetMessageType() const { return FAssetRegistryPackageMessage::MessageType; }
	virtual const TCHAR* GetDebugName() const { return TEXT("AssetRegistry"); }

	virtual void ClientTickPackage(FMPCollectorClientTickPackageContext& Context) override;
	virtual void ServerReceiveMessage(FMPCollectorServerMessageContext& Context, FCbObjectView Message) override;

private:
	UCookOnTheFlyServer& COTFS;
};


}
