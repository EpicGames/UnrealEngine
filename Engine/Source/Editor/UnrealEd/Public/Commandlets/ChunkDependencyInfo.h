// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

UE_DEPRECATED_HEADER(5.8, "Include Misc/ChunkDependencyInfo.h from the CoreUObject module instead.")

// This file is deprecated since UE 5.8. Use Misc/ChunkDependencyInfo.h instead!
#include "Misc/ChunkDependencyInfo.h"
