// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "AssetRegistry/AssetData.h"
#include "CoreMinimal.h"
#include "Commandlets/Commandlet.h"
#include "KismetCompilerModule.h"

#include "CompileAllBlueprintsCommandlet.generated.h"

UCLASS()
class UCompileAllBlueprintsCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()

	// Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	// End UCommandlet Interface

protected:
	/* Handles setting config variables from the command line
	* @Param Params command line parameter string
	*/
	virtual void InitCommandLine(const FString& Params);
	
	/* Loads the Kismet Compiler Module */
	virtual void InitKismetBlueprintCompiler();

	/* Stores FAssetData for every BlueprintAsset inside of the BlueprintAssetList member variable */
	virtual void BuildBlueprintAssetList();
	
	/* Loads and builds all blueprints in the BlueprintAssetList member variable */
	virtual void BuildBlueprints();

	/* Determines if we should try and Load / Build the asset based on config variables 
	* @Param Asset What asset to check
	* @Return True if we should build the asset
	*/
	virtual bool ShouldBuildAsset(FAssetData const& Asset) const;
	
	/* Handles attempting to compile an individual blueprint asset.
	* @Param Blueprint Asset that is compiled.
	*/
	virtual void CompileBlueprint(UBlueprint* Blueprint);

	/* Handles parsing out a string of command line tag pairs into the passed in output tag collection. Breaks these tags into a tag and a value to expect, or just a tag if any value will do.
	* @Param FullTagString The string from the commandline that needs to be parsed
	* @Param OutputAssetTags The collection of asset tags to output results to
	*/
	virtual void ParseTagPairs(const FString& FullTagString, /*OUT*/TArray<TPair<FString, TArray<FString>>>& OutputAssetTags);

	/* Handles parsing out a string of ignore folders and puts it into the IgnoreFolders member variable
	* @Param FullIgnoreFolderString the string passedi n from the commandline to parse out individual folder names from
	*/
	virtual void ParseIgnoreFolders(const FString& FullIgnoreFolderString);

	/* Handles parsing out all the different files to allow from a file. These files will not be built. 
	* @Param ListFilePath the filepath to parse an allow list from 
	*/
	virtual void ParseAllowList(const FString& ListFilePath);

	/* Checks if the passed in asset has any tag inside of the passed in tag collection. 
	* @Param Asset  Asset to check all tags of
	* @Param TagCollectionToCheck Which Tag Collection to check
	* @Return True if it contains a match from the tag collection. False otherwise
	*/
	virtual bool CheckHasTagInList(FAssetData const& Asset, const TArray<TPair<FString, TArray<FString>>>& TagCollectionToCheck) const;

	/* Checks if the passed in asset is included in the allow list.
	* @Param Asset  Asset to check against the allow list
	* @Return True if the asset is in the list. False otherwise.
	*/
	virtual bool IsAssetAllowed(FAssetData const& Asset) const;

	/* Handles outputting the results to the log */
	virtual void LogResults();

	//CommandLine Config Variables
	bool bResultsOnly;
	bool bSimpleAssetList;
	bool bCompileSkeletonOnly;
	bool bCookedOnly;
	bool bDirtyOnly;
	TArray<FString> IgnoreFolders;
	TArray<FString> FileAllowList;
	TArray<TPair<FString, TArray<FString>>> RequireAssetTags;
	TArray<TPair<FString, TArray<FString>>> ExcludeAssetTags;
	FTopLevelAssetPath BlueprintBaseClassName;

	//Variables to store overall results
	int TotalNumFailedLoads;
	int TotalNumFatalIssues;
	int TotalNumWarnings;
	TArray<FString> AssetsWithErrorsOrWarnings;

	IKismetCompilerInterface* KismetBlueprintCompilerModule;
	TArray<FAssetData> BlueprintAssetList;
};
