// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Commandlets/Commandlet.h"
#include "DetectOrphanedLocalizedAssetsCommandlet.generated.h"

UCLASS()
class UDetectOrphanedLocalizedAssetsCommandlet : public UCommandlet
{
    GENERATED_UCLASS_BODY()

public:
	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface

	static const FString UsageText;	 
};
