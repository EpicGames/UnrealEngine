// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Editor.h: Unreal editor public header file.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"

/**
 * Contains stats about a single resource in a package file.
 */
struct FObjectResourceStat
{
	/** the complete path name for this resource */
	int32 ResourceNameIndex;

	/** the name of the class for this resource */
	FName ClassName;

	/** the size of this resource, on disk */
	int32 ResourceSize;

	/** Standard Constructor */
	FObjectResourceStat( FName InClassName, const FString& InResourceName, int32 InResourceSize );

	/** Copy constructor */
	FObjectResourceStat( const FObjectResourceStat& Other )
	{
		ResourceNameIndex = Other.ResourceNameIndex;
		ClassName = Other.ClassName;
		ResourceSize = Other.ResourceSize;
	}
};

/**
 * A mapping of class name to the resource stats for objects of that class
 */
class FClassResourceMap : public TMultiMap<FName,FObjectResourceStat>
{
};

struct FPackageResourceStat
{
	/** the name of the package this struct contains resource stats for */
	FName				PackageName;

	/** the filename of the package; will be different from PackageName if this package is one of the loc packages */
	FName				PackageFilename;

	/** the map of 'Class name' to 'object resources of that class' for this package */
	FClassResourceMap	PackageResources;

	/**
	 * Constructor
	 */
	FPackageResourceStat( FName InPackageName )
	: PackageName(InPackageName)
	{ }

	/**
	 * Creates a new resource stat using the specified parameters.
	 *
	 * @param	ResourceClassName	the name of the class for the resource
	 * @param	ResourcePathName	the complete path name for the resource
	 * @param	ResourceSize		the size on disk for the resource
	 *
	 * @return	a pointer to the FObjectResourceStat that was added
	 */
	struct FObjectResourceStat* AddResourceStat( FName ResourceClassName, const FString& ResourcePathName, int32 ResourceSize );
};

struct FKismetResourceStat
{
	/** the name of the kismet object this struct contains stats for */
	FName				ObjectName;

	/** the number of references to the kismet object */
	int32					ReferenceCount;

	/** array of files that reference this kismet object */
	TArray<FString>		ReferenceSources;

	FKismetResourceStat( FName InObjectName )
	: ObjectName(InObjectName), ReferenceCount(0)
	{ }

	FKismetResourceStat( FName InObjectName, int32 InRefCount )
	: ObjectName(InObjectName), ReferenceCount(InRefCount)
	{ }
};
typedef TMap<FName,FKismetResourceStat> KismetResourceMap;


enum EReportOutputType
{
	/** write the results to the log only */
	OUTPUTTYPE_Log,

	/** write the results to a CSV file */
	OUTPUTTYPE_CSV,

	/** write the results to an XML file (not implemented) */
	OUTPUTTYPE_XML,
};

/**
 * Generates various types of reports for the list of resources collected by the AnalyzeCookedContent commandlet.  Each derived version of this struct
 * generates a different type of report.
 */
struct FResourceStatReporter
{
	EReportOutputType OutputType;

	/**
	 * Creates a report using the specified stats.  The type of report created depends on the reporter type.
	 *
	 * @param	ResourceStats	the list of resource stats to create a report for.
	 *
	 * @return	true if the report was created successfully; false otherwise.
	 */
	virtual bool CreateReport( const TArray<struct FPackageResourceStat>& ResourceStats )=0;

	/** Constructor */
	FResourceStatReporter()
	: OutputType(OUTPUTTYPE_Log)
	{}

	/** Destructor */
	virtual ~FResourceStatReporter() = default;
};

/**
 * This reporter generates a report on the disk-space taken by each asset type.
 */
struct FResourceStatReporter_TotalMemoryPerAsset : public FResourceStatReporter
{
	/**
	 * Creates a report using the specified stats.  The type of report created depends on the reporter type.
	 *
	 * @param	ResourceStats	the list of resource stats to create a report for.
	 *
	 * @return	true if the report was created successfully; false otherwise.
	 */
	virtual bool CreateReport( const TArray<struct FPackageResourceStat>& ResourceStats );
};

/**
 * This reporter generates a report which displays objects which are duplicated into more than one package.
 */
struct FResourceStatReporter_AssetDuplication : public FResourceStatReporter
{
	/**
	 * Creates a report using the specified stats.  The type of report created depends on the reporter type.
	 *
	 * @param	ResourceStats	the list of resource stats to create a report for.
	 *
	 * @return	true if the report was created successfully; false otherwise.
	 */
	virtual bool CreateReport( const TArray<struct FPackageResourceStat>& ResourceStats );
};

struct FResourceDiskSize
{
	FString ClassName;
	uint64 TotalSize;

	/** Default constructor */
	FResourceDiskSize( FName InClassName )
	: ClassName(InClassName.ToString()), TotalSize(0)
	{}

	/** Copy constructor */
	FResourceDiskSize( const FResourceDiskSize& Other )
	{
		ClassName = Other.ClassName;
		TotalSize = Other.TotalSize;
	}
};

//====================================================================
// UDiffPackagesCommandlet and helper structs
//====================================================================

/**
 * Contains an object and the object's path name.
 */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
struct UE_DEPRECATED(5.3, "DiffPackagesCommandlet has been deleted") FObjectReference
{
	UObject* Object;
	FString ObjectPathName;

	FObjectReference( UObject* InObject )
	: Object(InObject)
	{
		if ( Object != NULL )
		{
			ObjectPathName = Object->GetPathName();
		}
	}
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS


/**
 * Represents a single top-level object along with all its subobjects.
 */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
struct UE_DEPRECATED(5.3, "DiffPackagesCommandlet has been deleted.") FObjectGraph
{
	/**
	 * The list of objects in this object graph.  The first element is always the root object.
	 */
	TArray<struct FObjectReference> Objects;

	/**
	 * Constructor
	 *
	 * @param	RootObject			the top-level object for this object graph
	 * @param	PackageIndex		the index [into the Packages array] for the package that this object graph belongs to
	 * @param	ObjectsToIgnore		optional list of objects to not include in this object graph, even if they are contained within RootObject
	 */
	FObjectGraph( UObject* RootObject, int32 PackageIndex, TArray<struct FObjectComparison>* ObjectsToIgnore=NULL);

	/**
	 * Returns the root of this object graph.
	 */
	UObject* GetRootObject() const;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

