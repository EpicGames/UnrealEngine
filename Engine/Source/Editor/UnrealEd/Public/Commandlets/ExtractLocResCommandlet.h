// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commandlets/Commandlet.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "UObject/UObjectGlobals.h"

#include "ExtractLocResCommandlet.generated.h"

class UObject;

/**
 * Commandlet to extract the localization data from a binary LocRes file and dump it as human readable CSV.
 */
UCLASS()
class UExtractLocResCommandlet : public UCommandlet
{
	GENERATED_BODY()

public:

	virtual int32 Main(const FString& Params);
};
