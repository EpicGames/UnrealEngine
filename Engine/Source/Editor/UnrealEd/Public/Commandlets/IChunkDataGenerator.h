// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "CoreMinimal.h"
#endif


class FAssetRegistryGenerator;
class FSandboxPlatformFile;
class ITargetPlatform;
namespace UE::Cook { class ICookInfo; }

/**
 * Interface for generating extra data when creating streaming install manifests.
 * @note See FAssetRegistryGenerator.
 */
class IChunkDataGenerator
{
public:
	virtual ~IChunkDataGenerator() = default;

	struct FBeginGenerateContext
	{
	public:
		/**
		 * Get the list of ChunkIds that are referenced from the current cook. Files for other ChunkIds might
		 * exist on disk from previous cooks, and should be deleted.
		 */
		TConstArrayView<int32> GetCurrentChunkIds();
		/** The TargetPlatform for which chunks are being generated. */
		const ITargetPlatform* GetTargetPlatform();
		/** The sandbox location that staging data for this chunks should be written to. */
		FSandboxPlatformFile* GetSandboxFile();

	private:
		TConstArrayView<int32> CurrentChunkIds;
		const ITargetPlatform* TargetPlatform = nullptr;
		FSandboxPlatformFile* SandboxFile = nullptr;

		friend FAssetRegistryGenerator;
	};

	/**
	 * Called to clear or udpate chunk data files from a previous incremental cook. This will be called
	 * before the call to GenerateChunkDataFiles for each of the listed chunks.
	 */
	virtual void BeginGenerateChunkDataFiles(FBeginGenerateContext& Context)
	{
	}

	/**
	 * Called to generate any additional files that should be added to the given chunk.
	 *
	 * @param InChunkId The ID of the chunk to generate the files for.
	 * @param InPackagesInChunk The set of packages that are in the chunk.
	 * @param TargetPlatform The platform this chunk is for.
	 * @param InSandboxFile The sandbox location that staging data for this chunk should be written to.
	 * @param OutChunkFilenames Array of filenames that belong to this chunk (to be appended to).
	 * @return true if successful. If this function returns false, the deprecated one will be called
	 */
	virtual void GenerateChunkDataFiles(const int32 InChunkId, const TSet<FName>& InPackagesInChunk,
		const ITargetPlatform* TargetPlatform, FSandboxPlatformFile* InSandboxFile, TArray<FString>& OutChunkFilenames)
	{
	}

	UNREALED_API static void AddChunkDataGeneratorFactory(const TFunction<TSharedRef<IChunkDataGenerator>(const UE::Cook::ICookInfo&)>& InChunkGeneratorFactory);
	UNREALED_API static const TArray<TFunction<TSharedRef<IChunkDataGenerator>(const UE::Cook::ICookInfo&)>>& GetChunkDataGeneratorFactories();
private:
	static TArray<TFunction<TSharedRef<IChunkDataGenerator>(const UE::Cook::ICookInfo&)>> GeneratorsDataGeneratorFactories;
};


///////////////////////////////////////////////////////
// Inline implementations
///////////////////////////////////////////////////////


inline TConstArrayView<int32> IChunkDataGenerator::FBeginGenerateContext::GetCurrentChunkIds()
{
	return CurrentChunkIds;
}
inline const ITargetPlatform* IChunkDataGenerator::FBeginGenerateContext::GetTargetPlatform()
{
	return TargetPlatform;
}
inline FSandboxPlatformFile* IChunkDataGenerator::FBeginGenerateContext::GetSandboxFile()
{
	return SandboxFile;
}
