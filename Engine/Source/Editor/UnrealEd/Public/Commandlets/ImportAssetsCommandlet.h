// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"

#include "ImportAssetsCommandlet.generated.h"

class UAutomatedAssetImportData;

UCLASS()
class UImportAssetsCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()

	/** Runs the commandlet */
	virtual int32 Main(const FString& Params) override;

private:
	/**
	 * Parses the command line parameters 
	 * @param InParams - The parameters to parse
	 */
	bool ParseParams(const FString& InParams);

	/**
	 * Parses import settings from a json file
	 *
	 * @param InImportSettingsFile - The filename to parse
	 */
	bool ParseImportSettings(const FString& InImportSettingsFile);

	/**
	 * Imports and saves assets
	 *
	 * @param AssetImportList	List of import data to import.  Each element in the list represents a list of assets using the same import settings
	 */
	bool ImportAndSave(const TArray<UAutomatedAssetImportData*>& AssetImportList);

	/** Loads a level to be used for spawning actors from import factories */
	bool LoadLevel(const FString& LevelToLoad);

	/** Prints command line arguments */
	void PrintUsage();

	/**
	 * Clears dirty flag from all packages
	 */
	void ClearDirtyPackages();
private:
	/** Command line args */
	FString ImportSettingsPath;

	/** Whether or not to show help instead of performing an import */
	bool bShowHelp;

	/**
	 * Settings used when an import settings file is not used 
	 * or as a base to fallback on if settings are not overridden by the file 
	 */
	UPROPERTY()
	TObjectPtr<UAutomatedAssetImportData> GlobalImportData;

	/** List of import data to import.  Each element in the list represents a list of assets using the same import settings */
	UPROPERTY()
	TArray<TObjectPtr<UAutomatedAssetImportData>> ImportDataList;

	/** If true we allow source control operations during import */
	bool bAllowSourceControl;

	/** If source control is allowed and we could successfully connect */
	bool bHasSourceControl;
};
