// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectAnnotation.h"

class FPackageSourceControlHelper;
class ULevel;
class UWorld;

namespace WorldPartitionCommandletHelpers
{
	UNREALED_API DECLARE_LOG_CATEGORY_EXTERN(LogWorldPartitionCommandletUtils, Log, All);

	UWorld* LoadAndInitWorld(const FString& LevelToLoad);
	UWorld* LoadWorld(const FString& LevelToLoad);
	ULevel* InitLevel(UWorld* World);

	bool Checkout(const TArray<UPackage*>& PackagesToCheckout, FPackageSourceControlHelper& SCHelper);
	bool Checkout(UPackage* PackagesToCheckout, FPackageSourceControlHelper& SCHelper);
	bool Save(const TArray<UPackage*>& PackagesToSave);
	bool Save(UPackage* PackagesToSave);
	bool AddToSourceControl(const TArray<UPackage*>& PackagesToAdd, FPackageSourceControlHelper& SCHelper);
	bool AddToSourceControl(UPackage* PackageToAdd, FPackageSourceControlHelper& SCHelper);
	bool Delete(const TArray<UPackage*>& PackagesToDelete, FPackageSourceControlHelper& SCHelper);
	bool Delete(UPackage* PackagesToDelete, FPackageSourceControlHelper& SCHelper);

	template<class T>
	bool CheckoutSaveAdd(T&& ToSave, FPackageSourceControlHelper& SCHelper)
	{
		if (Checkout(ToSave, SCHelper))
		{
			if (Save(ToSave))
			{
				return AddToSourceControl(ToSave, SCHelper);
			}
		}

		return false;
	}
}