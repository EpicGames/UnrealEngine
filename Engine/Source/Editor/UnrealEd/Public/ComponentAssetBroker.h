// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SubclassOf.h"
#include "Components/ActorComponent.h"

//////////////////////////////////////////////////////////////////////////
// List of component types

typedef TArray< TSubclassOf<UActorComponent> > FComponentClassList;

//////////////////////////////////////////////////////////////////////////
// IComponentAssetBroker

/**
 * This class knows how to get or set the asset on a particular kind of actor component
 *
 * One asset type can be associated with multiple component types, but any given component
 * type only understands how to be created from a single asset type (for now)
 */
class IComponentAssetBroker
{
public:
	/** Reports the component class this broker knows how to handle */
	//virtual TSubclassOf<UActorComponent> GetSupportedComponentClass()=0;

	/** Reports the asset class this broker knows how to handle */
	virtual UClass* GetSupportedAssetClass()=0;

	/** Assign the assigned asset to the supplied component */
	virtual bool AssignAssetToComponent(UActorComponent* InComponent, UObject* InAsset)=0;

	/** Get the currently assigned asset from the component */
	virtual UObject* GetAssetFromComponent(UActorComponent* InComponent)=0;

	virtual ~IComponentAssetBroker() = default;
};

//////////////////////////////////////////////////////////////////////////
// FComponentAssetBrokerageage

/** Utility class that associates assets with component classes */
class FComponentAssetBrokerage
{
public:
	/** Find set of components that support this asset */
	static UNREALED_API TArray< TSubclassOf<UActorComponent> > GetComponentsForAsset(const UObject* InAsset);

	/** Get the primary component for this asset*/
	static UNREALED_API TSubclassOf<UActorComponent> GetPrimaryComponentForAsset(UClass* InAssetClass);

	/** Assign the assigned asset to the supplied component */
	static UNREALED_API bool AssignAssetToComponent(UActorComponent* InComponent, UObject* InAsset);

	/** Get the currently assigned asset from the component */
	static UNREALED_API UObject* GetAssetFromComponent(UActorComponent* InComponent);

	/** See if this component supports assets of any type */
	static UNREALED_API bool SupportsAssets(UActorComponent* InComponent);

	/** Register a component class for a specified asset class */
	static UNREALED_API void RegisterAssetToComponentMapping(UClass* InAssetClass, TSubclassOf<UActorComponent> InComponentClass, bool bSetAsPrimary);

	/** Unregister a component type for a specified asset class */
	static UNREALED_API void UnregisterAssetToComponentMapping(UClass* InAssetClass, TSubclassOf<UActorComponent> InComponentClass);

	/** Try to find the broker for the specified component type */
	static UNREALED_API TSharedPtr<IComponentAssetBroker> FindBrokerByComponentType(TSubclassOf<UActorComponent> InComponentClass);

	/** Try to find the *primary* broker for the specified asset type */
	static UNREALED_API TSharedPtr<IComponentAssetBroker> FindBrokerByAssetType(UClass* InAssetClass);

	/** Get the currently supported assets, optionally filtered by the supplied component class */
	static UNREALED_API TArray<UClass*> GetSupportedAssets(UClass* InFilterComponentClass = NULL);


	/** Register a component class for a specified asset class */
	static UNREALED_API void RegisterBroker(TSharedPtr<IComponentAssetBroker> Broker, TSubclassOf<UActorComponent> InComponentClass, bool bSetAsPrimary, bool bMapComponentForAssets);

	/** Unregister a component type for a specified asset class */
	static UNREALED_API void UnregisterBroker(TSharedPtr<IComponentAssetBroker> Broker);

	/** Shut down the brokerage; should only be called by the editor during shutdown */
	static UNREALED_API void PRIVATE_ShutdownBrokerage();

private:
	FComponentAssetBrokerage() {}

	static void InitializeMap();

private:
	// Map from an asset class to all component classes that can use it
	// The first one in the array is the 'primary'
	static TMap<UClass*, FComponentClassList> AssetToComponentClassMap;

	// Map from component type to the broker for that type
	static TMap< TSubclassOf<UActorComponent>, TSharedPtr<IComponentAssetBroker> > ComponentToBrokerMap;

	// Map from asset type to the *primary* broker for that type
	static TMap< UClass*, TArray< TSharedPtr<IComponentAssetBroker> > > AssetToBrokerMap;

	static bool bInitializedBuiltinMap;
	static bool bShutSystemDown;
};
