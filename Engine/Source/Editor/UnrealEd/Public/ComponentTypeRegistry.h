// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SubclassOf.h"
#include "Components/ActorComponent.h"

class FComponentClassComboEntry;
class UBlueprintGeneratedClass;

DECLARE_MULTICAST_DELEGATE(FOnComponentTypeListChanged);

typedef TSharedPtr<class FComponentClassComboEntry> FComponentClassComboEntryPtr;

struct FComponentTypeEntry
{
	/** Name of the component, as typed by the user */
	FString ComponentName;

	/** Name of the component, corresponds to asset name for blueprint components */
	FString ComponentAssetName;

	/** Optional pointer to the UClass, will be nullptr for blueprint components that aren't loaded */
	TObjectPtr<class UClass> ComponentClass;
};

struct FComponentTypeRegistry
{
	static UNREALED_API FComponentTypeRegistry& Get();

	/**
	 * Called when the user changes the text in the search box.
	 * @OutComponentList Pointer that will be set to the (globally shared) component type list.
	 * @return Delegate that can be used to handle change notifications. Change notifications are raised when entries are 
	 * added or removed from the component type list.
	 */
	UNREALED_API FOnComponentTypeListChanged& SubscribeToComponentList(TArray<FComponentClassComboEntryPtr>*& OutComponentList);
	UNREALED_API FOnComponentTypeListChanged& SubscribeToComponentList(const TArray<FComponentTypeEntry>*& OutComponentList);
	UNREALED_API FOnComponentTypeListChanged& GetOnComponentTypeListChanged();

	/**
	 * Called when a specific class has been updated and should force the component type registry to update as well
	 */
	UNREALED_API void InvalidateClass(TSubclassOf<UActorComponent> ClassToUpdate);

	/** Schedules a full update of the component type registry on the next frame */
	UNREALED_API void Invalidate();

	/**
	 * Attempts to locate the class entry corresponding to the given object path.
	 */
	UNREALED_API FComponentClassComboEntryPtr FindClassEntryForObjectPath(FTopLevelAssetPath InObjectPath) const;

private:
	void OnReloadComplete(EReloadCompleteReason Reason);
	void OnBlueprintGeneratedClassUnloaded(UBlueprintGeneratedClass* BlueprintGeneratedClass);

private:
	FComponentTypeRegistry();
	~FComponentTypeRegistry();
	struct FComponentTypeRegistryData* Data;
};
