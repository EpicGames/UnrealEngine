// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "Containers/Array.h"
#include "ConvexVolume.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Engine/EngineBaseTypes.h"
#include "GameFramework/Actor.h"
#include "GenericPlatform/ICursor.h"
#include "HitProxies.h"
#include "InputCoreTypes.h"
#include "Math/Box.h"
#include "Math/MathFwd.h"
#include "Math/Matrix.h"
#include "Math/Rotator.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementHandle.h"

#include "ComponentVisualizer.generated.h"

class AActor;
class FCanvas;
class FEditorViewportClient;
class FPrimitiveDrawInterface;
class FSceneView;
class FViewport;
class SWidget;
struct FViewportClick;

struct HComponentVisProxy : public HHitProxy
{
	DECLARE_HIT_PROXY(UNREALED_API);

	HComponentVisProxy(const UActorComponent* InComponent, EHitProxyPriority InPriority = HPP_Wireframe) 
	: HHitProxy(InPriority)
	, Component(InComponent)
	{}

	virtual EMouseCursor::Type GetMouseCursor() override
	{
		return EMouseCursor::Crosshairs;
	}

	virtual FTypedElementHandle GetElementHandle() const override
	{
		return Component.IsValid() ? 
			UEngineElementsLibrary::AcquireEditorComponentElementHandle(Component.Get()) :
			FTypedElementHandle();
	}

	TWeakObjectPtr<const UActorComponent> Component;
};

USTRUCT()
struct FPropertyNameAndIndex
{
public:
	GENERATED_USTRUCT_BODY()

	FPropertyNameAndIndex()
		: Name(NAME_None)
		, Index(INDEX_NONE)
	{}

	explicit FPropertyNameAndIndex(FName InName, int32 InIndex = 0)
		: Name(InName)
		, Index(InIndex)
	{}

	bool IsValid() const { return Name != NAME_None && Index != INDEX_NONE; }

	void Clear()
	{
		Name = NAME_None;
		Index = INDEX_NONE;
	}

	bool operator ==(const FPropertyNameAndIndex& InRHS) const
	{
		return (Name == InRHS.Name && Index == InRHS.Index);
	}

	UPROPERTY()
	FName Name;

	UPROPERTY()
	int32 Index;
};


/**
 * Describes a chain of properties from the parent actor of a given component, to the component itself.
 */
USTRUCT()
struct FComponentPropertyPath
{
public:
	 GENERATED_USTRUCT_BODY()

	FComponentPropertyPath() = default;
	explicit FComponentPropertyPath(const UActorComponent* Component) { Set(Component); }

	/** Resets the property path */
	void Reset()
	{
		ParentOwningActor = nullptr;
		LastResortComponentPtr = nullptr;
		PropertyChain.Reset();
	}

	/** Gets the parent owning actor for the component, or nullptr if it is not valid */
	AActor* GetParentOwningActor() const { return ParentOwningActor.Get(); }

	/** Gets a pointer to the component, or nullptr if it is not valid */
	UNREALED_API UActorComponent* GetComponent() const;

	/** Determines whether the property path is valid or not */
	UNREALED_API bool IsValid() const;

	bool operator ==(const FComponentPropertyPath& InRHS) const
	{
		return (ParentOwningActor == InRHS.ParentOwningActor && LastResortComponentPtr == InRHS.LastResortComponentPtr && PropertyChain == InRHS.PropertyChain);
	}

	bool operator !=(const FComponentPropertyPath& InRHS) const
	{
		return (ParentOwningActor != InRHS.ParentOwningActor || LastResortComponentPtr != InRHS.LastResortComponentPtr || PropertyChain != InRHS.PropertyChain);
	}

private:

	/** Sets the component referred to by the object */
	UNREALED_API void Set(const UActorComponent* Component);

	UPROPERTY()
	TWeakObjectPtr<AActor> ParentOwningActor;

	UPROPERTY()
	TWeakObjectPtr<UActorComponent> LastResortComponentPtr;

	UPROPERTY()
	TArray<FPropertyNameAndIndex> PropertyChain;
};



/** Base class for a component visualizer, that draw editor information for a particular component class */
class FComponentVisualizer : public TSharedFromThis<FComponentVisualizer>
{
public:
	FComponentVisualizer() {}
	virtual ~FComponentVisualizer() = default;

	/** */
	virtual void OnRegister() {}
	/** Show this visualizer if the component is indirectly selected */
	virtual bool ShouldShowForSelectedSubcomponents(const UActorComponent* Component) { return true; }
	/** Draw visualization for the supplied component */
	virtual void DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI) {}
	/** Draw HUD on viewport for the supplied component */
	virtual void DrawVisualizationHUD(const UActorComponent* Component, const FViewport* Viewport, const FSceneView* View, FCanvas* Canvas) {}
	/** */
	virtual bool VisProxyHandleClick(FEditorViewportClient* InViewportClient, HComponentVisProxy* VisProxy, const FViewportClick& Click) { return false; }
	/** 
	 * Determines whether, if the visualizer returns true from VisProxyHandleClick, the hitproxy element should
	 *  be automatically selected. By default, uses the value of Editor.ComponentVisualizer.AutoSelectComponent.
	 */
	UNREALED_API virtual bool ShouldAutoSelectElementOnHandleClick() const;
	/** */
	virtual void EndEditing() {}
	/** */
	virtual bool GetWidgetLocation(const FEditorViewportClient* ViewportClient, FVector& OutLocation) const { return false; }
	/** */
	virtual bool GetCustomInputCoordinateSystem(const FEditorViewportClient* ViewportClient, FMatrix& OutMatrix) const { return false; }
	/** */
	virtual bool HandleInputDelta(FEditorViewportClient* ViewportClient, FViewport* Viewport, FVector& DeltaTranslate, FRotator& DeltaRotate, FVector& DeltaScale) { return false; }
	/** */
	virtual bool HandleInputKey(FEditorViewportClient* ViewportClient,FViewport* Viewport,FKey Key,EInputEvent Event) { return false; }
	/** Handle click modified by Alt, Ctrl and/or Shift. The input HitProxy may not be on this component. */
	virtual bool HandleModifiedClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click) { return false; }
	/** Handle box select input */
	virtual bool HandleBoxSelect(const FBox& InBox, FEditorViewportClient* InViewportClient,FViewport* InViewport) { return false; }
	/** Handle frustum select input */
	virtual bool HandleFrustumSelect(const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, FViewport* InViewport) { return false; }
	/** Return whether focus on selection should focus on bounding box defined by active visualizer */
	virtual bool HasFocusOnSelectionBoundingBox(FBox& OutBoundingBox) { return false; }
	/** Pass snap input to active visualizer */
	virtual bool HandleSnapTo(const bool bInAlign, const bool bInUseLineTrace, const bool bInUseBounds, const bool bInUsePivot, AActor* InDestination) { return false;  }
	/** Gets called when the mouse tracking has started (dragging behavior) */
	virtual void TrackingStarted(FEditorViewportClient* InViewportClient) {}
	/** Gets called when the mouse tracking has stopped (dragging behavior) */
	virtual void TrackingStopped(FEditorViewportClient* InViewportClient, bool bInDidMove) {}
	/** Get currently edited component, this is needed to reset the active visualizer after undo/redo */
	virtual UActorComponent* GetEditedComponent() const { return nullptr;  }

	/** */
	virtual TSharedPtr<SWidget> GenerateContextMenu() const { return TSharedPtr<SWidget>(); }
	/** */
	virtual bool IsVisualizingArchetype() const { return false; }

	// So deprecated code expecting this as an inner class still works
	using FPropertyNameAndIndex UE_DEPRECATED(5.6, "Please update to use the external class instead of internal identifier") = ::FPropertyNameAndIndex;

	/** Notify that a component property has been modified */
	static UNREALED_API void NotifyPropertyModified(UActorComponent* Component, FProperty* Property, EPropertyChangeType::Type PropertyChangeType = EPropertyChangeType::Unspecified);

	/** Notify that many component properties have been modified */
	static UNREALED_API void NotifyPropertiesModified(UActorComponent* Component, const TArray<FProperty*>& Properties, EPropertyChangeType::Type PropertyChangeType = EPropertyChangeType::Unspecified);
};

struct FCachedComponentVisualizer
{
	FComponentPropertyPath ComponentPropertyPath;
	TSharedPtr<FComponentVisualizer> Visualizer;
	
	FCachedComponentVisualizer(UActorComponent* InComponent, TSharedPtr<FComponentVisualizer>& InVisualizer)
		: ComponentPropertyPath(InComponent)
		, Visualizer(InVisualizer)
	{}
};
