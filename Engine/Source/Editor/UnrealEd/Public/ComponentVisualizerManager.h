// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "ComponentVisualizer.h"

class FEditorViewportClient;
class FViewport;
class SWidget;
struct FViewportClick;

/** Class that managed active component visualizer and routes input to it */
class FComponentVisualizerManager
{
public:
	UNREALED_API FComponentVisualizerManager();
	virtual ~FComponentVisualizerManager() = default;


	/** Activate a component visualizer given a clicked proxy */
	UNREALED_API bool HandleProxyForComponentVis(FEditorViewportClient* InViewportClient, HHitProxy *HitProxy, const FViewportClick &Click);

	/** Get the active component visualizer. */
	UNREALED_API TSharedPtr<FComponentVisualizer> GetActiveComponentVis();

	/** Activate a component visualizer given the specific visualizer */
	UNREALED_API bool SetActiveComponentVis(FEditorViewportClient* InViewportClient, TSharedPtr<FComponentVisualizer>& InVisualizer);

	/** Clear active component visualizer */
	UNREALED_API void ClearActiveComponentVis();

	/** Handle a click on the specified level editor viewport client */
	UNREALED_API bool HandleClick(FEditorViewportClient* InViewportClient, HHitProxy *HitProxy, const FViewportClick &Click);

	/** Pass key input to active visualizer */
	UNREALED_API bool HandleInputKey(FEditorViewportClient* InViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event) const;

	/** Pass delta input to active visualizer */
	UNREALED_API bool HandleInputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale) const;

	/** Pass box select input to active visualizer */
	UNREALED_API bool HandleBoxSelect(const FBox& InBox, FEditorViewportClient* InViewportClient, FViewport* InViewport) const;

	/** Pass frustum select input to active visualizer */
	UNREALED_API bool HandleFrustumSelect(const FConvexVolume &InFrustum, FEditorViewportClient* InViewportClient, FViewport* InViewport) const;

	/** Return whether focus on selection should focus on bounding box defined by active visualizer */
	UNREALED_API bool HasFocusOnSelectionBoundingBox(FBox& OutBoundingBox) const;

	/** Pass snap input to active visualizer */
	UNREALED_API bool HandleSnapTo(const bool bInAlign, const bool bInUseLineTrace, const bool bInUseBounds, const bool bInUsePivot, AActor* InDestination);

	/** Get widget location from active visualizer */
	UNREALED_API bool GetWidgetLocation(const FEditorViewportClient* InViewportClient, FVector& OutLocation) const;

	/** Get custom widget coordinate system from active visualizer */
	UNREALED_API bool GetCustomInputCoordinateSystem(const FEditorViewportClient* InViewportClient, FMatrix& OutMatrix) const;

	/** Gets called when the mouse tracking has started (dragging behavior) */
	UNREALED_API void TrackingStarted(FEditorViewportClient* InViewportClient);

	/** Gets called when the mouse tracking has stopped (dragging behavior) */
	UNREALED_API void TrackingStopped(FEditorViewportClient* InViewportClient, bool bInDidMove);

	/** Generate context menu for the component visualizer */
	UNREALED_API TSharedPtr<SWidget> GenerateContextMenuForComponentVis() const;

	/** Returns whether there is currently an active visualizer */
	UNREALED_API bool IsActive() const;

	/** Returns whether the component being visualized is an archetype or not */
	UNREALED_API bool IsVisualizingArchetype() const;

private:
	/** Currently 'active' visualizer that we should pass input to etc */
	TWeakPtr<class FComponentVisualizer> EditedVisualizerPtr;

	/** The viewport client for the currently active visualizer */
	FEditorViewportClient* EditedVisualizerViewportClient;
};
