// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ConsolidationWindow.h: Dialog for displaying the asset consolidation tool.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"

class SConsolidateToolWidget;

class FConsolidateToolWindow
{
public:
	/**
	 * Attempt to add the provided objects to the consolidation panel; Only adds objects which are compatible with objects already existing within the panel, if any
	 *
	 * @param	InObjects			Objects to attempt to add to the panel
	 *
	 */
	UNREALED_API static void AddConsolidationObjects( const TArray<UObject*>& InObjects, UObject* SelectedItem = nullptr );

	/**
	 * Determine the compatibility of the passed in objects with the objects already present in the consolidation panel
	 *
	 * @param	InProposedObjects		Objects to check compatibility with vs. the objects already present in the consolidation panel
	 * @param	OutCompatibleObjects	[out]Objects from the passed in array which are compatible with those already present in the
	 *									consolidation panel, if any
	 *
	 * @return	true if all of the passed in objects are compatible, false otherwise
	 */
	UNREALED_API static bool DetermineAssetCompatibility( const TArray<UObject*>& InProposedObjects, TArray<UObject*>& OutCompatibleObjects );

private:
	/**
	 * A pointer to an existing instance of our Widget (if any)
	 */
	static TWeakPtr<SConsolidateToolWidget> WidgetInstance;
};
