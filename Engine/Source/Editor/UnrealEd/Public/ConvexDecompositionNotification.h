// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
ConvexDecompositionNotification.h: Defines the interface to update the ConvexDecomposition notification state
=============================================================================*/

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"

// To make the convex decomposition notification appear, you simply set 'IsActive' to true and assign
// the current status message in 'Status'.  Use the 'GConvexDecompositionNotificationState' pointer
// to get access to the global notification state.
class FConvexDecompositionNotificationState
{
public:
	bool		IsActive{ false };	// True if convex decomposition is currently active.
	FString		Status;				// The current status notification message to display
};

/** The global pointer to the notification for convex decomposition; used to set the active state and update messages. */
extern UNREALED_API FConvexDecompositionNotificationState* GConvexDecompositionNotificationState;
