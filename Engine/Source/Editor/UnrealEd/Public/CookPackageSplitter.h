// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "AssetRegistry/IAssetRegistry.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/List.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "CoreMinimal.h"
#endif
#include "Hash/Blake3.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PreprocessorHelpers.h"
#include "Misc/Optional.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

class UClass;
class UObject;
class UPackage;
struct FAssetDependency;
template <typename FuncType> class TFunctionRef;

namespace UE::Cook { class FCookDependency; }
namespace UE::Cook::CookPackageSplitter { struct FPopulateContextData; }

/**
 * This class is used for packages that need to be split into multiple runtime packages.
 * It provides the instructions to the cooker for how to split the package.
 */
class ICookPackageSplitter
{
public:
	// Static API functions - these static functions are referenced by REGISTER_COOKPACKAGE_SPLITTER
	// before creating an instance of the class.
	/**
	 * Return whether IsCachedCookedPlatformDataLoaded needs to return true for all UObjects in the
	 * generator package before ShouldSplit or ReportGenerationManifest can be called. If true this slows down
	 * our ability to parallelize the cook of the generated packages.
	 */
	static bool RequiresCachedCookedPlatformDataBeforeSplit() { return false; }
	/** Return whether the CookPackageSplitter subclass should handle the given SplitDataClass instance. */
	static bool ShouldSplit(UObject* SplitData) { return false; }
	/** Return DebugName for this SplitterClass in cook log messages. */
	static FString GetSplitterDebugName() { return TEXT("<NoNameSpecified>"); }


	// Virtual API functions - functions called from the cooker after creating the splitter.
	virtual ~ICookPackageSplitter() = default;

	/**
	 * Called immediately after construction. This is useful for adding a reference to the OwnerObject
	 * to prevent it from being garbage collected, if the class does not want to allow that possibility.
	 */
	virtual void Initialize(UPackage* OwnerPackage, UObject* OwnerObject)
	{
	}

	enum class ETeardown
	{
		Complete,
		Canceled,
	};
	/** Do teardown actions after all packages have saved, or when the cook is cancelled. Always called before destruction. */
	virtual void Teardown(ETeardown Status)
	{
	}

	/**
	 * If true, this splitter forces the Generator package objects it needs to remain referenced, and the cooker
	 * should expect them to still be in memory after a garbage collect so long as the splitter is alive.
	 */
	virtual bool UseInternalReferenceToAvoidGarbageCollect()
	{
		return false;
	}

	/**
	 * An ICookPackageSplitter for a single generator package normally is constructed only once and handles
	 * all generated packages for that generator, but during MPCook in cases of load balancing between CookWorkers,
	 * it is possible that the original splitter is destructed but then recreated later. This is guaranteed not
	 * to happen without a GarbageCollection pass in between, but that GarbageCollection may fail to destruct the
	 * generator package if it is still referenced from other packages or systems. Depending on the ICookPackageSplitter's
	 * implemenation, this failure to GC might cause an error, because changes made from the previous splitter are not 
	 * handled in the next splitter. If RequiresGeneratorPackageDestructBeforeResplit is true, the cooker will log this failure
	 * to GC the generator package as an error.
	 */
	virtual bool RequiresGeneratorPackageDestructBeforeResplit()
	{
		return false;
	}

	/**
	 * If true, this splitter requires that the given generated package be empty before calling the populate function.
	 * Returning false will allow the splitter to precreate packages instead of relying on the cooker to create them.
	 */
	virtual bool RequiresEmptyPackageBeforePopulate(const UPackage* Package)
	{
		return true;
	}

	/**
	 * Return value for the DoesGeneratedRequireGenerator function. All levels behave correctly, but provide
	 * different tradeoffs of guarantees to the splitter versus performance.
	 */
	enum class EGeneratedRequiresGenerator : uint8
	{
		/**
		 * ReportGenerationManifest will be called before PopulateGeneratedPackage. PopulateGenerator and
		 * PreSaveGenerator might or might not be called before. OutKeepReferencedPackages from PopulateGenerator
		 * will not be kept referenced after PostSaveGenerator. Best for performance.
		 */
		None,
		/**
		 * ReportGenerationManifest and PopulateGenerator will be called before PopulateGeneratedPackage.
		 * OutKeepReferencedPackages from PopulateGenerator will be kept referenced until all generated and generator
		 * packages call PostSave or until the splitter is destroyed. Performance cost: Possible extra calls to
		 * PopulateGeneratedPackage, possible unnecessary memory increase due to OutKeepReferencedPackages.
		 */
		Populate,
		/**
		 * ReportGenerationManifest, PopulateGenerator, PreSaveGenerator, and PostSaveGenerator will be called before
		 * PopulateGeneratedPackage. Performance cost: Progress on generated packages will be delayed until generator
		 * finishes saving. Possible unnecessary memory increase due to OutKeepReferencedPackages. Retraction is not
		 * possible in MPCook for the generated packages; they must all be saved on the same CookWorker that saves the
		 * generator.
		 */
		Save,
		Count,
	};
	/**
	 * Return capability setting which indicates which splitter functions acting on the parent generator package must
	 * be called on the splitter before splitter functions acting on the generated packages can be called. Also impacts
	 * the lifetime of memory guarantees for the generator functions. @see EGeneratedRequiresGenerator. Default is
	 * EGeneratedRequiresGenerator::None, which provides the best performance but the fewest guarantees.
	 * 
	 * Examples of dependencies and what capability level should be used:
	 *		ShouldSplit call reads data that is written by BeginCacheForCookedPlatformData:
	 *			EGeneratedRequiresGenerator::Save
	 *     PopulateGeneratedPackage or PreSaveGeneratedPackage read data that is written by PopulateGeneratorPackage:
	 *			EGeneratedRequiresGenerator::Populate
	 */
	virtual EGeneratedRequiresGenerator DoesGeneratedRequireGenerator()
	{
		 return EGeneratedRequiresGenerator::None;
	}

	/** Data sent to the cooker to describe each desired generated package. */
	struct FGeneratedPackage
	{
		/** Parent path for the generated package. If empty, uses the generator's package path. */
		FString GeneratedRootPath;
		/** Generated package relative to <GeneratedRootPath>/_Generated_. */
		FString RelativePath;
		/**
		 * Source packages outside of the generator package that will be incorporated into the generated
		 * package (e.g. ExternalActor packages). These are used to construct the PackageSavedHash for the generated
		 * package. Some objects use the PackageSavedHash during derived data construction as a change marker, adding
		 * the source packages here is important for those types to work.
		 *
		 * These packages are also recorded as dependencies in the AssetRegistry generated by the cook.
		 *
		 * During incrementalcook, changes to these packages cause the recook of the generated package, but that
		 * invalidation can also be accomplished without the other effects, and with more types of dependencies, using
		 * PopulateContext.ReportSaveDependency during PopulateGeneratedPackage and PreSaveGeneratedPackage.
		 */
		TArray<FAssetDependency> PackageDependencies;
		/**
		 * Hash of the data used to construct the generated package that is not covered by the dependencies.
		 * Changes to this hash will cause invalidation of the package during incremental cooks.
		 */
		FBlake3Hash GenerationHash;
		/*
		 * Must be called with true or false for each GeneratedPackage reported in the GenerationManifest.
		 * Report whether PopulateGeneratorPackage will populate the package as a map (contains a UWorld or ULevel),
		 * and it should therefore be assigned a .umap extension, or as a non-map package, with a .uasset extension.
		 */
		void SetCreateAsMap(bool bInCreateAsMap) { bCreateAsMap = bInCreateAsMap; }
		const TOptional<bool>& GetCreateAsMap() const { return bCreateAsMap; }

		/**
		 * This flag affects both the presence of a runtime dependency and the calculation of chunk assignments.
		 * The runtime dependency is only relevant if the project's runtime or tools is reading data from the
		 * generated AssetRegistry, for e.g. validation that all dependencies are staged.
		 * The calculation of chunk assignments is only relevant if the project is using streaming manifests
		 * (multiple pak and iostore files in the staged project).
		 * If true, the generator's chunk assignment will be copied onto this generated package's chunk assignment, AND
		 * the generated package will be marked as a runtime dependency of the generator.
		 * If false, the generator's chunk assignment will instead be read from UAssetManager::GetPackageChunkIds for the
		 * name of the generated package, and the generated package will be marked as an editoronly dependency of the
		 * generator.
		 */
		bool bCopyChunkAssignmentFromGenerator = true;
	private:
		TOptional<bool> bCreateAsMap;
	};

	/** Data sent to the cooker to describe what to generate. */
	struct FGenerationManifest
	{
		/** List of generated packages. */
		TArray<FGeneratedPackage> GeneratedPackages;
		/** Whether the package containing the generator should also be saved and added to the cook's output. */
		bool bCookSaveGeneratorPackage = true;
	};

	/** Return the list of packages to generate. */
	UE_DEPRECATED(5.7, "Implement ReportGenerationManifest instead.")
	virtual TArray<FGeneratedPackage> GetGenerateList(const UPackage* OwnerPackage, const UObject* OwnerObject)
	{
		return TArray<FGeneratedPackage>();
	}

	/** Return the list of packages to generate. */
	virtual FGenerationManifest ReportGenerationManifest(const UPackage* OwnerPackage, const UObject* OwnerObject)
	{
		// This function will become pure virtual after deprecation handling is removed.
		return FGenerationManifest();
	}

	/** Representation of generated packages prepared by the cooker. */
	struct FGeneratedPackageForPopulate
	{
		/** RelativePath reported in the GenerationManifest. */
		FString RelativePath;
		/** Root reported in the GenerationManifest. */
		FString GeneratedRootPath;
		/**
		 * Non-null UPackage. Possibly an empty placeholder package, but may contain modifications
		 * that were made during PopulateGeneratorPackage. Provided so that the generator package
		 * can create import references to objects that will be stored in the generated package.
		 */
		UPackage* Package = nullptr;
		/**
		 * *GetCreateAsMap reported in GenerationManifest. The package filename extension has already been set based
		 * on this.
		 */
		bool bCreatedAsMap = false;
	};

	/**
	 * Context passed into Populate, PreSave, and PostSave functions, on the Generator package and on the Generated
	 * packages. Some functions are only applicable in certain calls, see the description of each function.
	 */
	struct FPopulateContext
	{
	public:
		explicit FPopulateContext(UE::Cook::CookPackageSplitter::FPopulateContextData& InData);

		/** The generator package being split. */
		UNREALED_API UPackage* GetOwnerPackage() const;
		/** The SplitDataClass instance that this CookPackageSplitter instance was created for. */
		UNREALED_API UObject* GetOwnerObject() const;
		/**
		 * Placeholder UPackage and relative path information for all packages that will be generated.
		 * 
		 * This function is only available in Populate and PreSave calls. It returns an empty array during PostSave.
		 * This function is only available in calls on the generator package. It returns an empty array during calls to
		 * generated packages.
		 */
		UNREALED_API TConstArrayView<ICookPackageSplitter::FGeneratedPackageForPopulate>& GetGeneratedPackages();

		/**
		 * Returns true during calls on the generator package (e.g. PopulateGeneratorPackage). Returns false during
		 * calls on the generated packages (e.g. PopulateGeneratedPackage).
		 */
		UNREALED_API bool IsCalledOnGenerator() const;

		/**
		 * Returns the UPackage for which the event is being called. Returns the OwnerPackage if IsCalledOnGenerator,
		 * returns the targetgenerated package if !IsCalledOnGenerator.
		 * Guaranteed to not return nullptr.
		 */
		UNREALED_API UPackage* GetTargetPackage() const;

		/**
		 * Returns the FGeneratedPackageForPopulate for the package for which the event is being called.
		 * 
		 * Guaranteed to return non-null if !IsCalledOnGenerator().
		 * Returns null if IsCalledOnGenerator().
		 */
		UNREALED_API const FGeneratedPackageForPopulate* GetTargetGeneratedPackage() const;

		/**
		 * Report objects that will be moved into the Generator or Generated package during its save.
		 * This is optional - these reported objects are processed (BeginCacheForCookPlatformData) asynchronously
		 * instead of synchronously during save.
		 *
		 * This callback is only valid during Populate functions. It is ignored during PreSave and PostSave functions.
		 */
		UNREALED_API void ReportObjectToMove(UObject* Object);
		UNREALED_API void ReportObjectsToMove(TConstArrayView<UObject*> Objects);

		/**
		 * Report a package to keep referenced until the generator/generated package finishes save.
		 * When called for a Generator, if DoesGeneratedRequireGenerator() >= Populate, these will also be kept
		 * referenced until all generated packages finish saving or the splitter is destroyed.
		 *
		 * This is partially optional; the CookPackageSplitter can also manage the lifetime of the objects
		 * internally. But allowing objects necessary for the save to be garbage collected will cause performance
		 * problems and possibly errors, so either this method or some other internal method must be used.
		 * 
		 * This callback is only valid during Populate and PreSave functions. It is ignored during PostSave functions.
		 */
		UNREALED_API void ReportKeepReferencedPackage(UPackage* Package);
		UNREALED_API void ReportKeepReferencedPackages(TConstArrayView<UPackage*> Packages);

		/**
		 * Add the given FCookDependency to the build dependencies for the TargetGeneratedPackage. Incremental cooks
		 * will invalidate the package and recook it if the CookDependency changes.
		 * 
		 * This callback is only valid during Populate and PreSave functions. It is ignored during PostSave functions.
		 * This callback is only valid in calls on generated packages. It is ignored during calls to the generator.
		 */
		UNREALED_API void ReportSaveDependency(UE::Cook::FCookDependency CookDependency);
	private:
		UE::Cook::CookPackageSplitter::FPopulateContextData& Data;
	};

	/**
	 * Called before presaving the parent generator package, to give the generator a chance to inform the cooker which
	 * objects will be moved into the generator package that are not already present in it. This function will also
	 * be called before calling Populate and PreSave on generated packages, if that behavior is requested via
	 * DoesGeneratedRequireGenerator.
	 *
	 * After being called once, PopulateGeneratorPackage is guaranteed to not be called again until the splitter has
	 * been destroyed and the generator package has been garbage collected.
	 *
	 * This function will not be called if bCookSaveGeneratorPackage is false in the GenerationManifest, unless it
	 * is needed for generated packages as well via DoesGeneratedRequireGenerator.
	 *
	 * If this splitter returns false for virtual RequiresEmptyPackageBeforePopulate(GeneratedPackage), it can add or
	 * move objects into that generated package during PopulateGeneratorPackage. Doing so will prevent the targeted
	 * generated package from being garbage collected until the generator package can be destroyed. This is worse
	 * performance, but can be useful to reduce Splitter complexity.
	 * 
	 * If the splitter returns true for virtual RequiresEmptyPackageBeforePopulate(GeneratedPackage), adding or moving
	 * objects into that generated package during PopulateGeneratorPackage is an error, and those objects will be
	 * missing from the generated package when it is saved.
	 *
	 * @return True if successfully populated, false on error (this will cause a cook error).
	 */
	virtual bool PopulateGeneratorPackage(FPopulateContext& PopulateContext)
	{
		return true;
	}

	/**
	 * Called before saving the parent generator package, after PopulateGeneratorPackage but before
	 * PopulateGeneratedPackage for any generated packages. Make any required adjustments to the parent package before
	 * it is saved into the target domain.
	 * 
	 * This function will not be called if bCookSaveGeneratorPackage is false in the GenerationManifest.
	 * 
	 * @return True if successfully presaved, false on error (this will cause a cook error).
	 */
	virtual bool PreSaveGeneratorPackage(FPopulateContext& PopulateContext)
	{
		return true;
	}

	/**
	 * Called after saving the parent generator package. Undo any required adjustments to the parent package that
	 * were made in PreSaveGeneratorPackage, so that the package is once again ready for use in the editor or in
	 * future ReportGenerationManifest or PreSaveGeneratedPackage calls.

	 * This function will not be called if bCookSaveGeneratorPackage is false in the GenerationManifest.
	 */
	virtual void PostSaveGeneratorPackage(FPopulateContext& PopulateContext)
	{
	}

	/**
	 * Try to populate a generated package.
	 *
	 * Receive an empty UPackage generated from a GeneratedPackage in the GenerationManifest and populate it.
	 * Return a list of all the objects that will be moved into the Generated package during its save, so the cooker
	 * can call BeginCacheForCookedPlatformData on them before the move
	 * After returning, the given package will be queued for saving into the TargetDomain
	 * 
	 * PopulateGeneratedPackage is guaranteed to not be called again on the same generated package until the splitter
	 * has been destroyed and the generator package has been garbage collected.
	 * 
	 * @return True if successfully populated, false on error (this will cause a cook error).
	 */
	virtual bool PopulateGeneratedPackage(FPopulateContext& PopulateContext)
	{
		return true;
	}
	
	/**
	 * Called before saving a generated package, after PopulateGeneratedPackage. Make any required adjustments to the
	 * generated package before it is saved into the target domain.
	 * 
	 * @return True if successfully presaved, false on error (this will cause a cook error).
	 */
	virtual bool PreSaveGeneratedPackage(FPopulateContext& PopulateContext)
	{
		return true;
	};

	/**
	 * Called after saving a generated package. Undo any required adjustments to the parent package that
	 * were made in PreSaveGeneratedPackage, so that the parent package is once again ready for use in the editor or in
	 * future PreSaveGeneratedPackage calls.
	 */
	virtual void PostSaveGeneratedPackage(FPopulateContext& PopulateContext)
	{
	}

	/** Called when the Owner package needs to be reloaded after a garbage collect in order to populate a generated package. */
	virtual void OnOwnerReloaded(UPackage* OwnerPackage, UObject* OwnerObject) {}

	// Utility functions for Splitters

	/** The name of the _Generated_ subdirectory that is the parent directory of a splitter's generated packages. */
	UNREALED_API static const TCHAR* GetGeneratedPackageSubPath();
	/** Return true if the given path is a _Generated_ directory, or a subpath under it. */
	UNREALED_API static bool IsUnderGeneratedPackageSubPath(FStringView FileOrLongPackagePath);

	/**
	 * Return the full packagename that will be used for a GeneratedPackage, based on the GeneratorPackage's name and
	 * on the RelPath and optional GeneratedRootPath that the splitter provides in the FGeneratedPackage it reports in
	 * the GenerationManifest.
	 */
	UNREALED_API static FString ConstructGeneratedPackageName(FName OwnerPackageName, FStringView RelPath,
		FStringView GeneratedRootOverride = FStringView());


	// FGeneratedPackageForPreSave and FGeneratedPackageForPopulate were two different structs that were so far identical,
	// given separate identical definitions because we wanted to reserve the ability to change one but not the other.
	// But to simplify the API we have given up on that flexibility, and now use FGeneratedPackageForPopulate everywhere.
	UE_DEPRECATED(5.6, "Use FGeneratedPackageForPopulate instead.")
	typedef FGeneratedPackageForPopulate FGeneratedPackageForPreSave;
	UE_DEPRECATED(5.6, "Implement version that takes an FPopulateContext instead.")
	UNREALED_API virtual bool PopulateGeneratorPackage(UPackage* OwnerPackage, UObject* OwnerObject,
		const TArray<ICookPackageSplitter::FGeneratedPackageForPopulate>& GeneratedPackages,
		TArray<UObject*>& OutObjectsToMove, TArray<UPackage*>& OutKeepReferencedPackages);
	UE_DEPRECATED(5.6, "Implement version that takes an FPopulateContext instead.")
	UNREALED_API virtual bool PreSaveGeneratorPackage(UPackage* OwnerPackage, UObject* OwnerObject,
		const TArray<FGeneratedPackageForPopulate>& PlaceholderPackages, TArray<UPackage*>& OutKeepReferencedPackages);
	UE_DEPRECATED(5.6, "Implement version that takes an FPopulateContext instead.")
	UNREALED_API virtual void PostSaveGeneratorPackage(UPackage* OwnerPackage, UObject* OwnerObject);
	UE_DEPRECATED(5.6, "Implement version that takes an FPopulateContext instead.")
	UNREALED_API virtual bool PopulateGeneratedPackage(UPackage* OwnerPackage, UObject* OwnerObject,
		const FGeneratedPackageForPopulate& GeneratedPackage, TArray<UObject*>& OutObjectsToMove,
		TArray<UPackage*>& OutKeepReferencedPackages);
	UE_DEPRECATED(5.6, "Implement version that takes an FPopulateContext instead.")
	UNREALED_API virtual bool PreSaveGeneratedPackage(UPackage* OwnerPackage, UObject* OwnerObject,
		const FGeneratedPackageForPopulate& GeneratedPackage, TArray<UPackage*>& OutKeepReferencedPackages);
	UE_DEPRECATED(5.6, "Implement version that takes an FPopulateContext instead.")
	UNREALED_API virtual void PostSaveGeneratedPackage(UPackage* OwnerPackage, UObject* OwnerObject,
		const FGeneratedPackageForPopulate& GeneratedPackage);
	UE_DEPRECATED(5.6, "Deprecation support, do not call outside of cooker code.")
	UNREALED_API void WarnIfDeprecatedVirtualNotCalled(const TCHAR* FunctionName);

private:
	UE_DEPRECATED(5.7, "Deprecation support, do not read/write outside of cooker code.")
	bool bDeprecatedVirtualCalledAsExpected = false;
};

namespace UE::Cook::Private
{

/** Interface for internal use only (used by REGISTER_COOKPACKAGE_SPLITTER to register an ICookPackageSplitter for a class) */
class FRegisteredCookPackageSplitter
{
public:
	UNREALED_API FRegisteredCookPackageSplitter();
	UNREALED_API virtual ~FRegisteredCookPackageSplitter();

	virtual UClass* GetSplitDataClass() const = 0;
	virtual bool RequiresCachedCookedPlatformDataBeforeSplit() const = 0;
	virtual bool ShouldSplitPackage(UObject* Object) const = 0;
	virtual ICookPackageSplitter* CreateInstance(UPackage* OwnerPackage, UObject* OwnerObject) const = 0;
	virtual FString GetSplitterDebugName() const = 0;

	static UNREALED_API void ForEach(TFunctionRef<void(FRegisteredCookPackageSplitter*)> Func);

private:
	
	static UNREALED_API TLinkedList<FRegisteredCookPackageSplitter*>*& GetRegisteredList();
	TLinkedList<FRegisteredCookPackageSplitter*> GlobalListLink;
};

}

/**
 * Used to Register an ICookPackageSplitter for a class
 *
 * Example usage:
 *
 * // In header or cpp
 * class FMyCookPackageSplitter : public ICookPackageSplitter { ... }
 * 
 * // In cpp
 * REGISTER_COOKPACKAGE_SPLITTER(FMyCookPackageSplitter, UMySplitDataClass);
 */
#define DEFINE_COOKPACKAGE_SPLITTER(SplitterClass, SplitDataClass) \
friend class UE_JOIN(SplitterClass, SplitDataClass, _Register)

#define REGISTER_COOKPACKAGE_SPLITTER(SplitterClass, SplitDataClass) \
class UE_JOIN(SplitterClass, SplitDataClass, _Register) \
	: public UE::Cook::Private::FRegisteredCookPackageSplitter \
{ \
	virtual UClass* GetSplitDataClass() const override \
	{ \
		return SplitDataClass::StaticClass(); \
	} \
	virtual bool RequiresCachedCookedPlatformDataBeforeSplit() const override \
	{ \
		return SplitterClass::RequiresCachedCookedPlatformDataBeforeSplit(); \
	} \
	virtual bool ShouldSplitPackage(UObject* Object) const override \
	{ \
		return SplitterClass::ShouldSplit(Object); \
	} \
	virtual ICookPackageSplitter* CreateInstance(UPackage* OwnerPackage, UObject* OwnerObject) const override \
	{ \
		ICookPackageSplitter* Result = new SplitterClass(); \
		Result->Initialize(OwnerPackage, OwnerObject); \
		return Result; \
	} \
	virtual FString GetSplitterDebugName() const override \
	{ \
		return SplitterClass::GetSplitterDebugName(); \
	} \
}; \
namespace UE_JOIN(SplitterClass, SplitDataClass) \
{ \
	static UE_JOIN(SplitterClass, SplitDataClass, _Register) DefaultObject; \
}

#endif
