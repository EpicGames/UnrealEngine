// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ProfilingDebugging/CookStats.h"

#if ENABLE_COOK_STATS

#include "AnalyticsEventAttribute.h"
#include "Containers/Map.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "DerivedDataCacheUsageStats.h"

namespace CookProfiling
{
	extern const char* AssetTypesToAccumulate[];

	UNREALED_API void CollectAccumulatedDDCStats(
		TArray<FDerivedDataCacheResourceStat> DDCResourceUsageStats,
		TMap<FString, FDerivedDataCacheResourceStat>& StatsPerAssetType);

	UNREALED_API void CollectAttributesForAccumulatedDDCUsageStats(
		TMap<FString, FDerivedDataCacheResourceStat> StatsPerAssetType,
		TArray<FAnalyticsEventAttribute>& Attributes);

	UNREALED_API void GatherAccumulatedDDCStats(
		TArray<FAnalyticsEventAttribute>& Attributes);
}

#endif