// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CrashReporterSettings.h: Declares the UCrashReporterSettings class.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "CrashReporterSettings.generated.h"

/**
 * Implements per-project cooker settings exposed to the editor
 */
UCLASS(config = EditorPerProjectUserSettings, MinimalAPI)
class UCrashReporterSettings
	: public UObject
{
	GENERATED_UCLASS_BODY()

public:

	/**
	 * Directory for uploading locally built binaries and .PDB files
	 */
	UPROPERTY(EditAnywhere, config, Category = CrashReporter)
	FString UploadSymbolsPath;

	/**
	 * Local downstream PDB storage path (used for caching remote .PDB files)
	 */
	UPROPERTY(EditAnywhere, config, Category = CrashReporter)
	FString DownstreamStorage;

	/**
	 * Remote PDB storage (directory path or http/https URL)
	 */
	UPROPERTY(EditAnywhere, config, Category = CrashReporter)
	TArray<FString> RemoteStorage;
};
