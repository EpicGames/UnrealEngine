// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Text/SlateHyperlinkRun.h"
#include "Input/Reply.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FActiveTimerHandle;
class SScrollBox;
struct FSlateBrush;

/**
 * Credit screen widget that displays a scrolling list contributors.  
 */
class SCreditsScreen : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SCreditsScreen )
		{}
	SLATE_END_ARGS()

	/**
	 * Constructs the credits screen widgets                   
	 */
	UNREALED_API void Construct(const FArguments& InArgs);

private:
	/** Animates the credits during play */
	EActiveTimerReturnType RollCredits( double InCurrentTime, float InDeltaTime );

	/** Handles the user clicking the play/pause toggle button. */
	FReply HandleTogglePlayPause();

	/** Handles when the user scrolls so that we can stop the auto-scrolling when they scroll backwards. */
	void HandleUserScrolled(float ScrollOffset);

	/** Handles the user clicking links in the credits */
	void OnBrowserLinkClicked(const FSlateHyperlinkRun::FMetadata& Metadata);

	/** Gets the current brush (play or pause) icon for the play/pause button */
	const FSlateBrush* GetTogglePlayPauseBrush() const;

	/** The widget that scrolls the credits text */
	TSharedPtr<SScrollBox> ScrollBox;

	/** The handle to the active roll credits tick */
	TWeakPtr<FActiveTimerHandle> ActiveTimerHandle;

	/** The auto scroll rate in pixels per second */
	float ScrollPixelsPerSecond;

	/** The last recorded scroll position so we can detect the user scrolling up */
	float PreviousScrollPosition;

	/** If we are playing the auto scroll effect */
	bool bIsPlaying;
};

