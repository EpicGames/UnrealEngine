// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Engine/CurveTable.h"
#include "Kismet2/ListenerManager.h"
#include "UObject/NameTypes.h"

class UCurveTable;

struct FCurveTableEditorUtils
{
	enum class ECurveTableChangeInfo
	{
		/** The data corresponding to a single row has been changed */
		RowData,
		/** The data corresponding to the entire list of rows has been changed */
		RowList,
	};

	enum class ERowMoveDirection
	{
		Up,
		Down,
	};

	class FCurveTableEditorManager : public FListenerManager < UCurveTable, ECurveTableChangeInfo >
	{
		FCurveTableEditorManager() {}
	public:
		UNREALED_API static FCurveTableEditorManager& Get();

		class ListenerType : public InnerListenerType<FCurveTableEditorManager>
		{
		public:
			virtual void SelectionChange(const UCurveTable* CurveTable, FName RowName) { }
		};
	};

	typedef FCurveTableEditorManager::ListenerType INotifyOnCurveTableChanged;

	static UNREALED_API void BroadcastPreChange(UCurveTable* DataTable, ECurveTableChangeInfo Info);
	static UNREALED_API void BroadcastPostChange(UCurveTable* DataTable, ECurveTableChangeInfo Info);
};
