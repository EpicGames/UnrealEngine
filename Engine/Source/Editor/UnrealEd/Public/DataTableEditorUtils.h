// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Engine/DataTable.h"
#include "Kismet2/ListenerManager.h"
#include "Widgets/SWidget.h"
#include "Framework/Commands/UIAction.h"
#include "AssetRegistry/AssetData.h"

struct FDataTableEditorColumnHeaderData
{
	/** Unique ID used to identify this column */
	FName ColumnId;

	/** Display name of this column */
	FText DisplayName;

	/** The calculated width of this column taking into account the cell data for each row */
	float DesiredColumnWidth;

	/** The FProperty for the variable in this column */
	const FProperty* Property;
};

struct FDataTableEditorRowListViewData
{
	/** Unique ID used to identify this row */
	FName RowId;

	/** Display name of this row */
	FText DisplayName;

	/** The calculated height of this row taking into account the cell data for each column */
	float DesiredRowHeight;

	/** Insertion number of the row */
	uint32 RowNum;

	/** Array corresponding to each cell in this row */
	TArray<FText> CellData;
};

typedef TSharedPtr<FDataTableEditorColumnHeaderData> FDataTableEditorColumnHeaderDataPtr;
typedef TSharedPtr<FDataTableEditorRowListViewData>  FDataTableEditorRowListViewDataPtr;


enum class ERowInsertionPosition
{
	Above,
	Below,
	Bottom,
};

struct FDataTableEditorUtils
{
	enum class EDataTableChangeInfo
	{
		/** The data corresponding to a single row has been changed */
		RowData,
		/** The data corresponding to the entire list of rows has been changed */
		RowList,
	};

	enum class ERowMoveDirection
	{
		Up,
		Down,
	};

	class FDataTableEditorManager : public FListenerManager < UDataTable, EDataTableChangeInfo >
	{
		FDataTableEditorManager() {}
	public:
		UNREALED_API static FDataTableEditorManager& Get();

		class ListenerType : public InnerListenerType<FDataTableEditorManager>
		{
		public:
			virtual void SelectionChange(const UDataTable* DataTable, FName RowName) { }
		};
	};

	typedef FDataTableEditorManager::ListenerType INotifyOnDataTableChanged;

	static UNREALED_API bool RemoveRow(UDataTable* DataTable, FName Name);
	static UNREALED_API uint8* AddRow(UDataTable* DataTable, FName RowName);
	static UNREALED_API uint8* DuplicateRow(UDataTable* DataTable, FName SourceRowName, FName RowName);
	static UNREALED_API bool RenameRow(UDataTable* DataTable, FName OldName, FName NewName);
	static UNREALED_API bool MoveRow(UDataTable* DataTable, FName RowName, ERowMoveDirection Direction, int32 NumRowsToMoveBy = 1);
	static UNREALED_API bool SelectRow(const UDataTable* DataTable, FName RowName);
	static UNREALED_API bool DiffersFromDefault(UDataTable* DataTable, FName RowName);
	static UNREALED_API bool ResetToDefault(UDataTable* DataTable, FName RowName);

	static UNREALED_API uint8* AddRowAboveOrBelowSelection(UDataTable* DataTable, const FName& RowName, const FName& NewRowName, ERowInsertionPosition InsertPosition);

	static UNREALED_API void BroadcastPreChange(UDataTable* DataTable, EDataTableChangeInfo Info);
	static UNREALED_API void BroadcastPostChange(UDataTable* DataTable, EDataTableChangeInfo Info);

	/** Reads a data table and parses out editable copies of rows and columns */
	static UNREALED_API void CacheDataTableForEditing(const UDataTable* DataTable, TArray<FDataTableEditorColumnHeaderDataPtr>& OutAvailableColumns, TArray<FDataTableEditorRowListViewDataPtr>& OutAvailableRows);

	/** Generic version that works with any datatable-like structure */
	static UNREALED_API void CacheDataForEditing(const UScriptStruct* RowStruct, const TMap<FName, uint8*>& RowMap, TArray<FDataTableEditorColumnHeaderDataPtr>& OutAvailableColumns, TArray<FDataTableEditorRowListViewDataPtr>& OutAvailableRows);

	/** Returns all script structs that can be used as a data table row. This only includes loaded ones */
	static UNREALED_API TArray<UScriptStruct*> GetPossibleStructs();

	/** Fills in an array with all possible DataTable structs, unloaded and loaded */
	static UNREALED_API void GetPossibleStructAssetData(TArray<FAssetData>& StructAssets);
	
	/** Utility function which verifies that the specified struct type is viable for data tables */
	static UNREALED_API bool IsValidTableStruct(const UScriptStruct* Struct);

	/** Add a UI action for search for references, useful for customizations */
	static UNREALED_API void AddSearchForReferencesContextMenu(class FDetailWidgetRow& RowNameDetailWidget, FExecuteAction SearchForReferencesAction);

	/** Short description for a data or curve handle */
	static UNREALED_API FText GetHandleShortDescription(const UObject* TableAsset, FName RowName);

	/** Tooltip text for the data table row type */
	static UNREALED_API FText GetRowTypeInfoTooltipText(FDataTableEditorColumnHeaderDataPtr ColumnHeaderDataPtr);

	/** Doc excerpt name for the data table row type */
	static UNREALED_API FString GetRowTypeTooltipDocExcerptName(FDataTableEditorColumnHeaderDataPtr ColumnHeaderDataPtr);

	/** Link to variable type doc  */
	static UNREALED_API const FString VariableTypesTooltipDocLink;

	/** Delegate called when a data table struct is selected */
	DECLARE_DELEGATE_OneParam(FOnDataTableStructSelected, UScriptStruct*);

	/** Creates a combo box that allows selecting from the list of possible row structures */
	static UNREALED_API TSharedRef<SWidget> MakeRowStructureComboBox(FOnDataTableStructSelected OnSelected);
};
