// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DebugToolExec.h: Game debug tool implementation.
=============================================================================*/ 

#pragma once

#include "CoreMinimal.h"

/**
 * This class servers as an interface to debugging tools like "EDITOBJECT" which
 * can be invoked by console commands which are parsed by the exec function.
 */
class FDebugToolExec : public FExec
{
protected:
	/**
	 * Brings up a property window to edit the passed in object.
	 *
	 * @param Object	property to edit
	 * @param bShouldShowNonEditable	whether to show properties that are normally not editable under "None"
	 */
	void EditObject(UObject* Object, bool bShouldShowNonEditable);
	
	//~ Begin Exec Interface
	virtual bool Exec_Editor( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar ) override;
	//~ End Exec Interface

};
