// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Misc/Guid.h"

#include "EditorDiagnosticsColumns.generated.h"

/**
 * Column carrying an editor performance warning.
 */
USTRUCT(meta = (DisplayName = "Editor Performance Warning"))
struct FEditorPerformanceWarningColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta=(IgnoreForMemberInitializationTest))
	FGuid Id = FGuid::NewGuid();

	UPROPERTY()
	FName Instigator;

	UPROPERTY()
	FText Message;
};

/**
 * Column carrying an editor performance critical issue.
 */
USTRUCT(meta = (DisplayName = "Editor Performance Critical"))
struct FEditorPerformanceCriticalColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta=(IgnoreForMemberInitializationTest))
	FGuid Id = FGuid::NewGuid();

	UPROPERTY()
	FName Instigator;

	UPROPERTY()
	FText Message;
};