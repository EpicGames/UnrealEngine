// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"

class SModalDialogWithCheckbox;

/**
 * Opens a modal/blocking message box dialog (with an additional 'copy message text' button), and returns the result immediately
 * Internal use only. Call FMessageDialog::Open instead.
 *
 * @param InMessageType		The type of message box to display (e.g. 'ok', or 'yes'/'no' etc.)
 * @param InMessage			The message to display in the message box
 * @param InTitle			The title to display for the message box
 * @return					Returns the result of the user input
 */
UE_DEPRECATED(4.25, "OpenMsgDlgInt is deprecated, use FMessageDialog::Open instead.")
EAppReturnType::Type UNREALED_API OpenMsgDlgInt(EAppMsgType::Type InMessageType, const FText& InMessage, const FText& InTitle);

/**
 * Opens a modal/blocking message box dialog (with an additional 'copy message text' button), and returns the result immediately
 * Internal use only. Call FMessageDialog::Open instead.
 *
 * @param InMessageType		The type of message box to display (e.g. 'ok', or 'yes'/'no' etc.)
 * @param InDefaultValue	If the application is Unattended, the function will log and return DefaultValue
 * @param InMessage			The message to display in the message box
 * @param InTitle			The title to display for the message box
 * @return					Returns the result of the user input
*/
UE_DEPRECATED(4.25, "OpenMsgDlgInt is deprecated, use FMessageDialog::Open instead.")
EAppReturnType::Type UNREALED_API OpenMsgDlgInt(EAppMsgType::Type InMessageType, EAppReturnType::Type InDefaultValue, const FText& InMessage, const FText& InTitle);

DECLARE_DELEGATE_TwoParams(FOnMsgDlgResult, const TSharedRef<SWindow>&, EAppReturnType::Type);

/**
 * Opens a non-modal/non-blocking message box, which returns its result through a delegate/callback,
 * using a reference to the created window, to identify which dialog has returned a result (in case there are multiple dialog windows)
 *
 * @param InMessageType		The type of message box to display (e.g. 'ok', or 'yes'/'no' etc.)
 * @param InMessage			The message to display in the message box
 * @param InTitle			The title to display for the message box
 * @param ResultCallback	The delegate/callback instance, where results should be returned
 * @return					Returns the dialog window reference, which the calling code should store, to identify which dialog returned
 */
TSharedRef<SWindow> UNREALED_API OpenMsgDlgInt_NonModal(EAppMsgType::Type InMessageType, const FText& InMessage, const FText& InTitle,
											FOnMsgDlgResult ResultCallback);

/*-----------------------------------------------------------------------------
	FDragDropConfirmationDialog
-----------------------------------------------------------------------------*/
class FDragDropConfirmation
{
public:
	enum EResult
	{
		Folder,
		Contents,
		Cancel
	};

	UNREALED_API static EResult OpenDialog(const FString& ConfirmationTitle, const FString& Message, const FString& FolderOption, const FString& ContentsOption, const FString& CancelOption);
};

/*-----------------------------------------------------------------------------
	FSuppressableWarningDialog
-----------------------------------------------------------------------------*/
/**
 * A Dialog that displays a warning message to the user and provides the option to not display it in the future
 */
class FSuppressableWarningDialog 
{
public:

	/**
	 * Optional mode of operation for FSuppressableWarningDialog
	 */
	enum class EMode
	{
		/** Default behavior for dialog */
		Default,

		/** Dialog suppression will not persist after editor closes */
		DontPersistSuppressionAcrossSessions,

		/** Persist user result */
		PersistUserResponse
	};

	/**
	 * Struct used to initialize FSuppressableWarningDialog
	 * 
	 * User must provide confirm text, and cancel text (if using cancel button)
	 */
	struct FSetupInfo
	{
		/** Warning message displayed on the dialog */
		FText Message;

		/** Title shown at the top of the warning message window */
		FText Title;	

		/** The name of the setting which stores whether to display the warning in future */
		FString IniSettingName;

		/** The name of the file which stores the IniSettingName flag result */
		FString IniSettingFileName;

		/** If true the suppress checkbox defaults to true*/
		bool bDefaultToSuppressInTheFuture;

		/** If true suppression will not persist for future editor sessions */
		UE_DEPRECATED(5.4, "bDontPersistSuppressionAcrossSessions is deprecated, please use FSetupInfo::DialogMode instead.")
		bool bDontPersistSuppressionAcrossSessions;

		/** Optional mode of operation for FSuppressableWarningDialog */
		EMode DialogMode;

		/** Wrap message at specified length, zero or negative number will disable the wrapping */
		float WrapMessageAt;

		/** Text used on the button which will return FSuppressableWarningDialog::Confirm */
		FText ConfirmText;

		/** Text used on the button which will return FSuppressableWarningDialog::Cancel */
		FText CancelText;

		/** Test displayed next to the checkbox, defaulted to "Don't show this again" */
		FText CheckBoxText;

		/** Image used on the side of the warning, a default is provided. */
		struct FSlateBrush* Image;

		/**
		* Constructs a warning dialog setup object, used to initialize a warning dialog.
		*
		* @param Prompt					The message that appears to the user
		* @param Title					The title of the dialog
		* @param InIniSettingName		The name of the entry in the INI where the state of the "Disable this warning" check box is stored
		* @param InIniSettingFileName	The name of the INI where the state of the InIniSettingName flag is stored (defaults to GEditorPerProjectIni)
		*/
		FSetupInfo(const FText& InMessage, const FText& InTitle, const FString& InIniSettingName, const FString& InIniSettingFileName=GEditorPerProjectIni )
			: Message(InMessage)
			, Title(InTitle)
			, IniSettingName(InIniSettingName)
			, IniSettingFileName(InIniSettingFileName)
			, bDefaultToSuppressInTheFuture(false)
			, bDontPersistSuppressionAcrossSessions(false)
			, DialogMode(EMode::Default)
			, WrapMessageAt(512.0f)
			, ConfirmText()
			, CancelText()
			, CheckBoxText(NSLOCTEXT("ModalDialogs", "DefaultCheckBoxMessage", "Don't show this again"))
			, Image(NULL)
		{
		}
	};

	/** Custom return type used by ShowModal */
	enum EResult
	{
		Suppressed = -1,	// User previously suppressed dialog, in most cases this should be treated as confirm
		Cancel = 0,			// No/Cancel, normal usage would stop the current action
		Confirm = 1,		// Yes/Ok/Etc, normal usage would continue with action
	};

	/**
	 * Constructs FSuppressableWarningDialog
	 * 
	 * @param FSetupInfo Info - struct used to initialize the dialog. 
	 */
	UNREALED_API FSuppressableWarningDialog ( const FSuppressableWarningDialog::FSetupInfo& Info );
	
	/** Launches warning window, returns user response or suppressed */
	UNREALED_API EResult ShowModal() const;

private:

	/** Name of the flag which controls whether to launch the warning */
	FString IniSettingName;

	/** The name of the setting which stores the user response when dismissing the dialog. */
	FString ResponseIniSettingName;

	/** Name of the file which stores the IniSettingName flag result */
	FString IniSettingFileName;

	/** Cached warning text to output to the log if the warning is suppressed */
	FText Prompt;

	/** Cached pointer to the modal window */
	TSharedPtr<SWindow> ModalWindow;

	/** Cached pointer to the message box held within the window */
	TSharedPtr<class SModalDialogWithCheckbox> MessageBox;

	/** Optional mode of operation */
	EMode DialogMode;

	/** Set of session only suppressions */
	static TSet<FString> SuppressedInTheSession;
};


class SGenericDialogWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SGenericDialogWidget )
		: _UseScrollBox(true)
		, _ScrollBoxMaxHeight(300)
	{
	}
		
		/** Should this dialog use a scroll box for over-sized content? (default: true) */
		SLATE_ARGUMENT( bool, UseScrollBox )

		/** Max height for the scroll box (default: 300f) */
		SLATE_ARGUMENT( int32, ScrollBoxMaxHeight )

		/** Content for the dialog */
		SLATE_DEFAULT_SLOT( FArguments, Content )

		/** Called when the OK button is pressed */
		SLATE_EVENT( FSimpleDelegate, OnOkPressed )

	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs );

	/** Sets the window of this dialog. */
	void SetWindow( TSharedPtr<SWindow> InWindow )
	{
		MyWindow = InWindow;
	}

	UNREALED_API static void OpenDialog(const FText& InDialogTitle, const TSharedRef< SWidget >& DisplayContent, const FArguments& InArgs = FArguments(), bool bAsModalDialog = false);

private:
	FReply OnOK_Clicked(void);

private:
	/** Pointer to the containing window. */
	TWeakPtr< SWindow > MyWindow;

	FSimpleDelegate OkPressedDelegate;
};

namespace UE::Private
{
UNREALED_API TSharedRef<SWindow> CreateModalDialogWindow(const FText& InTitle, TSharedRef<SWidget> Contents, ESizingRule Sizing, FVector2D MinDimensions);
UNREALED_API void ShowModalDialogWindow(TSharedRef<SWindow> Window);
UNREALED_API void ShowNonModalDialogWindow(TSharedRef<SWindow> Window);
} // namespace UE::Private

/**
 * Base class for a dialog which can be shown modally and returns a user's selection after it is closed.
 */
template<typename ResultType>
class SModalEditorDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SGenericDialogWidget)
	{
	}
	SLATE_END_ARGS()

	ResultType ShowModalDialog(const FText& InTitle)
	{
		Window = UE::Private::CreateModalDialogWindow(InTitle, AsShared(), Sizing, MinDimensions);
		Window->SetWidgetToFocusOnActivate(GetWidgetToFocusOnActivate());
		TOptional<ResultType> Result;
		OnFinished.BindLambda([&Result](ResultType InResult) { Result.Emplace(MoveTemp(InResult)); });
		UE::Private::ShowModalDialogWindow(Window.ToSharedRef());
		return MoveTemp(Result.GetValue());
	}

	void ShowNonModal(const FText& InTitle, TDelegate<void(ResultType)> InOnFinished)
	{
		Window = UE::Private::CreateModalDialogWindow(InTitle, AsShared(), Sizing, MinDimensions);
		Window->SetWidgetToFocusOnActivate(GetWidgetToFocusOnActivate());
		OnFinished = InOnFinished;
		UE::Private::ShowNonModalDialogWindow(Window.ToSharedRef());
	}

	template<typename CallbackType>
	void ShowNonModal(const FText& InTitle, CallbackType InOnFinished)
	{
		Window = UE::Private::CreateModalDialogWindow(InTitle, AsShared(), Sizing, MinDimensions);
		Window->SetWidgetToFocusOnActivate(GetWidgetToFocusOnActivate());
		OnFinished.BindLambda(InOnFinished);
		UE::Private::ShowNonModalDialogWindow(Window.ToSharedRef());
	}

protected:
	// Derived classes call this function from their widget events to close the dialog and return the result to the calling context
	void ProvideResult(ResultType InResult)
	{
		Window->RequestDestroyWindow();
		Window.Reset();
		OnFinished.ExecuteIfBound(MoveTemp(InResult));
		OnFinished = {};
	}
	
	virtual TSharedPtr<SWidget> GetWidgetToFocusOnActivate() 
	{
		return {};
	}

	// Child classes can modify these
	ESizingRule Sizing = ESizingRule::Autosized;
	FVector2D MinDimensions = FVector2D(400.0f, 300.f);

private:
	TSharedPtr<SWindow> Window;
	TDelegate<void(ResultType)> OnFinished;
};

UE_DEPRECATED(4.26, "Creating groups (nested packages) is no longer supported. Use PromptUserIfExistingObject overload that does not take the Group paramater.")
UNREALED_API bool PromptUserIfExistingObject(const FString& Name, const FString& Package, const FString& Group, class UPackage* &Pkg );

UNREALED_API bool PromptUserIfExistingObject(const FString& Name, const FString& Package, class UPackage*& Pkg);

/**
* Helper method for popping up a directory dialog for the user.  OutDirectory will be 
* set to the empty string if the user did not select the OK button.
*
* @param	OutDirectory	[out] The resulting path.
* @param	Message			A message to display in the directory dialog.
* @param	DefaultPath		An optional default path.
* @return					true if the user selected the OK button, false otherwise.
*/
UNREALED_API bool PromptUserForDirectory(FString& OutDirectory, const FString& Message, const FString& DefaultPath);
