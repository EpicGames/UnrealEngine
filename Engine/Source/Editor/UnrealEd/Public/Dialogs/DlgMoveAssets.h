// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DlgMoveAssets.h: UnrealEd dialog for moving assets.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"

class SDlgMoveAsset;
class SWindow;

/** 
 * FDlgMoveAsset
 * 
 * Wrapper class for SDlgMoveAsset. This class creates and launches the dialog then awaits the
 * Result to return to the user. 
 */
class FDlgMoveAsset
{
public:

	/** 
	 * Constructs A move Asset Dialog
	 *
	 * @param bInLegacyOrMapPackage		-	Used to handle moving into or out of map packages, or legacy packages.
	 * @param InPackage					-	Package information of the current asset
	 * @param InGroup					-	Group information of the current asset
	 * @param InName					-	Name information of the current asset
	 */
	FDlgMoveAsset(bool bInLegacyOrMapPackage, const FString& InPackage, const FString& InGroup, const FString& InName, const FText& InTitle);

	/** Custom result types */
	enum EResult
	{
		Cancel = 0,
		OK,
		OKToAll,
	};

	/** 
	 * Displays the dialog in a blocking fashion
	 *
	 * returns User response in EResult form.
	 */
	EResult ShowModal();

	/** Accesses the NewPackage value */
	FString GetNewPackage() const;

	/** Accesses the NewGroup value */
	FString GetNewGroup() const;

	/** Accesses the NewName value */
	FString GetNewName() const;

private:

	/** Cached pointer to the modal window */
	TSharedPtr<SWindow> MoveAssetWindow;

	/** Cached pointer to the message box held within the window */
	TSharedPtr<class SDlgMoveAsset> MoveAssetWidget;
};
