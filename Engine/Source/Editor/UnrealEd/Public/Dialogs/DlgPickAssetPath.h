// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SWindow.h"

#define LOCTEXT_NAMESPACE "DlgPickAssetPath"

class SDlgPickAssetPath : public SWindow
{
public:
	SLATE_BEGIN_ARGS(SDlgPickAssetPath)
		: _AllowReadOnlyFolders(true)
		, _CanShowDevelopersFolder(true)
		, _ShowViewOptions(true)
		{}

		SLATE_ARGUMENT(FText, Title)
		SLATE_ARGUMENT(FText, DefaultAssetPath)

		/** If true, read only folders will be displayed */
		SLATE_ARGUMENT(bool, AllowReadOnlyFolders)
		SLATE_ARGUMENT(bool, CanShowDevelopersFolder)
		SLATE_ARGUMENT(bool, ShowViewOptions)

	SLATE_END_ARGS()

	SDlgPickAssetPath()
	:	UserResponse(EAppReturnType::Cancel)
	{
	}

	UNREALED_API void Construct(const FArguments& InArgs);

	/** Displays the dialog in a blocking fashion */
	UNREALED_API EAppReturnType::Type ShowModal();

	/** Gets the resulting asset path */
	UNREALED_API const FText& GetAssetPath();

	/** Gets the resulting asset name */
	UNREALED_API const FText& GetAssetName();

	/** Gets the resulting full asset path (path+'/'+name) */
	UNREALED_API FText GetFullAssetPath();

protected:
	void OnPathChange(const FString& NewPath);
	void OnNameChange(const FText& NewName, ETextCommit::Type CommitInfo);
	FReply OnButtonClick(EAppReturnType::Type ButtonID);

	bool ValidatePackage();

	EAppReturnType::Type UserResponse;
	FText AssetPath;
	FText AssetName;
};

#undef LOCTEXT_NAMESPACE
