// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SWindow.h"

#define LOCTEXT_NAMESPACE "DlgPickPath"

class SDlgPickPath : public SWindow
{
public:
	SLATE_BEGIN_ARGS(SDlgPickPath)
		: _AllowReadOnlyFolders(true)
		, _CanShowDevelopersFolder(true)
		, _ShowViewOptions(true)
		{}
	
		SLATE_ARGUMENT(FText, Title)
		SLATE_ARGUMENT(FText, DefaultPath)

		/** If true, read only folders will be displayed */
		SLATE_ARGUMENT(bool, AllowReadOnlyFolders)
		SLATE_ARGUMENT(bool, CanShowDevelopersFolder)
		SLATE_ARGUMENT(bool, ShowViewOptions)

	SLATE_END_ARGS()

	SDlgPickPath()
	:	UserResponse(EAppReturnType::Cancel)
	{
	}

	UNREALED_API void Construct(const FArguments& InArgs);

	/** Displays the dialog in a blocking fashion */
	UNREALED_API EAppReturnType::Type ShowModal();

	/** Gets the resulting asset path */
	UNREALED_API const FText& GetPath();

protected:
	void OnPathChange(const FString& NewPath);
	FReply OnButtonClick(EAppReturnType::Type ButtonID);

	bool ValidatePath();

	EAppReturnType::Type UserResponse;
	FText Path;
};

#undef LOCTEXT_NAMESPACE
