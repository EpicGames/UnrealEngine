// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/ArchiveUObject.h"
#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Engine/Brush.h"
#include "Model.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "Engine/Polys.h"

class FMenuBuilder;
class SWindow;
struct FReferenceTreeDataContainer;

/**
 * A struct representing a node in the reference graph
 */
struct FReferenceGraphNode 
{
	/** The object this node represents */
	UObject* Object;
	/** Links from this nodes object.  Each link represents a reference to the object. */
	TSet<FReferenceGraphNode*> Links;
	/** If the node has been visited while populating the reference tree.  This prevents circular references. */
	bool bVisited;

	FReferenceGraphNode( UObject* InObject )
		: Object( InObject ),
		  bVisited(false)
	{
	}

	/** 
	 * Returns the object that should be displayed on the graph
	 *
	 * @param bShowScriptReferences		If true we should return script objects.
	 */
	UObject* GetObjectToDisplay( bool bShowScriptReferences )
	{
		UObject* ObjectToDisplay = NULL;
		// Check to see if the object in this node is a component.  If it is try to display the actor that uses it.
		UActorComponent* Comp = Cast<UActorComponent>( Object );
		if( Comp )
		{
			if( Comp->GetOwner() )
			{
				// Use the components owner if it has one.
				ObjectToDisplay = Comp->GetOwner();
			}
			else if( Comp->GetOuter() && Comp->GetOuter()->IsA( AActor::StaticClass() ) )
			{
				// Use the components outer if it is an actor
				ObjectToDisplay = Comp->GetOuter();
			}
		}
		else if( Object->IsA( UPolys::StaticClass() ) )
		{
			// Special case handling for bsp.
			// Outer chain: Polys->UModel->ABrush
			UObject* PossibleModel = Object->GetOuter();
			if( PossibleModel && PossibleModel->IsA( UModel::StaticClass() ))
			{
				UObject* PossibleBrush = PossibleModel->GetOuter();

				if(PossibleBrush && PossibleBrush->IsA( ABrush::StaticClass() ) )
				{
					ObjectToDisplay = PossibleBrush;
				}
			}
		}
		else
		{
			ObjectToDisplay = Object;
		}

		if( ObjectToDisplay && ObjectToDisplay->HasAnyFlags( RF_ClassDefaultObject ) && !bShowScriptReferences )
		{
			// Don't return class default objects if we aren't showing script references
			ObjectToDisplay = NULL;
		}

		return ObjectToDisplay;
	}
};

typedef TMap<UObject*,FReferenceGraphNode*> FReferenceGraph;

/**
 * An archive for creating a reference graph of all UObjects
 */
class FArchiveGenerateReferenceGraph : public FArchiveUObject
{
public:
	FArchiveGenerateReferenceGraph( FReferenceGraph& OutGraph );
	FArchive& operator<<( UObject*& Object );
private:
	/** The object currently being serialized. */
	UObject* CurrentObject;
	/** The set of visited objects so we dont serialize something twice. */
	TSet< UObject* > VisitedObjects;
	/** Reference to the graph we are creating. */
	FReferenceGraph& ObjectGraph;
};

typedef TSharedPtr< struct FReferenceTreeDataContainer > FReferenceTreeItemPtr;

typedef STreeView< FReferenceTreeItemPtr > SReferenceTreeView;

class SReferenceTree : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SReferenceTree )
		: _Object(NULL)
	{}
		SLATE_ARGUMENT(UObject*, Object)
	SLATE_END_ARGS()

	UNREALED_API void Construct( const FArguments& InArgs );

	UNREALED_API ~SReferenceTree();

	static UNREALED_API TWeakPtr< SWindow > SingletonInstance;
	static UNREALED_API void OpenDialog(UObject* InObject);

private:
	/** Generates a row for the tree. */
	TSharedRef< ITableRow > OnGenerateRowForReferenceTree( FReferenceTreeItemPtr Item, const TSharedRef< STableViewBase >& OwnerTable );

	/** 
	 * Populates the tree, for a specific root object
	 *
	 * @param InRootObject			The root object to find the references of.
	 *
	 */
	void PopulateTree( UObject* RootObject );

	/** 
	 * Helper function for recursively generating the reference tree
	 *
	 * @param InNode				The node to find all references of.
	 * @param InParentNode			The generated tree node to parent all references to.
	 *
	 * @return						true if any references were found.
	 */ 
	bool PopulateTreeRecursive( FReferenceGraphNode& InNode, FReferenceTreeItemPtr InParentNode );

	/** Retrieves the children for a node in the tree. */
	void OnGetChildrenForReferenceTree( FReferenceTreeItemPtr InParent, TArray< FReferenceTreeItemPtr >& OutChildren );

	/** Builds the context menu widget. */
	TSharedPtr< SWidget > BuildMenuWidget();

	/** Fills the top menu for the window. */
	void FillViewEntries( FMenuBuilder& MenuBuilder );
	void FillOptionsEntries( FMenuBuilder& MenuBuilder );

	/** 
	 * Sets expansions states for every item in the tree.
	 *
	 * @param bInExpandItems		true if items should be expanded, false if they should be collapsed.
	 */
	void SetAllExpansionStates(bool bInExpansionState);

	/** 
	 * Helper function for recursion to set expansions states for every item in the tree.
	 *
	 * @param InNode				The node to recurse into.
	 * @param bInExpandItems		true if items should be expanded, false if they should be collapsed.
	 */
	void SetAllExpansionStates_Helper(FReferenceTreeItemPtr InNode, bool bInExpansionState);

	/** Callback when the editor's map changes. */
	void OnEditorMapChange(uint32 InMapChangeFlags);

	/** Cleans up the tree for a refresh. */
	void DestroyGraphAndTree();

	/** Called when the view properties menu option is chosen. */
	void OnMenuViewProperties( AActor* InActor );

	/** Called when the show object in editor menu option is chosen. */
	void OnMenuShowEditor( UObject* InObject );

	/** Toggles show script references and refreshes the tree. */
	void OnShowScriptReferences();

	/** Callback to see if the Show Script References should be checked. */
	bool OnShowScriptReferences_Checked() const
	{
		return bShowScriptRefs;
	}

	/** Callback when the OK button is clicked. */
	FReply OnOK_Clicked(void);

	/** Sets the window of this dialog. */
	void SetWindow( TSharedPtr<SWindow> InWindow )
	{
		MyWindow = InWindow;
	}

private:
	/** List holding the root object of the tree. */
	TArray< FReferenceTreeItemPtr > ReferenceTreeRoot;

	/** Slate Widget object for the Tree View. */
	TSharedPtr< SReferenceTreeView > ReferenceTreeView;

	/** The reference graph for all UObjects. */
	FReferenceGraph ReferenceGraph;

	/** If the tree should show script references. */
	bool bShowScriptRefs;

	/** Pointer to the containing window. */
	TWeakPtr< SWindow > MyWindow;
};
