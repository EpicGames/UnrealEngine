// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DlgBuildProgress.h: UnrealEd dialog for displaying map build progress and cancelling builds.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBorder.h"

class SBuildProgressWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SBuildProgressWidget ) {}
	SLATE_END_ARGS()

	SBuildProgressWidget();
	~SBuildProgressWidget();
	void Construct( const FArguments& InArgs );

	/**
	 * Progress Control Callbacks
	 */
	FText OnGetProgressText() const;
	FText OnGetBuildTimeText() const;
	TOptional<float> OnGetProgressFraction() const;


	/** The type of build that is occurring. */
	enum EBuildType
	{
		/** Do not know what is being built... */
		BUILDTYPE_Unknown,
		/** Geometry is being built. */
		BUILDTYPE_Geometry,
		/** Lighting is being built. */
		BUILDTYPE_Lighting,
		/** Paths are being built. */
		BUILDTYPE_Paths,
		/** HLODs are being built */
		BUILDTYPE_HLODs,
		/** Minimap is being built */
		BUILDTYPE_Minimap,
		/** Landscape Spline Meshes are being built */
		BUILDTYPE_LandscapeSplineMeshes,
		/** Texture streaming data is being built */
		BUILDTYPE_TextureStreaming, 
		/** Virtual texture data is being built */
		BUILDTYPE_VirtualTexture,
		/** All Landscape is being built*/
		BUILDTYPE_AllLandscape,
	};

	/** The various issues that can occur. */
	enum EBuildIssueType
	{
		/** A critical error has occurred. */
		BUILDISSUE_CriticalError,
		/** An error has occurred. */
		BUILDISSUE_Error,
		/** A warning has occurred. */
		BUILDISSUE_Warning
	};
	
	/**
	 *	Sets the current build type.
	 *	@param	InBuildType		The build that is occurring.
	 */
	void SetBuildType(EBuildType InBuildType);
	
	/**
	 * Updates the label displaying the current time.
	 */
	void UpdateTime();
	
	void UpdateProgressText();

	/**
	 * Sets the text that describes what part of the build we are currently on.
	 *
	 * @param StatusText	Text to set the status label to.
	 */
	void SetBuildStatusText( const FText& StatusText );

	/**
	 * Sets the build progress bar percentage.
	 *
	 *	@param ProgressNumerator		Numerator for the progress meter (its current value).
	 *	@param ProgressDenominitator	Denominiator for the progress meter (its range).
	 */
	void SetBuildProgressPercent( int32 InProgressNumerator, int32 InProgressDenominator );

	/**
	 * Records the application time in seconds; used in display of elapsed build time.
	 */
	void MarkBuildStartTime();

	/**
	 * Assembles the text containing the elapsed build time.
	 */
	FText BuildElapsedTimeText() const;

private:
	/**
	 * Callback for the Stop Build button, stops the current build.
	 */
	FReply OnStopBuild();

	/** Progress numerator */
	int32 ProgressNumerator;

	/** Progress denominator */
	int32 ProgressDenominator;

	/** Displays the elapsed time for the build */
	FText BuildStatusTime;

	/** Displays some status info about the build. */
	FText BuildStatusText;

	FText ProgressStatusText;

	/** The stop build button has been pressed*/
	bool			bStoppingBuild;

	/** Application time in seconds at which the build began. */
	FDateTime BuildStartTime;

	/** The type of build that is currently occurring. */
	EBuildType		BuildType;

	/** The warning/error/critical error counts. */
	int32				WarningCount;
	int32				ErrorCount;
	int32				CriticalErrorCount;
};
