// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "AssetRegistry/AssetData.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Types/SlateStructs.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "AssetDeleteModel.h"
#include "ContentBrowserDelegates.h"
#include "AssetThumbnail.h"

class FUICommandList;
class SCheckBox;
class SComboButton;
class SWindow;

/**
 * The dialog that appears to help users through the deletion process in the editor.
 * It helps them find references to assets being deleted and gives them options on how
 * to best handle cleaning up those remaining references.
 */
class SDeleteAssetsDialog : public SCompoundWidget
{
public:
	enum class EResult : uint8
	{
		Cancel,
		Delete,
		ForceDelete,
	};
	
	DECLARE_DELEGATE_OneParam(FOnDialogClosed, EResult);

	SLATE_BEGIN_ARGS(SDeleteAssetsDialog)
		: _Style(TEXT("DeleteAssetsDialog"))
		, _WidthOverride(FOptionalSize())
	{}
		// The style of the content reference widget (optional)
		SLATE_ARGUMENT(FName, Style)

		// The parent window hosting this dialog
		SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)

		/** When specified, the path box will request this fixed size. */
		SLATE_ATTRIBUTE(FOptionalSize, WidthOverride)

		/** Additional folders to delete */
		SLATE_ARGUMENT(TArray<FString>, FoldersToDelete)

		/** Called before this dialog is destroyed */
		SLATE_EVENT(FOnDialogClosed, OnDialogClosed)

	SLATE_END_ARGS()

public:
	virtual ~SDeleteAssetsDialog();

	// Construct a SContentReference
	void Construct( const FArguments& InArgs, TSharedRef<FAssetDeleteModel> InDeleteModel );

	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;

	// Function to be hooked up to the Cancel hotkey (Escape) for the dialog window
	void OnCancelHotkeyPressed();

	// Function to be hooked up to the Confirm hotkey (Enter) for the dialog window
	void OnConfirmHotkeyPressed();

	// Function to get the appropriate widget to focus when the parent window is activated
	TSharedPtr<SWidget> GetWidgetToFocusOnActivate();

private:
	TSharedRef<SWidget> CreateThumbnailWidget();
	TSharedRef<SWidget> MakeAssetViewForReferencerAssets();
	TSharedRef<SWidget> MakeConsolidationAssetPicker();
	TSharedRef<SWidget> BuildCantUseReplaceReferencesWidget();
	TSharedRef<SWidget> BuildReplaceReferencesWidget();
	TSharedRef<SWidget> BuildForceDeleteWidget();
	TSharedRef<SWidget> BuildProgressDialog();
	TSharedRef<SWidget> BuildDeleteDialog();

private:
	/** Active timer to tick the delete model until it reaches a "Finished" state */
	EActiveTimerReturnType TickDeleteModel( double InCurrentTime, float InDeltaTime );

	void HandleDeleteModelStateChanged( FAssetDeleteModel::EState NewState );

	/** Handler for when an asset context menu has been requested. */
	TSharedPtr<SWidget> OnGetAssetContextMenu( const TArray<FAssetData>& SelectedAssets );

	bool OnShouldConsolidationFilterAsset( const FAssetData& InAssetData ) const;
	bool OnShouldFilterAsset( const FAssetData& InAssetData ) const;
	void OnAssetSelectedFromConsolidationPicker(const FAssetData& AssetData);

	bool CanExecuteDeleteReferencers() const;
	void ExecuteDeleteReferencers();

	/** Handler for when the user double clicks, presses enter, or presses space on an asset */
	void OnAssetsActivated(const TArray<FAssetData>& ActivatedAssets, EAssetTypeActivationMethod::Type ActivationMethod);

	FReply Delete();
	FReply Cancel();
	FReply ForceDelete();
	FReply ReplaceReferences();
	void DeleteRelevantSourceContent();

	/** Gets the text to display in the on disk referencing assets section when it is empty. */
	FText GetReferencingAssetsEmptyText() const;

	/** Gets the text to display for the asset being used to replace references / consolidate. */
	FText GetConsolidateAssetName() const;

	/** Gets the text to display in the header for the 'how to proceed' section */
	FText GetHandleText() const;

	/** Get the text for the delete source content files tooltip */
	FText GetDeleteSourceContentTooltip() const;

	/** Returns the visibility of the section showing asset references on disk. */
	EVisibility GetAssetReferencesVisiblity() const;

	/** Returns the visibility of the 'Replace References' option */
	EVisibility GetReplaceReferencesVisibility() const;

	/** Returns the visibility of the 'Force Delete' option */
	EVisibility GetForceDeleteVisibility() const;

	/** Returns the visibility of the 'Delete' option */
	EVisibility GetDeleteVisibility() const;

	/** Returns the visibility of the 'Delete source content files' option */
	EVisibility GetDeleteSourceFilesVisibility() const;

	/** Returns if the 'Replace References' option should be available. */
	bool CanReplaceReferences() const;

	/** Returns if the 'Force Delete' option should be available. */
	bool CanForceDelete() const;

	/** Returns if the 'Delete' option should be available. */
	bool CanDelete() const;

	/** Gets the scanning text to display for the progress bar. */
	FText ScanningText() const;

	/** Gets the scanning progress for the progress bar */
	TOptional< float > ScanningProgressFraction() const;

	/** Gets the visibility of the memory references warning message */
	EVisibility GetReferencesVisiblity() const;

	/** Gets the visibility of the undo warning message */
	EVisibility GetUndoVisiblity() const;

	TSharedRef<ITableRow> HandleGenerateAssetRow( TSharedPtr<FPendingDelete> InItem, const TSharedRef<STableViewBase>& OwnerTable );

	TSharedRef<ITableRow> HandleGenerateFolderRow(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& InOwnerTable);

private:

	/** Whether the active timer is currently registered */
	bool bIsActiveTimerRegistered;

	/** The model used for deleting assets */
	TSharedPtr<FAssetDeleteModel> DeleteModel;

	TWeakPtr<SWindow> ParentWindow;

	// Widgets
	TSharedPtr< SBorder > RootContainer;
	TSharedPtr< SListView< TSharedPtr<FPendingDelete> > > ObjectsToDeleteList;
	TSharedPtr< SBorder > AssetReferenceNameBorderWidget;
	TSharedPtr< SComboButton > ConsolidationPickerComboButton;
	TSharedPtr< SCheckBox > DeleteSourceFilesCheckbox;
	TSharedPtr<SWidget> ForceDeleteButton;
	TSharedPtr<SWidget> DeleteButton;

	/** The selected asset we're going to consolidate the would be deleted assets into. */
	FAssetData ConsolidationAsset;

	/** The consolidation asset thumbnail */
	TSharedPtr<class FAssetThumbnail> ConsolidationAssetThumbnail;

	/** The delegate that allows us to request the currently selected assets in the On Disk References section. */
	FGetCurrentSelectionDelegate GetSelectedReferencerAssets;

	/** Command list for the context menu for the referencer assets */
	TSharedPtr< FUICommandList > ReferencerCommands;
	
	/** Additional folders to delete */
	TArray<TSharedPtr<FString>> FoldersToDelete;
	TSharedPtr<SListView<TSharedPtr<FString>>> FoldersToDeleteList;
	FOnDialogClosed OnDialogClosed;
};
