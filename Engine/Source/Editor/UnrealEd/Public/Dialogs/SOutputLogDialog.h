// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SWindow;

class SOutputLogDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SOutputLogDialog )	{}
		SLATE_ATTRIBUTE(TSharedPtr<SWindow>, ParentWindow)
		SLATE_ATTRIBUTE(FText, Header)	
		SLATE_ATTRIBUTE(FText, Log)	
		SLATE_ATTRIBUTE(FText, Footer)	
		SLATE_ATTRIBUTE(TArray<FText>, Buttons)
	SLATE_END_ARGS()

	/** Displays the modal dialog box */
	static UNREALED_API void Open(const FText& InTitle, const FText& InHeader, const FText& InLog, const FText& InFooter = FText::GetEmpty());

	/** Displays the modal dialog box with a custom set of buttons */
	static UNREALED_API int32 Open(const FText& InTitle, const FText& InHeader, const FText& InLog, const FText& InFooter, const TArray<FText>& InButtons);

	/** Constructs the dialog */
	void Construct( const FArguments& InArgs );

	/** Override behavior when a key is pressed */
	virtual	FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;

	/** Override the base method to allow for keyboard focus */
	virtual bool SupportsKeyboardFocus() const override;

	/** Compute the desired size for this dialog */
	virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;

protected:
	/** Copies the message text to the clipboard. */
	void CopyMessageToClipboard( );

private:
	/** Handles clicking a message box button. */
	FReply HandleButtonClicked( int32  );

	/** Handles clicking the 'Copy Message' button. */
	FReply HandleCopyMessageButtonClicked( );

	int32 Response;
	TSharedPtr<SWindow> ParentWindow;
	FText Header;
	FText Log;
	FText Footer;
	TArray<FText> Buttons;
	float MaxWidth;
};
