// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetPrivatizeModel.h"
#include "Widgets/SCompoundWidget.h"
#include "Types/SlateStructs.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Layout/SBorder.h"



/**
* The dialog that appears to help users through the process of marking assets private in the editor.
* It helps them find references to assets being marked private and notifies them of which references would become illegal
* and be nulled out.
*/
class SPrivateAssetsDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPrivateAssetsDialog)
	{}
		// The parent window hosting this dialog
		SLATE_ATTRIBUTE(TSharedPtr<SWindow>, ParentWindow)

	SLATE_END_ARGS()

public:
	virtual ~SPrivateAssetsDialog();

	void Construct(const FArguments& InArgs, const TSharedRef<FAssetPrivatizeModel> InPrivatizeModel);

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

private:

	TSharedRef<SWidget> BuildProgressDialog();
	TSharedRef<SWidget> BuildPrivatizeDialog();
	TSharedRef<SWidget> BuildAssetViewForReferencerAssets();

private:
	/** Active timer to tick the privatize model until it reaches a "Finished" state */
	EActiveTimerReturnType TickPrivatizeModel(double InCurrentTime, float InDeltaTime);

	/** Bound to the privatize model state change event */
	void HandlePrivatizeModelStateChanged(FAssetPrivatizeModel::EState NewState);

	/** Handles filtering the asset picker to only show the references affected by the operation */
	bool OnShouldFilterAsset(const FAssetData& InAssetData) const;

	FReply Privatize();
	FReply ForcePrivatize();
	FReply Cancel();

	FText GetWarningText() const;

	/** Gets the visibility of any warning text to show at the bottom of the dialog */
	EVisibility GetWarningTextVisibility() const;

	/** Gets the visibility of if there are any on disk assets to show */
	EVisibility GetAssetReferencesVisibility() const;

	/** Gets the scanning text to display for the progress bar */
	FText ScanningText() const;

	/** Gets the scanning progress for the progress bar */
	TOptional<float> ScanningProgressFraction() const;

	/** Handles generating the columns per pending private asset */
	TSharedRef<ITableRow> HandleGenerateAssetRow(TSharedPtr<FPendingPrivateAsset> InItem, const TSharedRef<STableViewBase>& OwnerTable);

private:
	/** Whether the active timer is currently registered */
	bool bIsActiveTimerRegistered;

	/** The model used for privatizing assets */
	TSharedPtr<FAssetPrivatizeModel> PrivatizeModel;

	/** Attributes*/
	TAttribute<TSharedPtr<SWindow>> ParentWindow;

	/** Widgets */
	TSharedPtr<SBorder> RootContainer;
	TSharedPtr<SListView<TSharedPtr<FPendingPrivateAsset>>> ObjectsToPrivatizeList;
};