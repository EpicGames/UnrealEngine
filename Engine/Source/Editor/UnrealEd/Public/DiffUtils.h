// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "PropertyPath.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakFieldPtr.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/STreeView.h"
#include "PropertyHandle.h"

class FBlueprintDifferenceTreeEntry;
class ISourceControlRevision;
class SWidget;
class UBlueprint;
class UObject;
class UStruct;
struct FRevisionInfo;
template <typename ItemType> class STreeView;

struct FResolvedProperty
{
	explicit FResolvedProperty()
		: Object(nullptr)
		, Property(nullptr)
	{
	}

	FResolvedProperty(const void* InObject, const FProperty* InProperty)
		: Object(InObject)
		, Property(InProperty)
	{
	}

	inline bool operator==(const FResolvedProperty& RHS) const
	{
		return Object == RHS.Object && Property == RHS.Property;
	}

	inline bool operator!=(const FResolvedProperty& RHS) const { return !(*this == RHS); }

	const void* Object;
	const FProperty* Property;
};

/**
 * FPropertySoftPath is a string of identifiers used to identify a single member of a UObject. It is primarily
 * used when comparing unrelated UObjects for Diffing and Merging, but can also be used as a key select
 * a property in a SDetailsView.
 */
struct FPropertySoftPath
{
	UNREALED_API FPropertySoftPath();
	UNREALED_API FPropertySoftPath(const FProperty* Property);
	UNREALED_API FPropertySoftPath(TArray<FName> InPropertyChain);
	UNREALED_API FPropertySoftPath(FPropertyPath InPropertyPath);
	UNREALED_API FPropertySoftPath(const FPropertySoftPath& MainPropertyPath, const FPropertySoftPath& SubPropertyPath);
	UNREALED_API FPropertySoftPath(const FPropertySoftPath& SubPropertyPath, const FProperty* LeafProperty);
	UNREALED_API FPropertySoftPath(const FPropertySoftPath& SubPropertyPath, int32 ContainerIndex);

	UNREALED_API FResolvedProperty Resolve(const UObject* Object) const;
	UNREALED_API FResolvedProperty Resolve(const UStruct* Struct, const void* StructData) const;
	UNREALED_API FPropertyPath ResolvePath(const UObject* Object) const;

	UNREALED_API FPropertySoftPath GetRootProperty(int32 Depth) const;
	UNREALED_API int32 TryReadIndex(int32 Index) const;

	/**
	 * Provides a string of all elements in the property chain using the following format
	 * e.g., "X[3] Y Z"
	 * @param NumberOfElements Optional argument to return a string containing only the last 'NumberOfElements' elements of the path
	 * @return String of all elements in the property chain
	 */
	UNREALED_API FString ToDisplayName(const int32 NumberOfElements = INDEX_NONE) const;

	/**
	 * Indicates whether a given path is a base path of the current path.
	 * @param PotentialBasePropertyPath The path to look for as a base common path
	 * @return Whether the path has the provided path as base path or not
	 */
	bool IsSubPropertyMatch(const FPropertySoftPath& PotentialBasePropertyPath) const
	{
		if (PropertyChain.Num() <= PotentialBasePropertyPath.PropertyChain.Num())
		{
			return false;
		}

		for (int32 CurChainElement = 0; CurChainElement < PotentialBasePropertyPath.PropertyChain.Num(); CurChainElement++)
		{
			if (PotentialBasePropertyPath.PropertyChain[CurChainElement] != PropertyChain[CurChainElement])
			{
				return false;
			}
		}

		return true;
	}

	/**
	 * Indicates whether a given list of property names is a base path of the current path.
	 * @param PotentialBasePropertyNameArray The list of property names to look for as a base common path
	 * @return Whether the path has the provided list of names as base path or not
	 */
	bool IsSubPropertyMatch(TConstArrayView<FName> PotentialBasePropertyNameArray) const
	{
		if (PropertyChain.Num() <= PotentialBasePropertyNameArray.Num())
		{
			return false;
		}

		for (int32 CurChainElement = 0; CurChainElement < PotentialBasePropertyNameArray.Num(); CurChainElement++)
		{
			if (PotentialBasePropertyNameArray[CurChainElement] != PropertyChain[CurChainElement].PropertyName)
			{
				return false;
			}
		}

		return true;
	}

	/**
	 * Indicates whether a given property name is a base path of the current path.
	 * @param PotentialBasePropertyName Name of the single property to match
	 * @return Whether the path contains a single element with the name provided or not
	 */
	bool IsSubPropertyMatch(const FName PotentialBasePropertyName) const
	{
		return IsSubPropertyMatch(MakeArrayView({PotentialBasePropertyName}));
	}

	inline bool operator==(FPropertySoftPath const& RHS) const
	{
		return PropertyChain == RHS.PropertyChain;
	}

	inline bool operator!=(FPropertySoftPath const& RHS) const
	{
		return !(*this == RHS);
	}

	inline FPropertySoftPath& operator+=(FPropertySoftPath const& RHS)
	{
		PropertyChain.Append(RHS.PropertyChain);
		return *this;
	}

private:
	struct FChainElement
	{
		// FName of the property
		FName PropertyName;

		// Display string of the property
		FString DisplayString;

		FChainElement(FName InPropertyName, const FString& InDisplayString = FString())
			: PropertyName(InPropertyName)
			, DisplayString(InDisplayString)
		{
			if (DisplayString.IsEmpty())
			{
				DisplayString = PropertyName.ToString();
			}
		}

		FChainElement(const FProperty* Property)
		{
			if (Property)
			{
				PropertyName = Property->GetFName();
				DisplayString = Property->GetAuthoredName();
			}
		}

		inline bool operator==(FChainElement const& RHS) const
		{
			return PropertyName == RHS.PropertyName;
		}

		inline bool operator!=(FChainElement const& RHS) const
		{
			return !(*this == RHS);
		}
	};
	static int32 TryReadIndex(const TArray<FChainElement>& LocalPropertyChain, int32& OutIndex);

	friend uint32 GetTypeHash( FPropertySoftPath const& Path );
	TArray<FChainElement> PropertyChain;
	const UStruct* RootTypeHint;
};

struct FSCSIdentifier
{
	FName Name;
	TArray< int32 > TreeLocation;
};

struct FSCSResolvedIdentifier
{
	FSCSIdentifier Identifier;
	const UObject* Object;
};

FORCEINLINE bool operator==( const FSCSIdentifier& A, const FSCSIdentifier& B )
{
	return A.Name == B.Name && A.TreeLocation == B.TreeLocation;
}

FORCEINLINE bool operator!=(const FSCSIdentifier& A, const FSCSIdentifier& B)
{
	return !(A == B);
}

FORCEINLINE uint32 GetTypeHash( FPropertySoftPath const& Path )
{
	uint32 Ret = 0;
	for (const FPropertySoftPath::FChainElement& PropertyElement : Path.PropertyChain)
	{
		Ret = Ret ^ GetTypeHash(PropertyElement.PropertyName);
	}
	return Ret;
}

// Trying to restrict us to this typedef because I'm a little skeptical about hashing FPropertySoftPath safely
typedef TSet< FPropertySoftPath > FPropertySoftPathSet;

namespace EPropertyDiffType
{
	enum Type
	{
		Invalid,

		PropertyAddedToA,
		PropertyAddedToB,
		PropertyValueChanged,
	};
}

struct FSingleObjectDiffEntry
{
	FSingleObjectDiffEntry()
		: Identifier()
		, DiffType(EPropertyDiffType::Invalid)
	{
	}

	FSingleObjectDiffEntry(const FPropertySoftPath& InIdentifier, EPropertyDiffType::Type InDiffType )
		: Identifier(InIdentifier)
		, DiffType(InDiffType)
	{
	}

	FPropertySoftPath Identifier;
	EPropertyDiffType::Type DiffType;
};

namespace ETreeDiffType
{
	enum Type
	{
		NODE_ADDED,
		NODE_REMOVED,
		NODE_TYPE_CHANGED,
		NODE_PROPERTY_CHANGED,
		NODE_MOVED,
		NODE_CORRUPTED,
		NODE_FIXED
		/** We could potentially try to identify hierarchy reorders separately from add/remove */
	};
}

struct FSCSDiffEntry
{
	FSCSDiffEntry( const FSCSIdentifier& InIdentifier, ETreeDiffType::Type InDiffType, const FSingleObjectDiffEntry& InPropertyDiff )
		: TreeIdentifier(InIdentifier)
		, DiffType(InDiffType)
		, PropertyDiff(InPropertyDiff)
	{
	}

	FSCSIdentifier TreeIdentifier;
	ETreeDiffType::Type DiffType;
	FSingleObjectDiffEntry PropertyDiff;
};

struct FSCSDiffRoot
{
	// use indices in FSCSIdentifier::TreeLocation to find hierarchy..
	TArray< FSCSDiffEntry > Entries;
};

namespace DiffUtils
{
	DECLARE_DELEGATE_OneParam(FOnGenerateCustomDiffEntries, TArray<FSingleObjectDiffEntry>&)
	DECLARE_DELEGATE_RetVal_TwoParams(TSharedRef<SWidget>, FOnGenerateCustomDiffEntryWidget, const FSingleObjectDiffEntry&, FText&)
	DECLARE_DELEGATE_FourParams(FOnOrganizeDiffEntries,
		TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>&,
		const TArray<FSingleObjectDiffEntry>&,
		TFunctionRef<TSharedPtr<FBlueprintDifferenceTreeEntry>(const FSingleObjectDiffEntry&)>,
		TFunctionRef<TSharedPtr<FBlueprintDifferenceTreeEntry>(FText&)>)

	struct FDiffParameters
	{
		FDiffParameters() = default;
		explicit FDiffParameters(const FPropertySoftPath& RootPath)
			: RootPath(RootPath)
		{
		}

		FPropertySoftPath RootPath;

		/** Predicate evaluated on visited properties to determine if they should be part of the comparison. */
		TFunction<bool(const FProperty&)> ShouldIgnorePropertyPredicate;

		/** Indicates if the elements of a static array should be compared. Otherwise, only the size is compared. */
		bool bShouldDiffArrayElements = true;
	};

	UNREALED_API const UObject* GetCDO(const UBlueprint* ForBlueprint);
	UE_DEPRECATED(5.3, "DiffUtils now requires root objects so that object topology can be meaningfully compared.")
	UNREALED_API void CompareUnrelatedStructs(const UStruct* StructA, const void* A, const UStruct* StructB, const void* B, TArray<FSingleObjectDiffEntry>& OutDifferingProperties);
	UNREALED_API void CompareUnrelatedStructs(const UStruct* StructA, const void* A, const UObject* OwningOuterA, const UStruct* StructB, const void* B, const
	                                          UObject* OwningOuterB, TArray<FSingleObjectDiffEntry>& OutDifferingProperties);
	UNREALED_API void CompareUnrelatedObjects(const UObject* A, const UObject* B, TArray<FSingleObjectDiffEntry>& OutDifferingProperties);
	UNREALED_API void CompareUnrelatedSCS(const UBlueprint* Old, const TArray< FSCSResolvedIdentifier >& OldHierarchy, const UBlueprint* New, const TArray< FSCSResolvedIdentifier >& NewHierarchy, FSCSDiffRoot& OutDifferingEntries );

	UE_DEPRECATED(5.3, "DiffUtils now requires root objects so that object topology can be meaningfully compared.")
	UNREALED_API bool Identical(const FResolvedProperty& AProp, const FResolvedProperty& BProp, const FPropertySoftPath& RootPath, TArray<FPropertySoftPath>& DifferingProperties);

	UE_DEPRECATED(5.5, "Use version with FDiffParameters instead.")
	UNREALED_API bool Identical(const FResolvedProperty& AProp, const FResolvedProperty& BProp, const UObject* OwningOuterA, const UObject* OwningOuterB, const FPropertySoftPath& RootPath, TArray<FPropertySoftPath>& DifferingProperties);

	/**
	 * DiffUtils now requires root objects so that object topology can be meaningfully compared.
	 * DiffUtils::Identical works similar to FProperty::Identical except when a UObject is found, that is in OwningOuter*,
	 * it's compared by topology instead. This allows sub-objects to diff correctly.
	 */
	UNREALED_API bool Identical(const FResolvedProperty& AProp, const FResolvedProperty& BProp, const UObject* OwningOuterA, const UObject* OwningOuterB);
	UNREALED_API bool Identical(const FResolvedProperty& AProp, const FResolvedProperty& BProp, const UObject* OwningOuterA, const UObject* OwningOuterB, FDiffParameters DiffParameters, TArray<FPropertySoftPath>& DifferingProperties);
	UNREALED_API bool Identical(const TSharedPtr<IPropertyHandle>& PropertyHandleA, const TSharedPtr<IPropertyHandle>& PropertyHandleB, const TArray<TWeakObjectPtr<UObject>>& OwningOutersA = {}, const TArray<TWeakObjectPtr<UObject>>& OwningOutersB = {});
	UNREALED_API bool Identical(TArray<FPropertySoftPath>& OutDifferingProperties, const TSharedPtr<IPropertyHandle>& PropertyHandleA, const TSharedPtr<IPropertyHandle>& PropertyHandleB, const TArray<TWeakObjectPtr<UObject>>& OwningOutersA = {}, const TArray<TWeakObjectPtr<UObject>>& OwningOutersB = {});
	UNREALED_API TArray<FPropertySoftPath> GetVisiblePropertiesInOrderDeclared(const UStruct* ForStruct, const FPropertySoftPath& Scope = FPropertySoftPath());

	UNREALED_API TArray<FPropertyPath> ResolveAll(const UObject* Object, const TArray<FPropertySoftPath>& InSoftProperties);
	UNREALED_API TArray<FPropertyPath> ResolveAll(const UObject* Object, const TArray<FSingleObjectDiffEntry>& InDifferences);

	/**
	 * @param InTempPackagePath		- filepath of the temporary uasset version (likely in /Saved/Temp/SourceControl)
	 * @param InOriginalPackagePath	- filepath of the original uasset in the content directory. Strictly speaking most assets don't need this parameter set
	 *								  but it's needed to instantiate OFPA Actors properly.
	 */
	UNREALED_API UPackage* LoadPackageForDiff(const FPackagePath& InTempPackagePath, const FPackagePath& InOriginalPackagePath);
	UNREALED_API UPackage* LoadPackageForDiff(TSharedPtr<ISourceControlRevision> Revision);
}

DECLARE_DELEGATE(FOnDiffEntryFocused);
DECLARE_DELEGATE_RetVal(TSharedRef<SWidget>, FGenerateDiffEntryWidget);

class FBlueprintDifferenceTreeEntry
{
public:
	FBlueprintDifferenceTreeEntry(FOnDiffEntryFocused InOnFocus, FGenerateDiffEntryWidget InGenerateWidget, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> > InChildren = TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >())
		: OnFocus(InOnFocus)
		, GenerateWidget(InGenerateWidget)
		, Children(InChildren) 
	{
		check( InGenerateWidget.IsBound() );
	}

	virtual ~FBlueprintDifferenceTreeEntry() = default;

	/** Displays message to user saying there are no differences */
	static UNREALED_API TSharedPtr<FBlueprintDifferenceTreeEntry> NoDifferencesEntry();

	/** Displays message to user warning that there may be undetected differences */
	static UNREALED_API TSharedPtr<FBlueprintDifferenceTreeEntry> UnknownDifferencesEntry();

	/** Create category message for the diff UI */
	static UNREALED_API TSharedPtr<FBlueprintDifferenceTreeEntry> CreateCategoryEntry(const FText& LabelText, const FText& ToolTipText, FOnDiffEntryFocused FocusCallback, const TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& Children, bool bHasDifferences);

	/** Create category message for the merge UI */
	static UNREALED_API TSharedPtr<FBlueprintDifferenceTreeEntry> CreateCategoryEntryForMerge(const FText& LabelText, const FText& ToolTipText, FOnDiffEntryFocused FocusCallback, const TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& Children, bool bHasRemoteDifferences, bool bHasLocalDifferences, bool bHasConflicts);
	
	FOnDiffEntryFocused OnFocus;
	FGenerateDiffEntryWidget GenerateWidget;
	TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> > Children;
};

namespace DiffTreeView
{
	UNREALED_API TSharedRef< STreeView<TSharedPtr< FBlueprintDifferenceTreeEntry > > > CreateTreeView(TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >* DifferencesList);
	UNREALED_API int32 CurrentDifference( TSharedRef< STreeView<TSharedPtr< FBlueprintDifferenceTreeEntry > > > TreeView, const TArray< TSharedPtr<class FBlueprintDifferenceTreeEntry> >& Differences );
	UNREALED_API void HighlightNextDifference(TSharedRef< STreeView<TSharedPtr< FBlueprintDifferenceTreeEntry > > > TreeView, const TArray< TSharedPtr<class FBlueprintDifferenceTreeEntry> >& Differences, const TArray< TSharedPtr<class FBlueprintDifferenceTreeEntry> >& RootDifferences);
	UNREALED_API void HighlightPrevDifference(TSharedRef< STreeView<TSharedPtr< FBlueprintDifferenceTreeEntry > > > TreeView, const TArray< TSharedPtr<class FBlueprintDifferenceTreeEntry> >& Differences, const TArray< TSharedPtr<class FBlueprintDifferenceTreeEntry> >& RootDifferences);
	UNREALED_API bool HasNextDifference(TSharedRef< STreeView<TSharedPtr< FBlueprintDifferenceTreeEntry > > > TreeView, const TArray< TSharedPtr<class FBlueprintDifferenceTreeEntry> >& Differences);
	UNREALED_API bool HasPrevDifference(TSharedRef< STreeView<TSharedPtr< FBlueprintDifferenceTreeEntry > > > TreeView, const TArray< TSharedPtr<class FBlueprintDifferenceTreeEntry> >& Differences);
}

struct FRevisionInfo;

namespace DiffViewUtils
{
	UNREALED_API FLinearColor LookupColor( bool bDiffers, bool bConflicts = false );
	UNREALED_API FLinearColor Differs();
	UNREALED_API FLinearColor Identical();
	UNREALED_API FLinearColor Missing();
	UNREALED_API FLinearColor Conflicting();

	UNREALED_API FText PropertyDiffMessage(FSingleObjectDiffEntry Difference, FText ObjectName);
	UNREALED_API FText SCSDiffMessage(const FSCSDiffEntry& Difference, FText ObjectName);
	UNREALED_API FText GetPanelLabel(const UObject* Asset, const FRevisionInfo& Revision, FText Label);

	UNREALED_API SHorizontalBox::FSlot::FSlotArguments Box(bool bIsPresent, FLinearColor Color);
}
