// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/DragAndDrop.h"
#include "Styling/AppStyle.h"
#include "DragAndDrop/ActorDragDropOp.h"

#define LOCTEXT_NAMESPACE "ActorDragDrop"

class FActorDragDropGraphEdOp : public FActorDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FActorDragDropGraphEdOp, FActorDragDropOp)

	enum ToolTipTextType
	{
		ToolTip_Compatible,
		ToolTip_Incompatible,
		ToolTip_MultipleSelection_Incompatible,
		ToolTip_CompatibleAttach,
		ToolTip_IncompatibleGeneric,
		ToolTip_CompatibleGeneric,
		ToolTip_CompatibleMultipleAttach,
		ToolTip_IncompatibleMultipleAttach,
		ToolTip_CompatibleDetach,
		ToolTip_CompatibleMultipleDetach
	};

	/** Set the appropriate tool tip when dragging functionality is active*/
	void SetToolTip(ToolTipTextType TextType, FText ParamText = FText::GetEmpty())
	{
		switch( TextType )
		{
		case ToolTip_Compatible:
			{
				CurrentHoverText = FText::Format( LOCTEXT("ToolTipCompatible", "'{0}' is compatible to replace object reference"), FText::FromString(Actors[0].Get()->GetActorLabel()) );
				CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
				break;
			}
		case ToolTip_Incompatible:
			{
			CurrentHoverText = FText::Format( LOCTEXT("ToolTipIncompatible", "'{0}' is not compatible to replace object reference"), FText::FromString(Actors[0].Get()->GetActorLabel()) );
				CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
				break;
			}
		case ToolTip_MultipleSelection_Incompatible:
			{
				CurrentHoverText = LOCTEXT("ToolTipMultipleSelectionIncompatible", "Cannot replace object reference when multiple objects are selected");
				CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
				break;
			}
		case ToolTip_CompatibleAttach:
			{
				CurrentHoverText = FText::Format( LOCTEXT("ToolTipCompatibleAttach", "Attach {0} to {1}"), FText::FromString(Actors[0].Get()->GetActorLabel()), ParamText);
				CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
				break;
			}
		case ToolTip_IncompatibleGeneric:
			{
				CurrentHoverText = ParamText;
				CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
				break;
			}
		case ToolTip_CompatibleGeneric:
			{
				CurrentHoverText = ParamText;
				CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
				break;
			}
		case ToolTip_CompatibleMultipleAttach:
			{
				CurrentHoverText = FText::Format( LOCTEXT("ToolTipCompatibleMultipleAttach", "Attach multiple objects to {0}"), ParamText);
				CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
				break;
			}
		case ToolTip_IncompatibleMultipleAttach:
			{
				CurrentHoverText = FText::Format( LOCTEXT("ToolTipIncompatibleMultipleAttach", "Cannot attach multiple objects to {0}"), ParamText);
				CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
				break;
			}
		case ToolTip_CompatibleDetach:
			{
				CurrentHoverText = FText::Format( LOCTEXT("ToolTipCompatibleDetach", "Detach {0} from {1}"), FText::FromString(Actors[0].Get()->GetActorLabel()), ParamText);
				CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
				break;
			}
		case ToolTip_CompatibleMultipleDetach:
			{
				CurrentHoverText = FText::Format( LOCTEXT("ToolTipCompatibleDetachMultiple", "Detach multiple objects from {0}"), ParamText);
				CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
				break;
			}
		}
	}
	
	static TSharedRef<FActorDragDropGraphEdOp> New(const TArray< TWeakObjectPtr<AActor> >& InActors)
	{
		TSharedRef<FActorDragDropGraphEdOp> Operation = MakeShareable(new FActorDragDropGraphEdOp);
		
		Operation->Init(InActors);
		Operation->SetupDefaults();

		Operation->Construct();
		return Operation;
	}
};

#undef LOCTEXT_NAMESPACE
