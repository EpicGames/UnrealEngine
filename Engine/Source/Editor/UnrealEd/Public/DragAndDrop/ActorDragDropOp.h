// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "Input/DragAndDrop.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "ClassIconFinder.h"

class FActorDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FActorDragDropOp, FDecoratedDragDropOp)

	/** Actor that we are dragging */
	TArray< TWeakObjectPtr< AActor > > Actors;

	void Init(const TArray< TWeakObjectPtr<AActor> >& InActors)
	{
		for(int32 i=0; i<InActors.Num(); i++)
		{
			if( InActors[i].IsValid() )
			{
				Actors.Add( InActors[i] );
			}
		}

		// Set text and icon
		UClass* CommonSelClass = NULL;
		CurrentIconBrush = FClassIconFinder::FindIconForActors(Actors, CommonSelClass);
		if(Actors.Num() == 0)
		{
			CurrentHoverText = NSLOCTEXT("FActorDragDropOp", "None", "None");
		}
		else if(Actors.Num() == 1)
		{
			// Find icon for actor
			const AActor* TheActor = Actors[0].Get();
			CurrentHoverText = FText::FromString(TheActor->GetActorLabel());
		}
		else
		{
			CurrentHoverText = FText::Format(NSLOCTEXT("FActorDragDropOp", "FormatActors", "{0} Actors"), FText::AsNumber(InActors.Num()));
		}

		SetupDefaults();
	}
};
