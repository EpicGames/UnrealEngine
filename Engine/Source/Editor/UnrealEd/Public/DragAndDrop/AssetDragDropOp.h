// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetRegistry/AssetData.h"
#include "AssetSelection.h"
#include "Input/DragAndDrop.h"
#include "Layout/Visibility.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "UObject/WeakInterfacePtr.h"

class FAssetThumbnail;
class IAssetFactoryInterface;
class UActorFactory;

class FAssetDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FAssetDragDropOp, FDecoratedDragDropOp)

	static UNREALED_API TSharedRef<FAssetDragDropOp> New(const FAssetData& InAssetData, 
		TScriptInterface<IAssetFactoryInterface> Factory = nullptr);

	static UNREALED_API TSharedRef<FAssetDragDropOp> New(TArray<FAssetData> InAssetData,
		TScriptInterface<IAssetFactoryInterface> Factory = nullptr);

	static UNREALED_API TSharedRef<FAssetDragDropOp> New(FString InAssetPath);

	static UNREALED_API TSharedRef<FAssetDragDropOp> New(TArray<FString> InAssetPaths);

	static UNREALED_API TSharedRef<FAssetDragDropOp> New(TArray<FAssetData> InAssetData, TArray<FString> InAssetPaths, 
		TScriptInterface<IAssetFactoryInterface> Factory = nullptr);

	//~ These overloads are redundant to the ones that use TScriptInterface<IAssetFactoryInterface>, but
	//~ we keep them temporarily in case a user just forward declared UActorFactory.
	static UNREALED_API TSharedRef<FAssetDragDropOp> New(const FAssetData& InAssetData, UActorFactory* ActorFactory);
	static UNREALED_API TSharedRef<FAssetDragDropOp> New(TArray<FAssetData> InAssetData, UActorFactory* ActorFactory);
	static UNREALED_API TSharedRef<FAssetDragDropOp> New(TArray<FAssetData> InAssetData, TArray<FString> InAssetPaths, UActorFactory* ActorFactory);

	/** @return true if this drag operation contains assets */
	bool HasAssets() const
	{
		return AssetData.Num() > 0;
	}

	/** @return true if this drag operation contains asset paths */
	bool HasAssetPaths() const
	{
		return AssetPaths.Num() > 0;
	}

	/** @return The assets from this drag operation */
	const TArray<FAssetData>& GetAssets() const
	{
		return AssetData;
	}

	/** @return The asset paths from this drag operation */
	const TArray<FString>& GetAssetPaths() const
	{
		return AssetPaths;
	}

	//~ TODO: UE_DEPRECATED(5.4, "Use GetAssetFactory instead.")
	UNREALED_API UActorFactory* GetActorFactory() const;

	/** @return The asset factory to use if converting this asset to an actor */
	UNREALED_API TScriptInterface<IAssetFactoryInterface> GetAssetFactory() const;

public:
	UNREALED_API virtual ~FAssetDragDropOp();

	UNREALED_API virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;

	UNREALED_API FText GetDecoratorText() const;

protected:
	UNREALED_API void Init(TArray<FAssetData> InAssetData, TArray<FString> InAssetPaths, TScriptInterface<IAssetFactoryInterface> InAssetFactory);
	//TODO: Would be nice to remove this, but users could have just forward declared UActorFactory...
	UNREALED_API void Init(TArray<FAssetData> InAssetData, TArray<FString> InAssetPaths, UActorFactory* InActorFactory);
	UNREALED_API virtual void InitThumbnail();

	UNREALED_API virtual bool HasFiles() const;
	UNREALED_API virtual bool HasFolders() const;

	UNREALED_API virtual int32 GetTotalCount() const;
	UNREALED_API virtual FText GetFirstItemText() const;

	/** Set a custom thumbnail widget to use instead of the default one */
	UNREALED_API void SetCustomThumbnailWidget(const TSharedRef<SWidget>& InNewThumbnailWidget);

private:
	/** Data for the assets this item represents */
	TArray<FAssetData> AssetData;

	/**
	 * Cached asset data with refreshed tags from the asset registry.  The FAssetData captured at drag start may have stale
	 * on-disk (Fixed) tags that are missing dynamically computed values (e.g. IsSourceValid).  After the asset is loaded
	 * during drag-over, the registry has fresh Loose tags.  Populated once per drag op by ExtractAssetDataFromDrag.
	 */
	mutable TOptional<TArray<FAssetData>> RefreshedAssetData;

	/** Allow access to RefreshedAssetData from ExtractAssetDataFromDrag. */
	friend TArray<FAssetData> AssetUtil::ExtractAssetDataFromDrag(const TSharedPtr<const FDragDropOperation>& Operation);

	/** Data for the asset paths this item represents */
	TArray<FString> AssetPaths;

	/** Custom thumbnail widget */
	TAttribute<TSharedPtr<SWidget>> CustomThumbnailWidget;

	/** The factory to use if converting this asset to a placed object */
	TWeakInterfacePtr<IAssetFactoryInterface> AssetFactory;

protected:
	/** Handle to the thumbnail resource */
	TSharedPtr<FAssetThumbnail> AssetThumbnail;

	/** The size of the thumbnail */
	int32 ThumbnailSize = 0;
};
