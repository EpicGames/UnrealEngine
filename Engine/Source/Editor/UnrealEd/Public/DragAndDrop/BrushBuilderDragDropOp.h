// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/DragAndDrop.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/AppStyle.h"
#include "Engine/BrushBuilder.h"
#include "Editor.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "LevelUtils.h"

class FBrushBuilderDragDropOp : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FBrushBuilderDragDropOp, FDragDropOperation)

	static TSharedRef<FBrushBuilderDragDropOp> New(TWeakObjectPtr<UBrushBuilder> InBrushBuilder, const FSlateBrush* InIconBrush, bool bInIsAddtive)
	{
		TSharedRef<FBrushBuilderDragDropOp> Operation = MakeShareable(new FBrushBuilderDragDropOp(InBrushBuilder, InIconBrush, bInIsAddtive));

		Operation->MouseCursor = EMouseCursor::GrabHandClosed;

		Operation->Construct();
		return Operation;
	}

	~FBrushBuilderDragDropOp()
	{
		UWorld* World = GEditor->GetEditorWorldContext().World();
		if ((World != nullptr) && (World->GetDefaultBrush() != nullptr))
		{
			// Hide the builder brush
			World->GetDefaultBrush()->SetIsTemporarilyHiddenInEditor(true);
		}
	}

	virtual void OnDrop( bool bDropWasHandled, const FPointerEvent& MouseEvent ) override
	{
		UWorld* World = GEditor->GetEditorWorldContext().World();
		if ( (World != nullptr) && (World->GetDefaultBrush() != nullptr))
		{
			ULevel* CurrentLevel = World->GetCurrentLevel();
			if (CurrentLevel != nullptr && !FLevelUtils::IsLevelLocked(CurrentLevel))
			{
				// Cut the BSP
				if (bDropWasHandled)
				{
					World->GetDefaultBrush()->BrushBuilder = DuplicateObject<UBrushBuilder>(BrushBuilder.Get(), World->GetDefaultBrush()->GetOuter());
					const TCHAR* Command = bIsAdditive ? TEXT("BRUSH ADD SELECTNEWBRUSH") : TEXT("BRUSH SUBTRACT SELECTNEWBRUSH");
					GEditor->Exec(World, Command);
				}

				// Hide the builder brush
				World->GetDefaultBrush()->SetIsTemporarilyHiddenInEditor(true);
			}
		}

		FDragDropOperation::OnDrop( bDropWasHandled, MouseEvent );
	}

	virtual void OnDragged( const FDragDropEvent& DragDropEvent ) override
	{
		UWorld* World = GEditor->GetEditorWorldContext().World();
		if ((World != nullptr) && (World->GetDefaultBrush() != nullptr))
		{
			if(CursorDecoratorWindow->IsVisible() && !World->GetDefaultBrush()->IsTemporarilyHiddenInEditor())
			{
				// Hide the builder brush if the decorator is visible
				World->GetDefaultBrush()->SetIsTemporarilyHiddenInEditor(true);
			}
		}

		FDragDropOperation::OnDragged(DragDropEvent);
	}

	virtual TSharedPtr<class SWidget> GetDefaultDecorator() const override
	{
		return SNew(SBox)
			.WidthOverride(100.0f)
			.HeightOverride(100.0f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("AssetThumbnail.ClassBackground"))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(IconBrush)
				]
			];
	}

	TWeakObjectPtr<UBrushBuilder> GetBrushBuilder() const
	{
		return BrushBuilder;
	}

private:
	FBrushBuilderDragDropOp(TWeakObjectPtr<UBrushBuilder> InBrushBuilder, const FSlateBrush* InIconBrush, bool bInIsAdditive)
		: BrushBuilder(InBrushBuilder)
		, IconBrush(InIconBrush)
		, bIsAdditive(bInIsAdditive)
	{
		// show the builder brush
		UWorld* World = GEditor->GetEditorWorldContext().World();
		if ((World != nullptr) && (World->GetDefaultBrush() != nullptr))
		{
			World->GetDefaultBrush()->SetIsTemporarilyHiddenInEditor(false);
		}
	}

private:

	/** The brush builder */
	TWeakObjectPtr<UBrushBuilder> BrushBuilder;

	/** The icon to display when dragging */
	const FSlateBrush* IconBrush;

	/** Additive or subtractive mode */
	bool bIsAdditive;
};
