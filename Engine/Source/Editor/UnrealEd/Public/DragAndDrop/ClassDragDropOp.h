// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Input/DragAndDrop.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/AppStyle.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Styling/SlateIconFinder.h"

class FClassDragDropOp : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FClassDragDropOp, FDragDropOperation)

	/** The classes to be dropped. */
	TArray< TWeakObjectPtr<UClass> > ClassesToDrop;
	
	/** The widget decorator to use */
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
	{
		// Just use the first class for the cursor decorator.
		const FSlateBrush* ClassIcon = FSlateIconFinder::FindIconBrushForClass( ClassesToDrop[0].Get() );

		// If the class icon is the default brush, do not put it in the cursor decoration window.
		if(ClassIcon)
		{
			return SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("Graph.ConnectorFeedback.Border"))
				.Content()
				[			
					SNew(SHorizontalBox)
					
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew( SImage )
						.Image( ClassIcon )
					]
					
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock) 
						.Text( FText::FromString(ClassesToDrop[0]->GetName()) )
					]
				];
		}

		return SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Graph.ConnectorFeedback.Border"))
			.Content()
			[			
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock) 
					.Text( FText::FromString(ClassesToDrop[0]->GetName()) )
				]
			];
	}

	static TSharedRef<FClassDragDropOp> New(TWeakObjectPtr<UClass> ClassToDrop)
	{
		TSharedRef<FClassDragDropOp> Operation = MakeShareable(new FClassDragDropOp);
		Operation->ClassesToDrop.Add(ClassToDrop);
		Operation->Construct();
		return Operation;
	}

protected:
	/** Adding hint text to all drag drop operations that drop locations could set, and the decorator could report. */
	FText HintText;
};
