// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "AssetTagItemTypes.h"
#include "CollectionManagerTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "GenericPlatform/ICursor.h"
#include "HAL/PlatformCrt.h"
#include "Input/DragAndDrop.h"
#include "Internationalization/Text.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class ICollectionContainer;
class SWidget;
struct FAssetData;

class FCollectionDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FCollectionDragDropOp, FDecoratedDragDropOp)

	/** Data for the collections this item represents */
	TArray<FCollectionRef> CollectionRefs;

	UE_DEPRECATED(5.6, "Use CollectionRefs instead.")
	TArray<FCollectionNameType> Collections;

	static UNREALED_API TSharedRef<FCollectionDragDropOp> New(TArray<FCollectionRef> InCollectionRefs, const EAssetTagItemViewMode InAssetTagViewMode = EAssetTagItemViewMode::Standard);

	UE_DEPRECATED(5.6, "Use the FCollectionRef overload instead.")
	static UNREALED_API TSharedRef<FCollectionDragDropOp> New(TArray<FCollectionNameType> InCollections, const EAssetTagItemViewMode InAssetTagViewMode = EAssetTagItemViewMode::Standard);
	
public:
	/** @return The assets from this drag operation */
	UNREALED_API TArray<FAssetData> GetAssets() const;

	UNREALED_API virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;

private:
	UNREALED_API FText GetDecoratorText() const;

	EAssetTagItemViewMode AssetTagViewMode = EAssetTagItemViewMode::Standard;
};
