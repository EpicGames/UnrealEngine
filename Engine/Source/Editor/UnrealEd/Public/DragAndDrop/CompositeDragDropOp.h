// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/DragAndDrop.h"
#include "DragAndDrop/DecoratedDragDropOp.h"

class FCompositeDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FCompositeDragDropOp, FDecoratedDragDropOp)

	FCompositeDragDropOp() : FDecoratedDragDropOp() {}

	void AddSubOp(const TSharedPtr<FDragDropOperation>& SubOp)
	{
		check(!SubOp->IsOfType<FCompositeDragDropOp>());
		SubOps.Add(SubOp);
	}

	template <typename T>
	bool HasSubOp()
	{
		for (auto& SubOp : SubOps)
		{
			if (SubOp->IsOfType<T>())
			{
				return true;
			}
		}
		return false;
	}

	template <typename T>
	TSharedPtr<T> GetSubOp()
	{
		for (auto& SubOp : SubOps)
		{
			if (SubOp->IsOfType<T>())
			{
				return StaticCastSharedPtr<T>(SubOp);
			}
		}
		return nullptr;
	}

	template <typename T>
	const TSharedPtr<const T> GetSubOp() const
	{
		for (const auto& SubOp : SubOps)
		{
			if (SubOp->IsOfType<T>())
			{
				return StaticCastSharedPtr<const T>(SubOp);
			}
		}
		return nullptr;
	}

	virtual void ResetToDefaultToolTip() override
	{
		FDecoratedDragDropOp::ResetToDefaultToolTip();

		for (const auto& SubOp : SubOps)
		{
			if (SubOp->IsOfType<FDecoratedDragDropOp>())
			{
				auto DecoratedSubOp = StaticCastSharedPtr<FDecoratedDragDropOp>(SubOp);
				if (DecoratedSubOp.IsValid())
				{
					DecoratedSubOp->ResetToDefaultToolTip();
				}
			}
		}
	}
private:
	TSharedPtr<FDragDropOperation> GetSubOpPtr(const FString& TypeId) const
	{
		for (const auto& SubOp : SubOps)
		{
			if (SubOp->IsOfTypeImpl(TypeId))
			{
				return SubOp;
			}
		}
		return nullptr;
	}
protected:
	TArray<TSharedPtr<FDragDropOperation>> SubOps;
};