// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/DragAndDrop.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/AppStyle.h"

class FDecoratedDragDropOp : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FDecoratedDragDropOp, FDragDropOperation)

	/** String to show as hover text */
	FText								CurrentHoverText;

	/** Icon to be displayed */
	const FSlateBrush*					CurrentIconBrush;

	/** The color of the icon to be displayed. */
	FSlateColor							CurrentIconColorAndOpacity;

	FDecoratedDragDropOp()
		: CurrentIconBrush(nullptr)
		, CurrentIconColorAndOpacity(FLinearColor::White)
		, DefaultHoverIcon(nullptr)
		, DefaultHoverIconColorAndOpacity(FLinearColor::White)
	{ }

	/** Overridden to provide public access */
	virtual void Construct() override
	{
		FDragDropOperation::Construct();
	}

	/** Set the decorator back to the icon and text defined by the default */
	virtual void ResetToDefaultToolTip()
	{
		CurrentHoverText = DefaultHoverText;
		CurrentIconBrush = DefaultHoverIcon;
		CurrentIconColorAndOpacity = DefaultHoverIconColorAndOpacity;
	}

	/** The widget decorator to use */
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
	{
		// Create hover widget
		return SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Graph.ConnectorFeedback.Border"))
			.Content()
			[			
				SNew(SHorizontalBox)
				
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding( 0.0f, 0.0f, 3.0f, 0.0f )
				.VAlign( VAlign_Center )
				[
					SNew( SImage )
					.Image( this, &FDecoratedDragDropOp::GetIcon )
					.ColorAndOpacity( this, &FDecoratedDragDropOp::GetIconColorAndOpacity)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign( VAlign_Center )
				[
					SNew(STextBlock) 
					.Text( this, &FDecoratedDragDropOp::GetHoverText )
				]
			];
	}

	FText GetHoverText() const
	{
		return CurrentHoverText;
	}

	const FSlateBrush* GetIcon( ) const
	{
		return CurrentIconBrush;
	}

	FSlateColor GetIconColorAndOpacity() const
	{
		return CurrentIconColorAndOpacity;
	}

	/** Set the text and icon for this tooltip */
	void SetToolTip(const FText& Text, const FSlateBrush* Icon)
	{
		CurrentHoverText = Text;
		CurrentIconBrush = Icon;
	}

	/** Setup some default values for the decorator */
	void SetupDefaults()
	{
		DefaultHoverText = CurrentHoverText;
		DefaultHoverIcon = CurrentIconBrush;
		DefaultHoverIconColorAndOpacity = CurrentIconColorAndOpacity;
	}

	/** Gets the default hover text for this drag drop op. */
	FText GetDefaultHoverText() const
	{
		return DefaultHoverText;
	}

protected:

	/** Default string to show as hover text */
	FText								DefaultHoverText;

	/** Default icon to be displayed */
	const FSlateBrush*					DefaultHoverIcon;

	/** Default color and opacity for the default icon to be displayed. */
	FSlateColor							DefaultHoverIconColorAndOpacity;
};
