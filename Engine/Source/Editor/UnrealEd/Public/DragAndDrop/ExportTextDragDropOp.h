// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "GameFramework/Actor.h"
#include "Input/DragAndDrop.h"
#include "Editor.h"
#include "Misc/StringOutputDevice.h"
#include "UObject/UObjectIterator.h"
#include "UObject/PropertyPortFlags.h"
#include "Exporters/Exporter.h"
#include "UnrealExporter.h"

class FExportTextDragDropOp : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FExportTextDragDropOp, FDragDropOperation)

	FString ActorExportText;
	int32 NumActors;

	static TSharedRef<FExportTextDragDropOp> New(const TArray<AActor*>& InActors)
	{
		TSharedRef<FExportTextDragDropOp> Operation = MakeShareable(new FExportTextDragDropOp);

		FStringOutputDevice Ar;
		const FSelectedActorExportObjectInnerContext Context(InActors);
		UExporter::ExportToOutputDevice( &Context, GWorld, NULL, Ar, TEXT("copy"), 0, PPF_DeepCompareInstances | PPF_ExportsNotFullyQualified);
		Operation->ActorExportText = Ar;
		Operation->NumActors = InActors.Num();
		
		Operation->Construct();

		return Operation;
	}
};

