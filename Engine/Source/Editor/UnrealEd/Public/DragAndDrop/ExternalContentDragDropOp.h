// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "Misc/Guid.h"

class FExternalContentDragDropOp : public FAssetDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FExternalContentDragDropOp, FAssetDragDropOp)

	FExternalContentDragDropOp() : Guid(FGuid::NewGuid()) {}
	virtual ~FExternalContentDragDropOp() = default;

	const FGuid& GetGuid() const { return Guid; }

private:
	FGuid Guid;
};
