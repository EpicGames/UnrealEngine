// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/World.h"
#include "Folder.h"
#include "Input/DragAndDrop.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "EditorFolderUtils.h"

/* A drag/drop operation when dragging actor folders */
class FFolderDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FFolderDragDropOp, FDecoratedDragDropOp)

	/** Array of folders that we are dragging */
	TArray<FName> Folders;
	/** World to which the folders belong */
	TWeakObjectPtr<UWorld> World;
	/** Root object of folder (can be invalid) */
	FFolder::FRootObject RootObject;

	void Init(TArray<FName> InFolders, UWorld* InWorld, const FFolder::FRootObject& InRootObject = FFolder::GetInvalidRootObject())
	{
		check(InWorld != nullptr);
		Folders = MoveTemp(InFolders);
		World = InWorld;
		RootObject = InRootObject;

		CurrentIconBrush = FAppStyle::Get().GetBrush(TEXT("SceneOutliner.FolderClosed"));
		if (Folders.Num() == 1)
		{
			CurrentHoverText = FText::FromName(FEditorFolderUtils::GetLeafName(Folders[0]));
		}
		else
		{
			CurrentHoverText = FText::Format(NSLOCTEXT("FFolderDragDropOp", "FormatFolders", "{0} Folders"), FText::AsNumber(Folders.Num()));
		}

		SetupDefaults();
	}
};