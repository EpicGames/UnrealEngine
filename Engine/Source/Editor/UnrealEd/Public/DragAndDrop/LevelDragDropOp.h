// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Engine/Level.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Engine/LevelStreaming.h"
#include "Styling/AppStyle.h"
#include "UObject/Package.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"

class FLevelDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FLevelDragDropOp, FDecoratedDragDropOp)

	/** The levels to be dropped. */
	TArray<TWeakObjectPtr<ULevel>> LevelsToDrop;
		
	/** The streaming levels to be dropped. */
	TArray<TWeakObjectPtr<ULevelStreaming>> StreamingLevelsToDrop;

	/** Whether content is good to drop on current site, used by decorator */
	bool bGoodToDrop;
	
	/** Inits the tooltip */
	void Init()
	{
		FString DefaultLevel(TEXT("None"));

		if (LevelsToDrop.Num() > 0 && LevelsToDrop[0].IsValid())
		{
			DefaultLevel = LevelsToDrop[0]->GetOutermost()->GetName();
		}
		else if (StreamingLevelsToDrop.Num() > 0 && StreamingLevelsToDrop[0].IsValid())
		{
			DefaultLevel = StreamingLevelsToDrop[0]->GetWorldAssetPackageName();
		}

		CurrentHoverText = FText::FromString(DefaultLevel);

		bGoodToDrop = LevelsToDrop.Num() > 0 || StreamingLevelsToDrop.Num() > 0;

		SetupDefaults();
	}

	/** The widget decorator to use */
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
	{
		return SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Graph.ConnectorFeedback.Border"))
			.Content()
			[			
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
					[
						SNew(STextBlock) 
						.Text( this, &FDecoratedDragDropOp::GetHoverText )
					]
			];
	}

	static TSharedRef<FLevelDragDropOp> New(const TArray<TWeakObjectPtr<ULevelStreaming>>& LevelsToDrop)
	{
		TSharedRef<FLevelDragDropOp> Operation = MakeShareable(new FLevelDragDropOp);
		Operation->StreamingLevelsToDrop.Append(LevelsToDrop);
		Operation->Init();
		Operation->Construct();
		return Operation;
	}

	static TSharedRef<FLevelDragDropOp> New(const TArray<TWeakObjectPtr<ULevel>>& LevelsToDrop)
	{
		TSharedRef<FLevelDragDropOp> Operation = MakeShareable(new FLevelDragDropOp);
		Operation->LevelsToDrop.Append(LevelsToDrop);
		Operation->Init();
		Operation->Construct();
		return Operation;
	}
};
