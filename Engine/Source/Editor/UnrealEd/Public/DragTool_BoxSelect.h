// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "LevelEditorViewport.h"
#include "EditorDragTools.h"

class FCanvas;
class UModel;

/**
 * Draws a box in the current viewport and when the mouse button is released,
 * it selects/unselects everything inside of it.
 */
class FDragTool_ActorBoxSelect : public FDragTool
{
public:
	explicit FDragTool_ActorBoxSelect( FLevelEditorViewportClient* InLevelViewportClient )
		: FDragTool(InLevelViewportClient->GetModeTools())
		, LevelViewportClient( InLevelViewportClient )
		, EditorViewportClient( InLevelViewportClient)
	{}

	explicit FDragTool_ActorBoxSelect( FEditorViewportClient* InEditorViewportClient )
		: FDragTool(InEditorViewportClient->GetModeTools())
		, LevelViewportClient( nullptr)
		, EditorViewportClient( InEditorViewportClient )
	{}

	/**
	 * Starts a mouse drag behavior.  The start location is snapped to the editor constraints if bUseSnapping is true.
	 *
	 * @param	InViewportClient	The viewport client in which the drag event occurred.
	 * @param	InStart				Where the mouse was when the drag started.
	 */
	UNREALED_API virtual void StartDrag(FEditorViewportClient* InViewportClient, const FVector& InStart, const FVector2D& InStartScreen) override;

	/* Updates the drag tool's end location with the specified delta.  The end location is
	 * snapped to the editor constraints if bUseSnapping is true.
	 *
	 * @param	InDelta		A delta of mouse movement.
	 */
	UNREALED_API virtual void AddDelta(const FVector& InDelta) override;

	/**
	* Ends a mouse drag behavior (the user has let go of the mouse button).
	*/
	UNREALED_API virtual void EndDrag() override;
	UNREALED_API virtual void Render(const FSceneView* View, FCanvas* Canvas) override;

private:
	/** 
	 * Calculates a box to check actors against 
	 * 
	 * @param OutBox	The created box.
	 */
	void CalculateBox( FBox& OutBox );

	/** 
	 * Returns true if the provided BSP node intersects with the provided frustum 
	 *
	 * @param InModel				The model containing BSP nodes to check
	 * @param NodeIndex				The index to a BSP node in the model.  This node is used for the bounds check.
	 * @param InFrustum				The frustum to check against.
	 * @param bUseStrictSelection	true if the node must be entirely within the frustum
	 */
	bool IntersectsBox( const UModel& InModel, int32 NodeIndex, const FBox& InBox, bool bUseStrictSelection ) const;
	
	/** Adds a hover effect to the passed in actor */
	void AddHoverEffect( AActor& InActor );

	/** Adds a hover effect to the passed in bsp surface */
	void AddHoverEffect( UModel& InModel, int32 SurfIndex );

	/** Removes a hover effect from the passed in actor */
	void RemoveHoverEffect( AActor& InActor );

	/** Removes a hover effect from the passed in bsp surface */
	void RemoveHoverEffect( UModel& InModel, int32 SurfIndex );

	/** List of BSP models to check for selection */
	TArray<UModel*> ModelsToCheck;

	FLevelEditorViewportClient* LevelViewportClient;
	FEditorViewportClient* EditorViewportClient;
};

