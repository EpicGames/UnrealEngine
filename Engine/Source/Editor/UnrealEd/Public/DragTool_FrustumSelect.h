// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "LevelEditorViewport.h"
#include "EditorDragTools.h"

class FCanvas;
class UModel;

/**
 * Draws a box in the current viewport and when the mouse button is released,
 * it selects/unselects everything inside of it.
 */
class FDragTool_ActorFrustumSelect : public FDragTool
{
public:
	explicit FDragTool_ActorFrustumSelect(FLevelEditorViewportClient* InLevelViewportClient)
		: FDragTool(InLevelViewportClient->GetModeTools())
		, LevelViewportClient( InLevelViewportClient )
		, EditorViewportClient( InLevelViewportClient)
	{}

	explicit FDragTool_ActorFrustumSelect(FEditorViewportClient* InEditorViewportClient)
	: FDragTool(InEditorViewportClient->GetModeTools())
	, LevelViewportClient( nullptr )
	, EditorViewportClient( InEditorViewportClient)
	{}

	/* Updates the drag tool's end location with the specified delta.  The end location is
	 * snapped to the editor constraints if bUseSnapping is true.
	 *
	 * @param	InDelta		A delta of mouse movement.
	 */
	UNREALED_API virtual void AddDelta( const FVector& InDelta ) override;

	/**
	 * Starts a mouse drag behavior.  The start location is snapped to the editor constraints if bUseSnapping is true.
	 *
	 * @param	InViewportClient	The viewport client in which the drag event occurred.
	 * @param	InStart				Where the mouse was when the drag started.
	 */
	UNREALED_API virtual void StartDrag(FEditorViewportClient* InViewportClient, const FVector& InStart, const FVector2D& InStartScreen) override;
	
	/**
	 * Ends a mouse drag behavior (the user has let go of the mouse button).
	 */
	UNREALED_API virtual void EndDrag() override;
	UNREALED_API virtual void Render(const FSceneView* View,FCanvas* Canvas) override;

private:
	/** 
	 * Calculates a frustum to check actors against 
	 *
	 * @param InView			Information about scene projection
	 * @param OutFrustum		The created frustum
	 * @param bUseBoxFrustum	If true a frustum out of the current dragged box will be created.  false will use the view frustum.
	 */
	UNREALED_API void CalculateFrustum( const FSceneView* InView, FConvexVolume& OutFrustum, bool bUseBoxFrustum ) const;

	/** 
	 * Returns true if the provided BSP node intersects with the provided frustum 
	 *
	 * @param InModel				The model containing BSP nodes to check
	 * @param NodeIndex				The index to a BSP node in the model.  This node is used for the bounds check.
	 * @param InFrustum				The frustum to check against.
	 * @param bUseStrictSelection	true if the node must be entirely within the frustum
	 */
	UNREALED_API bool IntersectsFrustum( const UModel& InModel, int32 NodeIndex, const FConvexVolume& InFrustum, bool bUseStrictSelection ) const;
	
	/** Adds a hover effect to the passed in actor */
	UNREALED_API void AddHoverEffect( AActor& InActor );
	
	/** Adds a hover effect to the passed in bsp surface */
	UNREALED_API void AddHoverEffect( UModel& InModel, int32 SurfIndex );

	/** Removes a hover effect from the passed in actor */
	UNREALED_API void RemoveHoverEffect( AActor& InActor );

	/** Removes a hover effect from the passed in bsp surface */
	UNREALED_API void RemoveHoverEffect( UModel& InModel, int32 SurfIndex );

	/** List of actors to repeatedly check when determining hover cues */
	TArray<AActor*> ActorsToCheck;

	/** List of BSP models to check for selection */
	TArray<UModel*> ModelsToCheck;

	FLevelEditorViewportClient* LevelViewportClient;
	FEditorViewportClient* EditorViewportClient;
};
