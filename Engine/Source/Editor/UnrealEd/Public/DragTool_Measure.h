// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorDragTools.h"

class FCanvas;
class FEditorViewportClient;
class FSceneView;

/**
 * Draws a line in the current viewport and displays the distance between
 * its start/end points in the center of it.
 */
class FDragTool_Measure : public FDragTool
{
public:
	explicit FDragTool_Measure(FEditorViewportClient* InViewportClient);

	/**
	 * Starts a mouse drag behavior.  The start location is snapped to the editor constraints if bUseSnapping is true.
	 *
	 * @param	InViewportClient	The viewport client in which the drag event occurred.
	 * @param	InStart				Where the mouse was when the drag started (world space).
	 * @param	InStartScreen		Where the mouse was when the drag started (screen space).
	 */
	virtual void StartDrag(FEditorViewportClient* InViewportClient, const FVector& InStart, const FVector2D& InStartScreen) override;

	/* Updates the drag tool's end location with the specified delta.  The end location is
	 * snapped to the editor constraints if bUseSnapping is true.
	 *
	 * @param	InDelta		A delta of mouse movement.
	 */
	virtual void AddDelta(const FVector& InDelta) override;

	virtual void Render(const FSceneView* View,FCanvas* Canvas) override;

private:
	/**
	 * Gets the world-space snapped position of the specified pixel position
	 */
	FVector2D GetSnappedPixelPos(FVector2D PixelPos);

	/** Pixel-space positions for start and end */
	FVector2D PixelStart, PixelEnd;

	FEditorViewportClient* ViewportClient;
};
