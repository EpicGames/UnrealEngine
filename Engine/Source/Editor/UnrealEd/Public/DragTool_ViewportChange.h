// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Editor/UnrealEdTypes.h"
#include "EditorDragTools.h"

class FCanvas;
class FEditorViewportClient;
class FLevelEditorViewportClient;
class FSceneView;

/**
 * Draws a line in the current viewport and display the view options
 */
class FDragTool_ViewportChange : public FDragTool
{
public:
	explicit FDragTool_ViewportChange(FLevelEditorViewportClient* InviewportClient);

	/**
	* Starts a mouse drag behavior.  The start location is snapped to the editor constraints if bUseSnapping is true.
	*
	* @param	InViewportClient	The viewport client in which the drag event occurred.
	* @param	InStart				Where the mouse was when the drag started (world space).
	* @param	InStartScreen		Where the mouse was when the drag started (screen space).
	*/
	virtual void StartDrag(FEditorViewportClient* InViewportClient, const FVector& InStart, const FVector2D& InStartScreen) override;

	/**
	 * Ends a mouse drag behavior (the user has let go of the mouse button).
	 */
	virtual void EndDrag() override;

	/* Updates the drag tool's end location with the specified delta.  The end location is
	* snapped to the editor constraints if bUseSnapping is true.
	*
	* @param	InDelta		A delta of mouse movement.
	*/
	virtual void AddDelta(const FVector& InDelta) override;

	virtual void Render(const FSceneView* View, FCanvas* Canvas) override;

private:
	FLevelEditorViewportClient* LevelViewportClient;

	ELevelViewportType ViewOption;

	FVector2D ViewOptionOffset;
};
