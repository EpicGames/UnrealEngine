// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "EdGraphNode_Comment.generated.h"

class INameValidatorInterface;
class UEdGraphPin;
class UObject;
struct FPropertyChangedEvent;
struct Rect;

typedef TArray<class UObject*> FCommentNodeSet;

UENUM()
namespace ECommentBoxMode
{
	enum Type : int
	{
		/** This comment box will move any fully contained nodes when it moves. */
		GroupMovement UMETA(DisplayName="Group Movement"),

		/** This comment box has no effect on nodes contained inside it. */
		NoGroupMovement UMETA(DisplayName="Comment")
	};
}

UCLASS(MinimalAPI)
class UEdGraphNode_Comment : public UEdGraphNode
{
public:
	GENERATED_BODY()

public:
	UNREALED_API UEdGraphNode_Comment(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/** Color to style comment with */
	UPROPERTY(EditAnywhere, Category=Comment)
	FLinearColor CommentColor;

	/** Size of the text in the comment box */
	UPROPERTY(EditAnywhere, Category=Comment, meta=(ClampMin=1, ClampMax=1000))
	int32 FontSize;

	/** Whether to show a zoom-invariant comment bubble when zoomed out (making the comment readable at any distance). */
	UPROPERTY(EditAnywhere, Category=Comment, meta=(DisplayName="Show Bubble When Zoomed"))
	uint32 bCommentBubbleVisible_InDetailsPanel:1;

	/** Whether to use Comment Color to color the background of the comment bubble shown when zoomed out. */
	UPROPERTY(EditAnywhere, Category=Comment, meta=(DisplayName="Color Bubble", EditCondition=bCommentBubbleVisible_InDetailsPanel))
	uint32 bColorCommentBubble:1;

	/** Whether the comment should move any fully enclosed nodes around when it is moved */
	UPROPERTY(EditAnywhere, Category=Comment)
	TEnumAsByte<ECommentBoxMode::Type> MoveMode;

	/** Details field if more info is needed to be communicated (will show up in tooltip) */
	UPROPERTY(EditAnywhere, Category=Comment, meta=(DisplayName="Details", MultiLine=true))
	FText NodeDetails;

	/** comment Depth */
	UPROPERTY()
	int32 CommentDepth;

public:

	//~ Begin UObject Interface
	UNREALED_API static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	UNREALED_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UNREALED_API virtual bool IsSelectedInEditor() const override;
	//~ End UObject Interface

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override {}
	UNREALED_API virtual FText GetTooltipText() const override;
	UNREALED_API virtual FLinearColor GetNodeCommentColor() const override;
	UNREALED_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual bool ShouldOverridePinNames() const override { return true; }
	UNREALED_API virtual FText GetPinNameOverride(const UEdGraphPin& Pin) const override;
	UNREALED_API virtual void ResizeNode(const FVector2f& NewSize) override;
	UNREALED_API virtual void PostPlacedNewNode() override;
	UNREALED_API virtual void OnRenameNode(const FString& NewName) override;
	UNREALED_API virtual TSharedPtr<class INameValidatorInterface> MakeNameValidator() const override;
	UNREALED_API virtual FString GetDocumentationLink() const override;
	UNREALED_API virtual FString GetDocumentationExcerptName() const override;
	UNREALED_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	//~ End UEdGraphNode Interface

	/** Add a node that will be dragged when this comment is dragged */
	UNREALED_API void	AddNodeUnderComment(UObject* Object);

	/** Clear all of the nodes which are currently being dragged with this comment */
	UNREALED_API void	ClearNodesUnderComment();

	/** Set the Bounds for the comment node */
	UNREALED_API void SetBounds(const class FSlateRect& Rect);

	/** Return the set of nodes underneath the comment */
	UNREALED_API const FCommentNodeSet& GetNodesUnderComment() const;

	/** Return the font size of the comment */
	virtual int32 GetFontSize() const { return FontSize; }

	/** Override the default selection state of this graph node */
	enum class ESelectionState : uint8 { Inherited, Selected, Deselected };
	UNREALED_API void SetSelectionState(const ESelectionState InSelectionState);

private:
	/** Nodes currently within the region of the comment */
	TArray<TObjectPtr<class UObject>>	NodesUnderComment;

	/** Constructing FText strings can be costly, so we cache the node's tooltip */
	FNodeTextCache CachedTooltip;

	/** Override the default selection state of this graph node */
	ESelectionState SelectionState = ESelectionState::Inherited;
};

