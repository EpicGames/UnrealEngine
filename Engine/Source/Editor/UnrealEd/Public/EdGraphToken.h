// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphPin.h"
#include "HAL/Platform.h"
#include "Logging/TokenizedMessage.h"
#include "UObject/WeakObjectPtr.h"

class FCompilerResultsLog;
class FField;
class UEdGraphNode;
class UObject;

/**
 * A Message Log token that links to an elemnt (node or pin) in an EdGraph
 */
class FEdGraphToken : public IMessageToken
{
public:
	/** Factory method, tokens can only be constructed as shared refs */	
	UNREALED_API static void Create(const UObject* InObject, FCompilerResultsLog* Log, FTokenizedMessage &OutMessage, TArray<UEdGraphNode*>& OutSourceNodes);
	UNREALED_API static void Create(const UEdGraphPin* InPin, FCompilerResultsLog* Log, FTokenizedMessage &OutMessage, TArray<UEdGraphNode*>& OutSourceNodes);
	UNREALED_API static void Create(const TCHAR* String, FCompilerResultsLog* Log, FTokenizedMessage &OutMessage, TArray<UEdGraphNode*>& OutSourceNodes);
	UNREALED_API static void Create(const FField* InField, FCompilerResultsLog* Log, FTokenizedMessage &OutMessage, TArray<UEdGraphNode*>& OutSourceNodes);

	/** Begin IMessageToken interface */
	virtual EMessageToken::Type GetType() const override
	{
		return EMessageToken::EdGraph;
	}

	UNREALED_API const UEdGraphPin* GetPin() const;
	UNREALED_API const UObject* GetGraphObject() const;

private:
	/** Private constructor */
	FEdGraphToken( const UObject* InObject, const UEdGraphPin* InPin );
	/** Helper to facilitate code reuse between ::Create overloads */
	static void CreateInternal(const UObject* InObject, FCompilerResultsLog* Log, FTokenizedMessage &OutMessage, TArray<UEdGraphNode*>& OutSourceNodes, const UEdGraphPin* Pin);

	/** An object being referenced by this token, if any */
	FWeakObjectPtr ObjectBeingReferenced;

	FEdGraphPinReference PinBeingReferenced;
};

