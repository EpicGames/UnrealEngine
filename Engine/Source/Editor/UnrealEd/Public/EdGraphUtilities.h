// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "Math/UnrealMathSSE.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FCompilerResultsLog;
class SGraphNode;
class SGraphPin;
class UEdGraph;
class UEdGraphNode;
class UFunction;
class UObject;

//////////////////////////////////////////////////////////////////////////

struct FGraphPanelNodeFactory : public TSharedFromThis<FGraphPanelNodeFactory>
{
public:
	virtual ~FGraphPanelNodeFactory() = default;
	virtual TSharedPtr<class SGraphNode> CreateNode(class UEdGraphNode* Node) const { return NULL; }
};

//////////////////////////////////////////////////////////////////////////

struct FGraphPanelPinFactory : public TSharedFromThis<FGraphPanelPinFactory>
{
public:
	virtual ~FGraphPanelPinFactory() = default;
	virtual TSharedPtr<class SGraphPin> CreatePin(class UEdGraphPin* Pin) const { return NULL; }
};

//////////////////////////////////////////////////////////////////////////

struct FGraphPanelPinConnectionFactory : public TSharedFromThis<FGraphPanelPinConnectionFactory>
{
public:
    virtual ~FGraphPanelPinConnectionFactory() = default;
    virtual class FConnectionDrawingPolicy* CreateConnectionPolicy(const class UEdGraphSchema* Schema, int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const class FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const { return nullptr; }
};

//////////////////////////////////////////////////////////////////////////

class FEdGraphUtilities
{
public:
	static UNREALED_API void RegisterVisualNodeFactory(TSharedPtr<FGraphPanelNodeFactory> NewFactory);
	static UNREALED_API void UnregisterVisualNodeFactory(TSharedPtr<FGraphPanelNodeFactory> OldFactory);

	static UNREALED_API void RegisterVisualPinFactory(TSharedPtr<FGraphPanelPinFactory> NewFactory);
	static UNREALED_API void UnregisterVisualPinFactory(TSharedPtr<FGraphPanelPinFactory> OldFactory);

    static UNREALED_API void RegisterVisualPinConnectionFactory(TSharedPtr<FGraphPanelPinConnectionFactory> NewFactory);
    static UNREALED_API void UnregisterVisualPinConnectionFactory(TSharedPtr<FGraphPanelPinConnectionFactory> OldFactory);

	/** After pasting nodes, need to perform some fixup for pins etc. */
	static UNREALED_API void PostProcessPastedNodes(TSet<class UEdGraphNode*>& SpawnedNodes);

	/** Finds the net associated with a particular pin */
	static UNREALED_API UEdGraphPin* GetNetFromPin(UEdGraphPin* Pin);

	/** 
	 *	Clones (deep copies) a UEdGraph, including all of it's nodes and pins and their links, maintaining a mapping from the clone to the source nodes (even across multiple clonings) 
	 *	A NewOuter of NULL indicates the transient package
	 */
	static UNREALED_API UEdGraph* CloneGraph(UEdGraph* Source, UObject* NewOuter, FCompilerResultsLog* MessageLog = NULL, bool bCloningForCompile = false);

	/**
	  * Clones the content from SourceGraph and merges it into MergeTarget;
	  * including merging/flattening all of the children from the SourceGraph into MergeTarget
	  *
	  * Also optionally populates *OutClonedNodes with the list of cloned objects.
	  *
	  * @param MergeTarget				The graph to merge the source graph into
	  * @param SourceGraph				Graph to merge from
	  * @param MessageLog				MessageLog to report errors and warnings to
	  * @param bRequireSchemaMatch		TRUE if the graphs must have the same schema for the merge to occur
	  * @param OutClonedNodes			Will populate with a full list of cloned nodes if provided
	  * @param bInIsCompiling			TRUE if the function is being called during compilation, this will eliminate some nodes that will not be compiled
	  */
	static UNREALED_API void CloneAndMergeGraphIn(UEdGraph* MergeTarget, UEdGraph* SourceGraph, FCompilerResultsLog& MessageLog, bool bRequireSchemaMatch, bool bInIsCompiling = false, TArray<UEdGraphNode*>* OutClonedNodes = NULL);

	/**
	 * Moves the contents of all of the children graphs of ParentGraph (recursively) into the MergeTarget graph.
	 * This does not clone, it's destructive to the ParentGraph
	 */
	static UNREALED_API void MergeChildrenGraphsIn(UEdGraph* MergeTarget, UEdGraph* ParentGraph, bool bRequireSchemaMatch, bool bInIsCompiling = false, FCompilerResultsLog* MessageLog = nullptr);

	/** Tries to rename the graph to have a name similar to BaseName */
	static UNREALED_API void RenameGraphCloseToName(UEdGraph* Graph, const FString& BaseName, int32 StartIndex = 1);
	static UNREALED_API void RenameGraphToNameOrCloseToName(UEdGraph* Graph, const FString& DesiredName);

	/**
	 * Exports a set of nodes to text
	 *
	 * @param	NodesToExport		Set of nodes to export, all nodes must be in the same graph
	 * @param	[out] ExportedText	The text representation will be stored here
	 */
	static UNREALED_API void ExportNodesToText(TSet<UObject*> NodesToExport, /*out*/ FString& ExportedText);

	/**
	 * Imports a set of previously exported nodes into a graph
	 *
	 * @param	DestinationGraph		The target graph to place the imported nodes in
	 * @param	TextToImport			The text representation of the nodes to import
	 * @param	[out] ImportedNodeSet	The set of nodes that were imported
	 */
	static UNREALED_API void ImportNodesFromText(UEdGraph* DestinationGraph, const FString& TextToImport, /*out*/ TSet<UEdGraphNode*>& ImportedNodeSet);

	/**
	 * Can we import nodes from this text ?
	 *
	 * @param	DestinationGraph		The target graph to place the imported nodes in
	 * @param	TextToImport			The text representation of the nodes to import
	 */
	static UNREALED_API bool CanImportNodesFromText(const UEdGraph* DestinationGraph, const FString& TextToImport);

	// Calculates an approximate bounding box for the specified nodes
	static UNREALED_API FIntRect CalculateApproximateNodeBoundaries(const TArray<UEdGraphNode*>& Nodes);

	/** Copy Common State of data from OldNode to NewNode **/
	static UNREALED_API void CopyCommonState(UEdGraphNode* OldNode, UEdGraphNode* NewNode);

	struct FNodeVisitor
	{
		TSet<UEdGraphNode*> VisitedNodes;

		void TouchNode(UEdGraphNode* Node)
		{
		}

		UNREALED_API void TraverseNodes(UEdGraphNode* Node);
	};

	/** returns true if the ParameterName is marked as a SetParam in the relevant Function */
	static UNREALED_API bool IsSetParam(const UFunction* Function, const FName ParameterName);

	/** returns true if the ParameterName is marked as a MapParam, MapKeyParam, or MapValueParam in the relevant Function */
	static UNREALED_API bool IsMapParam(const UFunction* Function, const FName ParameterName);

	/** returns true if the ParameterName is marked as ArrayTypeDependentParams */
	static UNREALED_API bool IsArrayDependentParam(const UFunction* Function, const FName ParameterName );

	/** returns true if the ParamaterName is marked as a Set, Map, or array param in the relevant function */
	static UNREALED_API bool IsDynamicContainerParam(const UFunction* Function, const FName ParameterName);

	/** returns the first pin marked as an ArrayParam, usually returning nullptr */
	static UNREALED_API UEdGraphPin* FindArrayParamPin(const UFunction* Function, const UEdGraphNode* Node);

	/** returns the first pin marked as an SetParam, usually returning nullptr */
	static UNREALED_API UEdGraphPin* FindSetParamPin(const UFunction* Function, const UEdGraphNode* Node);
	
	/** returns the first pin marked as an MapParam, usually returning nullptr */
	static UNREALED_API UEdGraphPin* FindMapParamPin(const UFunction* Function, const UEdGraphNode* Node);

	/** returns the first pin referred to in a , and | delineated list of pin names */
	static UNREALED_API UEdGraphPin* FindPinFromMetaData(const UFunction* Function, const UEdGraphNode* Node, FName MetaData );

	/**
	* Output a map of pin names to a set of connections given a function entry pin
	* 
	* @param Node				The entry pin to gather connections from
	* @param OutPinConnections	Output map of connection data
	*/
	static UNREALED_API void GetPinConnectionMap(const UEdGraphNode* Node, TMap<FString, TSet<UEdGraphPin*>>& OutPinConnections);

	/**
	* Reconnect the pin map to the given node
	* 
	* @param Node				The node to connect pins to
	* @param PinConnections		Map of pin connections to set
	*/
	static UNREALED_API void ReconnectPinMap(UEdGraphNode* Node, const TMap<FString, TSet<UEdGraphPin*>>& PinConnections);

	/** Copy pin default values between nodes for any pins with matching names */
	static UNREALED_API void CopyPinDefaults(const UEdGraphNode* NodeFrom, UEdGraphNode* NodeTo);

private:
	static UNREALED_API TArray< TSharedPtr<FGraphPanelNodeFactory> > VisualNodeFactories;
	static UNREALED_API TArray< TSharedPtr<FGraphPanelPinFactory> > VisualPinFactories;
    static UNREALED_API TArray< TSharedPtr<FGraphPanelPinConnectionFactory> > VisualPinConnectionFactories;
	friend class FNodeFactory;

	// Should never create an instance of this class
	FEdGraphUtilities() {}
};

//////////////////////////////////////////////////////////////////////////

/** A weak reference to a UEdGraphPin object that can remain valid through pin connection state changes that might trigger owner node reconstruction */
struct FWeakGraphPinPtr
{
	/** Default constructor */
	FORCEINLINE FWeakGraphPinPtr()
	{
		Reset();
	}

	/** Construct from another weak pointer */
	FORCEINLINE FWeakGraphPinPtr(const FWeakGraphPinPtr &OtherPinPtr)
	{
		(*this) = OtherPinPtr;
	}

	/** Construct from a UEdGraphPin object pointer */
	FORCEINLINE FWeakGraphPinPtr(const class UEdGraphPin *Pin)
	{
		(*this) = Pin;
	}

	/** Reset the weak pointer back to NULL state */
	FORCEINLINE void Reset()
	{
		PinName = TEXT("");
		PinReference = FEdGraphPinReference();
		NodeObjectPtr.Reset();
	}

	/** Assign from another weak pointer */
	FORCEINLINE void operator=(const FWeakGraphPinPtr &OtherPinPtr)
	{
		PinName = OtherPinPtr.PinName;
		PinReference = OtherPinPtr.PinReference;
		NodeObjectPtr = OtherPinPtr.NodeObjectPtr;
	}

	/** Assign from a UEdGraphPin object pointer */
	UNREALED_API void operator=(const class UEdGraphPin *Pin);

	/** Compare weak pointers for equality */ 
	FORCEINLINE bool operator==(const FWeakGraphPinPtr &OtherPinPtr) const
	{
		return PinName == OtherPinPtr.PinName
			&& PinReference == OtherPinPtr.PinReference
			&& NodeObjectPtr == OtherPinPtr.NodeObjectPtr;
	}

	/** Compare weak pointers for inequality */ 
	FORCEINLINE bool operator!=(const FWeakGraphPinPtr &OtherPinPtr) const
	{
		return PinName != OtherPinPtr.PinName
			|| PinReference.Get() != OtherPinPtr.PinReference.Get()
			|| NodeObjectPtr != OtherPinPtr.NodeObjectPtr;
	}

	/** Method to obtain the actual object reference from the weak pointer. Will attempt to fix up the pin object reference if it's no longer valid, otherwise returns NULL */
	UNREALED_API class UEdGraphPin *Get();

	/** Cast to object pointer type */
	FORCEINLINE operator class UEdGraphPin*()
	{
		return Get();
	};

	/** Arrow operator */
	FORCEINLINE class UEdGraphPin* operator->()
	{
		return Get();
	}

	/** Test for validity */
	FORCEINLINE bool IsValid()
	{
		return Get() != NULL;
	}

private:
	/** Pin name string */
	FName PinName;

	/** Weak reference to the UEdGraphPin object */
	FEdGraphPinReference PinReference;

	/** Weak reference to the UEdGraphNode object that owns the pin object */
	TWeakObjectPtr<class UEdGraphNode> NodeObjectPtr;
};
