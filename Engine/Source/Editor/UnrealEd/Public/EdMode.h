// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "UObject/GCObject.h"
#include "UnrealWidgetFwd.h"
#include "EditorComponents.h"
#include "EngineGlobals.h"
#include "EditorModeRegistry.h"
#include "Tools/UEdMode.h"
#include "Templates/SharedPointer.h"
#include "Tools/LegacyEdModeWidgetHelpers.h"

class FCanvas;
class FEditorModeTools;
class FEditorViewportClient;
class FModeTool;
class FModeToolkit;
class FPrimitiveDrawInterface;
class FSceneView;
class FViewport;
class UTexture2D;
struct FConvexVolume;
struct FViewportClick;

enum EModeTools : int8;
class FEditorViewportClient;
class HHitProxy;
struct FViewportClick;
class FModeTool;
class FEditorViewportClient;
struct FViewportClick;
struct FGizmoState;

/**
 * Base class for all editor modes.
 * 
 * Modes can be switched. Different modes may have different widgets / editing functionality / layouts.
 * Modes can also be compatible, so multiple modes can concurrently affect a single asset editor. 
 * Modes defined in separate modules, say from a plugin, can modify existing asset editors in a different module.
 */
class FEdMode : public TSharedFromThis<FEdMode>, public FGCObject, public FEditorCommonDrawHelper, public FLegacyEdModeWidgetHelper
{
public:
	/** Friends so it can access mode's internals on construction */
	friend class FEditorModeRegistry;

	UNREALED_API FEdMode();
	UNREALED_API virtual ~FEdMode();

	virtual void Initialize() {}

	UNREALED_API virtual bool MouseEnter( FEditorViewportClient* ViewportClient,FViewport* Viewport,int32 x, int32 y );

	UNREALED_API virtual bool MouseLeave( FEditorViewportClient* ViewportClient,FViewport* Viewport );

	UNREALED_API virtual bool MouseMove(FEditorViewportClient* ViewportClient,FViewport* Viewport,int32 x, int32 y);

	UNREALED_API virtual bool ReceivedFocus(FEditorViewportClient* ViewportClient,FViewport* Viewport);

	UNREALED_API virtual bool LostFocus(FEditorViewportClient* ViewportClient,FViewport* Viewport);

	/**
	 * Called when the mouse is moved while a window input capture is in effect
	 *
	 * @param	InViewportClient	Level editor viewport client that captured the mouse input
	 * @param	InViewport			Viewport that captured the mouse input
	 * @param	InMouseX			New mouse cursor X coordinate
	 * @param	InMouseY			New mouse cursor Y coordinate
	 *
	 * @return	true if input was handled
	 */
	UNREALED_API virtual bool CapturedMouseMove( FEditorViewportClient* InViewportClient, FViewport* InViewport, int32 InMouseX, int32 InMouseY );

	/** Process all captured mouse moves that occurred during the current frame */
	virtual bool ProcessCapturedMouseMoves( FEditorViewportClient* InViewportClient, FViewport* InViewport, const TArrayView<FIntPoint>& CapturedMouseMoves ) { return false; }

	UNREALED_API virtual bool InputKey(FEditorViewportClient* ViewportClient,FViewport* Viewport,FKey Key,EInputEvent Event);
	UNREALED_API virtual bool InputAxis(FEditorViewportClient* InViewportClient,FViewport* Viewport,int32 ControllerId,FKey Key,float Delta,float DeltaTime);
	UNREALED_API virtual bool InputDelta(FEditorViewportClient* InViewportClient,FViewport* InViewport,FVector& InDrag,FRotator& InRot,FVector& InScale);
	UNREALED_API virtual bool StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport);
	UNREALED_API virtual bool EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport);
	// Added for handling EDIT Command...
	virtual EEditAction::Type GetActionEditDuplicate() { return EEditAction::Skip; }
	virtual EEditAction::Type GetActionEditDelete() { return EEditAction::Skip; }
	virtual EEditAction::Type GetActionEditCut() { return EEditAction::Skip; }
	virtual EEditAction::Type GetActionEditCopy() { return EEditAction::Skip; }
	virtual EEditAction::Type GetActionEditPaste() { return EEditAction::Skip; }
	virtual EEditAction::Type GetActionDragDuplicate() { return GetActionEditDuplicate(); }
	virtual bool ProcessEditDuplicate() { return false; }
	virtual bool ProcessEditDelete() { return false; }
	virtual bool ProcessEditCut() { return false; }
	virtual bool ProcessEditCopy() { return false; }
	virtual bool ProcessEditPaste() { return false; }

	virtual bool BeginTransform(const FGizmoState& InState) { return false; }
	virtual bool EndTransform(const FGizmoState& InState) { return false; }
	
	UNREALED_API virtual void Tick(FEditorViewportClient* ViewportClient,float DeltaTime);

	virtual bool IsCompatibleWith(FEditorModeID OtherModeID) const { return false; }
	virtual void ActorMoveNotify() {}
	virtual void ActorsDuplicatedNotify(TArray<AActor*>& PreDuplicateSelection, TArray<AActor*>& PostDuplicateSelection, bool bOffsetLocations) {}
	virtual void ActorPropChangeNotify() {}
	virtual void MapChangeNotify() {}

	/** If the Edmode is handling its own mouse deltas, it can disable the MouseDeltaTacker */
	virtual bool DisallowMouseDeltaTracking() const { return false; }

	/** 
	 * Lets each mode/tool specify a pivot point around which the camera should orbit
	 * @param	OutPivot	The custom pivot point returned by the mode/tool
	 * @return	true if a custom pivot point was specified, false otherwise.
	 */
	virtual bool GetPivotForOrbit(FVector& OutPivot) const { return false; }

	/**
	 * Get a cursor to override the default with, if any.
	 * @return true if the cursor was overridden.
	 */
	virtual bool GetCursor(EMouseCursor::Type& OutCursor) const { return false; }

	/** Get override cursor visibility settings */
	virtual bool GetOverrideCursorVisibility(bool& bWantsOverride, bool& bHardwareCursorVisible, bool bSoftwareCursorVisible) const { return false; }

	/** Called before mouse movement is converted to drag/rot */
	virtual bool PreConvertMouseMovement(FEditorViewportClient* InViewportClient) { return false; }

	/** Called after mouse movement is converted to drag/rot */
	virtual bool PostConvertMouseMovement(FEditorViewportClient* InViewportClient) { return false;}

	virtual bool ShouldDrawBrushWireframe( AActor* InActor ) const { return true; }

	/** If Rotation Snap should be enabled for this mode*/ 
	UNREALED_API virtual bool IsSnapRotationEnabled();

	/** If this mode should override the snap rotation
	* @param	Rotation		The Rotation Override
	*
	* @return					True if you have overridden the value
	*/
	virtual bool SnapRotatorToGridOverride(FRotator& Rotation){ return false; };
	virtual void UpdateInternalData() {}

	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override {}
	virtual FString GetReferencerName() const override
	{
		return TEXT("FEdMode");
	}

	UNREALED_API virtual void Enter();
	UNREALED_API virtual void Exit();
	UNREALED_API virtual UTexture2D* GetVertexTexture();

	/**
	 * Lets each tool determine if it wants to use the editor widget or not.  If the tool doesn't want to use it,
	 * it will be fed raw mouse delta information (not snapped or altered in any way).
	 */
	UNREALED_API virtual bool UsesTransformWidget() const override;
	UNREALED_API virtual bool UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const override;
	UNREALED_API virtual bool UsesPropertyWidgets() const override;

	virtual void PostUndo() {}

	/** 
	 * Check to see if this EdMode wants to disallow AutoSave
	 * @return true if AutoSave can be applied right now
	 */
	virtual bool CanAutoSave() const { return true; }

	/**
	 * Lets each mode/tool handle box selection in its own way.
	 *
	 * @param	InBox	The selection box to use, in worldspace coordinates.
	 * @return		true if something was selected/deselected, false otherwise.
	 */
	UNREALED_API virtual bool BoxSelect( FBox& InBox, bool InSelect = true );

	/**
	 * Lets each mode/tool handle frustum selection in its own way.
	 *
	 * @param	InFrustum	The selection box to use, in worldspace coordinates.
	 * @return	true if something was selected/deselected, false otherwise.
	 */
	UNREALED_API virtual bool FrustumSelect( const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, bool InSelect = true );

	UNREALED_API virtual void SelectNone();
	virtual void SelectionChanged() {}

	UNREALED_API virtual bool HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click);

	/**
	 * Allows an editor mode to override the bounding box used to focus the viewport on a selection
	 * 
	 * @param Actor			The selected actor that is being considered for focus
	 * @param PrimitiveComponent	The component in the actor being considered for focus
	 * @param InOutBox		The box that should be computed for the actor and component
	 * @return bool			true if the mode overrides the box and populated InOutBox, false if it did not populate InOutBox
	 */
	virtual bool ComputeBoundingBoxForViewportFocus(AActor* Actor, UPrimitiveComponent* PrimitiveComponent, FBox& InOutBox) const { return false; }

	/** Handling SelectActor */
	virtual bool Select( AActor* InActor, bool bInSelected ) { return 0; }

	/** Check to see if an actor can be selected in this mode - no side effects */
	virtual bool IsSelectionAllowed( AActor* InActor, bool bInSelection ) const { return true; }

	/** Returns the editor mode identifier. */
	FEditorModeID GetID() const { return Info.ID; }

	/** Returns the editor mode information. */
	const FEditorModeInfo& GetModeInfo() const { return Info; }

	// Tools

	UNREALED_API void SetCurrentTool( EModeTools InID );
	UNREALED_API void SetCurrentTool( FModeTool* InModeTool );
	UNREALED_API FModeTool* FindTool( EModeTools InID );

	const TArray<FModeTool*>& GetTools() const		{ return Tools; }

	virtual void CurrentToolChanged() {}

	/** Returns the current tool. */
	//@{
	FModeTool* GetCurrentTool()				{ return CurrentTool; }
	const FModeTool* GetCurrentTool() const	{ return CurrentTool; }
	//@}

	/** @name Rendering */
	//@{
	/** Draws translucent polygons on brushes and volumes. */
	UNREALED_API virtual void Render(const FSceneView* View,FViewport* Viewport,FPrimitiveDrawInterface* PDI);
	//void DrawGridSection(int32 ViewportLocX,int32 ViewportGridY,FVector* A,FVector* B,float* AX,float* BX,int32 Axis,int32 AlphaCase,FSceneView* View,FPrimitiveDrawInterface* PDI);

	/** Overlays the editor hud (brushes, drag tools, static mesh vertices, etc*. */
	UNREALED_API virtual void DrawHUD(FEditorViewportClient* ViewportClient,FViewport* Viewport,const FSceneView* View,FCanvas* Canvas);
	//@}

	/**
	 * Called when attempting to duplicate the selected actors by alt+dragging,
	 * return true to prevent normal duplication.
	 */
	virtual bool HandleDragDuplicate() { return false; }

	/** True if this mode uses a toolkit mode (eventually they all should) */
	UNREALED_API virtual bool UsesToolkits() const;

	/** Gets the toolkit created by this mode */
	TSharedPtr<FModeToolkit> GetToolkit() { return Toolkit; }

	/** Returns the world this toolkit is editing */
	UNREALED_API UWorld* GetWorld() const;

	/** Returns the owning mode manager for this mode */
	UNREALED_API class FEditorModeTools* GetModeManager() const;

	/** 
	 * Called when the editor mode should rebuild its toolbar 
	 *
	 * @param ToolbarBuilder	The builder which should be used to add toolbar widgets
	 */
	virtual void BuildModeToolbar(class FToolBarBuilder& ToolbarBuilder) {}

	using FLegacyEdModeWidgetHelper::MD_MakeEditWidget;
	using FLegacyEdModeWidgetHelper::MD_ValidateWidgetUsing;
	using FLegacyEdModeWidgetHelper::CanCreateWidgetForStructure;
	using FLegacyEdModeWidgetHelper::CanCreateWidgetForProperty;
	using FLegacyEdModeWidgetHelper::ShouldCreateWidgetForProperty;
	using FPropertyWidgetInfo = FLegacyEdModeWidgetHelper::FPropertyWidgetInfo;

public:

	/** Request that this mode be deleted at the next convenient opportunity (FEditorModeTools::Tick) */
	UNREALED_API void RequestDeletion();

private:
	/** Called whenever a mode type is unregistered */
	UNREALED_API void OnModeUnregistered(FEditorModeID ModeID);

protected:
	/** Optional array of tools for this mode. */
	TArray<FModeTool*> Tools;

	/** The tool that is currently active within this mode. */
	FModeTool* CurrentTool;

	/** Information pertaining to this mode. Assigned by FEditorModeRegistry. */
	FEditorModeInfo Info;

	/** Editor Mode Toolkit that is associated with this toolkit mode */
	TSharedPtr<class FModeToolkit> Toolkit;
};
