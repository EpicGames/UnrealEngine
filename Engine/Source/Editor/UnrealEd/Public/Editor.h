// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/*-----------------------------------------------------------------------------
	Dependencies.
-----------------------------------------------------------------------------*/

#include "CoreMinimal.h"
#include "Templates/ScopedCallback.h"
#include "Engine/Level.h"
#include "AssetRegistry/AssetData.h"
#include "Editor/EditorEngine.h"
#include "Editor/UnrealEdTypes.h"
#include "Engine/StaticMesh.h"

#include "Subsystems/ImportSubsystem.h"


#define CAMERA_ZOOM_DAMPEN			200.f

class AStaticMeshActor;
class FEdMode;
class UFactory;
struct FGuid;

/** The shorthand identifier used for editor modes */
typedef FName FEditorModeID;

/** The editor object. */
extern UNREALED_API class UEditorEngine* GEditor;

/** Max length of a single folder in the content directory */
#define MAX_CONTENT_FOLDER_NAME_LENGTH 32
/** Max length of an asset name */
#define MAX_ASSET_NAME_LENGTH 64

/**
 * Returns the path to the engine's editor resources directory (e.g. "/../../Engine/Editor/")
 */
UNREALED_API const FString GetEditorResourcesDir();

/**
 * Helper struct for the FOnAssetsCanDelete delegate
 */
struct FCanDeleteAssetResult
{
public:
	FCanDeleteAssetResult() : bResult(true) {}
	FCanDeleteAssetResult(const FCanDeleteAssetResult&) = delete;
	FCanDeleteAssetResult(FCanDeleteAssetResult&&) = delete;

	void Set(const bool InValue) { bResult &= InValue; }
	bool Get() const { return bResult; }
private:
	bool bResult;
};

/**
 * Helper struct for the FOnLoadMap delegate
 */
struct FCanLoadMap
{
public:
	FCanLoadMap() : bResult(true) {}
	FCanLoadMap(const FCanLoadMap&) = delete;
	FCanLoadMap(FCanLoadMap&&) = delete;

	void SetFalse() { bResult = false; }
	bool Get() const { return bResult; }
private:
	bool bResult;
};

/**
 * Arguments for FOnPreDestructiveAssetActionWithArgs delegate
 */
struct FOnPreDestructiveAssetActionArgs
{
public:
	FOnPreDestructiveAssetActionArgs(const TArray<UObject*>& InObjects, const EDestructiveAssetActions InAction) :
		Objects(InObjects),
		Action(InAction)
	{
		Result.bSuccess = true;
	}

	FOnPreDestructiveAssetActionArgs(const TArray<UObject*>& InObjects, const EDestructiveAssetActions InAction, TArrayView<const FTopLevelAssetPath> InDestinations) :
		Objects(InObjects),
		Action(InAction),
		Destinations(InDestinations)
	{
		Result.bSuccess = true;
	}

	FOnPreDestructiveAssetActionArgs(const FOnPreDestructiveAssetActionArgs&) = delete;
	FOnPreDestructiveAssetActionArgs(FOnPreDestructiveAssetActionArgs&&) = delete;

	const TArray<UObject*>& Objects;
	const EDestructiveAssetActions Action;

	/* Destinations must equal the number of objects when Action set to AssetMove */
	const TArrayView<const FTopLevelAssetPath> Destinations;

	FResultMessage Result;
};

/** 
 * FEditorDelegates
 * Delegates used by the editor.
 **/
struct FEditorDelegates
{
	/** delegate type for map change events ( Params: uint32 MapChangeFlags (MapChangeEventFlags) ) */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnMapChanged, uint32);
	/** delegate type for editor mode change events ( Params: FEditorModeID NewMode ) */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnModeChanged, FEditorModeID);
	/** delegate type for editor camera movement */
	DECLARE_MULTICAST_DELEGATE_FourParams(FOnEditorCameraMoved, const FVector&, const FRotator&, ELevelViewportType, int32 );
	/** delegate type for dollying/zooming editor camera movement */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnDollyPerspectiveCamera, const FVector&, int32 );
	/** delegate type for pre save world events ( UWorld* World, FObjectPreSaveContext ObjectSaveContext ) */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPreSaveWorldWithContext, class UWorld*, FObjectPreSaveContext);
	/** delegate type for post save world events ( UWorld* World, FObjectPostSaveContext ObjectSaveContext ) */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPostSaveWorldWithContext, class UWorld*, FObjectPostSaveContext);
	/** delegate type for pre save external actors event, called by editor save codepaths and auto saves ( UWorld* World )*/
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPreSaveExternalActors, class UWorld*);
	/** delegate type for post save external actors event, called by editor save codepaths and auto saves ( UWorld* World )*/
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPostSaveExternalActors, class UWorld*);
	/** delegate for a PIE event (begin, end, pause/resume, etc) (Params: bool bIsSimulating) */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPIEEvent, const bool);
	/** delegate for a standalone local play event (Params: uint32 processID) */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnStandaloneLocalPlayEvent, const uint32);
	/** delegate type for beginning or finishing configuration of the properties of a new asset */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnNewAssetCreation, UFactory*);
	/** delegate for when assets are about to undergo a destructive action caused by the Editor UI (Delete, Rename, Move, Privatize, etc.) */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnPreDestructiveAssetAction, const TArray<UObject*>&, EDestructiveAssetActions, FResultMessage&);
	/** delegate for when assets are about to undergo a destructive action caused by the Editor UI (Delete, Rename, Move, Privatize, etc.) */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPreDestructiveAssetActionWithArgs, FOnPreDestructiveAssetActionArgs&);
	/** delegate type fired when new assets are being (re-)imported. Params: UFactory* InFactory, UClass* InClass, UObject* InParent, const FName& Name, const TCHAR* Type */
	DECLARE_MULTICAST_DELEGATE_FiveParams(FOnAssetPreImport, UFactory*, UClass*, UObject*, const FName&, const TCHAR*);
	/** delegate type fired when new assets have been (re-)imported. Note: InCreatedObject can be NULL if import failed. Params: UFactory* InFactory, UObject* InCreatedObject */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnAssetPostImport, UFactory*, UObject*);
	/** delegate type fired when new assets have been reimported. Note: InCreatedObject can be NULL if import failed. UObject* InCreatedObject */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnAssetReimport, UObject*);
	/** delegate type for finishing up construction of a new blueprint */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnFinishPickingBlueprintClass, UClass*);
	/** delegate type for triggering when new actors are dropped on to the viewport via drag and drop */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnNewActorsDropped, const TArray<UObject*>&, const TArray<AActor*>&);
	/** delegate type for triggering when new actors are placed on to the viewport. Triggers before NewActorsDropped if placement is caused by a drop action */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnNewActorsPlaced, UObject*, const TArray<AActor*>&);
	/** delegate type for triggering when an actor is replaced by another one in the editor. Triggers once AActor::EditorReplacedActor is completed */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnEditorActorReplaced, AActor*, AActor*);
	/** delegate type for when attempting to apply an object to an actor */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnApplyObjectToActor, UObject*, AActor*);
	/** delegate type for triggering when grid snapping has changed */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnGridSnappingChanged, bool, float);
	/** delegate type for triggering when focusing on a set of actors */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnFocusViewportOnActors, const TArray<AActor*>&);
	/** delegate type for triggering when a map starts loading */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnMapLoad, const FString& /* Filename */, FCanLoadMap& /*OutCanLoadMap*/);
	/** delegate type for triggering when a map is opened */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnMapOpened, const FString& /* Filename */, bool /*bAsTemplate*/);
	/** Delegate used for entering or exiting an editor mode */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnEditorModeTransitioned, FEdMode* /*Mode*/);
	/** Delegate used for entering or exiting an editor mode */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnEditorModeIDTransitioned, const FEditorModeID& /*Mode*/);
	/** delegate type to determine if a user requests can delete certain assets. */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnAssetsCanDelete, const TArray<UObject*>& /*InObjectToDelete*/, FCanDeleteAssetResult& /*OutCanDelete*/);
	/** delegate type for when a user requests to delete certain package */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPackageDeleted, UPackage*);
	/** delegate type for when a user requests to delete certain assets... It allows the addition of secondary assets that should also be deleted */
	UE_DEPRECATED(5.5, "FOnAssetsAddExtraObjectsToDelete has been deprecated, please use FOnAddExtraObjectsToDelete instead")
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnAssetsAddExtraObjectsToDelete, TArray<UObject*>&);
	/** delegate type for when a user requests to delete certain assets... It allows the addition of secondary assets that should also be deleted */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnAddExtraObjectsToDelete, const TArray<UObject*>&, TSet<UObject*>&);
	
	/** delegate type for when a user requests to delete certain assets... DOES NOT mean the asset(s) will be deleted (the user could cancel) */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnAssetsPreDelete, const TArray<UObject*>&);
	/** delegate type for when a user requested force deleting objects. The objects(s) will be deleted (no possibility to cancel), so implementations should delete references */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPreForceDeleteObjects, const TArray<UObject*>&);
	/** delegate type for when one or more assets have been deleted */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnAssetsDeleted, const TArray<UClass*>& /*DeletedAssetClasses*/);
	/** delegate type for when a user starts dragging something out of content browser (can be multiple assets) */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnAssetDragStarted, const TArray<FAssetData>&, class UActorFactory* /*FactoryToUse*/);
	/** delegate type for when a new level is added to the world */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnAddLevelToWorld, ULevel*);
	/** delegate type for when a texture is fit to surface  */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnFitTextureToSurface, UWorld*);
	/** delegate type for before edit cut actors is handled */
	DECLARE_MULTICAST_DELEGATE(FOnEditCutActorsBegin);
	/** delegate type for after edit cut actors is handled */
	DECLARE_MULTICAST_DELEGATE(FOnEditCutActorsEnd);
	/** delegate type for before edit copy actors is handled */
	DECLARE_MULTICAST_DELEGATE(FOnEditCopyActorsBegin);
	/** delegate type for after edit copy actors is handled */
	DECLARE_MULTICAST_DELEGATE(FOnEditCopyActorsEnd);
	/** delegate type for before edit paste actors is handled */
	DECLARE_MULTICAST_DELEGATE(FOnEditPasteActorsBegin);
	/** delegate type for after edit paste actors is handled */
	DECLARE_MULTICAST_DELEGATE(FOnEditPasteActorsEnd);
	/** delegate type for before edit duplicate actors is handled */
	DECLARE_MULTICAST_DELEGATE(FOnDuplicateActorsBegin);
	/** delegate type for after edit duplicate actors is handled */
	DECLARE_MULTICAST_DELEGATE(FOnDuplicateActorsEnd);	
	/** delegate type for before delete actors is handled */
	DECLARE_MULTICAST_DELEGATE(FOnDeleteActorsBegin);
	/** delegate type for after delete actors is handled */
	DECLARE_MULTICAST_DELEGATE(FOnDeleteActorsEnd);
	/** delegate type to handle viewing/editing a set of asset identifiers which are packages or ids */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnViewAssetIdentifiers, TArray<FAssetIdentifier>);
	/** delegate type to handle viewing/editing a set of asset identifiers (which are packages or ids) in the reference viewer */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnOpenReferenceViewer, const TArray<FAssetIdentifier>, const FReferenceViewerParams);
	/** delegate type for when the editor requests a restart, enables overriding how a restart is performed */
	DECLARE_DELEGATE_RetVal_OneParam(bool /*bSuccess*/, FOnRestartRequested, const FString& /*ProjectName*/);
	/** delegate for when the editor has booted */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnEditorBoot, double Duration);
	/** delegate for when the editor is about to load the default startup map - allows delegates to cancel the operation */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnEditorLoadDefaultStartupMap, FCanLoadMap&);
	/** delegate for when the editor has fully initialized */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnEditorInitialized, double Duration);
	/** delegate when external content resolves and can replace placeholder data */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnExternalContentResolved, const FGuid&, const FAssetData& /*PlaceholderAsset*/, const FAssetData& /*ResolvedAsset*/);
	/** delegate for an editor process is generating a large set of actors */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnWorldGenerationProcess, UWorld*);

	/** Called when the CurrentLevel is switched to a new level.  Note that this event won't be fired for temporary
		changes to the current level, such as when copying/pasting actors. */
	static UNREALED_API FSimpleMulticastDelegate NewCurrentLevel;
	/** Called when the map has changed */
	static UNREALED_API FOnMapChanged MapChange;
	/** Called when an actor is added to a layer */
	static UNREALED_API FSimpleMulticastDelegate LayerChange;
	/** Called after an undo/redo */
	static UNREALED_API FSimpleMulticastDelegate PostUndoRedo;
	/** surfprops changed */
	static UNREALED_API FSimpleMulticastDelegate SurfProps;
	/** Sent when requesting to display the properties of selected actors or BSP surfaces */
	static UNREALED_API FSimpleMulticastDelegate SelectedProps;
	/** Fits the currently assigned texture to the selected surfaces */
	static UNREALED_API FOnFitTextureToSurface FitTextureToSurface;
	/** Called when the editor mode is changed */
	static UNREALED_API FOnModeChanged ChangeEditorMode;
	/** Called when properties of an actor have changed */
	static UNREALED_API FSimpleMulticastDelegate ActorPropertiesChange;
	/** Called when the editor needs to be refreshed */
	static UNREALED_API FSimpleMulticastDelegate RefreshEditor;
	/** called when all browsers need to be refreshed */
	static UNREALED_API FSimpleMulticastDelegate RefreshAllBrowsers;
	/** called when the level browser need to be refreshed */
	static UNREALED_API FSimpleMulticastDelegate RefreshLevelBrowser;
	/** called when the layer browser need to be refreshed */
	static UNREALED_API FSimpleMulticastDelegate RefreshLayerBrowser;
	/** called when the primitive stats browser need to be refreshed */
	static UNREALED_API FSimpleMulticastDelegate RefreshPrimitiveStatsBrowser;
	/** Called when an action is performed which interacts with the content browser; 
	 *  load any selected assets which aren't already loaded */
	static UNREALED_API FSimpleMulticastDelegate LoadSelectedAssetsIfNeeded;
	/** Called when load errors are about to be displayed */
	static UNREALED_API FSimpleMulticastDelegate DisplayLoadErrors;
	/** Called when an editor mode is being entered */
	UE_DEPRECATED(4.24, "Use EditorModeIDEnter instead")
	static UNREALED_API FOnEditorModeTransitioned EditorModeEnter;
	/** Called when an editor mode is being exited */
	UE_DEPRECATED(4.24, "Use EditorModeIDExit instead")
	static UNREALED_API FOnEditorModeTransitioned EditorModeExit;
	/** Called when an editor mode ID is being entered */
	UE_DEPRECATED(5.0, "Use the asset editor's mode manager to scope mode enter notifications.")
	static UNREALED_API FOnEditorModeIDTransitioned EditorModeIDEnter;
	/** Called when an editor mode ID is being exited */
	UE_DEPRECATED(5.0, "Use the asset editor's mode manager to scope mode exit notifications.")
	static UNREALED_API FOnEditorModeIDTransitioned EditorModeIDExit;
	/** Sent when a PIE session has been requested to Start */
	static UNREALED_API FOnPIEEvent StartPIE;
	/** Sent when a PIE session is beginning (before we decide if PIE can run - allows clients to avoid blocking PIE) */
	static UNREALED_API FOnPIEEvent PreBeginPIE;
	/** Sent when a PIE session is beginning (but hasn't actually started yet) */
	static UNREALED_API FOnPIEEvent BeginPIE;
	/** Sent when a PIE session has fully started and after BeginPlay() has been called */
	static UNREALED_API FOnPIEEvent PostPIEStarted;
	/** Sent when a PIE session is ending, before anything else happens */
	static UNREALED_API FOnPIEEvent PrePIEEnded;
	/** Sent when a PIE session is ending */
	static UNREALED_API FOnPIEEvent EndPIE;
	/** Sent when a PIE session has completely shutdown */
	static UNREALED_API FOnPIEEvent ShutdownPIE;
	/** Sent when a PIE session is paused */
	static UNREALED_API FOnPIEEvent PausePIE;
	/** Sent when a PIE session is resumed */
	static UNREALED_API FOnPIEEvent ResumePIE;
	/** Sent when a PIE session is single-stepped */
	static UNREALED_API FOnPIEEvent SingleStepPIE;
	/** Sent just before the user switches between from PIE to SIE, or vice-versa.  Passes in whether we are currently in SIE */
	static UNREALED_API FOnPIEEvent OnPreSwitchBeginPIEAndSIE;
	/** Sent after the user switches between from PIE to SIE, or vice-versa.  Passes in whether we are currently in SIE */
	static UNREALED_API FOnPIEEvent OnSwitchBeginPIEAndSIE;
	/** Sent when a PIE session is cancelled */
	static UNREALED_API FSimpleMulticastDelegate CancelPIE;
	/** Sent when PC local play session is starting */
	static UNREALED_API FOnStandaloneLocalPlayEvent BeginStandaloneLocalPlay;
	/** Within a property window, the currently selected item was changed.*/
	static UNREALED_API FSimpleMulticastDelegate PropertySelectionChange;
	/** Called after Landscape layer infomap update have completed */
	static UNREALED_API FSimpleMulticastDelegate PostLandscapeLayerUpdated;
	/** Called before SaveWorld is processed */
	static UNREALED_API FOnPreSaveWorldWithContext PreSaveWorldWithContext;
	/** Called after SaveWorld is processed */
	static UNREALED_API FOnPostSaveWorldWithContext PostSaveWorldWithContext;
	/** Called before any number of external actors will be saved */
	static UNREALED_API FOnPreSaveExternalActors PreSaveExternalActors;
	/** Called after any number of external actors has been saved */
	static UNREALED_API FOnPostSaveExternalActors PostSaveExternalActors;
	/** Called before any asset validation happens via the Asset Validation subsystem. */
	static UNREALED_API FSimpleMulticastDelegate OnPreAssetValidation;
	/** Called after asset validation happens by the Asset Validation subsystem. */
	static UNREALED_API FSimpleMulticastDelegate OnPostAssetValidation;
	/** Called when finishing picking a new blueprint class during construction */
	static UNREALED_API FOnFinishPickingBlueprintClass OnFinishPickingBlueprintClass;
	/** Called when beginning configuration of a new asset */
	static UNREALED_API FOnNewAssetCreation OnConfigureNewAssetProperties;
	/** Called when an asset is about to undergo a destructive action caused by the Editor UI (Delete, Move, Rename, Privatize, etc.) */
	UE_DEPRECATED(5.8, "Use FOnPreDestructiveAssetActionWithArgs instead.")
	static UNREALED_API FOnPreDestructiveAssetAction OnPreDestructiveAssetAction;
	/** Called when an asset is about to undergo a destructive action caused by the Editor UI (Delete, Move, Rename, Privatize, etc.) */
	static UNREALED_API FOnPreDestructiveAssetActionWithArgs OnPreDestructiveAssetActionWithArgs;
	/** Called when finishing configuration of a new asset */
	static UNREALED_API FOnNewAssetCreation OnNewAssetCreated;
	/** Called when new assets are being (re-)imported. */
	UE_DEPRECATED(4.22, "Use the ImportSubsystem instead. GEditor->GetEditorSubsystem<UImportSubsystem>()")
	static UNREALED_API FOnAssetPreImport OnAssetPreImport;
	/** Called when new assets have been (re-)imported. */
	UE_DEPRECATED(4.22, "Use the ImportSubsystem instead. GEditor->GetEditorSubsystem<UImportSubsystem>()")
	static UNREALED_API FOnAssetPostImport OnAssetPostImport;
	/** Called after an asset has been reimported */
	UE_DEPRECATED(4.22, "Use the ImportSubsystem instead. GEditor->GetEditorSubsystem<UImportSubsystem>()")
	static UNREALED_API FOnAssetReimport OnAssetReimport;
	/** Called when new actors are dropped on to the viewport */
	static UNREALED_API FOnNewActorsDropped OnNewActorsDropped;
	/** Called when new actors are placed in the viewport */
	static UNREALED_API FOnNewActorsPlaced OnNewActorsPlaced;
	/** Called when an actor is replaced in the editor (once AActor::EditorReplacedActor is completed). The first parameter is the old actor, the second is the new actor. */
	static UNREALED_API FOnEditorActorReplaced OnEditorActorReplaced;
	/** Called when grid snapping is changed */
	static UNREALED_API FOnGridSnappingChanged OnGridSnappingChanged;
	/** Called when a lighting build has started */
	static UNREALED_API FSimpleMulticastDelegate OnLightingBuildStarted;
	/** Called when a lighting build has been kept */
	static UNREALED_API FSimpleMulticastDelegate OnLightingBuildKept;
	/** Called when a lighting build has failed (maybe called twice if cancelled) */
	static UNREALED_API FSimpleMulticastDelegate OnLightingBuildFailed;
	/** Called when a lighting build has succeeded */
	static UNREALED_API FSimpleMulticastDelegate OnLightingBuildSucceeded;
	/** Called when when attempting to apply an object to an actor (via drag drop) */
	static UNREALED_API FOnApplyObjectToActor OnApplyObjectToActor;
	/** Called when focusing viewport on a set of actors */
	static UNREALED_API FOnFocusViewportOnActors OnFocusViewportOnActors;
	/** Called before LoadMap is processed */
	static UNREALED_API FOnMapLoad OnMapLoad;
	/** Called when a map is opened, giving map name, and whether it was a template */
	static UNREALED_API FOnMapOpened OnMapOpened;
	/** Called when the editor camera is moved */
	static UNREALED_API FOnEditorCameraMoved OnEditorCameraMoved;
	/** Called when the editor camera is moved */
	static UNREALED_API FOnDollyPerspectiveCamera OnDollyPerspectiveCamera;
	/** Called on editor shutdown after packages have been successfully saved */
	static UNREALED_API FSimpleMulticastDelegate OnShutdownPostPackagesSaved;
	/** Called when one or more packages have been deleted through save */
	static UNREALED_API FOnPackageDeleted OnPackageDeleted;
	/** Called when the user requests assets to be deleted to determine if the operation is available.  */
	static UNREALED_API FOnAssetsCanDelete OnAssetsCanDelete;
	/** Called when the user requests certain assets be deletedand  allows the addition of secondary assets that should also be deleted */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "Delegate signature for OnAssetsAddExtraObjectsToDelete has been deprecated please use new version OnAddExtraObjectsToDelete")
	static UNREALED_API FOnAssetsAddExtraObjectsToDelete OnAssetsAddExtraObjectsToDelete;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS	
	static UNREALED_API FOnAddExtraObjectsToDelete OnAddExtraObjectsToDelete;	
	/** Called when the user requests certain assets be deleted (DOES NOT imply that the asset will be deleted... the user could cancel) */
	static UNREALED_API FOnAssetsPreDelete OnAssetsPreDelete;
	/** Called when one or more assets have been deleted */
	static UNREALED_API FOnAssetsDeleted OnAssetsDeleted;
	/** Called when a user starts dragging something out of content browser (can be multiple assets) */
	static UNREALED_API FOnAssetDragStarted OnAssetDragStarted;
	/** Called when the user requests objects to be force deleted.  There is no possibility to cancel once this callback is made */
    static UNREALED_API FOnPreForceDeleteObjects OnPreForceDeleteObjects;
	/** Called when a user changes the UInputSettings::bEnableGestureRecognizer setting to refresh the available actions. */
	static UNREALED_API FSimpleMulticastDelegate OnEnableGestureRecognizerChanged;
	/** Called when Action or Axis mappings have been changed */
	static UNREALED_API FSimpleMulticastDelegate OnActionAxisMappingsChanged;
	/** Called from FEditorUtils::AddLevelToWorld after the level is added successfully to the world. */
	static UNREALED_API FOnAddLevelToWorld OnAddLevelToWorld;
	/** Sent before edit cut is handled */
	static UNREALED_API FOnEditCutActorsBegin OnEditCutActorsBegin;
	/** Sent after edit cut is handled */
	static UNREALED_API FOnEditCutActorsEnd OnEditCutActorsEnd;
	/** Sent before edit copy is handled */
	static UNREALED_API FOnEditCopyActorsBegin OnEditCopyActorsBegin;
	/** Sent after edit copy is handled */
	static UNREALED_API FOnEditCopyActorsEnd OnEditCopyActorsEnd;
	/** Sent before edit paste is handled */
	static UNREALED_API FOnEditPasteActorsBegin OnEditPasteActorsBegin;
	/** Sent after edit paste is handled */
	static UNREALED_API FOnEditPasteActorsEnd OnEditPasteActorsEnd;
	/** Sent before duplicate is handled */
	static UNREALED_API FOnDuplicateActorsBegin OnDuplicateActorsBegin;
	/** Sent after duplicate is handled */
	static UNREALED_API FOnDuplicateActorsEnd OnDuplicateActorsEnd;
	/** Sent when delete begin called */
	static UNREALED_API FOnDeleteActorsBegin OnDeleteActorsBegin;
	/** Sent when delete end called */
	static UNREALED_API FOnDeleteActorsEnd OnDeleteActorsEnd;
	/** Called when you want to view things in the reference viewer, these are bound to by asset manager editor plugins */
	static UNREALED_API FOnOpenReferenceViewer OnOpenReferenceViewer;
	/** Called when you want to view things in the size map */
	static UNREALED_API FOnViewAssetIdentifiers OnOpenSizeMap;
	/** Called when you want to view things in the asset audit window */
	static UNREALED_API FOnViewAssetIdentifiers OnOpenAssetAudit;
	/** Called to try and edit an asset identifier, which could be a package or searchable name */
	static UNREALED_API FOnViewAssetIdentifiers OnEditAssetIdentifiers;
	/** Called when the editor requests a restart */
	static UNREALED_API FOnRestartRequested OnRestartRequested;
	/** Called when the editor has booted */
	static UNREALED_API FOnEditorBoot OnEditorBoot;
	/** Called when the editor is about to load the default startup map */
	static UNREALED_API FOnEditorLoadDefaultStartupMap OnEditorLoadDefaultStartupMap;
	/** Called when the editor has initialized */
	static UNREALED_API FOnEditorInitialized OnEditorInitialized;
	/** Called before the editor exits. This is called before any editor subsystems are deinitialized, and also before FCoreDelegates::OnEnginePreExit */
	static UNREALED_API FSimpleMulticastDelegate OnEditorPreExit;
	/** Called when external content gets resolved */
	static UNREALED_API FOnExternalContentResolved OnExternalContentResolved;
	/** Called when an editor process is about to generate a large set of actors */
	static UNREALED_API FOnWorldGenerationProcess OnPreWorldGenerationProcess;
	/** Called when an editor process is done generating a large set of actors */
	static UNREALED_API FOnWorldGenerationProcess OnPostWorldGenerationProcess;
};

/**
 * Scoped delegate wrapper
 */
 #define DECLARE_SCOPED_DELEGATE( CallbackName, TriggerFunc )						\
	class UNREALED_API FScoped##CallbackName##Impl										\
	{																				\
	public:																			\
		static void FireCallback() { TriggerFunc; }									\
	};																				\
																					\
	typedef TScopedCallback<FScoped##CallbackName##Impl> FScoped##CallbackName;

DECLARE_SCOPED_DELEGATE( ActorPropertiesChange, FEditorDelegates::ActorPropertiesChange.Broadcast() );
DECLARE_SCOPED_DELEGATE( RefreshAllBrowsers, FEditorDelegates::RefreshAllBrowsers.Broadcast() );

#undef DECLARE_SCOPED_DELEGATE


/** Texture alignment. */
enum ETAxis
{
	TAXIS_X                 = 0,
	TAXIS_Y                 = 1,
	TAXIS_Z                 = 2,
	TAXIS_WALLS             = 3,
	TAXIS_AUTO              = 4,
};






/**
 * MapChangeEventFlags defines flags passed to FEditorDelegates::MapChange global events
 */
namespace MapChangeEventFlags
{
	/** MapChangeEventFlags::Type */
	typedef uint32 Type;

	/** Default flags */
	inline const Type Default = 0;

	/** Set when a new map is created, loaded from disk, imported, etc. */
	inline const Type NewMap = 1 << 0;

	/** Set when a map rebuild occurred */
	inline const Type MapRebuild = 1 << 1;

	/** Set when a world was destroyed (torn down) */
	inline const Type WorldTornDown = 1 << 2;
}

/**
 * This class begins an object movement change when created and ends it when it falls out of scope
 */
class FScopedObjectMovement
{
public:
	/**
	 * Constructor.  Broadcasts a delegate to notify listeners an actor is about to move
	 */
	FScopedObjectMovement( UObject* InObject )
		: Object( InObject )
	{
		if( GEditor && Object.IsValid() )
		{
			GEditor->BroadcastBeginObjectMovement( *Object );
		}
	}

	/**
	 * Constructor.  Broadcasts a delegate to notify listeners an actor has moved
	 */
	~FScopedObjectMovement()
	{
		if( GEditor && Object.IsValid() )
		{
			GEditor->BroadcastEndObjectMovement( *Object );
		}
	}
private:
	/** The object being moved */
	TWeakObjectPtr<UObject> Object;
};

/**
 * Import the entire default properties block for the class specified
 * 
 * @param	Class		the class to import defaults for
 * @param	Text		buffer containing the text to be imported
 * @param	Warn		output device for log messages
 * @param	Depth		current nested subobject depth
 * @param	LineNumber	the starting line number for the defaultproperties block (used for log messages)
 *
 * @return	NULL if the default values couldn't be imported
 */

/**
 * Parameters for ImportObjectProperties
 */
struct FImportObjectParams
{
	/** the location to import the property values to */
	uint8*				DestData = nullptr;

	/** pointer to a buffer containing the values that should be parsed and imported */
	const TCHAR*		SourceText = nullptr;

	/** the struct for the data we're importing */
	UStruct*			ObjectStruct = nullptr;

	/** the original object that ImportObjectProperties was called for.
		if SubobjectOuter is a subobject, corresponds to the first object in SubobjectOuter's Outer chain that is not a subobject itself.
		if SubobjectOuter is not a subobject, should normally be the same value as SubobjectOuter */
	UObject*			SubobjectRoot = nullptr;

	/** the object corresponding to DestData; this is the object that will used as the outer when creating subobjects from definitions contained in SourceText */
	UObject*			SubobjectOuter = nullptr;

	/** output device to use for log messages */
	FFeedbackContext*	Warn = nullptr;

	/** current nesting level */
	int32					Depth = 0;

	/** used when importing defaults during script compilation for tracking which line we're currently for the purposes of printing compile errors */
	int32					LineNumber = INDEX_NONE;

	/** contains the mappings of instanced objects and components to their templates; used when recursively calling ImportObjectProperties; generally
		not necessary to specify a value when calling this function from other code */
	FObjectInstancingGraph* InInstanceGraph = nullptr;

	/** provides a mapping from an existing object, typically an actor, (which may no longer be loaded) to a new instance to which it should be remapped */
	TMap<FSoftObjectPath, UObject*>* ObjectRemapper = nullptr;

	/** True if we should call PreEditChange/PostEditChange on the object as it's imported.  Pass false here
		if you're going to do that on your own. */
	bool				bShouldCallEditChange = true;
};


/**
 * Parse and import text as property values for the object specified.
 * 
 * @param	InParams	Parameters for object import; see declaration of FImportObjectParams.
 *
 * @return	NULL if the default values couldn't be imported
 */
UNREALED_API const TCHAR* ImportObjectProperties( FImportObjectParams& InParams );

/**
 * Parse and import text as property values for the object specified.
 * 
 * @param	DestData			the location to import the property values to
 * @param	SourceText			pointer to a buffer containing the values that should be parsed and imported
 * @param	ObjectStruct		the struct for the data we're importing
 * @param	SubobjectRoot		the original object that ImportObjectProperties was called for.
 *								if SubobjectOuter is a subobject, corresponds to the first object in SubobjectOuter's Outer chain that is not a subobject itself.
 *								if SubobjectOuter is not a subobject, should normally be the same value as SubobjectOuter
 * @param	SubobjectOuter		the object corresponding to DestData; this is the object that will used as the outer when creating subobjects from definitions contained in SourceText
 * @param	Warn				output device to use for log messages
 * @param	Depth				current nesting level
 * @param	LineNumber			used when importing defaults during script compilation for tracking which line the defaultproperties block begins on
 * @param	InstanceGraph		contains the mappings of instanced objects and components to their templates; used when recursively calling ImportObjectProperties; generally
 *								not necessary to specify a value when calling this function from other code
 * @param	ObjectRemapper		used when duplicating objects, typically actors, to remap references from a source object to the duplicated object
 *
 * @return	NULL if the default values couldn't be imported
 */
const TCHAR* ImportObjectProperties(
	uint8*				DestData,
	const TCHAR*		SourceText,
	UStruct*			ObjectStruct,
	UObject*			SubobjectRoot,
	UObject*			SubobjectOuter,
	FFeedbackContext*	Warn,
	int32					Depth,
	int32					LineNumber = INDEX_NONE,
	FObjectInstancingGraph* InstanceGraph = nullptr,
	TMap<FSoftObjectPath, UObject*>* ObjectRemapper = nullptr
	);

//
// GBuildStaticMeshCollision - Global control for building static mesh collision on import.
//

extern bool GBuildStaticMeshCollision;

//
// Creating a static mesh from an array of triangles.
//
UStaticMesh* CreateStaticMesh(struct FMeshDescription& RawMesh,TArray<FStaticMaterial>& Materials,UObject* Outer,FName Name);

struct FMergeStaticMeshParams
{
	/**
	 *Constructor, setting all values to usable defaults 
	 */
	FMergeStaticMeshParams();

	/** A translation to apply to the verts in SourceMesh */
	FVector Offset;
	/** A rotation to apply to the verts in SourceMesh */
	FRotator Rotation;
	/** A uniform scale to apply to the verts in SourceMesh */
	float ScaleFactor;
	/** A non-uniform scale to apply to the verts in SourceMesh */
	FVector ScaleFactor3D;
	
	/** If true, DestMesh will not be rebuilt */
	bool bDeferBuild;
	
	/** If set, all triangles in SourceMesh will be set to this element index, instead of duplicating SourceMesh's elements into DestMesh's elements */
	int32 OverrideElement;

	/** If true, UVChannelRemap will be used to reroute UV channel values from one channel to another */
	bool bUseUVChannelRemapping;
	/** An array that can remap UV values from one channel to another */
	int32 UVChannelRemap[8];
	
	/* If true, UVScaleBias will be used to modify the UVs (AFTER UVChannelRemap has been applied) */
	bool bUseUVScaleBias;
	/* Scales/Bias's to apply to each UV channel in SourceMesh */
	FVector4 UVScaleBias[8];
};

/**
 * Merges SourceMesh into DestMesh, applying transforms along the way
 *
 * @param DestMesh The static mesh that will have SourceMesh merged into
 * @param SourceMesh The static mesh to merge in to DestMesh
 * @param Params Settings for the merge
 */
void MergeStaticMesh(UStaticMesh* DestMesh, UStaticMesh* SourceMesh, const FMergeStaticMeshParams& Params);

//
// Converting models to static meshes.
//
UNREALED_API void GetBrushMesh(ABrush* Brush, UModel* Model, struct FMeshDescription& OutMesh, TArray<FStaticMaterial>& OutMaterials);
UStaticMesh* CreateStaticMeshFromBrush(UObject* Outer,FName Name,ABrush* Brush,UModel* Model);
 
/**
 * Converts a static mesh to a brush.
 *
 * @param	Model					[out] The target brush.  Must be non-NULL.
 * @param	StaticMeshActor			The source static mesh.  Must be non-NULL.
 */
UNREALED_API void CreateModelFromStaticMesh(UModel* Model,AStaticMeshActor* StaticMeshActor);


/**
 * Sets GWorld to the passed in PlayWorld and sets a global flag indicating that we are playing
 * in the Editor.
 *
 * @param	PlayInEditorWorld		PlayWorld
 * @return	the original GWorld
 */
UNREALED_API UWorld* SetPlayInEditorWorld( UWorld* PlayInEditorWorld );

/**
 * Restores GWorld to the passed in one and reset the global flag indicating whether we are a PIE
 * world or not.
 *
 * @param EditorWorld	original world being edited
 */
UNREALED_API void RestoreEditorWorld( UWorld* EditorWorld );


/*-----------------------------------------------------------------------------
	Parameter parsing functions.
-----------------------------------------------------------------------------*/

bool GetFVECTOR( const TCHAR* Stream, const TCHAR* Match, FVector& Value );
bool GetFVECTOR( const TCHAR* Stream, FVector& Value );
const TCHAR* GetFVECTORSpaceDelimited( const TCHAR* Stream, FVector& Value );
bool GetFROTATOR( const TCHAR* Stream, const TCHAR* Match, FRotator& Rotation, int32 ScaleFactor );
bool GetFROTATOR( const TCHAR* Stream, FRotator& Rotation, int ScaleFactor );
const TCHAR* GetFROTATORSpaceDelimited( const TCHAR* Stream, FRotator& Rotation, int32 ScaleFactor );
bool GetBEGIN( const TCHAR** Stream, const TCHAR* Match );
bool GetEND( const TCHAR** Stream, const TCHAR* Match );
bool GetREMOVE( const TCHAR** Stream, const TCHAR* Match );
bool GetSUBSTRING(const TCHAR*	Stream, const TCHAR* Match, TCHAR* Value, int32 MaxLen);
TCHAR* SetFVECTOR( TCHAR* Dest, const FVector* Value );

/**
 * Takes an FName and checks to see that it is unique among all loaded objects.
 *
 * @param	InName		The name to check
 * @param	Outer		The context for validating this object name. Should be a group/package
 * @param	InReason	If the check fails, this string is filled in with the reason why.
 *
 * @return	true if the name is valid
 */

UNREALED_API bool IsUniqueObjectName( const FName& InName, UObject* Outer, FText* InReason = nullptr );

/**
 * Takes an FName and checks to see that it is unique among all loaded objects in all packages.
 *
 * @param	InName		The name to check
 * @param	InReason	If the check fails, this string is filled in with the reason why.
 *
 * @return	true if the name is valid
 */
UNREALED_API bool IsGloballyUniqueObjectName(const FName& InName, FText* InReason = nullptr);

/**
 * Takes an FName and checks to see that it is unique among all loaded objects.
 *
 * @param	InName		The name to check
 * @param	Outer		The context for validating this object name. Should be a group/package.
 * @param	InReason	If the check fails, this string is filled in with the reason why.
 *
 * @return	true if the name is valid
 */

UNREALED_API bool IsUniqueObjectName( const FName& InName, UObject* Outer, FText& InReason );


/**
 * Provides access to the FEditorModeTools for the level editor
 */
UNREALED_API class FEditorModeTools& GLevelEditorModeTools();

/**
 * Checks if FEditorModeTools is valid
 */
UE_DEPRECATED(5.5, "Checking the validity of the global mode manager is unnecessary. Instead use FLevelEditorModule::OnLevelEditorCreated to gate the access on the global mode manager.")
UNREALED_API bool GLevelEditorModeToolsIsValid();

namespace EditorUtilities
{
	/**
	 * Given an actor in a Simulation or PIE world, tries to find a counterpart actor in the editor world
	 *
	 * @param	Actor	The simulation world actor that we want to find a counterpart for
	 *
	 * @return	The found editor world actor, or NULL if we couldn't find a counterpart
	 */
	UNREALED_API AActor* GetEditorWorldCounterpartActor( AActor* Actor );

	/**
	 * Given an actor in the editor world, tries to find a counterpart actor in a Simulation or PIE world
	 *
	 * @param	Actor	The editor world actor that we want to find a counterpart for
	 *
	 * @return	The found Simulation or PIE world actor, or NULL if we couldn't find a counterpart
	 */
	UNREALED_API AActor* GetSimWorldCounterpartActor( AActor* Actor );

	/**
	 * Guiven an actor in the editor world, and SourceComponent from Simulation or PIE world
	 * find the matching component in the Editor World
	 *
	 * @param	SourceComponent	SouceCompoent in SIM world
	 * @param	TargetActor		TargetActor in editor world
	 *
	 * @return	the sound editor component or NULL if we couldn't find
	 */
	UNREALED_API UActorComponent* FindMatchingComponentInstance( UActorComponent* SourceComponent, AActor* TargetActor );

	/** Options for CopyActorProperties */
	namespace ECopyOptions
	{
		enum Type
		{
			/** Default copy options */
			Default = 0,

			/** Set this option to preview the changes and not actually copy anything.  This will count the number of properties that would be copied. */
			PreviewOnly = 1 << 0,

			/** Call PostEditChangeProperty for each modified property */
			CallPostEditChangeProperty = 1 << 1,

			/** Call PostEditMove if we detect that a transform property was changed */
			CallPostEditMove = 1 << 2,

			/** Copy only Edit and Interp properties.  Otherwise we copy all properties by default */
			OnlyCopyEditOrInterpProperties = 1 << 3,

			/** Propagate property changes to archetype instances if the target actor is a CDO */
			PropagateChangesToArchetypeInstances = 1 << 4,

			/** Filters out Blueprint Read-only properties */
			FilterBlueprintReadOnly = 1 << 5,

			/** Filters out properties that are marked instance only. */
			SkipInstanceOnlyProperties = 1 << 6,
		};
	}


	/** Copy options structure for CopyActorProperties */
	struct FCopyOptions
	{
		/** Implicit construction for an options enumeration */
		FCopyOptions(const ECopyOptions::Type InFlags, TFunction<bool(FProperty&, UObject&)> InPropertyFilter = TFunction<bool(FProperty&, UObject&)>()) 
			: Flags(InFlags)
			, PropertyFilter(InPropertyFilter)
		{}

		/** Check whether we can copy the specified property */
		bool CanCopyProperty(FProperty& Property, UObject& Object) const
		{
			return !PropertyFilter || PropertyFilter(Property, Object);
		}

		/** User-specified flags for the copy */
		ECopyOptions::Type Flags;

		/** User-specified custom property filter predicate */
		TFunction<bool(FProperty&, UObject&)> PropertyFilter;
	};

	/** Helper function for CopyActorProperties(). Copies a single property form a source object to a target object. */
	UNREALED_API void CopySingleProperty(const UObject* const InSourceObject, UObject* const InTargetObject, FProperty* const InProperty);

	/**
	 * Copies properties from one actor to another.  Designed for propagating changes made to PIE actors back to their EditorWorld
	 * counterpart, or pushing spawned actor changes back to a Blueprint CDO object.  You can pass the 'PreviewOnly' option to
	 * count the properties that would be copied instead of actually copying them.
	 *
	 * @param	SourceActor		Actor to copy properties from
	 * @param	TargetActor		Actor that will be modified with properties propagated from the source actor
	 * @param	Options			Optional options for this copy action (see ECopyOptions::Type)
	 *
	 * @return	The number of properties that were copied over (properties that were filtered out, or were already identical, are not counted.)
	 */
	UNREALED_API int32 CopyActorProperties( AActor* SourceActor, AActor* TargetActor, const FCopyOptions& Options = FCopyOptions(ECopyOptions::Default) );


	// ==== Multi step import utilities ====

	/**
	 * Parameters for the ImportCreateObjectStep and ImportObjectPropertiesStep. Used for multi steps import.
	 */
	struct FMultiStepsImportObjectParams
	{
		/** The location to import the property values to */
		uint8* DestData = nullptr;

		/** Text buffer containing the values that should be parsed and imported */
		FStringView SourceText;

		/** The struct for the data we're importing */
		UStruct* ObjectStruct = nullptr;

		/** The original object that ImportObjectProperties was called for.
			If SubobjectOuter is a subobject, corresponds to the first object in SubobjectOuter's Outer chain that is not a subobject itself.
			If SubobjectOuter is not a subobject, should normally be the same value as SubobjectOuter */
		UObject* SubobjectRoot = nullptr;

		/** The object corresponding to DestData; this is the object that will used as the outer when creating subobjects from definitions contained in SourceText */
		UObject* SubobjectOuter = nullptr;

		/** Output device to use for log messages */
		FFeedbackContext*	Warn = nullptr;

		/** Current nesting level */
		int32 Depth = 0;

		/** Used when importing defaults during script compilation for tracking which line we're currently for the purposes of printing compile errors */
		int32 LineNumber = INDEX_NONE;

		/** Contains the mappings of instanced objects and components to their templates; used when recursively calling ImportObjectProperties; generally
			not necessary to specify a value when calling this function from other code */
		FObjectInstancingGraph* InInstanceGraph = nullptr;

		/** 
		 * Provides a mapping from an exported path to a new instance to which it should be remapped 
		 * Imported object will be added to this map when possible during the create objects step.
		 */
		TMap<FSoftObjectPath, UObject*>* ObjectRemapper = nullptr;

		/** 
		 * Tell what properties shouldn't be imported when importing the properties
		 */
		TSet<FProperty*>* PropertiesToSkip = nullptr;

		/** True if we should call PreEditChange/PostEditChange on the object as it's imported. Pass false here
			if you're going to do that on your own. */
		bool bShouldCallEditChange = false;
	};

	/**
	 * Parse text and create the objects for the object specified.
	 * See ImportObjectPropertiesStep for the next step of the multi steps object import
	 * 
	 * @param	InParams	Parameters for object the multistep object import; see declaration of FMultiStepsImportObjectParams.
	 *
	 * @return	nullptr if the objects couldn't be imported
	 */
	UNREALED_API const TCHAR* ImportCreateObjectsStep(FMultiStepsImportObjectParams& InParams);

	/**
	 * Parse text and import the properties for the object specified and its subobjects.
	 * Call ImportCreateObjectsStep before calling this function to create the subobjects
	 * 
	 * @param	InParams	Parameters for object the multistep object import; see declaration of FMultiStepsImportObjectParams.
	 *
	 * @return	nullptr if the values couldn't be imported
	 */
	UNREALED_API const TCHAR* ImportObjectsPropertiesStep(FMultiStepsImportObjectParams& InParams);


}


extern UNREALED_API class FLevelEditorViewportClient* GCurrentLevelEditingViewportClient;

/** Tracks the last level editing viewport client that received a key press. */
extern UNREALED_API class FLevelEditorViewportClient* GLastKeyLevelEditingViewportClient;
