// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Settings/LevelEditorViewportSettings.h"

class AActor;
class FLevelEditorViewportClient;
class FSceneView;
class UActorFactory;
struct FViewportCursorLocation;

/** Positioning data struct */
struct FPositioningData
{
	FPositioningData(const FVector& InSurfaceLocation, const FVector& InSurfaceNormal)
		: SurfaceLocation(InSurfaceLocation)
		, SurfaceNormal(InSurfaceNormal)
		, PlacementExtent(0.f)
		, StartTransform(FTransform::Identity)
		, ActorFactory(nullptr)
		, bAlignRotation(false)
	{
		auto& Settings = GetDefault<ULevelEditorViewportSettings>()->SnapToSurface;
		bAlignRotation = Settings.bEnabled && Settings.bSnapRotation;
	}

	/** The surface location we want to position to */
	FVector SurfaceLocation;

	/** The surface normal we want to potentially align to */
	FVector SurfaceNormal;

	/** Placement extent offset to use (default = 0,0,0) */
	FPositioningData& UsePlacementExtent(const FVector& InPlacementExtent) { PlacementExtent = InPlacementExtent; return *this; }
	FVector PlacementExtent;

	/** The start transform we are using for positioning. Ensures a natural alignment to the surface when using rotation. (default = Identity) */
	FPositioningData& UseStartTransform(const FTransform& InStartTransform) { StartTransform = InStartTransform; return *this; }
	FTransform StartTransform;

	/** A factory to use for the alignment. Factories define the alignment routine and spawn offset amounts if necessary. */
	FPositioningData& UseFactory(const UActorFactory* InActorFactory) { ActorFactory = InActorFactory; return *this; }
	const UActorFactory* ActorFactory;

	/** Whether to align to the surface normal, or just snap to its position */
	FPositioningData& AlignToSurfaceRotation(bool bInAlignRotation) { bAlignRotation = bInAlignRotation; return *this; }
	bool bAlignRotation;
};

/** Structure used for positioning actors with snapping */
struct FSnappedPositioningData : FPositioningData
{
	FSnappedPositioningData(FLevelEditorViewportClient* InLevelViewportClient, const FVector& InSurfaceLocation, const FVector& InSurfaceNormal)
		: FPositioningData(InSurfaceLocation, InSurfaceNormal)
		, LevelViewportClient(InLevelViewportClient)
		, bDrawSnapHelpers(false)
	{}

	/** The level viewport - required for vertex snapping routines */
	FLevelEditorViewportClient* LevelViewportClient;

	/** Whether to draw vertex snapping helpers or not when snapping */
	FSnappedPositioningData& DrawSnapHelpers(bool bInDrawSnapHelpers) { bDrawSnapHelpers = bInDrawSnapHelpers; return *this; }
	bool bDrawSnapHelpers;

	/** Mask these construction helpers to return the correct type */
	FSnappedPositioningData& UsePlacementExtent(const FVector& InPlacementExtent) { PlacementExtent = InPlacementExtent; return *this; }
	FSnappedPositioningData& UseStartTransform(const FTransform& InStartTransform) { StartTransform = InStartTransform; return *this; }
	FSnappedPositioningData& UseFactory(const UActorFactory* InActorFactory) { ActorFactory = InActorFactory; return *this; }
	FSnappedPositioningData& AlignToSurfaceRotation(bool bInAlignRotation) { bAlignRotation = bInAlignRotation; return *this; }
};

struct FActorPositionTraceResult
{
	/** Enum representing the state of this result */
	enum ResultState
	{
		/** The trace found a valid hit target */
		HitSuccess,
		/** The trace found no valid targets, so chose a default position */
		Default,
		/** The trace failed entirely */
		Failed,
	};

	/** Constructor */
	FActorPositionTraceResult() : State(Failed), Location(0.f), SurfaceNormal(0.f, 0.f, 1.f) {}

	/** The state of this result */
	ResultState	State;

	/** The location of the preferred trace hit */
	FVector		Location;

	/** The surface normal of the trace hit */
	FVector		SurfaceNormal;

	/** Pointer to the actor that was hit, if any. nullptr otherwise */
	TWeakObjectPtr<AActor>	HitActor;
};

struct FActorPositioning
{
	/** Trace the specified world to find a position to snap actors to
	 *
	 *	@param	Cursor			The cursor position and direction to trace at
	 *	@param	View			The scene view that we are tracing on
	 *	@param	IgnoreActors	Optional array of actors to exclude from the trace
	 *
	 *	@return	Result structure containing the location and normal of a trace hit, or a default position in front of the camera on failure
	*/
	static UNREALED_API FActorPositionTraceResult TraceWorldForPositionWithDefault(const FViewportCursorLocation& Cursor, const FSceneView& View, const TArray<AActor*>* IgnoreActors = nullptr);
	
	/** Trace the specified world to find a position to snap actors to
	 *
	 *	@param	Cursor			The cursor position and direction to trace at
	 *	@param	View			The scene view that we are tracing on
	 *	@param	IgnoreActors	Optional array of actors to exclude from the trace
	 *
	 *	@return	Result structure containing the location and normal of a trace hit, or empty on failure
	*/
	static UNREALED_API FActorPositionTraceResult TraceWorldForPosition(const FViewportCursorLocation& Cursor, const FSceneView& View, const TArray<AActor*>* IgnoreActors = nullptr);

	/** Trace the specified world to find a position to snap actors to
	 *
	 *	@param	World			The world to trace
	 *	@param	InSceneView		The scene view that we are tracing on
	 *	@param	RayStart		The start of the ray in world space
	 *	@param	RayEnd			The end of the ray in world space
	 *	@param	IgnoreActors	Optional array of actors to exclude from the trace
	 *
	 *	@return	Result structure containing the location and normal of a trace hit, or empty on failure
	*/
	static UNREALED_API FActorPositionTraceResult TraceWorldForPosition(const UWorld& InWorld, const FSceneView& InSceneView, const FVector& RayStart, const FVector& RayEnd, const TArray<AActor*>* IgnoreActors = nullptr);

	/** Get a transform that should be used to spawn the specified actor using the global editor click location and plane
	 *
	 *	@param	Actor			The actor to spawn
	 *	@param	bSnap			Whether to perform snapping on the placement transform or not (defaults to true)
	 *	@param	InCursor		Optional pre-calculated cursor location
	 *
	 *	@return	The expected transform for the specified actor
	*/
	static UNREALED_API FTransform GetCurrentViewportPlacementTransform(const AActor& Actor, bool bSnap = true, const FViewportCursorLocation* InCursor = nullptr);

	/** Get a default actor position in front of the camera
	 *
	 *	@param	InActor				The actor
	 *	@param	InCameraOrigin		The location of the camera
	 *	@param	InCameraDirection	The view direction of the camera
	 *
	 *	@return	The expected transform for the specified actor
	*/
	static UNREALED_API FVector GetActorPositionInFrontOfCamera(const AActor& InActor, const FVector& InCameraOrigin, const FVector& InCameraDirection);
	
	/**
	 *	Get the snapped position and rotation transform for an actor to snap it to the specified surface. This will potentially align the actor to the specified surface normal, and offset it based on factory settings
	 *
	 *	@param	PositioningData			Positioning data used to calculate the transformation
	 *
	 *	@return	The expected transform for the specified actor
	 */
	static UNREALED_API FTransform GetSnappedSurfaceAlignedTransform(const FSnappedPositioningData& PositioningData);

	/**
	 *	Get the position and rotation transform for an actor to snap it to the specified surface. This will potentially align the actor to the specified surface normal, and offset it.
	 *
	 *	@param	PositioningData			Positioning data used to calculate the transformation
	 *
	 *	@return	The expected transform for the specified actor
	 */
	static UNREALED_API FTransform GetSurfaceAlignedTransform(const FPositioningData& PositioningData);
};
