// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector.h"
#include "UObject/WeakObjectPtr.h"

struct FCollisionQueryParams;
class FSceneView;
struct FViewportCursorLocation;
class UObject;

namespace UE::Positioning 
{
	struct FObjectPositioningTraceResult
	{
		/** Enum representing the state of this result */
		enum EResultState
		{
			/** The trace found a valid hit target */
			HitSuccess,
			/** The trace found no valid targets, so chose a default position */
			Default,
			/** The trace failed entirely */
			Failed,
		};

		/** Constructor */
		FObjectPositioningTraceResult() : State(Failed), Location(0.f), SurfaceNormal(0.f, 0.f, 1.f) {}

		/** The state of this result */
		EResultState State = EResultState::Failed;

		/** The location of the preferred trace hit */
		FVector Location = FVector::ZeroVector;

		/** The surface normal of the trace hit */
		FVector SurfaceNormal = FVector::UnitZ();

		/** 
		 * Pointer to the object that was hit, if any.
		 * NOTE: This currently only works reliably for giving back an owning actor,
		 * if there is one, so it is not yet reliable otherwise.
		 */
		TWeakObjectPtr<UObject> HitObject;
	};

	/** 
	 * Trace the specified world to find a position to place an object. If nothing is hit, use a default
	 * position in front of the camera (determined by ULevelEditorViewportSettings::BackgroundDropDistance)
	 *
	 *	@param	Cursor			The cursor position and direction to trace at
	 *	@param	View			The scene view that we are tracing on
	 *	@param	CollisionQueryParams	Optional existing collision query parameters to 
	 *    use, except for TraceTag and bTraceComplex, which are overriden by the function.
	 *
	 *	@return	Result structure containing the location and normal of a trace hit, or a default position in front of the camera on failure
	*/
	UNREALED_API FObjectPositioningTraceResult TraceWorldForPositionWithDefault(const FViewportCursorLocation& Cursor, const FSceneView& View, FCollisionQueryParams* CollisionQueryParams);
	
	/** 
	 * Trace the specified world to find a position to place an object.
	 *
	 *	@param	Cursor			The cursor position and direction to trace at
	 *	@param	View			The scene view that we are tracing on
	 *	@param	CollisionQueryParams	Optional existing collision query parameters to 
	 *    use, except for TraceTag and bTraceComplex, which are overriden by the function.
	 *
	 *	@return	Result structure containing the location and normal of a trace hit, or empty on failure
	*/
	UNREALED_API FObjectPositioningTraceResult TraceWorldForPosition(const FViewportCursorLocation& Cursor, const FSceneView& View, FCollisionQueryParams* CollisionQueryParams);

	/** 
	 * Trace the specified world to find a position to place an object.
	 *
	 *	@param	World			The world to trace
	 *	@param	InSceneView		The scene view that we are tracing on
	 *	@param	RayStart		The start of the ray in world space
	 *	@param	RayEnd			The end of the ray in world space
	 *	@param	CollisionQueryParams	Optional existing collision query parameters to 
	 *    use, except for TraceTag and bTraceComplex, which are overriden by the function.
	 *
	 *	@return	Result structure containing the location and normal of a trace hit, or empty on failure
	*/
	UNREALED_API FObjectPositioningTraceResult TraceWorldForPosition(const UWorld& InWorld, const FSceneView& InSceneView, const FVector& RayStart, const FVector& RayEnd, FCollisionQueryParams* CollisionQueryParams);
}