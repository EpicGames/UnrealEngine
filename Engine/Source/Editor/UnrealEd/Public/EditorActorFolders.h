// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorEditorContextState.h"
#include "Folder.h"
#include "IActorEditorContextClient.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/GCObject.h"
#include "WorldFolders.h"
#include "EditorActorFolders.generated.h"

class FObjectPostSaveContext;
class AActor;
class UActorFolder;
class FWorldPartitionActorDesc;
class FWorldPartitionActorDescInstance;

/** Multicast delegates for broadcasting various folder events */

//~ Begin Deprecated
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnActorFolderCreate, UWorld&, FName);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnActorFolderDelete, UWorld&, FName);
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnActorFolderMove, UWorld&, FName /* src */, FName /* dst */);
//~ End Deprecated

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnActorFolderCreated, UWorld&, const FFolder&);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnActorFolderDeleted, UWorld&, const FFolder&);
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnActorFolderMoved, UWorld&, const FFolder& /* src */, const FFolder& /* dst */);

UCLASS()
class UActorEditorContextActorFolderState : public UActorEditorContextClientState
{
	GENERATED_BODY()

public:
	UPROPERTY(VisibleAnywhere, Category = "Actor Folder")
	FName FolderPath;
};


/** Class responsible for managing an in-memory representation of actor folders in the editor */
struct FActorFolders : public FGCObject, public IActorEditorContextClient
{
	UNREALED_API FActorFolders();
	UNREALED_API ~FActorFolders();

	// FGCObject Interface
	UNREALED_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return "FActorFolders";
	}
	// End FGCObject Interface

	/** Singleton access */
	static UNREALED_API FActorFolders& Get();

	/** Folder creation and deletion events. Called whenever a folder is created or deleted in a world. */
	static UNREALED_API FOnActorFolderCreated OnFolderCreated;
	static UNREALED_API FOnActorFolderMoved 	OnFolderMoved;
	static UNREALED_API FOnActorFolderDeleted OnFolderDeleted;
	
	static UNREALED_API FFolder GetActorDescInstanceFolder(UWorld& InWorld, const FWorldPartitionActorDescInstance* InActorDescInstance);
	/** Apply an operation to each actor desc in the given list of folders. */
			
	static UNREALED_API void ForEachActorDescInstanceInFolders(UWorld& InWorld, const TSet<FName>& InPaths, TFunctionRef<bool(const FWorldPartitionActorDescInstance*)> Operation, const FFolder::FRootObject& InFolderRootObject = FFolder::GetInvalidRootObject());

	/** Apply an operation to each actor in the given list of folders. Will stop when operation returns false. */
	static UNREALED_API void ForEachActorInFolders(UWorld& InWorld, const TArray<FName>& InPaths, TFunctionRef<bool(AActor*)> Operation, const FFolder::FRootObject& InFolderRootObject = FFolder::GetInvalidRootObject());
	static UNREALED_API void ForEachActorInFolders(UWorld& InWorld, const TSet<FName>& InPaths, TFunctionRef<bool(AActor*)> Operation, const FFolder::FRootObject& InFolderRootObject = FFolder::GetInvalidRootObject());

	/** Get an array of actors from a list of folders */
	static UNREALED_API void GetActorsFromFolders(UWorld& InWorld, const TArray<FName>& InPaths, TArray<AActor*>& OutActors, const FFolder::FRootObject& InFolderRootObject = FFolder::GetInvalidRootObject());
	static UNREALED_API void GetActorsFromFolders(UWorld& InWorld, const TSet<FName>& InPaths, TArray<AActor*>& OutActors, const FFolder::FRootObject& InFolderRootObject = FFolder::GetInvalidRootObject());

	/** Get an array of weak actor pointers from a list of folders */
	static UNREALED_API void GetWeakActorsFromFolders(UWorld& InWorld, const TArray<FName>& InPaths, TArray<TWeakObjectPtr<AActor>>& OutActors, const FFolder::FRootObject& InFolderRootObject = FFolder::GetInvalidRootObject());
	static UNREALED_API void GetWeakActorsFromFolders(UWorld& InWorld, const TSet<FName>& InPaths, TArray<TWeakObjectPtr<AActor>>& OutActors, const FFolder::FRootObject& InFolderRootObject = FFolder::GetInvalidRootObject());

	/** Tests whether a folder container exists for the specified world */
	UNREALED_API bool IsInitializedForWorld(UWorld& InWorld) const;
	
	/** Initialize a folder container for the specified world */
	UNREALED_API void InitializeForWorld(UWorld& InWorld);

	/** Get a default folder name under the specified parent path */
	UNREALED_API FFolder GetDefaultFolderName(UWorld& InWorld, const FFolder& InParentFolder);
	
	/** Get a new default folder name that would apply to the current selection */
	UNREALED_API FFolder GetDefaultFolderForSelection(UWorld& InWorld, TArray<FFolder>* InSelectedFolders = nullptr);

	/** Get folder name that is unique under specified parent path */
	UNREALED_API FFolder GetFolderName(UWorld& InWorld, const FFolder& InParentFolder, const FName& InLeafName);

	/** Create a new folder in the specified world, of the specified path */
	UNREALED_API bool CreateFolder(UWorld& InWorld, const FFolder& InFolder);

	/** Same as CreateFolder, but moves the current actor selection into the new folder as well */
	UNREALED_API void CreateFolderContainingSelection(UWorld& InWorld, const FFolder& InFolder);

	/** Sets the folder path for all the selected actors */
	UNREALED_API void SetSelectedFolderPath(const FFolder& InFolder) const;

	/** Delete the specified folder in the world */
	UNREALED_API void DeleteFolder(UWorld& InWorld, const FFolder& InFolderToDelete);

	/** Rename the specified path to a new name */
	UNREALED_API bool RenameFolderInWorld(UWorld& InWorld, const FFolder& OldPath, const FFolder& NewPath);

	/** Notify that a root object has been removed. Cleanup of existing folders with with root object. */
	UNREALED_API void OnFolderRootObjectRemoved(UWorld& InWorld, const FFolder::FRootObject& InFolderRootObject);

	/** Return if folder exists */
	UNREALED_API bool ContainsFolder(UWorld& InWorld, const FFolder& InFolder);

	/** Return the folder expansion state */
	UNREALED_API bool IsFolderExpanded(UWorld& InWorld, const FFolder& InFolder);

	/** Set the folder expansion state */
	UNREALED_API void SetIsFolderExpanded(UWorld& InWorld, const FFolder& InFolder, bool bIsExpanded);

	/** Iterate on all folders of a world and pass it to the provided operation. */
	UNREALED_API void ForEachFolder(UWorld& InWorld, TFunctionRef<bool(const FFolder&)> Operation);

	/** Iterate on all world's folders with the given root object and pass it to the provided operation. */
	UNREALED_API void ForEachFolderWithRootObject(UWorld& InWorld, const FFolder::FRootObject& InFolderRootObject, TFunctionRef<bool(const FFolder&)> Operation);

	/** Get the folder properties for the specified path. Returns nullptr if no properties exist */
	UNREALED_API FActorFolderProps* GetFolderProperties(UWorld& InWorld, const FFolder& InFolder);

	//~ Begin IActorEditorContextClient interface
	virtual void OnExecuteActorEditorContextAction(UWorld* InWorld, const EActorEditorContextAction& InType, class AActor* InActor = nullptr) override;
	virtual void CaptureActorEditorContextState(UWorld* InWorld, UActorEditorContextStateCollection* InStateCollection) const override;
	virtual void RestoreActorEditorContextState(UWorld* InWorld, const UActorEditorContextStateCollection* InStateCollection) override;
	virtual bool GetActorEditorContextDisplayInfo(UWorld* InWorld, FActorEditorContextClientDisplayInfo& OutDiplayInfo) const override;
	virtual bool CanResetContext(UWorld* InWorld) const override { return true; };
	virtual TSharedRef<SWidget> GetActorEditorContextWidget(UWorld* InWorld) const override;
	virtual FOnActorEditorContextClientChanged& GetOnActorEditorContextClientChanged() override { return ActorEditorContextClientChanged; }
	//~ End IActorEditorContextClient interface
	UNREALED_API FFolder GetActorEditorContextFolder(UWorld& InWorld, bool bMustMatchCurrentLevel = true) const;
	UNREALED_API void SetActorEditorContextFolder(UWorld& InWorld, const FFolder& InFolder);

private:
	
	static UNREALED_API FFolder::FRootObject GetWorldFolderRootObject(UWorld& InWorld);

	/** Broadcast when actor folder is created. */
	UNREALED_API void BroadcastOnActorFolderCreated(UWorld& InWorld, const FFolder& InFolder);

	/** Broadcast when actor folder is deleted. */
	UNREALED_API void BroadcastOnActorFolderDeleted(UWorld& InWorld, const FFolder& InFolder);

	/** Broadcast when actor folder has moved. */
	UNREALED_API void BroadcastOnActorFolderMoved(UWorld& InWorld, const FFolder& InSrcFolder, const FFolder& InDstFolder);

	/** Get or create a folder container for the specified world */
	UNREALED_API UWorldFolders& GetOrCreateWorldFolders(UWorld& InWorld);

	/** Create and update a folder container for the specified world */
	UNREALED_API UWorldFolders& CreateWorldFolders(UWorld& InWorld);

	/** Called when an actor's folder has changed */
	UNREALED_API void OnActorFolderChanged(const AActor* InActor, FName OldPath);

	/** Called when an actor folder is added */
	UNREALED_API void OnActorFolderAdded(UActorFolder* InActorFolder);

	/** Called after a world has been saved */
	UNREALED_API void OnWorldSaved(UWorld* World, FObjectPostSaveContext ObjectSaveContext);

	/** Attempt to save the folders state */
	UNREALED_API void SaveWorldFoldersState(UWorld* World);

	/** Remove any references to folder arrays for dead worlds */
	UNREALED_API void Housekeeping();

	/** Add a folder to the folder map for the specified world. Does not trigger any events. */
	UNREALED_API bool AddFolderToWorld(UWorld& InWorld, const FFolder& InFolder);

	/** Removed folders from specified world. Can optionally trigger delete events. */
	UNREALED_API void RemoveFoldersFromWorld(UWorld& InWorld, const TArray<FFolder>& InFolders, bool bBroadcastDelete);

	/** Transient map of folders, keyed on world pointer */
	TMap<TWeakObjectPtr<UWorld>, TObjectPtr<UWorldFolders>> WorldFolders;

	/** Called when ActorEditorContextClient changed. */
	UNREALED_API void BroadcastOnActorEditorContextClientChanged(UWorld& InWorld);

	/** Delegate used to notify changes to ActorEditorContextSubsystem */
	FOnActorEditorContextClientChanged ActorEditorContextClientChanged;

	bool bAnyLevelsChanged;

	friend UWorldFolders;
};
