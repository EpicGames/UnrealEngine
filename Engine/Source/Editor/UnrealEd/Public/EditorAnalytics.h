// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FAnalyticsEventAttribute;

namespace EAnalyticsErrorCodes
{
	/**
	 * Error Codes for a variety of tool errors
	 * NOTE: this needs to be kept in sync with iPhonePackager.cs and AutomationTool/Program.cs
	 * Any new error added here needs a text string in TranslateErrorCode
	 */
	enum Type
	{
		UATNotFound = -1,
		Success = 0,
		Unknown = 1,
		Arguments = 2,
		UnknownCommand = 3,
		SDKNotFound = 10,
		ProvisionNotFound = 11,
		CertificateNotFound = 12,
		ManifestNotFound = 14,
		KeyNotFound = 15,
		ProvisionExpired = 16,
		CertificateExpired = 17,
		CertificateProvisionMismatch = 18,
		CodeUnsupported = 19,
		PluginsUnsupported = 20,
		UnknownCookFailure = 25,
		UnknownDeployFailure = 26,
		UnknownBuildFailure = 27,
		UnknownPackageFailure = 28,
		UnknownLaunchFailure = 29,
		StageMissingFile = 30,
		FailedToCreateIPA = 31,
		FailedToCodeSign = 32,
		DeviceBackupFailed = 33,
		AppUninstallFailed = 34,
		AppInstallFailed = 35,
		AppNotFound = 36,
		StubNotSignedCorrectly = 37,
		IPAMissingInfoPList = 38,
		DeleteFile = 39,
		DeleteDirectory = 40,
		CreateDirectory = 41,
		CopyFile = 42,
		OnlyOneObbFileSupported = 50,
		FailureGettingPackageInfo = 51,
		OnlyOneTargetConfigurationSupported = 52,
		ObbNotFound = 53,
		AndroidBuildToolsPathNotFound = 54,
		NoApkSuitableForArchitecture = 55,
		FilesInstallFailed = 56,
		RemoteCertificatesNotFound = 57,
		LauncherFailed = 100,
		UATLaunchFailure = 101,
		FailedToDeleteStagingDirectory = 102,
		MissingExecutable = 103,
		DeviceNotSetupForDevelopment = 150,
		DeviceOSNewerThanSDK = 151,
		TestFailure = 152,
		SymbolizedSONotFound = 153,
		LicenseNotAccepted = 154,
		AndroidOBBError = 155,
		SDKInstallFailed = 200,
		DeviceUpdateFailed = 201,
	};
};

class FEditorAnalytics
{
public:
	/**
	* Reports an event to the analytics system if it is enabled
	*
	* @param EventName - name of the event
	* @param PlatformName - name of the platform being used
	*/
	UNREALED_API static void ReportEvent(FString EventName, FString PlatformName, bool bHasCode);

	/**
	* Reports an event to the analytics system if it is enabled with some extra parameters
	*
	* @param EventName - name of the event
	* @param PlatformName - name of the platform being used
	* @param ExtraParams - extra data needed by the event
	*/
	UNREALED_API static void ReportEvent(FString EventName, FString PlatformName, bool bHasCode, const TArray<FAnalyticsEventAttribute>& ExtraParams);

	/**
	* Reports an event to the analytics system if it is enabled with some extra parameters with an error code
	*
	* @param EventName - name of the event
	* @param PlatformName - name of the platform being used
	* @param ErrorCode - error code of the event
	* @param ExtraParams - extra data needed by the event
	*/
	UNREALED_API static void ReportEvent(FString EventName, FString PlatformName, bool bHasCode, int32 ErrorCode, TArray<FAnalyticsEventAttribute>& ExtraParams);

	/**
	* Reports an event to the analytics system of a build requirements failure
	*
	* @param EventName - name of the event
	* @param PlatformName - name of the platform being used
	* @param Requirements - requirements that didn't pass
	*/
	UNREALED_API static void ReportBuildRequirementsFailure(FString EventName, FString PlatformName, bool bHasCode, int32 Requirements);

public:

	/* Translates the error code to something human readable */
	UNREALED_API static FString TranslateErrorCode(int32 ErrorCode);

	/* Determine whether the error code should cause the reporter to display the message in a dialog. */
	UNREALED_API static bool ShouldElevateMessageThroughDialog(const int32 ErrorCode);
};
