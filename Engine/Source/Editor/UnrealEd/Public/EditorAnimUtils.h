// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/UnrealType.h"
#include "Animation/AnimCurveTypes.h"
#include "AssetRegistry/AssetData.h"

class UAnimationAsset;
class UAnimBlueprint;
class UAnimSequenceBase;

namespace EditorAnimUtils
{
	struct FNameDuplicationRule
	{
		FString Prefix;
		FString Suffix;
		// replace string
		FString ReplaceFrom;
		FString ReplaceTo;
		// folder path
		FString FolderPath;

		FNameDuplicationRule() 
			: Prefix(), Suffix(), ReplaceFrom(), ReplaceTo(), FolderPath(TEXT("/Game"))
		{}

		UNREALED_API FString Rename(const UObject* Asset) const;
	};

	class FAnimationRetargetContext
	{
	public:
		UNREALED_API FAnimationRetargetContext(const TArray<FAssetData>& AssetsToRetarget, bool bRetargetReferredAssets, bool bInConvertAnimationDataInComponentSpaces, const FNameDuplicationRule& NameRule=FNameDuplicationRule());
		UNREALED_API FAnimationRetargetContext(TArray<TWeakObjectPtr<UObject>> AssetsToRetarget, bool bRetargetReferredAssets, bool bInConvertAnimationDataInComponentSpaces, const FNameDuplicationRule& NameRule=FNameDuplicationRule());

		/** Were we supplied anything that we can retarget */
		UNREALED_API bool HasAssetsToRetarget() const;

		/** Did we duplicate any assets */
		UNREALED_API bool HasDuplicates() const;

		/** Returns the UObject that was chosen to retarget if there was only one in the first place */
		UNREALED_API UObject* GetSingleTargetObject() const;

		/** Returns the duplicate of the supplied object if there is one, otherwise NULL */
		UNREALED_API UObject* GetDuplicate(const UObject* OriginalObject) const;

		/** return all duplicates */
		UNREALED_API TArray<UObject*> GetAllDuplicates() const;

		/** Duplicates the assets stored for retargetting, populating maps of original assets to new asset */
		UNREALED_API void DuplicateAssetsToRetarget(UPackage* DestinationPackage, const FNameDuplicationRule* NameRule);

		/** Retarget the contained assets */
		UNREALED_API void RetargetAnimations(USkeleton* OldSkeleton, USkeleton* NewSkeleton);
	
		UNREALED_API void AddRemappedAsset(UAnimationAsset* OriginalAsset, UAnimationAsset* NewAsset);

	private:
		/** Lists of assets to retarget. Populated from FAssetData supplied to constructor */
		TArray<UAnimationAsset*>	AnimationAssetsToRetarget;
		TArray<UAnimBlueprint*>		AnimBlueprintsToRetarget;

		/** Lists of original assets map to duplicate assets */
		TMap<UAnimationAsset*, UAnimationAsset*>	DuplicatedAnimAssets;
		TMap<UAnimBlueprint*, UAnimBlueprint*>		DuplicatedBlueprints;

		TMap<UAnimationAsset*, UAnimationAsset*>	RemappedAnimAssets;

		/** If we only chose one object to retarget store it here */
		UObject* SingleTargetObject;

		/** whether to convert animation data in component spaces */
		bool bConvertAnimationDataInComponentSpaces;

		/** Initialize the object, only to be called by constructors */
		void Initialize(TArray<TWeakObjectPtr<UObject>> AssetsToRetarget, bool bRetargetReferredAssets);
	};

	/**
	 * Retargets the supplied UObjects (as long as they are an animation asset), optionally duplicating them and retargeting their reference assets too
	 *
	 * @param NewSkeleton						The skeleton the supplied assets should be retargeted to
	 * @param AssetsToRetarget					The animation assets to copy/retarget
	 * @param bRetargetReferredAssets			If true retargets any assets referred to by assets in AssetsToRetarget. If false then the references are cleared.
	 * @param bDuplicatedAssetsBeforeRetarget	If true the assets are duplicated and then the duplicates are retargeted
	 * @param bConvertSpace						Do the conversion in component space of the animation to match new target
	 */
	UNREALED_API UObject* RetargetAnimations(USkeleton* OldSkeleton, USkeleton* NewSkeleton, TArray<TWeakObjectPtr<UObject>> AssetsToRetarget, bool bRetargetReferredAssets, const FNameDuplicationRule* NameRule, bool bConvertSpace);

	/**
	 * Retargets the supplied FAssetDatas (as long as they are an animation asset), optionally duplicating them and retargetting their reference assets too
	 *
	 * @param NewSkeleton						The skeleton the supplied assets should be retargeted to
	 * @param AssetsToRetarget					The animation assets to copy/retarget
	 * @param bRetargetReferredAssets			If true retargets any assets referred to by assets in AssetsToRetarget. If false then the references are cleared.
	 * @param bDuplicatedAssetsBeforeRetarget	If true the assets are duplicated and then the duplicates are retargeted
	 * @param bConvertSpace						Do the conversion in component space of the animation to match new target
	 */
	UNREALED_API UObject* RetargetAnimations(USkeleton* OldSkeleton, USkeleton* NewSkeleton, const TArray<FAssetData>& AssetsToRetarget, bool bRetargetReferredAssets, const FNameDuplicationRule* NameRule, bool bConvertSpace);

	/**
	 * Retargets the supplied FAnimationRetargetContext, optionally duplicating the assets and retargeting the assets reference assets too. Is called by other overloads of RetargetAnimations
	 *
	 * @param NewSkeleton						The skeleton the supplied assets should be retargeted to
	 * @param AssetsToRetarget					The animation assets to copy/retarget
	 * @param bRetargetReferredAssets			If true retargets any assets referred to by assets in AssetsToRetarget. If false then the references are cleared.
	 * @param bDuplicatedAssetsBeforeRetarget	If true the assets are duplicated and then the duplicates are retargeted
	 * @param bConvertSpace						Do the conversion in component space of the animation to match new target
	 */
	UNREALED_API UObject* RetargetAnimations(USkeleton* OldSkeleton, USkeleton* NewSkeleton, FAnimationRetargetContext& RetargetContext, bool bRetargetReferredAssets, const FNameDuplicationRule* NameRule);

	// Populates the supplied TArrays with any animation assets that this blueprint refers too
	UNREALED_API void GetAllAnimationSequencesReferredInBlueprint(UAnimBlueprint* AnimBlueprint, TArray<UAnimationAsset*>& AnimationAsset);

	// Replaces references to any animations found with the match animation from the map
	UNREALED_API void ReplaceReferredAnimationsInBlueprint(UAnimBlueprint* AnimBlueprint, const TMap<UAnimationAsset*, UAnimationAsset*>& AnimAssetReplacementMap);

	/**
	 * Duplicates the supplied AssetsToDuplicate and returns a map of original asset to duplicate
	 *
	 * @param	AssetsToDuplicate	The animations to duplicate
	 * @param	DestinationPackage	The package that the duplicates should be placed in
	 * @param	NameRule			Options for renaming the file, adding prefix/suffix or search/replace
	 *
	 * @return	TMap of original animation to duplicate
	 */
	UNREALED_API TMap<UObject*, UObject*> DuplicateAssetsInternal( const TArray<UObject*>& AssetsToDuplicate, UPackage* DestinationPackage, const FNameDuplicationRule* NameRule);

	/**
	 * Duplicates the supplied AssetsToDuplicate and returns a map of original asset to duplicate. Templated wrapper that calls DuplicateAssetInternal.
	 *
	* @param	AssetsToDuplicate	The animations to duplicate
	 * @param	DestinationPackage	The package that the duplicates should be placed in
	 * @param	NameRule			Options for renaming the file, adding prefix/suffix or search/replace
	 *
	 * @return	TMap of original animation to duplicate
	 */
	template<class AssetType>
	TMap<AssetType*, AssetType*> DuplicateAssets(const TArray<AssetType*>& AssetsToDuplicate, UPackage* DestinationPackage, const FNameDuplicationRule* NameRule)
	{
		TArray<UObject*> Assets;
		for(auto Iter = AssetsToDuplicate.CreateConstIterator(); Iter; ++Iter)
		{
			Assets.Add(*Iter);
		}

		TMap<UObject*, UObject*> AssetMap = DuplicateAssetsInternal(Assets, DestinationPackage, NameRule);

		TMap<AssetType*, AssetType*> ReturnMap;
		for(auto Iter = AssetMap.CreateIterator(); Iter; ++Iter)
		{
			ReturnMap.Add(Cast<AssetType>(Iter->Key), Cast<AssetType>(Iter->Value));
		}
		return ReturnMap;
	}

	template<class AssetType>
	void GetAssetsFromProperties(TArray<FProperty*> InProperties, UObject* Scope, TArray<AssetType*>& OutAssets)
	{
		check(Scope);

		OutAssets.Empty();
		for(FProperty* Prop : InProperties)
		{
			if(Prop)
			{
				if(FArrayProperty* ArrayProp = CastField<FArrayProperty>(Prop))
				{
					// Blueprint array
					FScriptArrayHelper Helper(ArrayProp, Prop->ContainerPtrToValuePtr<uint8>(Scope));
					const int32 ArrayNum = Helper.Num();
					for(int32 Idx = 0; Idx < ArrayNum; ++Idx)
					{
						// These were gathered from UObject types so we know this should succeed
						UObject** Object = (UObject**)Helper.GetRawPtr(Idx);
						if(AssetType* Asset = Cast<AssetType>(*Object))
						{
							OutAssets.Add(Asset);
						}
					}
				}
				else if(Prop->ArrayDim > 1)
				{
					// Native array
					for(int32 Idx = 0; Idx < Prop->ArrayDim; ++Idx)
					{
						if(UObject** ResolvedObject = Prop->ContainerPtrToValuePtr<UObject*>(Scope, Idx))
						{
							if(AssetType* Asset = Cast<AssetType>(*ResolvedObject))
							{
								OutAssets.Add(Asset);
							}
						}
					}
				}
				else if(UObject** ResolvedObject = Prop->ContainerPtrToValuePtr<UObject*>(Scope))
				{
					// Normal property
					if(AssetType* Asset = Cast<AssetType>(*ResolvedObject))
					{
						OutAssets.Add(Asset);
					}
				}
			}
		}
	}

	// utility functions
	UE_DEPRECATED(5.0, "CopyAnimCurves has been deprecated, see UAnimationBlueprintLibrary::CopyAnimationCurveNamesToSkeleton instead")
	UNREALED_API void CopyAnimCurves(USkeleton* OldSkeleton, USkeleton* NewSkeleton, UAnimSequenceBase* SequenceBase, const FName ContainerName, ERawCurveTrackTypes CurveType);

} // namespace EditorAnimUtils
