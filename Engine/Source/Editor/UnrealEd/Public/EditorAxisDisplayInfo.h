// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/AxisDisplayInfo.h"

#define UE_API UNREALED_API

class FEditorAxisDisplayInfo : public IAxisDisplayInfo
{
public:

	UE_API FEditorAxisDisplayInfo();
	virtual ~FEditorAxisDisplayInfo() = default;

	UE_API virtual EAxisList::Type GetAxisDisplayCoordinateSystem() const override;
	
	UE_API virtual FText GetAxisToolTip(EAxisList::Type Axis) const override;
	UE_API virtual FText GetAxisDisplayName(EAxisList::Type Axis) override;
	UE_API virtual FText GetAxisDisplayNameShort(EAxisList::Type Axis) override;
	

	UE_API virtual FLinearColor GetAxisColor(EAxisList::Type Axis) override;

	UE_API virtual bool UseForwardRightUpDisplayNames() override;

	UE_API virtual FText GetRotationAxisToolTip(EAxisList::Type Axis) const override;
	UE_API virtual FText GetRotationAxisName(EAxisList::Type Axis) override;
	UE_API virtual FText GetRotationAxisNameShort(EAxisList::Type Axis) override;
	
	UE_API virtual FIntVector4 DefaultAxisComponentDisplaySwizzle() const override;

private:
	// Maps the given axis from FLU -> XYZ if the AxisDisplayCoordinateSystem is XYZ
	UE_API EAxisList::Type MapAxis(EAxisList::Type Axis) const;

	// Inits info stored in settings, like the axis colors
	UE_API void InitSettingsInfo(double EditorStartupTime);

	TOptional<bool> bUseForwardRightUpDisplayNames;
	mutable TOptional<EAxisList::Type> AxisDisplayCoordinateSystem;

};

#undef UE_API
