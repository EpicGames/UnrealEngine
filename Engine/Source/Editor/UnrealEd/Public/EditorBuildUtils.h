// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	EditorBuildUtils.h: Utilities for building in the editor
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineBaseTypes.h"
#include "SceneTypes.h"
#include "RHIDefinitions.h"

struct FScopedSlowTask;

/** Names of the built-in editor build types. */
struct FBuildOptions
{
	/** Build all geometry */
	UNREALED_API static const FName BuildGeometry;
	/** Build only visible geometry */
	UNREALED_API static const FName BuildVisibleGeometry;
	/** Build lighting */
	UNREALED_API static const FName BuildLighting;
	/** Build all AI paths */
	UNREALED_API static const FName BuildAIPaths;
	/** Build only selected AI paths */
	UNREALED_API static const FName BuildSelectedAIPaths;
	/** Build everything */
	UNREALED_API static const FName BuildAll;
	/** Build everything and submit to source control */
	UNREALED_API static const FName BuildAllSubmit;
	/** Build everything except for paths only build selected */
	UNREALED_API static const FName BuildAllOnlySelectedPaths;
	/** Build HLODs */
	UNREALED_API static const FName BuildHierarchicalLOD;
	/** Build HLODs for selected actors */
	UNREALED_API static const FName BuildHierarchicalLOD_SelectedActors;
	/** Build minimap */
	UNREALED_API static const FName BuildMinimap;
	/** Build landscape spline meshes */
	UNREALED_API static const FName BuildLandscapeSplineMeshes;
	/** Build texture streaming */
	UNREALED_API static const FName BuildTextureStreaming;
	/** Build virtual textures */
	UNREALED_API static const FName BuildVirtualTexture;
	/** Build All Landscape Data */
	UNREALED_API static const FName BuildAllLandscape;
};
/**
 * Result of a custom editor build.
 */
enum class EEditorBuildResult : uint8
{
	Success,		// The build step completed successfully
	Skipped,		// The build step was skipped for some reason (e.g. cancelled)
	InProgress,		// The build step is running asynchronously
};

/**
 * Delegate to validate if a custom editor build can be executed.
 * @param UWorld* The world to run the build on.
 * @param FName The Id of the build being run (either the registered build Id, or one of the BuildAll types).
 * @return Whether the build step could run or not.
 */
DECLARE_DELEGATE_RetVal_TwoParams(bool, FCanDoEditorBuildDelegate, const UWorld*, FName);

/**
 * Delegate for performing a custom editor build.
 * @param UWorld* The world to run the build on.
 * @param FName The Id of the build being run (either the registered build Id, or one of the BuildAll types).
 * @return Status of the build step.
 */
DECLARE_DELEGATE_RetVal_TwoParams(EEditorBuildResult, FDoEditorBuildDelegate, UWorld*, FName);

/** Utility class to hold functionality for building within the editor */
class FEditorBuildUtils
{
public:
	/** Enumeration representing automated build behavior in the event of an error */
	enum EAutomatedBuildBehavior
	{
		ABB_PromptOnError,	// Modally prompt the user about the error and ask if the build should proceed
		ABB_FailOnError,	// Fail and terminate the automated build in response to the error
		ABB_ProceedOnError	// Acknowledge the error but continue with the automated build in spite of it
	};

	/** Helper struct to specify settings for an automated editor build */
	struct FEditorAutomatedBuildSettings
	{
		/** Constructor */
		UNREALED_API FEditorAutomatedBuildSettings();

		/** Behavior to take when a map build results in map check errors */
		EAutomatedBuildBehavior BuildErrorBehavior;

		/** Behavior to take when a map file cannot be checked out for some reason */
		EAutomatedBuildBehavior UnableToCheckoutFilesBehavior;

		/** Behavior to take when a map is discovered which has never been saved before */
		EAutomatedBuildBehavior NewMapBehavior;

		/** Behavior to take when a saveable map fails to save correctly */
		EAutomatedBuildBehavior FailedToSaveBehavior;

		/** Use SCC to checkout/checkin files */
		bool bUseSCC;

		/** If true, built map files not already in the source control depot will be added */
		bool bAutoAddNewFiles;

		/** If true, the editor will shut itself down upon completion of the automated build */
		bool bShutdownEditorOnCompletion;

		/** If true, the editor will check in all checked out packages */
		bool bCheckInPackages;

		/** Populate list with selected packages to check in */
		TArray<FString> PackagesToCheckIn;

		/** Changelist description to use for the submission of the automated build */
		FString ChangeDescription;
	};

	/**
	 * Start an automated build of all current maps in the editor. Upon successful conclusion of the build, the newly
	 * built maps will be submitted to source control.
	 *
	 * @param	BuildSettings		Build settings used to dictate the behavior of the automated build
	 * @param	OutErrorMessages	Error messages accumulated during the build process, if any
	 *
	 * @return	true if the build/submission process executed successfully; false if it did not
	 */
	static UNREALED_API bool EditorAutomatedBuildAndSubmit( const FEditorAutomatedBuildSettings& BuildSettings, FText& OutErrorMessages );

	/**
	 * Validate if the editor build matching the specified id can be executed.
	 * @param	InWorld				WorldContext
	 * @param	Id					Action Id specifying what kind of build is requested
	 * @return	true if the build can be executed for the provided world
	 */
	static UNREALED_API bool EditorCanBuild( UWorld* InWorld, FName Id );

	/**
	 * Perform an editor build with behavior dependent upon the specified id
	 *
	 * @param	InWorld				WorldContext
	 * @param	Id					Action Id specifying what kind of build is requested
	 * @param	bAllowLightingDialog True if the build lighting dialog should be displayed if we're building lighting only
	 *
	 * @return	true if the build completed successfully; false if it did not (or was manually canceled)
	 */
	static UNREALED_API bool EditorBuild( UWorld* InWorld, FName Id, const bool bAllowLightingDialog = true );

	/**
	 * Update every used material texture binding (see FMaterialTextureInfo::TextureIndex)
	 *
	 * @param	InWorld				WorldContext
	 */
	static UNREALED_API void UpdateTextureStreamingMaterialBindings( UWorld* InWorld );

	/**
	 * Perform an editor build for texture streaming
	 *
	 * @param	InWorld				WorldContext
	 * @param	SelectedViewMode	The viewmode to build the data for. Unkown when running the full build.
	 *
	 * @return	true if the build completed successfully; false if it did not (or was manually canceled)
	 */
	static UNREALED_API bool EditorBuildTextureStreaming( UWorld* InWorld, EViewModeIndex SelectedViewMode = VMI_Unknown);


	/**
	* Similar to EditorBuildTextureStreaming, but attempts to rebuild TextureStreamingData for all materials in loaded package(s).
	*
	* @param	Package				If non-NULL, only looks at materials under the specified package.
	*
	* @return	true if any packages were dirtied in the process.
	*/
	static UNREALED_API bool EditorBuildMaterialTextureStreamingData(UPackage* Package);

	/**
	 * Perform an editor build for virtual textures
	 *
	 * @param	InWorld				WorldContext
	 *
	 * @return	true if the build completed successfully; false if it did not (or was manually canceled)
	 */
	static UNREALED_API bool EditorBuildVirtualTexture(UWorld* InWorld);

	/**
	* Perform an editor build for All Landscape Data
	* @param	InWorld				WorldContext
	*/
	static UNREALED_API void EditorBuildAllLandscape(UWorld* InWorld);

	/** 
	* check if navigation build was was triggered from editor as user request
	*
	* @return	true if the build was triggered as user request; false if it did not 
	*/
	static bool IsBuildingNavigationFromUserRequest() { return bBuildingNavigationFromUserRequest; }

	/** 
	* call it to notify that navigation builder finished building 
	*/
	static void PathBuildingFinished() { bBuildingNavigationFromUserRequest = false; }

	/**
	 * Call this when an async custom build step has completed (successfully or not).
	 */
	static UNREALED_API void AsyncBuildCompleted();

	/**
	 * Is there currently an (async) build in progress?
	 */
	static UNREALED_API bool IsBuildCurrentlyRunning();

	/**
	 * Register a custom build type.
	 * @param Id The identifier to use for this build type.
	 * @param DoBuild The delegate to execute to run this build.
	 * @param BuildAllExtensionPoint If a valid name, run this build *before* running the build with this id when performing a Build All.
	 * @param MenuEntryLabel If non empty, will be used as label for the command in the menu. Otherwise `Build {FText::FromName(Id)}` will be used.
	 * @param MenuSectionLabel If non empty, will be used a label for a new submenu for this build type. Otherwise the entry will be created under `External Types`.
	 * @param bExternalProcess If build spawns an external process.
	 */
	static UNREALED_API void RegisterCustomBuildType(
		const FName Id,
		const FDoEditorBuildDelegate& DoBuild,
		const FName BuildAllExtensionPoint,
		const FText& MenuEntryLabel = FText::GetEmpty(),
		const FText& MenuSectionLabel = FText::GetEmpty(),
		bool bExternalProcess = false);
	
	/**
	 * Register a custom build type.
	 * @param Id The identifier to use for this build type.
	 * @param CanDoBuild The delegate to validate if a given build could be executed.
	 * @param DoBuild The delegate to execute to run this build.
	 * @param BuildAllExtensionPoint If a valid name, run this build *before* running the build with this id when performing a Build All.
	 * @param MenuEntryLabel If non empty, will be used as label for the command in the menu. Otherwise `Build {FText::FromName(Id)}` will be used.
	 * @param MenuSectionLabel If non empty, will be used a label for a new submenu for this build type. Otherwise the entry will be created under `External Types`.
	 * @param bExternalProcess If build spawns an external process.
	 */
	static UNREALED_API void RegisterCustomBuildType(
		const FName Id,
		const FCanDoEditorBuildDelegate& CanDoBuild,
		const FDoEditorBuildDelegate& DoBuild,
		const FName BuildAllExtensionPoint,
		const FText& MenuEntryLabel = FText::GetEmpty(),
		const FText& MenuSectionLabel = FText::GetEmpty(),
		bool bExternalProcess = false
		);

	/**
	 * Unregister a custom build type.
	 * @param Id The identifier of the build type to unregister.
	 */
	static UNREALED_API void UnregisterCustomBuildType(FName Id);

	/**
	 * Fills `RegisteredBuildTypes` with the names of all build types registered by 'RegisterCustomBuildType'.
	 * @param RegisteredBuildTypes Names of all registered build types.
	 */
	static UNREALED_API void GetBuildTypes(TArray<FName>& RegisteredBuildTypes);

	/**
	 * Fills provided arrays with localized menu entry and section labels.
	 * @param RegisteredBuildTypesEntryLabels Labels for each registered build types. Can be empty.
	 * @param RegisteredBuildTypesSectionLabels Labels for each registered build types section where the entry should reside. Can be empty.
	 */
	static UNREALED_API void GetBuildTypesLocalizedLabels(TArray<FText>& RegisteredBuildTypesEntryLabels, TArray<FText>& RegisteredBuildTypesSectionLabels);
	
	/**
	 * Runs another instance of the current executable with the provided command line arguments.
	 * On success the current level will be unloaded. Using `MapToLoad` allows to reload that specific level to reflect the changes.
	 * @param MapToLoad Map to load after successfully run the process
	 * @param ProgressText Message used by the progress dialog
	 * @param CancelledText Message shown to the user after cancelling the task 
	 * @param FailureText Message shown to the user if the process returned an error code
	 * @param CommandLineArguments Arguments passed to the other instance commandline
	 * @param OnProcRunningTick Function that will be called periodically while the builder process is running (can be used to update progress).
	 * 
	 * @return True if process completed without returning any error code.  
	 */
	static UNREALED_API bool RunWorldPartitionBuilder(
		const FString& MapToLoad,
		const FText& ProgressText,
		const FText& CancelledText,
		const FText& FailureText,
		const FString& CommandLineArguments,
		TFunction<void(FScopedSlowTask&)> UpdateSlowTask = TFunction<void(FScopedSlowTask&)>()
		);
private:

	/**
	 * Private helper method to log an error both to GWarn and to the build's list of accumulated errors
	 *
	 * @param	InErrorMessage			Message to log to GWarn/add to list of errors
	 * @param	OutAccumulatedErrors	List of errors accumulated during a build process so far
	 */
	static void LogErrorMessage( const FText& InErrorMessage, FText& OutAccumulatedErrors );

	/**
	 * Helper method to handle automated build behavior in the event of an error. Depending on the specified behavior, one of three
	 * results are possible:
	 *	a) User is prompted on whether to proceed with the automated build or not,
	 *	b) The error is regarded as a build-stopper and the method returns failure,
	 *	or
	 *	c) The error is acknowledged but not regarded as a build-stopper, and the method returns success.
	 * In any event, the error is logged for the user's information.
	 *
	 * @param	InBehavior				Behavior to use to respond to the error
	 * @param	InErrorMsg				Error to log
	 * @param	OutAccumulatedErrors	List of errors accumulated from the build process so far; InErrorMsg will be added to the list
	 *
	 * @return	true if the build should proceed after processing the error behavior; false if it should not
	 */
	static bool ProcessAutomatedBuildBehavior( EAutomatedBuildBehavior InBehavior, const FText& InErrorMsg, FText& OutAccumulatedErrors );

	/**
	 * Helper method designed to perform the necessary preparations required to complete an automated editor build
	 *
	 * @param	BuildSettings		Build settings that will be used for the editor build
	 * @param	OutPkgsToSubmit		Set of packages that need to be saved and submitted after a successful build
	 * @param	OutErrorMessages	Errors that resulted from the preparation (may or may not force the build to stop, depending on build settings)
	 *
	 * @return	true if the preparation was successful and the build should continue; false if the preparation failed and the build should be aborted
	 */
	static bool PrepForAutomatedBuild( const FEditorAutomatedBuildSettings& BuildSettings, TSet<UPackage*>& OutPkgsToSubmit, FText& OutErrorMessages );

	/**
	 * Helper method to submit packages to source control as part of the automated build process
	 *
	 * @param	InPkgsToSubmit	Set of packages which should be submitted to source control
	 * @param	BuildSettings	Build settings used during the automated build
	 */
	static void SubmitPackagesForAutomatedBuild( const TSet<UPackage*>& InPkgsToSubmit, const FEditorAutomatedBuildSettings& BuildSettings );

	/** 
	 * Trigger navigation builder to (re)generate NavMesh 
	 *
	 * @param	InOutWorld		WorldContext
	 * @param	Id	Build options requested
	 */
	static void TriggerNavigationBuilder(UWorld*& InOutWorld, FName Id);

	static bool WorldPartitionBuildNavigation(const FString& InLongPackageName);

	/** 
	 * Trigger HLOD builder to (re)generate HLOD actors
	 *
	 * @param	InWorld			WorldContext
	 */
	static void TriggerHierarchicalLODBuilder(UWorld* InWorld);

	/** 
	 * Trigger minimap builder to (re)generate minimap
	 *
	 * @param	InWorld			WorldContext
	 */
	static void TriggerMinimapBuilder(UWorld* InWorld);

	/**
	 * Trigger LandscapeSplineMeshes builder to (re)generate landscape spline meshes actors
	 *
	 * @param	InWorld			WorldContext
	 */
	static void TriggerLandscapeSplineMeshesBuilder(UWorld* InWorld);

	/** Intentionally hide constructors, etc. to prevent instantiation */
	FEditorBuildUtils();
	~FEditorBuildUtils();
	FEditorBuildUtils( const FEditorBuildUtils& );
	FEditorBuildUtils operator=( const FEditorBuildUtils& );

	// Allow the async build all handler to access custom build type info.
	friend class FBuildAllHandler;

	/** static variable to cache data about user request. navigation builder works in a background and we have to cache this information */
	static bool bBuildingNavigationFromUserRequest;

	/**
	 * Struct containing data for a custom build type.
	 */
	struct FCustomBuildType
	{
		FCanDoEditorBuildDelegate CanDoBuild;
		FDoEditorBuildDelegate DoBuild;
		FName BuildAllExtensionPoint;
		const FText MenuEntryLabel;
		const FText MenuSectionLabel;
		bool bExternalProcess;

		FCustomBuildType(
			const FDoEditorBuildDelegate& InDoBuild,
			const FName InBuildAllExtensionPoint,
			const FText& InMenuEntryLabel,
			const FText& InMenuSectionLabel,
			bool bInExternalProcess)
			: DoBuild(InDoBuild)
			, BuildAllExtensionPoint(InBuildAllExtensionPoint)
			, MenuEntryLabel(InMenuEntryLabel)
			, MenuSectionLabel(InMenuSectionLabel)
			, bExternalProcess(bInExternalProcess)
		{}

		FCustomBuildType(
			const FCanDoEditorBuildDelegate& InCanDoBuild,
			const FDoEditorBuildDelegate& InDoBuild,
			const FName InBuildAllExtensionPoint,
			const FText& InMenuEntryLabel,
			const FText& InMenuSectionLabel,
			bool bInExternalProcess)
			: FCustomBuildType(InDoBuild, InBuildAllExtensionPoint, InMenuEntryLabel, InMenuSectionLabel, bInExternalProcess)
		{
			CanDoBuild = InCanDoBuild;
		}
	};

	/** Map of custom build types registered with us. */
	static TMap<FName, FCustomBuildType> CustomBuildTypes;

	/** Set to a valid build type if an async build is in progress. */
	static FName InProgressBuildId;
};
