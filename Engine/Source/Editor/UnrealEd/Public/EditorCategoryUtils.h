// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Internationalization/Text.h"
#include "UObject/Class.h"

class UStruct;

/**
 * Wrapper for an enum detailing common editor categories. Users can reference
 * these categories in metadata using the enum value name in braces, like so:
 *
 *     UFUNCTION(Category="{Utilities}|MySubCategory")
 *
 * This gives users the ability to reference shared categories across the 
 * engine, but gives us the freedom to easily remap them as needed (also gives 
 * us the ability to easily localize these categories). Games can override these 
 * default mappings with the RegisterCategoryKey() function.
 */
struct FCommonEditorCategory
{
	enum EValue
	{
		// Function categories:
		AI,			
		Animation,	
		Audio,		
		Development,
		Effects,
		Gameplay,
		Input,
		Math,
		Networking,
		Pawn,
		Physics,
		Rendering,
		Transformation,
		Utilities,
		FlowControl,
		UserInterface,
		AnimNotify,
		BranchPoint,

		// Type library categories:
		String,
		Text,
		Name,
		Enum,
		Struct,
		Macro,
		Delegates,

		Class,
		Variables,
	};
};

/**
 * Set of utilities for parsing, querying, and resolving class/function/field 
 * category data.
 */
namespace FEditorCategoryUtils
{
	/**
	 * To facilitate simple category renaming/reordering, we offer a key 
	 * replacement system, where users can specify a key in their category 
	 * metadata that will evaluate to some fully qualified category. Use this
	 * function to register key/category mappings, or to override existing ones
	 * (like those pre-registered for all the "common" categories).
	 *
	 * In metadata, keys are denoted by braces, like {Utilities} here: 
	 *	UFUNCTION(Category="{Utilities}|MySubCategory")
	 * 
	 * @param  Key		A string key that people will use in metadata to reflect this category mapping.
	 * @param  Category	The qualified category path that you want the key expanded to.
	 * @param  Tooltip  An optional tooltip text to use for the category.  If not specified an attempt to find it from the NodeCategories UDN file will be made
	 */
	UNREALED_API void RegisterCategoryKey(const FString& Key, const FText& Category, const FText& Tooltip = FText::GetEmpty());

	/**
	 * @param  Key			A string key that people will use in metadata to reflect this category mapping.
	 * @param  Category		The qualified category path that you want the key expanded to.
	 * @param  DocLink		Path to the document page that contains the excerpt for this category
	 * @param  DocExcerpt	Name of the excerpt within the document page for this category
	 */
	UNREALED_API void RegisterCategoryKey(const FString& Key, const FText& Category, const FString& DocLink, const FString& DocExcerpt);

	/**
	 * Retrieves a qualified category path for the desired common category.
	 * 
	 * @param  CategoryId	The common category you want a path for.
	 * @return A text string, (empty if the common category was not registered)
	 */
	UNREALED_API const FText& GetCommonCategory(const FCommonEditorCategory::EValue CategoryId);

	/**
	 * Utility function that concatenates the supplied sub-category with one 
	 * that matches the root category id.
	 * 
	 * @param  RootCategory	An id denoting the root category that you want prefixed onto the sub-category.
	 * @param  SubCategory	A sub-category that you want postfixed to the root category.
	 * @return A concatenated text string, with the two categories separated by a pipe, '|', character.
	 */
	UNREALED_API FText BuildCategoryString(const FCommonEditorCategory::EValue RootCategory, const FText& SubCategory);

	/**
	 * Expands any keys found in the category string (any terms found in square 
	 * brackets), and sanitizes the name (spacing individual words, etc.).
	 * 
	 * @param  RootCategory	An id denoting the root category that you want prefixing the result.
	 * @param  SubCategory	A sub-category that you want postfixing the result.
	 * @return A concatenated text string, with the two categories separated by a pipe, '|', character.
	 */
	UNREALED_API FText GetCategoryDisplayString(const FText& UnsanitizedCategory);

	/**
	 * Expands any keys found in the category string (any terms found in square 
	 * brackets), and sanitizes the name (spacing individual words, etc.).
	 * 
	 * @param  RootCategory	An id denoting the root category that you want prefixing the result.
	 * @param  SubCategory	A sub-category that you want postfixing the result.
	 * @return A concatenated string, with the two categories separated by a pipe, '|', character.
	 */
	UNREALED_API FString GetCategoryDisplayString(const FString& UnsanitizedCategory);

	/**
	 * Parses out the class's "HideCategories" metadata, and returns it 
	 * segmented and sanitized.
	 * 
	 * @param  Class			The class you want to pull data from.
	 * @param  CategoriesOut	An array that will be filled with a list of hidden categories.
	 * @param  bHomogenize		Determines if the categories should be ran through expansion and display sanitation (useful even when not being displayed, for comparisons)
	 */
	UNREALED_API void GetClassHideCategories(const UStruct* Class, TArray<FString>& CategoriesOut, bool bHomogenize = true);

	/**
	 * Parses out the class's "ShowCategories" metadata, and returns it 
	 * segmented and sanitized.
	 * 
	 * @param  Class			The class you want to pull data from.
	 * @param  CategoriesOut	An array that will be filled with a list of shown categories.
	 */
	UNREALED_API void GetClassShowCategories(const UStruct* Class, TArray<FString>& CategoriesOut);

	/**
	 * Checks to see if the category associated with the supplied common 
	 * category id is hidden from the specified class.
	 * 
	 * @param  Class		The class you want to query.
	 * @param  CategoryId	An id associated with a category that you want to check.
	 * @return True if the common category is hidden, false if not.
	 */
	UNREALED_API bool IsCategoryHiddenFromClass(const UStruct* Class, const FCommonEditorCategory::EValue CategoryId);

	/**
	 * Checks to see if the specified category is hidden from the supplied class.
	 * 
	 * @param  Class	The class you want to query.
	 * @param  Category A category path that you want to check.
	 * @return True if the category is hidden, false if not.
	 */
	UNREALED_API bool IsCategoryHiddenFromClass(const UStruct* Class, const FText& Category);

	/**
	 * Checks to see if the specified category is hidden from the supplied class.
	 * 
	 * @param  Class	The class you want to query.
	 * @param  Category A category path that you want to check.
	 * @return True if the category is hidden, false if not.
	 */
	UNREALED_API bool IsCategoryHiddenFromClass(const UStruct* Class, const FString& Category);

	/**
	 * Checks to see if the specified category is hidden from the supplied Class, avoids recalculation of ClassHideCategories.
	 * Useful when checking the same class over and over again with different categories.
	 *
	 * @param ClassHideCategories	The categories tht have been hidden for the class
	 * @param  Class	The class you want to query.
	 * @param  Category A category path that you want to check.
	 * @return True if the category is hidden, false if not.
	 */
	UNREALED_API bool IsCategoryHiddenFromClass(const TArray<FString>& ClassHideCategories, const UStruct* Class, const FString& Category);
	/**
	 * Returns tooltip information for the specified category
	 * 
	 * @param  Category		The category that you want to tooltip details for.
	 * @param  Tooltip		[OUT] The tooltip to display for this category.
	 * @param  DocLink		[OUT] The link to the documentation page for this category.
	 * @param  DocExcerpt	[OUT] A category path that you want to check.
	 */
	UNREALED_API void GetCategoryTooltipInfo(const FString& Category, FText& Tooltip, FString& DocLink, FString& DocExcerpt);

	/**
	 * Returns the set of categories that should be hidden, categories that are both
	 * explicitly hidden and explicitly shown will not be included in this list (current
	 * behavior is that such categories should be shown). This occurs when you show
	 * a class that your parent has hidden.
	 * 
	 * @param  Class		The class you want to query.
	 * @return  The set of categories that should be hidden.
	 */
	UNREALED_API TSet<FString> GetHiddenCategories(const UStruct* Class);
};
