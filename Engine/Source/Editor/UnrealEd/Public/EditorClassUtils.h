// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Widgets/SToolTip.h"

template< typename ObjectType > class TAttribute;
struct FAssetData;

namespace FEditorClassUtils
{

	/**
	 * Gets the page that documentation for this class is contained on
	 *
	 * @param	InClass		Class we want to find the documentation page of
	 * @return				Path to the documentation page
	 */
	UNREALED_API FString GetDocumentationPage(const UClass* Class);

	/**
	 * Gets the excerpt to use for this class
	 * Excerpt will be contained on the page returned by GetDocumentationPage
	 *
	 * @param	InClass		Class we want to find the documentation excerpt of
	 * @return				Name of the to the documentation excerpt
	 */
	UNREALED_API FString GetDocumentationExcerpt(const UClass* Class);

	/**
	 * Gets the tooltip to display for a given class
	 *
	 * @param	InClass		Class we want to build a tooltip for
	 * @return				Shared reference to the constructed tooltip
	 */
	UNREALED_API TSharedRef<SToolTip> GetTooltip(const UClass* Class);

	/**
	 * Gets the tooltip to display for a given class with specified text for the tooltip
	 *
	 * @param	InClass			Class we want to build a tooltip for
	 * @param	OverrideText	The text to display on the standard tooltip
	 * @return					Shared reference to the constructed tooltip
	 */
	UNREALED_API TSharedRef<SToolTip> GetTooltip(const UClass* Class, const TAttribute<FText>& OverrideText);

	/**
	 * Returns the link path to the documentation for a given class
	 *
	 * @param	Class		Class we want to build a link for
	 * @return				The path to the documentation for the class
	 */
	UNREALED_API FString GetDocumentationLink(const UClass* Class, const FString& OverrideExcerpt = FString());

	/**
	 * Return link path from a specified excerpt
	 */
	UNREALED_API FString GetDocumentationLinkFromExcerpt(const FString& DocLink, const FString DocExcerpt);

	/**
	 * Returns the ID of the base documentation URL set for this class in its documentation excerpt.
	 *
	 * @param	Class		Class we want to build a link for
	 * @return				The ID of the base URL
	 */
	UNREALED_API FString GetDocumentationLinkBaseUrl(const UClass* Class, const FString& OverrideExcerpt = FString());

	/**
	 * Returns the ID of the base documentation URL set in the specified excerpt.
	 */
	UNREALED_API FString GetDocumentationLinkBaseUrlFromExcerpt(const FString& DocLink, const FString DocExcerpt);

	/**
	 * Creates a link widget to the documentation for a given class
	 *
	 * @param	Class		Class we want to build a link for
	 * @return				Shared pointer to the constructed tooltip
	 */
	UNREALED_API TSharedRef<SWidget> GetDocumentationLinkWidget(const UClass* Class);

	/**
	 * Create a link widget to the documentation for a potentially dynamic link widget.
	 * @param	Class		The attribute of the class to show documentation for.
	 */
	UNREALED_API TSharedRef<SWidget> GetDynamicDocumentationLinkWidget(const TAttribute<const UClass*>& ClassAttribute);

	/** Optional GetSourceLink parameters */
	struct FSourceLinkParams
	{
		/* Object to set blueprint debugging to in the case we have a blueprint generated class */
		TWeakObjectPtr<UObject> Object;

		/* Text format for blueprint links */
		const FText* BlueprintFormat = nullptr;
		
		/* Text format for C++ code file links */
		const FText* CodeFormat = nullptr;

		/** 
		 * If true, use default values for BlueprintFormat (Edit ...) and CodeFormat (Open ...) when unspecified
		 * If false, use only the class name when BlueprintFormat and CodeFormat are unspecified 
		 */
		bool bUseDefaultFormat = false;

		/** If true, use specified text format if the link is unavailable for some reason, otherwise use only the class name */
		bool bUseFormatIfNoLink = false;

		/** Whether a spacer widget is used if the link is unavailable for some reason */
		bool bEmptyIfNoLink = false;
	};

	/**
	 * Creates an hyperlink to the source code or blueprint for a given class
	 * (or a text block/spacer if the link is unavailable for some reason)
	 *
	 * @param	Class	Class we want to build a link for
	 * @param	Params	See FSourceLinkOptionalParams
	 * @return			Shared pointer to the constructed widget
	 */
	UNREALED_API TSharedRef<SWidget> GetSourceLink(const UClass* Class, const FSourceLinkParams& Params = FSourceLinkParams());

	/**
	 * Creates an hyperlink to the source code or blueprint for a given class
	 * (or a spacer if the link is unavailable for some reason)
	 *
	 * @param	Class			Class we want to build a link for
	 * @param	ObjectWeakPtr	Optional object to set blueprint debugging to in the case we are choosing a blueprint
	 * @return					Shared pointer to the constructed widget
	 */
	UNREALED_API TSharedRef<SWidget> GetSourceLink(const UClass* Class, const TWeakObjectPtr<UObject> ObjectWeakPtr);

	/**
	 * Creates an hyperlink to the source code or blueprint for a given class formatted however you need. Example "Edit {0}"
	 * (or a spacer if the link is unavailable for some reason)
	 *
	 * @param	Class			Class we want to build a link for
	 * @param	ObjectWeakPtr	Optional object to set blueprint debugging to in the case we are choosing a blueprint
	 * @param	BlueprintFormat	The text format for blueprint links
	 * @param	CodeFormat		The text format for C++ code file links
	 * @return					Shared pointer to the constructed widget
	 */
	UNREALED_API TSharedRef<SWidget> GetSourceLinkFormatted(const UClass* Class, const TWeakObjectPtr<UObject> ObjectWeakPtr, const FText& BlueprintFormat, const FText& CodeFormat);

	/**
	 * Fetches a UClass from the string name of the class
	 *
	 * @param	ClassName		Name of the class we want the UClass for
	 * @return					UClass pointer if it exists
	 */
	UNREALED_API UClass* GetClassFromString(const FString& ClassName);

	/**
	 * Returns whether the specified asset is a UBlueprint or UBlueprintGeneratedClass (or any of their derived classes)
	 * 
	 * @param	InAssetData		Reference to an asset data entry
	 * @param	bOutIsBPGC		Outputs whether the asset is a BlueprintGeneratedClass (or any of its derived classes)
	 * @return					Whether the specified asset is a UBlueprint or UBlueprintGeneratedClass (or any of their derived classes)
	 */
	UNREALED_API bool IsBlueprintAsset(const FAssetData& InAssetData, bool* bOutIsBPGC = nullptr);

	/**
	 * Gets the class path from the asset tag (i.e. GeneratedClassPath tag on blueprints)
	 * 
	 * @param	InAssetData		Reference to an asset data entry.
	 * @return					Class path or None if the asset cannot or doesn't have a class associated with it
	 */
	UNREALED_API FTopLevelAssetPath GetClassPathNameFromAssetTag(const FAssetData& InAssetData);

	/**
	 * Gets the object path of the class associated with the specified asset 
	 * (i.e. the BlueprintGeneratedClass of a Blueprint asset or the BlueprintGeneratedClass asset itself)
	 * 
	 * @param	InAssetData					Reference to an asset data entry
	 * @param	bGenerateClassPathIfMissing	Whether to generate a class path if the class is missing (and the asset can have a class associated with it)
	 * @return								Class path or None if the asset cannot or doesn't have a class associated with it
	 */
	UNREALED_API FTopLevelAssetPath GetClassPathNameFromAsset(const FAssetData& InAssetData, bool bGenerateClassPathIfMissing = false);

	/**
	 * Fetches the set of interface class object paths from an asset data entry containing the appropriate asset tag(s).
	 * 
	 * @param	InAssetData		Reference to an asset data entry.
	 * @param	OutClassPaths	One or more interface class object paths, or empty if the corresponding asset tag(s) were not found.
	 */
	UNREALED_API void GetImplementedInterfaceClassPathsFromAsset(const FAssetData& InAssetData, TArray<FString>& OutClassPaths);
};
