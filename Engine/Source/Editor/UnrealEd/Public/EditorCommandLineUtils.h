// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringView.h"
#include "Containers/UnrealString.h"

/** 
 * 
 */
struct FEditorCommandLineUtils
{
	/**
	 * Parses commands from the supplied command-line and invokes any tools/
	 * settings that we're specified.
	 *
	 * @param  CommandLine	The string to parse for command-line switches/params
	 */
	static void ProcessEditorCommands(const TCHAR* CommandLine);

	/**
	 * Generally, users supply the project path/name as the second argument of 
	 * the editor command-line, but in some cases (like for the -diff command), 
	 * we can infer the project from elements elsewhere in the command-line...
	 * This function serves as a fallback for determining the launching project
	 * path, and does special processing based off of commands that are found in
	 * the command-line.
	 *
	 * @param  EditorCmdLine	The string to parse a project path from.
	 * @param  ProjPathOut		[out] Will be filled out with a full .uproject file path (if one was successfully parsed out) from the command-line.
	 * @param  GameNameOut		[out] Will be filled out with a game name (if a project path was successfully parsed out) from the command-line.
	 * @return True if the command-line contained enough info to pull a .uproject file from.
	 */
	UNREALED_API static bool ParseGameProjectPath(const TCHAR* EditorCmdLine, FString& ProjPathOut, FString& GameNameOut);
};

constexpr const TCHAR* GEditorUIPidVariable = TEXT("UE_EditorUIPid");
