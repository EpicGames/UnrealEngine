// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	EditorComponents.h: Scene components used by the editor modes.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "UObject/GCObject.h"

class FPrimitiveDrawInterface;
class FSceneView;
class UMaterial;
class UMaterialInstanceDynamic;

class FGridWidget : public FGCObject
{
public:
	FGridWidget();

	void DrawNewGrid(const FSceneView* View,FPrimitiveDrawInterface* PDI);

	/** 
	 * FGCObject interface: Serializes the widget reference so they dont get garbage collected.
	 *
	 * @param Ar	FArchive to serialize with
	 */
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override
	{
		return "FGridWidget";
	}

private:
	UMaterial* GetActiveLevelGridMaterial();
	UMaterialInstanceDynamic* GetActiveLevelGridMID();
private:
	// ALU based
	TSoftObjectPtr<UMaterial> GPUBasedGridMaterial;
	// texture based
	TSoftObjectPtr<UMaterial> TextureBasedLevelGridMaterial;

	TObjectPtr<UMaterialInstanceDynamic> LevelGridMaterialInst;
};

/**
 * Implements some basic functionality used by all editor viewports.
 */
class FEditorCommonDrawHelper
{
public:

	UNREALED_API FEditorCommonDrawHelper();
	virtual UNREALED_API ~FEditorCommonDrawHelper();

	/** Renders the grid, pivot, and base info. */
	virtual void UNREALED_API Draw(const FSceneView* View,FPrimitiveDrawInterface* PDI);

	/**
	 * Renders the editor's pivot.
	 */
	void DrawPivot(const FSceneView* View,FPrimitiveDrawInterface* PDI);

	/** Draw green lines to indicate what the selected actor(s) are based on. */
	void DrawBaseInfo(const FSceneView* View,FPrimitiveDrawInterface* PDI);

private:

	/** Types of axis lines we can draw for orthographic views */
	struct EAxisLines
	{
		enum Type
		{
			// Major lines
			Major,

			// Minor lines
			Minor,
		};
	};


	/**
	 * Draws a section(vertical lines or horizontal lines) of a viewport's grid.
	 */
	static void DrawGridSection(double ViewportGridY,double MaxSize,FVector* A,FVector* B,FVector::FReal* AX, FVector::FReal* BX,int32 Axis,const FSceneView* View,FPrimitiveDrawInterface* PDI);

	/**
	 * Draws an origin axis line for the orthographic grid
	 */
	static void DrawOriginAxisLine(FVector* A,FVector* B, FVector::FReal* AX, FVector::FReal* BX,const FSceneView* View,FPrimitiveDrawInterface* PDI, const FLinearColor& Color);


	/** Draws the viewport grid. */
	void DrawOldGrid(const FSceneView* View,FPrimitiveDrawInterface* PDI);

public:
	uint32 bDrawGrid:1;
	uint32 bDrawPivot:1;
	uint32 bDrawBaseInfo:1;
	uint32 bDrawWorldBox:1;
	uint32 bDrawKillZ:1;
	float AxesLineThickness;

	/** Color of the grid on the main axes lines */
	FColor GridColorAxis;

	/** Color of the grid on major axes lines (every Nth grid step line) */
	FColor GridColorMajor;

	/** Color of grid lines at each grid interval */
	FColor GridColorMinor;

	float PerspectiveGridSize;

	FColor PivotColor;
	float PivotSize;

	/** The number of cells to draw. */
	int32 NumCells;

	FColor BaseBoxColor;

	ESceneDepthPriorityGroup DepthPriorityGroup;

	float GridDepthBias;

	FGridWidget* GridWidget;
};

namespace EditorActorSelectionDefs
{
	/** The maximum number of actors we should select before a performance warning message is displayed to the user */
	inline constexpr int32 MaxActorsToSelectBeforeWarning = 5000;
};

