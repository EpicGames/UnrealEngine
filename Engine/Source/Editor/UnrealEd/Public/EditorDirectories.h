// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"

/** The different directory identifiers */
namespace ELastDirectory
{
	enum Type
	{
		UNR,
		BRUSH,
		FBX,
		FBX_ANIM,
		GENERIC_IMPORT,
		GENERIC_EXPORT,
		GENERIC_OPEN,
		GENERIC_SAVE,
		MESH_IMPORT_EXPORT,
		WORLD_ROOT,
		LEVEL,
		PROJECT,
		NEW_ASSET,
		MAX
	};
};

class FEditorDirectories
{
public:
	UNREALED_API FEditorDirectories();

	static UNREALED_API FEditorDirectories& Get();

	/** Initializes the "LastDir" array with default directories for loading/saving files */
	UNREALED_API void LoadLastDirectories();

	/** Writes the current "LastDir" array back out to the config files */
	UNREALED_API void SaveLastDirectories();

	/**
	 *	Fetches the last directory used for the specified type
	 *
	 *	@param	InLastDir	the identifier for the directory type
	 *	@return	FString		the string that was last set
	 */
	UNREALED_API FString GetLastDirectory( ELastDirectory::Type InLastDir ) const;

	
	/**
	 *	Sets the last directory used for the specified type
	 *
	 *	@param	InLastDir	the identifier for the directory type
	 *	@param	InLastStr	the string to set it as
	 */
	UNREALED_API void SetLastDirectory( ELastDirectory::Type InLastDir, const FString& InLastStr );

	/**
	 *	Overrides how configuration is saved and the default directory
	 *
	 *	@param	InConfigFile	config file to save and load with
	 *	@param	InConfigSectionName	section name to use inside config file
	 *	@param	InDefaultDir	default directory to use if config file did not have saved directory
	 */
	UNREALED_API void SetOverride(const FString& InConfigFile, const FString& InConfigSectionName, const FString& InDefaultDir);

	/** Removes override of config file, config section and default directory */
	UNREALED_API void ResetOverride();

private:
	/** Array of the last directories used for various editor windows */
	FString LastDir[ELastDirectory::MAX];

	/** Config file to save and load with */
	FString ConfigFile;

	/** Section name to use inside config file */
	FString ConfigSectionName;

	/** Default directory to use if config file did not have saved directory */
	FString DefaultDir;

	/** Has loaded from config */
	bool bHasLoaded;
};
