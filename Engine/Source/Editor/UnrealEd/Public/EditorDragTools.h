// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"

class FCanvas;
class FEditorModeTools;
class FEditorViewportClient;
class FPrimitiveDrawInterface;
class FSceneView;

/**
 * The base class that all drag tools inherit from.
 * The drag tools implement special behaviors for the user clicking and dragging in a viewport.
 */
class FDragTool
{
public:
	UNREALED_API FDragTool(FEditorModeTools* InModeTools);
	virtual ~FDragTool() = default;

	/**
	 * Updates the drag tool's end location with the specified delta.  The end location is
	 * snapped to the editor constraints if bUseSnapping is true.
	 *
	 * @param	InDelta		A delta of mouse movement.
	 */
	UNREALED_API virtual void AddDelta( const FVector& InDelta );

	/**
	 * Starts a mouse drag behavior.  The start location is snapped to the editor constraints if bUseSnapping is true.
	 *
	 * @param	InViewportClient	The viewport client in which the drag event occurred.
	 * @param	InStart				Where the mouse was when the drag started.
	 */
	UNREALED_API virtual void StartDrag(FEditorViewportClient* InViewportClient, const FVector& InStartWorld, const FVector2D& InStartScreen);

	/**
	 * Ends a mouse drag behavior (the user has let go of the mouse button).
	 */
	UNREALED_API virtual void EndDrag();
	virtual void Render3D(const FSceneView* View,FPrimitiveDrawInterface* PDI) {}
	virtual void Render(const FSceneView* View,FCanvas* Canvas) {}

	/**
	 * Rendering stub for 2D viewport drag tools.
	 */
	virtual void Render( FCanvas* Canvas ) {}

	/** @return true if we are dragging */
	bool IsDragging() const { return bIsDragging; }

	/** Does this drag tool need to have the mouse movement converted to the viewport orientation? */
	bool bConvertDelta;

protected:
	FEditorModeTools* ModeTools;

	/** The start/end location of the current drag. */
	FVector Start, End, EndWk;

	/** If true, the drag tool wants to be passed grid snapped values. */
	bool bUseSnapping;

	/** These flags store the state of various buttons that were pressed when the drag was started. */
	bool bAltDown;
	bool bShiftDown;
	bool bControlDown;
	bool bLeftMouseButtonDown;
	bool bRightMouseButtonDown;
	bool bMiddleMouseButtonDown;
	/** True if we are dragging */
	bool bIsDragging;
};
