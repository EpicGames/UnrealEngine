// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Folder.h"
#include "HAL/Platform.h"
#include "Misc/Paths.h"
#include "UObject/NameTypes.h"

class FEditorFolderUtils
{
public:
	/** Get the leaf name of a specified folder path */
	static UNREALED_API FName GetLeafName(const FName& InPath);

	/** Get the parent path for the specified folder path */
	FORCEINLINE static FName GetParentPath(const FName& InPath)
	{
		return FName(*FPaths::GetPath(InPath.ToString()));
	}

	/** Check if the specified path is a child of the specified parent */
	static UNREALED_API bool PathIsChildOf(const FString& PotentialChild, const FString& Parent);
	static UNREALED_API bool PathIsChildOf(const FName& PotentialChild, const FName& Parent);
	static UNREALED_API void RenameFolder(const FFolder& InFolder, const FText& NewFolderName, UWorld* World);
};
