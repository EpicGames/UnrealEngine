// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	EditorLevelUtils.h: Editor-specific level management routines
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "EditorLevelUtils.generated.h"

class AActor;
class ULevel;
class ULevelStreaming;

DECLARE_LOG_CATEGORY_EXTERN(LogLevelTools, Warning, All);

UENUM(BlueprintType)
enum class ELevelVisibilityDirtyMode : uint8
{
	// Use when the user is causing the visibility change.  Will update transaction state and mark the package dirty.
	ModifyOnChange,
	// Use when code is causing the visibility change.
	DontModify
};

UCLASS(transient)
class UEditorLevelUtils : public UObject
{
	GENERATED_BODY()
public:
	/**
	 * Creates a new streaming level in the current world
	 *
	 * @param	LevelStreamingClass					The streaming class type instead to use for the level.
	 * @param	NewLevelPath						Optional path to the level package path format ("e.g /Game/MyLevel").  If empty, the user will be prompted during the save process.
	 * @param	bMoveSelectedActorsIntoNewLevel		If true, move any selected actors into the new level.
	 *
	 * @return	Returns the newly created level, or NULL on failure
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Creation")
	static UNREALED_API ULevelStreaming* CreateNewStreamingLevel(UPARAM(meta = (AllowAbstract = "false")) TSubclassOf<ULevelStreaming> LevelStreamingClass, const FString& NewLevelPath = TEXT(""), bool bMoveSelectedActorsIntoNewLevel = false);

	/**
	 * Makes the specified streaming level the current level for editing.
	 * The current level is where actors are spawned to when calling SpawnActor
	 *
	 * @return	true	If a level was removed.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Creation")
	static UNREALED_API void MakeLevelCurrent(ULevelStreaming* InStreamingLevel);


	/**
	 * Moves the specified list of actors to the specified streaming level. The new actors will be selected
	 *
	 * @param	ActorsToMove			List of actors to move
	 * @param	DestStreamingLevel		The destination streaming level of the current world to move the actors to
	 * @param	bWarnAboutReferences	Whether or not to show a modal warning about referenced actors that may no longer function after being moved
	 * @return							The number of actors that were successfully moved to the new level
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Creation")
	static UNREALED_API int32 MoveActorsToLevel(const TArray<AActor*>& ActorsToMove, ULevelStreaming* DestStreamingLevel, bool bWarnAboutReferences = true, bool bWarnAboutRenaming = true);

	/**
	 * Moves the currently selected actors to the specified streaming level. The new actors will be selected
	 *
	 * @param	DestStreamingLevel		The destination streaming level of the current world to move the actors to
	 * @param	bWarnAboutReferences	Whether or not to show a modal warning about referenced actors that may no longer function after being moved
	 * @return							The number of actors that were successfully moved to the new level
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Creation")
	static UNREALED_API int32 MoveSelectedActorsToLevel(ULevelStreaming* DestLevel, bool bWarnAboutReferences = true);

	
	/**
	 * Makes the specified level the current level for editing.
	 * The current level is where actors are spawned to when calling SpawnActor
	 * @param InLevel			The level to make current
	 * @param bForceOperation	True if the operation should succeed even if the level is locked.  In certian circumstances (like removing the current level, we must be able to set a new current level even if the only one left is locked)
	 *
	 * @return	true	If a level was removed.
	 */
	static UNREALED_API void MakeLevelCurrent(ULevel* InLevel, bool bEvenIfLocked = false);

	/**
	 * Move provided actors to destination level.
	 *
	 * @param	ActorsToMove						Actors to move.
	 * @param	DestLevel							Destination level to move the actors to.
	 * @param	bWarnAboutReferences				Whether or not to show a modal warning about referenced actors that may no longer function after being moved.
	 * @param	bWarnAboutRenaming					Whether or not to show a model warning for asset rename.
	 * @param	bMoveAllOrFail						Whether operation should fail if any of the actors fails to be moved.
	 * @param	OutActors							Optional, if not null the array will be filled with the newly moved actors.
	 *
	 * @return	Returns the number of actors moved.
	 */
	static UNREALED_API int32 MoveActorsToLevel(const TArray<AActor*>& ActorsToMove, ULevel* DestLevel, bool bWarnAboutReferences = true, bool bWarnAboutRenaming = true, bool bMoveAllOrFail = false, TArray<AActor*>* OutActors = nullptr);

	/**
	 * Copy provided actors to destination level.
	 *
	 * @param	ActorsToMove						Actors to copy.
	 * @param	DestLevel							Destination level to copy the actors to.
	 * @param	bWarnAboutReferences				Whether or not to show a modal warning about referenced actors that may no longer function after being copied.
	 * @param	bWarnAboutRenaming					Whether or not to show a model warning for asset rename.
	 * @param	bMoveAllOrFail						Whether operation should fail if any of the actors fails to be copied.
	 * @param	OutActors							Optional, if not null the array will be filled with the newly copied actors.
	 *
	 * @return	Returns the number of actors copied.
	 */
	static UNREALED_API int32 CopyActorsToLevel(const TArray<AActor*>& ActorsToMove, ULevel* DestLevel, bool bWarnAboutReferences = true, bool bWarnAboutRenaming = true, bool bMoveAllOrFail = false, TArray<AActor*>* OutActors = nullptr);

	/**
	 * Move selected actors to destination level.
	 *
	 * @param	DestLevel							Destination level to move the actors to.
	 * @param	bWarnAboutReferences				Whether or not to show a modal warning about referenced actors that may no longer function after being copied.
	 *
	 * @return	Returns the number of actors copied.
	 */
	static UNREALED_API int32 MoveSelectedActorsToLevel(ULevel* DestLevel, bool bWarnAboutReferences = true);

	/** 
	* Delegate used by MoveActorsToLevel() to check whether an actor can be moved 
	*/
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FCanMoveActorToLevelDelegate, const AActor* /* ActorToMove */, const ULevel* /* DestLevel */, bool& /* bOutCanMove */);
	static UNREALED_API FCanMoveActorToLevelDelegate CanMoveActorToLevelDelegate;

	/**
	* Delegate used by MoveActorsToLevel() to notify about actors being moved
	*/
	DECLARE_EVENT_TwoParams(UEditorLevelUtils, FOnMoveActorsToLevelEvent, const TArray<AActor*>& /* ActorsToMove */, const ULevel* /* DestLevel */);
	static UNREALED_API FOnMoveActorsToLevelEvent OnMoveActorsToLevelEvent;

	/**
	 * Creates a new streaming level and adds it to a world
	 *
	 * @param	InWorld								The world to add the streaming level to
	 * @param	LevelStreamingClass					The streaming class type instead to use for the level.
	 * @param	DefaultFilename						Optional file name for level.  If empty, the user will be prompted during the save process.
	 * @param	bMoveSelectedActorsIntoNewLevel		If true, move any selected actors into the new level.
	 * @param	InTemplateWorld						If valid, the new level will be a copy of the template world.
	 * @param	bInUseSaveAs						If true, show SaveAs dialog instead of Save with DefaultFilename
	 * @param	InPreSaveLevelOperation				Optional function to call before saving the created level
	 * @param	InTransform							The transform to apply to the streaming level.
	 * 
	 * @return	Returns the newly created level, or NULL on failure
	 */
	static UNREALED_API ULevelStreaming* CreateNewStreamingLevelForWorld(UWorld& World, TSubclassOf<ULevelStreaming> LevelStreamingClass, const FString& DefaultFilename = TEXT(""), bool bMoveSelectedActorsIntoNewLevel = false, UWorld* InTemplateWorld = nullptr, bool bInUseSaveAs = true, TFunction<void(ULevel*)> InPreSaveLevelOperation = TFunction<void(ULevel*)>(), const FTransform& InTransform = FTransform::Identity);

	/**
	 * Creates a new streaming level and adds it to a world
	 *
	 * @param	InWorld								The world to add the streaming level to
	 * @param	LevelStreamingClass					The streaming class type instead to use for the level.
	 * @param	bUseExternalActors					If level should use external actors.
	 * @param	DefaultFilename						file name for level.  If empty, the user will be prompted during the save process.
	 * @param	ActorsToMove						Optional, move provided actors into the new level.
	 * @param	InTemplateWorld						If valid, the new level will be a copy of the template world.
	 * @param	bInUseSaveAs						If true, show SaveAs dialog instead of Save with DefaultFilename
	 * @param	bIsPartitioned						If level should be partitioned (has precedence over bUseExternalActors).
	 * @param	InPreSaveLevelOperation				Optional function to call before saving the created level
	 * @param	InTransform							The transform to apply to the streaming level.
	 *
	 * @return	Returns the newly created level, or NULL on failure
	 */
	static UNREALED_API ULevelStreaming* CreateNewStreamingLevelForWorld(UWorld& World, TSubclassOf<ULevelStreaming> LevelStreamingClass, bool bUseExternalActors, const FString& DefaultFilename, const TArray<AActor*>* ActorsToMove = nullptr, UWorld* InTemplateWorld = nullptr, bool bInUseSaveAs = true, bool bIsPartitioned = false, TFunction<void(ULevel*)> InPreSaveLevelOperation = TFunction<void(ULevel*)>(), const FTransform& InTransform = FTransform::Identity);

	struct FCreateNewStreamingLevelForWorldParams
	{
		FCreateNewStreamingLevelForWorldParams(TSubclassOf<ULevelStreaming> InLevelStreamingClass, FString InDefaultFilename)
			: LevelStreamingClass(InLevelStreamingClass), DefaultFilename(InDefaultFilename) {}
		
		/** The streaming class type instead to use for the level */
		TSubclassOf<ULevelStreaming> LevelStreamingClass;

		/** file name for level.  If empty, the user will be prompted during the save process. */
		FString DefaultFilename;

		/** If valid, the new level will be a copy of the template world. */
		UWorld* TemplateWorld = nullptr;

		/** The transform to apply to the streaming level. */
		FTransform Transform = FTransform::Identity;
		
		/** Optional, move provided actors into the new level. */
		const TArray<AActor*>* ActorsToMove = nullptr;

		/** If level should use external actors. */
		bool bUseExternalActors = false;
		
		/** If level should be Partitioned (has precedence over bUseExternalActors). */
		bool bCreateWorldPartition = false;
		
		/** If level WorldPartition should have streamng enabled (only valid if bCreateWorldPartition is true) */
		bool bEnableWorldPartitionStreaming = true;
		
		/** If true, show SaveAs dialog instead of Save with DefaultFilename */
		bool bUseSaveAs = true;

		/** Optional function to call before saving the created level */
		TFunction<void(ULevel*)> PreSaveLevelCallback = nullptr;

		/** Optional function to call after the ULevelStreaming gets created. */
		TFunction<void(ULevelStreaming*)> LevelStreamingCreatedCallback = nullptr;
	};

	/**
	 * Creates a new streaming level and adds it to a world
	 *
	 * @param	InWorld			The world to add the streaming level to
	 * @param	InCreateParams	Parameters used to create this new streaming world
	 *
	 * @return	Returns the newly created level, or NULL on failure
	 */
	static UNREALED_API ULevelStreaming* CreateNewStreamingLevelForWorld(UWorld& InWorld, const FCreateNewStreamingLevelForWorldParams& InCreateParams);

	/**
	 * Adds the named level packages to the world.  Does nothing if all the levels already exist in the world.
	 *
	 * @param	InWorld				World in which to add the level
	 * @param	LevelPackageName	The base filename of the level package to add.
	 * @param	LevelStreamingClass	The streaming class type instead to use for the level.
	 *
	 * @return								The new level, or NULL if the level couldn't added.
	 */
	static UNREALED_API ULevel* AddLevelsToWorld(UWorld* InWorld, TArray<FString> LevelPackageNames, TSubclassOf<ULevelStreaming> LevelStreamingClass);

	/**
	 * Adds the named level package to the world.  Does nothing if the level already exists in the world.
	 * 
	 * Levels are not saved when added to the world. They can be saved with the "Save Map" function
	 *
	 * @param	InWorld				World in which to add the level.
	 * @param	LevelPackageName	The package name ("e.g /Game/MyLevel") of the level package to add.
	 * @param	LevelStreamingClass	The streaming class type to use for the level.
	 *
	 * @return								The new level, or NULL if the level couldn't added.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Creation", meta=(DisplayName="Add Level to World", ScriptName="AddLevelToWorld"))
	static ULevelStreaming* K2_AddLevelToWorld(UWorld* World, const FString& LevelPackageName, UPARAM(meta = (AllowAbstract = "false")) TSubclassOf<ULevelStreaming> LevelStreamingClass)
	{
		return AddLevelToWorld(World, *LevelPackageName, LevelStreamingClass, FTransform::Identity);
	}

	/**
	 * Removes given level from the world. Note, this will only work for sub-levels in the main level.
	 *
	 * Levels are not saved when added to the world. They can be saved with the "Save Map" function
	 *
	 * @param	InLevel				    Level asset to remove from the world.
	 * @param	bClearSelection			If true, it will clear the editor selection.
	 * @param	bResetTransactionBuffer	If true, it will reset the transaction buffer (i.e. clear undo history)
	 *
	 * @return							True if the level was successfully removed.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Creation", meta=(DisplayName="Remove Level From World", ScriptName="RemoveLevelFromWorld"))
	static bool K2_RemoveLevelFromWorld(ULevel* InLevel, bool bClearSelection = true, bool bResetTransactionBuffer = true)
	{
		return RemoveLevelFromWorld(InLevel, bClearSelection, bResetTransactionBuffer);
	}

	/**
	 * Adds the named level package to the world at the given position.  Does nothing if the level already exists in the world.
	 *
	 * @param	InWorld				World in which to add the level.
	 * @param	LevelPackageName	The package name ("e.g /Game/MyLevel") of the level package to add.
	 * @param	LevelStreamingClass	The streaming class type to use for the level.
	 * @param	LevelTransform		The origin of the new level in the world.
	 *
	 * @return								The new level, or NULL if the level couldn't added.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Creation", meta=(DisplayName="Add Level to World with Transform", ScriptName="AddLevelToWorldWithTransform"))
	static ULevelStreaming* K2_AddLevelToWorldWithTransform(UWorld* World, const FString& LevelPackageName, UPARAM(meta = (AllowAbstract = "false")) TSubclassOf<ULevelStreaming> LevelStreamingClass, const FTransform& LevelTransform)
	{
		return AddLevelToWorld(World, *LevelPackageName, LevelStreamingClass, LevelTransform);
	}

	/**
	 * Adds the named level package to the world.  Does nothing if the level already exists in the world.
	 *
	 * @param	InWorld				World in which to add the level
	 * @param	LevelPackageName	The base filename of the level package to add.
	 * @param	LevelStreamingClass	The streaming class type instead to use for the level.
	 * @param	InTransform			The transform to apply to the streaming level.
	 *
	 * @return								The new level, or NULL if the level couldn't added.
	 */
	static UNREALED_API ULevelStreaming* AddLevelToWorld(UWorld* InWorld, const TCHAR* LevelPackageName, TSubclassOf<ULevelStreaming> LevelStreamingClass, const FTransform& LevelTransform = FTransform::Identity);

	struct FAddLevelToWorldParams
	{
		FAddLevelToWorldParams(TSubclassOf<ULevelStreaming> InLevelStreamingClass, FName InLevelPackageName)
			: LevelStreamingClass(InLevelStreamingClass), PackageName(InLevelPackageName)
		{}

		/** The streaming class type instead to use for the level. */
		TSubclassOf<ULevelStreaming> LevelStreamingClass;
		
		/** The base filename of the level package to add. */
		FName PackageName;
		
		/** The transform to apply to the streaming level. */
		FTransform Transform = FTransform::Identity;

		/** Optional function to call after the ULevelStreaming gets created. */
		TFunction<void(ULevelStreaming*)> LevelStreamingCreatedCallback = nullptr;
	};

	static UNREALED_API ULevelStreaming* AddLevelToWorld(UWorld* InWorld, const FAddLevelToWorldParams& InParams);

private:

	static UNREALED_API ULevelStreaming* AddLevelToWorld_Internal(UWorld* InWorld, const FAddLevelToWorldParams& InParams);

	static UNREALED_API int32 CopyOrMoveActorsToLevel(const TArray<AActor*>& ActorsToMove, ULevel* DestLevel, bool bMoveActors, bool bWarnAboutReferences = true, bool bWarnAboutRenaming = true, bool bMoveAllOrFail = false, TArray<AActor*>* OutActors = nullptr);

public:
	/** Sets the LevelStreamingClass for the specified Level 
	  * @param	InLevel				The level for which to change the streaming class
	  * @param	LevelStreamingClass	The desired streaming class
	  *
	  * @return	The new streaming level object
	  */
	static UNREALED_API ULevelStreaming* SetStreamingClassForLevel(ULevelStreaming* InLevel, TSubclassOf<ULevelStreaming> LevelStreamingClass);

	/**
	 * Removes the specified level from the world.  Refreshes.
	 * @param InLevel			The level to remove.
	 * @param bClearSlection	If editor selection should be cleared. 
	 *
	 * @return	true	If a level was removed.
	 */
	static UNREALED_API bool RemoveLevelFromWorld(ULevel* InLevel, bool bClearSelection = true, bool bResetTransBuffer = true);

	/**
	 * Removes the specified levels from the world.  Refreshes.
	 * @param InLevels			The levels to remove.
	 * @param bClearSlection	If editor selection should be cleared.
	 *
	 * @return	true	If all levels were removed.
	 */
	static UNREALED_API bool RemoveLevelsFromWorld(TArray<ULevel*> InLevels, bool bClearSelection = true, bool bResetTransBuffer = true);

	/**
	 * Removes the specified LevelStreaming from the world, and Refreshes.
	 * Used to Clean up references of missing levels.
	 *
	 * @return	true	If a level was removed.
	 */
	static UNREALED_API bool RemoveInvalidLevelFromWorld(ULevelStreaming* InLevelStreaming);

	/** 
	 * Sets the actors within a level's visibility via bHiddenEdLevel.  
	 * Warning: modifies ULevel::bIsVisible and bHiddenEdLevel without marking packages dirty or supporting undo.  
	 * Calling code must restore to the original state before the user can interact with the levels.
	 */
	static UNREALED_API void SetLevelVisibilityTemporarily(ULevel* Level, bool bShouldBeVisible);

	/**
	 * Sets a level's visibility in the editor. Less efficient than SetLevelsVisibility when changing the visibility of multiple levels simultaneously.
	 *
	 * @param	Level					The level to modify.
	 * @param	bShouldBeVisible		The level's new visibility state.
	 * @param	bForceLayersVisible		If true and the level is visible, force the level's layers to be visible.
	 * @param	ModifyMode				ELevelVisibilityDirtyMode mode value.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility")
	static UNREALED_API void SetLevelVisibility(ULevel* Level, const bool bShouldBeVisible, const bool bForceLayersVisible, const ELevelVisibilityDirtyMode ModifyMode = ELevelVisibilityDirtyMode::ModifyOnChange);

	/**
	 * Sets a level's visibility in the editor. More efficient than SetLevelsVisibility when changing the visibility of multiple levels simultaneously.
	 *
	 * @param	Levels					The levels to modify.
	 * @param	bShouldBeVisible		The level's new visibility state for each level.
	 * @param	bForceLayersVisible		If true and the level is visible, force the level's layers to be visible.
	 * @param	ModifyMode				ELevelVisibilityDirtyMode mode value.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility")
	static UNREALED_API void SetLevelsVisibility(const TArray<ULevel*>& Levels, const TArray<bool>& bShouldBeVisible, const bool bForceLayersVisible, const ELevelVisibilityDirtyMode ModifyMode = ELevelVisibilityDirtyMode::ModifyOnChange);
	
	/** 
	 * Deselects all BSP surfaces in this level 
	 *
	 * @param InLevel		The level to deselect the surfaces of.
	 *
	 */	
	static UNREALED_API void DeselectAllSurfacesInLevel(ULevel* InLevel);

	/**
	 * Executes an operation on the set of all referenced worlds.
	 *
	 * @param	InWorld				World containing streaming levels
	 * @param	Operation			The operation to execute on the referenced worlds, return false to break iteration
	 * @param	bIncludeInWorld		If true, include the InWorld in the output list.
	 * @param	bOnlyEditorVisible	If true, only sub-levels that should be visible in-editor are included
	 */
	static UNREALED_API void ForEachWorlds(UWorld* InWorld, TFunctionRef<bool(UWorld*)> Operation, bool bIncludeInWorld, bool bOnlyEditorVisible = false);

	/**
	 * Assembles the set of all referenced worlds.
	 *
	 * @param	InWorld				World containing streaming levels
	 * @param	Worlds				[out] The set of referenced worlds.
	 * @param	bIncludeInWorld		If true, include the InWorld in the output list.
	 * @param	bOnlyEditorVisible	If true, only sub-levels that should be visible in-editor are included
	 */
	static UNREALED_API void GetWorlds(UWorld* InWorld, TArray<UWorld*>& OutWorlds, bool bIncludeInWorld, bool bOnlyEditorVisible = false);

	/**
	 * Returns all levels for a world.
	 *
	 * @param	World				World containing levels
	 * @return						Set of all levels
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility")
	static const TArray<ULevel*> GetLevels(UWorld* World);

private:
	/**
	* Utility methods used by RemoveLevelsFromWorld
	*/
	static void PrivateRemoveLevelFromWorld(ULevel* Level);
	static void PrivateDestroyLevel(ULevel* Level);

	static bool PrivateRemoveInvalidLevelFromWorld(ULevelStreaming* InLevelStreaming);
private:

};

// For backwards compatibility
typedef UEditorLevelUtils EditorLevelUtils;
