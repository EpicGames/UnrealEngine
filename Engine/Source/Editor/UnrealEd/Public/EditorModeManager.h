// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "UObject/GCObject.h"
#include "UnrealWidgetFwd.h"
#include "Editor.h"
#include "EditorUndoClient.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "EdMode.h"

#include "Elements/Framework/TypedElementSelectionSet.h"

struct FRotationContext;
class UEditorTransformGizmoContextObject;
class ITypedElementWorldInterface;
class FCanvas;
class FEditorViewportClient;
class FEdMode;
class FModeTool;
class FPrimitiveDrawInterface;
class FSceneView;
class FViewport;
class IToolkitHost;
class USelection;
struct FConvexVolume;
struct FViewportClick;
class UEdMode;
class UInteractiveGizmoManager;
class UInputRouter;
class UModeManagerInteractiveToolsContext;
class UTypedElementSelectionSet;
class IGizmoStateTarget;
class UEditorGizmoStateTarget;
struct FGizmoState;

/**
 * A helper class to store the state of the various editor modes.
 */
class FEditorModeTools : public FGCObject, public FEditorUndoClient, public TSharedFromThis<FEditorModeTools>
{
public:
	UNREALED_API FEditorModeTools();
	UNREALED_API virtual ~FEditorModeTools();

	/**
	 * Set the default editor mode for these tools
	 * 
	 * @param	DefaultModeID		The mode ID for the new default mode
	 */
	UNREALED_API void SetDefaultMode( const FEditorModeID DefaultModeID );

	/**
	 * Adds a new default mode to this tool's list of default modes.  You can have multiple default modes, but they all must be compatible with each other.
	 * 
	 * @param	DefaultModeID		The mode ID for the new default mode
	 */
	UNREALED_API void AddDefaultMode( const FEditorModeID DefaultModeID );

	/**
	 * Removes a default mode
	 * 
	 * @param	DefaultModeID		The mode ID for the default mode to remove
	 */
	UNREALED_API void RemoveDefaultMode( const FEditorModeID DefaultModeID );

	/**
	 * Returns whether or not the provided mode ID is a default mode
	 */
	bool IsDefaultMode(const FEditorModeID ModeID) const { return DefaultModeIDs.Contains(ModeID); }

	/**
	 * Activates the default modes defined by this class.  Note that there can be more than one default mode, and this call will activate them all in sequence.
	 */
	UNREALED_API void ActivateDefaultMode();

	/** 
	 * Returns true if the default modes are active.  Note that there can be more than one default mode, and this will only return true if all default modes are active.
	 */
	UNREALED_API bool IsDefaultModeActive() const;

	/**
	 * Activates an editor mode. Shuts down all other active modes which cannot run with the passed in mode.
	 * 
	 * @param InID		The ID of the editor mode to activate.
	 * @param bToggle	true if the passed in editor mode should be toggled off if it is already active.
	 */
	UNREALED_API void ActivateMode( FEditorModeID InID, bool bToggle = false );

	/**
	 * Deactivates an editor mode. 
	 * 
	 * @param InID		The ID of the editor mode to deactivate.
	 */
	UNREALED_API void DeactivateMode(FEditorModeID InID);

	/**
	 * Deactivate the mode and entirely purge it from memory. Used when a mode type is unregistered
	 */
	UNREALED_API void DestroyMode(FEditorModeID InID);



	/**
	 * Whether or not the mode toolbox (where mode details panels and some tools are) should be shown.
	 */
	UE_DEPRECATED(4.26, "Individual toolkit hosts, such as the level editor, should handle determining if they show a mode toolbox for hosted toolkits.")
	UNREALED_API bool ShouldShowModeToolbox() const;
protected:
	/** Exits the given editor mode */
	UNREALED_API void ExitMode(UEdMode* InMode);

	/** Removes the mode ID from the tools manager when a mode is unregistered */
	UNREALED_API void OnModeUnregistered(FEditorModeID ModeID);
		
	UNREALED_API void DeactivateModeAtIndex(int32 Index);
public:

	/**
	 * Deactivates all modes, note some modes can never be deactivated.
	 */
	UNREALED_API void DeactivateAllModes();

	UNREALED_API UEdMode* GetActiveScriptableMode(FEditorModeID InID) const;

	UNREALED_API virtual UTexture2D* GetVertexTexture() const;

	/**
	 * Returns true if the current mode is not the specified ModeID.  Also optionally warns the user.
	 *
	 * @param	ModeID			The editor mode to query.
	 * @param	ErrorMsg		If specified, inform the user the reason why this is a problem
	 * @param	bNotifyUser		If true, display the error as a notification, instead of a dialog
	 * @return					true if the current mode is not the specified mode.
	 */
	UNREALED_API bool EnsureNotInMode(FEditorModeID ModeID, const FText& ErrorMsg = FText::GetEmpty(), bool bNotifyUser = false) const;

	UNREALED_API FMatrix GetCustomDrawingCoordinateSystem() const;
	UNREALED_API FMatrix GetCustomInputCoordinateSystem() const;
	UNREALED_API FMatrix GetLocalCoordinateSystem() const;
	UNREALED_API FMatrix GetParentSpaceCoordinateSystem() const;
	UNREALED_API FMatrix GetExplicitCoordinateSystem() const;
	
	/** 
	 * Returns true if the passed in editor mode is active 
	 */
	UNREALED_API bool IsModeActive( FEditorModeID InID ) const;

	/**
	 * Returns a pointer to an active mode specified by the passed in ID
	 * If the editor mode is not active, NULL is returned
	 */
	UNREALED_API FEdMode* GetActiveMode( FEditorModeID InID );
	UNREALED_API const FEdMode* GetActiveMode( FEditorModeID InID ) const;

	template <typename SpecificModeType>
	SpecificModeType* GetActiveModeTyped( FEditorModeID InID )
	{
		return static_cast<SpecificModeType*>(GetActiveMode(InID));
	}

	template <typename SpecificModeType>
	const SpecificModeType* GetActiveModeTyped( FEditorModeID InID ) const
	{
		return static_cast<SpecificModeType*>(GetActiveMode(InID));
	}

	/**
	 * Returns the active tool of the passed in editor mode.
	 * If the passed in editor mode is not active or the mode has no active tool, NULL is returned
	 */
	UNREALED_API const FModeTool* GetActiveTool( FEditorModeID InID ) const;

	void SetShowWidget( bool InShowWidget )	{ bShowWidget = InShowWidget; }
	UNREALED_API bool GetShowWidget() const;

	/** Cycle the widget mode, forwarding queries to modes */
	UNREALED_API void CycleWidgetMode (void);

	/** Check with modes to see if the widget mode can be cycled */
	UNREALED_API bool CanCycleWidgetMode() const;

	/**Save Widget Settings to Ini file*/
	UNREALED_API void SaveWidgetSettings();
	/**Load Widget Settings from Ini file*/
	UNREALED_API void LoadWidgetSettings();

	/** Gets the widget axis to be drawn */
	UNREALED_API EAxisList::Type GetWidgetAxisToDraw( UE::Widget::EWidgetMode InWidgetMode ) const;

	/** Mouse tracking interface.  Passes tracking messages to all active modes */
	UNREALED_API bool StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport);
	UNREALED_API bool EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport);
	bool IsTracking() const { return bIsTracking; }

	UNREALED_API bool AllowsViewportDragTool() const;

	/** Notifies all active modes that a map change has occured */
	UNREALED_API void MapChangeNotify();

	/** Notifies all active modes to empty their selections */
	UNREALED_API void SelectNone();

	/** Notifies all active modes of box selection attempts */
	UNREALED_API bool BoxSelect( FBox& InBox, bool InSelect );

	/** Notifies all active modes of frustum selection attempts */
	UNREALED_API bool FrustumSelect( const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, bool InSelect );

	/** true if any active mode uses a transform widget */
	UNREALED_API bool UsesTransformWidget() const;

	/** true if any active mode uses the passed in transform widget */
	UNREALED_API bool UsesTransformWidget( UE::Widget::EWidgetMode CheckMode ) const;

	/** Sets the current widget axis */
	UNREALED_API void SetCurrentWidgetAxis( EAxisList::Type NewAxis );

	/** Notifies all active modes of mouse click messages. */
	UNREALED_API bool HandleClick(FEditorViewportClient* InViewportClient,  HHitProxy *HitProxy, const FViewportClick &Click );

	/**
	 * Allows editor modes to override the bounding box used to focus the viewport on a selection
	 * 
	 * @param Actor			The selected actor that is being considered for focus
	 * @param PrimitiveComponent	The component in the actor being considered for focus
	 * @param InOutBox		The box that should be computed for the actor and component
	 * @return bool			true if a mode overrides the box and populated InOutBox, false if it did not populate InOutBox
	 */
	UNREALED_API bool ComputeBoundingBoxForViewportFocus(AActor* Actor, UPrimitiveComponent* PrimitiveComponent, FBox& InOutBox);

	/** true if the passed in brush actor should be drawn in wireframe */	
	UNREALED_API bool ShouldDrawBrushWireframe( AActor* InActor ) const;

	/** true if brush vertices should be drawn */
	UNREALED_API bool ShouldDrawBrushVertices() const;

	/** Ticks all active modes */
	UNREALED_API void Tick( FEditorViewportClient* ViewportClient, float DeltaTime );

	// Convert any pending tools-context invalidations into ViewportClient->Invalidate() calls
	UNREALED_API void PropagateInvalidations( FEditorViewportClient* ViewportClient );

	/** Notifies all active modes of any change in mouse movement */
	UNREALED_API bool InputDelta( FEditorViewportClient* InViewportClient,FViewport* InViewport,FVector& InDrag,FRotator& InRot,FVector& InScale );

	/** Notifies all active modes of captured mouse movement */	
	UNREALED_API bool CapturedMouseMove( FEditorViewportClient* InViewportClient, FViewport* InViewport, int32 InMouseX, int32 InMouseY );

	/** Notifies all active modes of all captured mouse movement */	
	UNREALED_API bool ProcessCapturedMouseMoves( FEditorViewportClient* InViewportClient, FViewport* InViewport, const TArrayView<FIntPoint>& CapturedMouseMoves );

	/** 
	 * Notifies all active modes of keyboard input 
	 * @param bRouteToToolsContext If true, routes to the tools context and its input router before routing
	 *  to modes (and does not route to modes if tools context handles it). We currently need the ability to
	 *  set this to false due to some behaviors being routed in different conditions to legacy modes compared
	 *  to the input router (see its use in EditorViewportClient.cpp).
	 */
	UNREALED_API bool InputKey( FEditorViewportClient* InViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event, bool bRouteToToolsContext = true);

	/** Notifies all active modes of axis movement */
	UNREALED_API bool InputAxis( FEditorViewportClient* InViewportClient, FViewport* Viewport, int32 ControllerId, FKey Key, float Delta, float DeltaTime);

	UNREALED_API bool MouseEnter( FEditorViewportClient* InViewportClient, FViewport* Viewport, int32 X, int32 Y );
	
	UNREALED_API bool MouseLeave( FEditorViewportClient* InViewportClient, FViewport* Viewport );

	/** Notifies all active modes that the mouse has moved */
	UNREALED_API bool MouseMove( FEditorViewportClient* InViewportClient, FViewport* Viewport, int32 X, int32 Y );

	/** Notifies all active modes that a viewport has received focus */
	UNREALED_API bool ReceivedFocus( FEditorViewportClient* InViewportClient, FViewport* Viewport );

	/** Notifies all active modes that a viewport has lost focus */
	UNREALED_API bool LostFocus( FEditorViewportClient* InViewportClient, FViewport* Viewport );

	/** Draws all active modes */	
	UNREALED_API void DrawActiveModes( const FSceneView* InView, FPrimitiveDrawInterface* PDI );

	/** Renders all active modes */
	UNREALED_API void Render( const FSceneView* InView, FViewport* Viewport, FPrimitiveDrawInterface* PDI );

	/** Draws the HUD for all active modes */
	UNREALED_API void DrawHUD( FEditorViewportClient* InViewportClient,FViewport* Viewport,const FSceneView* View,FCanvas* Canvas );

	/** 
	 * Get a pivot point specified by any active modes around which the camera should orbit
	 * @param	OutPivot	The custom pivot point returned by the mode/tool
	 * @return	true if a custom pivot point was specified, false otherwise.
	 */
	UNREALED_API bool GetPivotForOrbit( FVector& OutPivot ) const;

	/** Calls PostUndo on all active modes */
	// Begin FEditorUndoClient
	UNREALED_API virtual void PostUndo(bool bSuccess) override;
	UNREALED_API virtual void PostRedo(bool bSuccess) override;
	// End of FEditorUndoClient

	/** True if we should allow widget move */
	UNREALED_API bool AllowWidgetMove() const;

	/** True if we should disallow mouse delta tracking. */
	UNREALED_API bool DisallowMouseDeltaTracking() const;
	
	/** True if legacy gizmos are required */
	UNREALED_API bool RequiresLegacyViewportInteractions() const;

	/** Get a cursor to override the default with, if any */
	UNREALED_API bool GetCursor(EMouseCursor::Type& OutCursor) const;

	/**
	 * Called to map a cursor reply to an actual widget to render.
	 *
	 * @return	the widget that should be rendered for this cursor, if any.
	 */
	UNREALED_API TSharedPtr<SWidget> MapCursor() const;

	UE_DEPRECATED(5.8, "Use GetCursorVisibilityOverride() instead. `bSoftwareCursorVisible` is not a ref and is hence broken")
	/** Get override cursor visibility settings */
	UNREALED_API bool GetOverrideCursorVisibility(bool& bWantsOverride, bool& bHardwareCursorVisible, bool bSoftwareCursorVisible) const;
	
	/** Get override cursor visibility settings */
	UNREALED_API bool GetCursorVisibilityOverride(bool& bOutWantsOverride, bool& bOutHardwareCursorVisible, bool& bOutSoftwareCursorVisible) const;

	/** Called before converting mouse movement to drag/rot */
	UNREALED_API bool PreConvertMouseMovement(FEditorViewportClient* InViewportClient);

	/** Called after converting mouse movement to drag/rot */
	UNREALED_API bool PostConvertMouseMovement(FEditorViewportClient* InViewportClient);

	/**
	 * Returns a good location to draw the widget at.
	 */
	UNREALED_API FVector GetWidgetLocation() const;

	/**
	 * Changes the current widget mode.
	 */
	UNREALED_API void SetWidgetMode( UE::Widget::EWidgetMode InWidgetMode );

	/**
	 * Allows you to temporarily override the widget mode.  Call this function again
	 * with WM_None to turn off the override.
	 */
	UNREALED_API void SetWidgetModeOverride( UE::Widget::EWidgetMode InWidgetMode );

	/**
	 * Retrieves the current widget mode, taking overrides into account.
	 */
	UNREALED_API UE::Widget::EWidgetMode GetWidgetMode() const;


	/**
	* Set Scale On The Widget
	*/
	UNREALED_API void SetWidgetScale(float InScale);

	/**
	*  Get Widget Scale
	*/
	UNREALED_API float GetWidgetScale() const;

	// FGCObject interface
	UNREALED_API virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FEditorModeTools");
	}
	// End of FGCObject interface

	/**
	 * Loads the state that was saved in the INI file
	 */
	UNREALED_API void LoadConfig(void);

	/**
	 * Saves the current state to the INI file
	 */
	UNREALED_API void SaveConfig(void);

	/** 
	 * Sets the pivot locations
	 * 
	 * @param Location 		The location to set
	 * @param bIncGridBase	Whether or not to also set the GridBase
	 */
	UNREALED_API void SetPivotLocation( const FVector& Location, const bool bIncGridBase );

	/**
	 * Multicast delegate for OnModeEntered and OnModeExited callbacks.
	 *
	 * First parameter:  The editor mode that was changed
	 * Second parameter:  True if entering the mode, or false if exiting the mode
	 */
	DECLARE_EVENT_TwoParams(FEditorModeTools, FEditorModeIDChangedEvent, const FEditorModeID&, bool);
	FEditorModeIDChangedEvent& OnEditorModeIDChanged() { return EditorModeIDChangedEvent; }

	/** delegate type for triggering when widget mode changed */
	DECLARE_EVENT_OneParam( FEditorModeTools, FWidgetModeChangedEvent, UE::Widget::EWidgetMode );
	FWidgetModeChangedEvent& OnWidgetModeChanged() { return WidgetModeChangedEvent; }

	/**	Broadcasts the WidgetModeChanged event */
	void BroadcastWidgetModeChanged(UE::Widget::EWidgetMode InWidgetMode) { WidgetModeChangedEvent.Broadcast(InWidgetMode); }

	/**	Broadcasts the EditorModeIDChanged event */
	void BroadcastEditorModeIDChanged(const FEditorModeID& ModeID, bool IsEnteringMode) { EditorModeIDChangedEvent.Broadcast(ModeID, IsEnteringMode); }

	/** delegate type for triggering when coordinate system changed */
	DECLARE_EVENT_OneParam(FEditorModeTools, FCoordSystemChangedEvent, ECoordSystem);
	FCoordSystemChangedEvent& OnCoordSystemChanged() { return CoordSystemChangedEvent; }

	/**	Broadcasts the CoordSystemChangedEvent event */
	void BroadcastCoordSystemChanged(ECoordSystem InCoordSystem) { CoordSystemChangedEvent.Broadcast(InCoordSystem); }

	/** Delegate type for triggering when a mode Editing Disallowed changed */
	DECLARE_EVENT(FEditorModeTools, FIsEditingDisallowedChangedEvent);
	FIsEditingDisallowedChangedEvent& OnIsEditingDisallowedChanged() { return IsEditingDisallowedChangedEvent; }

	/**	Broadcasts the IsEditingDisallowedChangedEvent event */
	void BroadcastIsEditingDisallowedChanged() { IsEditingDisallowedChangedEvent.Broadcast(); }

	/**
	 * Returns the current CoordSystem
	 * 
	 * @param bGetRawValue true when you want the actual value of CoordSystem, not the value modified by the state.
	 */
	UNREALED_API ECoordSystem GetCoordSystem(bool bGetRawValue = false) const;

	/** Sets the current CoordSystem */
	UNREALED_API void SetCoordSystem(ECoordSystem NewCoordSystem);

	/** Sets the hide viewport UI state */
	void SetHideViewportUI( bool bInHideViewportUI ) { bHideViewportUI = bInHideViewportUI; }

	/** Is the viewport UI hidden? */
	bool IsViewportUIHidden() const { return bHideViewportUI; }

	/** Called by Editors when they are about to close */
	UNREALED_API bool OnRequestClose();

	bool PivotShown;
	bool Snapping;
	bool SnappedActor;

	FVector CachedLocation;
	FVector PivotLocation;
	FVector SnappedLocation;
	FVector GridBase;

	/** The angle for the translate rotate widget */
	float TranslateRotateXAxisAngle;

	/** The angles for the 2d translate rotate widget */
	float TranslateRotate2DAngle;

	/** Draws in the top level corner of all FEditorViewportClient windows (can be used to relay info to the user). */
	FString InfoString;

	/** Sets the host for toolkits created via modes from this mode manager (can only be called once) */
	UNREALED_API void SetToolkitHost(TSharedRef<IToolkitHost> Host);

	/** Returns the host for toolkits created via modes from this mode manager */
	UNREALED_API TSharedPtr<IToolkitHost> GetToolkitHost() const;

	/** Check if toolkit host exists */
	UNREALED_API bool HasToolkitHost() const;

	/**
	 * Returns the set of selected actors.
	 */
	UNREALED_API virtual USelection* GetSelectedActors() const;

	/**
	 * @return the set of selected non-actor objects.
	 */
	UNREALED_API virtual USelection* GetSelectedObjects() const;

	/**
	 * Returns the set of selected components.
	 */
	UNREALED_API virtual USelection* GetSelectedComponents() const;

	/**
	 * Returns the selection set for the toolkit host.
	 * (i.e. the selection set for the level editor)
	 */
	UNREALED_API virtual UTypedElementSelectionSet* GetEditorSelectionSet() const;

	/**
	 * Stores the current selection under the given key, and clears the current selection state if requested.
	 */
	UNREALED_API void StoreSelection(FName SelectionStoreKey, bool bClearSelection = true);

	/**
	 * Restores the selection to the state that was stored using the given key.
	 */
	UNREALED_API void RestoreSelection(FName SelectionStoreKey);

	/**
	 * Returns the world that is being edited by this mode manager
	 */ 
	UNREALED_API virtual UWorld* GetWorld() const;

	/**
	 * Returns the currently hovered viewport client
	 */
	UNREALED_API FEditorViewportClient* GetHoveredViewportClient() const;

	/**
	 * Returns the currently focused viewport client
	 */
	UNREALED_API FEditorViewportClient* GetFocusedViewportClient() const;

	/**
	 * Whether or not the current selection has a scene component selected
 	 */
	UNREALED_API bool SelectionHasSceneComponent() const;
	UNREALED_API void SetSelectionHasSceneComponent(bool bHasSceneComponent);

	UNREALED_API bool IsSelectionAllowed(AActor* InActor, const bool bInSelected) const;

	UNREALED_API bool IsSelectionHandled(AActor* InActor, const bool bInSelected) const;

	UNREALED_API bool IsEditingDisallowed(AActor* InActor) const;

	UNREALED_API bool ProcessEditDuplicate();
	UNREALED_API bool ProcessEditDelete();
	UNREALED_API bool ProcessEditCut();
	UNREALED_API bool ProcessEditCopy();
	UNREALED_API bool ProcessEditPaste();
	UNREALED_API EEditAction::Type  GetActionEditDuplicate();
	UNREALED_API EEditAction::Type  GetActionEditDelete();
	UNREALED_API EEditAction::Type  GetActionEditCut();
	UNREALED_API EEditAction::Type  GetActionEditCopy();
	UNREALED_API EEditAction::Type GetActionEditPaste();
	UNREALED_API EEditAction::Type GetActionDragDuplicate();

	UE_DEPRECATED(5.0, "This function is redundant, and is handled as part of a call to ActivateMode.")
	UNREALED_API void DeactivateOtherVisibleModes(FEditorModeID InMode);
	UNREALED_API bool IsSnapRotationEnabled() const;
	UNREALED_API bool SnapRotatorToGridOverride(FRotator& InRotation) const;
	UNREALED_API void ActorsDuplicatedNotify(TArray<AActor*>& InPreDuplicateSelection, TArray<AActor*>& InPostDuplicateSelection, const bool bOffsetLocations);
	UNREALED_API void ActorMoveNotify();
	UNREALED_API void ActorSelectionChangeNotify();
	UNREALED_API void ElementSelectionChangeNotify();
	UNREALED_API void ActorPropChangeNotify();
	UNREALED_API void UpdateInternalData();
	UNREALED_API bool IsOnlyVisibleActiveMode(FEditorModeID InMode) const;
	UNREALED_API bool IsOnlyActiveMode(FEditorModeID InMode) const;

	/*
	* Sets the active Modes ToolBar Palette Tab to the named Palette
	*/
	//void  InvokeToolPaletteTab(FEditorModeID InMode, FName InPaletteName);

	/** returns true if all active EdModes are OK with an AutoSave happening now  */
	UNREALED_API bool CanAutoSave() const;
	
	/** returns true if all active EdModes are OK support operation on current asset */
	UNREALED_API bool IsOperationSupportedForCurrentAsset(EAssetOperation InOperation) const;

	UNREALED_API void RemoveAllDelegateHandlers();

	/** @return ToolsContext for this Mode Manager */
	UNREALED_API UModeManagerInteractiveToolsContext* GetInteractiveToolsContext() const;

	/** Begin New TRS Gizmo interface */
	UNREALED_API bool UsesViewportITF() const;
	UNREALED_API void SetSupportsViewportITF(bool bIsSupported);
	UNREALED_API bool SupportsViewportITF() const;
	UNREALED_API IGizmoStateTarget* GetGizmoStateTarget();
	UNREALED_API bool BeginTransform(const FGizmoState& InState);
	UNREALED_API bool EndTransform(const FGizmoState& InState) const;
	UNREALED_API bool OnTransformCanceled(const FGizmoState& InState) const;
	UNREALED_API bool HasOngoingTransform() const;
	UNREALED_API UEditorTransformGizmoContextObject* GetGizmoContext() const;
	UNREALED_API const FRotationContext& GetRotationContext() const;
	/** End New TRS Gizmo interface */
	
protected:


	/** Handles the notification when a world is going through GC to clean up any modes pending deactivation. */
	UNREALED_API void OnWorldCleanup(UWorld* InWorld, bool bSessionEnded, bool bCleanupResources);
	
	UNREALED_API void OnViewportListChanged();

	UNREALED_API virtual void DrawBrackets(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas);

	UNREALED_API void ForEachEdMode(TFunctionRef<bool(UEdMode*)> InCalllback) const;
	UNREALED_API bool TestAllModes(TFunctionRef<bool(UEdMode*)> InCalllback, bool bExpected) const;
	
	template <class InterfaceToCastTo>
	void ForEachEdMode(TFunctionRef<bool(InterfaceToCastTo*)> InCallback) const
	{
		ForEachEdMode([InCallback](UEdMode* Mode)
		{
			if (InterfaceToCastTo* CastedMode = Cast<InterfaceToCastTo>(Mode))
			{
				return InCallback(CastedMode);
			}

			return true;
		});
	}

	/** Returns the custom coordinate matrix using a callback-type request. */
	FMatrix GetCustomCoordinateSystem(TUniqueFunction<void(const TTypedElement<ITypedElementWorldInterface>&, FTransform&)>&& InGetTransformFunc) const;

	UNREALED_API void ExitAllModesPendingDeactivate();

	/** List of default modes for this tool.  These must all be compatible with each other. */
	TArray<FEditorModeID> DefaultModeIDs;

	/** A list of active editor modes. */
	TArray< TObjectPtr<UEdMode> > ActiveScriptableModes;

	/** The host of the toolkits created by these modes */
	TWeakPtr<IToolkitHost> ToolkitHost;

	/** A list of previously active editor modes that we will potentially recycle */
	TMap< FEditorModeID, TObjectPtr<UEdMode> > RecycledScriptableModes;

	/** A list of previously active editor modes that we will potentially recycle */
	TMap< FEditorModeID, TObjectPtr<UEdMode> > PendingDeactivateModes;

	/** The mode that the editor viewport widget is in. */
	UE::Widget::EWidgetMode WidgetMode;

	/** If the widget mode is being overridden, this will be != WM_None. */
	UE::Widget::EWidgetMode OverrideWidgetMode;

	/** If 1, draw the widget and let the user interact with it. */
	bool bShowWidget;

	/** if true, the viewports will hide all UI overlays */
	bool bHideViewportUI;

	/** if true the current selection has a scene component */
	bool bSelectionHasSceneComponent;

	/** Scale Factor for Widget*/
	float WidgetScale;

	TObjectPtr<UModeManagerInteractiveToolsContext> InteractiveToolsContext;

private:

	/** The coordinate system the widget is operating within. */
	ECoordSystem CoordSystem;

	/** Multicast delegate that is broadcast when a mode is entered or exited */
	FEditorModeIDChangedEvent EditorModeIDChangedEvent;

	/** Multicast delegate that is broadcast when a widget mode is changed */
	FWidgetModeChangedEvent WidgetModeChangedEvent;

	/** Multicast delegate that is broadcast when the coordinate system is changed */
	FCoordSystemChangedEvent CoordSystemChangedEvent;

	/** Multicast delegate that is broadcast when a mode Editing Disallowed changed */
	FIsEditingDisallowedChangedEvent IsEditingDisallowedChangedEvent;

	/** Flag set between calls to StartTracking() and EndTracking() */
	bool bIsTracking;

	/** Guard to prevent modes from entering as part of their exit routine */
	bool bIsExitingModesDuringTick = false;

	/** GizmoStateTarget used to handle a new TRS gizmo transform begin/end sequence. */
	TWeakObjectPtr<UEditorGizmoStateTarget> WeakGizmoStateTarget;

	/** GizmoContext used to get context specific gizmo properties. */
	mutable TWeakObjectPtr<UEditorTransformGizmoContextObject> WeakGizmoContext;

	/** Determines whether this mode tool can ever support TRS gizmos. */
	bool bViewportITF = true;

	/** Flag to track if we started a new TRS gizmo transform.
	 * NOTE: we have to use it for now as StartTracking / EndTracking iterates thru the modes even if the ITF context captured something.
	 * This might not be needed anymore in the future but as some routing behavior had to be kept for legacy reasons, we use that extra flag.
	 * If StartTracking / EndTracking were to be changed (which needs more testing), this could probably be removed.
	 */
	bool bHasOngoingTransform = false;

	FEditorViewportClient* HoveredViewportClient = nullptr;
	FEditorViewportClient* FocusedViewportClient = nullptr;

	TMap<FName, FTypedElementSelectionSetState> StoredSelectionSets;

	static TAutoConsoleVariable<int32> CVarEnableITFCursorOverrideSupport;
};
