// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Textures/SlateIcon.h"
#include "Tools/Modes.h"

class FEditorModeTools;

// Required forward declarations
class FEdMode;

DECLARE_DELEGATE_RetVal(TSharedRef<FEdMode>, FEditorModeFactoryCallback);
/**
 *	Class responsible for maintaining a list of registered editor mode types.
 *
 *	Example usage:
 *
 *	Register your mode type with:
 *		FEditorModeRegistry::Get().RegisterMode<FMyEditorMode>( FName( TEXT("MyEditorMode") ) );
 *	or:
 *		class FMyEditorModeFactory : public IEditorModeFactory
 *		{
 *			virtual void OnSelectionChanged( FEditorModeTools& Tools, UObject* ItemUndergoingChange ) const override;
 *			virtual FEditorModeInfo GetModeInfo() const override;
 *			virtual TSharedRef<FEdMode> CreateMode() const override;
 *		};
 *		TSharedRef<FMyEditorModeFactory> Factory = MakeShareable( new FMyEditorModeFactory );
 *		FEditorModeRegistry::Get().RegisterMode( FName( TEXT("MyEditorMode") ), Factory );
 *
 *	Unregister your mode when it is no longer available like so (this will prompt the destruction of any existing modes of this type):
 *		FEditorModeRegistry::Get().UnregisterMode( FName( TEXT("MyEditorMode") ) );
 */

struct IEditorModeFactory : public TSharedFromThis<IEditorModeFactory>
{
	/** Virtual destructor */
	virtual ~IEditorModeFactory() = default;

	/**
	 * Allows mode factories to handle selection change events, and potentially activate/deactivate modes

	 * @param Tools					The mode tools that are triggering the event
	 * @param ItemUndergoingChange	Either an actor being selected or deselected, or a selection set being modified (typically emptied)
	 */
	virtual void OnSelectionChanged(FEditorModeTools& Tools, UObject* ItemUndergoingChange) const { }

	/**
	 * Gets the information pertaining to the mode type that this factory creates
	 */
	virtual FEditorModeInfo GetModeInfo() const = 0;

	/**
	 * Create a new instance of our mode
	 */
	virtual TSharedRef<FEdMode> CreateMode() const = 0;
};

struct FEditorModeFactory : IEditorModeFactory
{
public:
	UNREALED_API FEditorModeFactory(const FEditorModeInfo& InModeInfo);
	UNREALED_API FEditorModeFactory(FEditorModeInfo&& InModeInfo);
	UNREALED_API virtual ~FEditorModeFactory();

	/** Information pertaining to this factory's mode */
	FEditorModeInfo ModeInfo;

	/** Callback used to create an instance of this mode type */
	FEditorModeFactoryCallback FactoryCallback;

	/**
	 * Gets the information pertaining to the mode type that this factory creates
	 */
	UNREALED_API virtual FEditorModeInfo GetModeInfo() const final;

	UNREALED_API virtual TSharedRef<FEdMode> CreateMode() const final;
};

/**
 * A registry of editor modes and factories
 */
class FEditorModeRegistry
{
	typedef TMap<FEditorModeID, TSharedRef<IEditorModeFactory>> FactoryMap;
	friend class ULegacyEdModeWrapper;
	friend class UAssetEditorSubsystem;

public:
	/**
	 * Singleton access
	 */
	UNREALED_API static FEditorModeRegistry& Get();

	/**
	 * Get a list of information for all currently registered modes, sorted by UI priority order
	 */
	UE_DEPRECATED(4.26, "Use UAssetEditorSubsystem::GetEditorModeInfoOrderedByPriority")
	UNREALED_API TArray<FEditorModeInfo> GetSortedModeInfo() const;

	/**
	 * Get a currently registered mode information for specified ID
	 */
	UE_DEPRECATED(4.26, "Use UAssetEditorSubsystem::FindEditorModeInfo")
	UNREALED_API FEditorModeInfo GetModeInfo(FEditorModeID ModeID) const;

	/**
	 * Registers an editor mode. Typically called from a module's StartupModule() routine.
	 *
	 * @param ModeID	ID of the mode to register
	 */
	UNREALED_API void RegisterMode(FEditorModeID ModeID, TSharedRef<IEditorModeFactory> Factory);

	/**
	 * Registers an editor mode type. Typically called from a module's StartupModule() routine.
	 *
	 * @param ModeID	ID of the mode to register
	 */
	template<class T>
	void RegisterMode(FEditorModeID ModeID, FText Name = FText(), FSlateIcon IconBrush = FSlateIcon(), bool bVisible = false, int32 PriorityOrder = MAX_int32)
	{
		TSharedRef<FEditorModeFactory> Factory = MakeShareable( new FEditorModeFactory(FEditorModeInfo(ModeID, Name, IconBrush, bVisible, PriorityOrder)) );

		Factory->FactoryCallback = FEditorModeFactoryCallback::CreateStatic([]() -> TSharedRef<FEdMode>{
			return MakeShareable(new T);
		});
		RegisterMode(ModeID, Factory);
	}

	/**
	 * Unregisters an editor mode. Typically called from a module's ShutdownModule() routine.
	 * Note: will exit the edit mode if it is currently active.
	 *
	 * @param ModeID	ID of the mode to unregister
	 */
	UNREALED_API void UnregisterMode(FEditorModeID ModeID);

	/**
	 * Event that is triggered whenever a mode is registered or unregistered
	 */
	UE_DEPRECATED(4.26, "Use UAssetEditorSubsystem::OnEditorModesChanged")
	FRegisteredModesChangedEvent& OnRegisteredModesChanged();
	
	/**
	 * Event that is triggered whenever a mode is registered
	 */
	UE_DEPRECATED(4.26, "Use UAssetEditorSubsystem::OnEditorModeRegistered")
	FOnModeRegistered& OnModeRegistered();
	
	/**
	 * Event that is triggered whenever a mode is unregistered
	 */
	UE_DEPRECATED(4.26, "Use UAssetEditorSubsystem::OnEditorModeUnregistered")
	FOnModeUnregistered& OnModeUnregistered();

	/**
	 * Const access to the internal factory map
	 */
	const FactoryMap& GetFactoryMap() const { return ModeFactories; }


private:
	/**
	 * Initialize this registry
	 */
	void Initialize();

	/**
	 * Shutdown this registry
	 */
	void Shutdown();

	/**
	 * Create a new instance of the mode registered under the specified ID
	 */
	TSharedPtr<FEdMode> CreateMode(FEditorModeID ModeID, FEditorModeTools& Owner);

	/** A map of editor mode IDs to factory callbacks */
	FactoryMap ModeFactories;

	bool bInitialized = false;
};
