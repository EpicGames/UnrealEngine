// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Engine/EngineBaseTypes.h"
#include "InputCoreTypes.h"
#include "Math/Box.h"
#include "Math/Rotator.h"
#include "Math/UnrealMathSSE.h"

class FCanvas;
class FEditorViewportClient;
class FPrimitiveDrawInterface;
class FSceneView;
class FViewport;
struct FConvexVolume;

enum EGeomSelectionType
{
	GS_Object,
	GS_Poly,
	GS_Edge,
	GS_Vertex,
};

enum EGeomSelectionStatus
{
	GSS_None = 0,
	GSS_Polygon = 1,
	GSS_Edge = 2,
	GSS_Vertex = 4,
};



enum EModeTools : int8
{
	MT_None,
	MT_InterpEdit,
	MT_GeometryModify,			// Modification of geometry through modifiers
	MT_Texture					// Modifying texture alignment via the widget
};


/**
 * Base class for all editor mode tools.
 */
class FModeTool
{
public:
	UNREALED_API FModeTool();
	UNREALED_API virtual ~FModeTool();

	/** Returns the name that gets reported to the editor. */
	virtual FString GetName() const		{ return TEXT("Default"); }

	// User input

	virtual bool MouseEnter( FEditorViewportClient* ViewportClient,FViewport* Viewport,int32 x, int32 y ) { return false; }

	virtual bool MouseLeave( FEditorViewportClient* ViewportClient,FViewport* Viewport ) { return false; }

	virtual bool MouseMove(FEditorViewportClient* ViewportClient,FViewport* Viewport,int32 x, int32 y) { return false; }

	virtual bool ReceivedFocus(FEditorViewportClient* ViewportClient,FViewport* Viewport) { return false; }

	virtual bool LostFocus(FEditorViewportClient* ViewportClient,FViewport* Viewport) { return false; }

	/**
	 * Called when the mouse is moved while a window input capture is in effect
	 *
	 * @param	InViewportClient	Level editor viewport client that captured the mouse input
	 * @param	InViewport			Viewport that captured the mouse input
	 * @param	InMouseX			New mouse cursor X coordinate
	 * @param	InMouseY			New mouse cursor Y coordinate
	 *
	 * @return	true if input was handled
	 */
	virtual bool CapturedMouseMove( FEditorViewportClient* InViewportClient, FViewport* InViewport, int32 InMouseX, int32 InMouseY ) { return false; }


	/**
	 * @return		true if the delta was handled by this editor mode tool.
	 */
	virtual bool InputAxis(FEditorViewportClient* InViewportClient, FViewport* Viewport, int32 ControllerId, FKey Key, float Delta, float DeltaTime)
	{
		return false;
	}

	/**
	 * @return		true if the delta was handled by this editor mode tool.
	 */
	virtual bool InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale)
	{
		return false;
	}

	/**
	 * @return		true if the key was handled by this editor mode tool.
	 */
	virtual bool InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event)
	{
		return false;
	}

	UNREALED_API virtual void Render(const FSceneView* View,FViewport* Viewport,FPrimitiveDrawInterface* PDI);
	UNREALED_API virtual void DrawHUD(FEditorViewportClient* ViewportClient,FViewport* Viewport,const FSceneView* View,FCanvas* Canvas);

	//@{
	virtual bool StartModify()	{ return 0; }
	virtual bool EndModify()	{ return 0; }
	//@}

	//@{
	virtual void StartTrans()	{}
	virtual void EndTrans()		{}
	//@}

	// Tick
	virtual void Tick(FEditorViewportClient* ViewportClient,float DeltaTime) {}

	/** @name Selections */
	//@{
	virtual void SelectNone() {}
	/** @return		true if something was selected/deselected, false otherwise. */
	virtual bool BoxSelect( FBox& InBox, bool InSelect = true )
	{
		return false;
	}
	//@}

	virtual bool FrustumSelect( const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, bool InSelect = true )
	{
		return false;
	}

	/** Returns the tool type. */
	EModeTools GetID() const			{ return ID; }

	/** Returns true if this tool wants to have input filtered through the editor widget. */
	bool UseWidget() const				{ return bUseWidget; }

protected:
	/** Which tool this is. */
	EModeTools ID;

	/** If true, this tool wants to have input filtered through the editor widget. */
	bool bUseWidget;
};



