// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/GCObject.h"
#include "Containers/ArrayView.h"

class FReimportHandler;

namespace UE::Interchange
{
	class FImportResult;
	using FAssetImportResultRef = TSharedRef< FImportResult, ESPMode::ThreadSafe >;
}


/** Reimport manager for package resources with associated source files on disk. */
class FReimportManager : FGCObject
{
public:
	/**
	 * Singleton function, provides access to the only instance of the class
	 *
	 * @return	Singleton instance of the manager
	 */
	UNREALED_API static FReimportManager* Instance();

	/**
	 * Register a reimport handler with the manager
	 *
	 * @param	InHandler	Handler to register with the manager
	 */
	UNREALED_API void RegisterHandler( FReimportHandler& InHandler );

	/**
	 * Unregister a reimport handler from the manager
	 *
	 * @param	InHandler	Handler to unregister from the manager
	 */
	UNREALED_API void UnregisterHandler( FReimportHandler& InHandler );

	/**
	 * Check to see if we have a handler to manage the reimporting of the object
	 *
	 * @param	Obj	Object we want to reimport
	 *
	 * @return	true if the object is capable of reimportatio
	 */
	UNREALED_API virtual bool CanReimport( UObject* Obj, TArray<FString> *ReimportSourceFilenames = nullptr) const;

	/**
	 * Attempt to reimport the specified object from its source by giving registered reimport
	 * handlers a chance to try to reimport the object
	 *
	 * @param	Obj	Object to try reimporting
	 * @param	bAskForNewFileIfMissing If the file is missing, open a dialog to ask for a new one
	 * @param	bShowNotification True to show a notification when complete, false otherwise
	 * @param	PreferredReimportFile if not empty, will be use in case the original file is missing and bAskForNewFileIfMissing is set to false
	 * @param	SourceFileIndex		which source file index you want to reimport default is INDEX_NONE(the factory will choose)
	 * @param	bAutomated		True to skip dialog prompts
	 *
	 * @return	true if the object was handled by one of the reimport handlers; false otherwise
	 */
	UNREALED_API virtual bool Reimport( UObject* Obj, bool bAskForNewFileIfMissing = false, bool bShowNotification = true, FString PreferredReimportFile = TEXT(""), FReimportHandler* SpecifiedReimportHandler = nullptr, int32 SourceFileIndex = INDEX_NONE, bool bForceNewFile = false, bool bAutomated = false, bool bInForceShowDialog = false);

	/**
	 * Attempt to reimport the specified object from its source by giving registered reimport
	 * handlers a chance to try to reimport the object
	 *
	 * @param	Obj	Object to try reimporting
	 * @param	bAskForNewFileIfMissing If the file is missing, open a dialog to ask for a new one
	 * @param	bShowNotification True to show a notification when complete, false otherwise
	 * @param	PreferredReimportFile if not empty, will be use in case the original file is missing and bAskForNewFileIfMissing is set to false
	 * @param	SourceFileIndex		which source file index you want to reimport default is INDEX_NONE(the factory will choose)
	 * @param	bAutomated		True to skip dialog prompts
	 *
	 * @return	FAssetImportResultRef
	 */
	UNREALED_API virtual UE::Interchange::FAssetImportResultRef ReimportAsync(UObject* Obj, bool bAskForNewFileIfMissing = false, bool bShowNotification = true, FString PreferredReimportFile = TEXT(""), FReimportHandler* SpecifiedReimportHandler = nullptr, int32 SourceFileIndex = INDEX_NONE, bool bInForceNewFile = false, bool bAutomated = false, bool bForceShowDialog = false);

	/**
	 * Attemp to reimport all specified objects. This function will verify that all source file exist and ask the user
	 * to decide on how the system will handle asset with missing source file path.
	 * Choice are:
	 * 1. Browse every missing source file path before starting the import process
	 * 2. Skip all asset that have a missing source file path
	 * 3. Cancel the whole reimport command
	 *
	 * * @param	ToImportObjects		Objects to try reimporting
	 * * @param	bShowNotification	True to show a notification when complete, false otherwise
	 * * @param	SourceFileIndex		which source file index you want to reimport default is INDEX_NONE(the factory will chooseP)
	 * * @param	bAutomated			True to skip dialog prompts
	 */
	UNREALED_API virtual void ValidateAllSourceFileAndReimport(TArray<UObject*> &ToImportObjects, bool bShowNotification = true, int32 SourceFileIndex = INDEX_NONE, bool bForceNewFile = false, bool bAutomated = false);

	/**
	* Attempt to reimport multiple objects from its source by giving registered reimport
	* handlers a chance to try to reimport the object
	*
	* @param	Objects	Objects to try reimporting
	* @param	bAskForNewFileIfMissing If the file is missing, open a dialog to ask for a new one
	* @param	bShowNotification True to show a notification when complete, false otherwise
	* @param	PreferredReimportFile if not empty, will be use in case the original file is missing and bAskForNewFileIfMissing is set to false
	* @param	SourceFileIndex which source file index you want to reimport default is INDEX_NONE(the factory will choose)
	* @param	bAutomated	True to skip dialog prompts
	*
	* @return	true if the objects all imported successfully, for more granular success reporting use FReimportManager::Reimport
	*/
	UNREALED_API virtual bool ReimportMultiple( TArrayView<UObject*> Objects, bool bAskForNewFileIfMissing = false, bool bShowNotification = true, FString PreferredReimportFile = TEXT(""), FReimportHandler* SpecifiedReimportHandler = nullptr, int32 SourceFileIndex = INDEX_NONE, bool bForceNewFile = false, bool bAutomated = false);

	/**
	 * Update the reimport paths for the specified object
	 *
	 * @param	Obj	Object to update
	 * @param	InFilenames The files we want to set to its import paths
	 */
	UNREALED_API virtual void UpdateReimportPaths( UObject* Obj, const TArray<FString>& InFilenames );

	/**
	* Update the reimport paths for the specified object
	*
	* @param	Obj	Object to update
	* @param	Filename The files we want to set to its import paths
	* @param	SourceFileIndex the source file index to set the filename
	*/
	UNREALED_API virtual void UpdateReimportPath(UObject* Obj, const FString& Filename, int32 SourceFileIndex);

	/**
	 * Gets the delegate that's fired prior to reimporting an asset
	 *
	 * @return The event delegate.
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FPreReimportNotification, UObject*);
	FPreReimportNotification& OnPreReimport(){ return PreReimport; }

	/**
	 * Gets the delegate that's fired after to reimporting an asset
	 *
	 * @param Whether the reimport was a success
	 * @return The event delegate.
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FPostReimportNotification, UObject*, bool);
	FPostReimportNotification& OnPostReimport(){ return PostReimport; }

	/** Opens a file dialog to request a new reimport path */
	UNREALED_API void GetNewReimportPath(UObject* Obj, TArray<FString>& InOutFilenames, int32 SourceFileIndex = INDEX_NONE);
	
	/** FGCObject interface */
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FReimportManager");
	}

private:
	/** Sort Reimport handlers by priority if they are unsorted */
	void SortHandlersIfNeeded();

	void OnInterchangePostReimported(UObject* ReimportAsset) const;

	/** Reimport handlers registered with this manager */
	TArray<FReimportHandler*> Handlers;

	/** True when the Handlers array has been modified such that it needs sorting */
	bool bHandlersNeedSorting;

	/** Delegate to call before the asset is reimported */
	FPreReimportNotification PreReimport;

	/** Delegate to call after the asset is reimported */
	FPostReimportNotification PostReimport;

	FDelegateHandle InterchangePostReimportedDelegateHandle;

	/** Constructor */
	FReimportManager();

	/** Destructor */
	~FReimportManager();

	/** Copy constructor; intentionally left unimplemented */
	FReimportManager( const FReimportManager& );

	/** Assignment operator; intentionally left unimplemented */
	FReimportManager& operator=( const FReimportManager& );
};

/** 
* The various results we can receive from an object re-import
*/
namespace EReimportResult
{
	enum Type
	{
		Failed,
		Succeeded,
		Cancelled
	};
}

/** 
* Reimport handler for package resources with associated source files on disk.
*/
class FReimportHandler
{
public:
	/** Constructor. Add self to manager */
	FReimportHandler() : bAutomatedReimport(false) { FReimportManager::Instance()->RegisterHandler( *this ); }
	/** Destructor. Remove self from manager */
	virtual ~FReimportHandler(){ FReimportManager::Instance()->UnregisterHandler( *this ); }
	
	/**
	 * Check to see if the handler is capable of reimporting the object
	 *
	 * @param	Obj	Object to attempt to reimport
	 * @param	OutFilenames	The filename(s) of the source art for the specified object
	 *
	 * @return	true if this handler is capable of reimporting the provided object
	 */
	virtual bool CanReimport( UObject* Obj, TArray<FString>& OutFilenames ) = 0;

	/**
	 * Sets the reimport path(s) for the specified object
	 *
	 * @param	Obj	Object for which to change the reimport path(s)
	 * @param	NewReimportPaths	The new path(s) to set on the object
	 */
	virtual void SetReimportPaths( UObject* Obj, const TArray<FString>& NewReimportPaths ) = 0;

	/**
	* Sets the reimport path(s) for the specified object at the specified index
	*
	* @param	Obj	Object for which to change the reimport path(s)
	* @param	NewReimportPaths	The new path(s) to set on the object
	* @param	SourceIndex			the index of the SourceFile to set the reimport path
	*/
	virtual void SetReimportPaths(UObject* Obj, const FString& NewReimportPath, const int32 SourceIndex)
	{
		TArray<FString> NewReimportPaths;
		NewReimportPaths.Add(NewReimportPath);
		SetReimportPaths(Obj, NewReimportPaths);
	}

	/**
	* If we want to import a specific source index and there is some settings to update in the asset import data before starting the re-import.
	*
	* @param	Obj - Object for which to change re-import settings.
	* @param	SourceIndex - the index of the SourceFile that will be re-imported
	*/
	virtual void SetReimportSourceIndex(UObject* Obj, const int32 SourceIndex)
	{
	}

	/**
	 * Attempt to reimport the specified object from its source
	 *
	 * @param	Obj	Object to attempt to reimport
	 *
	 * @return	EReimportResult::Succeeded if this handler was able to handle reimporting the provided object,
	 *			EReimportResult::Failed if this handler was unable to handle reimporting the provided object or
	 *			EReimportResult::Cancelled if the handler was cancelled part-way through re-importing the provided object.
	 */
	virtual EReimportResult::Type Reimport( UObject* Obj ) = 0;

	/**
	* Attempt to reimport the specified object from its source
	*
	* @param	Obj	Object to attempt to reimport
	* @param	SourceFileIndex	the index of the SourceFile to use to reimport the Obj
	*
	* @return	EReimportResult::Succeeded if this handler was able to handle reimporting the provided object,
	*			EReimportResult::Failed if this handler was unable to handle reimporting the provided object or
	*			EReimportResult::Cancelled if the handler was cancelled part-way through re-importing the provided object.
	*/
	virtual EReimportResult::Type Reimport(UObject* Obj, int32 SourceFileIndex)
	{
		return Reimport(Obj);
	}

	/**
	 * Get the import priority for this handler.
	 * Import handlers with higher priority values will take precedent over lower priorities.
	 */
	UNREALED_API virtual int32 GetPriority() const;

	/** Returns the UFactory object associated with this reimport handler */
	virtual TObjectPtr<UObject>* GetFactoryObject() const
	{
		return nullptr;
	}

	/** Call after multiple import happen */
	virtual void PostImportCleanUp() {}

	/**	Sets the preferred reimport path, for aiding in resolving reimporting objects where the
	 *	object path differs from the new path (ex. differing extensions).
	 */
	void SetPreferredReimportPath(const FString& Path) { PreferredReimportPath = Path; }

	/** Sets automated, when True dialog prompts should not appear. */
	virtual void SetAutomatedReimport(const bool InAutomatedReimport)
	{
		bAutomatedReimport = InAutomatedReimport;
	}

	/** When True dialog prompts should not appear. */
	virtual bool IsAutomatedReimport() const
	{
		return bAutomatedReimport;
	}

	virtual bool IsInterchangeFactory() const
	{
		return false;
	}

	/** Sets force show dialog, when True dialog will prompts even if the show dialog setting is false. */
	virtual void SetForceShowDialog(const bool InbForceShowDialog)
	{
		bForceShowDialog = InbForceShowDialog;
	}

	/** When True dialog prompts will prompts even if the show dialog setting is false. */
	virtual bool IsForceShowDialog() const
	{
		return bForceShowDialog;
	}

protected:
	FString PreferredReimportPath;
	bool bAutomatedReimport;
	bool bForceShowDialog = false;
};
