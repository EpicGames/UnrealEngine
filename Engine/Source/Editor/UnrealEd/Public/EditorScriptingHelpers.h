// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"

namespace EditorScriptingHelpers
{
	/*
	 * Check if the editor is in a valid state to run a command.
	 */
	bool UNREALED_API CheckIfInEditorAndPIE();

	/*
	* Check if the Path have a valid root
    */
	bool UNREALED_API HasValidRoot(const FString& ObjectPath);

	/*
	* Check if the Path is a valid ContentBrowser Path
	*/
	bool UNREALED_API IsAValidPath(const FString& Path, const TCHAR* InvalidChar, FString& OutFailureReason);

	/*
	 * Check if the AssetPath can be used to create a new asset
	 */
	bool UNREALED_API IsAValidPathForCreateNewAsset(const FString& ObjectPath, FString& OutFailureReason);

	/*
	 * From "AssetClass'/Game/Folder/MyAsset.MyAsset', "AssetClass /Game/Folder/MyAsset.MyAsset, "/Game/Folder/MyAsset.MyAsset", "/Game/Folder/", "/Game/Folder" "/Game/Folder/MyAsset.MyAsset:InnerAsset.2ndInnerAsset"
	 * and convert to "/Game/Folder"
	 */
	FString UNREALED_API ConvertAnyPathToLongPackagePath(const FString& AnyPath, FString& OutFailureReason);

	/*
	 * From "AssetClass'/Game/Folder/Package.Asset'", "AssetClass /Game/Folder/Package.Asset", "/Game/Folder/Package.Asset", "/Game/Folder/MyAsset" "/Game/Folder/Package.Asset:InnerAsset.2ndInnerAsset"
	 * and convert to "/Game/Folder/Package.Asset"
	 * @note: Object name is inferred from package name when missing
	 */
	FString UNREALED_API ConvertAnyPathToObjectPath(const FString& AssetPath, FString& OutFailureReason);

	/*
	 * From "AssetClass'/Game/Folder/Package.Asset'", "AssetClass /Game/Folder/Package.Asset", "/Game/Folder/Package.Asset", "/Game/Folder/MyAsset" "/Game/Folder/Package.Asset:InnerAsset.2ndInnerAsset"
	 * and convert to "/Game/Folder/Package.Asset" or "/Game/Folder/Package.Asset:InnerAsset"
	 * @note: Object name is inferred from package name when missing
	 */
	FString UNREALED_API ConvertAnyPathToSubObjectPath(const FString& AssetPath, FString& OutFailureReason);
}