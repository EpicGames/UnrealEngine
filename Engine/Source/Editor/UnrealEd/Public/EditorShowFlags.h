// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/InputChord.h"
#include "ShowFlags.h"

class FLevelEditorViewportClient;

/**
 * @todo Slate: Copied from wxLevelViewportToolBar.  We need both systems to work for now.                   
 */
class FShowFlagData
{
public:

	/** Name for identifying the show flag */
	FName			ShowFlagName; 
	/** The display name of this show flag */
	FText			DisplayName;
	/** The engine show flag index in FEngineShowFlags */
	uint32			EngineShowFlagIndex;
	/** Which group the flags should show up */
	EShowFlagGroup	Group;

	/** Chord for activating the show flag */
	FInputChord InputChord;

	/**
	 *  FShowFlagData constructor with no chord associated
	 *
	 * @param InName				The non-localized flag name string
	 * @param InDisplayName			The localized display name for the flag
	 * @param InEngineShowFlagIndex	Index on this flag in the array of flags in FEngineShowFlags
	 * @param InGroup				The group that the flag belongs to (default: SFG_Normal)
	 */
	FShowFlagData(const FString& InName, const FText& InDisplayName, const uint32 InEngineShowFlagIndex, EShowFlagGroup InGroup = SFG_Normal);

	/**
	 *  FShowFlagData constructor with no chord associated
	 *
	 * @param InName				The non-localized flag name string
	 * @param InDisplayName			The localized display name for the flag
	 * @param InEngineShowFlagIndex	Index on this flag in the array of flags in FEngineShowFlags
	 * @param InGroup				The group that the flag belongs to (default: SFG_Normal)
	 * @param InInputChord			InputChord to be used for the flag
	 */
	FShowFlagData(const FString& InName, const FText& InDisplayName, const uint32 InEngineShowFlagIndex, EShowFlagGroup InGroup, const FInputChord& InInputChord);

	bool IsEnabled(const FLevelEditorViewportClient* ViewportClient) const;

	void ToggleState(FLevelEditorViewportClient* ViewportClient) const;
};

UNREALED_API TArray<FShowFlagData>& GetShowFlagMenuItems();
