// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorState/WorldEditorState.h"
#include "ActorEditorContextEditorState.generated.h"

class UActorEditorContextStateCollection;

UCLASS(MinimalAPI)
class UActorEditorContextEditorState : public UWorldDependantEditorState
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UEditorState Interface
	virtual FText GetCategoryText() const override;

protected:
	virtual FOperationResult CaptureState() override;
	virtual FOperationResult RestoreState() const override;
	//~ End UEditorState Interface

	// Editor context state object, will be used to restore context when loading the bookmark.
	UPROPERTY(VisibleAnywhere, Category = ActorEditorContext, Instanced, meta=(ShowOnlyInnerProperties, DisplayName="Context"))
	TObjectPtr<UActorEditorContextStateCollection> ActorEditorContextStateCollection;

	UPROPERTY(EditAnywhere, Category = ActorEditorContext, meta=(DisplayName="Apply Context on Load"))
	bool bApplyActorEditorContextOnLoad;
};
