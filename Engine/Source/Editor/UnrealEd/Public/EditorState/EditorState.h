// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "EditorState.generated.h"

/**
 * EditorState is a container that can capture the state of a given editor subsystem and restore it at a later time.
 * To capture/restore states, you must use the UEditorStateSubsystem subsystem rather than dealing with this class directly.
 * @see UEditorStateSubsystem
 */
UCLASS(MinimalAPI, Abstract)
class UEditorState : public UObject
{
	GENERATED_BODY()

public:
	struct FOperationResult
	{
		enum EResult
		{
			Success,
			Skipped,
			Warning,
			Failure
		};

		explicit FOperationResult(EResult InResult, const FText& InResultText = FText())
			: Result(InResult)
			, ResultText(InResultText)
		{
		}

		bool operator == (EResult InResult) const
		{
			return Result == InResult;
		}

		bool operator != (EResult InResult) const
		{
			return Result == InResult;
		}

		EResult GetResult() const
		{
			return Result;
		}

		const FText& GetResultText() const
		{
			return ResultText;
		}

	private:
		EResult	 Result;
		FText	 ResultText;
	};

public:
	/** Get the category under which this state's properties should be displayed. */
	virtual FText GetCategoryText() const PURE_VIRTUAL(UEditorState::GetCategoryText, return FText::GetEmpty(); );
	
	/** Get a list of state types this editor state depends on. Dependent types will be restored after their dependencies. */
	virtual TArray<TSubclassOf<UEditorState>> GetDependencies() const { return {}; }

protected:
	/** Capture the state of the editor. Must be subclassed. */
	virtual FOperationResult CaptureState() PURE_VIRTUAL(UEditorState::CaptureState, return FOperationResult(FOperationResult::Failure); );

	/** Restore the state of the editor. Must be subclassed. */
	virtual FOperationResult RestoreState() const PURE_VIRTUAL(UEditorState::RestoreState, return FOperationResult(FOperationResult::Failure); );

private:
	/** Prevent access to UObject::GetWorld() as it's not relevant for this class and is error prone. */
	virtual class UWorld* GetWorld() const override final 
	{
		return nullptr;
	}

	friend class UEditorStateSubsystem;
};
