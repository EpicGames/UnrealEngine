// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorState/EditorState.h"
#include "Templates/EnableIf.h"
#include "UObject/GCObject.h"
#include "EditorStateCollection.generated.h"

/**
 * An immutable collection of editor state objects that can be queried and iterated upon.
  */
USTRUCT()
struct FEditorStateCollection
{
	GENERATED_BODY()

	template <typename TEditorStateType>
	typename TEnableIf<TIsDerivedFrom<TEditorStateType, UEditorState>::IsDerived, bool>::Type HasState() const
	{
		return GetState(TEditorStateType::StaticClass()) != nullptr;
	}

	template <typename TEditorStateType>
	typename TEnableIf<TIsDerivedFrom<TEditorStateType, UEditorState>::IsDerived, const TEditorStateType*>::Type GetState() const
	{
		const UEditorState* EditorState = GetState(TEditorStateType::StaticClass());
		return EditorState ? CastChecked<TEditorStateType>(EditorState) : nullptr;
	}

	template <typename TEditorStateType>
	typename TEnableIf<TIsDerivedFrom<TEditorStateType, UEditorState>::IsDerived, const TEditorStateType&>::Type GetStateChecked() const
	{
		const UEditorState* EditorState = GetState(TEditorStateType::StaticClass());
		return *CastChecked<TEditorStateType>(EditorState);
	}

	/** Iterate over each state, making sure iteration is done over dependant states first */
	void ForEachState(TFunctionRef<bool(const UEditorState*, bool)> InFunc, const TArray<TSubclassOf<UEditorState>>& InEditorStateTypeFilter = {}) const
	{
		ForEachState(this, InFunc, InEditorStateTypeFilter);
	}

	/** Iterate over each state, making sure iteration is done over dependant states first */
	void ForEachState(TFunctionRef<bool(UEditorState*, bool)> InFunc, const TArray<TSubclassOf<UEditorState>>& InEditorStateTypeFilter = {})
	{
		ForEachState(this, InFunc, InEditorStateTypeFilter);
	}

	bool HasStates() const
	{
		return !States.IsEmpty();
	}

	TArray<UEditorState*> GetStates() const
	{
		return States;
	}

	void PostSerialize(const FArchive& Ar)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		for (const TPair<TSubclassOf<UEditorState>, TObjectPtr<UEditorState>>& KeyValue : EditorStates_DEPRECATED)
		{
			States.Emplace(KeyValue.Value);
		}
		EditorStates_DEPRECATED.Reset();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

private:
	template <typename ThisType, typename FuncType>
	static void ForEachState(ThisType* Self, FuncType InFunc, const TArray<TSubclassOf<UEditorState>>& InEditorStateTypeFilter = {})
	{
		// Keep track of processed states to avoid cycles
		TMap<TSubclassOf<UEditorState>, bool> ProcessedStates;

		const auto ProcessState = [&ProcessedStates, &InFunc, Self](UEditorState* StateToProcess)
		{
			auto ProcessStateImpl = [&ProcessedStates, &InFunc, Self](UEditorState* StateToProcess, auto& ProcessStateRef) -> bool
			{
				// Avoid processing dependencies twice
				bool* bProcessedStateResult = ProcessedStates.Find(StateToProcess->GetClass());
				if (!bProcessedStateResult)
				{
					bool bProcessedDependenciesSuccessfully = true;

					// Process the dependencies - Stop if any error is encountered
					for (TSubclassOf<UEditorState> DependencyType : StateToProcess->GetDependencies())
					{
						UEditorState* DependencyState = Self->GetState(DependencyType);
						bProcessedDependenciesSuccessfully = DependencyState && ProcessStateRef(DependencyState, ProcessStateRef);

						if (!bProcessedDependenciesSuccessfully)
						{
							break;
						}
					}

					// Process the state itself
					bool bSuccess = InFunc(StateToProcess, bProcessedDependenciesSuccessfully);
					ProcessedStates.Add(StateToProcess->GetClass(), bSuccess);
					return bSuccess;
				}
				else
				{
					return *bProcessedStateResult;
				}
			};

			return ProcessStateImpl(StateToProcess, ProcessStateImpl);
		};

		for (const TObjectPtr<UEditorState>& State : Self->States)
		{
			if (!InEditorStateTypeFilter.IsEmpty() && !InEditorStateTypeFilter.Contains(State->GetClass()))
			{
				continue;
			}

			ProcessState(State.Get());
		}
	}

	UEditorState* GetState(TSubclassOf<UEditorState> InStateType) const
	{
		const TObjectPtr<UEditorState>* Result = States.FindByPredicate([&InStateType](const TObjectPtr<UEditorState> InEditorState)
		{
			return InEditorState && InEditorState->IsA(InStateType);
		});
		return Result ? *Result : nullptr;
	}

	TObjectPtr<UEditorState>& GetStateChecked(TSubclassOf<UEditorState> InStateType)
	{
		TObjectPtr<UEditorState>* Result = States.FindByPredicate([&InStateType](TObjectPtr<UEditorState> InEditorState)
		{
			return InEditorState && InEditorState->IsA(InStateType);
		});
		check(Result);
		return *Result;
	}

	UE_DEPRECATED(5.6, "Use the States array")
	UPROPERTY()
	TMap<TSubclassOf<UEditorState>, TObjectPtr<UEditorState>> EditorStates_DEPRECATED;

	UPROPERTY(Instanced)
	TArray<TObjectPtr<UEditorState>> States;
	
	friend class UEditorStateSubsystem;
};

// Add traits to let the engine know we have a PostSerialize() method on our struct
template<>
struct TStructOpsTypeTraits<FEditorStateCollection> : public TStructOpsTypeTraitsBase2<FEditorStateCollection>
{
	enum
	{
		WithPostSerialize = true,
		WithCopy = false
	};
};

// Wrapper to avoid GC of a standalone FEditorStateCollection (as it contains UObjects)
class FEditorStateCollectionGCObject : public FGCObject
{
public:
	FEditorStateCollection EditorStateCollection;

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddPropertyReferences(FEditorStateCollection::StaticStruct(), &EditorStateCollection);
	}

	virtual FString GetReferencerName() const override
	{
		return TEXT("FEditorStateCollection");
	}
};
