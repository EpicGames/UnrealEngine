// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "EditorState/EditorState.h"
#include "EditorState/EditorStateCollection.h"
#include "Templates/SubclassOf.h"
#include "EditorStateSubsystem.generated.h"

/**
 * Subsystem that allows the capture & restoration of editor states.
 */
UCLASS(MinimalAPI)
class UEditorStateSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	/** Utility method to retrieve the subsystem. Editor must have been initialized before this can be used. */
	static UNREALED_API UEditorStateSubsystem* Get();

	UNREALED_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UNREALED_API virtual void Deinitialize() override;

	/** Register an EditorState type so that it can be captured in the next CaptureEditorState() calls. */
	template <typename TEditorStateType>
	static typename TEnableIf<TIsDerivedFrom<TEditorStateType, UEditorState>::IsDerived, void>::Type RegisterEditorStateType()
	{
		UEditorStateSubsystem::Get()->RegisterEditorStateType(TEditorStateType::StaticClass());
	}

	/** Unregister a previously registed EditorState type. */
	template <typename TEditorStateType>
	static typename TEnableIf<TIsDerivedFrom<TEditorStateType, UEditorState>::IsDerived, void>::Type UnregisterEditorStateType()
	{
		UEditorStateSubsystem::Get()->UnregisterEditorStateType(TEditorStateType::StaticClass());
	}
	
	/**
	 * Captures the state of the editor using all the registered EditorState subclasses.
	 * @param OutState		Collection of states that will be populated with state data.
	 * @param InStateOuter	Outer to use for the UEditorState objects that may be created by this method.
	 */
	UNREALED_API void CaptureEditorState(FEditorStateCollection& OutState, UObject* InStateOuter);

	/**
	 * Captures the state of the editor using all the registered EditorState subclasses.
	 * @param OutState		Collection of states that will be populated with state data.
	 * @param InStateOuter	Outer to use for the UEditorState objects that may be created by this method.
	 */
	UNREALED_API void CaptureEditorState(FEditorStateCollection& OutState, const TArray<TSubclassOf<UEditorState>>& InEditorStateTypeFilter, UObject* InStateOuter);

	/**
	 * Restore the state of the editor using a state collection previously captured with CaptureEditorState().
	 * @param InState		Collection of states to restore.
	 */
	UNREALED_API void RestoreEditorState(const FEditorStateCollection& InState);

	/**
	 * Restore the state of the editor using a state collection previously captured with CaptureEditorState().
	 * @param InState		Collection of states to restore.
	 */
	UNREALED_API void RestoreEditorState(const FEditorStateCollection& InState, const TArray<TSubclassOf<UEditorState>>& InEditorStateTypeFilter);

	/**
	 * Returns true if the subsystem is currently restoring an editor state.
	 */
	UNREALED_API bool IsRestoringEditorState() const;
		
private:
	UNREALED_API void RegisterEditorStateType(TSubclassOf<UEditorState> InEditorStateType);
	UNREALED_API void UnregisterEditorStateType(TSubclassOf<UEditorState> InEditorStateType);

	void OutputOperationResult(const UEditorState* InState, bool bRestore, const UEditorState::FOperationResult& InOperationResult) const;

	TArray<TSubclassOf<UEditorState>> RegisteredEditorStateTypes;

	bool bIsRestoringEditorState = false;
};
