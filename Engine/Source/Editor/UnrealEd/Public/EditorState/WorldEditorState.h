// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorState/EditorState.h"
#include "WorldEditorState.generated.h"

class UWorld;

UCLASS(MinimalAPI)
class UWorldEditorState : public UEditorState
{
	GENERATED_UCLASS_BODY()

	UNREALED_API TSoftObjectPtr<UWorld> GetStateWorld() const;

	//~ Begin UEditorState Interface
	virtual FText GetCategoryText() const override;
private:
	virtual FOperationResult CaptureState() override;
	virtual FOperationResult RestoreState() const override;
	//~ End UEditorState Interface

	// World associated with the bookmark
	UPROPERTY(VisibleAnywhere, Category = General)
	TSoftObjectPtr<UWorld> World;
};

UCLASS(MinimalAPI, Abstract)
class UWorldDependantEditorState : public UEditorState
{
	GENERATED_UCLASS_BODY()

public:
	UNREALED_API UWorld* GetStateWorld() const;

protected:
	//~ Begin UEditorState Interface
	UNREALED_API virtual TArray<TSubclassOf<UEditorState>> GetDependencies() const override;
	//~ End UEditorState Interface
};
