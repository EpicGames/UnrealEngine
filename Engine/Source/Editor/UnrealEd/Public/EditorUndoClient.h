// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	EditorUndoClient.h: Declares the FEditorUndoClient interface.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"

struct FTransactionContext;
class FTransactionObjectEvent;

/**
 * Interface for tools wanting to handle undo/redo operations
 */
class FEditorUndoClient
{
public:
	/** Always unregister for undo on destruction, just in case. */
	UNREALED_API virtual ~FEditorUndoClient();

	/**
	 * Called to see if the context of the current undo/redo operation is a match for the client
	 * Default state matching old context-less undo is Context="" and PrimaryObject=NULL
	 *
	 * @param InContext					The transaction context
	 * @param TransactionObjectContexts	The transaction context of each object involved in this transaction
	 * 
	 * @return	True if client wishes to handle the undo/redo operation for this context. False otherwise
	 */
	virtual bool MatchesContext( const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts ) const { return true; }

	/** 
	 * Signal that client should run any PostUndo code
	 * @param bSuccess	If true than undo succeeded, false if undo failed
	 */
	virtual void PostUndo( bool bSuccess ) {};
	
	/** 
	 * Signal that client should run any PostRedo code
	 * @param bSuccess	If true than redo succeeded, false if redo failed
	 */
	virtual void PostRedo( bool bSuccess )	{};

	/** Return the transaction context for this client */
	virtual FString GetTransactionContext() const { return FString(); }
};

/** An undo client that registers itself in its constructor and unregisters itself in its destructor */
class FSelfRegisteringEditorUndoClient : public FEditorUndoClient
{
public:
	/** Register in constructor */
	UNREALED_API FSelfRegisteringEditorUndoClient();

	/** FEditorUndoClient already unregisters in its destructor */
};
