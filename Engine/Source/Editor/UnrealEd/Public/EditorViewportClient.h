// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Engine/EngineBaseTypes.h"
#include "Animation/CurveSequence.h"
#include "UObject/GCObject.h"
#include "Editor/UnrealEdTypes.h"
#include "SceneTypes.h"
#include "Engine/Scene.h"
#include "Camera/CameraTypes.h"
#include "UnrealWidgetFwd.h"
#include "ShowFlags.h"
#include "UnrealClient.h"
#include "SceneManagement.h"
#include "EditorComponents.h"
#include "Framework/Commands/Commands.h"
#include "Editor.h"
#include "ViewportClient.h"
#include "Misc/TransactionObjectEvent.h"
#include "Settings/EditorViewportSettings.h"

struct FAssetData;
class FCachedJoystickState;
class FCameraControllerConfig;
class FCameraControllerUserImpulseData;
class FCanvas;
class FDragTool;
class FEditorCameraController;
class FEditorModeTools;
class FEditorViewportClient;
class FEdMode;
class FMouseDeltaTracker;
class FPreviewScene;
struct FTypedElementHandle;
class IAssetFactoryInterface;
class SEditorViewport;
class UActorFactory;
class UTypedElementViewportInteraction;
enum class EViewStatusForScreenPercentage;
enum class EColorChannelMask : int8;
struct FGizmoState;
struct FViewportClientNavigationHelper;
namespace UE::RenderViewportFeedback
{
	class FReceiver;
}

/** Delegate called by FEditorViewportClient to check its visibility */
DECLARE_DELEGATE_RetVal( bool, FViewportStateGetter );

DECLARE_LOG_CATEGORY_EXTERN(LogEditorViewport, Log, All);

namespace EDragTool
{
	enum Type
	{
		BoxSelect,
		FrustumSelect,
		Measure,
		ViewportChange
	};
}

UE_DECLARE_TCOMMANDS(class FViewportNavigationCommands, UNREALED_API)

/**
* Unreal level editor actions
*/
class FViewportNavigationCommands : public TCommands<FViewportNavigationCommands>
{

public:
	UNREALED_API FViewportNavigationCommands();

	/**
	* Initialize commands
	*/
	UNREALED_API virtual void RegisterCommands() override;


	TSharedPtr< FUICommandInfo > Forward;
	TSharedPtr< FUICommandInfo > Backward;
	TSharedPtr< FUICommandInfo > Left;
	TSharedPtr< FUICommandInfo > Right;

	TSharedPtr< FUICommandInfo > WorldUp;
	TSharedPtr< FUICommandInfo > WorldDown;
	TSharedPtr< FUICommandInfo > LocalUp;
	TSharedPtr< FUICommandInfo > LocalDown;

	TSharedPtr< FUICommandInfo > FovZoomIn;
	TSharedPtr< FUICommandInfo > FovZoomOut;

	TSharedPtr< FUICommandInfo > RotateUp;
	TSharedPtr< FUICommandInfo > RotateDown;
	TSharedPtr< FUICommandInfo > RotateLeft;
	TSharedPtr< FUICommandInfo > RotateRight;
};

// FPrioritizedInputChord
//
// Provides a way to resolve conflict between overlapping keyboard commands. Instances of this class 
// are registered with an FEditorViewportClient instance and held in a prioritized list. When input 
// matches one of them the keys referenced by that one are ignored when evaluating all lower priority 
// instances.
//
struct FPrioritizedInputChord
{	
	FPrioritizedInputChord(const int32 InPriority, const FName InName, const EModifierKey::Type InModifierKeyFlags, const FKey InKey = EKeys::Invalid)
	: InputChord(InModifierKeyFlags, InKey)
	, Name(InName)
	, Priority(InPriority)
	{}

	FInputChord InputChord;
	FName Name;
	int32 Priority;
};

struct FInputEventState
{
public:
	FInputEventState( FViewport* InViewport, FKey InKey, EInputEvent InInputEvent )
		: Viewport( InViewport )
		, Key( InKey )
		, InputEvent( InInputEvent )
	{}

	FViewport* GetViewport() const { return Viewport; }
	EInputEvent GetInputEvent() const { return InputEvent; }
	FKey GetKey() const { return Key; }

	/** return true if the event causing button is a control key */
	bool IsCtrlButtonEvent() const { return (Key == EKeys::LeftControl || Key == EKeys::RightControl); }
	bool IsShiftButtonEvent() const { return (Key == EKeys::LeftShift || Key == EKeys::RightShift); }
	bool IsAltButtonEvent() const { return (Key == EKeys::LeftAlt || Key == EKeys::RightAlt); }
	bool IsCommandButtonEvent() const { return (Key == EKeys::LeftCommand || Key == EKeys::RightCommand); }

	bool IsLeftMouseButtonPressed() const { return IsButtonPressed( EKeys::LeftMouseButton ); }
	bool IsMiddleMouseButtonPressed() const { return IsButtonPressed( EKeys::MiddleMouseButton ); }
	bool IsRightMouseButtonPressed() const { return IsButtonPressed( EKeys::RightMouseButton ); }

	bool IsMouseButtonEvent() const { return (Key == EKeys::LeftMouseButton || Key == EKeys::MiddleMouseButton || Key == EKeys::RightMouseButton); } 
	bool IsButtonPressed( FKey InKey ) const { return Viewport->KeyState(InKey); }
	bool IsAnyMouseButtonDown() const { return IsButtonPressed(EKeys::LeftMouseButton) || IsButtonPressed(EKeys::MiddleMouseButton) || IsButtonPressed(EKeys::RightMouseButton); }

	/** return true if alt is pressed right now.  This will be true even if the event was for a different key but an alt key is currently pressed */
	bool IsAltButtonPressed() const { return !( IsAltButtonEvent() && InputEvent == IE_Released ) && ( IsButtonPressed( EKeys::LeftAlt ) || IsButtonPressed( EKeys::RightAlt ) ); }
	bool IsShiftButtonPressed() const { return !( IsShiftButtonEvent() && InputEvent == IE_Released ) && ( IsButtonPressed( EKeys::LeftShift ) || IsButtonPressed( EKeys::RightShift ) ); }
	bool IsCtrlButtonPressed() const { return !( IsCtrlButtonEvent() && InputEvent == IE_Released ) && ( IsButtonPressed( EKeys::LeftControl ) || IsButtonPressed( EKeys::RightControl ) ); }
	bool IsCommandButtonPressed() const { return !(IsCommandButtonEvent() && InputEvent == IE_Released) &&  (IsButtonPressed( EKeys::LeftCommand ) || IsButtonPressed( EKeys::RightCommand ) ); }
	bool IsSpaceBarPressed() const { return IsButtonPressed( EKeys::SpaceBar ); }

private:
	/** Viewport the event was sent to */
	FViewport* Viewport;
	/** Pressed Key */
	FKey Key;
	/** Key event */
	EInputEvent InputEvent;
};


/**
 * Contains information about a mouse cursor position within a viewport, transformed into the correct coordinate
 * system for the viewport.
 */
struct FViewportCursorLocation
{
public:
	UNREALED_API FViewportCursorLocation( const FSceneView* View, FEditorViewportClient* InViewportClient, int32 X, int32 Y, const bool bInClampToBounds = true );
	UNREALED_API virtual ~FViewportCursorLocation();

	const FVector&		GetOrigin()			const	{ return Origin; }
	const FVector&		GetDirection()		const	{ return Direction; }
	const FIntPoint&	GetCursorPos()		const	{ return CursorPos; }
	ELevelViewportType	GetViewportType()	const;
	FEditorViewportClient*	GetViewportClient()	const	{ return ViewportClient; }

private:
	FVector	Origin;
	FVector	Direction;
	FIntPoint CursorPos;
	FEditorViewportClient* ViewportClient;
};

struct FViewportClick : public FViewportCursorLocation
{
public:
	UNREALED_API FViewportClick( const FSceneView* View, FEditorViewportClient* ViewportClient, FKey InKey, EInputEvent InEvent, int32 X, int32 Y );
	UNREALED_API virtual ~FViewportClick();

	/** @return The 2D screenspace cursor position of the mouse when it was clicked. */
	const FIntPoint&	GetClickPos()	const	{ return GetCursorPos(); }
	const FKey&			GetKey()		const	{ return Key; }
	EInputEvent			GetEvent()		const	{ return Event; }

	virtual bool	IsControlDown()	const			{ return ControlDown; }
	virtual bool	IsShiftDown()	const			{ return ShiftDown; }
	virtual bool	IsAltDown()		const			{ return AltDown; }

private:
	FKey		Key;
	EInputEvent	Event;
	bool		ControlDown,
				ShiftDown,
				AltDown;
};


struct FDropQuery
{
	FDropQuery()
		: bCanDrop(false)
	{}

	/** True if it's valid to drop the object at the location queried */
	bool bCanDrop;

	/** Optional hint text that may be returned to the user. */
	FText HintText;
};


/** 
 * Stores the transformation data for the viewport camera
 */
struct FViewportCameraTransform
{
public:
	UNREALED_API FViewportCameraTransform();

	/** Sets the transform's location */
	UNREALED_API void SetLocation( const FVector& Position );

	/** Sets the transform's rotation */
	void SetRotation( const FRotator& Rotation )
	{
		ViewRotation = Rotation;
	}

	/** Sets the location to look at during orbit */
	void SetLookAt( const FVector& InLookAt )
	{
		LookAt = InLookAt;
	}

	/** Set the ortho zoom amount */
	void SetOrthoZoom( float InOrthoZoom )
	{
		ensure(InOrthoZoom >= MIN_ORTHOZOOM && InOrthoZoom <= MAX_ORTHOZOOM);
		OrthoZoom = InOrthoZoom;
	}

	void SetMaxLocation( double InMaxLocation )
	{
		MaxLocation = InMaxLocation;
	}

	/** Check if transition curve is playing. */
	UNREALED_API bool IsPlaying();

	/** @return The transform's location */
	FORCEINLINE const FVector& GetLocation() const { return ViewLocation; }
	
	/** @return The transform's rotation */
	FORCEINLINE const FRotator& GetRotation() const { return ViewRotation; }

	/** @return The look at point for orbiting */
	FORCEINLINE const FVector& GetLookAt() const { return LookAt; }

	/** @return The ortho zoom amount */
	FORCEINLINE float GetOrthoZoom() const { return OrthoZoom; }

	/**
	 * Animates from the current location to the desired location
	 *
	 * @param InDesiredLocation	The location to transition to
	 * @param bInstant			If the desired location should be set instantly rather than transitioned to over time
	 */
	UNREALED_API void TransitionToLocation( const FVector& InDesiredLocation, TWeakPtr<SWidget> EditorViewportWidget, bool bInstant );

	/**
	 * Updates any current location transitions
	 *
	 * @return true if there is currently a transition
	 */
	UNREALED_API bool UpdateTransition();

	/**
	 * Computes a matrix to use for viewport location and rotation when orbiting
	 */
	UNREALED_API FMatrix ComputeOrbitMatrix() const;

private:
	/** The time when a transition to the desired location began */
	//double TransitionStartTime;

	/** Curve for animating between locations */
	TSharedPtr<struct FCurveSequence> TransitionCurve;
	/** Current viewport Position. */
	FVector	ViewLocation;
	/** Current Viewport orientation; valid only for perspective projections. */
	FRotator ViewRotation;
	/** Desired viewport location when animating between two locations */
	FVector	DesiredLocation;
	/** When orbiting, the point we are looking at */
	FVector LookAt;
	/** Viewport start location when animating to another location */
	FVector StartLocation;
	/** Ortho zoom amount */
	float OrthoZoom;
	/** Location is clamped to a box around the origin with this radius */
	double MaxLocation;
};

/** Parameter struct for editor viewport view modifiers */
struct FEditorViewportViewModifierParams
{
	FEditorViewportClient* ViewportClient = nullptr;

	FMinimalViewInfo ViewInfo;

	void AddPostProcessBlend(const FPostProcessSettings& Settings, float Weight)
	{
		check(PostProcessSettings.Num() == PostProcessBlendWeights.Num());
		PostProcessSettings.Add(Settings);
		PostProcessBlendWeights.Add(Weight);
	}

private:
	TArray<FPostProcessSettings> PostProcessSettings;
	TArray<float> PostProcessBlendWeights;

	friend class FEditorViewportClient;
};

/** Delegate for modifying view parameters of an editor viewport. */
DECLARE_MULTICAST_DELEGATE_OneParam(FEditorViewportViewModifierDelegate, FEditorViewportViewModifierParams&);

/** Viewport client for editor viewports. Contains common functionality for camera movement, rendering debug information, etc. */
class FEditorViewportClient : public FCommonViewportClient, public FViewElementDrawer, public FGCObject
{
public:
	friend class FMouseDeltaTracker;

	UNREALED_API FEditorViewportClient(FEditorModeTools* InModeTools, FPreviewScene* InPreviewScene = nullptr, const TWeakPtr<SEditorViewport>& InEditorViewportWidget = nullptr);

	UNREALED_API virtual ~FEditorViewportClient();

	/** Non-copyable */
	FEditorViewportClient(const FEditorViewportClient&) = delete;
	FEditorViewportClient& operator=(const FEditorViewportClient&) = delete;

	/**
	 * Retrieves the FPreviewScene used by this instance of FEditorViewportClient.
	 *
	 * @return		The internal FPreviewScene pointer.
	 */
	FPreviewScene* GetPreviewScene()
	{
		return PreviewScene;
	}

	/**
	 * Overrides the realtime state of this viewport until RemoveViewportsRealtimeOverride is called.
	 * The state of this override is not preserved between editor sessions. 
	 *
	 * @param bShouldBeRealtime	If true, this viewport will be realtime, if false this viewport will not be realtime
	 * @param SystemDisplayName	This display name of whatever system is overriding realtime. This name is displayed to users in the viewport options menu	
	 */
	UNREALED_API void AddRealtimeOverride(bool bShouldBeRealtime, FText SystemDisplayName);

	/**
	 * Returns whether there's a realtime override registered with the given system name.
	 */
	UNREALED_API bool HasRealtimeOverride(FText SystemDisplayName) const;

	/**
	 * Removes the most recent realtime override registered with the given system name.
	 *
	 * @param SystemDisplayName	The display name of the override to be removed.
	 * @param bCheckMissingOverride Whether to assert if no matching realtime override was found. Should be `false` if users can operate within the viewport between enable/disable calls as they can remove overrides.
	 * @return Whether any matching realtime override was found.
	 */
	UNREALED_API bool RemoveRealtimeOverride(FText SystemDisplayName, bool bCheckMissingOverride = true);

	/**
	 * Removes the last added realtime override.
	 * WARNING: it's recommended to use RemoveRealtimeOverride to prevent removing an override that belongs to another systems than yours.
	 *
	 * @return   Whether there was any realtime override to be popped.
	 */
	UNREALED_API bool PopRealtimeOverride();

	/**
	 * Toggles whether or not the viewport updates in realtime and returns the updated state.
	 * Note: This value is saved between editor sessions so it should not be used for temporary states.  For that see SetRealtimeOverride
	 *
	 * @return		The current state of the realtime flag.
	 */
	UNREALED_API bool ToggleRealtime();

	/** 
	 * Sets whether or not the viewport updates in realtime.
	 * Note: This value is saved between editor sessions so it should not be used for temporary states.  For that see SetRealtimeOverride
	 */
	UNREALED_API void SetRealtime(bool bInRealtime);

	/** @return		True if viewport is in realtime mode, false otherwise. */
	bool IsRealtime() const
	{ 
		return (RealtimeOverrides.Num() > 0 ? RealtimeOverrides.Last().bIsRealtime : bIsRealtime) || GFrameCounter < RealTimeUntilFrameNumber;
	}

	/**
	 * Get the number of real-time frames to draw (overrides bRealtime)
	 * @note When non-zero, the viewport will render RealTimeFrameCount frames in real-time mode, then revert back to bIsRealtime
	 * this can be used to ensure that not only the viewport renders a frame, but also that the world ticks
	 * @param NumExtraFrames 		The number of extra real time frames to draw
	 */
	virtual void RequestRealTimeFrames(uint64 NumRealTimeFrames = 1)
	{
		RealTimeUntilFrameNumber = FMath::Max(GFrameCounter + NumRealTimeFrames, RealTimeUntilFrameNumber);
	}

	/**
	 * Saves the realtime state to a config location.  Does not save any temp overrides
	 */ 
	UNREALED_API void SaveRealtimeStateToConfig(bool& ConfigVar) const;

	/**
	 * @return true if there are any temp realtime overrides set
	 */
	bool IsRealtimeOverrideSet() const { return RealtimeOverrides.Num() > 0; }
	
	/**
	 * @return true if the underlying realtime mode and the last override (if any) match the given value.
	 */
	UNREALED_API bool DoRealtimeAndOverridesMatch(bool bInValue) const; 
	
	/**
	 * @return  If an override is set this returns the message indicating what set it
	 */
	UNREALED_API FText GetRealtimeOverrideMessage() const;

	UE_DEPRECATED(4.25, "SetRealtime no longer takes in bStoreCurrentValue parameter. For temporary overrides use AddRealtimeOverride")
	UNREALED_API void SetRealtime(bool bInRealtime, bool bStoreCurrentValue);

	/**
	 * Restores realtime setting to stored value. This will only enable realtime and 
	 * never disable it (unless bAllowDisable is true)
	 */
	UE_DEPRECATED(4.25, "To save and restore realtime state non-permanently use AddRealtimeOverride and RemoveRealtimeOverride")
	UNREALED_API void RestoreRealtime(const bool bAllowDisable = false);

	// this set ups camera for both orbit and non orbit control
	UNREALED_API void SetCameraSetup(const FVector& LocationForOrbiting, const FRotator& InOrbitRotation, const FVector& InOrbitZoom, const FVector& InOrbitLookAt, const FVector& InViewLocation, const FRotator &InViewRotation );

	/** Callback for toggling the camera lock flag. */
	UNREALED_API virtual void SetCameraLock();

	/** Callback for checking the camera lock flag. */
	UNREALED_API bool IsCameraLocked() const;

	/** Callback for toggling the grid show flag. */
	UNREALED_API void SetShowGrid();

	/** Callback for checking the grid show flag. */
	UNREALED_API bool IsSetShowGridChecked() const;

	/** Sets the show bounds flag */
	UNREALED_API void SetShowBounds(bool bShow);

	/** Callback for toggling the bounds show flag. */
	UNREALED_API void ToggleShowBounds();

	/** Callback for checking the bounds show flag. */
	UNREALED_API bool IsSetShowBoundsChecked() const;

	/** Callback for toggling the collision geometry show flag. */
	UNREALED_API void SetShowCollision();

	/** Callback for checking the collision geometry show flag. */
	UNREALED_API bool IsSetShowCollisionChecked() const;
	
	/** Callback for toggling the geometry inspection show wireframe flag. */
	UNREALED_API void ToggleGeometryInspectionShowWireframe();

	/** Callback for checking the geometry inspection show wireframe flag. */
	UNREALED_API bool IsSetGeometryInspectionShowWireframe() const;

	/** Callback for setting a specific channel mask flag */
	UNREALED_API void SetChannelMask(EColorChannelMask InColorChannelMask);

	/** Callback for checking a specific channel mask flag */
	UNREALED_API bool IsChannelBeingMasked(EColorChannelMask InColorChannelMask) const;

	/** Gets ViewportCameraTransform object for the current viewport type */
	FViewportCameraTransform& GetViewTransform()
	{
		return IsPerspective() ? ViewTransformPerspective : ViewTransformOrthographic;
	}

	const FViewportCameraTransform& GetViewTransform() const
	{
		return IsPerspective() ? ViewTransformPerspective : ViewTransformOrthographic;
	}

	/** Sets the location of the viewport's camera */
	void SetViewLocation( const FVector& NewLocation )
	{
		FViewportCameraTransform& ViewTransform = GetViewTransform();
		ViewTransform.SetLocation(NewLocation);
	}

	/** Sets the location of the viewport's camera */
	void SetViewRotation( const FRotator& NewRotation )
	{
		FViewportCameraTransform& ViewTransform = GetViewTransform();
		ViewTransform.SetRotation(NewRotation);
	}

	/** 
	 * Sets the look at location of the viewports camera for orbit *
	 * 
	 * @param LookAt The new look at location
	 * @param bRecalulateView	If true, will recalculate view location and rotation to look at the new point immediatley
	 */
	void SetLookAtLocation( const FVector& LookAt, bool bRecalculateView = false )
	{
		FViewportCameraTransform& ViewTransform = GetViewTransform();

		ViewTransform.SetLookAt(LookAt);

		if( bRecalculateView )
		{
			FMatrix OrbitMatrix = ViewTransform.ComputeOrbitMatrix();
			OrbitMatrix = OrbitMatrix.InverseFast();

			ViewTransform.SetRotation(OrbitMatrix.Rotator());
			ViewTransform.SetLocation(OrbitMatrix.GetOrigin());
		}
	}
	/** Perform default camera movement that would normally happen* This can be used by systems to override how the camera is triggered*/
	UNREALED_API void PeformDefaultCameraMovement(FVector& Drag, FRotator& Rot, FVector& Scale);

	/** Sets ortho zoom amount */
	void SetOrthoZoom( float InOrthoZoom ) 
	{
		FViewportCameraTransform& ViewTransform = GetViewTransform();

		// A zero ortho zoom is not supported and causes NaN/div0 errors
		check(InOrthoZoom != 0);
		ViewTransform.SetOrthoZoom(InOrthoZoom);
	}

	/** @return the current viewport camera location */
	const FVector& GetViewLocation() const
	{
		const FViewportCameraTransform& ViewTransform = GetViewTransform();
		return ViewTransform.GetLocation();
	}

	/** @return the current viewport camera rotation */
	const FRotator& GetViewRotation() const
	{
		const FViewportCameraTransform& ViewTransform = GetViewTransform();
		return ViewTransform.GetRotation();
	}

	/** @return the current look at location */
	const FVector& GetLookAtLocation() const
	{
		const FViewportCameraTransform& ViewTransform = GetViewTransform();
		return ViewTransform.GetLookAt();
	}

	/** @return the current ortho zoom amount */
	float GetOrthoZoom() const
	{
		const FViewportCameraTransform& ViewTransform = GetViewTransform();
		return ViewTransform.GetOrthoZoom();
	}

	/** @return The number of units per pixel displayed in this viewport */
	UNREALED_API float GetOrthoUnitsPerPixel(const FViewport* Viewport) const;
	
	/** @return The forward vector of the camera view */
	UNREALED_API FVector GetForwardVector() const;

	/** Get a prettified string representation of the specified unreal units */
	static UNREALED_API FString UnrealUnitsToSiUnits(float UnrealUnits);

	void RemoveCameraRoll()
	{
		FRotator Rotation = GetViewRotation();
		Rotation.Roll = 0;
		SetViewRotation( Rotation );
	}

	UNREALED_API void SetInitialViewTransform(ELevelViewportType ViewportType, const FVector& ViewLocation, const FRotator& ViewRotation, float InOrthoZoom );

	UNREALED_API void TakeHighResScreenShot();

	/** Called when an editor mode has been (de)activated */
	UNREALED_API void OnEditorModeIDChanged(const FEditorModeID& EditorModeID, bool bIsEntering);

	/** FViewElementDrawer interface */
	UNREALED_API virtual void Draw(const FSceneView* View,FPrimitiveDrawInterface* PDI) override;
	UNREALED_API virtual void Draw(FViewport* Viewport,FCanvas* Canvas) override;

	/**
	 * Gets the world space cursor info from the current mouse position
	 *
	 * @return					An FViewportCursorLocation containing information about the mouse position in world space.
	 */
	UNREALED_API FViewportCursorLocation GetCursorWorldLocationFromMousePos();

	/** FViewportClient interface */
	UNREALED_API virtual bool ProcessScreenShots(FViewport* Viewport) override;
	UNREALED_API virtual void RedrawRequested(FViewport* Viewport) override;
	UNREALED_API virtual void RequestInvalidateHitProxy(FViewport* Viewport) override;
	UNREALED_API virtual bool InputKey(const FInputKeyEventArgs& EventArgs) override;
	UNREALED_API virtual bool InputAxis(const FInputKeyEventArgs& Args) override;
	UNREALED_API virtual bool InputGesture(FViewport* Viewport, const FInputDeviceId DeviceId, EGestureEvent GestureType, EGesturePhase GesturePhase, const FVector2D& GestureDelta, bool bIsDirectionInvertedFromDevice, const uint64 Timestamp) override;
	UNREALED_API virtual void ReceivedFocus(FViewport* Viewport) override;
	UNREALED_API virtual void MouseEnter(FViewport* Viewport,int32 x, int32 y) override;
	UNREALED_API virtual void MouseMove(FViewport* Viewport,int32 x, int32 y) override;
	UNREALED_API virtual void MouseLeave( FViewport* Viewport ) override;
	UNREALED_API virtual EMouseCursor::Type GetCursor(FViewport* Viewport,int32 X,int32 Y) override;
	UNREALED_API virtual TOptional<TSharedRef<SWidget>> MapCursor(FViewport* Viewport, const FCursorReply& CursorReply) override;
	UNREALED_API virtual void CapturedMouseMove( FViewport* InViewport, int32 InMouseX, int32 InMouseY ) override;
	UNREALED_API virtual void ProcessAccumulatedPointerInput(FViewport* InViewport) override;
	UNREALED_API virtual bool IsOrtho() const override;
	UNREALED_API virtual void LostFocus(FViewport* Viewport) override;
	UNREALED_API virtual FStatUnitData* GetStatUnitData() const override;
	UNREALED_API virtual FStatHitchesData* GetStatHitchesData() const override;
	UNREALED_API virtual const TArray<FString>* GetEnabledStats() const override;
	UNREALED_API virtual void SetEnabledStats(const TArray<FString>& InEnabledStats) override;
	UNREALED_API virtual bool IsStatEnabled(const FString& InName) const override;

	virtual bool BeginTransform(const FGizmoState& InState) { return false; }
	virtual bool EndTransform(const FGizmoState& InState) { return false; }
	virtual bool OnTransformCanceled(const FGizmoState& InState) { return EndTransform(InState); }
	
protected:

	UNREALED_API virtual bool Internal_InputKey(const FInputKeyEventArgs& EventArgs);
	UNREALED_API virtual bool Internal_InputAxis(FViewport* Viewport, FInputDeviceId DeviceID, FKey Key, float Delta, float DeltaTime, int32 NumSamples=1, bool bGamepad=false);

public:
	UNREALED_API virtual void AddAssociation(FViewport& InViewport) override;
	UNREALED_API virtual void RemoveAssociation(FViewport& InViewport) override;
	
	/** FGCObject interface */
	UNREALED_API virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	UNREALED_API virtual FString GetReferencerName() const override;

	/**
	 * Called when the user clicks in the viewport
	 *
	 * @param View			The view of the scene in the viewport
	 * @param HitProxy		Any hit proxy that was clicked
	 * @param Key			The key that caused the click event
	 * @param Event			State of the pressed key
	 * @param HitX			The X location of the mouse
	 * @param HitY			The Y location of the mouse
	 */
	UNREALED_API virtual void ProcessClick(FSceneView& View, HHitProxy* HitProxy, FKey Key, EInputEvent Event, uint32 HitX, uint32 HitY);

	/**
	 * Override to specify what should happen when a context menu is requested
	 * @param HitProxyElement the clicked element
	 */
	virtual void SummonContextMenu(const FTypedElementHandle& HitProxyElement = FTypedElementHandle()) {}

	/**
	 * Called when mouse movement tracking begins
	 *
	 * @param InInputState		The current mouse and keyboard input state
	 * @param bIsDraggingWidget	True if a widget is being dragged
	 * @param bNudge			True if we are tracking due to a nudge movement with the arrow keys
	 */
	virtual void TrackingStarted( const struct FInputEventState& InInputState, bool bIsDraggingWidget, bool bNudge ) {}
	
	/**
	 * Called when mouse movement tracking stops
	 */
	virtual void TrackingStopped() {}

	/**
	 * Called to give the viewport client a chance to handle widgets being moved
	 *
	 * @param InViewport	The viewport being rendered
	 * @param CurrentAxis	The current widget axis being moved
	 * @param Drag			The amount the widget was translated (the value depends on the coordinate system of the widget.  See GetWidgetCoordSystem )
	 * @param Rot			The amount the widget was rotated  (the value depends on the coordinate system of the widget.  See GetWidgetCoordSystem )
	 * @param Scale			The amount the widget was scaled (the value depends on the coordinate system of the widget.  See GetWidgetCoordSystem )
	 */
	UNREALED_API virtual bool InputWidgetDelta(FViewport* InViewport, EAxisList::Type CurrentAxis, FVector& Drag, FRotator& Rot, FVector& Scale);

	/**
	 * Sets the current widget mode
	 */
	UNREALED_API virtual void SetWidgetMode(UE::Widget::EWidgetMode NewMode);

	/**
	 * Whether or not the new widget mode can be set in this viewport
	 */
	UNREALED_API virtual bool CanSetWidgetMode(UE::Widget::EWidgetMode NewMode) const;

	/**
	 * Whether or not the widget mode can be cycled
	 */
	virtual bool CanCycleWidgetMode() const { return true; }

	/**
	 * @return The current display mode for transform widget 
	 */
	UNREALED_API virtual UE::Widget::EWidgetMode GetWidgetMode() const;

	/**
	 * @return The world space location of the transform widget
	 */
	UNREALED_API virtual FVector GetWidgetLocation() const;

	/**
	 * @return The current coordinate system for drawing and input of the transform widget.  
	 * For world coordiante system return the identity matrix
	 */
	UNREALED_API virtual FMatrix GetWidgetCoordSystem() const;

	/**
	* @return The local coordinate system for the transform widget.
	* For world coordiante system return the identity matrix
	*/
	UNREALED_API virtual FMatrix GetLocalCoordinateSystem() const;
	/**
	 * Sets the coordinate system space to use
	 */
	UNREALED_API virtual void SetWidgetCoordSystemSpace( ECoordSystem NewCoordSystem );

	/**
	 * @return The coordinate system space (world or local) to display the widget in
	 */
	UNREALED_API virtual ECoordSystem GetWidgetCoordSystemSpace() const;

	/**
	 * Sets the current axis being manipulated by the transform widget
	 */
	UNREALED_API virtual void SetCurrentWidgetAxis( EAxisList::Type InAxis );

	/**
	 * Adjusts the current transform widget size by the provided delta value
	 */
	UNREALED_API void AdjustTransformWidgetSize(const int32 SizeDelta);

	/**
	 * Called to do any additional set up of the view for rendering
	 * 
	 * @param ViewFamily	The view family being rendered
	 * @param View			The view being rendered
	 */
	UNREALED_API virtual void SetupViewForRendering( FSceneViewFamily& ViewFamily, FSceneView& View );

	/**
	 * Called to draw onto the viewports 2D canvas
	 *
	 * @param InViewport	The viewport being rendered
	 * @param View			The view of the scene to be rendered
	 * @param Canvas		The canvas to draw on
	 */
	UNREALED_API virtual void DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas);

	// Draws a visualization of the preview light if it was recently moved
	UNREALED_API virtual void DrawPreviewLightVisualization(const FSceneView* View, FPrimitiveDrawInterface* PDI);

	/**
	 * Render the drag tool in the viewport
	 */
	UNREALED_API void RenderDragTool(const FSceneView* View, FCanvas* Canvas);

	/**
	 * Configures the specified FSceneView object with the view and projection matrices for this viewport.
	 * @param	View		The view to be configured.  Must be valid.
	 * @param	StereoPass	Which eye we're drawing this view for when in stereo mode
	 * @return	A pointer to the view within the view family which represents the viewport's primary view.
	 */
	UNREALED_API virtual FSceneView* CalcSceneView(FSceneViewFamily* ViewFamily, const int32 StereoViewIndex = INDEX_NONE);

	/** 
	 * @return The scene being rendered in this viewport
	 */
	UNREALED_API virtual FSceneInterface* GetScene() const;

	/**
	 * @return The background color of the viewport 
	 */
	UNREALED_API virtual FLinearColor GetBackgroundColor() const;

	/**
	 * Called to override any post process settings for the view
	 *
	 * @param View	The view to override post process settings on
	 */
	virtual void OverridePostProcessSettings( FSceneView& View ) {};

	/**
	 * Ticks this viewport client
	 */
	UNREALED_API virtual void Tick(float DeltaSeconds);

	/**
	 * Called each frame to update the viewport based on delta mouse movements
	 */
	UNREALED_API virtual void UpdateMouseDelta();
	
	/**
	 * Called each frame to update the viewport based on delta trackpad gestures
	 */
	UNREALED_API virtual void UpdateGestureDelta();


	/** 
	* Use the viewports Scene to get a world. 
	* Will use Global world instance of the scene or its world is invalid.
	*
	* @return 		A valid pointer to the viewports world scene.
	*/
	UNREALED_API virtual UWorld* GetWorld() const override;

	/** If true, this is a level editor viewport */
	virtual bool IsLevelEditorClient() const { return false; }
	
	/**
	 * Called to make a drag tool when the user starts dragging in the viewport
	 *
	 * @param DragToolType	The type of drag tool to make
	 * @return The new drag tool
	 */
	UNREALED_API virtual TSharedPtr<FDragTool> MakeDragTool( EDragTool::Type DragToolType );

	/** @return true if a drag tool can be used */
	UNREALED_API virtual bool CanUseDragTool() const;

	/** @return Whether or not to orbit the camera */
	UNREALED_API virtual bool ShouldOrbitCamera() const;

	UNREALED_API bool IsMovingCamera() const;

	virtual void UpdateLinkedOrthoViewports( bool bInvalidate = false ) {}

	/**
	 * @return true to lock the pitch of the viewport camera
	 */
	UNREALED_API virtual bool ShouldLockPitch() const;

	/**
	 * Called when the mouse cursor is hovered over a hit proxy
	 * 
	 * @param HoveredHitProxy	The hit proxy currently hovered over
	 */
	UNREALED_API virtual void CheckHoveredHitProxy( HHitProxy* HoveredHitProxy );

	//~ TODO: UE_DEPRECATED(5.4,"Use HasDropPreviewElements instead.")
	/** Returns true if a placement dragging actor exists */
	virtual bool HasDropPreviewActors() const { return false; }

	//~ TODO: UE_DEPRECATED(5.4,"Use UpdateDropPreviewElements instead.")
	/**
	 * If dragging an actor for placement, this function updates its position.
	 *
	 * @param MouseX						The position of the mouse's X coordinate
	 * @param MouseY						The position of the mouse's Y coordinate
	 * @param DroppedObjects				The Objects that were used to create preview objects
	 * @param out_bDroppedObjectsVisible	Output, returns if preview objects are visible or not
	 *
	 * Returns true if preview actors were updated
	 */
	virtual bool UpdateDropPreviewActors(int32 MouseX, int32 MouseY, const TArray<UObject*>& DroppedObjects, bool& out_bDroppedObjectsVisible, UActorFactory* FactoryToUse = nullptr) { return false; }

	//~ TODO: UE_DEPRECATED(5.4,"Use DestroyDropPreviewElements instead.")
	/**
	 * If dragging an actor for placement, this function destroys the actor.
	 */
	virtual void DestroyDropPreviewActors() {}

	/** Returns true if a placement drag preview elements exists */
	virtual bool HasDropPreviewElements() const { return false; }

	/**
	 * If dragging items for placement, this function updates their position.
	 *
	 * @param MouseX						The position of the mouse's X coordinate
	 * @param MouseY						The position of the mouse's Y coordinate
	 * @param DroppedObjects				The Objects that were used to create preview objects
	 * @param out_bDroppedObjectsVisible	Output, returns if preview objects are visible or not
	 *
	 * Returns true if preview elements were updated
	 */
	virtual bool UpdateDropPreviewElements(int32 MouseX, int32 MouseY, 
		const TArray<UObject*>& DroppedObjects, bool& out_bDroppedObjectsVisible, 
		TScriptInterface<IAssetFactoryInterface> Factory = nullptr) { return false; }

	/**
	 * If dragging items for placement, this function destroys the items.
	 */
	virtual void DestroyDropPreviewElements() {}

	/**
	 * Checks the viewport to see if the given object can be dropped using the given mouse coordinates local to this viewport
	 *
	 * @param MouseX			The position of the mouse's X coordinate
	 * @param MouseY			The position of the mouse's Y coordinate
	 * @param AssetInfo			Asset in question to be dropped
	 */
	virtual FDropQuery CanDropObjectsAtCoordinates(int32 MouseX, int32 MouseY, const FAssetData& AssetInfo) { return FDropQuery(); }

	struct FDropObjectOptions
	{
		//~ This default constructor just exists as a workaround for a bug in some compilers
		//~ where trying to use a nested class as a default parameter in a member function of
		//~ the same outer class will fail to compile if the nested class has default member 
		//~ initializers and an implicitly declared default constructor.
		FDropObjectOptions() {}

		// Flag that when true, will only attempt a drop on the actor targeted by the mouse position. Defaults to false.
		bool bOnlyDropOnTarget = false;
		// If true, a drop preview will be spawned instead of a normal result.
		bool bCreateDropPreview = false;
		// If true, select the newly dropped elements
		bool bSelectOutput = true;
		// The preferred factory to use (optional)
		TScriptInterface<IAssetFactoryInterface> FactoryToUse = nullptr;
	};

	/**
	 * Attempts to intelligently drop the given objects in the viewport, using the given mouse coordinates local to this viewport
	 *
	 * @param MouseX			The position of the mouse's X coordinate
	 * @param MouseY			The position of the mouse's Y coordinate
	 * @param DroppedObjects	The Objects to be placed into the editor via this viewport
	 * @param OutNewObjects		The new actor objects that were created
	 * @param Options			Additional options
	 */
	UNREALED_API virtual bool DropObjectsAtCoordinates(int32 MouseX, int32 MouseY, const TArray<UObject*>& DroppedObjects, TArray<FTypedElementHandle>& OutNewObjects,
		const FDropObjectOptions& Options = FDropObjectOptions())
		// Forwards to the other overload (which returns false) during deprecation. Will return false directly once the other
		// overload is removed.
		;

	UE_DEPRECATED(5.4, "Use the overload that takes FDropObjectOptions instead.")
	virtual bool DropObjectsAtCoordinates(int32 MouseX, int32 MouseY, const TArray<UObject*>& DroppedObjects, TArray<AActor*>& OutNewActors,
		bool bOnlyDropOnTarget = false, bool bCreateDropPreview = false, bool bSelectActors = true,
		UActorFactory* FactoryToUse = nullptr) { return false; }

	/** Returns true if the viewport is allowed to be possessed for previewing cinematic sequences or keyframe animations*/
	bool AllowsCinematicControl() const { return bAllowCinematicControl; }

	/** Sets whether or not this viewport is allowed to be possessed by cinematic/scrubbing tools */
	void SetAllowCinematicControl( bool bInAllowCinematicControl) { bAllowCinematicControl = bInAllowCinematicControl; }

	/** 
	 * Normally we disable all viewport rendering when the editor is minimized, but the 
	 * render commands may need to be processed regardless (like if we were outputting to a monitor via capture card). 
	 * This provides the viewport a way to keep rendering, regardless of the editor's window status.
	 */
	virtual bool WantsDrawWhenAppIsHidden() const { return false; }

	/** Should this viewport use app time instead of world time. */
	virtual bool UseAppTime() const { return IsRealtime() && !IsSimulateInEditorViewport(); }
public:
	/** True if the window is maximized or floating */
	UNREALED_API bool IsVisible() const;

	bool IsSimulateInEditorViewport() const 
	{ 
		return bIsSimulateInEditorViewport;
	}

	/**
	 * Returns true if status information should be displayed over the viewport
	 *
	 * @return	true if stats should be displayed
	 */
	bool ShouldShowStats() const
	{
		return bShowStats;
	}

	/**
	 * Sets whether or not stats info is displayed over the viewport
	 *
	 * @param	bWantStats	true if stats should be displayed
	 */
	UNREALED_API virtual void SetShowStats( bool bWantStats ) override;
	
	/**
	 * Sets how the viewport is displayed (lit, wireframe, etc) for the current viewport type
	 * 
	 * @param	InViewModeIndex				View mode to set for the current viewport type
	 */
	UNREALED_API virtual void SetViewMode(EViewModeIndex InViewModeIndex);
	
	/**
	 * Sets how the viewport is displayed (Geometry Inspection Modes) and displays warning
	 * 
	 * @param	InViewModeIndex				View mode to set for the current viewport type
	 */
	UNREALED_API virtual void SetGeometryInspectionViewMode(EViewModeIndex InViewModeIndex);

	/**
	 * Sets the viewport type to Geometry Inspection Zebra mode, horizontal or vertical type
	 * 
	 * @param	bHorizontalZebraMode		Controls if this is horizontal or vertical stripe type
	 */
	UNREALED_API virtual void SetZebraViewMode(bool bHorizontalZebraMode);
	
	/**
	 * Sets Geometry Inspection Zebra mode stripe count
	 * 
	 * @param	InStripeCount				Number of stripes Zebra mode should have
	 */
	UNREALED_API virtual void SetZebraViewModeStripeCount(uint8 InStripeCount);

	/**
	 * Sets how the viewport is displayed (lit, wireframe, etc)
	 * 
	 * @param	InPerspViewModeIndex		View mode to set when this viewport is of type LVT_Perspective
	 * @param	InOrthoViewModeIndex		View mode to set when this viewport is not of type LVT_Perspective
	 */
	UNREALED_API void SetViewModes(const EViewModeIndex InPerspViewModeIndex, const EViewModeIndex InOrthoViewModeIndex);

	/** Set the viewmode param. */
	UNREALED_API void SetViewModeParam(int32 InViewModeParam);

	/**
	 * @return The current view mode in this viewport, for the current viewport type
	 */
	UNREALED_API EViewModeIndex GetViewMode() const;

	/**
	 * @return The view mode to use when this viewport is of type LVT_Perspective
	 */
	EViewModeIndex GetPerspViewMode() const
	{
		return PerspViewModeIndex;
	}

	/**
	 * @return The view mode to use when this viewport is not of type LVT_Perspective
	 */
	EViewModeIndex GetOrthoViewMode() const
	{
		return OrthoViewModeIndex;
	}

	/** @return True if InViewModeIndex is the current view mode index */
	bool IsViewModeEnabled(EViewModeIndex InViewModeIndex) const
	{
		return GetViewMode() == InViewModeIndex;
	}

	/** @return True if ViewMode is Geometry Inspection Zebra in specified orientation */
	UNREALED_API bool IsZebraViewModeEnabled(bool bHorizontal) const;

	/** @return Configured number of stripes in Geometry Inspection Zebra view mode */
	UNREALED_API static uint8 GetZebraViewModeStripeCount();

	/** @return True if InViewModeIndex is the current view mode param */
	UNREALED_API bool IsViewModeParam(int32 InViewModeParam) const;

	/**
	 * Invalidates this viewport and optionally child views.
	 *
	 * @param	bInvalidateChildViews		[opt] If true (the default), invalidate views that see this viewport as their parent.
	 * @param	bInvalidateHitProxies		[opt] If true (the default), invalidates cached hit proxies too.
	 */
	UNREALED_API void Invalidate(bool bInvalidateChildViews=true, bool bInvalidateHitProxies=true);
	
	/**
	 * Gets the dimensions of the viewport
	 */
	UNREALED_API void GetViewportDimensions( FIntPoint& OutOrigin, FIntPoint& OutSize );
	
	/**
	 * Determines which axis InKey and InDelta most refer to and returns a corresponding FVector.  This
	 * vector represents the mouse movement translated into the viewports/widgets axis space.
	 *
	 * @param	InNudge		If 1, this delta is coming from a keyboard nudge and not the mouse
	 */
	UNREALED_API FVector TranslateDelta( FKey InKey, float InDelta, bool InNudge );

	/**
	 * Returns the effective viewport type (taking into account any actor locking or camera possession)
	 */
	UNREALED_API virtual ELevelViewportType GetViewportType() const;

	/**
	 * Set the viewport type of the client
	 *
	 * @param InViewportType	The viewport type to set the client to
	 */
	UNREALED_API virtual void SetViewportType( ELevelViewportType InViewportType );
	
	/**
	 * Rotate through viewport view options
	 */
	UNREALED_API virtual void RotateViewportType();

	/**
	* @return If the viewport option in the array is the active viewport type
	*/
	UNREALED_API bool IsActiveViewportTypeInRotation() const;

	/**
	 * @return If InViewportType is the active viewport type 
	 */
	UNREALED_API bool IsActiveViewportType(ELevelViewportType InViewportType) const;

	/** Returns true if this viewport is perspective. */
	UNREALED_API bool IsPerspective() const;

	/** Is the aspect ratio currently constrained? */
	UNREALED_API virtual bool IsAspectRatioConstrained() const;

	/** 
	 * Focuses the viewport to the center of the bounding box ensuring that the entire box is in view 
	 *
	 * @param BoundingBox			The box to focus
	 * @param bInstant			Whether or not to focus the viewport instantly or over time
	 */
	UNREALED_API void FocusViewportOnBox( const FBox& BoundingBox, bool bInstant = false );

	/**
	 * Gets whether focusing on a box should clip to selection.
	 */
	bool GetClipToSelectionOnFocus() const { return bClipToSelectionOnFocus; }

	/**
	 * Sets whether focusing on a box should clip to selection.
	 */
	void SetClipToSelectionOnFocus(bool bInClipToSelection);

	/**
	 * Translates the viewport so that the given LookAt point is at the center of viewport, while maintaining current Location/LookAt distance
	 *
	 * @param NewLookAt			The new NewLookAt point to focus on
	 * @param bInstant			Whether or not to focus the viewport instantly or over time
	 */
	UNREALED_API void CenterViewportAtPoint(const FVector& NewLookAt, bool bInstant = false);

	FEditorCameraController* GetCameraController(void) { return CameraController; }

	UNREALED_API void InputAxisForOrbit(FViewport* Viewport, const FVector& DragDelta, FVector& Drag, FRotator& Rot);

	/**
	 * Implements screenshot capture for editor viewports.  Should be called by derived class' InputKey.
	 */
	UNREALED_API bool InputTakeScreenshot(FViewport* Viewport, FKey Key, EInputEvent Event);

	/**
	 * Opens the screenshot in the uses default bitmap viewer (determined by OS)
	 */
	UNREALED_API void OpenScreenshot( FString SourceFilePath );

	/**
	 * Takes the screenshot capture, this is called by keybinded events as well InputTakeScreenshot
	 *
	 * @param Viewport				The viewport to take a screenshot of.
	 * @param bInvalidateViewport	Some methods already invalidate the viewport before calling this method.
	 */
	UNREALED_API void TakeScreenshot(FViewport* Viewport, bool bInvalidateViewport);

	/**
	 * Converts a generic movement delta into drag/rotation deltas based on the viewport and keys held down.
	 */
	UNREALED_API void ConvertMovementToDragRot( const FVector& InDelta, FVector& InDragDelta, FRotator& InRotDelta ) const;
	UNREALED_API void ConvertMovementToOrbitDragRot(const FVector& InDelta, FVector& InDragDelta, FRotator& InRotDelta) const;

	
	/** Toggle between orbit camera and fly camera */
	UNREALED_API void ToggleOrbitCamera( bool bEnableOrbitCamera );

	/**
	 * Sets the camera view location such that the LookAtPoint point is at the specified location.
	 */
	UNREALED_API void SetViewLocationForOrbiting(const FVector& LookAtPoint, float DistanceToCamera = 256.f );

	/**
	 * Moves the viewport Scamera according to the specified drag and rotation.
	 */
	UNREALED_API void MoveViewportCamera( const FVector& InDrag, const FRotator& InRot, bool bDollyCamera = false );

	/** 
	 * Get the custom pivot point around which the camera should orbit for this viewport
	 * @param	OutPivot	The custom pivot point specified by the viewport
	 * @return	true if a custom pivot point was specified, false otherwise.
	 */
	UNREALED_API virtual bool GetPivotForOrbit(FVector& OutPivot) const;

	// Utility functions to return the modifier key states
	UNREALED_API bool IsAltPressed() const;
	UNREALED_API bool IsCtrlPressed() const;
	UNREALED_API bool IsShiftPressed() const;
	UNREALED_API bool IsCmdPressed() const;
	
	// Functions for registering and evaluating prioritized input chords. 
	// When a prioritized chord evaluates true its required keys are 
	// ignored by all lower priority chords, providing a way to resolve 
	// conflict between input key combinations.
	UNREALED_API void RegisterPrioritizedInputChord(const FPrioritizedInputChord& InInputCord);
	UNREALED_API void UnregisterPrioritizedInputChord(const FName InInputCordName);
	UNREALED_API bool IsPrioritizedInputChordPressed(const FName InInputCordName) const;

	/**
	 * Utility function to return whether the command accepts the key states
	 * @param InCommand The command being checked
	 * @param InOptionalKey (Optional) input key being tested against. If not specified, the current viewport's key state's key will be used
	 * @return True if one of the command's chords accepts the input :
	 */
	UNREALED_API bool IsCommandChordPressed(const TSharedPtr<FUICommandInfo> InCommand, FKey InOptionalKey = FKey()) const;

	/** @return True if the window is in an immersive viewport */
	UNREALED_API bool IsInImmersiveViewport() const;

	void ClearAudioFocus() { bHasAudioFocus = false; }
	
	void SetAudioFocus() { bHasAudioFocus = true; }

	UNREALED_API void MarkMouseMovedSinceClick();

	/** Determines whether this viewport is currently allowed to use Absolute Movement */
	UNREALED_API bool IsUsingAbsoluteTranslation(bool bAlsoCheckAbsoluteRotation = false) const;

	bool IsForcedRealtimeAudio() const { return bForceAudioRealtime; }

	/** true to force realtime audio to be true, false to stop forcing it */
	void SetForcedAudioRealtime(bool bShouldForceAudioRealtime)
	{
		bForceAudioRealtime = bShouldForceAudioRealtime;
	}

	/** @return true if a mouse button is down and it's movement being tracked for operations inside the viewport */
	bool IsTracking() const { return bIsTracking; }

	UNREALED_API EAxisList::Type GetCurrentWidgetAxis() const;

	/** Overrides current cursor. */
	UNREALED_API void SetRequiredCursorOverride( bool WantOverride, EMouseCursor::Type RequiredCursor = EMouseCursor::Default ); 

	/** Overrides current widget mode */
	UNREALED_API void SetWidgetModeOverride(UE::Widget::EWidgetMode InWidgetMode);

	/** Get the camera speed for this viewport */
	UNREALED_API float GetCameraSpeed() const;
	
	/** Gets the settings that control camera speed for this viewport */
	UNREALED_API virtual const FEditorViewportCameraSpeedSettings& GetCameraSpeedSettings() const;
	
	/** Sets the camera speed settings for this viewport */
	UNREALED_API virtual void SetCameraSpeedSettings(const FEditorViewportCameraSpeedSettings& InCameraSpeedSettings);
	
	/** Get the camera speed for this viewport based on the specified speed setting */
	UE_DEPRECATED(5.7, "Integer-based camera speed settings are deprecated.")
	UNREALED_API float GetCameraSpeed(int32 SpeedSetting) const;

	/** Set the speed setting for the camera in this viewport */
	UE_DEPRECATED(5.7, "Integer-based camera speed settings are deprecated.")
	UNREALED_API virtual void SetCameraSpeedSetting(int32 SpeedSetting) final;

	/** Get the camera speed setting for this viewport */
	UE_DEPRECATED(5.7, "Integer-based camera speed settings are deprecated.")
	UNREALED_API virtual int32 GetCameraSpeedSetting() const final;

	/** Get the camera speed scalar for this viewport */
	UE_DEPRECATED(5.7, "Please use GetCameraSpeed() directly")
	UNREALED_API virtual float GetCameraSpeedScalar() const final;

	UE_DEPRECATED(5.7, "Please use SetCameraSpeedSettings()")
	/** Set the camera speed scalar for this viewport */
	UNREALED_API virtual void SetCameraSpeedScalar(float SpeedScalar) final;

	/** Editor mode tool manager being used for this viewport client */
	FEditorModeTools* GetModeTools() const
	{
		return ModeTools.Get();
	}

	/** Legacy adapter for a toolkit to take ownership of a mode manager that may have been created by this viewport client */
	UE_DEPRECATED(5.0, "This function is meant for legacy edge cases (pre-UAssetEditor), where the toolkit may not be the original owner of the mode manager.")
	UNREALED_API void TakeOwnershipOfModeManager(TSharedPtr<FEditorModeTools>& ModeManagerPtr);

	/** Get the editor viewport widget */
	TSharedPtr<SEditorViewport> GetEditorViewportWidget() const { return EditorViewportWidget.Pin(); }


	UNREALED_API virtual FMatrix CalcViewRotationMatrixForControllingActorView(const FRotator& InViewRotation) const;
	
	/**
	 * Computes a matrix to use for viewport location and rotation
	 */
	UNREALED_API virtual FMatrix CalcViewRotationMatrix(const FRotator& InViewRotation) const;

protected:

	/** true if this window is allowed to be possessed by cinematic tools for previewing sequences in real-time */
	bool bAllowCinematicControl;

	/** The default camera speed settings for the viewport */
	FEditorViewportCameraSpeedSettings CameraSpeedSettings;

	/** Camera speed setting */
	UE_DEPRECATED(5.7, "Use the new GetCameraSpeed() and SetCameraSpeed() functions.")
	int32 CameraSpeedSetting;

	/** Camera speed scalar */
	UE_DEPRECATED(5.7, "Use the new GetCameraSpeed() and SetCameraSpeed() functions.")
	float CameraSpeedScalar;

public:

	UNREALED_API void DrawBoundingBox(FBox &Box, FCanvas* InCanvas, const FSceneView* InView, const FViewport* InViewport, const FLinearColor& InColor, const bool bInDrawBracket, const FString &InLabelText) ;
	
	/**
	 * Draws a screen space bounding box around the specified actor
	 *
	 * @param	InCanvas		Canvas to draw on
	 * @param	InView			View to render
	 * @param	InViewport		Viewport we're rendering into
	 * @param	InActor			Actor to draw a bounding box for
	 * @param	InColor			Color of bounding box
	 * @param	bInDrawBracket	True to draw a bracket, otherwise a box will be rendered
	 * @param	bInLabelText	Optional label text to draw
	 */
	UNREALED_API void DrawActorScreenSpaceBoundingBox( FCanvas* InCanvas, const FSceneView* InView, FViewport* InViewport, AActor* InActor, const FLinearColor& InColor, const bool bInDrawBracket, const FString& InLabelText = TEXT( "" ) );

	UNREALED_API void SetGameView(bool bGameViewEnable);

	UNREALED_API virtual void SetVREditView(bool bGameViewEnable);
	/**
	 * Returns true if this viewport is excluding non-game elements from its display
	 */
	virtual bool IsInGameView() const override { return bInGameViewMode; }

	/**
	 * Aspect ratio bar display settings
	 */
	void SetShowAspectRatioBarDisplay(bool bEnable)
	{
		EngineShowFlags.SetCameraAspectRatioBars(bEnable);
		Invalidate(false,false);
	}

	void SetShowSafeFrameBoxDisplay(bool bEnable)
	{
		EngineShowFlags.SetCameraSafeFrames(bEnable);
		Invalidate(false,false);
	}

	bool IsShowingAspectRatioBarDisplay() const
	{
		return EngineShowFlags.CameraAspectRatioBars == 1;
	}

	bool IsShowingSafeFrameBoxDisplay() const
	{
		return EngineShowFlags.CameraSafeFrames == 1;
	}

	/** Get the near clipping plane for this viewport. */
	UNREALED_API float GetNearClipPlane() const;

	/** Override the near clipping plane. Set to a negative value to disable the override. */
	UNREALED_API void OverrideNearClipPlane(float InNearPlane);

	/** Get the far clipping plane override for this viewport. */
	UNREALED_API float GetFarClipPlaneOverride() const;
	
	/** Override the far clipping plane. Set to a negative value to disable the override. */
	UNREALED_API void OverrideFarClipPlane(const float InFarPlane); 

	/** Get the orthographic near plane override for this viewport. */
	UNREALED_API TOptional<double> GetOrthographicNearPlaneOverride() const;
	
	/** Set the orthographic near plane override for this viewport. */
	UNREALED_API void SetOrthographicNearPlaneOverride(const TOptional<double>& InValue);
	
	/** Get the orthographic far plane override for this viewport. */
	UNREALED_API TOptional<double> GetOrthographicFarPlaneOverride() const;
	
	/** Set the orthographic far plane override for this viewport. */
	UNREALED_API void SetOrthographicFarPlaneOverride(const TOptional<double>& InValue);
	
	/** Gets the bounds of the world being represented. */
	UNREALED_API FBox GetWorldBounds() const;

	/** When collision draw mode changes, this function allows hidden objects to be drawn, so hidden colliding objects can be seen */
	UNREALED_API void UpdateHiddenCollisionDrawing();
	/** Returns the scene depth at the given viewport X,Y */
	UNREALED_API float GetSceneDepthAtLocation(int32 X, int32 Y);

	/** Returns the location of the object at the given viewport X,Y */
	UNREALED_API FVector GetHitProxyObjectLocation(int32 X, int32 Y);

	/** Returns the map allowing to convert from the viewmode param to a name. */
	TMap<int32, FName>& GetViewModeParamNameMap() { return ViewModeParamNameMap; }

	/** Show or hide the widget. */
	UNREALED_API void ShowWidget(const bool bShow);
	bool GetShowWidget() const { return bShowWidget; }

	/**
	 * Returns whether or not the flight camera is active
	 *
	 * @return true if the flight camera is active
	 */
	UNREALED_API bool IsFlightCameraActive() const;

	/** Delegate handler fired when a show flag is toggled */
	UNREALED_API virtual void HandleToggleShowFlag(FEngineShowFlags::EShowFlag EngineShowFlagIndex);

	/** Delegate handler fired to determine the state of a show flag */
	UNREALED_API virtual bool HandleIsShowFlagEnabled(FEngineShowFlags::EShowFlag EngineShowFlagIndex) const;

	/**
	 * Changes the buffer visualization mode for this viewport.
	 *
	 * @param InName	The ID of the required visualization mode
	 */
	UNREALED_API void ChangeBufferVisualizationMode( FName InName );

	/**
	 * Checks if a buffer visualization mode is selected.
	 * 
	 * @param InName	The ID of the required visualization mode
	 * @return	true if the supplied buffer visualization mode is checked
	 */
	UNREALED_API bool IsBufferVisualizationModeSelected( FName InName ) const;

	/**
	 * Returns the FText display name associated with CurrentBufferVisualizationMode.
	 */
	UNREALED_API FText GetCurrentBufferVisualizationModeDisplayName() const;

	/**
	 * Changes the Nanite visualization mode for this viewport.
	 *
	 * @param InName	The ID of the required visualization mode
	 */
	UNREALED_API void ChangeNaniteVisualizationMode(FName InName);

	/**
	 * Checks if a Nanite visualization mode is selected.
	 *
	 * @param InName	The ID of the required visualization mode
	 * @return	true if the supplied Nanite visualization mode is checked
	 */
	UNREALED_API bool IsNaniteVisualizationModeSelected(FName InName) const;

	/**
	 * Returns the FText display name associated with CurrentNaniteVisualizationMode.
	 */
	UNREALED_API FText GetCurrentNaniteVisualizationModeDisplayName() const;

	/**
	 * Changes the Lumen visualization mode for this viewport.
	 *
	 * @param InName	The ID of the required visualization mode
	 */
	UNREALED_API void ChangeLumenVisualizationMode(FName InName);

	/**
	 * Checks if a Lumen visualization mode is selected.
	 *
	 * @param InName	The ID of the required visualization mode
	 * @return	true if the supplied Lumen visualization mode is checked
	 */
	UNREALED_API bool IsLumenVisualizationModeSelected(FName InName) const;

	/**
	 * Returns the FText display name associated with CurrentLumenVisualizationMode.
	 */
	UNREALED_API FText GetCurrentLumenVisualizationModeDisplayName() const;

	/**
	 * Changes the MegaLights visualization mode for this viewport.
	 *
	 * @param InName	The ID of the required visualization mode
	 */
	UNREALED_API void ChangeMegaLightsVisualizationMode(FName InName);

	/**
	 * Checks if a MegaLights visualization mode is selected.
	 *
	 * @param InName	The ID of the required visualization mode
	 * @return	true if the supplied MegaLights visualization mode is checked
	 */
	UNREALED_API bool IsMegaLightsVisualizationModeSelected(FName InName) const;

	/**
	 * Returns the FText display name associated with CurrentMegaLightsVisualizationMode.
	 */
	UNREALED_API FText GetCurrentMegaLightsVisualizationModeDisplayName() const;

	/**
	 * Changes the Substrate visualization mode for this viewport.
	 *
	 * @param InName	The ID of the required visualization mode
	 */
	UNREALED_API void ChangeSubstrateVisualizationMode(FName InName);

	/**
	 * Checks if a Substrate visualization mode is selected.
	 *
	 * @param InName	The ID of the required visualization mode
	 * @return	true if the supplied Substrate visualization mode is checked
	 */
	UNREALED_API bool IsSubstrateVisualizationModeSelected(FName InName) const;

	/**
	 * Returns the FText display name associated with CurrentSubstrateVisualizationMode.
	 */
	UNREALED_API FText GetCurrentSubstrateVisualizationModeDisplayName() const;

	/**
	 * Changes the Groom visualization mode for this viewport.
	 *
	 * @param InName	The ID of the required visualization mode
	 */
	UNREALED_API void ChangeGroomVisualizationMode(FName InName);

	/**
	 * Checks if a Groom visualization mode is selected.
	 *
	 * @param InName	The ID of the required visualization mode
	 * @return	true if the supplied Groom visualization mode is checked
	 */
	UNREALED_API bool IsGroomVisualizationModeSelected(FName InName) const;

	/**
	 * Returns the FText display name associated with CurrentGroomVisualizationMode.
	 */
	UNREALED_API FText GetCurrentGroomVisualizationModeDisplayName() const;

	/**
	* Changes the virtual shadow map visualization mode for this viewport.
	*
	* @param InName	The ID of the required visualization mode
	*/
	UNREALED_API void ChangeVirtualShadowMapVisualizationMode(FName InName);

	/**
	* Checks if a virtual shadow map visualization mode is selected.
	*
	* @param InName	The ID of the required visualization mode
	* @return	true if the supplied virtual shadow map visualization mode is checked
	*/
	UNREALED_API bool IsVirtualShadowMapVisualizationModeSelected(FName InName) const;

	/**
	* Returns the FText display name associated with CurrentVirtualShadowMapVisualizationMode.
	*/
	UNREALED_API FText GetCurrentVirtualShadowMapVisualizationModeDisplayName() const;

	/**
	* Changes the virtual texture visualization mode for this viewport.
	*
	* @param InName	The ID of the required visualization mode
	*/
	UNREALED_API void ChangeVirtualTextureVisualizationMode(FName InName);

	/**
	* Checks if a virtual texture visualization mode is selected.
	*
	* @param InName	The ID of the required visualization mode
	* @return	true if the supplied virtual texture visualization mode is checked
	*/
	UNREALED_API bool IsVirtualTextureVisualizationModeSelected(FName InName) const;
	
	/**
	* Returns the FText display name associated with CurrentVirtualTextureVisualizationMode.
	*/
	UNREALED_API FText GetCurrentVirtualTextureVisualizationModeDisplayName() const;

	/**
	* Changes the actor coloration visualization mode for this viewport.
	*
	* @param InName	The ID of the required visualization mode
	*/
	UNREALED_API void ChangeActorColorationVisualizationMode(FName InName);

	/**
	* Checks if an actor coloration visualization mode is selected.
	*
	* @param InName	The ID of the required visualization mode
	* @return	true if the supplied actor coloration visualization mode is checked
	*/
	UNREALED_API bool IsActorColorationVisualizationModeSelected(FName InName) const;

	/**
	* Returns the FText display name associated with current actor coloration visualization mode.
	*/
	UNREALED_API FText GetCurrentActorColorationVisualizationModeDisplayName() const;

	/**
	* Returns whether visualize debug material is enabled.
	*/
	UNREALED_API bool IsVisualizeCalibrationMaterialEnabled() const;

	/**
	 * Changes the ray tracing debug visualization mode for this viewport
	 *
	 * @param InName	The ID of the required ray tracing debug visualization mode
	 */
	UNREALED_API void ChangeRayTracingDebugVisualizationMode(FName InName);

	/**
	 * Checks if a ray tracing debug visualization mode is selected
	 *
	 * @param InName	The ID of the required ray tracing debug visualization mode
	 * @return	true if the supplied ray tracing debug visualization mode is checked
	 */
	UNREALED_API bool IsRayTracingDebugVisualizationModeSelected(FName InName) const;

	/**
	 * Changes the GPU Skin Cache visualization mode for this viewport
	 *
	 * @param InName	The ID of the required GPU Skin Cache visualization mode
	 */
	UNREALED_API void ChangeGPUSkinCacheVisualizationMode(FName InName);

	/**
	 * Checks if a GPU Skin Cache visualization mode is selected
	 *
	 * @param InName	The ID of the required GPU Skin Cache visualization mode
	 * @return	true if the supplied GPU Skin Cache visualization mode is checked
	 */
	UNREALED_API bool IsGPUSkinCacheVisualizationModeSelected(FName InName) const;

	/**
	* Returns the FText display name associated with CurrentGPUSkinCacheVisualizationMode.
	*/
	UNREALED_API FText GetCurrentGPUSkinCacheVisualizationModeDisplayName() const;

	/** @return True if PreviewResolutionFraction is supported. */
	UNREALED_API bool SupportsPreviewResolutionFraction() const;

	/** @return the view status to select the corresponding default screen percentage behavior */
	UNREALED_API EViewStatusForScreenPercentage GetViewStatusForScreenPercentage() const;

	/** @return default resolution fraction for UI based on display resolution and user settings. */
	UNREALED_API float GetDefaultPrimaryResolutionFractionTarget() const;

	/** @return whether preview screen percentage for UI. */
	UNREALED_API bool IsPreviewingScreenPercentage() const;

	/** Sets whether preview screen percentage for UI. */
	UNREALED_API void SetPreviewingScreenPercentage(bool bIsPreviewing);

	/** @return preview screen percentage for UI. */
	UNREALED_API int32 GetPreviewScreenPercentage() const;

	/** Set preview screen percentage on UI behalf. */
	UNREALED_API void SetPreviewScreenPercentage(int32 PreviewScreenPercentage);

	/** @return True if DPI preview is supported. */
	UNREALED_API bool SupportsLowDPIPreview() const;

	/** @return whether previewing for low DPI. */
	UNREALED_API bool IsLowDPIPreview() const;

	/** Set whether previewing for low DPI. */
	UNREALED_API void SetLowDPIPreview(bool LowDPIPreview);
	
	/** Mouse info is usually transformed to gizmo space before FEdMode handles it, this allows raw delta X and Y access */
	FMouseDeltaTracker* GetMouseDeltaTracker() const { return  MouseDeltaTracker; }

	virtual uint32 GetCachedMouseX() const { return CachedMouseX; }
	virtual uint32 GetCachedMouseY() const { return CachedMouseY; }

	/** @return True if the camera speed should be scaled by its view distance. */
	UNREALED_API virtual bool ShouldScaleCameraSpeedByDistance() const;

	/**
	 * Enable customization of the EngineShowFlags for rendering. After calling this function,
	 * the provided OverrideFunc will be passed a copy of .EngineShowFlags in ::Draw() just before rendering setup.
	 * Changes made to the ShowFlags will be used for that frame but .EngineShowFlags will not be modified.
	 * @param OverrideFunc custom override function that will be called every frame until override is disabled.
	 */
	UNREALED_API void EnableOverrideEngineShowFlags(TUniqueFunction<void(FEngineShowFlags&)> OverrideFunc);

	/** Disable EngineShowFlags override if enabled */
	UNREALED_API void DisableOverrideEngineShowFlags();

	/** @return true if Override EngineShowFlags are currently enabled */
	bool IsEngineShowFlagsOverrideEnabled() const { return !! OverrideShowFlagsFunc; }

	inline bool GetIsCurrentLevelEditingFocus() const								{ return bIsCurrentLevelEditingFocus; }
	inline void SetIsCurrentLevelEditingFocus(bool bInIsCurrentLevelEditingFocus)	{ bIsCurrentLevelEditingFocus = bInIsCurrentLevelEditingFocus; }

	FViewportClientNavigationHelper* GetViewportNavigationHelper() const
	{
		return ViewportNavigationHelper;
	}

	/**
	 * Translates the viewport camera by the specified amount.
	 */
	UNREALED_API void TranslateViewportCamera(const FVector& InTranslation);

protected:
	/** Invalidates the viewport widget (if valid) to register its active timer */
	UNREALED_API void InvalidateViewportWidget();

	/** Constant for how much the camera safe zone rectangle is inset when being displayed in the editor */
	static UNREALED_API float const SafePadding;
	/**
	 * Called when the perspective viewport camera moves
	 */
	virtual void PerspectiveCameraMoved() {}

	/** Updates the rotate widget with the passed in delta rotation. */
	UNREALED_API void ApplyDeltaToRotateWidget(const FRotator& InRot);

	/** Invalidates this and other linked viewports (anything viewing the same scene) */
	UNREALED_API virtual void RedrawAllViewportsIntoThisScene();

	/** FCommonViewportClient interface */
	UNREALED_API virtual float UpdateViewportClientWindowDPIScale() const override;

	/**
	 * Used to store the required cursor visibility states and override cursor appearance
	 */
	struct FRequiredCursorState
	{
		/** Should the software cursor be visible */
		bool	bSoftwareCursorVisible;

		/** Should the hardware cursor be visible */
		bool	bHardwareCursorVisible;

		/** Should the software cursor position be reset to pre-drag */
		bool	bDontResetCursor;

		/** Should we override the cursor appearance with the value in RequiredCursor */
		bool	bOverrideAppearance;

		/** What the cursor should look like */
		EMouseCursor::Type RequiredCursor;
	};

	/**
	 * Updates the visibility of the hardware and software cursors according to the viewport's state.
	 */
	UNREALED_API void UpdateAndApplyCursorVisibility();

	/** Setup the cursor visibility state we require and store in RequiredCursorVisibiltyAndAppearance struct */
	UNREALED_API void UpdateRequiredCursorVisibility();
	
	/** Sets the required hardware and software cursor. */
	UNREALED_API void SetRequiredCursor(const bool bHardwareCursorVisible, const bool bSoftwareCursorVisible);

	/** 
	 * Apply the required cursor visibility states from the RequiredCursorVisibiltyAndAppearance struct 
	 * @param	View				True - Set the position of the software cursor if being made visible. This defaults to FALSE.
	*/
	UNREALED_API void ApplyRequiredCursorVisibility(  bool bUpdateSoftwareCursorPostion = false );

	UNREALED_API bool ShouldUseMoveCanvasMovement() const;


	/**
	 * Draws viewport axes
	 *
	 * @param	InViewport		Viewport we're rendering into
	 * @param	InCanvas		Canvas to draw on
	 * @param	InRotation		Specifies the rotation to apply to the axes
	 * @param	InAxis			Specifies which axes to draw
	 */
	UNREALED_API void DrawAxes(FViewport* Viewport,FCanvas* Canvas, const FRotator* InRotation = nullptr, EAxisList::Type InAxis = EAxisList::XYZ);

	/**
	 * Draws viewport scale units
	 *
	 * @param	InViewport		Viewport we're rendering into
	 * @param	InCanvas		Canvas to draw on
	 * @param	InView			Scene view used for rendering
	 */
	UNREALED_API void DrawScaleUnits(FViewport* Viewport, FCanvas* Canvas, const FSceneView& InView);

	/**
	 * Starts tracking the mouse due to mouse input
	 *
	 * @param InputState	The mouse and keyboard input state at the time the mouse input happened
	 * @param View			The view of the scene in this viewport
	 */
	UNREALED_API void StartTrackingDueToInput( const struct FInputEventState& InputState, FSceneView& View );

	/**
	 * Handles clicking in the viewport
	 *
	 * @param InputState	The mouse and keyboard input state at the time the click happened
	 * @param View			The view of the scene in this viewport
	 */
	UNREALED_API void ProcessClickInViewport( const FInputEventState& InputState, FSceneView& View );

	/**
	 * Handles double clicking in the viewport
	 *
	 * @param InputState	The mouse and keyboard input state at the time the click happened
	 * @param View			The view of the scene in this viewport
	 */
	UNREALED_API void ProcessDoubleClickInViewport( const struct FInputEventState& InputState, FSceneView& View );

	/**
	 * Called when a user zooms the ortho viewport
	 *
	 * @param InputState	The current state of mouse and keyboard buttons
	 * @param Scale			Allows modifying the default amount of zoom
	 */
	UNREALED_API void OnOrthoZoom( const struct FInputEventState& InputState, float Scale = 1.0f );

	/**
	 * Called when a user dollys the perspective camera
	 *
	 * @param InputState	The current state of mouse and keyboard buttons
	 */
	UNREALED_API void OnDollyPerspectiveCamera( const struct FInputEventState& InputState );

	/**
	 * Called when a user changes the camera speed
	 *
	 * @param InputState	The current state of mouse and keyboard buttons
	 */
	UNREALED_API void OnChangeCameraSpeed( const struct FInputEventState& InputState );
	
	/**
	 * Stops any mouse tracking
	 */
	UNREALED_API void StopTracking();

	/**
	 * Aborts mouse tracking (stop and cancel)
	 */
	UNREALED_API virtual void AbortTracking();

	/** Enables or disables camera lock **/
	UNREALED_API void EnableCameraLock(bool bEnable);

	/**
	 * Gets a joystick state cache for the specified controller ID
	 */
	UNREALED_API FCachedJoystickState* GetJoystickState(const uint32 InControllerID);

	/** Helper used by DrawSafeFrames to get the current safe frame aspect ratio. */
	virtual bool GetActiveSafeFrame(float& OutAspectRatio) const { return false; }

	/** Helper function to calculate the safe frame rectangle on the current viewport */
	UNREALED_API bool CalculateEditorConstrainedViewRect(FSlateRect& OutSafeFrameRect, FViewport* InViewport, const float DPIScale);

	virtual void NudgeSelectedObjects(const struct FInputEventState& InputState) {}

private:
	/** @return Whether or not the camera should be panned or dollied */
	UNREALED_API bool ShouldPanOrDollyCamera() const;

	UNREALED_API void ConditionalCheckHoveredHitProxy();

	/** Returns true if perspective flight camera input mode is currently active in this viewport */
	UNREALED_API bool IsFlightCameraInputModeActive() const;


	/** Moves a perspective camera */
	UNREALED_API void MoveViewportPerspectiveCamera( const FVector& InDrag, const FRotator& InRot, bool bDollyCamera = false );

	
	/**Applies Joystick axis control to camera movement*/
	UNREALED_API void UpdateCameraMovementFromJoystick(const bool bRelativeMovement, FCameraControllerConfig& InConfig);

	/**
	 * Updates real-time camera movement.  Should be called every viewport tick!
	 *
	 * @param	DeltaTime	Time interval in seconds since last update
	 */
	UNREALED_API void UpdateCameraMovement( float DeltaTime );

	/**
	 * Updates the camera based on external movement sources.
	 */
	bool ConsumeExternalCameraMovement();

	/**
	 * Directs the camera to orbit based on a screen space movement delta
	 */
	void OrbitPerspectiveCamera(const FRotator& InOrbit);

	/**
	 * Dollies the camera towards or away from its orbit point
	 */
	void DollyPerspectiveCamera(double InDelta);

	/**
	 * Forcibly disables lighting show flags if there are no lights in the scene, or restores lighting show
	 * flags if lights are added to the scene.
	 */
	UNREALED_API void UpdateLightingShowFlags( FEngineShowFlags& InOutShowFlags );

	/** InOut might get adjusted depending on viewmode or viewport type */
	UNREALED_API void ApplyEditorViewModeAdjustments(FEngineShowFlags& InOut) const;

	/** Renders the safe frame lines. */
	UNREALED_API void DrawSafeFrames(FViewport& Viewport, FSceneView& View, FCanvas& Canvas);

	UNREALED_API void DrawSafeFrameQuad( FCanvas &Canvas, FVector2D V1, FVector2D V2 );
	
	virtual FEngineShowFlags* GetEngineShowFlags() override
	{ 
		return &EngineShowFlags; 
	}

	/**
	 * Set a specific stat to either enabled or disabled (returns true if there are any stats enabled)
	 */
	UNREALED_API int32 SetStatEnabled(const TCHAR* InName, const bool bEnable, const bool bAll = false);

	/** Delegate handler to see if a stat is enabled on this viewport */
	UNREALED_API void HandleViewportStatCheckEnabled(const TCHAR* InName, bool& bOutCurrentEnabled, bool& bOutOthersEnabled);

	/** Delegate handler for when stats are enabled in a viewport */
	UNREALED_API void HandleViewportStatEnabled(const TCHAR* InName);

	/** Delegate handler for when stats are disabled in a viewport */
	UNREALED_API void HandleViewportStatDisabled(const TCHAR* InName);

	/** Delegate handler for when all stats are disabled in a viewport */
	UNREALED_API void HandleViewportStatDisableAll(const bool bInAnyViewport);

	/** Delegate handler for when a window DPI changes and we might need to adjust the scenes resolution */
	UNREALED_API void HandleWindowDPIScaleChanged(TSharedRef<SWindow> InWindow);

	/** Handle the camera about to be moved or stopped **/
	virtual void BeginCameraMovement(bool bHasMovement) {}
	virtual void EndCameraMovement() {}

	UNREALED_API virtual float GetMinimumOrthoZoom() const;

public:
	UE_DEPRECATED(5.7, "Camera speed range is now controlled with FEditorViewportClient::SetCameraSpeedRange()")
	static UNREALED_API const uint32 MaxCameraSpeeds;

	/** Delegate used to get whether or not this client is in an immersive viewport */
	FViewportStateGetter ImmersiveDelegate;

	/** Delegate used to get the visibility of this client from a slate viewport layout and tab configuration */
	FViewportStateGetter VisibilityDelegate; 

	FViewport*				Viewport;

	/** Viewport camera transform data for perspective viewports */
	FViewportCameraTransform		ViewTransformPerspective;

	/** Viewport camera transform data for orthographic viewports */
	FViewportCameraTransform		ViewTransformOrthographic;

	/** The viewport type. */
	ELevelViewportType		ViewportType;

	/** The viewport's scene view state. */
	FSceneViewStateReference ViewState;

	/** Viewport view state when stereo rendering is enabled */
	TArray<FSceneViewStateReference> StereoViewStates;

	/** A set of flags that determines visibility for various scene elements. */
	FEngineShowFlags		EngineShowFlags;

	/** Previous value for engine show flags, used for toggling. */
	FEngineShowFlags		LastEngineShowFlags;

	/** Editor setting to allow designers to override the automatic expose */
	FExposureSettings		ExposureSettings;

	/** Editor setting to adjust wireframe opacity */
	float WireframeOpacity = 0.2f;

	FName CurrentBufferVisualizationMode;
	FName CurrentNaniteVisualizationMode;
	FName CurrentLumenVisualizationMode;
	FName CurrentMegaLightsVisualizationMode;
	FName CurrentSubstrateVisualizationMode;
	FName CurrentGroomVisualizationMode;
	FName CurrentVirtualShadowMapVisualizationMode;
	FName CurrentVirtualTextureVisualizationMode;

	FName CurrentRayTracingDebugVisualizationMode;
	FName CurrentGPUSkinCacheVisualizationMode;

	/** The number of frames since this viewport was last drawn.  Only applies to linked orthographic movement. */
	int32 FramesSinceLastDraw;

	/** Index of this view in the editor's list of views */
	int32 ViewIndex;

	/** Viewport's current horizontal field of view (can be modified by locked cameras etc.) */
	float ViewFOV;
	/** Viewport's stored horizontal field of view (saved in ini files). */
	float FOVAngle;

	float AspectRatio;
	
	/** true if we've forced the SHOW_Lighting show flag off because there are no lights in the scene */
	bool bForcingUnlitForNewMap;

	/** true if the widget's axis is being controlled by an active mouse drag. */
	bool bWidgetAxisControlledByDrag;
	
	/** The number of pending viewport redraws. */
	bool bNeedsRedraw;

	bool bNeedsLinkedRedraw;

	/** If, following the next redraw, we should invalidate hit proxies on the viewport */
	bool bNeedsInvalidateHitProxy;
	
	/** True if the orbit camera is currently being used */
	bool bUsingOrbitCamera;
	
	/** If true, numpad keys will be used to move camera in perspective viewport */
	bool bUseNumpadCameraControl;

	/**
	 * true if all input is rejected from this viewport
	 */
	bool					bDisableInput;
	
	/** If true, draw the axis indicators when the viewport is perspective. */
	bool					bDrawAxes;

	/** If true, draw the axis indicators when EngineShowFlags.Game is set. */
	bool					bDrawAxesGame;

	/** If true, the listener position will be set */
	bool					bSetListenerPosition;

	// Override the LOD of landscape in this viewport
	int8 LandscapeLODOverride;

	/** Parameters for masking specific color channels for visualization */
	struct FColorChannelMaskParams
	{
		/** The color channel to mask to, whose value will be output as a grayscale color */
		EColorChannelMask ColorChannelMask;

		/** When true, the alpha channel is inverted when being visualized; does not effect the actual alpha value, just its visualization */
		bool bInvertAlphaChannelMask;

		/** Indicates that a checkerboard should be alpha blended underneath the viewport to help visualize the view's alpha */
		bool bDrawAlphaBlendedCheckerboard;

		FColorChannelMaskParams();
	} ChannelMaskParams;

	/** If true, draw vertices for selected BSP brushes and static meshes if the large vertices show flag is set. */
	bool bDrawVertices;

	/** List of view modifiers to apply on view parameters. */
	FEditorViewportViewModifierDelegate ViewModifiers;

	/** Whether view modifiers should be called and applied. */
	bool bShouldApplyViewModifiers;

	TSharedPtr<UE::RenderViewportFeedback::FReceiver, ESPMode::ThreadSafe> RenderViewportFeedback = nullptr;

protected:
	/** Editor mode tools provided to this instance. */
	TSharedPtr<FEditorModeTools> ModeTools;

	FWidget*				Widget;

	/** Whether the widget should be drawn. */
	bool bShowWidget;

	FMouseDeltaTracker*		MouseDeltaTracker;

	/** If true, the canvas has been been moved using bMoveCanvas Mode*/
	bool bHasMouseMovedSinceClick;

	/** Cursor visibility and appearance information */
	FRequiredCursorState RequiredCursorVisibiltyAndAppearance;

	/** Cached state of joystick axes and buttons*/
	TMap<int32, FCachedJoystickState*> JoystickStateMap;

	/** Camera controller object that's used for piloting the camera around */
	FEditorCameraController* CameraController;
	
	/** Current cached impulse state */
	FCameraControllerUserImpulseData* CameraUserImpulseData;

	/**
	 * Can be used to add location, rotation, and FOV deltas to the viewport camera.
	 * Should be used by ITF-based Viewport Interactions to control the camera.
	 */
	FViewportClientNavigationHelper* ViewportNavigationHelper;

	/** When we have LOD locking, it's slow to force redraw of other viewports, so we delay invalidates to reduce the number of redraws */
	double TimeForForceRedraw;

	/** Extra camera speed scale for flight camera */
	UE_DEPRECATED(5.7, "Please use GetCameraSpeed() and SetCameraSpeed() instead.")
	float FlightCameraSpeedScale;

	bool bUseControllingActorViewInfo;
	FMinimalViewInfo ControllingActorViewInfo;
	TOptional<EAspectRatioAxisConstraint> ControllingActorAspectRatioAxisConstraint;
	TArray<FPostProcessSettings> ControllingActorExtraPostProcessBlends;
	TArray<float> ControllingActorExtraPostProcessBlendWeights;

	/* Updated on each mouse drag start */
	uint32 LastMouseX;
	uint32 LastMouseY;

	/** Represents the last known mouse position. If the mouse stops moving it's not the current but the last position before the current location. */
	uint32 CachedMouseX;
	uint32 CachedMouseY;

	/** Represents the last mouse position. It is constantly updated on tick so it can also be the current position. */
	int32 CachedLastMouseX = 0;
	int32 CachedLastMouseY = 0;

	/** True is the use is controling the light via a shorcut*/
	bool bUserIsControllingAtmosphericLight0 = false;
	bool bUserIsControllingAtmosphericLight1 = false;
	float UserIsControllingAtmosphericLightTimer = 0.0f;
	FTransform UserControlledAtmosphericLightMatrix;


	// -1, -1 if not set
	FIntPoint CurrentMousePos;

	/**
	 * true when within a FMouseDeltaTracker::StartTracking/EndTracking block.
	 */
	bool bIsTracking;

	/**
	 * true if the user is dragging by a widget handle.
	 */
	bool bDraggingByHandle;

	/** Cumulative camera drag and rotation deltas from trackpad gesture in current Tick */
	FVector CurrentGestureDragDelta;
	FRotator CurrentGestureRotDelta;

	float GestureMoveForwardBackwardImpulse;

	/** If true, force this viewport to use real time audio regardless of other settings */
	bool bForceAudioRealtime;

	/** Overrides bIsRealtime until GFrameCounter is >= this number. Used to force viewports to draw a number of frames even if they are otherwise non-realtime viewports. */
	uint64 RealTimeUntilFrameNumber;

	/** if the viewport is currently realtime */
	bool bIsRealtime;
	
	/** True if we should draw stats over the viewport */
	bool bShowStats;

	/** If true, this viewport gets to set audio parameters (e.g. set the listener) */
	bool bHasAudioFocus;
	
	/** true when we are in a state where we can check the hit proxy for hovering. */
	bool bShouldCheckHitProxy;

	/** True if this viewport uses a draw helper */
	bool bUsesDrawHelper;

	/** True if this level viewport can be used to view Simulate in Editor sessions */
	bool bIsSimulateInEditorViewport;

	/** Camera Lock or not **/
	bool bCameraLock;

	/** Is the camera moving? **/
	bool bIsCameraMoving;

	/** Is the camera moving at the beginning of the tick? **/
	bool bIsCameraMovingOnTick;

	/** Draw helper for rendering common editor functionality like the grid */
	FEditorCommonDrawHelper DrawHelper;

	/** The editor viewport widget this client is attached to */
	TWeakPtr<SEditorViewport> EditorViewportWidget;

	/** The viewport interaction instance for this viewport */
	TObjectPtr<UTypedElementViewportInteraction> ViewportInteraction;

	/** The scene used for the viewport. Owned externally */
	FPreviewScene* PreviewScene;

	/** default set up for toggling back **/
	FRotator DefaultOrbitRotation = FRotator::ZeroRotator;
	FVector DefaultOrbitLocation = FVector::ZeroVector;
	FVector DefaultOrbitZoom = FVector::ZeroVector;
	FVector DefaultOrbitLookAt = FVector::ZeroVector;

	/** 
	 * Temporary realtime overrides and user messages that are not saved between editor sessions.
	 * If any value has been pushed into the container, this viewport determines realtime from the last item,
	 * otherwise it reads the real realtime value.
	 */
	struct FRealtimeOverride
	{
		FText SystemDisplayName;
		bool bIsRealtime = false;

		FRealtimeOverride(bool bInIsRealtime, FText InSystemDisplayName);
	};
	TArray<FRealtimeOverride> RealtimeOverrides;
	
	/** Custom override function that will be called every ::Draw() until override is disabled */
	TUniqueFunction<void(FEngineShowFlags&)> OverrideShowFlagsFunc;

	/**  When focusing viewport on a box, treat boxes smaller than this size as being this size */
	FVector::FReal MinimumFocusRadius = 10.0;

protected:
	// Used for the display of the current preview light after it has been adjusted
	FVector2D MovingPreviewLightSavedScreenPos;
	float MovingPreviewLightTimer;

public:
	/* Default view mode for perspective viewports */
	static UNREALED_API const EViewModeIndex DefaultPerspectiveViewMode;

	/* Default view mode for orthographic viewports */
	static UNREALED_API const EViewModeIndex DefaultOrthoViewMode;
	
	/** Flag to lock the viewport fly camera */
	bool bLockFlightCamera;
protected:
	/** Data needed to display per-frame stat tracking when STAT UNIT is enabled */
	mutable FStatUnitData StatUnitData;

	/** Data needed to display per-frame stat tracking when STAT HITCHES is enabled */
	mutable FStatHitchesData StatHitchesData;

	/** A list of all the stat names which are enabled for this viewport */
	TArray<FString> EnabledStats;

	// Used by the custom temporal upscaler plugin
public:
	class ICustomTemporalUpscalerData
	{
	public:
		virtual ~ICustomTemporalUpscalerData() = default;
	};

	TSharedPtr<ICustomTemporalUpscalerData> GetCustomTemporalUpscalerData() const
	{
		return CustomTemporalUpscalerData;
	}

	void SetCustomTemporalUpscalerData(TSharedPtr<ICustomTemporalUpscalerData> InCustomTemporalUpscalerData)
	{
		CustomTemporalUpscalerData = InCustomTemporalUpscalerData;
	}

private:

	void OnObjectTransacted(UObject* Object, const FTransactionObjectEvent& Event);
	void OnPostLoadMap(uint32 MapChangeFlags);

	TSharedPtr<ICustomTemporalUpscalerData> CustomTemporalUpscalerData;

private:
	/** Controles resolution fraction for previewing in editor viewport at different screen percentage. */
	bool bIsPreviewingResolutionFraction = false;
	TOptional<float> PreviewResolutionFraction;

	// DPI mode for scene rendering.
	enum class ESceneDPIMode
	{
		// Uses r.Editor.Viewport.HighDPI.
		EditorDefault,

		// Force emulating low DPI.
		EmulateLowDPI,

		// Force using high dpi.
		HighDPI
	};
	ESceneDPIMode SceneDPIMode;

	/* View mode to set when this viewport is of type LVT_Perspective */
	EViewModeIndex PerspViewModeIndex;

	/* View mode to set when this viewport is not of type LVT_Perspective */
	EViewModeIndex OrthoViewModeIndex;

	/* View mode param */
	int32 ViewModeParam;
	FName ViewModeParamName;

	/* A map converting the viewmode param into an asset name. The map gets updated while the menu is populated. */
	TMap<int32, FName> ViewModeParamNameMap;

	/** near plane adjustable for each editor view, if < 0 GNearClippingPlane should be used. */
	float NearPlane;

	/** If > 0, overrides the view's far clipping plane with a plane at the specified distance. */
	float FarPlane;
	
	/** The location of the near clipping plane in orthographic view. */
	TOptional<double> OrthographicNearPlane;
	
	/** The location of the far clipping plane in orthographic view. */
	TOptional<double> OrthographicFarPlane;
	
	/** Whether focusing on a box should clip the near plane to that selection. */
	bool bClipToSelectionOnFocus = false;
	
	/** The cached bounds of the world. */
	FBox CachedWorldBounds;
	
	/** If true, we are in Game View mode*/
	bool bInGameViewMode;

	/** If true, we are in VR Edit View mode*/
	bool bInVREditViewMode;

	/** If true, the viewport widget should be invalidated on the next tick (needed to ensure thread safety) */
	bool bShouldInvalidateViewportWidget;

	/*
	 * When we drag the manipulator(no camera movement involve) in absolute we need a view to compute the delta
	 * We want to use the view when the user start dragging and use the same view until the displacement is done.
	 * Using the same view allow us to move the camera and not disrupt the movement continuity.
	 */
	FSceneView* DragStartView;
	FSceneViewFamily *DragStartViewFamily;

	TArray<FIntPoint> CapturedMouseMoves;

	/** A list of input key combinations, sorted by priority. See FPrioritizedInputChord. */
	TArray<FPrioritizedInputChord> PrioritizedInputChords;

	/** True if this is the current viewport client in editing */
	bool bIsCurrentLevelEditingFocus = false;

	/**
	 * true during FEditorViewportClient::StopTracking. Used to guard against reentry.
	 */
	bool bIsTrackingBeingStopped;

	/** true if Geometry Inspection Mode Wireframe option is on. */
	bool bIsGeometryInspectionWireframeOn = true;
};




class FEditorViewportStats
{
public:
	enum Category
	{
		CAT_PERSPECTIVE_KEYBOARD_WASD,
		CAT_PERSPECTIVE_KEYBOARD_UP_DOWN,
		CAT_PERSPECTIVE_KEYBOARD_FOV_ZOOM,
		CAT_PERSPECTIVE_MOUSE_PAN,
		CAT_PERSPECTIVE_MOUSE_DOLLY,
		CAT_PERSPECTIVE_MOUSE_SCROLL,
		CAT_PERSPECTIVE_MOUSE_ORBIT_ROTATION,
		CAT_PERSPECTIVE_MOUSE_ORBIT_PAN,
		CAT_PERSPECTIVE_MOUSE_ORBIT_ZOOM,
		CAT_PERSPECTIVE_GESTURE_SCROLL,
		CAT_PERSPECTIVE_GESTURE_MAGNIFY,

		CAT_ORTHOGRAPHIC_KEYBOARD_WASD,
		CAT_ORTHOGRAPHIC_KEYBOARD_UP_DOWN,
		CAT_ORTHOGRAPHIC_KEYBOARD_FOV_ZOOM,
		CAT_ORTHOGRAPHIC_MOUSE_PAN,
		CAT_ORTHOGRAPHIC_MOUSE_ZOOM,
		CAT_ORTHOGRAPHIC_MOUSE_SCROLL,
		CAT_ORTHOGRAPHIC_MOUSE_ORBIT_ROTATION,
		CAT_ORTHOGRAPHIC_MOUSE_ORBIT_PAN,
		CAT_ORTHOGRAPHIC_MOUSE_ORBIT_ZOOM,
		CAT_ORTHOGRAPHIC_GESTURE_SCROLL,
		CAT_ORTHOGRAPHIC_GESTURE_MAGNIFY,

		CAT_MAX
	};

	/**
	 * Used commits a single usage record for whichever category is sent to it.
	 */
	static UNREALED_API void Used(Category InCategory);

	/**
	 * Begins the frame for capturing using statements.  If nothing is logged between the begin and
	 * end frame we reset the last using tracking variable in EndFrame.
	 */
	static UNREALED_API void BeginFrame();

	/**
	 * Using commits a single usage record for whichever category is sent to it only if it's different
	 * from the last category that was sent to Using.  This should be used to capture usage data for modes
	 * where it's difficult to tell when they ended.
	 */
	static UNREALED_API void Using(Category InCategory);

	/**
	 * Doesn't use anything, but ensures that the last using item is not reset.
	 */
	static UNREALED_API void NoOpUsing();

	/**
	 * Use EndEndUsing to manually reset the Using state so that the next call to Using will commit a new record.
	 * Useful when you know a transition has occurred like the mouse button has been released.
	 */
	static UNREALED_API void EndFrame();

	static UNREALED_API void SendUsageData();

private:
	static void Initialize();

	static bool bInitialized;
	static bool bUsingCalledThisFrame;
	static Category LastUsing;
	static int32 DataPoints[CAT_MAX];
};
