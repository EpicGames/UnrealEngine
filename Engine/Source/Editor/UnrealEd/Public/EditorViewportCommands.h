// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once 

#include "CoreMinimal.h"
#include "Engine/EngineBaseTypes.h"
#include "Widgets/SWidget.h"
#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"
#include "Engine/TextureStreamingTypes.h"

class FEditorViewportClient;
class FMenuBuilder;
class FUICommandList;

UE_DECLARE_TCOMMANDS(class FEditorViewportCommands, UNREALED_API)

/**
 * Class containing commands for editor viewport actions common to all viewports
 */
class FEditorViewportCommands : public TCommands<FEditorViewportCommands>
{
public:
	UNREALED_API FEditorViewportCommands();
	
	/** Viewport pane configurations */
	TSharedPtr< FUICommandInfo > ViewportConfig_OnePane;
	TSharedPtr< FUICommandInfo > ViewportConfig_TwoPanesH;
	TSharedPtr< FUICommandInfo > ViewportConfig_TwoPanesV;
	TSharedPtr< FUICommandInfo > ViewportConfig_ThreePanesLeft;
	TSharedPtr< FUICommandInfo > ViewportConfig_ThreePanesRight;
	TSharedPtr< FUICommandInfo > ViewportConfig_ThreePanesTop;
	TSharedPtr< FUICommandInfo > ViewportConfig_ThreePanesBottom;
	TSharedPtr< FUICommandInfo > ViewportConfig_FourPanesLeft;
	TSharedPtr< FUICommandInfo > ViewportConfig_FourPanesRight;
	TSharedPtr< FUICommandInfo > ViewportConfig_FourPanesTop;
	TSharedPtr< FUICommandInfo > ViewportConfig_FourPanesBottom;
	TSharedPtr< FUICommandInfo > ViewportConfig_FourPanes2x2;


	/** Changes the viewport to perspective view */
	TSharedPtr< FUICommandInfo > Perspective;

	/** Changes the viewport to top view */
	TSharedPtr< FUICommandInfo > Top;

	/** Changes the viewport to bottom view */
	TSharedPtr< FUICommandInfo > Bottom;

	/** Changes the viewport to left view */
	TSharedPtr< FUICommandInfo > Left;

	/** Changes the viewport to right view */
	TSharedPtr< FUICommandInfo > Right;

	/** Changes the viewport to front view */
	TSharedPtr< FUICommandInfo > Front;

	/** Changes the viewport to back view */
	TSharedPtr< FUICommandInfo > Back;

	/** Rotate through viewport view options */
	TSharedPtr< FUICommandInfo > Next;

	/** Changes the viewport to wireframe */
	TSharedPtr< FUICommandInfo > WireframeMode;

	/** Changes the viewport to unlit mode */
	TSharedPtr< FUICommandInfo > UnlitMode;

	/** Changes the viewport to lit mode */
	TSharedPtr< FUICommandInfo > LitMode;

	/** Changes the viewport to lit wireframe mode */
	TSharedPtr< FUICommandInfo > LitWireframeMode;

	TSharedPtr< FUICommandInfo > PathTracingMode;
	TSharedPtr< FUICommandInfo > RayTracingDebugMode;

	/** Changes the viewport to detail lighting mode */
	TSharedPtr< FUICommandInfo > DetailLightingMode;

	/** Changes the viewport to reflection override mode */
	TSharedPtr< FUICommandInfo > ReflectionOverrideMode;

	/** Changes the viewport to lighting only */
	TSharedPtr< FUICommandInfo > LightingOnlyMode;

	/** Changes the viewport to light complextiy mode */
	TSharedPtr< FUICommandInfo > LightComplexityMode;

	/** Changes the viewport to shader complexity mode */
	TSharedPtr< FUICommandInfo > ShaderComplexityMode;

	/** Changes the viewport to quad overdraw mode */
	TSharedPtr< FUICommandInfo > QuadOverdrawMode;

	/** Changes the viewport to shader complexity with quad overdraw mode */
	TSharedPtr< FUICommandInfo > ShaderComplexityWithQuadOverdrawMode;

	/** Changes the viewport to primitive distance accuracy mode */
	TSharedPtr< FUICommandInfo > TexStreamAccPrimitiveDistanceMode;

	/** Changes the viewport to mesh UV density accuracy view mode */
	TSharedPtr< FUICommandInfo > TexStreamAccMeshUVDensityMode;
	TSharedPtr< FUICommandInfo > TexStreamAccMeshUVDensityAll; // TexStreamAccMeshUVDensityMode, but used in the view mode options menu.
	TSharedPtr< FUICommandInfo > TexStreamAccMeshUVDensitySingle[TEXSTREAM_MAX_NUM_UVCHANNELS];

	/** Changes the viewport to material texture scale accuracy mode */
	TSharedPtr< FUICommandInfo > TexStreamAccMaterialTextureScaleMode;
	TSharedPtr< FUICommandInfo > TexStreamAccMaterialTextureScaleAll; // Similar as TexStreamAccMaterialTextureScaleMode, but used in the view mode options menu.
	TSharedPtr< FUICommandInfo > RequiredTextureResolutionMode;

	/** Changes the viewport to stationary light overlap mode */
	TSharedPtr< FUICommandInfo > StationaryLightOverlapMode;

	/** Changes the viewport to lightmap density mode */
	TSharedPtr< FUICommandInfo > LightmapDensityMode;

	/** Shows the group menu item for LOD and HLOD coloration modes */
	TSharedPtr< FUICommandInfo > GroupLODColorationMode;

	/** Changes the viewport to LOD index coloration mode */
	TSharedPtr< FUICommandInfo > LODColorationMode;

	/** Changes the viewport to HLOD index coloration mode */
	TSharedPtr< FUICommandInfo > HLODColorationMode;

	/** Shows the group menu item for streaming deficit modes */
	TSharedPtr< FUICommandInfo > GroupStreamingDeficitMode;

	/** Changes the viewport to streaming texture deficit mode (WantedMips vs ResidentMips) */
	TSharedPtr< FUICommandInfo > StreamingTextureDeficitMode;

	/** Changes the viewport to streaming texture residency mode (MaxLODs vs ResidentMips) */
	TSharedPtr< FUICommandInfo > StreamingTextureResidencyMode;

	/** Changes the viewport to streaming mesh LOD deficit mode (WantedMips vs ResidentMips) */
	TSharedPtr< FUICommandInfo > StreamingMeshLODDeficitMode;

	/** Changes the viewport to streaming mesh LOD residency mode (MaxLODs vs ResidentLODs) */
	TSharedPtr< FUICommandInfo > StreamingMeshLODResidencyMode;

	/** Changes the viewport to visualize the buffer content */
	TSharedPtr< FUICommandInfo > VisualizeBufferMode;

	/** Changes the viewport to visualize Nanite content */
	TSharedPtr< FUICommandInfo > VisualizeNaniteMode;

	/** Changes the viewport to visualize Lumen debug views */
	TSharedPtr< FUICommandInfo > VisualizeLumenMode;

	/** Changes the viewport to visualize MegaLights debug views */
	TSharedPtr< FUICommandInfo > VisualizeMegaLightsMode;

	/** Changes the viewport to visualize Substrate debug views */
	TSharedPtr< FUICommandInfo > VisualizeSubstrateMode;

	/** Changes the viewport to visualize Groom debug views */
	TSharedPtr< FUICommandInfo > VisualizeGroomMode;

	/** Changes the viewport to visualize virtual shadow map content */
	TSharedPtr< FUICommandInfo > VisualizeVirtualShadowMapMode;

	/** Changes the viewport to visualize virtual texture content */
	TSharedPtr< FUICommandInfo > VisualizeVirtualTextureMode;

	/** Changes the viewport to Skin Cache mode */
	TSharedPtr< FUICommandInfo > VisualizeGPUSkinCacheMode;

	TSharedPtr< FUICommandInfo > VisualizeLWCComplexity;

	/** Collision Draw Mode */
	TSharedPtr< FUICommandInfo > CollisionPawn;
	TSharedPtr< FUICommandInfo > CollisionVisibility;

	/** Geometry Inspection Modes */
	TSharedPtr< FUICommandInfo > GeometryInspectionClayMode;
	TSharedPtr< FUICommandInfo > GeometryInspectionZebraHorizontalMode;
	TSharedPtr< FUICommandInfo > GeometryInspectionZebraVerticalMode;
	TSharedPtr< FUICommandInfo > GeometryInspectionFrontBackFaceMode;
	TSharedPtr< FUICommandInfo > GeometryInspectionRandomColorMode;
	TSharedPtr< FUICommandInfo > GeometryInspectionShowWireframe;

	/** Changes the viewport to show only specific color channels */
	TSharedPtr< FUICommandInfo > AllChannelsMask;
	TSharedPtr< FUICommandInfo > RedChannelMask;
	TSharedPtr< FUICommandInfo > GreenChannelMask;
	TSharedPtr< FUICommandInfo > BlueChannelMask;
	TSharedPtr< FUICommandInfo > AlphaChannelMask;

	/** Toggles realtime rendering in the viewport */
	TSharedPtr< FUICommandInfo > ToggleRealTime;

	/** Toggles showing stats in the viewport */
	TSharedPtr< FUICommandInfo > ToggleStats;

	/** Toggles showing fps in the viewport */
	TSharedPtr< FUICommandInfo > ToggleFPS;

	/** Allows the grid size setting to by changed by one */
	TSharedPtr< FUICommandInfo > IncrementPositionGridSize;

	/** Allows the grid size setting to by changed by one */
	TSharedPtr< FUICommandInfo > DecrementPositionGridSize;

	/** Allows the rotation grid size setting to by changed by one */
	TSharedPtr< FUICommandInfo > IncrementRotationGridSize;

	/** Allows the rotation grid size setting to by changed by one */
	TSharedPtr< FUICommandInfo > DecrementRotationGridSize;

	/** Command to capture screen */
	TSharedPtr< FUICommandInfo > ScreenCapture;

	/** Captures the viewport and updates the project thumbnail png file */
	TSharedPtr< FUICommandInfo > ScreenCaptureForProjectThumbnail;

	/** Select Mode */
	TSharedPtr< FUICommandInfo > SelectMode;

	/** Translate Mode */
	TSharedPtr< FUICommandInfo > TranslateMode;

	/** Rotate Mode */
	TSharedPtr< FUICommandInfo > RotateMode;

	/** Scale Mode */
	TSharedPtr< FUICommandInfo > ScaleMode;

	/** TranslateRotate Mode */
	TSharedPtr< FUICommandInfo > TranslateRotateMode;

	/** TranslateRotate Mode */
	TSharedPtr< FUICommandInfo > TranslateRotate2DMode;

	/** Shrink the level editor transform widget */
	TSharedPtr< FUICommandInfo > ShrinkTransformWidget;

	/** Expand the level editor transform widget */
	TSharedPtr< FUICommandInfo > ExpandTransformWidget;

	/** World relative coordinate system */
	TSharedPtr< FUICommandInfo > RelativeCoordinateSystem_World;

	/** Local relative coordinate system */
	TSharedPtr< FUICommandInfo > RelativeCoordinateSystem_Local;

	/** Parent-relative coordinate system */
	TSharedPtr<FUICommandInfo> RelativeCoordinateSystem_Parent;

	/** Explicit coordinate system (Rotation Order/Gimbal or Parent-relative coordinate system */
	TSharedPtr<FUICommandInfo> RelativeCoordinateSystem_Explicit;

	TSharedPtr< FUICommandInfo > CycleTransformGizmos;
	TSharedPtr< FUICommandInfo > CycleTransformGizmoCoordSystem;
	
	TSharedPtr<FUICommandInfo> IncreaseGizmoSize;
	TSharedPtr<FUICommandInfo> DecreaseGizmoSize;
	TSharedPtr<FUICommandInfo> ResetGizmoSize;

	TSharedPtr< FUICommandInfo > FocusAllViewportsToSelection;
	TSharedPtr< FUICommandInfo > FocusViewportToSelection;
	TSharedPtr< FUICommandInfo > FocusAndClipViewportToSelection;
	TSharedPtr< FUICommandInfo > FocusOutlinerToSelection;
	TSharedPtr< FUICommandInfo > FocusOutlinerToContextFolder;
	
	TSharedPtr<FUICommandInfo> UseOrthographicClippingPlanes;
	TSharedPtr<FUICommandInfo> DismissOrthographicClippingPlanes;

	/** Toggle automatic exposure */
	TSharedPtr< FUICommandInfo > ToggleAutoExposure;
	/** Toggle game settings exposure */
	TSharedPtr< FUICommandInfo > ToggleInGameExposure;
	/** Toggle the in-viewport context menu */
	TSharedPtr< FUICommandInfo > ToggleInViewportContextMenu;

	/** Toggle to override the viewport screen percentage */
	TSharedPtr< FUICommandInfo > ToggleOverrideViewportScreenPercentage;
	/** Opens the advanced viewport settings */
	TSharedPtr< FUICommandInfo > OpenEditorPerformanceProjectSettings;
	TSharedPtr< FUICommandInfo > OpenEditorPerformanceEditorPreferences;

	/**
	 * Grid commands
	 */

	/** Enables or disables all snapping */
	TSharedPtr< FUICommandInfo > Snapping;

	/** Enables or disables snapping to the grid when dragging objects around */
	TSharedPtr< FUICommandInfo > LocationGridSnap;

	/** Enables or disables snapping to a rotational grid while rotating objects */
	TSharedPtr< FUICommandInfo > RotationGridSnap;

	/** Enables or disables snapping to a 2D layer */
	TSharedPtr< FUICommandInfo > Layer2DSnap;

	/** Enables or disables snapping to a scaling grid while scaling objects */
	TSharedPtr< FUICommandInfo > ScaleGridSnap;

	/** Temporarily toggles snapping for the current transform mode (translate/rotate/scale) while held; quick-press toggles permanently */
	TSharedPtr< FUICommandInfo > CurrentTRSSnapToggle;

	/** Enables or disables snapping to surfaces when dragging in the viewport */
	TSharedPtr< FUICommandInfo > SurfaceSnapping;

	/** Enables or disables rotating to surface normal when snapping to surfaces */
	TSharedPtr< FUICommandInfo > RotateToSurfaceNormal;

	/** Enables or disables scaling the perspective camera speed, based on the distance between the camera and its look-at position */
	TSharedPtr<FUICommandInfo> ToggleDistanceBasedCameraSpeed;

public:
	/** Registers our commands with the binding system */
	UNREALED_API virtual void RegisterCommands() override;
};

/**
 * Get the viewmode otions menu label.
 * 
 * @param	ViewModeIndex	The current viewmode, as the options change  per viewmode.

 * @return	The label.
 */
UNREALED_API FText GetViewModeOptionsMenuLabel(EViewModeIndex ViewModeIndex);

/**
 * Build the viewmode otions menu.
 * 
 * @param	CommandList		The command list to be used with the menu builder.
 * @param	ViewModeIndex	The current viewmode, as the options change  per viewmode.
 * @param	FeatureLevel	The feature level to build the option menu for.
 * @param	ParamNameMaps	A map allowing to convert from each menu index to an asset name.

 * @return	The widget containing the viewmode options.
 */
UE_DEPRECATED(5.8, "Use UE::UnrealEd::BuildViewModeOptionsMenu instead.")
UNREALED_API TSharedRef<SWidget> BuildViewModeOptionsMenu(TSharedPtr<FUICommandList> CommandList, EViewModeIndex ViewModeIndex, ERHIFeatureLevel::Type FeatureLevel, TMap<int32, FName>& ParamNameMap);

namespace UE::UnrealEd
{
	struct FViewModeOptionsArgs
	{
		EViewModeIndex ViewModeIndex;
		ERHIFeatureLevel::Type FeatureLevel;
		TMap<int32, FName>& ParamNameMap;
		TSharedPtr<FEditorViewportClient> ViewportClient;
	};

	/**
	 * Build the viewmode otions menu.
	 * 
	 * @param FMenuBuilder The menu builder to create the menu
	 * @param FViewModeArgs The parameters to customize the view mode menu
	 */
	UNREALED_API void BuildViewModeOptionsMenu(FMenuBuilder& InMenuBuilder, const FViewModeOptionsArgs& InArgs);
}