// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetEditorViewportLayout.h"

class SSplitter;
class SWidget;
class FString;
class FName;

class FEditorViewportLayout2x2 : public FAssetEditorViewportPaneLayout
{
public:
	// IAssetEditorViewportLayoutConfiguration overrides
	virtual TSharedRef<SWidget> MakeViewportLayout(TSharedPtr<FAssetEditorViewportLayout> InParentLayout, const FString& LayoutString) override;
	virtual void ReplaceWidget(TSharedRef<SWidget> OriginalWidget, TSharedRef<SWidget> ReplacementWidget) override;
	virtual const FName& GetLayoutTypeName() const override;
	virtual void SaveLayoutString(const FString& SpecificLayoutString) const override;

protected:
	/** The splitter widget */
	TSharedPtr< class SSplitter2x2 > SplitterWidget;
};
