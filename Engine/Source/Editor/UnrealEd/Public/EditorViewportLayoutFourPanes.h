// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "EditorViewportLayout.h"
#include "Widgets/Layout/SSplitter.h"
#include "AssetEditorViewportLayout.h"

class FEditorViewportLayoutFourPanes : public FAssetEditorViewportPaneLayout
{
public:
	// IAssetEditorViewportLayoutConfiguration overrides
	virtual TSharedRef<SWidget> MakeViewportLayout(TSharedPtr<FAssetEditorViewportLayout> InParentLayout, const FString& LayoutString) override;
	virtual void SaveLayoutString(const FString& SpecificLayoutString) const override;
	virtual void ReplaceWidget(TSharedRef<SWidget> OriginalWidget, TSharedRef<SWidget> ReplacementWidget) override;

protected:
	virtual TSharedRef<SWidget> MakeFourPanelWidget(
		TSharedRef<SWidget> Viewport0, TSharedRef<SWidget> Viewport1, TSharedRef<SWidget> Viewport2, TSharedRef<SWidget> Viewport3,
		float PrimarySplitterPercentage, float SecondarySplitterPercentage0, float SecondarySplitterPercentage1) = 0;

	/** The splitter widgets */
	TSharedPtr< class SSplitter > PrimarySplitterWidget;
	TSharedPtr< class SSplitter > SecondarySplitterWidget;
};


// FEditorlViewportLayoutFourPanesLeft /////////////////////////////

class FEditorViewportLayoutFourPanesLeft : public FEditorViewportLayoutFourPanes
{
public:
	virtual const FName& GetLayoutTypeName() const override;

protected:
	virtual TSharedRef<SWidget> MakeFourPanelWidget(
		TSharedRef<SWidget> Viewport0, TSharedRef<SWidget> Viewport1, TSharedRef<SWidget> Viewport2, TSharedRef<SWidget> Viewport3,
		float PrimarySplitterPercentage, float SecondarySplitterPercentage0, float SecondarySplitterPercentage1) override;
};


// FEditorViewportLayoutFourPanesRight /////////////////////////////

class FEditorViewportLayoutFourPanesRight : public FEditorViewportLayoutFourPanes
{
public:
	virtual const FName& GetLayoutTypeName() const override;

protected:
	virtual TSharedRef<SWidget> MakeFourPanelWidget(
		TSharedRef<SWidget> Viewport0, TSharedRef<SWidget> Viewport1, TSharedRef<SWidget> Viewport2, TSharedRef<SWidget> Viewport3,
		float PrimarySplitterPercentage, float SecondarySplitterPercentage0, float SecondarySplitterPercentage1) override;
};


// FEditorViewportLayoutFourPanesTop /////////////////////////////

class FEditorViewportLayoutFourPanesTop : public FEditorViewportLayoutFourPanes
{
public:
	virtual const FName& GetLayoutTypeName() const override;

protected:
	virtual TSharedRef<SWidget> MakeFourPanelWidget(
		TSharedRef<SWidget> Viewport0, TSharedRef<SWidget> Viewport1, TSharedRef<SWidget> Viewport2, TSharedRef<SWidget> Viewport3,
		float PrimarySplitterPercentage, float SecondarySplitterPercentage0, float SecondarySplitterPercentage1) override;
};


// FEditorViewportLayoutFourPanesBottom /////////////////////////////

class FEditorViewportLayoutFourPanesBottom : public FEditorViewportLayoutFourPanes
{
public:
	virtual const FName& GetLayoutTypeName() const override;

protected:
	virtual TSharedRef<SWidget> MakeFourPanelWidget(
		TSharedRef<SWidget> Viewport0, TSharedRef<SWidget> Viewport1, TSharedRef<SWidget> Viewport2, TSharedRef<SWidget> Viewport3,
		float PrimarySplitterPercentage, float SecondarySplitterPercentage0, float SecondarySplitterPercentage1) override;
};
