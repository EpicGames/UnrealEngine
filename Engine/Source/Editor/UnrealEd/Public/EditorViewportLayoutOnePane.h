// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetEditorViewportLayout.h"
#include "Templates/SharedPointer.h"

class SHorizontalBox;
class SWidget;
class FString;
class FName;

class FEditorViewportLayoutOnePane : public FAssetEditorViewportPaneLayout
{
public:
	// IAssetEditorViewportLayoutConfiguration overrides
	virtual TSharedRef<SWidget> MakeViewportLayout(TSharedPtr<FAssetEditorViewportLayout> InParentLayout, const FString& LayoutString) override;
	virtual const FName& GetLayoutTypeName() const override;
	virtual void SaveLayoutString(const FString& SpecificLayoutString) const override;
	virtual void ReplaceWidget(TSharedRef<SWidget> OriginalWidget, TSharedRef<SWidget> ReplacementWidget) override;

protected:
	/** The viewport widget parent box */
	TSharedPtr< SHorizontalBox > ViewportBox;
};
