// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetEditorViewportLayout.h"

class SSplitter;
class SWidget;
class FName;

class FEditorViewportLayoutThreePanes : public FAssetEditorViewportPaneLayout
{
public:
	/**
	 * Creates the viewports and splitter for the two panes vertical layout                   
	 */
	virtual TSharedRef<SWidget> MakeViewportLayout(TSharedPtr<FAssetEditorViewportLayout> InParentLayout, const FString& LayoutString) override;
	virtual void SaveLayoutString(const FString& SpecificLayoutString) const override;
	virtual void ReplaceWidget(TSharedRef<SWidget> OriginalWidget, TSharedRef<SWidget> ReplacementWidget) override;

protected:
	virtual TSharedRef<SWidget> MakeThreePanelWidget(
		const TSharedRef<SWidget>& ViewportKey0, const TSharedRef<SWidget>& ViewportKey1, const TSharedRef<SWidget>& ViewportKey2,
		float PrimarySplitterPercentage, float SecondarySplitterPercentage) = 0;

	/** The splitter widgets */
	TSharedPtr< SSplitter > PrimarySplitterWidget;
	TSharedPtr< SSplitter > SecondarySplitterWidget;
	FName PerspectiveViewportConfigKey;
};

// FEditorViewportLayoutThreePanesLeft /////////////////////////////

class FEditorViewportLayoutThreePanesLeft : public FEditorViewportLayoutThreePanes
{
public:
	virtual const FName& GetLayoutTypeName() const override;

protected:
	virtual TSharedRef<SWidget> MakeThreePanelWidget(
		const TSharedRef<SWidget>& ViewportKey0, const TSharedRef<SWidget>& ViewportKey1, const TSharedRef<SWidget>& ViewportKey2,
		float PrimarySplitterPercentage, float SecondarySplitterPercentage) override;
};


// FEditorViewportLayoutThreePanesRight /////////////////////////////

class FEditorViewportLayoutThreePanesRight : public FEditorViewportLayoutThreePanes
{
public:
	virtual const FName& GetLayoutTypeName() const override;

protected:
	virtual TSharedRef<SWidget> MakeThreePanelWidget(
		const TSharedRef<SWidget>& ViewportKey0, const TSharedRef<SWidget>& ViewportKey1, const TSharedRef<SWidget>& ViewportKey2,
		float PrimarySplitterPercentage, float SecondarySplitterPercentage) override;
};


// FEditorViewportLayoutThreePanesTop /////////////////////////////

class FEditorViewportLayoutThreePanesTop : public FEditorViewportLayoutThreePanes
{
public:
	virtual const FName& GetLayoutTypeName() const override;

protected:
	virtual TSharedRef<SWidget> MakeThreePanelWidget(
		const TSharedRef<SWidget>& ViewportKey0, const TSharedRef<SWidget>& ViewportKey1, const TSharedRef<SWidget>& ViewportKey2,
		float PrimarySplitterPercentage, float SecondarySplitterPercentage) override;
};


// FEditorViewportLayoutThreePanesBottom /////////////////////////////

class FEditorViewportLayoutThreePanesBottom : public FEditorViewportLayoutThreePanes
{
public:
	virtual const FName& GetLayoutTypeName() const override;

protected:
	virtual TSharedRef<SWidget> MakeThreePanelWidget(
		const TSharedRef<SWidget>& ViewportKey0, const TSharedRef<SWidget>& ViewportKey1, const TSharedRef<SWidget>& ViewportKey2,
		float PrimarySplitterPercentage, float SecondarySplitterPercentage) override;
};
