// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetEditorViewportLayout.h"
#include "Types/SlateEnums.h"

class SSplitter;
class SWidget;
class FString;
class FName;

template <EOrientation TOrientation>
class TEditorViewportLayoutTwoPanes : public FAssetEditorViewportPaneLayout
{
public:
	virtual TSharedRef<SWidget> MakeViewportLayout(TSharedPtr<FAssetEditorViewportLayout> InParentLayout, const FString& LayoutString) override;
	virtual void ReplaceWidget(TSharedRef<SWidget> OriginalWidget, TSharedRef<SWidget> ReplacementWidget) override;
	virtual void SaveLayoutString(const FString& SpecificLayoutString) const override;

private:
	/** The splitter widget */
	TSharedPtr< class SSplitter > SplitterWidget;
};

// FEditorViewportLayoutTwoPanesVert /////////////////////////////
class FEditorViewportLayoutTwoPanesVert : public TEditorViewportLayoutTwoPanes<EOrientation::Orient_Vertical>
{
public:
	virtual const FName& GetLayoutTypeName() const override;
};


// FEditorViewportLayoutTwoPanesHoriz /////////////////////////////
class FEditorViewportLayoutTwoPanesHoriz : public TEditorViewportLayoutTwoPanes<EOrientation::Orient_Horizontal>
{
public:
	virtual const FName& GetLayoutTypeName() const override;
};
