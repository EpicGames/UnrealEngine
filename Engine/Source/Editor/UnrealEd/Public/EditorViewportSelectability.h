// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/PrimitiveComponent.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "GenericPlatform/ICursor.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Misc/Optional.h"

#define UE_API UNREALED_API

class AActor;
class FCanvas;
class FEditorViewportClient;
class FLevelEditorViewportClient;
class FUICommandInfo;
class HHitProxy;
class ITypedElementWorldInterface;
class UTypedElementSelectionSet;
struct FConvexVolume;
struct FTypedElementHandle;
struct FViewportClick;

DECLARE_DELEGATE_RetVal(UWorld*, FOnGetWorld);
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnIsObjectSelectableInViewport, UObject* /*InObject*/);

/**
 * Manages level actor viewport selectability and hovered visual states.
 * Contains static methods to enable outside modules to implement their own management.
 */
class FEditorViewportSelectability
{
public:
	/** Default text to display in the viewport when selection is limited as a helpful reminder to the user. */
	static UE_API const FText DefaultLimitedSelectionText;

	/** Registers an additional viewport selectability predicate that is ANDed with the primary predicate. */
	static UE_API FDelegateHandle RegisterObjectSelectablePredicate(const FOnIsObjectSelectableInViewport& InPredicate);

	/** Unregisters a predicate previously registered with RegisterObjectSelectablePredicate. */
	static UE_API void UnregisterObjectSelectablePredicate(const FDelegateHandle InHandle);

	/** Evaluates the registered additional viewport selectability predicates for an object. */
	static UE_API bool PassesAdditionalObjectSelectablePredicates(UObject* const InObject);

	/** Evaluates the registered additional viewport selectability predicates for an hit proxy. */
	static UE_API bool PassesAdditionalObjectSelectablePredicates(HHitProxy* const InHitProxy);

	/** Updates a single primitive component's hovered state and visuals. */
	static UE_API void UpdatePrimitiveVisuals(const bool bInSelectedLimited, UPrimitiveComponent* const InPrimitive, const TOptional<FColor>& InColor = TOptional<FColor>());

	/** Updates a list of hovered primitive component's hovered state and visuals */
	static UE_API bool UpdateHoveredPrimitive(const bool bInSelectedLimited
		, UPrimitiveComponent* const InPrimitiveComponent
		, TMap<TWeakObjectPtr<UPrimitiveComponent>, TOptional<FColor>>& InOutHoveredPrimitiveComponents
		, const TFunctionRef<bool(UObject*)>& InSelectablePredicate);

	/** Updates an actors hovered state and visuals. */
	static UE_API bool UpdateHoveredActorPrimitives(const bool bInSelectedLimited
		, AActor* const InActor
		, TMap<TWeakObjectPtr<UPrimitiveComponent>, TOptional<FColor>>& InOutHoveredPrimitiveComponents
		, const TFunctionRef<bool(UObject*)>& InSelectablePredicate);

	static UE_API FText GetLimitedSelectionText(const TSharedPtr<FUICommandInfo>& InToggleAction, const FText& InDefaultText = DefaultLimitedSelectionText);

	static UE_API void DrawEnabledTextNotice(FCanvas* const InCanvas, const FText& InText);

	FEditorViewportSelectability() = delete;
	UE_API FEditorViewportSelectability(const FOnGetWorld& InOnGetWorld, const FOnIsObjectSelectableInViewport& InOnIsObjectSelectableInViewport);

	/** Enables or disables the selectability tool */
	UE_API void EnableLimitedSelection(const bool bInEnabled);

	bool IsSelectionLimited() const
    {
    	return bSelectionLimited;
    }

	/** @return True if the specified object is selectable in the viewport and not made unselectable by the Sequencer selection limiting. */
	UE_API bool IsObjectSelectableInViewport(UObject* const InObject) const;

	/** Updates hover visual states based on current selection limiting settings */
	UE_API void UpdateSelectionLimitedVisuals(const bool bInClearHovered);

	UE_API void DeselectNonSelectableActors();

	UE_API bool GetCursorForHovered(EMouseCursor::Type& OutCursor) const;

	UE_API void UpdateHoverFromHitProxy(HHitProxy* const InHitProxy);

	UE_API bool HandleClick(FEditorViewportClient* const InViewportClient, HHitProxy* const InHitProxy, const FViewportClick& InClick);

	UE_API void StartTracking(FEditorViewportClient* const InViewportClient, FViewport* const InViewport);
	UE_API void EndTracking(FEditorViewportClient* const InViewportClient, FViewport* const InViewport);

	/** Selects or deselects all actors in a level world that are inside a defined box. */
	UE_API bool BoxSelectWorldActors(FBox& InBox, FEditorViewportClient* const InEditorViewportClient, const bool bInSelect);
	/** Selects or deselects all actors in a level world that are inside a defined convex volume. */
	UE_API bool FrustumSelectWorldActors(const FConvexVolume& InFrustum, FEditorViewportClient* const InEditorViewportClient, const bool bInSelect);

protected:
	static UE_API TMap<FDelegateHandle, FOnIsObjectSelectableInViewport>& GetAdditionalObjectSelectablePredicates();

	static UE_API bool IsActorSelectableClass(const AActor& InActor);

	static UE_API bool IsActorInLevelHiddenLayer(const AActor& InActor, FLevelEditorViewportClient* const InLevelEditorViewportClient);

	static UE_API TTypedElement<ITypedElementWorldInterface> GetTypedWorldElementFromActor(const AActor& InActor);

	/**
	 * Selects or deselects actors in a world. If no actors are specified, uses all the actors in the level.
	 * 
	 * @param InPredicate Function to use to check if the actor should be selected/deselected
	 * @param InActors Optional list of actors to selected/deselected
	 * @param bInSelect If true, selects the actors. If false, deselects the actors
	 * @param bInClearSelection If true, clears the current selection before selecting the new actors
	 * @return True if atleast one new actor was selected/deselected
	 */
	static UE_API bool SelectActorsByPredicate(UWorld* const InWorld
		, const bool bInSelect
		, const bool bInClearSelection
		, const TFunctionRef<bool(AActor*)> InPredicate
		, const TArray<AActor*>& InActors = {});

	/** Updates an actors hovered state and visuals. */
	UE_API void UpdateHoveredActorPrimitives(AActor* const InActor);

	static UE_API UTypedElementSelectionSet* GetLevelEditorSelectionSet();

	UE_API bool IsTypedElementSelectable(const FTypedElementHandle& InElementHandle) const;

	UE_API void GetSelectionElements(AActor* const InActor, const TFunctionRef<void(const TTypedElement<ITypedElementWorldInterface>&)> InPredicate);

	bool bSelectionLimited = false;

	/** Hovered primitives and their last overlay color before we apply the hover overlay */
	TMap<TWeakObjectPtr<UPrimitiveComponent>, TOptional<FColor>> HoveredPrimitiveComponents;

	/** Mouse cursor to display for the viewport when selection is limited. */
	TOptional<EMouseCursor::Type> MouseCursor;

	FOnGetWorld OnGetWorld;

	/** Delegate used to check if an object is selectable in the viewport */
	FOnIsObjectSelectableInViewport OnIsObjectSelectableInViewportDelegate;

	FVector DragStartPosition;
	FVector DragEndPosition;
	FIntRect DragSelectionRect;
};

#undef UE_API
