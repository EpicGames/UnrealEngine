// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"

#define UE_API UNREALED_API

class FEditorViewportClient;

DECLARE_DELEGATE_RetVal(bool, FOnIsViewportSelectionLimited);
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnIsObjectSelectableInViewport, UObject* /*InObject*/);
DECLARE_DELEGATE_RetVal(FText, FOnGetViewportSelectionLimitedText);

/**
 * Creates a link between a viewport and an outside module without requiring extra dependencies.
 * This could be moved to Editor/UnrealEd module to allow other modules that may need this functionality to access.
 */
class FEditorViewportSelectabilityBridge
{
public:
	FEditorViewportSelectabilityBridge() = delete;
	UE_API FEditorViewportSelectabilityBridge(const TWeakPtr<FEditorViewportClient>& InEditorViewportClientWeak);

	/** @return Delegate used to check if viewport selection is limited */
	UE_API FOnIsViewportSelectionLimited& OnIsViewportSelectionLimited();
	UE_API bool IsViewportSelectionLimited() const;

	/** @return Delegate used to check if an object is selectable in the viewport */
	UE_API FOnIsObjectSelectableInViewport& OnGetIsObjectSelectableInViewport();
	/** @return True if the specified object is selectable in the viewport and not made unselectable by the Sequencer selection limiting */
	UE_API bool IsObjectSelectableInViewport(UObject* const InObject) const;

	/** @return Delegate used to get the text to display in the viewport when selection is limited */
	UE_API FOnGetViewportSelectionLimitedText& OnGetViewportSelectionLimitedText();
	UE_API FText GetViewportSelectionLimitedText() const;

private:
	TWeakPtr<FEditorViewportClient> EditorViewportClientWeak;

	FOnIsViewportSelectionLimited IsViewportSelectionLimitedDelegate;
	FOnIsObjectSelectableInViewport GetIsObjectSelectableInViewportDelegate;
	FOnGetViewportSelectionLimitedText GetViewportSelectionLimitedTextDelegate;
};

#undef UE_API
