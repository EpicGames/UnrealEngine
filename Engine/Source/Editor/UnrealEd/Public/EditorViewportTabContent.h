// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Docking/SDockTab.h"
#include "ViewportTabContent.h"
#include "AssetEditorViewportLayout.h"

class FEditorViewportLayout;
class SDockTab;

/**
 * Represents the content in a viewport tab in an editor.
 * Each SDockTab holding viewports in an editor contains and owns one of these.
 */
class FEditorViewportTabContent : public FViewportTabContent, public TSharedFromThis<FEditorViewportTabContent>
{
public:
	/** @return The string used to identify the layout of this viewport tab */
	const FString& GetLayoutString() const
	{
		return LayoutString;
	}
	
	const TSharedPtr<FEditorViewportLayout> GetActiveViewportLayout() const
	{
		return ActiveViewportLayout;
	}
	   
	UNREALED_API TSharedPtr<FEditorViewportLayout> ConstructViewportLayoutByTypeName(const FName& TypeName, bool bSwitchingLayouts);

	UNREALED_API virtual void Initialize(AssetEditorViewportFactoryFunction Func, TSharedPtr<SDockTab> InParentTab, const FString& InLayoutString);

	UNREALED_API TSharedPtr<SAssetEditorViewport> CreateSlateViewport(FName InTypeName, const FAssetEditorViewportConstructionArgs& ConstructionArgs) const;

	/**
	* Sets the current layout by changing the contained layout object
	* 
	* @param ConfigurationName		The name of the layout (for the names in namespace EditorViewportConfigurationNames)
	*/
	UNREALED_API void SetViewportConfiguration(const FName& ConfigurationName) override;

	UNREALED_API TSharedPtr<SEditorViewport> GetFirstViewport();

	UNREALED_API void UpdateViewportTabWidget();

	/** Save any configuration required to persist state for this viewport layout */
	UNREALED_API void SaveConfig() const;

	/**
	 * Refresh the current layout.
	 * @note Does not save config
	 */
	UNREALED_API void RefreshViewportConfiguration();

	/**
	 * Attempt to find a viewport creation factory function in the set of registered factory functions.
	 * 
	 * @param InTypeName The name of the viewport creation factory to find.
	 * @returns the creation function if one is found with the given typename, otherwise nullptr.
	 */
	UNREALED_API const AssetEditorViewportFactoryFunction* FindViewportCreationFactory(FName InTypeName) const;

protected:
	UNREALED_API virtual TSharedPtr<FEditorViewportLayout> FactoryViewportLayout(bool bIsSwitchingLayouts);
	UNREALED_API virtual FName GetLayoutTypeNameFromLayoutString() const;

private:
	using AssetEditorViewportCreationFactories = TMap<FName, AssetEditorViewportFactoryFunction>;
	AssetEditorViewportCreationFactories ViewportCreationFactories;
};
