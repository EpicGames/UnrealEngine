// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FUICommandInfo;
class IEditorViewportLayoutEntity;
struct FAssetEditorViewportConstructionArgs;

/** Definition of a custom viewport */
struct FEditorViewportTypeDefinition
{
	typedef TFunction<TSharedRef<IEditorViewportLayoutEntity>(const FAssetEditorViewportConstructionArgs&)> FFactoryFunctionType;

	template<typename T>
	static FEditorViewportTypeDefinition FromType(const TSharedPtr<FUICommandInfo>& ActivationCommand)
	{
		return FViewportTypeDefinition([](const FAssetEditorViewportConstructionArgs& Args) -> TSharedRef<IEditorViewportLayoutEntity> {
			return MakeShareable(new T(Args));
		}, ActivationCommand);
	}

	FEditorViewportTypeDefinition(const FFactoryFunctionType& InFactoryFunction, const TSharedPtr<FUICommandInfo>& InActivationCommand)
		: ActivationCommand(InActivationCommand)
		, FactoryFunction(InFactoryFunction)
	{}

	/** A UI command for toggling activation this viewport */
	TSharedPtr<FUICommandInfo> ActivationCommand;

	/** Function used to create a new instance of the viewport */
	FFactoryFunctionType FactoryFunction;
};
