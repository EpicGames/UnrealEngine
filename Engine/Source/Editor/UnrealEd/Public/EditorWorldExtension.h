// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Templates/SubclassOf.h"
#include "Templates/Casts.h"
#include "Engine/EngineBaseTypes.h"
#include "UObject/GCObject.h"
#include "EditorWorldExtension.generated.h"

struct FWorldContext;
class UEditorWorldExtensionCollection;
class UEditorWorldExtensionManager;
class UEditorWorldExtension;
class FViewport;
class AActor;
class FEditorViewportClient;

enum class EEditorWorldExtensionTransitionState : uint8
{
	TransitionNone,
	TransitionAll,
	TransitionPIEOnly,
	TransitionNonPIEOnly
};

USTRUCT()
struct FEditorWorldExtensionActorData
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<AActor> Actor = nullptr;

	UPROPERTY()
	bool bValidForPIE = false;
};

UCLASS(MinimalAPI)
class UEditorWorldExtension : public UObject
{
	GENERATED_BODY()

	friend class UEditorWorldExtensionCollection;
public:
	
	/** Default constructor */
	UNREALED_API UEditorWorldExtension();

	/** Default destructor */
	UNREALED_API virtual ~UEditorWorldExtension();

	/** Initialize extension */
	virtual void Init() {};

	/** Shut down extension when world is destroyed */
	virtual void Shutdown() {};

	/** Give base class the chance to tick */
	virtual void Tick( float DeltaSeconds ) {};

	UNREALED_API virtual bool InputKey( FEditorViewportClient* InViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event);
	UNREALED_API virtual bool InputAxis( FEditorViewportClient* InViewportClient, FViewport* Viewport, int32 ControllerId, FKey Key, float Delta, float DeltaTime);

	/** Gets the world owning this extension */
	UNREALED_API virtual UWorld* GetWorld() const override;

	/** Gets the world owning this extension's non-PIE valid actors when current world is a play world */
	UNREALED_API virtual UWorld* GetLastEditorWorld() const;

	/**  Spawns a transient actor that we can use in the current world of this extension (templated for convenience) */
	template<class T>
	inline T* SpawnTransientSceneActor(const FString& ActorName, const bool bWithSceneComponent = false, const EObjectFlags InObjectFlags = EObjectFlags::RF_Transient | EObjectFlags::RF_DuplicateTransient, const bool bValidForPIE = false )
	{
		return CastChecked<T>(SpawnTransientSceneActor(T::StaticClass(), ActorName, bWithSceneComponent, InObjectFlags, bValidForPIE));
	}

	/** Spawns a transient actor that we can use in the current world of this extension */
	UNREALED_API AActor* SpawnTransientSceneActor(TSubclassOf<AActor> ActorClass, const FString& ActorName, const bool bWithSceneComponent = false, const EObjectFlags InObjectFlags = EObjectFlags::RF_Transient | EObjectFlags::RF_DuplicateTransient, const bool bValidForPIE = false);

	/** Destroys a transient actor we created earlier */
	UNREALED_API void DestroyTransientActor(AActor* Actor);

	/** Sets if this extension should be ticked. */
	UNREALED_API void SetActive(const bool bInActive);

	/** If this extension is currently being ticked. */
	UNREALED_API bool IsActive() const;

	/** Get the owning collection of extensions */
	UNREALED_API UEditorWorldExtensionCollection* GetOwningCollection();

	/** Executes command */
	UNREALED_API bool ExecCommand(const FString& InCommand);

protected:
	
	/** Reparent actors to a new world */
	UNREALED_API virtual void TransitionWorld(UWorld* NewWorld, EEditorWorldExtensionTransitionState TransitionState);

	/** Give child class a chance to act on entering simulate mode */
	virtual void EnteredSimulateInEditor() {};
	
	/** Give child class a chance to act on leaving simulate mode */
	virtual void LeftSimulateInEditor(UWorld* SimulateWorld) {};

	/** The collection of extensions that is owning this extension */
	UEditorWorldExtensionCollection* OwningExtensionsCollection;

private:

	/** Reparent the actors to a new world. */
	UNREALED_API void ReparentActor(AActor* Actor, UWorld* NewWorld);

	/** Let the FEditorWorldExtensionCollection set the world of this extension before init */
	UNREALED_API void InitInternal(UEditorWorldExtensionCollection* InOwningExtensionsCollection);

	UPROPERTY()
	TArray<FEditorWorldExtensionActorData> ExtensionActors;

	/** If this extension is currently being ticked */
	bool bActive;
};

/**
 * Holds a collection of UEditorExtension
 */
UCLASS(MinimalAPI)
class UEditorWorldExtensionCollection : public UObject
{
	GENERATED_BODY()

	friend class UEditorWorldExtensionManager;
public:

	/** Default constructor */
	UNREALED_API UEditorWorldExtensionCollection();

	/** Default destructor */
	UNREALED_API virtual ~UEditorWorldExtensionCollection();

	/** Gets the world from the world context */
	UNREALED_API virtual UWorld* GetWorld() const override;

	/** Gets the last editor world, will only be non-null when current world is a play world. */
	UNREALED_API UWorld* GetLastEditorWorld() const;

	/**
	 * Checks if the passed extension already exists and creates one if it doesn't.
	 * @param EditorExtensionClass the subclass of an extension to create if necessary and add.
	 */
	UNREALED_API UEditorWorldExtension* AddExtension(TSubclassOf<UEditorWorldExtension> EditorExtensionClass);

	/** 
	 * Adds an extension to the collection
	 * @param	EditorExtension			The UEditorExtension that will be created, initialized and added to the collection.
	 */
	UNREALED_API void AddExtension( UEditorWorldExtension* EditorExtension );
	
	/** 
	 * Removes an extension from the collection and calls Shutdown() on the extension
	 * @param	EditorExtension			The UEditorExtension to remove.  It must already have been added.
	 */
	UNREALED_API void RemoveExtension( UEditorWorldExtension* EditorExtension );

	/**
	 * Find an extension based on the class
	 * @param	EditorExtensionClass	The class to find an extension with
	 * @return							The first extension that is found based on class
	 */
	UNREALED_API UEditorWorldExtension* FindExtension( TSubclassOf<UEditorWorldExtension> EditorExtensionClass );

	/** Ticks all extensions */
	UNREALED_API void Tick( float DeltaSeconds );

	/** Notifies all extensions of keyboard input */
	UNREALED_API bool InputKey( FEditorViewportClient* InViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event);

	/** Notifies all extensions of axis movement */
	UNREALED_API bool InputAxis( FEditorViewportClient* InViewportClient, FViewport* Viewport, int32 ControllerId, FKey Key, float Delta, float DeltaTime);
	
	/** Show or hide all the actors of extensions that belong to this collection. */
	UNREALED_API void ShowAllActors(const bool bShow);

private:

	/** Sets the world for this collection and gives every extension an opportunity to transition */
	UNREALED_API void SetWorld(UWorld* NewWorld, EEditorWorldExtensionTransitionState TransitionState /* = EEditorWorldExtensionTransitionState::TransitionAll */);

	/** Transitions actors in every extension to the specified world */
	UNREALED_API void TransitionWorld(UWorld* NewWorld, EEditorWorldExtensionTransitionState TransitionState);

	/** Called by the editor after PIE or Simulate is started */
	UNREALED_API void PostPIEStarted( bool bIsSimulatingInEditor );

	/** Called just before PIE or Simulate ends */
	UNREALED_API void OnPreEndPIE(bool bWasSimulatingInEditor);

	/** Called when PIE or Simulate ends */
	UNREALED_API void OnEndPIE( bool bWasSimulatingInEditor );

	/** Called when switching between play and simulate */
	UNREALED_API void SwitchPIEAndSIE(bool bIsSimulatingInEditor);

	/** World context */
	TWeakObjectPtr<UWorld> Currentworld;

	/** After entering Simulate or PIE, this stores the counterpart editor world to the play world, so that we
        know this collection needs to transition back to editor world after Simulate or PIE finishes. */
	TWeakObjectPtr<UWorld> LastEditorWorld;

	/** List of extensions along with their reference count.  Extensions will only be truly removed and Shutdown() after their
	    reference count drops to zero. */
	typedef TTuple<UEditorWorldExtension*, int32> FEditorExtensionTuple;
	TArray<FEditorExtensionTuple> EditorExtensions;
};

/**
 * Holds a map of extension collections paired with worlds
 */
UCLASS(MinimalAPI)
class UEditorWorldExtensionManager	: public UObject
{
	GENERATED_BODY()

public:
	/** Default constructor */
	UNREALED_API UEditorWorldExtensionManager();

	/** Default destructor */
	UNREALED_API virtual ~UEditorWorldExtensionManager();

	/** Gets the editor world wrapper that is found with the world passed.
	 * Adds one for this world if there was non found. */
	UNREALED_API UEditorWorldExtensionCollection* GetEditorWorldExtensions(UWorld* InWorld, const bool bCreateIfNeeded = true);

	/** Ticks all the collections */
	UNREALED_API void Tick( float DeltaSeconds );

private:

	/** Adds a new editor world wrapper when a new world context was created */
	UEditorWorldExtensionCollection* OnWorldAdd(UWorld* World);

	/** Adds a new editor world wrapper when a new world context was created */
	void OnWorldContextRemove(FWorldContext& InWorldContext);

	UEditorWorldExtensionCollection* FindExtensionCollection(const UWorld* InWorld);

	/** Map of all the editor world maps */
	UPROPERTY()
	TArray<TObjectPtr<UEditorWorldExtensionCollection>> EditorWorldExtensionCollection;
};
