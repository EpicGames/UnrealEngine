// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	EditorWorldUtils.h: Editor-specific world management utilities
=============================================================================*/

#pragma once

#include "Engine/World.h"

/**
 * A helper RAII class to initialize / destroy an editor world.
 * The world will be added to the root set and initialized as an editor world using the
 * provided initialization values. 
 * On destruction, will destroy the world and unroot it.
 * This class will also set GWorld & the EditorWorldContext to this world.
 */
class FScopedEditorWorld
{
public:
	/**
	 * Constructor - Initialize the provided world as an editor world.
	 * @param InWorld					The world to manage.
	 * @param InInitializationValues	The initialization values to use for the world.
	 */
	UNREALED_API FScopedEditorWorld(UWorld* InWorld, const UWorld::InitializationValues& InInitializationValues, EWorldType::Type InWorldType = EWorldType::Editor);

	/**
	 * FStringView Constructor - Load the specified package & initialize the world as an editor world.
	 * @param InLongPackageName			Path to a package containing a world.
	 * @param InInitializationValues	The initialization values to use for the world.
	 */
	UNREALED_API FScopedEditorWorld(const FStringView InLongPackageName, const UWorld::InitializationValues& InInitializationValues, EWorldType::Type InWorldType = EWorldType::Editor);

	/**
	 * SoftObjectPtr Constructor - Initialize the provided world as an editor world.
	 * @param InWorld					World soft object pointer.
	 * @param InInitializationValues	The initialization values to use for the world.
	 */
	UNREALED_API FScopedEditorWorld(const TSoftObjectPtr<UWorld>& InSoftWorld, const UWorld::InitializationValues& InInitializationValues, EWorldType::Type InWorldType = EWorldType::Editor);

	/**
	 * Destructor - Destroy the provided world.
	 */
	UNREALED_API ~FScopedEditorWorld();

	/**
	 * Obtain the world managed by this scope, or null if the initialization failed.
	 */
	UNREALED_API UWorld* GetWorld() const;

private:
	FScopedEditorWorld();
	void Init(UWorld* InWorld, const UWorld::InitializationValues& InInitializationValues, EWorldType::Type InWorldType = EWorldType::Editor);

	UWorld* World;
	UWorld* PrevGWorld;
};

/**
 * Load a world package, managing the WorldTypePreLoadMap to ensure the correct world type is specified in UWorld::PostLoad()
 */
UNREALED_API UPackage* LoadWorldPackageForEditor(const FStringView InLongPackageName, EWorldType::Type InWorldType = EWorldType::Editor, uint32 InLoadFlags = LOAD_None);
