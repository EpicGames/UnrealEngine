// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDetailsInterface.h"
#include "Templates/UniquePtr.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ActorElementDetailsInterface.generated.h"

struct FTypedElementHandle;

UCLASS(MinimalAPI)
class UActorElementDetailsInterface : public UObject, public ITypedElementDetailsInterface
{
	GENERATED_BODY()

public:
	UNREALED_API virtual TUniquePtr<ITypedElementDetailsObject> GetDetailsObject(const FTypedElementHandle& InElementHandle) override;
};
