// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Elements/Actor/ActorElementAssetDataInterface.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ActorElementEditorAssetDataInterface.generated.h"

class UObject;
struct FAssetData;
struct FTypedElementHandle;

UCLASS(MinimalAPI)
class UActorElementEditorAssetDataInterface : public UActorElementAssetDataInterface
{
	GENERATED_BODY()

public:
	UNREALED_API virtual TArray<FAssetData> GetAllReferencedAssetDatas(const FTypedElementHandle& InElementHandle, const FTypedElementAssetDataReferencedOptions& InOptions) override;
};
