// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementWorldInterface.h"

#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "CoreMinimal.h"
#include "Exporters/Exporter.h"
#include "GameFramework/Actor.h"
#include "UObject/UnrealType.h"

#include "ActorElementEditorCopyAndPaste.generated.h"

class AActor;
class AGroupActor;

UCLASS(BlueprintType, MinimalAPI)
class UActorElementsCopy : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadWrite, Category = "TypedElement|Actor|Copy")
	TArray<TObjectPtr<AActor>> ActorsToCopy;
};

UCLASS(MinimalAPI)
class UActorElementsExporterT3D : public UExporter
{
public:
	GENERATED_BODY()

public:
	UNREALED_API UActorElementsExporterT3D(const FObjectInitializer& ObjectInitializer = FObjectInitializer());

	//~ Begin UExporter Interface
	UNREALED_API virtual bool ExportText(const FExportObjectInnerContext* Context, UObject* Object, const TCHAR* Type, FOutputDevice& Out, FFeedbackContext* Warn, uint32 PortFlags = 0) override;
	//~ End UExporter Interface
};

struct FActorElementEditorPasteImporter : public FWorldElementPasteImporter
{
	UNREALED_API virtual void Import(FContext& Context) override;
	
	UNREALED_API virtual TArray<FTypedElementHandle> GetImportedElements() override;
	
private:
	void PostImportProcess();
	
	FContext Context;

	struct FImportedActor
	{
		AActor* ActorPtr = nullptr;
		FStringView InnerText;

		UClass* Class;
		FName Name;
		FString Archetype;
		FName ParentActor;
		FName SocketNameString;
		FString GroupActor;
		FString GroupFolder;
		FString ActorFolderPath;
		FString ExportedActorFullName;
	};

	TArray<FImportedActor> ImportedActors;

	// Handle the attachment
	TMap<FName, AActor*> NameToImportedActors;

	// Handle the groups
	TMap<FString, AGroupActor*> NewGroups;

	bool bPostImportProcessingNeeded;
};
