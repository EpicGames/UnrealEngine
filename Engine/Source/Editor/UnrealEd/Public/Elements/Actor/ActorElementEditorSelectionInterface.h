// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Actor/ActorElementSelectionInterface.h"
#include "Elements/Framework/TypedElementListFwd.h"
#include "Templates/UniquePtr.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ActorElementEditorSelectionInterface.generated.h"

class AActor;
class ITypedElementTransactedElement;
class UObject;
struct FTypedElementHandle;
struct FTypedElementIsSelectedOptions;

UCLASS(MinimalAPI)
class UActorElementEditorSelectionInterface : public UActorElementSelectionInterface
{
	GENERATED_BODY()

public:
	UNREALED_API virtual bool SelectElement(const FTypedElementHandle& InElementHandle, const FTypedElementListPtr& InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions) override;
	UNREALED_API virtual bool DeselectElement(const FTypedElementHandle& InElementHandle, const FTypedElementListPtr& InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions) override;
	UNREALED_API virtual bool IsElementSelected(const FTypedElementHandle& InElementHandle, const FTypedElementListConstPtr& SelectionSetPtr, const FTypedElementIsSelectedOptions& InSelectionOptions) override;
	UNREALED_API virtual bool ShouldPreventTransactions(const FTypedElementHandle& InElementHandle) override;
	UNREALED_API virtual TUniquePtr<ITypedElementTransactedElement> CreateTransactedElementImpl() override;

	static UNREALED_API bool IsActorSelected(const AActor* InActor, FTypedElementListConstRef InSelectionSet, const FTypedElementIsSelectedOptions& InSelectionOptions);
};
