// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementViewportInteraction.h"
#include "Elements/Framework/TypedElementAssetEditorToolkitHostMixin.h"

class AActor;

class FActorElementEditorViewportInteractionCustomization : public FTypedElementViewportInteractionCustomization
{
public:
	UNREALED_API virtual void GizmoManipulationStarted(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const UE::Widget::EWidgetMode InWidgetMode) override;
	UNREALED_API virtual void GizmoManipulationDeltaUpdate(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const UE::Widget::EWidgetMode InWidgetMode, const EAxisList::Type InDragAxis, const FInputDeviceState& InInputState, const FTransform& InDeltaTransform, const FVector& InPivotLocation) override;
	UNREALED_API virtual void GizmoManipulationStopped(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const UE::Widget::EWidgetMode InWidgetMode, const ETypedElementViewportInteractionGizmoManipulationType InManipulationType) override;
	UNREALED_API virtual void MirrorElement(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const FVector& InMirrorScale, const FVector& InPivotLocation) override;
	UNREALED_API virtual bool GetFocusBounds(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, FBoxSphereBounds& OutBounds) override;
	
	UNREALED_API static void ApplyDeltaToActor(AActor* InActor, const bool InIsDelta, const FVector* InDeltaTranslationPtr, const FRotator* InDeltaRotationPtr, const FVector* InDeltaScalePtr, const FVector& InPivotLocation, const FInputDeviceState& InInputState);
};
