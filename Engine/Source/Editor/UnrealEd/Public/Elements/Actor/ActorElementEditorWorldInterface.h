// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Actor/ActorElementWorldInterface.h"
#include "ActorElementEditorWorldInterface.generated.h"

UCLASS(MinimalAPI)
class UActorElementEditorWorldInterface : public UActorElementWorldInterface
{
	GENERATED_BODY()

public:
	UNREALED_API virtual bool GetPivotOffset(const FTypedElementHandle& InElementHandle, FVector& OutPivotOffset) override;
	UNREALED_API virtual bool SetPivotOffset(const FTypedElementHandle& InElementHandle, const FVector& InPivotOffset) override;
	UNREALED_API virtual void NotifyMovementStarted(const FTypedElementHandle& InElementHandle) override;
	UNREALED_API virtual void NotifyMovementOngoing(const FTypedElementHandle& InElementHandle) override;
	UNREALED_API virtual void NotifyMovementEnded(const FTypedElementHandle& InElementHandle) override;
	UNREALED_API virtual bool CanDeleteElement(const FTypedElementHandle& InElementHandle) override;
	UNREALED_API virtual bool DeleteElements(TArrayView<const FTypedElementHandle> InElementHandles, UWorld* InWorld, UTypedElementSelectionSet* InSelectionSet, const FTypedElementDeletionOptions& InDeletionOptions) override;
	UNREALED_API virtual bool CanDuplicateElement(const FTypedElementHandle& InElementHandle) override;
	UNREALED_API virtual void DuplicateElements(TArrayView<const FTypedElementHandle> InElementHandles, UWorld* InWorld, const FVector& InLocationOffset, TArray<FTypedElementHandle>& OutNewElements) override;
	UNREALED_API virtual bool CanCopyElement(const FTypedElementHandle& InElementHandle) override;
	UNREALED_API virtual void CopyElements(TArrayView<const FTypedElementHandle> InElementHandles, FOutputDevice& Out) override;
	UNREALED_API virtual TSharedPtr<FWorldElementPasteImporter> GetPasteImporter() override;
	UNREALED_API virtual bool IsElementInConvexVolume(const FTypedElementHandle& Handle, const FConvexVolume& InVolume, bool bMustEncompassEntireElement = false);
	UNREALED_API virtual bool IsElementInBox(const FTypedElementHandle& Handle, const FBox& InBox, bool bMustEncompassEntireElement = false);
	UNREALED_API virtual TArray<FTypedElementHandle> GetSelectionElementsFromSelectionFunction(const FTypedElementHandle& InElementHandle, const FWorldSelectionElementArgs& SelectionArgs, const TFunction<bool(const FTypedElementHandle&, const FWorldSelectionElementArgs&)>& SelectionFunction) override;
};
