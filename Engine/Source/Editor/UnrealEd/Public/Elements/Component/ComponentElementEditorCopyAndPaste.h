// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementWorldInterface.h"

#include "CoreMinimal.h"
#include "Exporters/Exporter.h"
#include "Components/ActorComponent.h"

#include "ComponentElementEditorCopyAndPaste.generated.h"

UCLASS(BlueprintType, MinimalAPI)
class UComponentElementsCopy : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadWrite, Category = "TypedElement|Component|Copy")
	TArray<TObjectPtr<UActorComponent>> ComponentsToCopy;
};

UCLASS(MinimalAPI)
class UComponentElementsExporterT3D : public UExporter
{
public:
	GENERATED_BODY()

public:
	UNREALED_API explicit UComponentElementsExporterT3D(const FObjectInitializer& ObjectInitializer = FObjectInitializer());

	//~ Begin UExporter Interface
	UNREALED_API virtual bool ExportText(const FExportObjectInnerContext* Context, UObject* Object, const TCHAR* Type, FOutputDevice& Out, FFeedbackContext* Warn, uint32 PortFlags = 0) override;
	//~ End UExporter Interface
};

struct FComponentElementEditorPasteImporter : public FWorldElementPasteImporter
{
public:
	UNREALED_API virtual void Import(FContext& Context) override;
	
	UNREALED_API virtual TArray<FTypedElementHandle> GetImportedElements() override;
private:
	TArray<TObjectPtr<UActorComponent>> ImportedComponents;
};
