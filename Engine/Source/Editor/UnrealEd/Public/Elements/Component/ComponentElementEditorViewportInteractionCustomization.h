// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementViewportInteraction.h"
#include "Elements/Framework/TypedElementAssetEditorToolkitHostMixin.h"

class FProperty;
class USceneComponent;

class FComponentElementEditorViewportInteractionCustomization : public FTypedElementViewportInteractionCustomization
{
public:
	UNREALED_API virtual bool GetGizmoPivotLocation(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const UE::Widget::EWidgetMode InWidgetMode, FVector& OutPivotLocation) override;
	UNREALED_API virtual void GizmoManipulationDeltaUpdate(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const UE::Widget::EWidgetMode InWidgetMode, const EAxisList::Type InDragAxis, const FInputDeviceState& InInputState, const FTransform& InDeltaTransform, const FVector& InPivotLocation) override;
	UNREALED_API virtual bool GetFocusBounds(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, FBoxSphereBounds& OutBounds) override;
	UNREALED_API virtual bool GetScaleForGizmo(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, FVector& OutScale) override;
	
	static UNREALED_API void ApplyDeltaToComponent(USceneComponent* InComponent, const bool InIsDelta, const FVector* InDeltaTranslationPtr, const FRotator* InDeltaRotationPtr, const FVector* InDeltaScalePtr, const FVector& InPivotLocation, const FInputDeviceState& InInputState);
};
