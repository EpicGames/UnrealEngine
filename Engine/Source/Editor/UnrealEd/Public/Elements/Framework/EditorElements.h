// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

extern UNREALED_API FSimpleMulticastDelegate OnRegisterEditorElementsDelegate;

void RegisterEditorElements();
