// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Object/ObjectElementSelectionInterface.h"
#include "Templates/UniquePtr.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ObjectElementEditorSelectionInterface.generated.h"

class ITypedElementTransactedElement;
class UObject;
struct FTypedElementHandle;

UCLASS(MinimalAPI)
class UObjectElementEditorSelectionInterface : public UObjectElementSelectionInterface
{
	GENERATED_BODY()

public:
	UNREALED_API virtual bool ShouldPreventTransactions(const FTypedElementHandle& InElementHandle) override;
	UNREALED_API virtual TUniquePtr<ITypedElementTransactedElement> CreateTransactedElementImpl() override;

	static UNREALED_API bool ShouldObjectPreventTransactions(const UObject* InObject);
};
