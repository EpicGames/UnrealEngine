// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDetailsInterface.h"
#include "SMInstanceElementDetailsInterface.generated.h"

UCLASS(MinimalAPI)
class USMInstanceElementDetailsInterface : public UObject, public ITypedElementDetailsInterface
{
	GENERATED_BODY()

public:
	UNREALED_API virtual TUniquePtr<ITypedElementDetailsObject> GetDetailsObject(const FTypedElementHandle& InElementHandle) override;
};
