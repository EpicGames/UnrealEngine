// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/SMInstance/SMInstanceElementSelectionInterface.h"
#include "SMInstanceElementEditorSelectionInterface.generated.h"

UCLASS(MinimalAPI)
class USMInstanceElementEditorSelectionInterface : public USMInstanceElementSelectionInterface
{
	GENERATED_BODY()

public:
	UNREALED_API virtual bool IsElementSelected(const FTypedElementHandle& InElementHandle, const FTypedElementListConstPtr& InSelectionSet, const FTypedElementIsSelectedOptions& InSelectionOptions) override;
	UNREALED_API virtual bool SelectElement(const FTypedElementHandle& InElementHandle, const FTypedElementListPtr& InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions) override;
	UNREALED_API virtual bool DeselectElement(const FTypedElementHandle& InElementHandle, const FTypedElementListPtr& InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions) override;
	UNREALED_API virtual bool ShouldPreventTransactions(const FTypedElementHandle& InElementHandle) override;
	UNREALED_API virtual TUniquePtr<ITypedElementTransactedElement> CreateTransactedElementImpl() override;
};
