// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/SMInstance/SMInstanceElementWorldInterface.h"
#include "SMInstanceElementEditorWorldInterface.generated.h"

UCLASS(MinimalAPI)
class USMInstanceElementEditorWorldInterface : public USMInstanceElementWorldInterface
{
	GENERATED_BODY()

public:
	UNREALED_API virtual bool CanDeleteElement(const FTypedElementHandle& InElementHandle) override;
	UNREALED_API virtual bool DeleteElement(const FTypedElementHandle& InElementHandle, UWorld* InWorld, UTypedElementSelectionSet* InSelectionSet, const FTypedElementDeletionOptions& InDeletionOptions) override;
	UNREALED_API virtual bool DeleteElements(TArrayView<const FTypedElementHandle> InElementHandles, UWorld* InWorld, UTypedElementSelectionSet* InSelectionSet, const FTypedElementDeletionOptions& InDeletionOptions) override;
	UNREALED_API virtual bool CanDuplicateElement(const FTypedElementHandle& InElementHandle) override;
	UNREALED_API virtual FTypedElementHandle DuplicateElement(const FTypedElementHandle& InElementHandle, UWorld* InWorld, const FVector& InLocationOffset) override;
	UNREALED_API virtual void DuplicateElements(TArrayView<const FTypedElementHandle> InElementHandles, UWorld* InWorld, const FVector& InLocationOffset, TArray<FTypedElementHandle>& OutNewElements) override;
	UNREALED_API virtual bool CanPromoteElement(const FTypedElementHandle& InElementHandle) override;
	UNREALED_API virtual FTypedElementHandle PromoteElement(const FTypedElementHandle& InElementHandle, UWorld* OverrideWorld /* = nullptr */) override;
	UNREALED_API virtual bool IsElementInConvexVolume(const FTypedElementHandle& Handle, const FConvexVolume& InVolume, bool bMustEncompassEntireElement = false);
	UNREALED_API virtual bool IsElementInBox(const FTypedElementHandle& Handle, const FBox& InBox, bool bMustEncompassEntireElement = false);
};
