// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Factories.h: Unreal Engine factory types.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/Class.h"
#include "Animation/MorphTarget.h"
#include "UObject/ObjectInstancingGraph.h"

UNREALED_API DECLARE_LOG_CATEGORY_EXTERN(LogEditorFactories, Log, All);

class Error;
class USkeletalMesh;
class UStaticMesh;

/**  
 * This class is a simple customizable object factory driven from a text buffer.  
 * Subclasses need to implement CanCreateClass and ProcessConstructedObject.
 */
class FCustomizableTextObjectFactory
{
protected:
	FFeedbackContext* WarningContext;
	FObjectInstancingGraph InstanceGraph;
public:
	/** Constructor for the factory; takes a context for emitting warnings such as GWarn */
	UNREALED_API FCustomizableTextObjectFactory(FFeedbackContext* InWarningContext);
	virtual ~FCustomizableTextObjectFactory() = default;

	/**
	 *	Parse a text buffer and factories objects from it, subject to the restrictions imposed by CanCreateClass()
	 *
	 *	@param	InParent	Usually the parent sequence, but might be a package for example. Used as outer for created SequenceObjects.
	 *	@param	Flags		Flags used when creating AnimObjects
	 *	@param	TextBuffer	Text buffer with descriptions of nodes
	 */
	UNREALED_API void ProcessBuffer(UObject* InParent, EObjectFlags Flags, const FString& TextBuffer);

	UNREALED_API void ProcessBuffer(UObject* InParent, EObjectFlags Flags, const TCHAR* TextBuffer);

	/**
	 *	Determine if it is possible to create objects from the specified TextBuffer
	 */
	UNREALED_API bool CanCreateObjectsFromText(const FString& TextBuffer) const;

protected:
	/** Return true if the an object of type ObjectClass is allowed to be created; If false is returned, the object and subobjects will be ignored. */
	UNREALED_API virtual bool CanCreateClass(UClass* ObjectClass, bool& bOmitSubObjs) const;

	/** Return true if the an object of type ObjectClass is allowed to be created with a given name under the provided parent; If false is returned, the object and subobjects will be ignored. */
	UNREALED_API virtual bool CanCreateObject(UObject* InParent, UClass* ObjectClass, const FName& InDesiredName) const;

	/** This is called on each created object after the property text is imported */
	UNREALED_API virtual void ProcessConstructedObject(UObject* CreatedObject);

	/** Post handling of constructed objects by the factory */
	virtual void PostProcessConstructedObjects() {};

	/** Provide the opportunity to change the name during import */
	virtual void UpdateObjectName(UClass* ObjectClass, UObject* InParent, FName& InOutObjName) {};

	/** Util to ensure that InName is a valid name for a new object within InParent. Will rename any existing object within InParent if it is called InName. */
	static UNREALED_API void ClearObjectNameUsage(UObject* InParent, FName InName);

	/** If we cant do anything with the line ourselves hand off to child class */
	virtual void ProcessUnidentifiedLine(const FString& StrLine) {}

	/** Allow child class to override new object parent (only called when parent supplied to ProcessBuffer is NULL */
	virtual UObject* GetParentForNewObject(const UClass* ObjClass) {return NULL;}
};


/** morph target import error codes */
enum EMorphImportError
{
	// success
	MorphImport_OK=0,
	// target mesh exists
	MorphImport_AlreadyExists,
	// source file was not loaded
	MorphImport_CantLoadFile,
	// source file format is invalid
	MorphImport_InvalidMeshFormat,
	// source mesh vertex data doesn't match base
	MorphImport_MismatchBaseMesh,
	// source mesh is missing its metadata
	// needs to be reimported
	MorphImport_ReimportBaseMesh,
	// LOD index was out of range by more than 1
	MorphImport_InvalidLODIndex,
	// Missing morph target
	MorphImport_MissingMorphTarget,
	// max
	MorphImport_MAX
};

/**
 * Utility class for importing a new morph target
 */
class FMorphTargetBinaryImport
{
public:
	/** for outputing warnings */
	FFeedbackContext* Warn;
	/** base mesh lod entry to use */
	int32 BaseLODIndex;
	/** the base mesh */
	UObject* BaseMesh;

	FMorphTargetBinaryImport( USkeletalMesh* InSrcMesh, int32 LODIndex=0, FFeedbackContext* InWarn=GWarn );
	FMorphTargetBinaryImport( UStaticMesh* InSrcMesh, int32 LODIndex=0, FFeedbackContext* InWarn=GWarn );
	virtual ~FMorphTargetBinaryImport() = default;

    virtual USkeletalMesh* CreateSkeletalMesh(const TCHAR* SrcFilename, EMorphImportError* Error ) = 0;
};
