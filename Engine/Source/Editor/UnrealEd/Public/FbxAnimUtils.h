// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UAnimSequence;
class UCurveTable;
class USkeletalMesh;

namespace fbxsdk
{
	class FbxAnimCurve;
	class FbxNode;
	class FbxProperty;
}

//Define interface for exporting fbx animation
namespace FbxAnimUtils
{
	//Function to export fbx animation
	UNREALED_API void ExportAnimFbx( const FString& ExportFilename, UAnimSequence* AnimSequence, USkeletalMesh* Mesh, bool BatchMode, bool &OutExportAll, bool &OutCancelExport);

	/** 
	 * Import curves from a named node into the supplied curve table.
	 * @param	InFbxFilename	The FBX file to import from
	 * @param	InCurveNodeName	The name of the node in the FBX scene that contains our curves
	 * @param	InOutCurveTable	The curve table to fill with curves
	 * @return true if the import was successful
	 */
	UNREALED_API bool ImportCurveTableFromNode(const FString& InFbxFilename, const FString& InCurveNodeName, UCurveTable* InOutCurveTable, float& OutPreRoll);

	/** Helper function used to determine whether we can import a particular curve type. Note InDataType is EFbxType */
	UNREALED_API bool IsSupportedCurveDataType(int32 InDataType);

	/** 
	 * Helper function used to check whether we should import the specified curve.
	 * @param	Curve							The curve to check
	 * @param	bDoNotImportWithZeroValues		Whether to skip zero-valued curves
	 */
	UNREALED_API bool ShouldImportCurve(fbxsdk::FbxAnimCurve* Curve, bool bDoNotImportWithZeroValues);

	/**
	 * Helper function that extracts custom attribute curves from a specified Fbx node, omiting attributes directly applied to the bones.
	 * @param	InNode							The node to extract curves from
	 * @param	bInDoNotImportCurveWithZero		Whether to skip zero-valued curves
	 * @param	ImportFunction					Function called to perform operations on the extracted curve(s)
	 */
	UNREALED_API void ExtractAttributeCurves(fbxsdk::FbxNode* InNode, bool bInDoNotImportCurveWithZero, TFunctionRef<void(fbxsdk::FbxAnimCurve* /*InCurve*/, const FString& /*InCurveName*/)> ImportFunction);

	/**
	 * Helper function that extracts custom attribute from a specified Fbx node, selecting only attributes applied to the bones.
	 * @param	InNode							The node to extract custom attribute from
	 * @param	bInDoNotImportCurveWithZero		Whether to skip zero-valued curves
	 * @param	ImportFunction					Function called to perform operations on the extracted curve(s)
	 */
	UNREALED_API void ExtractNodeAttributes(fbxsdk::FbxNode* InNode, bool bInDoNotImportCurveWithZero, bool bImportAllCustomAttributes, TFunctionRef<void(fbxsdk::FbxProperty& /*InProperty*/, fbxsdk::FbxAnimCurve* /*InCurve*/, const FString& /*InCurveName*/)> ImportFunction);

} // namespace FbxAnimUtils
