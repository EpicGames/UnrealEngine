// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"
#include "Misc/SecureHash.h"
#include "Factories/FbxAnimSequenceImportData.h"
#include "Factories/FbxImportUI.h"
#include "Logging/TokenizedMessage.h"
#include "Factories/FbxStaticMeshImportData.h"
#include "Factories/FbxTextureImportData.h"
#include "Factories/FbxSceneImportFactory.h"
#include "Materials/MaterialInterface.h"
#include "MeshBuild.h"
#include "Algo/LevenshteinDistance.h"

class AActor;
class ACameraActor;
class ALight;
class Error;
class FSkeletalMeshImportData;
class UActorComponent;
class UAnimSequence;
class UFbxSkeletalMeshImportData;
class ULightComponent;
class UMaterial;
class UMaterialInstanceConstant;
class UPhysicsAsset;
class USkeletalMesh;
class USkeleton;
class UStaticMesh;
class UTexture;
struct FExpressionInput;
struct FRichCurve;
struct FStaticMaterial;
struct FSkeletalMaterial;

// Temporarily disable a few warnings due to virtual function abuse in FBX source files
#pragma warning( push )

#pragma warning( disable : 4263 ) // 'function' : member function does not override any base class virtual member function
#pragma warning( disable : 4264 ) // 'virtual_function' : no override available for virtual member function from base 'class'; function is hidden

// Include the fbx sdk header
// temp undef/redef of _O_RDONLY because kfbxcache.h (included by fbxsdk.h) does
// a weird use of these identifiers inside an enum.
#ifdef _O_RDONLY
#define TMP_UNFBX_BACKUP_O_RDONLY _O_RDONLY
#define TMP_UNFBX_BACKUP_O_WRONLY _O_WRONLY
#undef _O_RDONLY
#undef _O_WRONLY
#endif

//Robert G. : Packing was only set for the 64bits platform, but we also need it for 32bits.
//This was found while trying to trace a loop that iterate through all character links.
//The memory didn't match what the debugger displayed, obviously since the packing was not right.
#pragma pack(push,8)

#if PLATFORM_WINDOWS
// _CRT_SECURE_NO_DEPRECATE is defined but is not enough to suppress the deprecation
// warning for vsprintf and stricmp in VS2010.  Since FBX is able to properly handle the non-deprecated
// versions on the appropriate platforms, _CRT_SECURE_NO_DEPRECATE is temporarily undefined before
// including the FBX headers

// The following is a hack to make the FBX header files compile correctly under Visual Studio 2012 and Visual Studio 2013
#if _MSC_VER >= 1700
	#define FBX_DLL_MSC_VER 1600
#endif


#endif // PLATFORM_WINDOWS

// FBX casts null pointer to a reference
THIRD_PARTY_INCLUDES_START
#include <fbxsdk.h>
THIRD_PARTY_INCLUDES_END

#pragma pack(pop)




#ifdef TMP_UNFBX_BACKUP_O_RDONLY
#define _O_RDONLY TMP_FBX_BACKUP_O_RDONLY
#define _O_WRONLY TMP_FBX_BACKUP_O_WRONLY
#undef TMP_UNFBX_BACKUP_O_RDONLY
#undef TMP_UNFBX_BACKUP_O_WRONLY
#endif

#pragma warning( pop )

class FSkeletalMeshImportData;
class FSkelMeshOptionalImportData;
class ASkeletalMeshActor;
struct FbxSceneInfo;
struct FExistingStaticMeshData;

DECLARE_LOG_CATEGORY_EXTERN(LogFbx, Log, All);

#define DEBUG_FBX_NODE( Prepend, FbxNode ) FPlatformMisc::LowLevelOutputDebugStringf( TEXT("%s %s\n"), UTF8_TO_TCHAR(Prepend), UTF8_TO_TCHAR( FbxNode->GetName() ) )

#define FBX_METADATA_PREFIX TEXT("FBX.")

namespace UnFbx
{

enum EFBXReimportDialogReturnOption
{
	FBXRDRO_Ok,
	FBXRDRO_ResetToFbx,
	FBXRDRO_Cancel,
	FBXRDRO_MAX,
};

struct FBXImportOptions
{
	// General options
	bool bCanShowDialog;
	bool bIsImportCancelable;
	bool bImportScene;
	bool bImportAsSkeletalGeometry;
	bool bImportAsSkeletalSkinning;
	bool bImportMaterials;
	bool bInvertNormalMap;
	bool bImportTextures;
	bool bImportLOD;
	bool bUsedAsFullName;
	bool bConvertScene;
	bool bForceFrontXAxis;
	bool bConvertSceneUnit;
	bool bRemoveNameSpace;
	FVector ImportTranslation;
	FRotator ImportRotation;
	float ImportUniformScale;
	EFBXNormalImportMethod NormalImportMethod;
	EFBXNormalGenerationMethod::Type NormalGenerationMethod;
	bool bComputeWeightedNormals;
	bool bTransformVertexToAbsolute;
	bool bBakePivotInVertex;
	EFBXImportType ImportType;
	// Static Mesh options
	bool bCombineToSingle;
	EVertexColorImportOption::Type VertexColorImportOption;
	FColor VertexOverrideColor;
	float DistanceFieldResolutionScale;
	bool bRemoveDegenerates;
	bool bBuildReversedIndexBuffer;
	bool bBuildNanite;
	bool bGenerateLightmapUVs;
	bool bOneConvexHullPerUCX;
	bool bAutoGenerateCollision;
	FName StaticMeshLODGroup;
	bool bImportStaticMeshLODs;
	bool bAutoComputeLodDistances;
	TArray<float> LodDistances;
	int32 MinimumLodNumber;
	int32 LodNumber;
	// Material import options
	class UMaterialInterface *BaseMaterial;
	FString BaseColorName;
	FString BaseDiffuseTextureName;
	FString BaseEmissiveColorName;
	FString BaseNormalTextureName;
	FString BaseEmmisiveTextureName;
	FString BaseSpecularTextureName;
	FString BaseOpacityTextureName;
	EMaterialSearchLocation MaterialSearchLocation;
	//If true the materials will be reorder to follow the fbx order
	bool bReorderMaterialToFbxOrder;
	// Skeletal Mesh options
	bool bImportMorph;
	bool bImportVertexAttributes;
	bool bImportAnimations;
	bool bUpdateSkeletonReferencePose;
	bool bResample;
	int32 ResampleRate;
	bool bSnapToClosestFrameBoundary;
	bool bImportRigidMesh;
	bool bUseT0AsRefPose;
	bool bPreserveSmoothingGroups;
	bool bKeepSectionsSeparate;
	FOverlappingThresholds OverlappingThresholds;
	bool bImportMeshesInBoneHierarchy;
	bool bCreatePhysicsAsset;
	UPhysicsAsset *PhysicsAsset;
	bool bImportSkeletalMeshLODs;
	// Animation option
	USkeleton* SkeletonForAnimation;
	EFBXAnimationLengthImportType AnimationLengthImportType;
	FIntPoint AnimationRange;
	FString AnimationName;
	bool	bPreserveLocalTransform;
	bool	bDeleteExistingMorphTargetCurves;
	bool	bImportCustomAttribute;
	bool	bDeleteExistingCustomAttributeCurves;
	bool	bDeleteExistingNonCurveCustomAttributes;
	bool	bImportBoneTracks;
	bool	bSetMaterialDriveParameterOnCustomAttribute;
	bool	bAddCurveMetadataToSkeleton;
	bool	bRemoveRedundantKeys;
	bool	bDoNotImportCurveWithZero;
	bool	bResetToFbxOnMaterialConflict;
	TArray<FString> MaterialCurveSuffixes;

	/** This allow to add a prefix to the material name when unreal material get created.	
	*   This prefix can just modify the name of the asset for materials (i.e. TEXT("Mat"))
	*   This prefix can modify the package path for materials (i.e. TEXT("/Materials/")).
	*   Or both (i.e. TEXT("/Materials/Mat"))
	*/
	FName MaterialBasePath;

	//This data allow to override some fbx Material(point by the uint64 id) with existing unreal material asset
	TMap<uint64, class UMaterialInterface*> OverrideMaterials;

	bool ShouldImportNormals() const
	{
		return NormalImportMethod == FBXNIM_ImportNormals || NormalImportMethod == FBXNIM_ImportNormalsAndTangents;
	}

	bool ShouldImportTangents() const
	{
		return NormalImportMethod == FBXNIM_ImportNormalsAndTangents;
	}

	void ResetForReimportAnimation()
	{
		bImportMorph = true;
		AnimationLengthImportType = FBXALIT_ExportedTime;
	}

	static void ResetOptions(FBXImportOptions *OptionsToReset)
	{
		check(OptionsToReset != nullptr);
		*OptionsToReset = FBXImportOptions();
	}
};

#define INVALID_UNIQUE_ID 0xFFFFFFFFFFFFFFFF

class FFbxAnimCurveHandle
{
public:
	enum CurveTypeDescription
	{
		Transform_Translation_X,
		Transform_Translation_Y,
		Transform_Translation_Z,
		Transform_Rotation_X,
		Transform_Rotation_Y,
		Transform_Rotation_Z,
		Transform_Scaling_X,
		Transform_Scaling_Y,
		Transform_Scaling_Z,
		NotTransform,
	};

	FFbxAnimCurveHandle()
	{
		UniqueId = INVALID_UNIQUE_ID;
		Name.Empty();
		ChannelIndex = 0;
		CompositeIndex = 0;
		KeyNumber = 0;
		AnimationTimeSecond = 0.0f;
		AnimCurve = nullptr;
		CurveType = NotTransform;
	}

	//Identity Data
	uint64 UniqueId;
	FString Name;
	int32 ChannelIndex;
	int32 CompositeIndex;

	//Curve Information
	int32 KeyNumber;
	float AnimationTimeSecond;

	//Pointer to the curve data
	FbxAnimCurve* AnimCurve;

	CurveTypeDescription CurveType;
};

class FFbxAnimPropertyHandle
{
public:
	FFbxAnimPropertyHandle()
	{
		Name.Empty();
		DataType = eFbxFloat;
	}

	FFbxAnimPropertyHandle(const FFbxAnimPropertyHandle &PropertyHandle)
	{
		Name = PropertyHandle.Name;
		DataType = PropertyHandle.DataType;
		CurveHandles = PropertyHandle.CurveHandles;
	}

	FString Name;
	EFbxType DataType;
	TArray<FFbxAnimCurveHandle> CurveHandles;
};

class FFbxAnimNodeHandle
{
public:
	FFbxAnimNodeHandle()
	{
		UniqueId = INVALID_UNIQUE_ID;
		Name.Empty();
		AttributeUniqueId = INVALID_UNIQUE_ID;
		AttributeType = FbxNodeAttribute::eUnknown;
	}

	FFbxAnimNodeHandle(const FFbxAnimNodeHandle &NodeHandle)
	{
		UniqueId = NodeHandle.UniqueId;
		Name = NodeHandle.Name;
		AttributeUniqueId = NodeHandle.AttributeUniqueId;
		AttributeType = NodeHandle.AttributeType;
		NodeProperties = NodeHandle.NodeProperties;
		AttributeProperties = NodeHandle.AttributeProperties;
	}

	uint64 UniqueId;
	FString Name;
	TMap<FString, FFbxAnimPropertyHandle> NodeProperties;

	uint64 AttributeUniqueId;
	FbxNodeAttribute::EType AttributeType;
	TMap<FString, FFbxAnimPropertyHandle> AttributeProperties;
};

class FFbxCurvesAPI
{
public:
	FFbxCurvesAPI()
	{
		Scene = nullptr;
	}
	//Name API
	UNREALED_API void GetAllNodeNameArray(TArray<FString> &AllNodeNames) const;
	UNREALED_API void GetAnimatedNodeNameArray(TArray<FString> &AnimatedNodeNames) const;
	UNREALED_API void GetNodeAnimatedPropertyNameArray(const FString &NodeName, TArray<FString> &AnimatedPropertyNames) const;
	UNREALED_API void GetCustomStringPropertyArray(const FString& NodeName, TArray<TPair<FString, FString>>& CustomPropertyPairs) const;

	UE_DEPRECATED(4.21, "Please use FRichCurve version instead to get tangent weight support")
	UNREALED_API void GetCurveData(const FString& NodeName, const FString& PropertyName, int32 ChannelIndex, int32 CompositeIndex, FInterpCurveFloat& CurveData, bool bNegative) const;
	
	//This one should be use only by the sequencer the key tangents data is transform to fit the expected data we have in the old matinee code
	UNREALED_API void GetCurveDataForSequencer(const FString& NodeName, const FString& PropertyName, int32 ChannelIndex, int32 CompositeIndex, FRichCurve& RichCurve, bool bNegative, float Scale = 1.0f) const;

	UNREALED_API void GetCurveData(const FString& NodeName, const FString& PropertyName, int32 ChannelIndex, int32 CompositeIndex, FRichCurve& CurveData, bool bNegative) const;

	
	UNREALED_API void GetBakeCurveData(const FString& NodeName, const FString& PropertyName, int32 ChannelIndex, int32 CompositeIndex, TArray<float>& CurveData, float PeriodTime, float StartTime = 0.0f, float StopTime= -1.0f, bool bNegative = false, float Scale = 1.0f) const;

	//Handle API
	UNREALED_API void GetAllNodePropertyCurveHandles(const FString& NodeName, const FString& PropertyName, TArray<FFbxAnimCurveHandle> &PropertyCurveHandles) const;
	UNREALED_API void GetCurveHandle(const FString& NodeName, const FString& PropertyName, int32 ChannelIndex, int32 CompositeIndex, FFbxAnimCurveHandle &CurveHandle) const;
	
	UE_DEPRECATED(4.21, "Please use FRichCurve version instead to get tangent weight support")
	UNREALED_API void GetCurveData(const FFbxAnimCurveHandle &CurveHandle, FInterpCurveFloat& CurveData, bool bNegative) const;
	
	//This one should be use only by the sequencer the key tangents data is transform to fit the expected data we have in the old matinee code
	UNREALED_API void GetCurveDataForSequencer(const FFbxAnimCurveHandle &CurveHandle, FRichCurve& RichCurve, bool bNegative, float Scale = 1.0f) const;

	UNREALED_API void GetCurveData(const FFbxAnimCurveHandle &CurveHandle, FRichCurve& CurveData, bool bNegative, float Scale  = 1.0f) const;

	UNREALED_API void GetBakeCurveData(const FFbxAnimCurveHandle &CurveHandle, TArray<float>& CurveData, float PeriodTime, float StartTime = 0.0f, float StopTime = -1.0f, bool bNegative = false, float Scale = 1.0f) const;

	UNREALED_API void GetConvertedNonTransformCurveData(const FString& NodeName, bool bUseSequencerCurve, float UniformScale, TMap<FName, FRichCurve>& OutCurves);
	
	//Conversion API
	UE_DEPRECATED(4.21, "Please use FRichCurve version instead to get tangent weight support")
	UNREALED_API void GetConvertedTransformCurveData(const FString& NodeName, FInterpCurveFloat& TranslationX, FInterpCurveFloat& TranslationY, FInterpCurveFloat& TranslationZ,
													 FInterpCurveFloat& EulerRotationX, FInterpCurveFloat& EulerRotationY, FInterpCurveFloat& EulerRotationZ, 
													 FInterpCurveFloat& ScaleX, FInterpCurveFloat& ScaleY, FInterpCurveFloat& ScaleZ,
													 FTransform& DefaultTransform) const;
	
	UNREALED_API void GetConvertedTransformCurveData(const FString& NodeName, FRichCurve& TranslationX, FRichCurve& TranslationY, FRichCurve& TranslationZ,
		FRichCurve& EulerRotationX, FRichCurve& EulerRotationY, FRichCurve& EulerRotationZ,
		FRichCurve& ScaleX, FRichCurve& ScaleY, FRichCurve& ScaleZ,
		FTransform& DefaultTransform, bool bUseSequencerCurve, float UniformScale = 1.0f) const;

	FbxScene* Scene;
	TMap<uint64, FFbxAnimNodeHandle> CurvesData;
	TMap<uint64, FTransform> TransformData;

private:
	EInterpCurveMode GetUnrealInterpMode(FbxAnimCurveKey FbxKey) const;
};

struct FbxMeshInfo
{
	FString Name;
	uint64 UniqueId;
	int32 FaceNum;
	int32 VertexNum;
	bool bTriangulated;
	int32 MaterialNum;
	bool bIsSkelMesh;
	FString SkeletonRoot;
	int32 SkeletonElemNum;
	FString LODGroup;
	int32 LODLevel;
	int32 MorphNum;
};

//Node use to store the scene hierarchy transform will be relative to the parent
struct FbxNodeInfo
{
	FString ObjectName;
	uint64 UniqueId;
	FbxAMatrix Transform;
	FbxVector4 RotationPivot;
	FbxVector4 ScalePivot;
	
	FString AttributeName;
	uint64 AttributeUniqueId;
	FString AttributeType;

	FString ParentName;
	uint64 ParentUniqueId;
};

struct FbxSceneInfo
{
	// data for static mesh
	int32 NonSkinnedMeshNum;
	
	//data for skeletal mesh
	int32 SkinnedMeshNum;

	// common data
	int32 TotalGeometryNum;
	int32 TotalMaterialNum;
	int32 TotalTextureNum;
	
	TArray<FbxMeshInfo> MeshInfo;
	TArray<FbxNodeInfo> HierarchyInfo;
	
	/* true if it has animation */
	bool bHasAnimation;
	bool bHasAnimationOnSkeletalMesh;
	double FrameRate;
	double TotalTime;

	void Reset()
	{
		NonSkinnedMeshNum = 0;
		SkinnedMeshNum = 0;
		TotalGeometryNum = 0;
		TotalMaterialNum = 0;
		TotalTextureNum = 0;
		MeshInfo.Empty();
		HierarchyInfo.Empty();
		bHasAnimation = false;
		bHasAnimationOnSkeletalMesh = false;
		FrameRate = 0.0;
		TotalTime = 0.0;
	}
};

/**
* FBX basic data conversion class.
*/
class FFbxDataConverter
{
public:
	static void SetJointPostConversionMatrix(FbxAMatrix ConversionMatrix) { JointPostConversionMatrix = ConversionMatrix; }
	static const FbxAMatrix &GetJointPostConversionMatrix() { return JointPostConversionMatrix; }

	static void SetAxisConversionMatrix(FbxAMatrix ConversionMatrix) { AxisConversionMatrix = ConversionMatrix; AxisConversionMatrixInv = ConversionMatrix.Inverse(); }
	static const FbxAMatrix &GetAxisConversionMatrix() { return AxisConversionMatrix; }
	static const FbxAMatrix &GetAxisConversionMatrixInv() { return AxisConversionMatrixInv; }

	static UNREALED_API FVector ConvertPos(FbxVector4 Vector);
	static UNREALED_API FVector ConvertDir(FbxVector4 Vector);
	static UNREALED_API FRotator ConvertEuler(FbxDouble3 Euler);
	static UNREALED_API FVector ConvertScale(FbxDouble3 Vector);
	static UNREALED_API FVector ConvertScale(FbxVector4 Vector);
	static UNREALED_API FRotator ConvertRotation(FbxQuaternion Quaternion);
	static UNREALED_API FVector ConvertRotationToFVect(FbxQuaternion Quaternion, bool bInvertRot);
	static UNREALED_API FQuat ConvertRotToQuat(FbxQuaternion Quaternion);
	static UNREALED_API float ConvertDist(FbxDouble Distance);
	static UNREALED_API bool ConvertPropertyValue(FbxProperty& FbxProperty, FProperty& UnrealProperty, union UPropertyValue& OutUnrealPropertyValue);
	static UNREALED_API FTransform ConvertTransform(FbxAMatrix Matrix);
	static UNREALED_API FMatrix ConvertMatrix(const FbxAMatrix& Matrix);
	static UNREALED_API FbxAMatrix ConvertMatrix(const FMatrix& Matrix);

	/*
	 * Convert fbx linear space color to sRGB FColor
	 */
	static UNREALED_API FColor ConvertColor(FbxDouble3 Color);

	static UNREALED_API FbxVector4 ConvertToFbxPos(FVector Vector);
	static UNREALED_API FbxVector4 ConvertToFbxRot(FVector Vector);
	static UNREALED_API FbxVector4 ConvertToFbxScale(FVector Vector);
	
	/*
	* Convert sRGB FColor to fbx linear space color
	*/
	static UNREALED_API FbxDouble3   ConvertToFbxColor(FColor Color);
	static UNREALED_API FbxString	ConvertToFbxString(FName Name);
	static UNREALED_API FbxString	ConvertToFbxString(const FString& String);

	// FbxCamera with no rotation faces X with Y-up while ours faces X with Z-up so add a -90 degrees roll to compensate
	static FRotator GetCameraRotation() { return FRotator(0.f, 0.f, -90.f); }

	// FbxLight with no rotation faces -Z while ours faces Y so add a 90 degrees pitch to compensate
	static FRotator GetLightRotation() { return FRotator(0.f, 90.f, 0.f); }

private:
	static UNREALED_API FbxAMatrix JointPostConversionMatrix;
	static UNREALED_API FbxAMatrix AxisConversionMatrix;
	static UNREALED_API FbxAMatrix AxisConversionMatrixInv;
};

FBXImportOptions* GetImportOptions( class FFbxImporter* FbxImporter, UFbxImportUI* ImportUI, bool bShowOptionDialog, bool bIsAutomated, const FString& FullPath, bool& OutOperationCanceled, bool& OutImportAll, bool bIsObjFormat, const FString& InFilename, bool bForceImportType = false, EFBXImportType ImportType = FBXIT_StaticMesh);
UNREALED_API void ApplyImportUIToImportOptions(UFbxImportUI* ImportUI, FBXImportOptions& InOutImportOptions);

struct FImportedMaterialData
{
public:
	void AddImportedMaterial( const FbxSurfaceMaterial& FbxMaterial, UMaterialInterface& UnrealMaterial );
	bool IsAlreadyImported( const FbxSurfaceMaterial& FbxMaterial, FName ImportedMaterialName ) const;
	UMaterialInterface* GetUnrealMaterial( const FbxSurfaceMaterial& FbxMaterial ) const;
	void Clear();
private:
	/** Mapping of FBX material to Unreal material.  Some materials in FBX have the same name so we use this map to determine if materials are unique */
	TMap<const FbxSurfaceMaterial*, TWeakObjectPtr<UMaterialInterface> > FbxToUnrealMaterialMap;
	TSet<FName> ImportedMaterialNames;
};

enum EFbxCreator
{
	Blender,
	Unknow
};

class FFbxHelper
{
public:
	/**
	* This function is use to compute the weight between two name.
	*/
	static float NameCompareWeight(const FString& A, const FString& B)
	{
		FString Longer = A;
		FString Shorter = B;
		if (A.Len() < B.Len())
		{
			Longer = B;
			Shorter = A;
		}

		if (Longer.Compare(Shorter, ESearchCase::CaseSensitive) == 0)
		{
			return 1.0f;
		}
		if (Longer.Compare(Shorter, ESearchCase::IgnoreCase) == 0)
		{
			return 0.98f;
		}
		// We do the contain so it is giving better result since often we compare thing like copy and paste string name
		// we want to match: BackZ
		// between: Paste_BackZ and BackX
		// EditDistance for Paste_BackZ is 5/11 =0.45
		// EditDistance for BackX is 4/5= 0.8
		// The contains weight for Paste_BackZ is 0.98- (0.25*(1.0-5/11)) = 0.844
		if (Longer.Contains(Shorter, ESearchCase::CaseSensitive))
		{
			return 0.98f - 0.25f*(1.0f - ((float)(Shorter.Len()) / (float)(Longer.Len())));
		}
		if (Longer.Contains(Shorter, ESearchCase::IgnoreCase))
		{
			return 0.96f - 0.25f*(1.0f - ((float)(Shorter.Len()) / (float)(Longer.Len())));
		}


		float LongerLength = (float)Longer.Len();
		if (LongerLength == 0)
		{
			return 1.0f;
		}
		return (LongerLength - Algo::LevenshteinDistance(Longer, Shorter)) / LongerLength;
	}
};

/**
 * Main FBX Importer class.
 */
class FFbxImporter
{
public:
	~FFbxImporter();
	/**
	 * Returns the importer singleton. It will be created on the first request.
	 */
	UNREALED_API static FFbxImporter* GetInstance(bool bDoNotCreate = false);
	static void DeleteInstance();

	/**
	* Clear all data that need to be clear when we start importing a fbx file.
	*/
	void ClearAllCaches()
	{
		//Clear the mesh name cache use to ensure unique mesh name and avoid name clash
		MeshNamesCache.Reset();
		
		//this cache is use to prevent a node to be transform twice. it has to be reset everytime we
		//read a new fbx file
		TransformSettingsToFbxApply.Reset();
	}

	/**
	 * The asset tool have a filter mecanism for ImportAsset, return true if the asset can be imported, false otherwise
	 */
	bool CanImportClass(UClass* Class) const;

	/**
	 * The asset tool have a filter mecanism for CreateAsset, return true if the asset can be created, false otherwise
	 */
	bool CanCreateClass(UClass* Class) const;

	/**
	 * Detect if the FBX file has skeletal mesh model. If there is deformer definition, then there is skeletal mesh.
	 * In this function, we don't need to import the scene. But the open process is time-consume if the file is large.
	 *
	 * @param InFilename	FBX file name. 
	 * @return int32 -1 if parse failed; 0 if geometry ; 1 if there are deformers; 2 otherwise
	 */
	int32 GetImportType(const FString& InFilename);

	/**
	 * Get detail infomation in the Fbx scene
	 *
	 * @param Filename Fbx file name
	 * @param SceneInfo return the scene info
	 * @return bool true if get scene info successfully
	 */
	bool GetSceneInfo(FString Filename, FbxSceneInfo& SceneInfo, bool bPreventMaterialNameClash = false);

	/**
	 * Initialize Fbx file for import.
	 *
	 * @param Filename
	 * @param bParseStatistics
	 * @return bool
	 */
	bool OpenFile(FString Filename);

	/*
	Make sure the file header is read
	*/
	bool ReadHeaderFromFile(const FString& Filename, bool bPreventMaterialNameClash = false);
	
	/**
	 * Import Fbx file.
	 *
	 * @param Filename
	 * @return bool
	 */
	bool ImportFile(FString Filename, bool bPreventMaterialNameClash = false);
	
	/**
	 * Convert the scene from the current options.
	 * The scene will be converted to RH -Y or RH X depending if we force a front X axis or not
	 */
	void ConvertScene();

	/**
	 * Attempt to load an FBX scene from a given filename.
	 *
	 * @param Filename FBX file name to import.
	 * @returns true on success.
	 */
	UNREALED_API bool ImportFromFile(const FString& Filename, const FString& Type, bool bPreventMaterialNameClash = false);

	/**
	 * Prime the importer with an already existing scene object.
	 *
	 * @param Scene - The scene we want to load.
	 */
	UE_DEPRECATED(5.1, "Do not use this function.")
	UNREALED_API void SetScene(FbxScene* InScene);

	/**
	 * Retrieve the FBX loader's error message explaining its failure to read a given FBX file.
	 * Note that the message should be valid even if the parser is successful and may contain warnings.
	 *
	 * @ return TCHAR*	the error message
	 */
	const TCHAR* GetErrorMessage() const
	{
		return *ErrorMessage;
	}

	/**
	 * Retrieve the object inside the FBX scene from the name
	 *
	 * @param ObjectName	Fbx object name
	 * @param Root	Root node, retrieve from it
	 * @return FbxNode*	Fbx object node
	 */
	FbxNode* RetrieveObjectFromName(const TCHAR* ObjectName, FbxNode* Root = NULL);

	/**
	* Find the first node containing a mesh attribute for the specified LOD index.
	*
	* @param NodeLodGroup	The LOD group fbx node
	* @param LodIndex		The index of the LOD we search the mesh node
	*/
	FbxNode* FindLODGroupNode(FbxNode* NodeLodGroup, int32 LodIndex, FbxNode *NodeToFind = nullptr);

	/**
	* Find the all the node containing a mesh attribute for the specified LOD index.
	*
	* @param OutNodeInLod   All the mesh node under the lod group
	* @param NodeLodGroup	The LOD group fbx node
	* @param LodIndex		The index of the LOD we search the mesh node
	*/
	UNREALED_API void FindAllLODGroupNode(TArray<FbxNode*> &OutNodeInLod, FbxNode* NodeLodGroup, int32 LodIndex);

	/**
	* Find the first parent node containing a eLODGroup attribute.
	*
	* @param ParentNode		The node where to start the search.
	*/
	FbxNode *RecursiveFindParentLodGroup(FbxNode *ParentNode);

	/**
	 * Creates a static mesh with the given name and flags, imported from within the FBX scene.
	 * @param InParent
	 * @param Node	Fbx Node to import
	 * @param Name	the Unreal Mesh name after import
	 * @param Flags
	 * @param InStaticMesh	if LODIndex is not 0, this is the base mesh object. otherwise is NULL
	 * @param LODIndex	 LOD level to import to
	 *
	 * @returns UObject*	the UStaticMesh object.
	 */
	UNREALED_API UStaticMesh* ImportStaticMesh(UObject* InParent, FbxNode* Node, const FName& Name, EObjectFlags Flags, UFbxStaticMeshImportData* ImportData, UStaticMesh* InStaticMesh = NULL, int LODIndex = 0, const FExistingStaticMeshData* ExistMeshDataPtr = nullptr);

	/**
	* Creates a static mesh from all the meshes in FBX scene with the given name and flags.
	*
	* @param InParent
	* @param MeshNodeArray	Fbx Nodes to import
	* @param Name	the Unreal Mesh name after import
	* @param Flags
	* @param InStaticMesh	if LODIndex is not 0, this is the base mesh object. otherwise is NULL
	* @param LODIndex	 LOD level to import to
	* @param OrderedMaterialNames  If not null, the original fbx ordered materials name will be use to reorder the section of the mesh we currently import
	*
	* @returns UObject*	the UStaticMesh object.
	*/
	UNREALED_API UStaticMesh* ImportStaticMeshAsSingle(UObject* InParent, TArray<FbxNode*>& MeshNodeArray, const FName InName, EObjectFlags Flags, UFbxStaticMeshImportData* TemplateImportData, UStaticMesh* InStaticMesh, int LODIndex = 0, const FExistingStaticMeshData* ExistMeshDataPtr = nullptr);

	/**
	* Finish the import of the staticmesh after all LOD have been process (cannot be call before all LOD are imported). There is two main operation done by this function
	* 1. Build the staticmesh render data
	* 2. Reorder the material array to follow the fbx file material order
	*/
	UNREALED_API void PostImportStaticMesh(UStaticMesh* StaticMesh, TArray<FbxNode*>& MeshNodeArray, int32 LODIndex = 0);
    
	static void UpdateStaticMeshImportData(UStaticMesh *StaticMesh, UFbxStaticMeshImportData* StaticMeshImportData);
	static void UpdateSkeletalMeshImportData(USkeletalMesh *SkeletalMesh, UFbxSkeletalMeshImportData* SkeletalMeshImportData, int32 SpecificLod, TArray<FName> *ImportMaterialOriginalNameData, TArray<FImportMeshLodSectionsData> *ImportMeshLodData);
	void ImportStaticMeshGlobalSockets( UStaticMesh* StaticMesh );
	void ImportStaticMeshLocalSockets( UStaticMesh* StaticMesh, TArray<FbxNode*>& MeshNodeArray);

	/*
	 * Add a GeneratedLOD to the staticmesh at the specified LOD index
	 */
	void AddStaticMeshSourceModelGeneratedLOD(UStaticMesh* StaticMesh, int32 LODIndex);

	/**
	* Return the node that match the mesh name. Return nullptr in case there is no match
	*/
	FbxNode* GetMeshNodesFromName(const FString& ReimportMeshName, TArray<FbxNode*>& FbxMeshArray);

	/**
	 * re-import Unreal static mesh from updated Fbx file
	 * if the Fbx mesh is in LODGroup, the LOD of mesh will be updated
	 *
	 * @param Mesh the original Unreal static mesh object
	 * @return UObject* the new Unreal mesh object
	 */
	UStaticMesh* ReimportStaticMesh(UStaticMesh* Mesh, UFbxStaticMeshImportData* TemplateImportData);

	/**
	* re-import Unreal static mesh from updated scene Fbx file
	* if the Fbx mesh is in LODGroup, the LOD of mesh will be updated
	*
	* @param Mesh the original Unreal static mesh object
	* @return UObject* the new Unreal mesh object
	*/
	UStaticMesh* ReimportSceneStaticMesh(uint64 FbxNodeUniqueId, uint64 FbxMeshUniqueId, UStaticMesh* Mesh, UFbxStaticMeshImportData* TemplateImportData);
	
	/**
	* re-import Unreal skeletal mesh from updated Fbx file
	* If the Fbx mesh is in LODGroup, the LOD of mesh will be updated.
	* If the FBX mesh contains morph, the morph is updated.
	* Materials, textures and animation attached in the FBX mesh will not be updated.
	*
	* @param Mesh the original Unreal skeletal mesh object
	* @return UObject* the new Unreal mesh object
	*/
	USkeletalMesh* ReimportSkeletalMesh(USkeletalMesh* Mesh, UFbxSkeletalMeshImportData* TemplateImportData, uint64 SkeletalMeshFbxUID = 0xFFFFFFFFFFFFFFFF, TArray<FbxNode*> *OutSkeletalMeshArray = nullptr);

	/**
	 * Creates a skeletal mesh from Fbx Nodes with the given name and flags, imported from within the FBX scene.
	 * These Fbx Nodes bind to same skeleton. We need to bind them to one skeletal mesh.
	 *
	 * @param InParent
	 * @param NodeArray	Fbx Nodes to import
	 * @param Name	the Unreal Mesh name after import
	 * @param Flags
	 * @param FbxShapeArray	Fbx Morph objects.
	 * @param OutData - Optional import data to populate
	 * @param bCreateRenderData - Whether or not skeletal mesh rendering data will be created.
	 * @param OrderedMaterialNames  If not null, the original fbx ordered materials name will be use to reorder the section of the mesh we currently import
	 *
	 * @return The USkeletalMesh object created
	 */

	class FImportSkeletalMeshArgs
	{
	public:
		FImportSkeletalMeshArgs()
			: InParent(nullptr)
			, NodeArray()
			, Name(NAME_None)
			, Flags(RF_NoFlags)
			, TemplateImportData(nullptr)
			, LodIndex(0)
			, FbxShapeArray(nullptr)
			, OutData(nullptr)
			, bCreateRenderData(true)
			, OrderedMaterialNames(nullptr)
			, ImportMaterialOriginalNameData(nullptr)
			, ImportMeshSectionsData(nullptr)
			, bMapMorphTargetToTimeZero(false)
		{}

		UObject* InParent;
		TArray<FbxNode*> NodeArray;
		FName Name;
		EObjectFlags Flags;
		UFbxSkeletalMeshImportData* TemplateImportData;
		int32 LodIndex;
		TArray<FbxShape*> *FbxShapeArray;
		FSkeletalMeshImportData* OutData;
		bool bCreateRenderData;
		TArray<FName> *OrderedMaterialNames;

		TArray<FName> *ImportMaterialOriginalNameData;
		FImportMeshLodSectionsData *ImportMeshSectionsData;
		bool bMapMorphTargetToTimeZero;
	};

	UNREALED_API USkeletalMesh* ImportSkeletalMesh(FImportSkeletalMeshArgs &ImportSkeletalMeshArgs);

	/**
	 * Add to the animation set, the animations contained within the FBX scene, for the given skeletal mesh
	 *
	 * @param Skeleton	Skeleton that the animation belong to
	 * @param SortedLinks	skeleton nodes which are sorted
	 * @param Filename	Fbx file name
	 * @param NodeArray node array of FBX meshes
	 */
	UAnimSequence* ImportAnimations(USkeleton* Skeleton, UObject* Outer, TArray<FbxNode*>& SortedLinks, const FString& Name, UFbxAnimSequenceImportData* TemplateImportData, TArray<FbxNode*>& NodeArray);

	/**
	 * Get Animation Time Span - duration of the animation
	 */
	UNREALED_API FbxTimeSpan GetAnimationTimeSpan(FbxNode* RootNode, FbxAnimStack* AnimStack);

	/**
	* When we get exported time we call GetanimationInterval from fbx sdk and it return the layer 0 by default
	* This function return the sum of all layer instead of just the layer 0.
	*/
	void GetAnimationIntervalMultiLayer(FbxNode* RootNode, FbxAnimStack* AnimStack, FbxTimeSpan& AnimTimeSpan);

	/**
	 * Import one animation from CurAnimStack
	 *
	 * @param Skeleton	Skeleton that the animation belong to
	 * @param DestSeq 	Sequence it's overwriting data to
	 * @param Filename	Fbx file name	(not whole path)
	 * @param SortedLinks	skeleton nodes which are sorted
	 * @param NodeArray node array of FBX meshes
	 * @param CurAnimStack 	Animation Data
	 * @param ResampleRate	Resample Rate for data
	 * @param AnimTimeSpan	AnimTimeSpan
	 */
	bool ImportAnimation(USkeleton* Skeleton, UAnimSequence* DestSeq, const FString& FileName, TArray<FbxNode*>& SortedLinks, TArray<FbxNode*>& NodeArray, FbxAnimStack* CurAnimStack, const int32 ResampleRate, const FbxTimeSpan AnimTimeSpan, const bool bReimport);
	/**
	* Calculate the global Sample Rate for all the nodes in the FbxAnimStack pass in parameter
	*
	* @param FbxAnimStack	The anim stack we want to know the best sample rate
	*/
	int32 GetGlobalAnimStackSampleRate(FbxAnimStack* CurAnimStack);
	/**
	 * Calculate Max Sample Rate - separate out of the original ImportAnimations
	 *
	 * @param SortedLinks	skeleton nodes which are sorted
	 * @param NodeArray node array of FBX meshes
	 */
	int32 GetMaxSampleRate(TArray<FbxNode*>& SortedLinks);
	/**
	 * Validate Anim Stack - multiple check for validating animstack
	 *
	 * @param SortedLinks	skeleton nodes which are sorted
	 * @param NodeArray node array of FBX meshes
	 * @param CurAnimStack 	Animation Data
	 * @param ResampleRate	Resample Rate for data
	 * @param AnimTimeSpan	AnimTimeSpan	 
	 */	
	bool ValidateAnimStack(TArray<FbxNode*>& SortedLinks, TArray<FbxNode*>& NodeArray, FbxAnimStack* CurAnimStack, int32 ResampleRate, bool bImportMorph, bool bSnapToClosestFrameBoundary, FbxTimeSpan &AnimTimeSpan);

	/**
	 * Import Fbx Morph object for the Skeletal Mesh.
	 * In Fbx, morph object is a property of the Fbx Node.
	 *
	 * @param SkelMeshNodeArray - Fbx Nodes that the base Skeletal Mesh construct from
	 * @param BaseSkelMesh - base Skeletal Mesh
	 * @param LODIndex - LOD index
	 */
	UNREALED_API void ImportFbxMorphTarget(TArray<FbxNode*> &SkelMeshNodeArray, USkeletalMesh* BaseSkelMesh, int32 LODIndex, FSkeletalMeshImportData &BaseSkeletalMeshImportData, const bool bSkinControlPointToTimeZero);

	/**
	 * Import LOD object for skeletal mesh
	 *
	 * @param InSkeletalMesh - LOD mesh object
	 * @param BaseSkeletalMesh - base mesh object
	 * @param DesiredLOD - LOD level
	   @param ReregisterAssociatedComponents - if NULL, just re-registers all SkinnedMeshComponents but if you set the specific components, will only re-registers those components
	 */
	UNREALED_API bool ImportSkeletalMeshLOD(USkeletalMesh* InSkeletalMesh, USkeletalMesh* BaseSkeletalMesh, int32 DesiredLOD, UFbxSkeletalMeshImportData* TemplateImportData = nullptr);

	/**
	 * Empties the FBX scene, releasing its memory.
	 * Currently, we can't release KFbxSdkManager because Fbx Sdk2010.2 has a bug that FBX can only has one global sdkmanager.
	 * From Fbx Sdk2011, we can create multiple KFbxSdkManager, then we can release it.
	 */
	UNREALED_API void ReleaseScene();

	/**
	 * If the node model is a collision model, then fill it into collision model list
	 *
	 * @param Node Fbx node
	 * @return true if the node is a collision model
	 */
	bool FillCollisionModelList(FbxNode* Node);

	/**
	 * Import collision models for one static mesh if it has collision models
	 *
	 * @param StaticMesh - mesh object to import collision models
	 * @param NodeName - name of Fbx node that the static mesh constructed from
	 * @return return true if the static mesh has collision model and import successfully
	 */
	bool ImportCollisionModels(UStaticMesh* StaticMesh, const FbxString& NodeName);

	//help
	static FString MakeName(const ANSICHAR* name);
	static FString MakeString(const ANSICHAR* Name);
	FName MakeNameForMesh(FString InName, FbxObject* FbxObject);

	// meshes
	
	/**
	* Get all Fbx skeletal mesh objects in the scene. these meshes are grouped by skeleton they bind to
	*
	* @param Node Root node to find skeletal meshes
	* @param outSkelMeshArray return Fbx meshes they are grouped by skeleton
	*/
	UNREALED_API void FillFbxSkelMeshArrayInScene(FbxNode* Node, TArray< TArray<FbxNode*>* >& outSkelMeshArray, bool ExpandLOD, bool bCombineSkeletalMesh, bool bForceFindRigid = false);
	
	/**
	 * Find FBX meshes that match Unreal skeletal mesh according to the bone of mesh
	 *
	 * @param FillInMesh     Unreal skeletal mesh
	 * @param bExpandLOD     flag that if expand FBX LOD group when get the FBX node
	 * @param OutFBXMeshNodeArray  return FBX mesh nodes that match the Unreal skeletal mesh
	 * 
	 * @return the root bone that bind to the FBX skeletal meshes
	 */
	FbxNode* FindFBXMeshesByBone(const FName& RootBoneName, bool bExpandLOD, TArray<FbxNode*>& OutFBXMeshNodeArray);
	
	/**
	* Get mesh count (including static mesh and skeletal mesh, except collision models) and find collision models
	*
	* @param Node			Root node to find meshes
	* @param bCountLODs		Whether or not to count meshes in LOD groups
	* @return int32 mesh count
	*/
	int32 GetFbxMeshCount(FbxNode* Node,bool bCountLODs, int32 &OutNumLODGroups );
	
	/**
	* Fill the collision models array by going through all mesh node recursively
	*
	* @param Node Root node to find collision meshes
	*/
	UNREALED_API void FillFbxCollisionMeshArray(FbxNode* Node);

	/**
	* Get all Fbx mesh objects
	*
	* @param Node Root node to find meshes
	* @param outMeshArray return Fbx meshes
	*/
	UNREALED_API void FillFbxMeshArray(FbxNode* Node, TArray<FbxNode*>& outMeshArray, UnFbx::FFbxImporter* FFbxImporter);

	/**
	* Get all Fbx mesh objects not under a LOD group and all LOD group node
	*
	* @param Node Root node to find meshes
	* @param outLODGroupArray return Fbx LOD group
	* @param outMeshArray return Fbx meshes with no LOD group
	*/
	UNREALED_API void FillFbxMeshAndLODGroupArray(FbxNode* Node, TArray<FbxNode*>& outLODGroupArray, TArray<FbxNode*>& outMeshArray);

	/**
	 * Returns if the passed FbxNode can be used as a skeleton bone in Unreal.
	 * 
	 * @return bool
	 */
	bool IsUnrealBone(FbxNode* Link);

	/**
	 * Returns if the passed FbxNode can be used as a transform attribute in Unreal.
	 * 
	 * @return bool
	 */
	bool IsUnrealTransformAttribute(FbxNode* Link);

	/**
	* Fill FBX skeletons to OutSortedLinks recursively
	*
	* @param Link Fbx node of skeleton root
	* @param OutSortedLinks
	*/
	void RecursiveBuildSkeleton(FbxNode* Link, TArray<FbxNode*>& OutSortedLinks);

	/**
	 * Fill FBX skeletons to OutSortedLinks
	 *
	 * @param ClusterArray Fbx clusters of FBX skeletal meshes
	 * @param OutSortedLinks
	 */
	void BuildSkeletonSystem(TArray<FbxCluster*>& ClusterArray, TArray<FbxNode*>& OutSortedLinks);

	/**
	 * Get Unreal skeleton root from the FBX skeleton node.
	 * Mesh and dummy can be used as skeleton.
	 *
	 * @param Link one FBX skeleton node
	 */
	FbxNode* GetRootSkeleton(FbxNode* Link);
	
	/**
	 * Get the object of import options
	 *
	 * @return FBXImportOptions
	 */
	UNREALED_API FBXImportOptions* GetImportOptions() const;

	/*
	* This function show a dialog to let the user know what will be change in the skeleton if the fbx is imported
	*/
	static void ShowFbxSkeletonConflictWindow(USkeletalMesh *SkeletalMesh, USkeleton* Skeleton, ImportCompareHelper::FSkeletonCompareData& SkeletonCompareData);

	template<typename TMaterialType>
	static void PrepareAndShowMaterialConflictDialog(const TArray<TMaterialType>& CurrentMaterial, TArray<TMaterialType>& ResultMaterial, TArray<int32>& RemapMaterial, TArray<FName>& RemapMaterialName, bool bCanShowDialog, bool bIsPreviewDialog, bool bForceResetOnConflict, EFBXReimportDialogReturnOption& OutReturnOption);
	/*
	* This function show a dialog to let the user resolve the material conflict that arise when re-importing a mesh
	*/
	template<typename TMaterialType>
	static void ShowFbxMaterialConflictWindow(const TArray<TMaterialType>& InSourceMaterials, const TArray<TMaterialType>& InResultMaterials, TArray<int32>& RemapMaterials, TArray<bool>& FuzzyRemapMaterials, EFBXReimportDialogReturnOption& OutReturnOption, bool bIsPreviewConflict = false);

	/**
	 * Apply asset import settings for transform to an FBX node
	 *
	 * @param Node Node to apply transform settings too
	 * @param AssetData the asset data object to get transform data from
	 */
	void ApplyTransformSettingsToFbxNode(FbxNode* Node, UFbxAssetImportData* AssetData);

	/**
	 * Remove asset import settings for transform to an FBX node
	 *
	 * @param Node Node to apply transform settings too
	 * @param AssetData the asset data object to get transform data from
	 */
	void RemoveTransformSettingsFromFbxNode(FbxNode* Node, UFbxAssetImportData* AssetData);

	/**
	 * Populate the given matrix with the correct information for the asset data, in
	 * a format that matches FBX internals or without conversion
	 *
	 * @param OutMatrix The matrix to fill
	 * @param AssetData The asset data to extract the transform info from
	 */
	void BuildFbxMatrixForImportTransform(FbxAMatrix& OutMatrix, UFbxAssetImportData* AssetData);

	/**
	 * Import FbxCurve to Curve
	 */
	static bool ImportCurve(const FbxAnimCurve* FbxCurve, FRichCurve& RichCurve, const FbxTimeSpan &AnimTimeSpan, const bool bNegative = false, const float ValueScale = 1.f, const bool bAutoSetTangents = true);

	/**
	 * Merge all layers of one AnimStack to one layer.
	 *
	 * @param AnimStack     AnimStack which layers will be merged
	 * @param ResampleRate  resample rate for the animation
	 */
	void MergeAllLayerAnimation(FbxAnimStack* AnimStack, int32 ResampleRate);

	/**
	 * Get the UObjects that have been created so far during the import process.
	 */
	const TArray<TWeakObjectPtr<UObject>>& GetCreatedObjects() const { return CreatedObjects; }
private:

	/**
	* This function fill the last imported Material name. Those named are used to reorder the mesh sections
	* during a re-import. In case material names use the skinxx workflow the LastImportedMaterialNames array
	* will be empty to let the system reorder the mesh sections with the skinxx workflow.
	*
	* @param LastImportedMaterialNames	This array will be filled with the BaseSkelMesh Material original imported names
	* @param BaseSkelMesh				Skeletal mesh holding the last imported material names. If null the LastImportedMaterialNames will be empty;
	* @param OrderedMaterialNames		if not null, it will be used to fill the LastImportedMaterialNames array. except if the names are using the _skinxx workflow
	*/
	void FillLastImportMaterialNames(TArray<FName> &LastImportedMaterialNames, USkeletalMesh* BaseSkelMesh, TArray<FName> *OrderedMaterialNames);

	/**
	* Verify that all meshes are also reference by a fbx hierarchy node. If it found some Geometry
	* not reference it will add a tokenized error.
	*/
	void ValidateAllMeshesAreReferenceByNodeAttribute();

	/*
	 * Parse the fbx and create some LODGroup for matching LODX_ prefix
	 * Simply change the hierarchy to incorporate LODGroup and child all LODX_ prefix with the matching geometry name.
	 * The LODX_ prefix support X from 0 to 9
	 * The X parameter do not have to be continuous, but the LOD will be set in the correct order.
	 */
	void ConvertLodPrefixToLodGroup();

	/**
	* Recursive search for a node having a mesh attribute
	*
	* @param Node	The node from which we start the search for the first node containing a mesh attribute
	*/
	FbxNode *RecursiveGetFirstMeshNode(FbxNode* Node, FbxNode* NodeToFind = nullptr);

	/**
	* Recursive search for a node having a mesh attribute
	*
	* @param Node	The node from which we start the search for the first node containing a mesh attribute
	*/
	void RecursiveGetAllMeshNode(TArray<FbxNode *> &OutAllNode, FbxNode* Node);

	/**
	 * ActorX plug-in can export mesh and dummy as skeleton.
	 * For the mesh and dummy in the skeleton hierarchy, convert them to FBX skeleton.
	 *
	 * @param Node          root skeleton node
	 * @param SkelMeshes    skeletal meshes that bind to this skeleton
	 * @param bImportNestedMeshes	if true we will import meshes nested in bone hierarchies instead of converting them to bones
	 */
	void RecursiveFixSkeleton(FbxNode* Node, TArray<FbxNode*> &SkelMeshes, bool bImportNestedMeshes );
	
	/**
	* Get all Fbx skeletal mesh objects which are grouped by skeleton they bind to
	*
	* @param Node Root node to find skeletal meshes
	* @param outSkelMeshArray return Fbx meshes they are grouped by skeleton
	* @param SkeletonArray
	* @param ExpandLOD flag of expanding LOD to get each mesh
	*/
	void RecursiveFindFbxSkelMesh(FbxNode* Node, TArray< TArray<FbxNode*>* >& outSkelMeshArray, TArray<FbxNode*>& SkeletonArray, bool ExpandLOD);
	
	/**
	* Get all Fbx rigid mesh objects which are grouped by skeleton hierarchy
	*
	* @param Node Root node to find skeletal meshes
	* @param outSkelMeshArray return Fbx meshes they are grouped by skeleton hierarchy
	* @param SkeletonArray
	* @param ExpandLOD flag of expanding LOD to get each mesh
	*/
	void RecursiveFindRigidMesh(FbxNode* Node, TArray< TArray<FbxNode*>* >& outSkelMeshArray, TArray<FbxNode*>& SkeletonArray, bool ExpandLOD);

	/**
	 * Import Fbx Morph object for the Skeletal Mesh.  Each morph target import processing occurs in a different thread 
	 *
	 * @param SkelMeshNodeArray - Fbx Nodes that the base Skeletal Mesh construct from
	 * @param BaseSkelMesh - base Skeletal Mesh
	 * @param LODIndex - LOD index of the skeletal mesh
	 * @param bSkinControlPointToTimeZero Use the pose at T0 to map the morph targets onto, since the base mesh was imported at that time, rather than the ref pose.
	 */
	void ImportMorphTargetsInternal( TArray<FbxNode*>& SkelMeshNodeArray, USkeletalMesh* BaseSkelMesh, int32 LODIndex, FSkeletalMeshImportData &BaseSkeletalMeshImportData, bool bMapMorphTargetToTimeZero);

	/**
	* sub-method called from ImportSkeletalMeshLOD method
	*
	* @param InSkeletalMesh - newly created mesh used as LOD
	* @param BaseSkeletalMesh - the destination mesh object 
	* @param DesiredLOD - the LOD index to import into. A new LOD entry is created if one doesn't exist
	*/
	void InsertNewLODToBaseSkeletalMesh(USkeletalMesh* InSkeletalMesh, USkeletalMesh* BaseSkeletalMesh, int32 DesiredLOD, UFbxSkeletalMeshImportData* TemplateImportData);

	/**
	* Method used to verify if the geometry is valid. For example, if the bounding box is tiny we should warn
	* @param StaticMesh - The imported static mesh which we'd like to verify
	*/
	void VerifyGeometry(UStaticMesh* StaticMesh);

	/**
	 * Get the epsilon value, according to the import settings, below which two vertex positions should be considered coincident.
	 */
	float GetPointComparisonThreshold() const;

	/**
	 * Get the epsilon value, according to the import settings, below which a triangle should be considered to have "zero area" and therefore be degenerate.
	 */
	float GetTriangleAreaThreshold() const;

	/**
	* When there is some materials with the same name we add a clash suffixe _ncl1_x.
	* Example, if we have 3 materials name shader we will get (shader, shader_ncl1_1, shader_ncl1_2).
	*/
	void FixMaterialClashName();

	/**
	* Node with no name we will name it "ncl1_x" x is a a unique counter.
	*/
	void EnsureNodeNameAreValid(const FString& BaseFilename);

	/**
	 * Remove metadata starting with the FBX_METADATA_PREFIX value associated to the specified UObject.
	 * This is used to ensure we do not restore old fbx meta data during reimports.
	 * 
	 * @param Object	The UObject for which we want the FBX metadata removed.
	 */
	static void RemoveFBXMetaData(const UObject* Object);

private:
	/**
	 * Helper structure to pass around the common animation parameters.
	 */
	struct FAnimCurveImportSettings
	{
		FAnimCurveImportSettings(UAnimSequence* InDestSeq, const TArray<FbxNode*>& InNodeArray, const TArray<FbxNode*>& InSortedLinks, const TArray<FName>& InFbxRawBoneNames, const FbxTimeSpan& InAnimTimeSpan)
			: DestSeq(InDestSeq)
			, NodeArray(InNodeArray)
			, SortedLinks(InSortedLinks)
			, FbxRawBoneNames(InFbxRawBoneNames)
			, AnimTimeSpan(InAnimTimeSpan)
		{
		}

		UAnimSequence* DestSeq;
		const TArray<FbxNode*>& NodeArray;
		const TArray<FbxNode*>& SortedLinks;
		const TArray<FName>& FbxRawBoneNames;
		const FbxTimeSpan& AnimTimeSpan;
	};

	/**
	 * Import the blendshape curves into the UAnimSequence.
	 * 
	 * @param AnimImportSettings	Common settings to import animation.
	 * @param CurAnimStack			The current anim stack we are importing.
	 * @param OutKeyCount			Out parameter returning the number of keys imported, used to set the number of keys in the sequencer.
 	 * @param bReimport				Flag indicating whether or not this operation is part of reimporting.
	 */
	void ImportBlendShapeCurves(FAnimCurveImportSettings& AnimImportSettings, FbxAnimStack* CurAnimStack, int32& OutKeyCount, const bool bReimport);

	/**
	 * Import the custom attributes curves into the UAnimSequence.
	 * 
	 * @param AnimImportSettings	Common settings to import animation.
	 * @param OutKeyCount			Out parameter returning the number of keys imported, used to set the number of keys in the sequencer.
	 * @param OutCurvesNotFound		Out parameter returning a list of curves name already present in the AnimSequence that were not imported, indicating that some curve data are missing during a reimport.
 	 * @param bReimport				Flag indicating whether or not this operation is part of reimporting.
	 */
	void ImportAnimationCustomAttribute(FAnimCurveImportSettings& AnimImportSettings, int32& OutKeyCount, TArray<FString>& OutCurvesNotFound, const bool bReimport);

	/**
	 * Import the bone transforms curves into the UAnimSequence.
	 * 
	 * @param Skeleton				The skeleton for which we are currently importing the animations.
	 * @param AnimImportSettings	Common settings to import animation.
	 * @param SkeletalMeshRootNode	The fbx root node of the skeletalmesh.
	 * @param ResampleRate			The rate at which the animations are resampled.
	 * @param OutTotalNumKeys		Out parameter returning the number of keys imported, used to set the number of keys in the sequencer.
 	 * @param bReimport				Flag indicating whether or not this operation is part of reimporting.
	 */
	void ImportBoneTracks(USkeleton* Skeleton, FAnimCurveImportSettings& AnimImportSettings, FbxNode* SkeletalMeshRootNode, const int32 ResampleRate, int32& OutTotalNumKeys, const bool bReimport);

public:
	// current Fbx scene we are importing. Make sure to release it after import
	FbxScene* Scene;
	FBXImportOptions* ImportOptions;

	//We cache the hash of the file when we open the file. This is to avoid calculating the hash many time when importing many asset in one fbx file.
	FMD5Hash Md5Hash;

	struct FFbxMaterial
	{
		FbxSurfaceMaterial* FbxMaterial;
		UMaterialInterface* Material;

		FFbxMaterial()
			: FbxMaterial(nullptr)
			, Material(nullptr)
		{}

		FString GetName() const { return FbxMaterial ? UTF8_TO_TCHAR(FbxMaterial->GetName()) : (Material != nullptr ? Material->GetName() : TEXT("None")); }
	};

	FbxGeometryConverter* GetGeometryConverter() { return GeometryConverter; }

	/*
	 * Cleanup the fbx file data so we can read again another file
	 */
	void PartialCleanUp();

	FString GetFbxFileVersion() { return FbxFileVersion; }
	FString GetFileCreator() { return FbxFileCreator; }
	FString GetFileUnitSystem() { return FString(UTF8_TO_TCHAR(FileUnitSystem.GetScaleFactorAsString(false).Buffer())); }
	FString GetFileAxisDirection();

protected:
	enum IMPORTPHASE
	{
		NOTSTARTED,
		FILEOPENED,
		IMPORTED,
		FIXEDANDCONVERTED,
	};
	
	/**
	* Make material Unreal asset name from the Fbx material
	*
	* @param FbxMaterial Material from the Fbx node
	* @return Sanitized asset name
	*/
	FString GetMaterialFullName(const FbxSurfaceMaterial& FbxMaterial) const;

	FString GetMaterialBasePackageName(const FString& MaterialFullName) const;

	static TSharedPtr<FFbxImporter> StaticInstance;
	static TSharedPtr<FFbxImporter> StaticPreviewInstance;
	
	//make sure we are not applying two time the option transform to the same node
	TArray<FbxNode*> TransformSettingsToFbxApply;

	// scene management
	FFbxDataConverter Converter;
	FbxGeometryConverter* GeometryConverter;
	FbxManager* SdkManager;
	FbxImporter* Importer;
	IMPORTPHASE CurPhase;
	FString ErrorMessage;
	// base path of fbx file
	FString FileBasePath;
	TWeakObjectPtr<UObject> Parent;
	FString FbxFileVersion;
	FString FbxFileCreator;

	//Original File Info
	FbxAxisSystem FileAxisSystem;
	FbxSystemUnit FileUnitSystem;

	// Flag that the mesh is the first mesh to import in current FBX scene
	// FBX scene may contain multiple meshes, importer can import them at one time.
	// Initialized as true when start to import a FBX scene
	bool bFirstMesh;
	
	//Value is true if the file was create by blender
	EFbxCreator FbxCreator;
	
	// Set when importing skeletal meshes if the merge bones step fails. Used to track
	// YesToAll and NoToAll for an entire scene
	EAppReturnType::Type LastMergeBonesChoice;

	/**
	 * A map holding the original name of the renamed fbx nodes, 
	 * It is used namely to associate collision meshes to their corresponding static mesh if it has been renamed. 
	 */
	FbxMap<FbxString, FbxString> NodeUniqueNameToOriginalNameMap;

	/**
	 * A map holding pairs of fbx texture that needs to be renamed with the
	 * associated string to avoid name conflicts.
	 */
	TMap<FbxFileTexture*, FString> FbxTextureToUniqueNameMap;

	/**
	 * Collision model list. The key is fbx node name
	 * If there is an collision model with old name format, the key is empty string("").
	 */
	FbxMap<FbxString, TSharedPtr< FbxArray<FbxNode* > > > CollisionModels;
	 
	TArray<TWeakObjectPtr<UObject>> CreatedObjects;

	FFbxImporter();


	/**
	 * Set up the static mesh data from Fbx Mesh.
	 *
	 * @param StaticMesh Unreal static mesh object to fill data into
	 * @param LODIndex	LOD level to set up for StaticMesh
	 * @return bool true if set up successfully
	 */
	bool BuildStaticMeshFromGeometry(FbxNode* Node, UStaticMesh* StaticMesh, TArray<FFbxMaterial>& MeshMaterials, int LODIndex,
									 EVertexColorImportOption::Type VertexColorImportOption, const TMap<FVector3f, FColor>& ExistingVertexColorData, const FColor& VertexOverrideColor);
	
	/**
	 * Clean up for destroy the Importer.
	 */
	void CleanUp();

	/**
	* Compute the global matrix for Fbx Node
	* If we import scene it will return identity plus the pivot if we turn the bake pivot option
	*
	* @param Node	Fbx Node
	* @return KFbxXMatrix*	The global transform matrix
	*/
	FbxAMatrix ComputeTotalMatrix(FbxNode* Node);
	
	/**
	* Compute the matrix for skeletal Fbx Node
	* If we import don't import a scene it will call ComputeTotalMatrix with Node as the parameter. If we import a scene
	* it will return the relative transform between the RootSkeletalNode and Node.
	*
	* @param Node	Fbx Node
	* @param Node	Fbx RootSkeletalNode
	* @return KFbxXMatrix*	The global transform matrix
	*/
	FbxAMatrix ComputeSkeletalMeshTotalMatrix(FbxNode* Node, FbxNode *RootSkeletalNode);
	/**
	 * Check if there are negative scale in the transform matrix and its number is odd.
	 * @return bool True if there are negative scale and its number is 1 or 3. 
	 */
	bool IsOddNegativeScale(FbxAMatrix& TotalMatrix);

	// various actors, current the Fbx importer don't importe them
	/**
	 * Import Fbx light
	 *
	 * @param FbxLight fbx light object 
	 * @param World in which to create the light
	 * @return ALight*
	 */
	ALight* CreateLight(FbxLight* InLight, UWorld* InWorld );	
	/**
	* Import Light detail info
	*
	* @param FbxLight
	* @param UnrealLight
	* @return  bool
	*/
	bool FillLightComponent(FbxLight* Light, ULightComponent* UnrealLight);
	/**
	* Import Fbx Camera object
	*
	* @param FbxCamera Fbx camera object
	* @param World in which to create the camera
	* @return ACameraActor*
	*/
	ACameraActor* CreateCamera(FbxCamera* InCamera, UWorld* InWorld);

	// meshes
	/**
	* Fill skeletal mesh data from Fbx Nodes.  If this function needs to triangulate the mesh, then it could invalidate the
	* original FbxMesh pointer.  Hence FbxMesh is a reference so this function can set the new pointer if need be.  
	*
	* @param ImportData object to store skeletal mesh data
	* @param FbxMesh	Fbx mesh object belonging to Node
	* @param FbxSkin	Fbx Skin object belonging to FbxMesh
	* @param FbxShape	Fbx Morph object, if not NULL, we are importing a morph object.
	* @param SortedLinks    Fbx Links(bones) of this skeletal mesh
	* @param FbxMatList  All material names of the skeletal mesh
	* @param RootNode       The skeletal mesh root fbx node.
	* @param ExistingVertexColorData Map of the existing vertex color data, used in the case we want to ignore the FBX vertex color during reimport.
	*
	* @returns bool*	true if import successfully.
	*/
    bool FillSkelMeshImporterFromFbx(FSkeletalMeshImportData& ImportData, FbxMesh*& Mesh, FbxSkin* Skin, 
										FbxShape* Shape, TArray<FbxNode*> &SortedLinks, const TArray<FbxSurfaceMaterial*>& FbxMaterials, FbxNode *RootNode, const TMap<FVector3f, FColor>& ExistingVertexColorData);
public:

	/**
	* Fill FSkeletalMeshIMportData from Fbx Nodes and FbxShape Array if exists.  
	*
	* @param NodeArray	Fbx node array to look at
	* @param TemplateImportData template import data 
	* @param FbxShapeArray	Fbx Morph object, if not NULL, we are importing a morph object.
	* @param OutData    FSkeletalMeshImportData output data
	* @param ExistingVertexColorData Map of the existing vertex color data, used in the case we want to ignore the FBX vertex color during reimport.
	*
	* @returns bool*	true if import successfully.
	*/
	bool FillSkeletalMeshImportData(TArray<FbxNode*>& NodeArray, UFbxSkeletalMeshImportData* TemplateImportData, TArray<FbxShape*> *FbxShapeArray,
									FSkeletalMeshImportData* OutData, TArray<FbxNode*>& OutImportedSkeletonLinkNodes, TArray<FName> &LastImportedMaterialNames, 
									const bool bIsReimport, const TMap<FVector3f, FColor>& ExistingVertexColorData, bool& bMapMorphTargetToTimeZero);

protected:

	bool ReplaceSkeletalMeshGeometryImportData(const USkeletalMesh* SkeletalMesh, FSkeletalMeshImportData* ImportData, int32 LodIndex);
	bool ReplaceSkeletalMeshSkinningImportData(const USkeletalMesh* SkeletalMesh, FSkeletalMeshImportData* ImportData, int32 LodIndex);

	/**
	* Fill the Points in FSkeletalMeshIMportData from a Fbx Node and a FbxShape if it exists.
	*
	* @param OutData    FSkeletalMeshImportData output data
	* @param RootNode	The root node of the Fbx
	* @param Node		The node to get the points from
	* @param FbxShape	Fbx Morph object, if not NULL, we are importing a morph object.
	*
	* @returns bool		true if import successfully.
	*/
	bool FillSkeletalMeshImportPoints(FSkeletalMeshImportData* OutData, FbxNode* RootNode, FbxNode* Node, FbxShape* FbxShape);

	/**
	* Fill the Points in FSkeletalMeshIMportData from Fbx Nodes and FbxShape Array if it exists.
	*
	* @param OutData		FSkeletalMeshImportData output data
	* @param NodeArray		Fbx node array to look at
	* @param FbxShapeArray	Fbx Morph object, if not NULL, we are importing a morph object.
	* @param ModifiedPoints	Set of points indices for which we've modified the value in OutData
	* @param bInUseT0AsRefPose Use the pose at T0 to map the morph targets onto, since the base mesh was imported at that time, rather than the ref pose.
	*
	* @returns bool			true if import successfully.
	*/
	bool GatherPointsForMorphTarget(FSkeletalMeshImportData* OutData, TArray<FbxNode*>& NodeArray, TArray< FbxShape* >* FbxShapeArray, TSet<uint32>& ModifiedPoints, bool bSkinControlPointToTimeZero);

	/**
	 * Import bones from skeletons that NodeArray bind to.
	 *
	 * @param NodeArray							Fbx Nodes to import, they are bound to the same skeleton system
	 * @param ImportData						object to store skeletal mesh data
	 * @param OutSortedLinks					return all skeletons (bone nodes) sorted by depth traversal
	 * @param bOutDiffPose
	 * @param bDisableMissingBindPoseWarning
	 * @param bUseTime0AsRefPose				in/out - Use Time 0 as Ref Pose
	 * @param SkeletalMeshNode					A pointer to the skeletal mesh node used when we need to calculate the relative transform during scene import
	 * @param bIsReimport						Are we reimporting
	 */
	bool ImportBones(TArray<FbxNode*>& NodeArray, FSkeletalMeshImportData &ImportData, UFbxSkeletalMeshImportData* TemplateData, TArray<FbxNode*> &OutSortedLinks, bool& bOutDiffPose, bool bDisableMissingBindPoseWarning, bool & bUseTime0AsRefPose, FbxNode *SkeletalMeshNode, bool bIsReimport);
	
	/**
	 * Skins the control points of the given mesh or shape using either the default pose for skinning or the first frame of the
	 * default animation.  The results are saved as the last X verts in the given FSkeletalMeshBinaryImport
	 *
	 * @param SkelMeshImporter object to store skeletal mesh data
	 * @param FbxMesh	The Fbx mesh object with the control points to skin
	 * @param FbxShape	If a shape (aka morph) is provided, its control points will be used instead of the given meshes
	 * @param bUseT0	If true, then the pose at time=0 will be used instead of the ref pose
	 */
	void SkinControlPointsToPose(FSkeletalMeshImportData &ImportData, FbxMesh* Mesh, FbxShape* Shape, bool bUseT0 );
	
	// anims
	/**
	 * Check if the Fbx node contains animation
	 *
	 * @param Node Fbx node
	 * @return bool true if the Fbx node contains animation.
	 */
	//bool IsAnimated(FbxNode* Node);

	/**
	* Fill each Trace for AnimSequence with Fbx skeleton animation by key
	*
	* @param Node   Fbx skeleton node
	* @param AnimSequence
	* @param TakeName
	* @param bIsRoot if the Fbx skeleton node is root skeleton
	* @param Scale scale factor for this skeleton node
	*/
	bool FillAnimSequenceByKey(FbxNode* Node, UAnimSequence* AnimSequence, const char* TakeName, FbxTime& Start, FbxTime& End, bool bIsRoot, FbxVector4 Scale);


	// material
	/**
	 * Import each material Input from Fbx Material
	 *
	 * @param FbxMaterial	Fbx material object
	 * @param UnrealMaterial
	 * @param MaterialProperty The material component to import
	 * @param MaterialInput
	 * @param bSetupAsNormalMap
	 * @param UVSet
	 * @return bool	
	 */
	bool CreateAndLinkExpressionForMaterialProperty(	const FbxSurfaceMaterial& FbxMaterial,
														UMaterial* UnrealMaterial,
														const char* MaterialProperty ,
														FExpressionInput& MaterialInput, 
														bool bSetupAsNormalMap,
														TArray<FString>& UVSet,
														const FVector2D& Location );
	/**
	* Create and link texture to the right material parameter value
	*
	* @param FbxMaterial	Fbx material object
	* @param UnrealMaterial
	* @param MaterialProperty The material component to import
	* @param ParameterValue
	* @param bSetupAsNormalMap
	* @return bool
	*/
	bool LinkMaterialProperty(const FbxSurfaceMaterial& FbxMaterial,
		UMaterialInstanceConstant* UnrealMaterial,
		const char* MaterialProperty,
		FName ParameterValue,
		bool bSetupAsNormalMap);
	/**
	 * Add a basic white diffuse color if no expression is linked to diffuse input.
	 *
	 * @param unMaterial Unreal material object.
	 */
	void FixupMaterial( const FbxSurfaceMaterial& FbxMaterial, UMaterial* unMaterial);
	
	/**
	 * Get material mapping array according "Skinxx" flag in material name
	 *
	 * @param FSkeletalMeshBinaryImport& The unreal skeletal mesh.
	 */
	void SetMaterialSkinXXOrder(FSkeletalMeshImportData& ImportData);
	
	void SetMaterialOrderByName(FSkeletalMeshImportData& ImportData, TArray<FName> LastImportedMaterialNames);

	/**
	* Make sure there is no unused material in the raw data. Unused material are material refer by node but not refer by any geometry face
	*
	* @param FSkeletalMeshBinaryImport& The unreal skeletal mesh.
	*/
	void CleanUpUnusedMaterials(FSkeletalMeshImportData& ImportData);

	/**
	 * Create materials from Fbx node.
	 * Only setup channels that connect to texture, and setup the UV coordinate of texture.
	 * If diffuse channel has no texture, one default node will be created with constant.
	 * If a material cannot be imported a nullptr will be insterted in the outMaterials array in its place.
	 *
	 * @param FbxNode  Fbx node
	 * @param outMaterials Unreal Materials we created
	 * @param UVSets UV set name list
	 * @return int32 material count that created from the Fbx node
	 */
	void FindOrImportMaterialsFromNode(FbxNode* FbxNode, TArray<UMaterialInterface*>& outMaterials, TArray<FString>& UVSets, bool bForSkeletalMesh);

	/**
	 * Tries to find an existing UnrealMaterial from the FbxMaterial, returns nullptr if could not find a material.
	 * The function will look for materials imported by the FbxFactory first,
	 * and then search into the asset database using the passed MaterialSearchLocation search scope.
	 * 
	 * @param FbxMaterial				The FbxMaterial used to search the UnrealMaterial
	 * @param MaterialSearchLocation	The asset database search scope.
	 * @return							The UMaterialInterfaceFound, returns nullptr if no material was found.
	 */
	UMaterialInterface* FindExistingMaterialFromFbxMaterial(const FbxSurfaceMaterial& FbxMaterial, EMaterialSearchLocation MaterialSearchLocation);

	/**
	 * Create Unreal material from Fbx material.
	 * Only setup channels that connect to texture, and setup the UV coordinate of texture.
	 * If diffuse channel has no texture, one default node will be created with constant.
	 *
	 * @param KFbxSurfaceMaterial*  Fbx material
	 * @param outUVSets
	 * @param bForSkeletalMesh		If set to true, the material target usage will be set to "SkeletalMesh".
	 * @return						The created material.
	 */
	UMaterialInterface* CreateUnrealMaterial(const FbxSurfaceMaterial& FbxMaterial, TArray<FString>& OutUVSets, bool bForSkeletalMesh);


	/**
	 * Visit all materials of one node, import textures from materials.
	 *
	 * @param Node FBX node.
	 */
	void ImportTexturesFromNode(FbxNode* Node);
	
	/**
	 * Generate Unreal texture object from FBX texture.
	 *
	 * @param FbxTexture FBX texture object to import.
	 * @param bSetupAsNormalMap Flag to import this texture as normal map.
	 * @return UTexture* Unreal texture object generated.
	 */
	UTexture* ImportTexture(FbxFileTexture* FbxTexture, bool bSetupAsNormalMap);
	
	/**
	 *
	 *
	 * @param
	 * @return UMaterial*
	 */
	//UMaterial* GetImportedMaterial(KFbxSurfaceMaterial* pMaterial);

	/**
	* Check if the meshes in FBX scene contain smoothing group info.
	* It's enough to only check one of mesh in the scene because "Export smoothing group" option affects all meshes when export from DCC.
	* To ensure only check one time, use flag bFirstMesh to record if this is the first mesh to check.
	*
	* @param FbxMesh Fbx mesh to import
	*/
	void CheckSmoothingInfo(FbxMesh* FbxMesh);

	/**
	 * check if two faces belongs to same smoothing group
	 *
	 * @param ImportData
	 * @param Face1 one face of the skeletal mesh
	 * @param Face2 another face
	 * @return bool true if two faces belongs to same group
	 */
	bool FacesAreSmoothlyConnected( FSkeletalMeshImportData &ImportData, int32 Face1, int32 Face2 );

	/**
	 * Make un-smooth faces work.
	 *
	 * @param ImportData
	 * @return int32 number of points that added when process unsmooth faces
	*/
	int32 DoUnSmoothVerts(FSkeletalMeshImportData &ImportData, bool bDuplicateUnSmoothWedges = true);
	
	/**
	* Fill the FbxNodeInfo structure recursively to reflect the FbxNode hierarchy. The result will be an array sorted with the parent first
	*
	* @param SceneInfo   The scene info to modify
	* @param Parent      The parent FbxNode
	* @param ParentInfo  The parent FbxNodeInfo
	*/
	void TraverseHierarchyNodeRecursively(FbxSceneInfo& SceneInfo, FbxNode *ParentNode, FbxNodeInfo &ParentInfo);


	//
	// for sequencer import
	//
public:
	UNREALED_API void PopulateAnimatedCurveData(FFbxCurvesAPI &CurvesAPI);

protected:
	void LoadNodeKeyframeAnimationRecursively(FFbxCurvesAPI &CurvesAPI, FbxNode* NodeToQuery);
	void LoadNodeKeyframeAnimation(FbxNode* NodeToQuery, FFbxCurvesAPI &CurvesAPI);
	void SetupTransformForNode(FbxNode *Node);

	/** Create a new asset from the package and objectname and class */
	static UObject* CreateAssetOfClass(UClass* AssetClass, FString ParentPackageName, FString ObjectName, bool bAllowReplace = false);

	/* Templated function to create an asset with given package and name */
	template< class T>
	static T* CreateAsset(FString ParentPackageName, FString ObjectName, bool bAllowReplace = false)
	{
		return (T*)CreateAssetOfClass(T::StaticClass(), ParentPackageName, ObjectName, bAllowReplace);
	}

	/**
	 * Fill up and verify bone names for animation 
	 */
	void FillAndVerifyBoneNames(USkeleton* Skeleton, TArray<FbxNode*>& SortedLinks, TArray<FName> & OutRawBoneNames, FString Filename);
	/**
	 * Is valid animation data
	 */
	bool IsValidAnimationData(TArray<FbxNode*>& SortedLinks, TArray<FbxNode*>& NodeArray, int32& ValidTakeCount);

	/**
	 * Retrieve pose array from bind pose
	 *
	 * Iterate through Scene:Poses, and find valid bind pose for NodeArray, and return those Pose if valid
	 *
	 */
	bool RetrievePoseFromBindPose(const TArray<FbxNode*>& NodeArray, FbxArray<FbxPose*> & PoseArray) const;

	/** Import the user-defined properties on the node as FBX metadata on the object */
	void ImportNodeCustomProperties(UObject* Object, FbxNode* Node, bool bPrefixTagWithNodeName = false);

public:
	/** Import and set up animation related data from mesh **/
	void SetupAnimationDataFromMesh(USkeletalMesh * SkeletalMesh, UObject* InParent, TArray<FbxNode*>& NodeArray, UFbxAnimSequenceImportData* ImportData, const FString& Filename);

	/** error message handler */
	UNREALED_API void AddTokenizedErrorMessage(TSharedRef<FTokenizedMessage> Error, FName FbxErrorName );
	void ClearTokenizedErrorMessages();
	void FlushToTokenizedErrorMessage(enum EMessageSeverity::Type Severity);

	float GetOriginalFbxFramerate() { return OriginalFbxFramerate; }

	/**
	 * Returns true if the last import operation was canceled.
	 */
	bool GetImportOperationCancelled() const { return bImportOperationCanceled; }

private:
	friend class FFbxLoggerSetter;
	friend struct FFbxScopedOperation;

	// logger set/clear function
	class FFbxLogger * Logger;
	UNREALED_API void SetLogger(class FFbxLogger * InLogger);
	UNREALED_API void ClearLogger();

	FImportedMaterialData ImportedMaterialData;

	//Cache to create unique name for mesh. This is use to fix name clash
	TArray<FString> MeshNamesCache;

	float OriginalFbxFramerate;

	/** 
	 * Holds if the current import operation was canceled or not.
	 */
	bool bImportOperationCanceled = false;

	/**
	 * Internal counter used to group import operation together when canceling (ie: Import Skeletal Mesh can trigger Import of Morph Targets.)
	 */
	int32 ImportOperationStack = 0;

private:

	/**
	 * Import FbxCurve to anim sequence
	 */
	bool ImportCurveToAnimSequence(class UAnimSequence * TargetSequence, const FString& CurveName, const FbxAnimCurve* FbxCurve, int32 CurveFlags,const FbxTimeSpan& AnimTimeSpan, const bool bReimport, float ValueScale = 1.f) const;

	/**
	 * Import rich Curves to anim sequence
	 */
	bool ImportRichCurvesToAnimSequence(class UAnimSequence * TargetSequence, const TArray<FString>& CurveNames, const TArray<FRichCurve> RichCurves, int32 CurveFlags, const bool bReimport) const;
	
	/**
	 * Given a primary blend shape channel curve and inbetween target full weights,
	 * generate curves for each target as if they are standalone blend shapes
	 * while preserving the animation
	 */
	TArray<FRichCurve> ResolveWeightsForBlendShapeCurve(FRichCurve& ChannelWeightCurve, const TArray<float>& InbetweenFullWeights) const;
	
	/**
	 * Given a primary blend shape channel curve value and inbetween target full weights,
	 * calculate the curve value for each target as if they are standalone blend shapes
	 * while preserving the animation
	 */
	void ResolveWeightsForBlendShape(const TArray<float>& InbetweenFullWeights , float InWeight, float& OutMainWeight, TArray<float>& OutInbetweenWeights) const;
	
	/**
	 * Import custom attribute (curve or not) to the associated bone.
	 *
	 * @return Returns true if the given custom attribute was properly added to the bone, false otherwise.
	 */
	bool ImportCustomAttributeToBone(class UAnimSequence* TargetSequence, FbxProperty& InProperty, FName BoneName, const FString& CurveName, const FbxAnimCurve* FbxCurve, const FbxTimeSpan& AnimTimeSpan, const bool bReimport, float ValueScale=1.f);
};


/** message Logger for FBX. Saves all the messages and prints when it's destroyed */
class FFbxLogger
{
	UNREALED_API FFbxLogger();
	UNREALED_API ~FFbxLogger();

	/** Error messages **/
	TArray<TSharedRef<FTokenizedMessage>> TokenizedErrorMessages;

	/* The logger will show the LogMessage only if at least one TokenizedErrorMessage have a severity of Error or CriticalError*/
	bool ShowLogMessageOnlyIfError;

	friend class FFbxImporter;
	friend class FFbxLoggerSetter;
};

/**
* This class is to make sure Logger isn't used by outside of purpose.
* We add this only top level of functions where it needs to be handled
* if the importer already has logger set, it won't set anymore
*/
class FFbxLoggerSetter
{
	class FFbxLogger Logger;
	FFbxImporter * Importer;

public:
	FFbxLoggerSetter(FFbxImporter * InImpoter, bool ShowLogMessageOnlyIfError = false)
		: Importer(InImpoter)
	{
		// if impoter doesn't have logger, sets it
		if(Importer->Logger == NULL)
		{
			Logger.ShowLogMessageOnlyIfError = ShowLogMessageOnlyIfError;
			Importer->SetLogger(&Logger);
		}
		else
		{
			// if impoter already has logger set
			// invalidated Importer to make sure it doesn't clear
			Importer = NULL;
		}
	}

	~FFbxLoggerSetter()
	{
		if(Importer)
		{
			Importer->ClearLogger();
		}
	}
};

struct FFbxScopedOperation
{
public:
	FFbxScopedOperation(FFbxImporter* FbxImporter);
	~FFbxScopedOperation();

private:
	FFbxImporter* Importer;
};

} // namespace UnFbx


