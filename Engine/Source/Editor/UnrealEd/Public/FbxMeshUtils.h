// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "CoreMinimal.h"

class UFbxImportUI;
class UStaticMesh;
class USkeletalMesh;
class UObject;

namespace FbxMeshUtils
{
	/**
	 * Imports a mesh LOD to the given static mesh
	 *
	 * @param BaseStaticMesh	The static mesh to import the LOD to
	 * @param Filename			The filename of the FBX file containing the LOD
	 * @param LODLevel			The level of the lod to import
	 * @return					Whether or not the import succeeded
	 */
	UNREALED_API bool ImportStaticMeshLOD( UStaticMesh* BaseStaticMesh, const FString& Filename, int32 LODLevel, bool bAsync);

	/**
	 * Imports a mesh as the high res source model of the given static mesh
	 *
	 * @param BaseStaticMesh	The static mesh to import the high res source model to
	 * @param Filename			The filename of the FBX file containing the mesh
	 * @return					Whether or not the import succeeded
	 */
	UNREALED_API bool ImportStaticMeshHiResSourceModel( UStaticMesh* BaseStaticMesh, const FString& Filename, bool bAsync);

	/**
	 * Imports a skeletal mesh LOD to the given skeletal mesh
	 *
	 * @param Mesh				The skeletal mesh to import the LOD to
	 * @param Filename			The filename of the FBX file containing the LOD
	 * @param LODLevel			The level of the lod to import
	 * @return					Whether or not the import succeeded
	 */
	UNREALED_API bool ImportSkeletalMeshLOD( USkeletalMesh* Mesh, const FString& Filename, int32 LODLevel, bool bAsync);

	/**
	 * Imports a mesh LOD to the given mesh
	 * Opens file dialog to do so!
	 *
	 * @param Mesh				The mesh to import the LOD to
	 * @param LODLevel			The level of the lod to import
	 * @return					Whether or not the import succeeded
	 */
	UNREALED_API TFuture<bool> ImportMeshLODDialog( UObject* Mesh, int32 LODLevel, bool bNotifyCB = true, bool bReimportWithNewFile = false);

	/**
	 * Imports a mesh as the high res source model of the given mesh
	 * Opens file dialog to do so!
	 *
	 * @param Mesh				The mesh to import the high res mesh to
	 * @return					Whether or not the import succeeded
	 */
	UNREALED_API bool ImportStaticMeshHiResSourceModelDialog( UStaticMesh* StaticMesh );

	/**
	 * Remove the nanite data for the specified staticmesh
	 *
	 * @param Mesh				The mesh to import the high res mesh to
	 * @return					Whether or not the import succeeded
	 */
	UNREALED_API bool RemoveStaticMeshHiRes(UStaticMesh* StaticMesh);

	/**
	 * Sets import option before importing
	 *
	 * @param ImportUI			The importUI you'd like to apply to
	 */
	UNREALED_API void SetImportOption(UFbxImportUI* ImportUI);

}  //end namespace ExportMeshUtils
