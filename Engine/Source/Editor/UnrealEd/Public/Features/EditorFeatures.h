// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NameTypes.h"

namespace EditorFeatures
{
	/** Allows the user to manage a list of installed plugins */
	UNREALED_API extern const FName PluginsEditor;
}

