// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeature.h"
#include "ModuleDescriptor.h"

struct FSlateDynamicImageBrush;
class IPlugin;
class IDetailLayoutBuilder;
struct FPluginDescriptor;
class SWidget;

/**
 * Description of a plugin template
 */
struct FPluginTemplateDescription
{
	/** Name of this template in the GUI */
	FText Name;

	/** Description of this template in the GUI */
	FText Description;

	/** Path to the directory containing template files */
	FString OnDiskPath;

	/** Brush resource for the image that is dynamically loaded */
	TSharedPtr< FSlateDynamicImageBrush > PluginIconDynamicImageBrush;

	/** Sorting priority (higer values go to the top of the list) */
	int32 SortPriority = 0;

	/** Can the plugin contain content? */
	bool bCanContainContent = false;

	/** Can the plugin be in the engine folder? */
	bool bCanBePlacedInEngine = true;

	/** What is the expected ModuleDescriptor type for this plugin? */
	EHostType::Type ModuleDescriptorType;

	/** What is the expected Loading Phase for this plugin? */
	ELoadingPhase::Type LoadingPhase;

	/** Can the plugin be used in a content only project? */
	bool bSupportsContentOnlyProjects;
	
	/** Python script to run after a plugin is created from this template */
	FString PostCreatePythonScriptPath;
	FString PostCreatePythonScriptArguments;

	/** Called just before the plugin is created */
	virtual void CustomizeDescriptorBeforeCreation(FPluginDescriptor& Descriptor) {}

	/** Called after the plugin has been created */
	virtual void OnPluginCreated(TSharedPtr<IPlugin> NewPlugin) {}

	/** Called to perform *additional* path validation when the path is modified (the bCanBePlacedInEngine validation will have already occurred and passed by this point) */
	virtual bool ValidatePathForPlugin(const FString& ProposedAbsolutePluginPath, FText& OutErrorMessage)
	{
		OutErrorMessage = FText::GetEmpty();
		return true;
	}

	/** Called to enforce any restrictions this template has on paths when it is first selected (so it doesn't generate an error unnecessarily) */
	virtual void UpdatePathWhenTemplateSelected(FString& InOutPath) {}

	/** Called to change away from special folders if needed */
	virtual void UpdatePathWhenTemplateUnselected(FString& InOutPath) {}

	/** Called to optionally set the default plugin name text */
	virtual void UpdatePluginNameTextWhenTemplateSelected(FText& OutPluginNameText) {}

	/** Called to optionally unset the default plugin name text */
	virtual void UpdatePluginNameTextWhenTemplateUnselected(FText& OutPluginNameText) {}

	/** Constructor */
	FPluginTemplateDescription(FText InName, FText InDescription, FString InOnDiskPath, bool InCanContainContent, EHostType::Type InModuleDescriptorType, ELoadingPhase::Type InLoadingPhase = ELoadingPhase::Default, bool InSupportsContentOnlyProjects = true, FString InPostCreatePythonScriptPath = FString(), FString InPostCreatePythonScriptArguments = FString())
		: Name(InName)
		, Description(InDescription)
		, OnDiskPath(InOnDiskPath)
		, bCanContainContent(InCanContainContent)
		, ModuleDescriptorType(InModuleDescriptorType)
		, LoadingPhase(InLoadingPhase)
		, bSupportsContentOnlyProjects(InSupportsContentOnlyProjects)
		, PostCreatePythonScriptPath(InPostCreatePythonScriptPath)
		, PostCreatePythonScriptArguments(InPostCreatePythonScriptArguments)
	{
	}

	virtual ~FPluginTemplateDescription() = default;
};

struct FPluginEditingContext
{
	TSharedPtr<IPlugin> PluginBeingEdited;

};

/**
 * Description of a plugin editor extension
 */
struct FPluginEditorExtension : public TSharedFromThis<FPluginEditorExtension>
{
	virtual ~FPluginEditorExtension() = default;
	virtual void CommitEdits(FPluginDescriptor& Descriptor) = 0;
};

/** Notification when editing a plugin to allow further customization */
DECLARE_DELEGATE_RetVal_TwoParams(TSharedPtr<FPluginEditorExtension>, FOnPluginBeingEdited, FPluginEditingContext& /*InPluginContext*/, IDetailLayoutBuilder& /*DetailBuilder*/);

using FPluginEditorExtensionHandle = int32;
const FPluginEditorExtensionHandle PluginEditorExtensionInvalidHandle = 0;

/**
 * Feature interface for a Plugins management UI
 */
class IPluginsEditorFeature : public IModularFeature
{

public:
	/**
	  * Registers the specified plugin template with the New Plugin wizard
	  */
	virtual void RegisterPluginTemplate(TSharedRef<FPluginTemplateDescription> Template) = 0;

	/**
	  * Unregisters the specified plugin template from the New Plugin wizard
	  */
	virtual void UnregisterPluginTemplate(TSharedRef<FPluginTemplateDescription> Template) = 0;

	/**
	  * Further customize the detail panel experience when editing a specific plugin
	  */
	virtual FPluginEditorExtensionHandle RegisterPluginEditorExtension(FOnPluginBeingEdited Extension) = 0;

	/**
	  * Remove a customization
	  */
	virtual void UnregisterPluginEditorExtension(FPluginEditorExtensionHandle ExtensionHandle) = 0;

	/**
	  * Returns the currently registered plugin templates.
	  */
	virtual const TArray<TSharedRef<FPluginTemplateDescription>>& GetAddedPluginTemplates() const = 0;

	/**
	  * Returns this PluginEditor's default plugin templates.
	  */
	virtual const TArray<TSharedRef<FPluginTemplateDescription>>& GetDefaultPluginTemplates() const
	{
		static const TArray<TSharedRef<FPluginTemplateDescription>> EmptyResult;
		return EmptyResult;
	}

	/**
	 * Open the plugin editor for a plugin
	 */
	virtual void OpenPluginEditor(TSharedRef<IPlugin> PluginToEdit, TSharedPtr<SWidget> ParentWidget, FSimpleDelegate OnEditCommitted) = 0;
};

