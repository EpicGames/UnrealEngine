// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FeedbackContextEditor.h: Feedback context tailored to UnrealEd

=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Misc/MessageDialog.h"
#include "Misc/FeedbackContext.h"
#include "Widgets/SWindow.h"

class FContextSupplier;
class SBuildProgressWidget;

/**
 * A FFeedbackContext implementation for use in UnrealEd.
 */
class FFeedbackContextEditor : public FFeedbackContext
{
	/** Slate slow task widget */
	TWeakPtr<class SWindow> SlowTaskWindow;

	/** Special Windows/Widget popup for building */
	TWeakPtr<class SWindow> BuildProgressWindow;
	TSharedPtr<class SBuildProgressWidget> BuildProgressWidget;

	bool HasTaskBeenCancelled;
	bool bFallbackSplashScreenVisible = false;

	double SlowTaskStartTime = 0;

public:

	UNREALED_API FFeedbackContextEditor();

	UNREALED_API virtual void StartSlowTask( const FText& Task, bool bShowCancelButton=false ) override;
	UNREALED_API virtual void FinalizeSlowTask( ) override;
	UNREALED_API virtual void ProgressReported( const float TotalProgressInterp, FText DisplayMessage ) override;
	UNREALED_API virtual bool IsPlayingInEditor() const override;

	void SetContext( FContextSupplier* InSupplier ) override {}

	/**
	 * Whether or not the user has canceled out of the progress dialog 
	 * (i.e. the ongoing slow task or the last one that ran).
	 * The user cancel flag is reset when starting a new root slow task.
	 */
	UNREALED_API virtual bool ReceivedUserCancel() override;

	UNREALED_API void OnUserCancel();

	virtual bool YesNof( const FText& Question ) override
	{
		return EAppReturnType::Yes == FMessageDialog::Open( EAppMsgType::YesNo, Question );
	}

	/** 
	 * Show the Build Progress Window 
	 * @return Handle to the Build Progress Widget created
	 */
	UNREALED_API TWeakPtr<class SBuildProgressWidget> ShowBuildProgressWindow() override;
	
	/** Close the Build Progress Window */
	UNREALED_API void CloseBuildProgressWindow() override;
};
