// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetRegistry/AssetData.h"
#include "PackageTools.h"
#include "ISourceControlProvider.h"
#include "UObject/TextProperty.h"
#include "FileHelpers.generated.h"

class ULevel;

enum EFileInteraction
{
	FI_Load,
	FI_Save,
	FI_ImportScene,
	FI_ExportScene
};

namespace EAutosaveContentPackagesResult
{
	enum Type
	{
		Success,
		NothingToDo,
		Failure
	};
}

/**
 * This class is a wrapper for editor loading and saving functionality
 * It is meant to contain only functions that can be executed in script (but are also allowed in C++).
 * It is separated from FEditorFileUtils to ensure new easier to use methods can be created without breaking FEditorFileUtils backwards compatibility
 * However this should be used in place of FEditorFileUtils wherever possible as the goal is to deprecate FEditorFileUtils eventually
 */
UCLASS(transient)
class UEditorLoadingAndSavingUtils : public UObject
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category= "Editor Scripting | Editor Loading and Saving")
	static UNREALED_API UWorld* NewBlankMap(bool bSaveExistingMap);

	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Loading and Saving")
	static UNREALED_API UWorld* NewMapFromTemplate(const FString& PathToTemplateLevel, bool bSaveExistingMap);

	/**
	 * Prompts the user to save the current map if necessary, the presents a load dialog and
	 * loads a new map if selected by the user.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Loading and Saving")
	static UNREALED_API UWorld* LoadMapWithDialog();

	/**
	 * Loads the specified map.  Does not prompt the user to save the current map.
	 *
	 * @param	Filename		Level package filename, including path.
	 * @return					true if the map was loaded successfully.
	 */
	UFUNCTION(BlueprintCallable, Category="Editor Scripting | Editor Loading and Saving")
	static UNREALED_API UWorld* LoadMap(const FString& Filename);

	/**
	 * Saves the specified map, returning true on success.
	 *
	 * @param	World			The world to save.
	 * @param	AssetPath		The valid content directory path and name for the asset.  E.g "/Game/MyMap"
	 *
	 * @return					true if the map was saved successfully.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Loading and Saving")
	static UNREALED_API bool SaveMap(UWorld* World, const FString& AssetPath);

	/**
	 * Save all packages.
	 * Assume all dirty packages should be saved and check out from source control (if enabled).
	 *
	 * @param		PackagesToSave				The list of packages to save.  Both map and content packages are supported
	 * @param		bCheckDirty					If true, only packages that are dirty in PackagesToSave will be saved
	 * @return									true on success, false on fail.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Loading and Saving")
	static UNREALED_API bool SavePackages(const TArray<UPackage*>& PackagesToSave, bool bOnlyDirty);

	/**
	* Save all packages. Optionally prompting the user to select which packages to save.
	* Prompt the user to select which dirty packages to save and check them out from source control (if enabled).
	*
	* @param		PackagesToSave				The list of packages to save.  Both map and content packages are supported
	* @param		bCheckDirty					If true, only packages that are dirty in PackagesToSave will be saved
	* @return									true on success, false on fail.
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Loading and Saving")
	static UNREALED_API bool SavePackagesWithDialog(const TArray<UPackage*>& PackagesToSave, bool bOnlyDirty);

	/**
	 * Looks at all currently loaded packages and saves them if their "bDirty" flag is set.
	 * Assume all dirty packages should be saved and check out from source control (if enabled).
	 *
	 * @param	bSaveMapPackages			true if map packages should be saved
	 * @param	bSaveContentPackages		true if we should save content packages.
	 * @return								true on success, false on fail.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Loading and Saving")
	static UNREALED_API bool SaveDirtyPackages(const bool bSaveMapPackages, const bool bSaveContentPackages);

	/**
	 * Looks at all currently loaded packages and saves them if their "bDirty" flag is set.
	 * Prompt the user to select which dirty packages to save and check them out from source control (if enabled).
	 *

	 * @param	bSaveMapPackages			true if map packages should be saved
	 * @param	bSaveContentPackages		true if we should save content packages.
	 * @return								true on success, false on fail.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Loading and Saving")
	static UNREALED_API bool SaveDirtyPackagesWithDialog(const bool bSaveMapPackages, const bool bSaveContentPackages);

	/**
	 * Saves the active level, prompting the use for checkout if necessary.
	 *
	 * @return	true on success, False on fail
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Loading and Saving")
	static UNREALED_API bool SaveCurrentLevel();

	/**
	 * Appends array with all currently dirty map packages.
	 *
	 * @param OutDirtyPackages Array to append dirty packages to.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Loading and Saving")
	static UNREALED_API void GetDirtyMapPackages(TArray<UPackage*>& OutDirtyPackages);

	/**
	 * Appends array with all currently dirty content packages.
	 *
	 * @param OutDirtyPackages Array to append dirty packages to.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Loading and Saving")
	static UNREALED_API void GetDirtyContentPackages(TArray<UPackage*>& OutDirtyPackages);

	/**	
	 * Imports a file such as (FBX or obj) and spawns actors f into the current level
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Loading and Saving")
	static UNREALED_API void ImportScene(const FString& Filename);


	/**
	 * Exports the current scene 
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Loading and Saving")
	static UNREALED_API void ExportScene(bool bExportSelectedActorsOnly);

	/**
	* Unloads a list of packages
	*
	* @param PackagesToUnload Array of packages to unload.
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Loading and Saving")
	static UNREALED_API void UnloadPackages(const TArray<UPackage*>& PackagesToUnload, bool& bOutAnyPackagesUnloaded, FText& OutErrorMessage);

	/**
	 * Helper function that attempts to reload the specified top-level packages.
	 *
	 * @param	PackagesToReload		The list of packages that should be reloaded
	 * @param	bOutAnyPackagesReloaded	True if the set of loaded packages was changed
	 * @param	OutErrorMessage			An error message specifying any problems with reloading packages
	 * @param	InteractionMode			Whether the function is allowed to ask the user questions (such as whether to reload dirty packages)
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Loading and Saving")
	static UNREALED_API void ReloadPackages(const TArray<UPackage*>& PackagesToReload, bool& bOutAnyPackagesReloaded, FText& OutErrorMessage, const EReloadPackagesInteractionMode InteractionMode = EReloadPackagesInteractionMode::Interactive);

	/**
	 * Fully loads the given packages, so that UE no longer has a write lock on their on-disk files.
	 *
	 * @param Packages Array of packages to fully load.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Loading and Saving")
	static UNREALED_API void FullyLoadPackages(const TArray<UPackage*>& Packages);

	/**
	 * Fully loads the given assets, so that UE no longer has a write lock on their on-disk files.
	 *
	 * @param Assets Array of assets to fully load.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Loading and Saving")
	static UNREALED_API void FullyLoadAssets(const TArray<UObject*>& Assets);
};


/**
 * For saving map files through the main editor frame.
 */
class FEditorFileUtils
{
public:
	/** Dirty Package Ignore */
	using FShouldIgnorePackageFunctionRef = TFunctionRef<bool(UPackage*)>;
	struct FShouldIgnorePackage
	{
		static bool Default(UPackage*) { return false; }
	};

	
	/** Used to decide how to handle garbage collection. */
	enum EGarbageCollectionOption
	{
		GCO_SkipGarbageCollection	= 0,
		GCO_CollectGarbage			= 1,
	};

	DECLARE_MULTICAST_DELEGATE(FOnLoadMapStart);
	static UNREALED_API FOnLoadMapStart& GetOnLoadMapStartDelegate();

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnLoadMapEnd, const FString& Filename);
	static UNREALED_API FOnLoadMapEnd& GetOnLoadMapEndDelegate();


	/** Sets the active level filename so that "Save" operates on this file and "SaveAs" must be used on others */
	static void RegisterLevelFilename(UObject* Object, const FString& NewLevelFilename);
	
	////////////////////////////////////////////////////////////////////////////
	// ResetLevelFilenames

	/**
	 * Clears current level filename so that the user must SaveAs on next Save.
	 * Called by NewMap() after the contents of the map are cleared.
	 * Also called after loading a map template so that the template isn't overwritten.
	 */
	static void ResetLevelFilenames();	

	////////////////////////////////////////////////////////////////////////////
	// Loading

	DECLARE_DELEGATE_OneParam(FOnLevelsChosen, const TArray<FAssetData>& /*SelectedLevels*/);
	DECLARE_DELEGATE(FOnLevelPickingCancelled);

	/**
	 * Opens a non-modal dialog to allow the user to choose a level
	 *
	 * @param	OnLevelsChosen				Delegate executed when one more more levels have been selected
	 * @param	OnLevelPickingDialogClosed	Delegate executed when the level picking dialog is closed
	 * @param	bAllowMultipleSelection		If true, more than one level can be chosen
	 */
	static UNREALED_API void OpenLevelPickingDialog(const FOnLevelsChosen& OnLevelsChosen, const FOnLevelPickingCancelled& OnLevelPickingCancelled, bool bAllowMultipleSelection);

	/**
	 * Returns true if the specified map filename is valid for loading or saving.
	 * When returning false, OutErrorMessage is supplied with a display string describing the reason why the map name is invalid.
	 */
	static UNREALED_API bool IsValidMapFilename(const FString& MapFilename, FText& OutErrorMessage);

	/**
	 * Unloads the specified package potentially containing an inactive world.
	 * When returning false, OutErrorMessage is supplied with a display string describing the reason why the world could not be unloaded.
	 */
	static UNREALED_API bool AttemptUnloadInactiveWorldPackage(UPackage* PackageToUnload, FText& OutErrorMessage);

	/**
	 * Prompts the user to save the current map if necessary, the presents a load dialog and
	 * loads a new map if selected by the user.
	 */
	static UNREALED_API bool LoadMap();

	/**
	 * Loads the specified map.  Does not prompt the user to save the current map.
	 *
	 * @param	Filename		Map package filename, including path.
	 *
	 * @param	LoadAsTemplate	Forces the map to load into an untitled outermost package
	 *							preventing the map saving over the original file.
	 * @param	bShowProgress	Whether to show a progress dialog as the map loads\
	 * @return	true on success, false otherwise
	 */
	static UNREALED_API bool LoadMap(const FString& Filename, bool LoadAsTemplate = false, const bool bShowProgress=true);

	////////////////////////////////////////////////////////////////////////////
	// Saving

	/**
	 * Saves the specified map package, returning true on success.
	 *
	 * @param	World			The world to save.
	 * @param	Filename		Map package filename, including path.
	 *
	 * @return					true if the map was saved successfully.
	 */
	static UNREALED_API bool SaveMap(UWorld* World, const FString& Filename );

	/**
	 * Saves the specified level.  SaveAs is performed as necessary.
	 *
	 * @param	Level				The level to be saved.
	 * @param	DefaultFilename		File name to use for this level if it doesn't have one yet (or empty string to prompt)
	 * @param	OutSavedFilename	When returning true, this string will be set to the filename of the saved level.
	 *
	 * @return				true if the level was saved.
	 */
	static UNREALED_API bool SaveLevel(ULevel* Level, const FString& DefaultFilename = TEXT( "" ), FString* OutSavedFilename = nullptr );

	/** 
	 * Saves packages which contain map data but are not map packages themselves. 
	 * 
	 * @param	World				The world map data packages to be saved.
	 * @param	bCheckDirty			If true, only packages that are dirty will be saved.
	 * 
	 * @return				true if the data packages were saved.
	 */
	static UNREALED_API bool SaveMapDataPackages(UWorld* World, bool bCheckDirty);

	/**
	 * Does a SaveAs for the specified assets.
	 *
	 * @param Assets The collection of assets to save.
	 * @param SavedAssets The collection of corresponding saved assets (contains original asset if not resaved).
	 */
	UNREALED_API static void SaveAssetsAs(const TArray<UObject*>& Assets, TArray<UObject*>& OutSavedAssets);

	/**
	 * Does a saveAs for the specified level.
	 *
	 * @param	Level				The Level to be SaveAs'd.
	 * @param	OutSavedFilename	When returning true, this string will be set to the filename of the saved level.
	 * @return						true if the world was saved.
	 */
	UNREALED_API static bool SaveLevelAs(ULevel* Level, FString* OutSavedFilename = nullptr);

	/**
	 * Get the autosave filename for the given package.
	 * 
	 * @param	Package						Package to get the autosave filename for.
	 * @param	AbsoluteAutosaveDir			Autosave directory.
	 * @param	AutosaveIndex				Integer prepended to autosave filenames.
	 * @param	PackageExt					Extension to use for the given package. Note: This must include the dot.
	 */
	static FString GetAutoSaveFilename(UPackage* const Package, const FString& AbsoluteAutosaveDir, const int32 AutoSaveIndex, const FString& PackageExt);

	/**
	 * Saves all levels to the specified directory.
	 *
	 * @param	AbsoluteAutosaveDir			Autosave directory.
	 * @param	AutosaveIndex				Integer prepended to autosave filenames..
	 * @param	bForceIfNotInList			Should the save be forced if the package is dirty, but not in DirtyPackagesForAutoSave?
	 * @param	DirtyPackagesForAutoSave	A set of packages that are considered by the auto-save system to be dirty, you should check this to see if a package needs saving
	 */
	static bool AutosaveMap(const FString& AbsoluteAutosaveDir, const int32 AutosaveIndex, const bool bForceIfNotInList, const TSet< TWeakObjectPtr<UPackage>, TWeakObjectPtrSetKeyFuncs<TWeakObjectPtr<UPackage>> >& DirtyPackagesForAutoSave);

	/**
	 * Saves all levels to the specified directory.
	 *
	 * @param	AbsoluteAutosaveDir			Autosave directory.
	 * @param	AutosaveIndex				Integer prepended to autosave filenames..
	 * @param	bForceIfNotInList			Should the save be forced if the package is dirty, but not in DirtyPackagesForAutoSave?
	 * @param	DirtyPackagesForAutoSave	A set of packages that are considered by the auto-save system to be dirty, you should check this to see if a package needs saving
	 */
	static EAutosaveContentPackagesResult::Type AutosaveMapEx(const FString& AbsoluteAutosaveDir, const int32 AutosaveIndex, const bool bForceIfNotInList, const TSet< TWeakObjectPtr<UPackage>, TWeakObjectPtrSetKeyFuncs<TWeakObjectPtr<UPackage>> >& DirtyPackagesForAutoSave);

	/**
	 * Saves all asset packages to the specified directory.
	 *
	 * @param	AbsoluteAutosaveDir			Autosave directory.
	 * @param	AutosaveIndex				Integer prepended to autosave filenames.
	 * @param	bForceIfNotInList			Should the save be forced if the package is dirty, but not in DirtyPackagesForAutoSave?
	 * @param	DirtyPackagesForAutoSave	A set of packages that are considered by the auto-save system to be dirty, you should check this to see if a package needs saving
	 *
	 * @return	true if one or more packages were autosaved; false otherwise
	 */
	static bool AutosaveContentPackages(const FString& AbsoluteAutosaveDir, const int32 AutosaveIndex, const bool bForceIfNotInList, const TSet< TWeakObjectPtr<UPackage>, TWeakObjectPtrSetKeyFuncs<TWeakObjectPtr<UPackage>> >& DirtyPackagesForAutoSave);

	/**
	 * Saves all asset packages to the specified directory.
	 *
	 * @param	AbsoluteAutosaveDir			Autosave directory.
	 * @param	AutosaveIndex				Integer prepended to autosave filenames.
	 * @param	bForceIfNotInList			Should the save be forced if the package is dirty, but not in DirtyPackagesForAutoSave?
	 * @param	DirtyPackagesForAutoSave	A set of packages that are considered by the auto-save system to be dirty, you should check this to see if a package needs saving
	 *
	 * @return	Success if saved at least one faile. NothingToDo if there was nothing to save. Failure on at least one auto-save failure.
	 */
	static EAutosaveContentPackagesResult::Type AutosaveContentPackagesEx(const FString& AbsoluteAutosaveDir, const int32 AutosaveIndex, const bool bForceIfNotInList, const TSet< TWeakObjectPtr<UPackage>, TWeakObjectPtrSetKeyFuncs<TWeakObjectPtr<UPackage>> >& DirtyPackagesForAutoSave);

	/**
	 * Looks at all currently loaded packages and saves them if their "bDirty" flag is set, optionally prompting the user to select which packages to save)
	 * 
	 * @param	bPromptUserToSave			true if we should prompt the user to save dirty packages we found. false to assume all dirty packages should be saved.  Regardless of this setting the user will be prompted for checkout(if needed) unless bFastSave is set
	 * @param	bSaveMapPackages			true if map packages should be saved
	 * @param	bSaveContentPackages		true if we should save content packages. 
	 * @param	bFastSave					true if we should do a fast save. (I.E dont prompt the user to save, dont prompt for checkout, and only save packages that are currently writable).  Note: Still prompts for SaveAs if a package needs a filename
	 * @param	bNotifyNoPackagesSaved		true if a notification should be displayed when no packages need to be saved.
	 * @param	bCanBeDeclined				true if the user prompt should contain a "Don't Save" button in addition to "Cancel", which won't result in a failure return code.
	 * @param	bOutPackagesNeededSaving	when not NULL, will be set to true if there was any work to be done, and false otherwise.
	 * @param	ShouldIgnorePackageFunction callback that allows filtering if package should be saved or not.
	 * @param	bSkipExternalObjectSave		true if saved package should skip saving their own external packages.
	 * @return								true on success, false on fail.
	 */
	UNREALED_API static bool SaveDirtyPackages(const bool bPromptUserToSave, const bool bSaveMapPackages, const bool bSaveContentPackages, const bool bFastSave = false, const bool bNotifyNoPackagesSaved = false, const bool bCanBeDeclined = true, bool* bOutPackagesNeededSaving = NULL, const FShouldIgnorePackageFunctionRef& ShouldIgnorePackageFunction = FShouldIgnorePackage::Default, bool bSkipExternalObjectSave = false);

	/**
	* Looks at all currently loaded packages and saves them if their "bDirty" flag is set and they include specified clasees, optionally prompting the user to select which packages to save)
	*
	* @param	SaveContentClasses			save only specified classes or children classes
	* @param	bPromptUserToSave			true if we should prompt the user to save dirty packages we found. false to assume all dirty packages should be saved.  Regardless of this setting the user will be prompted for checkout(if needed) unless bFastSave is set
	* @param	bFastSave					true if we should do a fast save. (I.E dont prompt the user to save, dont prompt for checkout, and only save packages that are currently writable).  Note: Still prompts for SaveAs if a package needs a filename
	* @param	bNotifyNoPackagesSaved		true if a notification should be displayed when no packages need to be saved.
	* @param	bCanBeDeclined				true if the user prompt should contain a "Don't Save" button in addition to "Cancel", which won't result in a failure return code.
	* @return								true on success, false on fail.
	*/
	UNREALED_API static bool SaveDirtyContentPackages(TArray<UClass*>& SaveContentClasses, const bool bPromptUserToSave, const bool bFastSave = false, const bool bNotifyNoPackagesSaved = false, const bool bCanBeDeclined = true);

	/**
	 * Returns true if there currently are any dirty world packages.
	 *
	 * @param ShouldIgnorePackageFunction Allows filtering out some dirty packages.
	 */
	UNREALED_API static bool HasDirtyWorldPackages(const FShouldIgnorePackageFunctionRef& ShouldIgnorePackageFunction = FShouldIgnorePackage::Default);

	/**
	 * Appends array with all currently dirty world packages.
	 *
	 * @param OutDirtyPackages Array to append dirty packages to.
	 * @param ShouldIgnorePackageFunction Allows filtering out some dirty packages.
	 */
	UNREALED_API static void GetDirtyWorldPackages(TArray<UPackage*>& OutDirtyPackages, const FShouldIgnorePackageFunctionRef& ShouldIgnorePackageFunction = FShouldIgnorePackage::Default);

	/**
	 * Returns true if there currently are any dirty content packages.
	 *
	 * @param ShouldIgnorePackageFunction Allows filtering out some dirty packages.
	 */
	UNREALED_API static bool HasDirtyContentPackages(const FShouldIgnorePackageFunctionRef& ShouldIgnorePackageFunction = FShouldIgnorePackage::Default);

	/**
	 * Appends array with all currently dirty content packages.
	 *
	 * @param OutDirtyPackages Array to append dirty packages to.
	 * @param ShouldIgnorePackageFunction Allows filtering out some dirty packages.
	 */
	UNREALED_API static void GetDirtyContentPackages(TArray<UPackage*>& OutDirtyPackages, const FShouldIgnorePackageFunctionRef& ShouldIgnorePackageFunction = FShouldIgnorePackage::Default);

	/**
	 * Returns true if there currently are any dirty packages.
	 *
	 * @param ShouldIgnorePackageFunction Allows filtering out some dirty packages.
	 */
	UNREALED_API static bool HasDirtyPackages(const FShouldIgnorePackageFunctionRef& ShouldIgnorePackageFunction = FShouldIgnorePackage::Default);

	/**
	 * Appends array with all currently dirty packages
	 *
	 * @param OutDirtyPackages Array to append dirty packages to.
	 * @param ShouldIgnorePackageFunction Allows filtering out some dirty packages.
	 */
	UNREALED_API static void GetDirtyPackages(TArray<UPackage*>& OutDirtyPackages, const FShouldIgnorePackageFunctionRef& ShouldIgnorePackageFunction = FShouldIgnorePackage::Default);

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPrepareWorldsForExplicitSave, TArray<UWorld*> Worlds);
	UNREALED_API static FOnPrepareWorldsForExplicitSave OnPrepareWorldsForExplicitSave;

	/**
	 * Broadcasts the OnPrepareWorldsForExplicitSave event to allow validation and changes to worlds before they're saved.
	 * Used for Editor triggered explicit saves caused by user input such as ctrl+s, or clicking the save button.
	 * Changes to worlds include dirtying the world itself or affecting its ExternalActors through Adds/Removes/Edits.
	 * 
	 * 
	 * @param Worlds A list of the worlds that will be broadcasted.
	 */
	UNREALED_API static void PrepareWorldsForExplicitSave(TArray<UWorld*> Worlds);

	/**
	 * Saves the active level, prompting the use for checkout if necessary.
	 *
	 * @return	true on success, False on fail
	 */
	UNREALED_API static bool SaveCurrentLevel();


	/** Enum used for prompt returns */
	enum EPromptReturnCode
	{
		PR_Success,		/** The user has answered in the affirmative to all prompts, and execution succeeded */
		PR_Failure,		/** The user has answered in the affirmative to prompts, but an operation(s) has failed during execution */
		PR_Declined,	/** The user has declined out of the prompt; the caller should continue whatever it was doing */
		PR_Cancelled	/** The user has cancelled out of a prompt; the caller should abort whatever it was doing */
	};

	struct FPromptForCheckoutAndSaveParams
	{
		FPromptForCheckoutAndSaveParams()
			: Title(NSLOCTEXT("PackagesDialogModule", "PackagesDialogTitle", "Save Content")),
			  Message(NSLOCTEXT("PackagesDialogModule", "PackagesDialogMessage", "Select Content to Save"))
		{}

		bool bCheckDirty = false;                       /** If true, only packages that are dirty in PackagesToSave will be saved	*/
		bool bPromptToSave = false;                     /** If true the user will be prompted with a list of packages to save, otherwise all passed in packages are saved */
		bool bAlreadyCheckedOut = false;                /** If true, the user will not be prompted with the source control dialog */
		bool bCanBeDeclined = true;                     /** If true, offer a "Don't Save" option in addition to "Cancel", which will not result in a cancellation return code. */
		bool bIsExplicitSave = false;                   /** If true, marks the save as explicit. Explicit saves are triggered by user facing actions such as Save As, or ctrl + s*/
		FText Title;                                    /** If bPromptToSave true provides a dialog title */
		FText Message;                                  /** If bPromptToSave true provides a dialog message */
		TArray<UPackage*>* OutFailedPackages = nullptr; /** [out] If specified, will be filled in with all of the packages that failed to save successfully */
	};

	/**
	 * Optionally prompts the user for which of the provided packages should be saved, and then additionally prompts the user to check-out any of
	 * the provided packages which are under source control. If the user cancels their way out of either dialog, no packages are saved. It is possible the user
	 * will be prompted again, if the saving process fails for any reason. In that case, the user will be prompted on a package-by-package basis, allowing them
	 * to retry saving, skip trying to save the current package, or to again cancel out of the entire dialog. If the user skips saving a package that failed to save,
	 * the package will be added to the optional OutFailedPackages array, and execution will continue. After all packages are saved (or not), the user is provided with
	 * a warning about any packages that were writable on disk but not in source control, as well as a warning about which packages failed to save.
	 *
	 * @param		PackagesToSave				The list of packages to save.  Both map and content packages are supported 
	 * @param		bCheckDirty					If true, only packages that are dirty in PackagesToSave will be saved	
	 * @param		bPromptToSave				If true the user will be prompted with a list of packages to save, otherwise all passed in packages are saved
	 * @param		Title						If bPromptToSave true provides a dialog title
	 * @param		Message						If bPromptToSave true provides a dialog message
	 * @param		OutFailedPackages			[out] If specified, will be filled in with all of the packages that failed to save successfully
	 * @param		bAlreadyCheckedOut			If true, the user will not be prompted with the source control dialog
	 * @param		bCanBeDeclined				If true, offer a "Don't Save" option in addition to "Cancel", which will not result in a cancellation return code.
	 *
	 * @return		An enum value signifying success, failure, user declined, or cancellation. If any packages at all failed to save during execution, the return code will be 
	 *				failure, even if other packages successfully saved. If the user cancels at any point during any prompt, the return code will be cancellation, even though it
	 *				is possible some packages have been successfully saved (if the cancel comes on a later package that can't be saved for some reason). If the user opts the "Don't
	 *				Save" option on the dialog, the return code will indicate the user has declined out of the prompt. This way calling code can distinguish between a decline and a cancel
	 *				and then proceed as planned, or abort its operation accordingly.
	 */
	UNREALED_API static EPromptReturnCode PromptForCheckoutAndSave(const TArray<UPackage*>& PackagesToSave, FPromptForCheckoutAndSaveParams& InOutParams);
	UNREALED_API static EPromptReturnCode PromptForCheckoutAndSave(const TArray<UPackage*>& PackagesToSave, bool bCheckDirty, bool bPromptToSave, const FText& Title, const FText& Message, TArray<UPackage*>* OutFailedPackages = NULL, bool bAlreadyCheckedOut = false, bool bCanBeDeclined = true);
	UNREALED_API static EPromptReturnCode PromptForCheckoutAndSave( const TArray<UPackage*>& PackagesToSave, bool bCheckDirty, bool bPromptToSave, TArray<UPackage*>* OutFailedPackages = NULL, bool bAlreadyCheckedOut = false, bool bCanBeDeclined = true );

	////////////////////////////////////////////////////////////////////////////
	// Import/Export

	/**
	 * Presents the user with a file dialog for importing.
	 * If the import is not a merge (bMerging is false), AskSaveChanges() is called first.
	 */
	UNREALED_API static void Import();
	UNREALED_API static void Import(const FString& InFilename);
	UNREALED_API static void Export(bool bExportSelectedActorsOnly);			// prompts user for file etc.

	////////////////////////////////////////////////////////////////////////////
	// Source Control

	/**
	 * Prompt the user with a check-box dialog allowing them to check out the provided packages
	 * from source control, if desired
	 *
	 * @param	bCheckDirty								If true, non-dirty packages won't be added to the dialog
	 * @param	PackagesToCheckOut						Reference to array of packages to prompt the user with for possible check out
	 * @param	OutPackagesCheckedOutOrMadeWritable		If not NULL, this array will be populated with packages that the user selected to check out or make writable.
	 * @param	OutPackagesNotNeedingCheckout			If not NULL, this array will be populated with packages that the user was not prompted about and do not need to be checked out to save.  Useful for saving packages even if the user canceled the checkout dialog.
	 * @param	bPromptingAfterModify					If true, we are prompting the user after an object has been modified, which changes the cancel button to "Ask me later".
	 * @param	bAllowSkip								If true, user can skip the checkout / make writable.
	 *
	 * @return	true if the user did not cancel out of the dialog and has potentially checked out some files
	 *			(or if there is no source control integration); false if the user cancelled the dialog
	 */
	UNREALED_API static bool PromptToCheckoutPackages(bool bCheckDirty, const TArray<UPackage*>& PackagesToCheckOut, TArray< UPackage* >* OutPackagesCheckedOutOrMadeWritable = NULL, TArray< UPackage* >* OutPackagesNotNeedingCheckout = NULL, const bool bPromptingAfterModify = false, const bool bAllowSkip = false );
	
	/**
	 * Check out the specified packages from source control and report any errors while checking out
	 *
	 * @param	PkgsToCheckOut							Reference to array of packages to check out 
	 * @param	OutPackagesCheckedOut					If not NULL, this array will be populated with packages that were checked out.
	 * @param	bErrorIfAlreadyCheckedOut				true to consider being unable to checkout a package because it is already checked out an error, false to allow this without error
	 * @param	bConfirmPackageBranchCheckOutStatus		true to prompt user on whether a package that is checked out or modified in another branch should be checked out, false to silently attempt check out.
	 *
	 * @return	true if all the packages were checked out successfully
	 */
	UNREALED_API static ECommandResult::Type CheckoutPackages(const TArray<UPackage*>& PkgsToCheckOut, TArray<UPackage*>* OutPackagesCheckedOut = NULL, const bool bErrorIfAlreadyCheckedOut = true, const bool bConfirmPackageBranchCheckOutStatus = true);

	/**
	 * Check out the specified packages from source control and report any errors while checking out
	 *
	 * @param	PkgsToCheckOut							Reference to array of package names to check out
	 * @param	OutPackagesCheckedOut					If not NULL, this array will be populated with packages that were checked out.
	 * @param	bErrorIfAlreadyCheckedOut				true to consider being unable to checkout a package because it is already checked out an error, false to allow this without error
	 *
	 * @return	the result of the check out operation
	 */
	UNREALED_API static ECommandResult::Type CheckoutPackages(const TArray<FString>& PkgsToCheckOut, TArray<FString>* OutPackagesCheckedOut = NULL, const bool bErrorIfAlreadyCheckedOut = true);

	/**
	 * Prompt the user with a check-box dialog allowing them to check out relevant level packages 
	 * from source control
	 *
	 * @param	bCheckDirty					If true, non-dirty packages won't be added to the dialog
	 * @param	SpecificLevelsToCheckOut	If specified, only the provided levels' packages will display in the
	 *										dialog if they are under source control; If nothing is specified, all levels
	 *										referenced by GWorld whose packages are under source control will be displayed
	 * @param	OutPackagesNotNeedingCheckout	If not null, this array will be populated with packages that the user was not prompted about and do not need to be checked out to save.  Useful for saving packages even if the user canceled the checkout dialog.
	 *
	 * @return	true if the user did not cancel out of the dialog and has potentially checked out some files (or if there is
	 *			no source control integration); false if the user cancelled the dialog
	 */
	UNREALED_API static bool PromptToCheckoutLevels(bool bCheckDirty, const TArray<ULevel*>& SpecificLevelsToCheckOut, TArray<UPackage*>* OutPackagesNotNeedingCheckout = NULL);

	/**
	 * Overloaded version of PromptToCheckOutLevels which prompts the user with a check-box dialog allowing
	 * them to check out the relevant level package if necessary
	 *
	 * @param	bCheckDirty				If true, non-dirty packages won't be added to the dialog
	 * @param	SpecificLevelToCheckOut	The level whose package will display in the dialog if it is
	 *									under source control
	 *
	 * @return	true if the user did not cancel out of the dialog and has potentially checked out some files (or if there is
	 *			no source control integration); false if the user cancelled the dialog
	 */
	UNREALED_API static bool PromptToCheckoutLevels(bool bCheckDirty, ULevel* SpecificLevelToCheckOut);

	/** Loads a simple example map */
	UNREALED_API static void LoadDefaultMapAtStartup();
	
	/**
	 * Save all packages corresponding to the specified world, with the option to override their path and also
	 * apply a prefix.
	 *
	 * @param   InWorld         The world to save (including its children)
	 * @param	RootPath		Root Path override, replaces /Game/ in original name
	 * @param	Prefix			Optional prefix for base filename, can be NULL
	 * @param	OutFilenames	The file names of all successfully saved worlds will be added to this
	 * @return					true if at least one level was saved.
	 *							If bPIESaving, will be true is ALL worlds were saved.
	 */
	static bool SaveWorlds(UWorld* InWorld, const FString& RootPath, const TCHAR* Prefix, TArray<FString>& OutFilenames);

	/** Whether or not we're in the middle of loading the simple startup map */
	static bool IsLoadingStartupMap() { return bIsLoadingDefaultStartupMap; }

	/** Whether or not saving the map package should save the external objects packages */
	static bool ShouldSkipExternalObjectSave() { return bSkipExternalObjectSave; }

	/**
	 * Returns a file filter string appropriate for a specific file interaction.
	 *
	 * @param	Interaction		A file interaction to get a filter string for.
	 * @return					A filter string.
	 */
	UNREALED_API static FString GetFilterString(EFileInteraction Interaction);

	/**
	 * Looks for package files in the known content paths on disk.
	 *
	 * @param	OutPackages		All found package filenames.
	 */
	UNREALED_API static void FindAllPackageFiles(TArray<FString>& OutPackages);

	/**
	 * Looks for source control submittable files in the known content paths on disk.
	 *
	 * @param	OutPackages		All found package filenames and their source control state
	 * @param	bIncludeMaps	If true, also adds maps to the list
	 */
	UNREALED_API static void FindAllSubmittablePackageFiles(TMap<FString, FSourceControlStatePtr>& OutPackages, const bool bIncludeMaps);

	/**
	 * Looks for source control submittable non-package project files for the current project.
	 *
	 * @param	OutProjectFiles	All found project filenames and their source control state.
	 */
	UNREALED_API static void FindAllSubmittableProjectFiles(TMap<FString, FSourceControlStatePtr>& OutProjectFiles);

	/**
	 * Looks for config files for the current project.
	 *
	 * @param	OutConfigFiles	All found config filenames.
	 */
	UNREALED_API static void FindAllConfigFiles(TArray<FString>& OutConfigFiles);

	/**
	 * Looks for source control submittable non-package config files for the current project.
	 *
	 * @param	OutConfigFiles	All found config filenames and their source control state.
	 */
	UNREALED_API static void FindAllSubmittableConfigFiles(TMap<FString, FSourceControlStatePtr>& OutConfigFiles);

	/**
	 * Helper function used to decide whether a package name is a map package or not. Map packages aren't added to the additional package list.
	 *
	 * @param	ObjectPath		The path to the package to test
	 * @return					True if package is a map
	 */
	UNREALED_API static bool IsMapPackageAsset(const FString& ObjectPath);

	/**
	* Helper function used to decide whether a package name is a map package or not. Map packages aren't added to the additional package list.
	*
	* @param	ObjectPath		The path to the package to test
	* @param	MapFilePath		OUT parameter that returns the map file path if it exists.
	* @return					True if package is a map
	*/
	UNREALED_API static bool IsMapPackageAsset(const FString& ObjectPath, FString& MapFilePath);

	/** 
	 * Helper function used to extract the package name from the object path
	 *
	 * @param	ObjectPath		The path to the package to test
	 * @return					The package name from the string
	 */
	UNREALED_API static FString ExtractPackageName(const FString& ObjectPath);

	////////////////////////////////////////////////////////////////////////////
	// File

	UNREALED_API static FString GetFilename(const FName& PackageName);

	UNREALED_API static FString GetFilename(UObject* LevelObject);

private:
	using FEnumeratePackageFunctionRef = TFunctionRef<bool(UPackage*)>;

	static void EnumerateDirtyWorldPackages(const FEnumeratePackageFunctionRef& Callback);

	static void EnumerateDirtyContentPackages(const FEnumeratePackageFunctionRef& Callback);

	/** Private method used to build a list of items requiring checkout */
	static bool AddCheckoutPackageItems(bool bCheckDirty, const TArray<UPackage*>& PackagesToCheckOut, const TSet<FName>& ReadOnlyPackages, TArray<UPackage*>* OutPackagesNotNeedingCheckout, bool* bOutHavePackageToCheckOut);

	/** Callback from PackagesDialog used to update the list of items when the source control state changes */
	static void UpdateCheckoutPackageItems(bool bCheckDirty, TArray<UPackage*> PackagesToCheckOut, TSet<FName> ReadOnlyPackages, TArray<UPackage*>* OutPackagesNotNeedingCheckout);
	
	/** Prompts for package checkout without reentrance check */
	static bool PromptToCheckoutPackagesInternal(bool bCheckDirty, const TArray<UPackage*>& PackagesToCheckOut, TArray<UPackage*>* OutPackagesCheckedOutOrMadeWritable, TArray<UPackage*>* OutPackagesNotNeedingCheckout, const bool bPromptingAfterModify, const bool bAllowSkip);

	/** Tries to checkout packages without prompting and only prompts (for revert) if that fails */
	static bool AutomaticCheckoutOrPromptToRevertPackages(const TArray<UPackage*>& PackagesToCheckOut, TArray<UPackage*>* OutPackagesCheckedOutOrMadeWritable, TArray<UPackage*>* OutPackagesNotNeedingCheckout, TArray<UPackage*>* OutPackagesToRevert);

	/** Tries to make the given packages writable */
	static void MakePackagesWritable(const TArray<UPackage*>& PackagesToMakeWritable, TArray<UPackage*>* OutPackagesMadeWritable, TArray<UPackage*>* OutPackagesMadeWritableFailed);

	static bool bIsLoadingDefaultStartupMap;

	/** Flag used to determine if the checkout and save prompt is already open to prevent re-entrance */
	static bool bIsPromptingForCheckoutAndSave;
	
	/** Flag used in SaveMap to skip saving external objects and only save the map */
	static bool bSkipExternalObjectSave;

	// Set of packages to ignore for save/checkout when using SaveAll.
	static TSet<FString> PackagesNotSavedDuringSaveAll;

	// Set of packages which should no longer prompt for checkouts / to be made writable
	static TSet<FString> PackagesNotToPromptAnyMore;

	static FOnLoadMapStart OnLoadMapStart;
	static FOnLoadMapEnd OnLoadMapEnd;

};
