// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"

class FFractureToolDelegates
{
public:

	/** Return a single FFractureToolDelegates object */
	static UNREALED_API FFractureToolDelegates& Get();

	FSimpleMulticastDelegate OnFractureExpansionBegin;
	FSimpleMulticastDelegate OnFractureExpansionUpdate;
	FSimpleMulticastDelegate OnFractureExpansionEnd;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnVisualizationSettingsChanged, bool);
	FOnVisualizationSettingsChanged OnVisualizationSettingsChanged;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnUpdateExplodedView, uint8, uint8);
	FOnUpdateExplodedView OnUpdateExplodedView;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnUpdateFractureLevelView, uint8);
	FOnUpdateFractureLevelView OnUpdateFractureLevelView;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnUpdateVisualizationSettings, bool);
	FOnUpdateVisualizationSettings OnUpdateVisualizationSettings;
};
