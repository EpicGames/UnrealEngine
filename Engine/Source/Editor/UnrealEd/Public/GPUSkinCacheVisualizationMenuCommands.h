// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "UObject/NameTypes.h"
#include "Templates/SharedPointer.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Commands/Commands.h"

class FEditorViewportClient;

class UNREALED_API FGPUSkinCacheVisualizationMenuCommands : public TCommands<FGPUSkinCacheVisualizationMenuCommands>
{
public:
	enum class FGPUSkinCacheVisualizationType : uint8
	{
		Overview,
		Memory,
		RayTracingLODOffset
	};

	struct FGPUSkinCacheVisualizationRecord
	{
		FName Name;
		TSharedPtr<FUICommandInfo> Command;
		FGPUSkinCacheVisualizationType Type;

		FGPUSkinCacheVisualizationRecord()
			: Name()
			, Command()
			, Type(FGPUSkinCacheVisualizationType::Overview)
		{
		}
	};

	typedef TMultiMap<FName, FGPUSkinCacheVisualizationRecord> TGPUSkinCacheVisualizationModeCommandMap;
	typedef TGPUSkinCacheVisualizationModeCommandMap::TConstIterator TCommandConstIterator;

	FGPUSkinCacheVisualizationMenuCommands();

	TCommandConstIterator CreateCommandConstIterator() const;

	static void BuildVisualisationSubMenu(FMenuBuilder& Menu);

	virtual void RegisterCommands() override;

	void BindCommands(FUICommandList& CommandList, const TSharedPtr<FEditorViewportClient>& Client) const;

	inline bool IsPopulated() const
	{
		return CommandMap.Num() > 0;
	}

private:
	void BuildCommandMap();
	bool AddCommandTypeToMenu(FMenuBuilder& Menu, const FGPUSkinCacheVisualizationType Type, bool bSeparatorBefore = false) const;

	static void ChangeGPUSkinCacheVisualizationMode(TWeakPtr<FEditorViewportClient> WeakClient, FName InName);
	static bool IsGPUSkinCacheVisualizationModeSelected(TWeakPtr<FEditorViewportClient> WeakClient, FName InName);

	TGPUSkinCacheVisualizationModeCommandMap CommandMap;
};
