// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Engine/Blueprint.h"
#include "Editor.h"

#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "Widgets/Text/STextBlock.h"
#include "IDetailGroup.h"
#include "IDetailPropertyRow.h"
#include "UObject/ConstructorHelpers.h"
#include "Widgets/Layout/SBox.h"
#include "GameFramework/GameModeBase.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "PropertyCustomizationHelpers.h"
#include "IDocumentation.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "EditorClassUtils.h"
#include "Subsystems/AssetEditorSubsystem.h"

#define LOCTEXT_NAMESPACE "FGameModeInfoCustomizer"

static FString GameModeCategory(LOCTEXT("GameModeCategory", "GameMode").ToString());

/** Class to help customize a GameMode class picker, to show settings 'withing' GameMode. */
class FGameModeInfoCustomizer : public TSharedFromThis<FGameModeInfoCustomizer>
{
public:
	FGameModeInfoCustomizer(UObject* InOwningObject, FName InGameModePropertyName)
	{
		OwningObject = InOwningObject;
		GameModePropertyName = InGameModePropertyName;
		CachedGameModeClass = nullptr;
	}

	/** Create widget for the name of a default class property */
	TSharedRef<SWidget> CreateGameModePropertyLabelWidget(FName PropertyName)
	{
		FProperty* Prop = FindFieldChecked<FProperty>(AGameModeBase::StaticClass(), PropertyName);

		FString DisplayName = Prop->GetDisplayNameText().ToString();
		if (DisplayName.Len() == 0)
		{
			DisplayName = Prop->GetName();
		}
		DisplayName = FName::NameToDisplayString(DisplayName, false);

		return
			SNew(STextBlock)
			.Text(FText::FromString(DisplayName))
			.ToolTip(IDocumentation::Get()->CreateToolTip(Prop->GetToolTipText(), NULL, TEXT("Shared/Types/AGameMode"), Prop->GetName()))
			.Font(IDetailLayoutBuilder::GetDetailFont());
	}

	/** Create widget fo modifying a default class within the current GameMode */
	void CustomizeGameModeDefaultClass(IDetailGroup& Group, FName DefaultClassPropertyName)
	{
		// Find the metaclass of this property
		FClassProperty* ClassProp = FindFieldChecked<FClassProperty>(AGameModeBase::StaticClass(), DefaultClassPropertyName);

		UClass* MetaClass = ClassProp->MetaClass;
		const bool bAllowNone = !(ClassProp->PropertyFlags & CPF_NoClear);

		// This is inconsistent with all the other browsers, so disabling it for now
		//TAttribute<bool> CanBrowseAtrribute = TAttribute<bool>::Create( TAttribute<bool>::FGetter::CreateSP( this, &FGameModeInfoCustomizer::CanBrowseDefaultClass, DefaultClassPropertyName) ) ;

		// Add a row for choosing a new default class
		Group.AddWidgetRow()
		.NameContent()
		[
			CreateGameModePropertyLabelWidget(DefaultClassPropertyName)
		]
		.ValueContent()
		.MaxDesiredWidth(0)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(125.0f)
				[
					SNew(SClassPropertyEntryBox)
					.AllowNone(bAllowNone)
					.MetaClass(MetaClass)
					.IsEnabled(this, &FGameModeInfoCustomizer::AllowModifyGameMode)
					.SelectedClass(this, &FGameModeInfoCustomizer::OnGetDefaultClass, DefaultClassPropertyName)
					.OnSetClass(FOnSetClass::CreateSP(this, &FGameModeInfoCustomizer::OnSetDefaultClass, DefaultClassPropertyName))
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				PropertyCustomizationHelpers::MakeUseSelectedButton(FSimpleDelegate::CreateSP(this, &FGameModeInfoCustomizer::OnMakeSelectedDefaultClassClicked, DefaultClassPropertyName))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				PropertyCustomizationHelpers::MakeBrowseButton(FSimpleDelegate::CreateSP(this, &FGameModeInfoCustomizer::OnBrowseDefaultClassClicked, DefaultClassPropertyName))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				PropertyCustomizationHelpers::MakeNewBlueprintButton(FSimpleDelegate::CreateSP(this, &FGameModeInfoCustomizer::OnMakeNewDefaultClassClicked, DefaultClassPropertyName))
			]
		];
	}

	/** Add special customization for the GameMode setting */
	void CustomizeGameModeSetting(IDetailLayoutBuilder& LayoutBuilder, IDetailCategoryBuilder& CategoryBuilder)
	{
		// Add GameMode picker widget
		DefaultGameModeClassHandle = LayoutBuilder.GetProperty(GameModePropertyName);
		check(DefaultGameModeClassHandle.IsValid());

		IDetailPropertyRow& DefaultGameModeRow = CategoryBuilder.AddProperty(DefaultGameModeClassHandle);
		// SEe if we are allowed to choose 'no' GameMode
		const bool bAllowNone = !(DefaultGameModeClassHandle->GetProperty()->PropertyFlags & CPF_NoClear);

		// This is inconsistent with other property 
		//TAttribute<bool> CanBrowseAtrribute = TAttribute<bool>(this, &FGameModeInfoCustomizer::CanBrowseGameMode);

		DefaultGameModeRow
		.ShowPropertyButtons(false)
		.CustomWidget()
		.NameContent()
		[
			DefaultGameModeClassHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MaxDesiredWidth(0)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(125.0f)
				[
					SNew(SClassPropertyEntryBox)
					.AllowNone(bAllowNone)
					.MetaClass(AGameModeBase::StaticClass())
					.SelectedClass(this, &FGameModeInfoCustomizer::GetCurrentGameModeClass)
					.OnSetClass(FOnSetClass::CreateSP(this, &FGameModeInfoCustomizer::SetCurrentGameModeClass))
				]
			]

			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				PropertyCustomizationHelpers::MakeUseSelectedButton(FSimpleDelegate::CreateSP(this, &FGameModeInfoCustomizer::OnUseSelectedGameModeClicked))
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				PropertyCustomizationHelpers::MakeBrowseButton(FSimpleDelegate::CreateSP(this, &FGameModeInfoCustomizer::OnBrowseGameModeClicked))
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				PropertyCustomizationHelpers::MakeNewBlueprintButton(FSimpleDelegate::CreateSP(this, &FGameModeInfoCustomizer::OnClickNewGameMode))
			]
		];

		static FName SelectedGameModeDetailsName(TEXT("SelectedGameModeDetails"));		
		IDetailGroup& Group = CategoryBuilder.AddGroup(SelectedGameModeDetailsName, LOCTEXT("SelectedGameModeDetails", "Selected GameMode"));

		// Then add rows to show key properties and let you edit them
		CustomizeGameModeDefaultClass(Group, GET_MEMBER_NAME_CHECKED(AGameModeBase, DefaultPawnClass));
		CustomizeGameModeDefaultClass(Group, GET_MEMBER_NAME_CHECKED(AGameModeBase, HUDClass));
		CustomizeGameModeDefaultClass(Group, GET_MEMBER_NAME_CHECKED(AGameModeBase, PlayerControllerClass));
		CustomizeGameModeDefaultClass(Group, GET_MEMBER_NAME_CHECKED(AGameModeBase, GameStateClass));
		CustomizeGameModeDefaultClass(Group, GET_MEMBER_NAME_CHECKED(AGameModeBase, PlayerStateClass));
		CustomizeGameModeDefaultClass(Group, GET_MEMBER_NAME_CHECKED(AGameModeBase, SpectatorClass));
	}

	/** Get the currently set GameMode class */
	const UClass* GetCurrentGameModeClass() const
	{
		FString ClassName;
		DefaultGameModeClassHandle->GetValueAsFormattedString(ClassName);

		// Blueprints may have type information before the class name, so make sure and strip that off now
		ConstructorHelpers::StripObjectClass(ClassName);

		// Do we have a valid cached class pointer? (Note: We can't search for the class while a save is happening)
		const UClass* GameModeClass = CachedGameModeClass.Get();
		if ((!GameModeClass || GameModeClass->GetPathName() != ClassName) && !UE::IsSavingPackage())
		{
			GameModeClass = FEditorClassUtils::GetClassFromString(ClassName);
			CachedGameModeClass = MakeWeakObjectPtr(const_cast<UClass*>(GameModeClass));
		}
		return GameModeClass;
	}

	void SetCurrentGameModeClass(const UClass* NewGameModeClass)
	{
		if (DefaultGameModeClassHandle->SetValueFromFormattedString((NewGameModeClass) ? NewGameModeClass->GetPathName() : TEXT("None")) == FPropertyAccess::Success)
		{
			CachedGameModeClass = MakeWeakObjectPtr(const_cast<UClass*>(NewGameModeClass));
		}
	}

	/** Get the CDO from the currently set GameMode class */
	AGameModeBase* GetCurrentGameModeCDO() const
	{
		UClass* GameModeClass = const_cast<UClass*>( GetCurrentGameModeClass() );
		if (GameModeClass != NULL)
		{
			return GameModeClass->GetDefaultObject<AGameModeBase>();
		}
		else
		{
			return NULL;
		}
	}

	/** Find the current default class by property name */
	const UClass* OnGetDefaultClass(FName ClassPropertyName) const
	{
		UClass* CurrentDefaultClass = NULL;
		const UClass* GameModeClass = GetCurrentGameModeClass();
		if (GameModeClass != NULL)
		{
			FClassProperty* ClassProp = FindFieldChecked<FClassProperty>(GameModeClass, ClassPropertyName);
			CurrentDefaultClass = (UClass*)ClassProp->GetObjectPropertyValue(ClassProp->ContainerPtrToValuePtr<void>(GetCurrentGameModeCDO()));
		}
		return CurrentDefaultClass;
	}

	/** Set a new default class by property name */
	void OnSetDefaultClass(const UClass* NewDefaultClass, FName ClassPropertyName)
	{
		const UClass* GameModeClass = GetCurrentGameModeClass();
		if (GameModeClass != NULL && AllowModifyGameMode())
		{
			FClassProperty* ClassProp = FindFieldChecked<FClassProperty>(GameModeClass, ClassPropertyName);
			const UClass** DefaultClassPtr = ClassProp->ContainerPtrToValuePtr<const UClass*>(GetCurrentGameModeCDO());
			*DefaultClassPtr = NewDefaultClass;

			UBlueprint* Blueprint = Cast<UBlueprint>(GameModeClass->ClassGeneratedBy);
			if (Blueprint)
			{
				// Indicate that the BP has changed and would need to be saved.
				FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
			}
		}
	}

	bool CanBrowseDefaultClass(FName ClassPropertyName) const
	{
		return CanSyncToClass(OnGetDefaultClass(ClassPropertyName));
	}

	void OnBrowseDefaultClassClicked(FName ClassPropertyName)
	{
		SyncBrowserToClass(OnGetDefaultClass(ClassPropertyName));
	}

	void OnMakeNewDefaultClassClicked(FName ClassPropertyName)
	{
		FClassProperty* ClassProp = FindFieldChecked<FClassProperty>(AGameModeBase::StaticClass(), ClassPropertyName);

		UBlueprint* Blueprint = FKismetEditorUtilities::CreateBlueprintFromClass(LOCTEXT("CreateNewBlueprint", "Create New Blueprint"), ClassProp->MetaClass, FString::Printf(TEXT("New%s"),*ClassProp->MetaClass->GetName()));

		if(Blueprint != NULL && Blueprint->GeneratedClass)
		{
			OnSetDefaultClass(Blueprint->GeneratedClass, ClassPropertyName);

			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Blueprint);
		}
	}

	void OnMakeSelectedDefaultClassClicked(FName ClassPropertyName)
	{
		FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();

		FClassProperty* ClassProp = FindFieldChecked<FClassProperty>(AGameModeBase::StaticClass(), ClassPropertyName);
		const UClass* SelectedClass = GEditor->GetFirstSelectedClass(ClassProp->MetaClass);
		if (SelectedClass)
		{
			OnSetDefaultClass(SelectedClass, ClassPropertyName);
		}
	}

	bool CanBrowseGameMode() const
	{
		return CanSyncToClass(GetCurrentGameModeClass());
	}

	void OnBrowseGameModeClicked()
	{
		SyncBrowserToClass(GetCurrentGameModeClass());
	}

	bool CanSyncToClass(const UClass* Class) const
	{
		return (Class != NULL && Class->ClassGeneratedBy != NULL);
	}

	void SyncBrowserToClass(const UClass* Class)
	{
		if (CanSyncToClass(Class))
		{
			UBlueprint* Blueprint = Cast<UBlueprint>(Class->ClassGeneratedBy);
			if (ensure(Blueprint != NULL))
			{
				TArray<UObject*> SyncObjects;
				SyncObjects.Add(Blueprint);
				GEditor->SyncBrowserToObjects(SyncObjects);
			}
		}
	}

	void OnUseSelectedGameModeClicked()
	{
		FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();

		const UClass* SelectedClass = GEditor->GetFirstSelectedClass(AGameModeBase::StaticClass());
		if (SelectedClass)
		{
			DefaultGameModeClassHandle->SetValueFromFormattedString(SelectedClass->GetPathName());
		}
	}

	void OnClickNewGameMode()
	{
		// Create a new GameMode BP
		UBlueprint* Blueprint = FKismetEditorUtilities::CreateBlueprintFromClass(LOCTEXT("CreateNewGameMode", "Create New GameMode"), AGameModeBase::StaticClass(), TEXT("NewGameMode"));
		// if that worked, assign it
		if(Blueprint != NULL && Blueprint->GeneratedClass)
		{
			DefaultGameModeClassHandle->SetValueFromFormattedString(Blueprint->GeneratedClass->GetPathName());
		}
	}

	/** Are we allowed to modify the currently selected GameMode */
	bool AllowModifyGameMode() const
	{
		// Only allow editing GameMode BP, not native class!
		const UBlueprintGeneratedClass* GameModeBPClass = Cast<UBlueprintGeneratedClass>(GetCurrentGameModeClass());
		return (GameModeBPClass != NULL);
	}

private:
	/** Object that owns the pointer to the GameMode we want to customize */
	TWeakObjectPtr<UObject>	OwningObject;
	/** Name of GameMode property inside OwningObject */
	FName GameModePropertyName;
	/** Handle to the DefaultGameMode property */
	TSharedPtr<IPropertyHandle> DefaultGameModeClassHandle;
	/** Cached class pointer from the DefaultGameModeClassHandle */
	mutable TWeakObjectPtr<UClass> CachedGameModeClass;
};

#undef LOCTEXT_NAMESPACE
