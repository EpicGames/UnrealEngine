// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintUtilities.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Engine/LevelStreaming.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandList.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformMath.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"

class FActiveTimerHandle;
class FAssetEditorToolkit;
class FMenuBuilder;
class FReply;
class SGraphPanel;
class SNotificationItem;
class SWidget;
class UEdGraph;
class UEdGraphNode;
class UEdGraphPin;
struct FDiffSingleResult;
struct FInputChord;
struct FNotificationInfo;
struct FPropertyChangedEvent;
struct FSlateBrush;
struct Rect;

class SGraphEditor;

DECLARE_DELEGATE_ThreeParams( FOnNodeTextCommitted, const FText&, ETextCommit::Type, UEdGraphNode* );
DECLARE_DELEGATE_RetVal_ThreeParams( bool, FOnNodeVerifyTextCommit, const FText&, UEdGraphNode*, FText& );
DECLARE_MULTICAST_DELEGATE(FOnGraphContentMenuDismissed);
DECLARE_DELEGATE_ThreeParams(FOnGraphEditorTickDelegate, const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime);

typedef TSet<class UObject*> FGraphPanelSelectionSet;

/** Info about how to draw the graph */
struct FGraphAppearanceInfo
{
	FGraphAppearanceInfo()
		: CornerImage(NULL)
		, InstructionFade(1.f)
	{
	}

	/** Image to draw in corner of graph */
	const FSlateBrush* CornerImage;
	/** Text to write in corner of graph */
	FText CornerText;
	/** If set, will be used as override for PIE notify text */
	FText PIENotifyText;
	/** If set, will be used as override for read only text */
	FText ReadOnlyText;
	/** Text to display if the graph is empty (to guide the user on what to do) */
	FText InstructionText;
	/** Bottom left warning text used for instance by Substrate */
	FText WarningText;
	/** Allows graphs to nicely fade instruction text (or completely hide it). */
	TAttribute<float> InstructionFade;
};

/** Struct used to return info about action menu */
struct FActionMenuContent
{
	explicit FActionMenuContent( TSharedRef<SWidget> InContent, TSharedPtr<SWidget> InWidgetToFocus = TSharedPtr<SWidget>() )
		: Content( InContent )
		, WidgetToFocus( InWidgetToFocus )
	{
	}

	FActionMenuContent()
	: Content(SNullWidget::NullWidget)
	{	
	}

	TSharedRef<SWidget> Content;
	TSharedPtr<SWidget> WidgetToFocus;
	FOnGraphContentMenuDismissed OnMenuDismissed;
};

/**
 * Interface and wrapper for GraphEditor widgets.
 * Gracefully handles the GraphEditorModule being unloaded.
 */
class SGraphEditor : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam( FOnSelectionChanged, const FGraphPanelSelectionSet& )

	DECLARE_DELEGATE_OneParam( FOnFocused, const TSharedRef<SGraphEditor>& );

	UE_DEPRECATED(5.6, "Please use the delegate accepting FVector2f.")
	DECLARE_DELEGATE_ThreeParams( FOnDropActor, const TArray< TWeakObjectPtr<class AActor> >&, class UEdGraph*, const FVector2D& );
	DECLARE_DELEGATE_ThreeParams( FOnDropActors, const TArray< TWeakObjectPtr<class AActor> >&, class UEdGraph*, const FVector2f& );

	UE_DEPRECATED(5.6, "Please use the delegate accepting FVector2f.")
	DECLARE_DELEGATE_ThreeParams( FOnDropStreamingLevel, const TArray< TWeakObjectPtr<class ULevelStreaming> >&, class UEdGraph*, const FVector2D& );
	DECLARE_DELEGATE_ThreeParams( FOnDropStreamingLevels, const TArray< TWeakObjectPtr<class ULevelStreaming> >&, class UEdGraph*, const FVector2f& );

	DECLARE_DELEGATE( FActionMenuClosed );

	UE_DEPRECATED(5.6, "Please use the delegate accepting FVector2f.")
	DECLARE_DELEGATE_RetVal_FiveParams( FActionMenuContent, FOnCreateActionMenu, UEdGraph*, const FVector2D&, const TArray<UEdGraphPin*>&, bool, FActionMenuClosed );
	DECLARE_DELEGATE_RetVal_FiveParams( FActionMenuContent, FOnCreateActionMenuAtLocation, UEdGraph*, const FVector2f&, const TArray<UEdGraphPin*>&, bool, FActionMenuClosed );

	DECLARE_DELEGATE_RetVal_FiveParams( FActionMenuContent, FOnCreateNodeOrPinMenu, UEdGraph*, const UEdGraphNode*, const UEdGraphPin*, FMenuBuilder*, bool);

	UE_DEPRECATED(5.6, "Please use the delegate accepting FVector2f.")
	DECLARE_DELEGATE_RetVal_TwoParams( FReply, FOnSpawnNodeByShortcut, FInputChord, const FVector2D& );
	DECLARE_DELEGATE_RetVal_TwoParams( FReply, FOnSpawnNodeByShortcutAtLocation, FInputChord, const FVector2f& );

	DECLARE_DELEGATE( FOnNodeSpawnedByKeymap );

	DECLARE_DELEGATE_TwoParams( FOnDisallowedPinConnection, const UEdGraphPin*, const UEdGraphPin* );

	DECLARE_DELEGATE( FOnDoubleClicked );

	DECLARE_DELEGATE_OneParam( FOnNodeSingleClicked, UObject* );

	DECLARE_DELEGATE_RetVal_TwoParams( FReply, FOnMouseButtonDown, const FGeometry&, const FPointerEvent& );

	/** Info about events occurring in/on the graph */
	struct FGraphEditorEvents
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FGraphEditorEvents() = default;
		FGraphEditorEvents(const FGraphEditorEvents& InEvents) = default;
		~FGraphEditorEvents() = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		/** Called when selection changes */
		FOnSelectionChanged OnSelectionChanged;
		/** Called when a node is double clicked */
		FSingleNodeEvent OnNodeDoubleClicked;
		/* Called when focus moves to graph */
		FOnFocused OnFocused;
		/* Called when an actor is dropped on graph */
		PRAGMA_DISABLE_DEPRECATION_WARNINGS FOnDropActor OnDropActor; PRAGMA_ENABLE_DEPRECATION_WARNINGS
		FOnDropActors OnDropActors;
		/* Called when a streaming level is dropped on graph */
		PRAGMA_DISABLE_DEPRECATION_WARNINGS FOnDropStreamingLevel OnDropStreamingLevel; PRAGMA_ENABLE_DEPRECATION_WARNINGS
		FOnDropStreamingLevels OnDropStreamingLevels;
		/** Called when text is being committed on the graph to verify */
		FOnNodeVerifyTextCommit OnVerifyTextCommit;
		/** Called when text is committed on the graph */
		FOnNodeTextCommitted OnTextCommitted;
		/** Called to create context menu for right clicking in empty area */
		PRAGMA_DISABLE_DEPRECATION_WARNINGS FOnCreateActionMenu OnCreateActionMenu; PRAGMA_ENABLE_DEPRECATION_WARNINGS
		FOnCreateActionMenuAtLocation OnCreateActionMenuAtLocation;
		/** Called to create context menu for right clicking a node or pin, same parameters as GetContextMenuActions on schema */
		FOnCreateNodeOrPinMenu OnCreateNodeOrPinMenu;
		/** Called to spawn a node in the graph using a shortcut */
		PRAGMA_DISABLE_DEPRECATION_WARNINGS FOnSpawnNodeByShortcut OnSpawnNodeByShortcut; PRAGMA_ENABLE_DEPRECATION_WARNINGS
		FOnSpawnNodeByShortcutAtLocation OnSpawnNodeByShortcutAtLocation;
		/** Called when a keymap spawns a node */
		FOnNodeSpawnedByKeymap OnNodeSpawnedByKeymap;
		/** Called when the user generates a warning tooltip because a connection was invalid */
		FOnDisallowedPinConnection OnDisallowedPinConnection;
		/** Called when the graph itself is double clicked */
		FOnDoubleClicked OnDoubleClicked;
		/** Called when the graph is clicked */
		FOnMouseButtonDown OnMouseButtonDown;
		/** Called when a node is single-clicked without drag */
		FOnNodeSingleClicked OnNodeSingleClicked;
	};


	SLATE_BEGIN_ARGS(SGraphEditor)
		: _AdditionalCommands( static_cast<FUICommandList*>(NULL) )
		, _IsEditable(true)
		, _DisplayAsReadOnly(false)
		, _IsEmpty(false)
		, _GraphToEdit(NULL)
		, _AutoExpandActionMenu(false)
		, _ShowGraphStateOverlay(true)
		, _AllowConnectionSlicing(true)
		, _GraphSupportsInvalidation(false)
		{}

		SLATE_ARGUMENT( TSharedPtr<FUICommandList>, AdditionalCommands )
		SLATE_ATTRIBUTE( bool, IsEditable )		
		SLATE_ATTRIBUTE( bool, DisplayAsReadOnly )		
		SLATE_ATTRIBUTE( bool, IsEmpty )	
		SLATE_ARGUMENT( TSharedPtr<SWidget>, TitleBar )
		SLATE_ATTRIBUTE( FGraphAppearanceInfo, Appearance )
		SLATE_EVENT( FEdGraphEvent, OnGraphModuleReloaded )
		SLATE_ARGUMENT( UEdGraph*, GraphToEdit )
	
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		UE_DEPRECATED(5.1, "GraphToDiff is no longer supported. Use DiffResults instead")
		SLATE_ARGUMENT( UEdGraph*, GraphToDiff )
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
		SLATE_ARGUMENT( TSharedPtr<TArray<FDiffSingleResult>>, DiffResults )
		SLATE_ATTRIBUTE( int32, FocusedDiffResult )

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		SLATE_ARGUMENT( FGraphEditorEvents, GraphEvents)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		SLATE_ARGUMENT( bool, AutoExpandActionMenu )
		SLATE_ARGUMENT( TWeakPtr<FAssetEditorToolkit>, AssetEditorToolkit)
		SLATE_EVENT(FSimpleDelegate, OnNavigateHistoryBack)
		SLATE_EVENT(FSimpleDelegate, OnNavigateHistoryForward)
		SLATE_EVENT(FOnGraphEditorTickDelegate, OnTick);

		/** Show overlay elements for the graph state such as the PIE and read-only borders and text */
		SLATE_ATTRIBUTE(bool, ShowGraphStateOverlay)
		SLATE_ATTRIBUTE(bool, AllowConnectionSlicing)

		/** If set to true, the GraphPanel will be surrounded with a Dynamic Invalidation Panel */
		SLATE_ARGUMENT(bool, GraphSupportsInvalidation)
	SLATE_END_ARGS()

	/**
	 * Loads the GraphEditorModule and constructs a GraphEditor as a child of this widget.
	 *
	 * @param InArgs   Declaration params from which to construct the widget.
	 */
	UNREALED_API void Construct( const FArguments& InArgs );

	/** @return The current graph being edited */
	UEdGraph* GetCurrentGraph() const
	{
		return EdGraphObj;
	}

	UE_DEPRECATED(5.6, "Slate positions are represented in floats. Please use the function returning FVector2f.")
	virtual FVector2D GetPasteLocation() const UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION
	{
		if (Implementation.IsValid())
		{
			return FVector2D(Implementation->GetPasteLocation2f());
		}
		else
		{
			return FVector2D::ZeroVector;
		}
	}
	virtual UE::Slate::FDeprecateVector2DResult GetPasteLocation2f() const
	{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return UE::Slate::CastToVector2f(GetPasteLocation());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/* Set new viewer location and optionally set the current bookmark */
	UE_DEPRECATED(5.6, "Slate positions are represented in floats. Please use the function returning FVector2f.")
	virtual void SetViewLocation(const FVector2D& Location, float ZoomAmount, const FGuid& BookmarkId = FGuid()) UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION
	{
		if (Implementation.IsValid())
		{
			Implementation->SetViewLocation(UE::Slate::CastToVector2f(Location), ZoomAmount, BookmarkId);
		}
	}
	virtual void SetViewLocation(const FVector2f& Location, float ZoomAmount, const FGuid& BookmarkId = FGuid())
	{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		SetViewLocation(FVector2D(Location), ZoomAmount, BookmarkId);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Gets the view location of the graph
	 *
	 * @param OutLocation		Will have the current view location
	 * @param OutZoomAmount		Will have the current zoom amount
	 */
	UE_DEPRECATED(5.6, "Slate positions are represented in floats. Please use the function returning FVector2f.")
	virtual void GetViewLocation(FVector2D& OutLocation, float& OutZoomAmount)
	{
		if (Implementation.IsValid())
		{
			FVector2f TempLocation;
			Implementation->GetViewLocation(TempLocation, OutZoomAmount);
			OutLocation = FVector2D(TempLocation);
		}
	}
	virtual void GetViewLocation(FVector2f& OutLocation, float& OutZoomAmount)
	{
		FVector2D TempLocation;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		GetViewLocation(TempLocation, OutZoomAmount);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
		OutLocation = UE::Slate::CastToVector2f(TempLocation);
	}

	/**
	 * Gets the current graph view bookmark
	 *
	 * @param OutBookmarkId		Will have the current bookmark ID
	 */
	virtual void GetViewBookmark(FGuid& OutBookmarkId)
	{
		if (Implementation.IsValid())
		{
			Implementation->GetViewBookmark(OutBookmarkId);
		}
	}

	/** Check if node title is visible with optional flag to ensure it is */
	virtual bool IsNodeTitleVisible(const class UEdGraphNode* Node, bool bRequestRename)
	{
		bool bResult = false;
		if (Implementation.IsValid())
		{
			bResult = Implementation->IsNodeTitleVisible(Node, bRequestRename);
		}
		return bResult;
	}

	/* Lock two graph editors together */
	virtual void LockToGraphEditor(TWeakPtr<SGraphEditor> Other)
	{
		if (Implementation.IsValid())
		{
			Implementation->LockToGraphEditor(Other);
		}
	}

	/* Unlock two graph editors from each other */
	virtual void UnlockFromGraphEditor(TWeakPtr<SGraphEditor> Other)
	{
		if (Implementation.IsValid())
		{
			Implementation->UnlockFromGraphEditor(Other);
		}
	}

	/** Bring the specified node into view */
	virtual void JumpToNode( const class UEdGraphNode* JumpToMe, bool bRequestRename = false, bool bSelectNode = true )
	{
		if (Implementation.IsValid())
		{
			Implementation->JumpToNode(JumpToMe, bRequestRename, bSelectNode);
		}
	}

	/** Bring the specified pin into view */
	virtual void JumpToPin( const class UEdGraphPin* JumpToMe )
	{
		if (Implementation.IsValid())
		{
			Implementation->JumpToPin(JumpToMe);
		}
	}

	/** Pin visibility modes */
	enum EPinVisibility
	{
		Pin_Show,
		Pin_HideNoConnection,
		Pin_HideNoConnectionNoDefault
	};

	/*Set the pin visibility mode*/
	virtual void SetPinVisibility(EPinVisibility InVisibility)
	{
		if (Implementation.IsValid())
		{
			Implementation->SetPinVisibility(InVisibility);
		}
	}

	/** Register an active timer on the graph editor. */
	virtual TSharedRef<FActiveTimerHandle> RegisterActiveTimer(float TickPeriod, FWidgetActiveTimerDelegate TickFunction)
	{
		if (Implementation.IsValid())
		{
			return Implementation->RegisterActiveTimer(TickPeriod, TickFunction);
		}
		return TSharedPtr<FActiveTimerHandle>().ToSharedRef();
	}

	/** @return a reference to the list of selected graph nodes */
	virtual const FGraphPanelSelectionSet& GetSelectedNodes() const
	{
		static FGraphPanelSelectionSet NoSelection;

		if (Implementation.IsValid())
		{
			return Implementation->GetSelectedNodes();
		}
		else
		{
			return NoSelection;
		}
	}

	/** Clear the selection */
	virtual void ClearSelectionSet()
	{
		if (Implementation.IsValid())
		{
			Implementation->ClearSelectionSet();
		}
	}

	/** Set the selection status of a node */
	virtual void SetNodeSelection(UEdGraphNode* Node, bool bSelect)
	{
		if (Implementation.IsValid())
		{
			Implementation->SetNodeSelection(Node, bSelect);
		}
	}
	
	/** Select all nodes */
	virtual void SelectAllNodes()
	{
		if (Implementation.IsValid())
		{
			Implementation->SelectAllNodes();
		}		
	}

	virtual class UEdGraphPin* GetGraphPinForMenu()
	{
		if ( Implementation.IsValid() )
		{
			return Implementation->GetGraphPinForMenu();
		}
		else
		{
			return NULL;
		}
	}

	virtual class UEdGraphNode* GetGraphNodeForMenu()
	{
		if ( Implementation.IsValid() )
		{
			return Implementation->GetGraphNodeForMenu();
		}
		else
		{
			return NULL;
		}
	}

	// Zooms out to fit either all nodes or only the selected ones
	virtual void ZoomToFit(bool bOnlySelection)
	{
		if (Implementation.IsValid())
		{
			return Implementation->ZoomToFit(bOnlySelection);
		}
	}

	/** Get Bounds for selected nodes, false if nothing selected*/
	virtual bool GetBoundsForSelectedNodes( class FSlateRect& Rect, float Padding  )
	{
		if (Implementation.IsValid())
		{
			return Implementation->GetBoundsForSelectedNodes(Rect, Padding);
		}
		return false;
	}

	/** Get Bounds for the specified node, returns false on failure */
	virtual bool GetBoundsForNode( const UEdGraphNode* InNode, class FSlateRect& Rect, float Padding ) const
	{
		if (Implementation.IsValid())
		{
			return Implementation->GetBoundsForNode(InNode, Rect, Padding);
		}
		return false;
	}

	virtual void StraightenConnections()
	{
		if (Implementation.IsValid())
		{
			return Implementation->StraightenConnections();
		}
	}

	virtual void StraightenConnections(UEdGraphPin* SourcePin, UEdGraphPin* PinToAlign = nullptr) const
	{
		if (Implementation.IsValid())
		{
			return Implementation->StraightenConnections(SourcePin, PinToAlign);
		}
	}

	virtual void RefreshNode(UEdGraphNode& Node)
	{
		if (Implementation.IsValid())
		{
			return Implementation->RefreshNode(Node);
		}
	}

	// Invoked to let this widget know that the GraphEditor module has been reloaded
	UNREALED_API void OnModuleReloaded();

	// Invoked to let this widget know that the GraphEditor module is being unloaded.
	UNREALED_API void OnModuleUnloading();

	UNREALED_API void NotifyPrePropertyChange(const FString& PropertyName);
	UNREALED_API void NotifyPostPropertyChange(const FPropertyChangedEvent& PropertyChangeEvent, const FString& PropertyName);

	/** Invoked when the Graph being edited changes in some way. */
	virtual void NotifyGraphChanged()
	{
		if (Implementation.IsValid())
		{
			Implementation->NotifyGraphChanged();
		}
	}

	/* Get the title bar if there is one */
	virtual TSharedPtr<SWidget> GetTitleBar() const
	{
		if (Implementation.IsValid())
		{
			return Implementation->GetTitleBar();
		}
		return TSharedPtr<SWidget>();
	}

	/** Show notification on graph */
	virtual void AddNotification(FNotificationInfo& Info, bool bSuccess)
	{
		if (Implementation.IsValid())
		{
			Implementation->AddNotification(Info, bSuccess);
		}
	}

	virtual TSharedPtr<SNotificationItem> AddNotification(FNotificationInfo& Info)
	{
		if (Implementation.IsValid())
		{
			TSharedPtr<SNotificationItem> Notification = Implementation->AddNotification(Info);
			
			if (Notification.IsValid())
			{
				return Notification;
			}
		}
		return nullptr;
	}

	/** Capture keyboard */
	virtual void CaptureKeyboard()
	{
		if (Implementation.IsValid())
		{
			Implementation->CaptureKeyboard();
		}
	}

	/** Sets the current node, pin and connection factory. */
	virtual void SetNodeFactory(const TSharedRef<class FGraphNodeFactory>& NewNodeFactory)
	{
		if (Implementation.IsValid())
		{
			Implementation->SetNodeFactory(NewNodeFactory);
		}
	}
	
	/** Common methods for MaterialEditor and BlueprintEditor's focusing related nodes feature */
	UNREALED_API void ResetAllNodesUnrelatedStates();

	UNREALED_API void FocusCommentNodes(TArray<UEdGraphNode*> &CommentNodes, TArray<UEdGraphNode*> &RelatedNodes);

	virtual void OnCollapseNodes()
	{
		if (Implementation.IsValid())
		{
			Implementation->OnCollapseNodes();
		}
	}

	virtual bool CanCollapseNodes() const
	{
		return Implementation.IsValid() ? Implementation->CanCollapseNodes() : false;
	}

	virtual void OnExpandNodes()
	{
		if (Implementation.IsValid())
		{
			Implementation->OnExpandNodes();
		}
	}

	virtual bool CanExpandNodes() const
	{
		return Implementation.IsValid() ? Implementation->CanExpandNodes() : false;
	}

	virtual void OnAlignTop()
	{
		if (Implementation.IsValid())
		{
			Implementation->OnAlignTop();
		}
	}

	virtual void OnAlignMiddle()
	{
		if (Implementation.IsValid())
		{
			Implementation->OnAlignMiddle();
		}
	}

	virtual void OnAlignBottom()
	{
		if (Implementation.IsValid())
		{
			Implementation->OnAlignBottom();
		}
	}

	virtual void OnAlignLeft()
	{
		if (Implementation.IsValid())
		{
			Implementation->OnAlignLeft();
		}
	}

	virtual void OnAlignCenter()
	{
		if (Implementation.IsValid())
		{
			Implementation->OnAlignCenter();
		}
	}

	virtual void OnAlignRight()
	{
		if (Implementation.IsValid())
		{
			Implementation->OnAlignRight();
		}
	}


	virtual void OnStraightenConnections()
	{
		if (Implementation.IsValid())
		{
			Implementation->OnStraightenConnections();
		}
	}


	virtual void OnDistributeNodesH()
	{
		if (Implementation.IsValid())
		{
			Implementation->OnDistributeNodesH();
		}
	}

	virtual void OnDistributeNodesV()
	{
		if (Implementation.IsValid())
		{
			Implementation->OnDistributeNodesV();
		}
	}

	virtual void OnStackNodesH()
	{
		if (Implementation.IsValid())
		{
			Implementation->OnStackNodesH();
		}
	}

	virtual void OnStackNodesV()
	{
		if (Implementation.IsValid())
		{
			Implementation->OnStackNodesV();
		}
	}

	virtual int32 GetNumberOfSelectedNodes() const
	{
		if (Implementation.IsValid())
		{
			return Implementation->GetNumberOfSelectedNodes();
		}
		return 0;
	}


	/** Returns the currently selected node if there is a single node selected (if there are multiple nodes selected or none selected, it will return nullptr) */
	virtual UEdGraphNode* GetSingleSelectedNode() const
	{
		if (Implementation.IsValid())
		{
			return Implementation->GetSingleSelectedNode();
		}
		return nullptr;
	}

	// Returns the first graph editor that is viewing the specified graph
	UNREALED_API static TSharedPtr<SGraphEditor> FindGraphEditorForGraph(const UEdGraph* Graph);


	/** Returns the graph panel used for this graph editor */
	virtual SGraphPanel* GetGraphPanel() const
	{
		if (Implementation.IsValid())
		{
			return Implementation->GetGraphPanel();
		}
		return nullptr;
	}

	/** Sets the on tick delegate for the graph editor */
	virtual void SetOnTick(FOnGraphEditorTickDelegate InOnTickDelegate)
	{
		if (Implementation.IsValid())
		{
			Implementation->SetOnTick(InOnTickDelegate);
		}
	}

protected:
	/** Invoked when the underlying Graph is being changed. */
	virtual void OnGraphChanged(const struct FEdGraphEditAction& InAction)
	{
		if (Implementation.IsValid())
		{
			Implementation->OnGraphChanged(InAction);
		}
	}

private:
	static void RegisterGraphEditor(const TSharedRef<SGraphEditor>& InGraphEditor);

	void ConstructImplementation(const FArguments& InArgs);
	static void UpgradeDeprecatedDelegates(FGraphEditorEvents& EventsToUpdate);
protected:
	/** The Graph we are currently editing */
	UEdGraph* EdGraphObj;

	TWeakPtr<FAssetEditorToolkit> AssetEditorToolkit;
private:
	/** The actual implementation of the GraphEditor */
	TSharedPtr<SGraphEditor> Implementation;

	/** Active GraphEditor wrappers; we will notify these about the module being unloaded so they can handle it gracefully. */
	UNREALED_API static TArray< TWeakPtr<SGraphEditor> > AllInstances;

	// This callback is triggered whenever the graph module is reloaded
	FEdGraphEvent OnGraphModuleReloadedCallback;

	// The graph editor module needs to access AllInstances, but no-one else should be able to
	friend class FGraphEditorModule;
};
