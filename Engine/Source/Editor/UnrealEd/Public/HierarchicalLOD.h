// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/MemStack.h"
#include "LODCluster.h"
#include "Engine/EngineTypes.h"

#include "Engine/DeveloperSettings.h"
#include "HierarchicalLOD.generated.h"

class AHierarchicalLODVolume;
class ALODActor;

/*=============================================================================
	HierarchicalLOD.h: Hierarchical LOD definition.
=============================================================================*/

class AActor;
class AHierarchicalLODVolume;
class ALODActor;
class UHierarchicalLODSetup;
class ULevel;
class UWorld;

UCLASS(config = Engine, meta = (DisplayName = "Hierarchical LOD"), defaultconfig, MinimalAPI)
class UHierarchicalLODSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

public:
	/** If enabled will force the project set HLOD level settings to be used across all levels in the project when Building Clusters */
	UPROPERTY(EditAnywhere, config, Category = HLODSystem)
	bool bForceSettingsInAllMaps;

	/** If enabled, will save LOD actors descriptions in the HLOD packages */
	UPROPERTY(EditAnywhere, config, Category = HLODSystem)
	bool bSaveLODActorsToHLODPackages;

	/** When set in combination with */
	UPROPERTY(EditAnywhere, config, Category = HLODSystem, meta=(editcondition="bForceSettingsInAllMaps"))
	TSoftClassPtr<UHierarchicalLODSetup> DefaultSetup;
	
	UPROPERTY(config, EditAnywhere, Category = HLODSystem, AdvancedDisplay, meta = (DisplayName = "Directories containing maps used for building HLOD data through the commandlet", RelativeToGameContentDir))
	TArray<FDirectoryPath> DirectoriesForHLODCommandlet;

	UPROPERTY(config, EditAnywhere, Category = HLODSystem, AdvancedDisplay, meta = (DisplayName = "Map UAssets used for building HLOD data through the ", RelativeToGameContentDir, LongPackageName))
	TArray<FFilePath> MapsToBuild;

	/** Base material used for creating a Constant Material Instance as the Proxy Material */
	UPROPERTY(EditAnywhere, config, Category = HLODSystem)
	TSoftObjectPtr<class UMaterialInterface> BaseMaterial;

#if WITH_EDITOR
	static UNREALED_API bool IsValidFlattenMaterial(const UMaterialInterface* InBaseMaterial, bool bShowToaster);

	UNREALED_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
};

/**
 *
 *	This is Hierarchical LOD builder
 *
 * This builds list of clusters and make sure it's sorted in the order of lower cost to high and merge clusters
 **/
struct FHierarchicalLODBuilder
{
	UNREALED_API FHierarchicalLODBuilder(UWorld* InWorld, bool bInPersistentLevelOnly = false);

	/** DO NOT USE. This constructor is for internal usage only for hot-reload purposes. */
	UNREALED_API FHierarchicalLODBuilder();

	/**
	* Build, Builds the clusters and spawn LODActors with their merged Static Meshes
	*/
	UNREALED_API void Build();
	
	/**
	* PreviewBuild, Builds the clusters and spawns LODActors but without actually creating/merging new StaticMeshes
	*/
	UNREALED_API void PreviewBuild();

	/**
	* Clear all the HLODs and the ALODActors that were created for them
	*/
	UNREALED_API void ClearHLODs();

	/**
	* Clear only the ALODActorsPreview 
	*/
	UNREALED_API void ClearPreviewBuild();

	/** Builds the LOD meshes for all LODActors inside of the World's Levels */
	UNREALED_API void BuildMeshesForLODActors(bool bForceAll);

	/** Saves HLOD meshes for actors in all the World's levels */
	UNREALED_API void SaveMeshesForActors();

	/** Get the list of mesh packages to save for a given level */
	UNREALED_API void GetMeshesPackagesToSave(ULevel* InLevel, TSet<UPackage*>& InHLODPackagesToSave, const FString& PreviousLevelName = "");

	/** Delete HLOD packages that are empty. */
	UNREALED_API void DeleteEmptyHLODPackages(ULevel* InLevel);

	/** 
	 * @param	bInForce	Whether to force the recalculation of this actor's build flag. If this is false then the cached flag is used an only recalculated every so often.
	 * @return whether a build is needed (i.e. any LOD actors are dirty) 
	 */
	UNREALED_API bool NeedsBuild(bool bInForce = false) const;

	/**
	* Build a single LOD Actor's mesh
	*
	* @param LODActor - LODActor to build mesh for
	* @param LODLevel - LODLevel to build the mesh for
	*/
	UNREALED_API void BuildMeshForLODActor(ALODActor* LODActor, const uint32 LODLevel);

private:
	/**
	* Builds the clusters (HLODs) for InLevel
	*
	* @param InLevel - Level for which the HLODs are currently being build
	*/
	void BuildClusters(ULevel* InLevel);

	/** Generates a single cluster for the ULevel (across all HLOD levels) */
	void GenerateAsSingleCluster(const int32 NumHLODLevels, ULevel* InLevel);

	/**
	* Initializes the clusters, creating one for each actor within the level eligble for HLOD-ing
	*
	* @param InLevel - Level for which the HLODs are currently being build
	* @param LODIdx - LOD index we are building
	* @param CullCost - Test variable for tweaking HighestCost
	*/
	void InitializeClusters(ULevel* InLevel, const int32 LODIdx, float CullCost, bool const bVolumesOnly);

	/**
	* Merges clusters and builds actors for resulting (valid) clusters
	*
	* @param InLevel - Level for which the HLODs are currently being build
	* @param LODIdx - LOD index we are building, used for determining which StaticMesh LOD to use
	* @param HighestCost - Allowed HighestCost for this LOD level
	* @param MinNumActors - Minimum number of actors for this LOD level
	*/
	void MergeClustersAndBuildActors(ULevel* InLevel, const int32 LODIdx, float HighestCost, int32 MinNumActors);

	/**
	* Finds the minimal spanning tree MST for the clusters by sorting on their cost ( Lower == better )
	*/
	void FindMST();

	/* Retrieves HierarchicalLODVolumes and creates a cluster for each individual one
	*
	* @param InLevel	Level for which the HLODs are currently being build
	* @param LODIdx		LOD index to process
	*/
	void HandleHLODVolumes(ULevel* InLevel, int32 LODIdx);

	/**
	* Determine whether or not this level should have HLODs built for it in the specified world
	*
	* @param World - The world the level is part of
	* @param Level - The level to test
	* @return bool
	*/
	bool ShouldBuildHLODForLevel(const UWorld* World, const ULevel* Level) const;

	/**
	* Determine whether or not this actor is eligble for HLOD creation
	*
	* @param Actor - Actor to test
	* @return bool
	*/
	bool ShouldGenerateCluster(AActor* Actor, const int32 HLODLevelIndex);
	
	/**
	* if criteria matches, creates new LODActor and replace current Actors with that. We don't need
	* this clears previous actors and sets to this new actor
	* this is required when new LOD is created from these actors, this will be replaced
	* to save memory and to reduce memory increase during this process, we discard previous actors and replace with this actor
	*
	* @param InLevel - Level for which currently the HLODs are being build
	* @param LODIdx - LOD index to build
	* @return ALODActor*
	*/
	ALODActor* CreateLODActor(const FLODCluster& InCluster, ULevel* InLevel, const int32 LODIdx);

	/**
	* Deletes LOD actors from the world	
	*
	* @param InLevel - Level to delete the actors from
	* @return void
	*/
	void DeleteLODActors(ULevel* InLevel);

	/**
	 * Create a temporary level in which we'll spawn newly created LODActor.
	 * This is to avoid dirtying the main level when no changes are detected.
	 */
	void CreateTempLODActorLevel(ULevel* InLevel);

	/**
	 * Delete the temporary LODActor level.
	 */
	void ApplyClusteringChanges(ULevel* InLevel);

	/** Array of LOD Clusters - this is only valid within scope since mem stack allocator */
	TArray<FLODCluster, TMemStackAllocator<>> Clusters;

	/** Owning world HLODs are created for */
	UWorld*	World;

	/** Whether we should build HLOD for all sublevels or only the persistent level */
	bool bPersistentLevelOnly;

	/** Array of LOD clusters created for the HierachicalLODVolumes found within the level */
	TMap<AHierarchicalLODVolume*, FLODCluster> HLODVolumeClusters;	
	TMap<ALODActor*, AHierarchicalLODVolume*> HLODVolumeActors;

	const UHierarchicalLODSettings* HLODSettings;

	/** LOD Actors per HLOD level */
	TArray<TArray<ALODActor*>> LODLevelLODActors;

	/** Valid Static Mesh actors in level (populated during initialize clusters) */
	TArray<AActor*> ValidStaticMeshActorsInLevel;
	/** Actors which were rejected from the previous HLOD level(s) */
	TArray<AActor*> RejectedActorsInLevel;

	/** Temporary LODActor levels */
	ULevel* TempLevel;

	/** Previous LODActors found in level */
	TArray<ALODActor*> OldLODActors;

	/** Newly spawned LODActors from cluster(s) rebuilding */
	TArray<ALODActor*> NewLODActors;
};
