// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Volume.h"
#include "HierarchicalLODVolume.generated.h"

/** An invisible volume used to manually define/create an HLOD cluster. */
UCLASS(HideCategories = (Actor, Collision, Cooking, Input, LOD, Physics, Replication, Rendering), MinimalAPI)
class AHierarchicalLODVolume : public AVolume
{
	GENERATED_UCLASS_BODY()

	virtual bool IsEditorOnly() const override { return true; }
	virtual bool NeedsLoadForClient() const override { return false; }
	virtual bool NeedsLoadForServer() const override { return false; }
	virtual bool IsLevelBoundsRelevant() const override { return false; }

	bool IsActorIncluded(const AActor* InActor) const;

public:
	bool AppliesToHLODLevel(int32 LODIdx) const;

public:
	/** When set this volume will incorporate actors which bounds overlap with the volume, otherwise only actors which are completely inside of the volume are incorporated */
	UPROPERTY(EditAnywhere, Category = "HLOD Volume")
	bool bIncludeOverlappingActors;

	/** If set, this volume will only be applied to HLOD levels contained in the array.  If empty, it will apply to ALL HLOD levels */
	UPROPERTY(EditAnywhere, Category = "HLOD Volume")
	TArray<int32> ApplyOnlyToSpecificHLODLevels;
};
