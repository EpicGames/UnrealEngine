// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"

/**
 * Public interface for Actor Editor Context Clients
 */

DECLARE_MULTICAST_DELEGATE_OneParam(FOnActorEditorContextClientChanged, struct IActorEditorContextClient*);

class UWorld;
class AActor;
class UActorEditorContextStateCollection;

struct FActorEditorContextClientDisplayInfo
{
	FActorEditorContextClientDisplayInfo() : Brush(nullptr) {}
	FString Title;
	const FSlateBrush* Brush;
};

enum class EActorEditorContextAction
{
	ApplyContext,
	ResetContext,
	PushContext,
	PushDuplicateContext,
	PopContext,
	InitializeContextFromActor,
};

struct IActorEditorContextClient
{
	virtual void OnExecuteActorEditorContextAction(UWorld* InWorld, const EActorEditorContextAction& InType, AActor* InActor = nullptr) = 0;
	virtual void CaptureActorEditorContextState(UWorld* InWorld, UActorEditorContextStateCollection* InStateCollection) const = 0;
	virtual void RestoreActorEditorContextState(UWorld* InWorld, const UActorEditorContextStateCollection* InStateCollection) = 0;
	virtual bool GetActorEditorContextDisplayInfo(UWorld* InWorld, FActorEditorContextClientDisplayInfo& OutDiplayInfo) const = 0;
	virtual bool CanResetContext(UWorld* InWorld) const = 0;
	virtual TSharedRef<SWidget> GetActorEditorContextWidget(UWorld* InWorld) const = 0;
	virtual FOnActorEditorContextClientChanged& GetOnActorEditorContextClientChanged() = 0;
};