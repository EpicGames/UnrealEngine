// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Widgets/SViewport.h"

class FEditorViewportClient;
class FViewport;
class FSceneViewport;

/**
 * Public interface to SLevelViewport
 */
class IAssetViewport
{

public:
	/**
	 * Begins a play in editor session in this viewport. Swaps the current editor client with the passed in client
	 *
	 * @param PlayClient	The client to swap with the editor client
	 */
	virtual void StartPlayInEditorSession( class UGameViewportClient* PlayClient, const bool bInSimulateInEditor ) = 0;

	/**
	 * Ends a currently active play in editor session in this viewport
	 */
	virtual void EndPlayInEditorSession() = 0;

	/**
	 * Swaps the active PIE viewport client with the level editor viewport client for simulate in editor
	 * It is not valid to call this unless we have an active play in editor session
	 */
	virtual void SwapViewportsForSimulateInEditor() = 0;

	/**
	 * Swaps the active SIE viewport client with the inactive PIE viewport (Swaps back to the game)
 	 * It is not valid to call this unless we have an active simulate in editor session
	 */
	virtual void SwapViewportsForPlayInEditor() = 0;

	/** Called by the editor when simulate mode is started with this viewport */
	virtual void OnSimulateSessionStarted() = 0;

	/** Called by the editor when simulate mode with this viewport finishes */
	virtual void OnSimulateSessionFinished() = 0;
	
	/**
	 * Registers a game viewport with the Slate application so that specific messages can be routed directly to this level viewport if it is an active PIE viewport
	 */
	virtual void RegisterGameViewportIfPIE() = 0;

	/**
	 * @return true if this viewport has a play in editor session (could be inactive)                   
	 */
	virtual bool HasPlayInEditorViewport() const = 0; 

	virtual TSharedPtr<FSceneViewport> GetSharedActiveViewport() const  = 0;

	/**
	 * @return The editor client for this viewport
	 */
	virtual FEditorViewportClient& GetAssetViewportClient() = 0;

	/**
	 * Gets the active viewport.
	 *
	 * @return The Active Viewport.
	 */
	virtual FViewport* GetActiveViewport() = 0;

	/**
	 * Attempts to switch this viewport into immersive mode
	 *
	 * @param	bWantImmersive Whether to switch to immersive mode, or switch back to normal mode
	 * @param	bAllowAnimation	True to allow animation when transitioning, otherwise false
	 */
	virtual void MakeImmersive( const bool bWantImmersive, const bool bAllowAnimation ) = 0;

	/**
	 * @return true if this viewport is in immersive mode, false otherwise
	 */
	virtual bool IsImmersive () const = 0;

	/**
	 * Called when game view should be toggled
	 */
	virtual void ToggleGameView() = 0;

	/**
	 * @return true if we are in game view                   
	 */
	virtual bool IsInGameView() const = 0;
	
	/** Adds a widget overlaid over the viewport */
	virtual void AddOverlayWidget(TSharedRef<SWidget> OverlaidWidget, int32 ZOrder=INDEX_NONE) = 0;

	/** Removes a widget that was previously overlaid on to this viewport */
	virtual void RemoveOverlayWidget(TSharedRef<SWidget> OverlaidWidget) = 0;

	/** Returns the SLevelViewport widget (const). This is NOT the actual SViewport widget, just a wrapper that contains one. */
	virtual TSharedRef< const SWidget > AsWidget() const = 0;

	/** Returns the SLevelViewport widget. This is NOT the actual SViewport widget, just a wrapper that contains one. */
	virtual TSharedRef< SWidget > AsWidget() = 0;

	/** Returns the SViewport widget contained within the SLevelViewport. This is the actual viewport widget that renders the level. */
	virtual TWeakPtr< class SViewport > GetViewportWidget() = 0;
};
