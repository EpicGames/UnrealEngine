// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeature.h"
#include "Stats/Stats.h"

struct FRequestPlaySessionParams;

#define UE_API UNREALED_API

namespace UE::PIE
{

class IOutOfDateAssetCompilation : public IModularFeature
{
public:
	virtual ~IOutOfDateAssetCompilation() = default;

	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName("OutOfDateAssetCompilation");
		return FeatureName;
	}

	bool HandleOutOfDateAssets(const FRequestPlaySessionParams& InPlaySessionParams, TArray<UObject*>& OutAssetsWithErrors) const
	{
		QUICK_SCOPE_CYCLE_COUNTER(IOutOfDateAssetCompilation_HandleOutOfDateAssets);
		return HandleOutOfDateAssets_Internal(InPlaySessionParams, OutAssetsWithErrors);
	}

	virtual bool HandleOutOfDateAssets_Internal(const FRequestPlaySessionParams& InPlaySessionParams, TArray<UObject*>& OutAssetsWithErrors) const = 0;

	// Present user with dialog showing all assets with errors, returns whether the user chose to ignore the errors
	static UE_API bool ShowCompilationErrorsDialog(TConstArrayView<UObject*> InAssetsWithErrors, const FText& AssetTypeText, TFunction<void(const TWeakObjectPtr<UObject>&)> InOpenAssetFunc);
};
}

#undef UE_API // UNREALED_API
