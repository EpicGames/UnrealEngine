// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#include "Features/IModularFeature.h"
#include "Templates/ValueOrError.h"
#include "UObject/NameTypes.h"

#define UE_API UNREALED_API

class IPIEAuthorizer : public IModularFeature
{
public:
	virtual ~IPIEAuthorizer() = default;

	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName(TEXT("PIEAuthorizer"));
		return FeatureName;
	}

	/**
	 * Helper function that can be used to iterate through all registered implementations of IPIEAuthorizer
	 * to gate play-in-editor functionality, where a PIE session may be
	 * undesirable given some external plugin's state.
	 *
	 * @param bIsSimulateInEditor Whether the request is for Simulate In Editor (SIE) or Play In Editor (PIE)
	 * @return Whether PIE is allowed and optional error description if denied.
	 * @note This function will not require any interactions with users and can be called for "silent" queries
	 */
	static UE_API TValueOrError<bool, FText> IsPIEAuthorized(bool bIsSimulateInEditor);

	/**
	 * Potentially blocking methods that can be used to gate play-in-editor functionality, where a PIE session may be
	 * undesirable given some external plugin's state.
	 *
	 * @param bIsSimulateInEditor Whether the request is for Simulate In Editor (SIE) or Play In Editor (PIE)
	 * @return Whether PIE is allowed and optional error description if denied.
	 * @note This method might require interactions with users.
	 */
	TValueOrError<bool, FText> RequestPIEPermission(bool bIsSimulateInEditor) const;

protected:

	/**
	 * Non-blocking methods that can be overridden to gate play-in-editor functionality.
	 *
	 * @param bIsSimulateInEditor Whether the request is for Simulate In Editor (SIE) or Play In Editor (PIE)
	 * @return Whether PIE is allowed and optional error description if denied.
	 * @note This method should NEVER require interactions with users.
	 */
	virtual TValueOrError<bool, FText> IsPIEAuthorizedInternal(bool bIsSimulateInEditor) const = 0;

	/**
	 * Used to gate play-in-editor functionality, where a PIE session may be
	 * undesirable given some external plugin's state.
	 *
	 * @param bIsSimulateInEditor Whether the request is for Simulate In Editor (SIE) or Play In Editor (PIE)
	 * @return Whether PIE is allowed and optional error description if denied.
	 * @note This method might require interactions with users.
	 */
	virtual UE_API TValueOrError<bool, FText> RequestPIEPermissionInternal(bool bIsSimulateInEditor) const;

	UE_DEPRECATED(5.8, "Use the version taking a localized string")
	virtual UE_API bool RequestPIEPermission(bool bIsSimulateInEditor, class FString& OutReason) const final;
};

#undef UE_API
