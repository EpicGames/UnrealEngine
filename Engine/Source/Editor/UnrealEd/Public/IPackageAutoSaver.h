// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"

#define UE_API UNREALED_API

/** An interface to handle the creation, destruction, and restoration of auto-saved packages */
class IPackageAutoSaver
{
public:
	IPackageAutoSaver()
	{
	}

	virtual ~IPackageAutoSaver() = default;

	/**
	 * Update the auto-save count based on the delta-value provided
	 *
	 * @param DeltaSeconds Delta-time (in seconds) since the last update
	 */
	virtual void UpdateAutoSaveCount(const float DeltaSeconds) = 0;

	/** Resets the auto-save timer */
	virtual void ResetAutoSaveTimer() = 0;

	/** Forces auto-save timer to equal auto-save time limit, causing an auto-save attempt */
	virtual void ForceAutoSaveTimer() = 0;

	/** 
	 * Pause the autosaver timer-based saving, preventing auto-save from being triggered automatically.
	 * The autosave can still be triggered manually with the bForceAutoSave parameter in AttemptAutoSave().
	 * Each call to SuspendAutoSave() must be paired with a ResumeAutoSave() call.
	 */
	virtual void SuspendAutoSave() = 0;

	/** Resume the autosave timer-based saving, effectively unpausing it. */
	virtual void ResumeAutoSave() = 0;

	/** Returns if the autosave timer is currently suspended or not. */
	virtual bool IsAutoSaveSuspended() const = 0;

	/**
	 * Forces the auto-save timer to be the auto-save time limit less the passed in value
	 *
	 * @param TimeTillAutoSave The time that will remain until the auto-save limit.
	 */
	virtual void ForceMinimumTimeTillAutoSave(const float TimeTillAutoSave = 10.0f) = 0;

	/**
	 * Attempts to auto-save the level and/or content packages, if those features are enabled.
	 * 
	 * @param bForceAutoSave Force the auto-save to happen, regardless of the current timer or other mode restrictions/user interaction. The caller is responsible for ensuring the editor is in a good state to auto-save when passing bForceAutoSave as true.
	 * 
	 * @return True if an auto-save ran successfully, false otherwise.
	 */
	virtual bool AttemptAutoSave(const bool bForceAutoSave = false) = 0;

	/** @return If we are currently auto-saving (default is to only check for transient auto-saves for legacy reasons) */
	virtual bool IsAutoSaving(const EPackageAutoSaveType AutoSaveType = EPackageAutoSaveType::Transient) const = 0;

	/** Load the restore file from disk (if present) */
	virtual void LoadRestoreFile() = 0;

	/**
	 * Update the file on disk that's used to restore auto-saved packages in the event of a crash
	 * 
	 * @param bRestoreEnabled Is the restore enabled, or is it disabled because we've shut-down cleanly, or are running under the debugger?
	 */
	virtual void UpdateRestoreFile(const bool bRestoreEnabled) = 0;

	/** @return Does we have any information about packages that can be restored */
	virtual bool HasPackagesToRestore() const = 0;

	/** Offer the user the chance to restore any packages that were dirty and have auto-saves */
	virtual void OfferToRestorePackages() = 0;

	/** Called when packages are deleted in the editor */
	virtual void OnPackagesDeleted(const TArray<UPackage*>& DeletedPackages) = 0;

	/**
	 * Set a flag to bypass the recovery UI prompt and to automatically decline package recovery. The auto-saver
	 * flow runs as usual but when the user is normally prompted for recovery, the system will bypass the prompt and select
	 * to dismiss automatically if the flag it set. The flag must be raised before the Engine reach the recovery point
	 * during the boot process, otherwise, it has no effect.
	 */
	virtual void DisableRestorePromptAndDeclinePackageRecovery() = 0;
};

/**
 * Helper scope guard struct used for suspending the autosave.
 * While active, the auto-saver won't trigger automatically.
 */
struct FPackageAutoSaverScopeGuard : private FNoncopyable
{
	UE_API FPackageAutoSaverScopeGuard();
	UE_API FPackageAutoSaverScopeGuard(FPackageAutoSaverScopeGuard&& Other);
	UE_API ~FPackageAutoSaverScopeGuard();
	UE_API FPackageAutoSaverScopeGuard& operator=(FPackageAutoSaverScopeGuard&& Other);
private:
	bool bSuspended = false;
};

#undef UE_API