// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FUICommandInfo;
/**
 * Sets or gets the various LOD levels associated with the controller.
 * Used especially for asset viewports.
 */
class IPreviewLODController
{
public:
	virtual ~IPreviewLODController() = default;

	/** Gets the current LOD. A value of INDEX_NONE implies an 'Auto' LOD. */
	virtual int32 GetCurrentLOD() const = 0;
	
	/** Gets whether a given LOD is active. */
	virtual bool IsLODSelected(int32 LODIndex) const = 0;
	
	/** Gets the number of LODs currently supported. */
	virtual int32 GetLODCount() const = 0;
	
	/** Sets the LOD level by index. A value of INDEX_NONE implies an 'Auto' LOD. */
	virtual void SetLODLevel(int32 LODIndex) = 0;
	
	/** Provide LOD commands that should be included before any auto-generated LOD menu items. */
	virtual void FillLODCommands(TArray<TSharedPtr<FUICommandInfo>>& Commands) {}
	
	/**
	 * Set which LOD level an auto-generated list of LOD options should start from.
	 * Allows certain LOD levels to be handled by commands provided by `FillLODCommands()`.
	 */
	virtual int32 GetAutoLODStartingIndex() const { return INDEX_NONE; }
};