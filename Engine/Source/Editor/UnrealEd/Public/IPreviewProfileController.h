// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Sets or gets the available preview profiles. The module(s) offering a preview (like AdvancedPreviewScene) should
 * implement this interface to enable this module (UnrealEd) to generically display or change the current preview profile.
 * 
 * @note This interface was added to prevent circular dependencies between AdvancedPreviewScene and UnrealEd modules.
 */
class IPreviewProfileController
{
public:
	virtual ~IPreviewProfileController() = default;

	/** Returns the list of available preview profiles names. */
	virtual TArray<FString> GetPreviewProfiles(int32& OutCurrentProfileIndex) const = 0;

	/** Set the specified preview profiles as the active one. */
	virtual bool SetActiveProfile(const FString& ProfileName) = 0;

	/** Returns the preview profiles currently active. */
	virtual FString GetActiveProfile() const = 0;

	/** Returns true if user has added one or more of their own profiles */
	virtual bool HasAnyUserProfiles() const = 0;

	/** Invoked after the list of available profiles has changed. */
	DECLARE_EVENT(IPreviewProfileController, FOnPreviewProfileListChanged);
	virtual FOnPreviewProfileListChanged& OnPreviewProfileListChanged() = 0;

	/** Invoked after the active preview profile changed. */
	DECLARE_EVENT(IPreviewProfileController, FOnPreviewProfileChanged);
	virtual FOnPreviewProfileChanged& OnPreviewProfileChanged() = 0;
};
