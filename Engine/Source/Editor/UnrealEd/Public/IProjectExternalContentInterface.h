// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class IPlugin;

namespace UE::Core
{
	class FVersePath;
}

/** Interface to manage project references to external content */
class IProjectExternalContentInterface
{
public:
	IProjectExternalContentInterface() = default;
	IProjectExternalContentInterface(const IProjectExternalContentInterface&) = delete;
	virtual ~IProjectExternalContentInterface() = default;

	IProjectExternalContentInterface& operator=(const IProjectExternalContentInterface&) = delete;

	/** Return whether the project can reference external content */
	virtual bool IsEnabled() const = 0;

	UE_DEPRECATED(5.8, "No longer used.")
	virtual bool HasExternalContent(const FString& VersePath) const { return false; }

	UE_DEPRECATED(5.8, "No longer used.")
	virtual bool IsExternalContentLoaded(const FString& VersePath) const { return false; }

	UE_DEPRECATED(5.8, "No longer used.")
	virtual TArray<FString> GetExternalContentVersePaths() const { return {}; }

	/**
	 * Called upon AddExternalContent completion
	 * @param bSuccess Whether the external content was successfully added to the project
	 * @param Plugins List of loaded plugins hosting the external content
	 */
	DECLARE_DELEGATE_TwoParams(FAddExternalContentComplete, bool /*bSuccess*/, const TArray<TSharedRef<IPlugin>>& /*Plugins*/);

	/**
	 * Add a reference to external content to the project and asynchronously downloads/loads the external content
	 * @param VersePath External content Verse path
	 * @param ReferencingPlugin Plugin that should reference the external content, or unset if unspecified
	 * @param CompleteCallback See FAddExternalContentComplete
	 */
	virtual void AddExternalContent(const UE::Core::FVersePath& VersePath, TSharedPtr<IPlugin> ReferencingPlugin = {}, FAddExternalContentComplete CompleteCallback = {})
	{
		CompleteCallback.ExecuteIfBound(false, /*Plugins=*/{});
	}

	UE_DEPRECATED(5.8, "Use the FVersePath overload.")
	virtual void AddExternalContent(const FString& VersePath, FAddExternalContentComplete CompleteCallback = FAddExternalContentComplete())
	{
		CompleteCallback.ExecuteIfBound(false, /*Plugins=*/{});
	}

	UE_DEPRECATED(5.8, "No longer used.")
	DECLARE_DELEGATE_OneParam(FRemoveExternalContentComplete, bool /*bSuccess*/);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	UE_DEPRECATED(5.8, "No longer used.")
	virtual void RemoveExternalContent(TConstArrayView<FString> VersePaths, FRemoveExternalContentComplete CompleteCallback = FRemoveExternalContentComplete())
	{
		CompleteCallback.ExecuteIfBound(false);
	}

	UE_DEPRECATED(5.8, "No longer used.")
	void RemoveExternalContent(const FString& VersePath, FRemoveExternalContentComplete CompleteCallback = FRemoveExternalContentComplete())
	{
		RemoveExternalContent(MakeArrayView(&VersePath, 1), MoveTemp(CompleteCallback));
	}

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};
