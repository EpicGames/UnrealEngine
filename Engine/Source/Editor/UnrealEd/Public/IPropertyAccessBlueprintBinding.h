// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "EdGraph/EdGraph.h"
#include "Features/IModularFeatures.h"

class FExtender;
class UBlueprint;
class UEdGraph;
class UEdGraphNode;
class UEdGraphPin;

// Delegate used to set the context ID of a property access
DECLARE_DELEGATE_OneParam(FOnSetPropertyAccessContextId, FName /*InContextId*/);

// Delegate used to set the context ID of a property access
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnCanSetPropertyAccessContextId, FName /*InContextId*/);

// Delegate used to get the context ID of a property access
DECLARE_DELEGATE_RetVal(FName, FOnGetPropertyAccessContextId);

// Modular feature allowing property access to bind to a Blueprint
class IPropertyAccessBlueprintBinding : public IModularFeature
{
public:
	virtual ~IPropertyAccessBlueprintBinding() = default;
	
	// A context in which a Blueprint binding can take place
	struct FContext
	{
		// The blueprint we would like to bind to
		const UBlueprint* Blueprint = nullptr;

		// The graph we would like to bind to
		const UEdGraph* Graph = nullptr;

		// The node we would like to bind to
		const UEdGraphNode* Node = nullptr;

		// The pin we would like to bind to
		const UEdGraphPin* Pin = nullptr;
	};
	
	// @return true if this binding can bind to the specified context
	virtual bool CanBindToContext(const FContext& InContext) const = 0;

	// Args passed to MakeBindingMenuExtender
	struct FBindingMenuArgs
	{
		// Delegate used to set the context ID of a property access
		FOnSetPropertyAccessContextId OnSetPropertyAccessContextId;

		// Delegate used to control the enabled state of context IDs
		FOnCanSetPropertyAccessContextId OnCanSetPropertyAccessContextId;
		
		// Delegate used to get the context ID of a property access
		FOnGetPropertyAccessContextId OnGetPropertyAccessContextId;
	};
	
	// Make a menu extender to be used in property access binding menus
	virtual TSharedPtr<FExtender> MakeBindingMenuExtender(const FContext& InContext, const FBindingMenuArgs& InArgs) const = 0;
};
