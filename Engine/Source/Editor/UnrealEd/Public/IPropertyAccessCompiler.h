// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/ArrayView.h"
#include "UObject/ObjectMacros.h"
#include "Features/IModularFeature.h"

#include "IPropertyAccessCompiler.generated.h"

class UObject;
struct FPropertyAccessLibrary;
enum class EPropertyAccessCopyType : uint8;

// DEPRECATED - The various batching of property copy
UENUM()
enum class EPropertyAccessBatchType : uint8
{
	// Copies designed to be called one at a time via ProcessCopy
	Unbatched,

	// Copies designed to be processed in one call to ProcessCopies
	Batched,
};

enum class EPropertyAccessHandleType : int32
{
	Copy,

	Access,
};

// Handle used to track property accesses that are being compiled
struct FPropertyAccessHandle
{
public:
	// Get the internal index
	int32 GetId() const { return Id; }

	// Check this handle for validity
	bool IsValid() const { return Id != INDEX_NONE; }

	// Get the type
	EPropertyAccessHandleType GetType() const { return Type; }

	FPropertyAccessHandle() = default;
	
	FPropertyAccessHandle(int32 InId, EPropertyAccessHandleType InType)
		: Id(InId)
		, Type(InType)
	{
	}

	inline bool operator==(const FPropertyAccessHandle& Other) const
	{
		return Id == Other.Id;
	} 

private:
	// Index used to track accesses by external systems
	int32 Id = INDEX_NONE;

	// The type of this access
	EPropertyAccessHandleType Type = EPropertyAccessHandleType::Copy;
};

inline uint32 GetTypeHash(const FPropertyAccessHandle& Value)
{
	return HashCombine(GetTypeHash(Value.GetId()), GetTypeHash(Value.GetType()));
}

// Handle used to describe property accesses that have been compiled
struct FCompiledPropertyAccessHandle
{
public:
	FCompiledPropertyAccessHandle() = default;
	
	// Get the index into the batch
	int32 GetId() const { return Id; }

	// Get the index of the batch
	int32 GetBatchId() const { return BatchId; }

	// Check this handle for validity
	bool IsValid() const { return Id != INDEX_NONE; }

private:
	friend class FPropertyAccessLibraryCompiler;

	FCompiledPropertyAccessHandle(int32 InId, int32 InBatchId, EPropertyAccessHandleType InType)
		: Id(InId)
		, BatchId(InBatchId)
		, Type(InType)
	{
	}

	// Index into the batch
	int32 Id = INDEX_NONE;

	// Index of the batch
	int32 BatchId = INDEX_NONE;

	// The type of this access
	EPropertyAccessHandleType Type = EPropertyAccessHandleType::Copy;
};

// A helper used to compile a property access library
class IPropertyAccessLibraryCompiler
{
public:
	virtual ~IPropertyAccessLibraryCompiler() = default;

	// Begin compilation - reset the library to its default state
	virtual void BeginCompilation() = 0;
	
	UE_DEPRECATED(5.0, "Please use BeginCompilation without a class arg")
	virtual void BeginCompilation(const UClass* InClass) {}

	// Add a copy to the property access library we are compiling
	// @return a handle to the pending copy. This can be resolved to a true copy index by calling GetCompiledHandle
	virtual FPropertyAccessHandle AddCopy(TArrayView<FString> InSourcePath, TArrayView<FString> InDestPath, const FName& InContextId, UObject* InAssociatedObject = nullptr) = 0;
	
	UE_DEPRECATED(5.0, "Please use AddCopy with a context ID that returns a handle")
	virtual int32 AddCopy(TArrayView<FString> InSourcePath, TArrayView<FString> InDestPath, EPropertyAccessBatchType InBatchType, UObject* InAssociatedObject = nullptr)
	{
		FPropertyAccessHandle Handle = AddCopy(InSourcePath, InDestPath, FName(NAME_None), InAssociatedObject); 
		return Handle.GetId();
	}

	// Add an access to the property access library we are compiling
	// @return a handle to the pending access. This can be resolved to a true access index by calling GetCompiledHandle
	virtual FPropertyAccessHandle AddAccess(TArrayView<FString> InPath, UObject* InAssociatedObject = nullptr) = 0;

	// Post-process the library to finish compilation. @return true if compilation succeeded.
	virtual bool FinishCompilation() = 0;

	// Iterate any errors we have with compilation
	virtual void IterateErrors(TFunctionRef<void(const FText&, UObject*)> InFunction) const = 0;

	// Maps the initial copy handle to a true handle, post compilation
	virtual FCompiledPropertyAccessHandle GetCompiledHandle(FPropertyAccessHandle InHandle) const = 0;

	// Get the access type for the specified handle
	virtual EPropertyAccessCopyType GetCompiledHandleAccessType(FPropertyAccessHandle InHandle) const = 0;
};
