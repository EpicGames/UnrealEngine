// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Misc/Attribute.h"
#include "Features/IModularFeature.h"
#include "Framework/SlateDelegates.h"
#include "Input/Reply.h"
#include "Styling/SlateColor.h"
#include "UObject/Field.h"
#include "UObject/UnrealType.h"

class UBlueprint;
class IPropertyHandle;
class UEdGraph;
class FExtender;
class SWidget;
class FMenuBuilder;
struct FSlateBrush;
struct FButtonStyle;
struct FEdGraphPinType;
class IPropertyAccessLibraryCompiler;
struct FPropertyAccessLibrary;

/** An element in a binding chain */
struct FBindingChainElement
{
	FBindingChainElement(FProperty* InProperty, int32 InArrayIndex = INDEX_NONE)
		: Field(InProperty)
		, ArrayIndex(InArrayIndex)
	{}

	FBindingChainElement(UFunction* InFunction)
		: Field(InFunction)
		, ArrayIndex(INDEX_NONE)
	{}

	/** Field that this this chain element refers to */
	FFieldVariant Field;

	/** Optional array index if this element refers to an array */
	int32 ArrayIndex = INDEX_NONE;
};

/** 
 * Info about a redirector binding. 
 * Redirector bindings allow 
 */
struct FRedirectorBindingInfo
{
	FRedirectorBindingInfo(FName InName, const FText& InDescription, UStruct* InStruct)
		: Name(InName)
		, Description(InDescription)
		, Struct(InStruct)
	{}

	/** The name of the binding */
	FName Name = NAME_None;

	/** Description of the binding, used as tooltip text */
	FText Description;

	/** The struct that the binding will output */
	UStruct* Struct = nullptr;
};

/**
 * Binding context struct allow to describe information for a struct to bind to using the binding widget. An array of structs is passed to the widget to describe the context in which the binding exists.
 * When the widget selectes a property from binding context struct array, the first FBindingChainElement's index correlates to the array passed to the widget.
 */
struct FBindingContextStruct
{
	FBindingContextStruct() = default;
	
	FBindingContextStruct(UStruct* InStruct, const FSlateBrush* InIcon = nullptr, const FText& InDisplayText = FText::GetEmpty(), const FText& InTooltipText = FText::GetEmpty(), const FText& InSection = FText::GetEmpty())
		: Struct(InStruct)
		, Icon(InIcon)
		, DisplayText(InDisplayText)
		, TooltipText(InTooltipText)
		, Section(InSection)
	{}

	/** The struct to bind to. */
	UStruct* Struct = nullptr;

	/** Icon to display in the popup menu. */ 
	const FSlateBrush* Icon = nullptr;

	/** Color of the icon to display in the popup menu. */ 
	TOptional<FLinearColor> Color;

	/** Text to display for this item in the popup. If left empty, struct's display text will be used. */
	FText DisplayText;

	/** Tooltip to show, or if empty, the tool tip will be set to the same as popup text. */
	FText TooltipText;

	/** Name of the section to put the struct to. If left empty, no section will be created. */
	FText Section;

	/** Category separated by | of the struct. Will display as submenus. Not part of the section to avoid converting Text to string and back. */
	FString Category;
};


/** Delegate used to generate a new binding function's name */
DECLARE_DELEGATE_RetVal(FString, FOnGenerateBindingName);

/** Delegate used to open a binding (e.g. a function) */
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnGotoBinding, FName /*InPropertyName*/);

/** Delegate used to see if we can open a binding (e.g. a function) */
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnCanGotoBinding, FName /*InPropertyName*/);

/** Delegate used to check whether a property is considered for binding. Returning false will discard the property and all child properties. */
DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnCanAcceptPropertyOrChildrenWithBindingChain, FProperty* /*InProperty*/, TConstArrayView<FBindingChainElement> /*InBindingChain*/);

// UE_DEPRECATED(5.4, "Please use OnCanAcceptPropertyOrChildrenWithBindingChain instead.")
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnCanAcceptPropertyOrChildren, FProperty* /*InProperty*/);

/** Delegate used to check whether a property can be bound to the property in question */
DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnCanBindPropertyWithBindingChain, FProperty* /*InProperty*/, TConstArrayView<FBindingChainElement> /*InBindingChain*/);

// UE_DEPRECATED(5.4, "Please use OnCanBindPropertyWithBindingChain instead.")
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnCanBindProperty, FProperty* /*InProperty*/);

/** Delegate used to check whether a function can be bound to the property in question */
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnCanBindFunction, UFunction* /*InFunction*/);

/** Delegate called to see if a class can be bound to */
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnCanBindToClass, UClass* /*InClass*/);

// UE_DEPRECATED(5.5, "Please use OnCanBindToContextStructWithIndex instead.")
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnCanBindToContextStruct, UStruct* /*InStruct*/);

/** Delegate called to see if a class can be bound to */
DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnCanBindToContextStructWithIndex, UStruct* /*InStruct*/, int32 /*InStructIndex*/);

/** Delegate called to see if a subobject can be bound to */
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnCanBindToSubObjectClass, UClass* /*InSubObjectClass*/);

/** Delegate called to add a binding */
DECLARE_DELEGATE_TwoParams(FOnAddBinding, FName /*InPropertyName*/, const TArray<FBindingChainElement>& /*InBindingChain*/);

/** Delegate called to remove a binding */
DECLARE_DELEGATE_OneParam(FOnRemoveBinding, FName /*InPropertyName*/);

/** Delegate called to see if we can remove a binding (ie. if it exists) */
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnCanRemoveBinding, FName /*InPropertyName*/);

/** Delegate called once a new function binding has been created */
DECLARE_DELEGATE_TwoParams(FOnNewFunctionBindingCreated, UEdGraph* /*InFunctionGraph*/, UFunction* /*InFunction*/);

/** Delegate called to resolve true type of the instance */
DECLARE_DELEGATE_RetVal_OneParam(UStruct*, FOnResolveIndirection, const TArray<FBindingChainElement>& /*InBindingChain*/);

/** Delegate called once a drag-drop event is dropped on the binding widget */
DECLARE_DELEGATE_RetVal_TwoParams(FReply, FOnDrop, const FGeometry&, const FDragDropEvent&);

/** Delegate called to see if the property has any bindings */
DECLARE_DELEGATE_RetVal(bool, FOnHasAnyBindings);

/** Delegate called to see if the custom generate menu delegated can be used */
DECLARE_DELEGATE_RetVal(bool, FOnCanUseCustomGenerateMenu);

/** Setup arguments structure for a property binding widget */
struct FPropertyBindingWidgetArgs
{
	// Macro needed to avoid deprecation errors when the struct is copied or created in the default methods.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FPropertyBindingWidgetArgs() = default;
	FPropertyBindingWidgetArgs(const FPropertyBindingWidgetArgs&) = default;
	FPropertyBindingWidgetArgs(FPropertyBindingWidgetArgs&&) = default;
	FPropertyBindingWidgetArgs& operator=(const FPropertyBindingWidgetArgs&) = default;
	FPropertyBindingWidgetArgs& operator=(FPropertyBindingWidgetArgs&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	/** An optional bindable property */
	FProperty* Property = nullptr;

	/** An optional signature to use to match binding functions */
	UFunction* BindableSignature = nullptr;

	/** Delegate used to generate a new binding function's name */
	FOnGenerateBindingName OnGenerateBindingName;

	/** Delegate used to open a bound generated function */
	FOnGotoBinding OnGotoBinding;

	/** Delegate used to see if we can open a binding (e.g. a function) */
	FOnCanGotoBinding OnCanGotoBinding;

	/** Delegate used to check whether a property is considered for binding. Returning false will discard the property and all child properties. */
	FOnCanAcceptPropertyOrChildrenWithBindingChain OnCanAcceptPropertyOrChildrenWithBindingChain;

	UE_DEPRECATED(5.4, "Please use OnCanAcceptPropertyOrChildrenWithBindingChain instead.")
	FOnCanAcceptPropertyOrChildren OnCanAcceptPropertyOrChildren;
	
	/** Delegate used to check whether a property can be bound to the property in question */
	FOnCanBindPropertyWithBindingChain OnCanBindPropertyWithBindingChain;

	UE_DEPRECATED(5.4, "Please use OnCanBindPropertyWithBindingChain instead.")
	FOnCanBindProperty OnCanBindProperty;

	/** Delegate used to check whether a function can be bound to the property in question */
	FOnCanBindFunction OnCanBindFunction;

	/** Delegate called to see if a class can be bound to */
	FOnCanBindToClass OnCanBindToClass;

	UE_DEPRECATED(5.5, "Please use OnCanBindToContextStructWithIndex instead.")
	FOnCanBindToContextStruct OnCanBindToContextStruct;

	/** Delegate called to see if a context struct can be directly bound to */
	FOnCanBindToContextStructWithIndex OnCanBindToContextStructWithIndex;
	
	/** Delegate called to see if a subobject can be bound to */
	FOnCanBindToSubObjectClass OnCanBindToSubObjectClass;

	/** Delegate called to add a binding */
	FOnAddBinding OnAddBinding;

	/** Delegate called to remove a binding */
	FOnRemoveBinding OnRemoveBinding;

	/** Delegate called to see if we can remove remove a binding (ie. if it exists) */
	FOnCanRemoveBinding OnCanRemoveBinding;

	/** Delegate called to see if the property has any bindings */
	FOnHasAnyBindings OnHasAnyBindings;

	/** Delegate called once a new function binding has been created */
	FOnNewFunctionBindingCreated OnNewFunctionBindingCreated;

	/** Delegate called to resolve true type of the instance */
	FOnResolveIndirection OnResolveIndirection;
	
	/** Delegate called when a property is dropped on the property binding widget */
	FOnDrop OnDrop;

	/** Delegate called to see if the custom generate menu delegated can be used */
	FOnCanUseCustomGenerateMenu OnCanUseCustomGenerateMenu;

	/** Delegate called to generate an alternative/custom version of the menu */
	FOnGetContent OnGenerateMenu;

	/** The current binding's text label */
	TAttribute<FText> CurrentBindingText;
	
	/** The current binding's text label color */
	TAttribute<FSlateColor> CurrentBindingTextColor;

	/** The current binding's tooltip text label */
	TAttribute<FText> CurrentBindingToolTipText;
	
	/** The current binding's image */
	TAttribute<const FSlateBrush*> CurrentBindingImage;

	/** The current binding's color */
	TAttribute<FLinearColor> CurrentBindingColor;

	/** Menu extender */
	TSharedPtr<FExtender> MenuExtender;

	/** Optional style override for bind button */
	const FButtonStyle* BindButtonStyle = nullptr;

	/** The maximum level of depth to generate */
	uint8 MaxDepth = 10;
	
	/** Whether to generate pure bindings */
	bool bGeneratePureBindings = true;

	/** Whether to allow function bindings (to "this" class of the blueprint in question) */
	bool bAllowFunctionBindings = true;

	/** Whether to allow function library bindings in addition to the passed-in blueprint's class */
	bool bAllowFunctionLibraryBindings = false;
	
	/** Whether to allow property bindings */
	bool bAllowPropertyBindings = true;	
	
	/** Whether to allow array element bindings */
	bool bAllowArrayElementBindings = false;

	/** Whether to allow struct member bindings */
	bool bAllowStructMemberBindings = true;

	/** Whether to allow new bindings to be made from within the widget's UI */
	bool bAllowNewBindings = true;

	/** Whether to allow UObject functions as non-leaf nodes */
	bool bAllowUObjectFunctions = false;
	
	/** Whether to allow only functions marked thread safe */
	bool bAllowOnlyThreadSafeFunctions = false;

	/** Whether to allow UScriptStruct functions as non-leaf nodes */
	bool bAllowStructFunctions = false;	

	/** Whether to format the SPropertyBinding widget to use a link icon (versus the standard combo button) */
	bool bUseLinkIconStyle = false;

	/** When using the link icon style, use a compact mode, ie don't display target type/name */
	bool bLinkIconStyleCompactMode = false;

	/** Authorize or not the link icon display in the property binding widget. */
	bool bCanDisplayLinkIcon = true;
};

/** Enum describing the result of ResolvePropertyAccess */
enum class EPropertyAccessResolveResult
{
	/** Resolution of the path failed */
	Failed,

	/** DEPRECATED - Resolution of the path succeeded and the property is internal to the initial context */
	SucceededInternal,

	/** DEPRECATED - Resolution of the path failed and the property is external to the initial context (i.e. uses an object/redirector indirection) */
	SucceededExternal,

	/** Resolution of the path succeeded */
	Succeeded,
};

/**
 * Result of a property access resolve. Provides information about what the compiler was able to determine about the
 * property access.
 */
struct FPropertyAccessResolveResult
{
	// The success fail of the resolve
	EPropertyAccessResolveResult Result = EPropertyAccessResolveResult::Failed;

	// Whether the resolve was determined to be thread safe
	bool bIsThreadSafe = false;
};

/** Enum describing property compatibility */
enum class EPropertyAccessCompatibility
{
	// Properties are incompatible
	Incompatible,

	// Properties are directly compatible
	Compatible,	

	// Properties can be copied with a simple type promotion
	Promotable,
};

// Context struct describing relevant characteristics of a property copy.
// Used by client code to determine batch ID when called back via FOnPropertyAccessDetermineBatchId
struct FPropertyAccessCopyContext
{
	// The object (usually a K2 node) in which the context takes place. Used for error reporting.
	UObject* Object;
	
	// User-define context name, passed via IPropertyAccessLibraryCompiler::AddCopy
	FName ContextId;

	// Source path as text, for error reporting
	FText SourcePathAsText;

	// Dest path as text, for error reporting
	FText DestPathAsText;
	
	// Whether the source path is thread safe
	bool bSourceThreadSafe;

	// Whether the dest path is thread safe
	bool bDestThreadSafe;
};

// Delegate used to determine batch ID (index) for a particular copy context
DECLARE_DELEGATE_RetVal_OneParam(int32, FOnPropertyAccessDetermineBatchId, const FPropertyAccessCopyContext& /*InContext*/);

// Context used to create a property access library compiler
struct FPropertyAccessLibraryCompilerArgs
{
	FPropertyAccessLibraryCompilerArgs(FPropertyAccessLibrary& InLibrary, const UClass* InClassContext)
		: Library(InLibrary)
		, ClassContext(InClassContext)
	{}

	// The library that will be built
	FPropertyAccessLibrary& Library;

	// The class that provides a root context for the library to be built in
	const UClass* ClassContext;

	// Delegate used to determine batch ID (index) for a particular copy context. If this is not set, then all copies
	// will be batched together in batch 0
	FOnPropertyAccessDetermineBatchId OnDetermineBatchId;
};

/** Editor support for property access system */
class IPropertyAccessEditor : public IModularFeature
{
public:
	virtual ~IPropertyAccessEditor() = default;

	/** 
	 * Make a property binding widget.
	 * @param	InBlueprint		The blueprint that the binding will exist within
	 * @param	InArgs			Optional arguments for the widget
	 * @return a new binding widget
	 */
	virtual TSharedRef<SWidget> MakePropertyBindingWidget(UBlueprint* InBlueprint, const FPropertyBindingWidgetArgs& InArgs = FPropertyBindingWidgetArgs()) const = 0;

	/** 
	 * EXPERIMENTAL: Make a property binding menu for a struct.
	 * @param	MenuBuilder					The menu to add entries to
	 * @param	InArgs						arguments for filtering properties etc. 
	 * @param	InPropertyName				The Property Name to pass to the OnAddBinding callback
	 * @param	InOwnerStruct				The Struct/Class to get properties from
	 * @param	InBindingChain				The base binding chain to InOwnerStruct
	 */
	virtual void FillPropertyMenu(FMenuBuilder& MenuBuilder, const FPropertyBindingWidgetArgs& InArgs, FName InPropertyName, UStruct* InOwnerStruct,
										TArray<TSharedPtr<FBindingChainElement>> InBindingChain) const = 0;
	
	/**
	 * Make a property binding widget.
	 * @param	InBindingContextStructs		An array of structs the binding will exist within
	 * @param	InArgs						Optional arguments for the widget
	 * @return a new binding widget
	 */
	virtual TSharedRef<SWidget> MakePropertyBindingWidget(const TArray<FBindingContextStruct>& InBindingContextStructs, const FPropertyBindingWidgetArgs& InArgs = FPropertyBindingWidgetArgs()) const = 0;
	
	/** Resolve a property access, returning the leaf property and array index if any. @return result structure for more information about the result */
	virtual FPropertyAccessResolveResult ResolvePropertyAccess(const UStruct* InStruct, TArrayView<const FString> InPath, FProperty*& OutProperty, int32& OutArrayIndex) const = 0;

	/** Args used to resolve property access segments. The various functions will be called per-segment when resolved. */
	struct FResolvePropertyAccessArgs
	{
		/** Function called when a property is resolved. Array index is valid for static array properties. For dynamic array properties, use ArrayFunction. */
		TFunction<void(int32 /*SegmentIndex*/, FProperty* /*Property*/, int32 /*StaticArrayIndex*/)> PropertyFunction;

		/** Function called when a dynamic array is resolved. */
		TFunction<void(int32 /*SegmentIndex*/, FArrayProperty* /*Property*/, int32 /*ArrayIndex*/)> ArrayFunction;

		/** Function called when a function is resolved. */
		TFunction<void(int32 /*SegmentIndex*/, UFunction* /*Function*/, FProperty* /*ReturnProperty*/)> FunctionFunction;

		/** 
		 * Whether to use the most up to date classes when traversing the path. 
		 * This can be useful for situations where we are resolving against potentially out of date
		 * classes, but the resulting path will not be valid to use or persist due to functions and properties
		 * being on skeleton classes 
		 */
		bool bUseMostUpToDateClasses = false;
	};
	
	/**
	 * Resolve a property path to a structure, calling back for each segment in path segment order if resolution succeed
	 * @return true if resolution succeeded
	 */
	virtual FPropertyAccessResolveResult ResolvePropertyAccess(const UStruct* InStruct, TArrayView<const FString> InPath, const FResolvePropertyAccessArgs& InArgs) const = 0;

	UE_DEPRECATED(5.0, "Please use ResolvePropertyAccess")
	virtual EPropertyAccessResolveResult ResolveLeafProperty(const UStruct* InStruct, TArrayView<FString> InPath, FProperty*& OutProperty, int32& OutArrayIndex) const
	{
		const FPropertyAccessResolveResult Result = ResolvePropertyAccess(InStruct, InPath, OutProperty, OutArrayIndex);
		return Result.Result;
	}

	// Get the compatibility of the two supplied properties. Ordering matters for promotion (A->B).
	virtual EPropertyAccessCompatibility GetPropertyCompatibility(const FProperty* InPropertyA, const FProperty* InPropertyB) const = 0;

	// Get the compatibility of the two supplied pin types. Ordering matters for promotion (A->B).
	virtual EPropertyAccessCompatibility GetPinTypeCompatibility(const FEdGraphPinType& InPinTypeA, const FEdGraphPinType& InPinTypeB) const = 0;

	// Makes a string path from a binding chain
	virtual void MakeStringPath(const TArray<FBindingChainElement>& InBindingChain, TArray<FString>& OutStringPath) const = 0;

	// Make a property access library compiler, used for building a FPropertyAccessLibrary
	virtual TUniquePtr<IPropertyAccessLibraryCompiler> MakePropertyAccessCompiler(const FPropertyAccessLibraryCompilerArgs& InArgs) const = 0;

	// Make a text representation of a property path
	// @param	InPath		The path to use to generate the text path
	// @param	InStruct	Optional struct to resolve against - if this is supplied then the text path can use 'correct' display names
	virtual FText MakeTextPath(const TArray<FString>& InPath, const UStruct* InStruct = nullptr) const = 0;
};
