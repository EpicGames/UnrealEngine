// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"

class IStaticMeshEditor;
class UStaticMeshSocket;

class ISocketManager : public SCompoundWidget
{
public:
	/** Retrieves the first selected socket. */
	UE_DEPRECATED(5.6, "GetSelectedSocket is deprecated, use GetSelectedSockets instead.")
	virtual UStaticMeshSocket* GetSelectedSocket() const = 0;
	
	/** Retrieves the selected socket. */
	virtual TArray<UStaticMeshSocket*> GetSelectedSockets() const = 0;

	/** Returns true if at least one socket is selected. */
	virtual bool HasSelectedSockets() const = 0;

	/** Sets the selected socket. */
	virtual void SetSelectedSocket(UStaticMeshSocket* InSelectedSocket) = 0;

	/** Adds a socket to the already selected ones. */
	virtual void AddSelectedSocket(UStaticMeshSocket* InSelectedSocket) = 0;

	/** Removes a given socket from the current selection. */
	virtual void RemoveSelectedSocket(const UStaticMeshSocket* InSelectedSocket) = 0;

	/** Deletes the first selected socket. */
	UE_DEPRECATED(5.6, "DeleteSelectedSocket is deprecated, use DeleteSelectedSockets instead.")
	virtual void DeleteSelectedSocket() = 0;

	/** Deletes the selected sockets. */
	virtual void DeleteSelectedSockets() = 0;

	/** Duplicate the first selected socket. */
	UE_DEPRECATED(5.6, "DuplicateSelectedSocket is deprecated, use DuplicateSelectedSockets instead.")
	virtual void DuplicateSelectedSocket() = 0;
	
	/** Duplicate the selected sockets. */
	virtual void DuplicateSelectedSockets() = 0;

	/** Request a rename on the selected socket */
	virtual void RequestRenameSelectedSocket() = 0;

	/** Updates the StaticMesh currently being edited */
	virtual void UpdateStaticMesh() = 0;

	/** Creates a socket manager instance. */
	UNREALED_API static TSharedPtr<ISocketManager> CreateSocketManager(TSharedPtr<class IStaticMeshEditor> InStaticMeshEditor, FSimpleDelegate InOnSocketSelectionChanged );
};
