// Copyright Epic Games, Inc. All Rights Reserved.

/*-----------------------------------------------------------------------------
	Data structures only used for importing skeletal meshes and animations.
-----------------------------------------------------------------------------*/

#pragma once

#include "CoreMinimal.h"
#include "Containers/IndirectArray.h"
#include "Engine/SkinnedAssetCommon.h"
#include "ReferenceSkeleton.h"
#include "Engine/SkeletalMesh.h"
#include "Rendering/SkeletalMeshLODImporterData.h"
#include "SkeletalMeshTypes.h"

class UAssetImportData;
class UMorphTarget;
class UPhysicsAsset;
class USkeletalMeshSocket;
class USkeleton;
class UThumbnailInfo;
class FSkeletalMeshLODModel;
enum class ESkinVertexColorChannel : uint8;

struct FExistingMeshLodSectionData
{
	FExistingMeshLodSectionData(FName InImportedMaterialSlotName, bool InbCastShadow, bool InbVisibleInRayTracing, bool InbRecomputeTangents, ESkinVertexColorChannel InRecomputeTangentsVertexMaskChannel, int32 InGenerateUpTo, bool InbDisabled)
	: ImportedMaterialSlotName(InImportedMaterialSlotName)
	, bCastShadow(InbCastShadow)
	, bVisibleInRayTracing(InbVisibleInRayTracing)
	, bRecomputeTangents(InbRecomputeTangents)
	, RecomputeTangentsVertexMaskChannel(InRecomputeTangentsVertexMaskChannel)
	, GenerateUpTo(InGenerateUpTo)
	, bDisabled(InbDisabled)
	{}
	FName ImportedMaterialSlotName;
	bool bCastShadow;
	bool bVisibleInRayTracing;
	bool bRecomputeTangents;
	ESkinVertexColorChannel RecomputeTangentsVertexMaskChannel;
	int32 GenerateUpTo;
	bool bDisabled;
};


struct FStreamableRenderAssetData
{
	int32 NumCinematicMipLevels;
	FPerQualityLevelInt NoRefStreamingLODBias;
	bool NeverStream;
	bool bGlobalForceMipLevelsToBeResident;

	void Save(const USkeletalMesh* SkeletalMesh)
	{
		NumCinematicMipLevels = SkeletalMesh->NumCinematicMipLevels;
		NoRefStreamingLODBias = SkeletalMesh->GetNoRefStreamingLODBias();
		NeverStream = SkeletalMesh->NeverStream;
		bGlobalForceMipLevelsToBeResident = SkeletalMesh->bGlobalForceMipLevelsToBeResident;
	}
	void Restore(USkeletalMesh* SkeletalMesh) const
	{
		SkeletalMesh->NumCinematicMipLevels = NumCinematicMipLevels;
		SkeletalMesh->SetNoRefStreamingLODBias(NoRefStreamingLODBias);
		SkeletalMesh->NeverStream = NeverStream;
		SkeletalMesh->bGlobalForceMipLevelsToBeResident = bGlobalForceMipLevelsToBeResident;
	}
};

struct FExistingSkelMeshData
{
	TArray<USkeletalMeshSocket*>			ExistingSockets;
	TArray<FInlineReductionCacheData>		ExistingInlineReductionCacheData;
	TIndirectArray<FSkeletalMeshLODModel>	ExistingLODModels;
	TArray<FMeshDescription>				ExistingLODMeshDescriptions;
	TArray<FSkeletalMeshLODInfo>			ExistingLODInfo;
	FReferenceSkeleton						ExistingRefSkeleton;
	TArray<FSkeletalMaterial>				ExistingMaterials;
	bool									bSaveRestoreMaterials;
	TArray<UMorphTarget*>					ExistingMorphTargets;
	TArray<UPhysicsAsset*>					ExistingPhysicsAssets;
	UPhysicsAsset*							ExistingShadowPhysicsAsset;
	USkeleton*								ExistingSkeleton;
	USkeletalMeshLODSettings*				ExistingLODSettings;
	TSubclassOf<UAnimInstance>				ExistingPostProcessAnimBlueprint;
	TSoftObjectPtr<UObject>					ExistingDefaultAnimatingRig;
	UMeshDeformer*							ExistingDefaultMeshDeformer;
	UMeshDeformerCollection*				ExistingTargetMeshDeformers;
	//////////////////////////////////////////////////////////////////////////

	bool									bExistingUseFullPrecisionUVs;
	bool									bExistingUseHighPrecisionTangentBasis;

	TWeakObjectPtr<UAssetImportData>		ExistingAssetImportData;
	TWeakObjectPtr<UThumbnailInfo>			ExistingThumbnailInfo;

	TArray<UClothingAssetBase*>				ExistingClothingAssets;

	bool UseMaterialNameSlotWorkflow;
	//The existing import material data (the state of sections before the reimport)
	TArray<FName> ExistingImportMaterialOriginalNameData;
	TArray<TArray<FExistingMeshLodSectionData>> ExistingImportMeshLodSectionMaterialData;
	//The last import material data (fbx original data before user changes)
	TArray<FName> LastImportMaterialOriginalNameData;
	TArray<TArray<FName>> LastImportMeshLodSectionMaterialData;

	FSkeletalMeshSamplingInfo				ExistingSamplingInfo;
	FPerPlatformInt							MinLOD;
	FPerQualityLevelInt						QualityLevelMinLOD;
	FPerPlatformBool						DisableBelowMinLodStripping;
	bool									bOverrideLODStreamingSettings;
	FPerPlatformBool						bSupportLODStreaming;
	FPerPlatformInt							MaxNumStreamedLODs;
	FPerPlatformInt							MaxNumOptionalLODs;

	TMap<UAssetUserData*, bool>				ExistingAssetUserData;

	USkeletalMesh::FOnMeshChanged			ExistingOnMeshChanged;

	TMap<FName, FString> ExistingUMetaDataTagValues;

	FVector PositiveBoundsExtension;
	FVector NegativeBoundsExtension;

	bool bExistingSupportRayTracing;
	int32 ExistingRayTracingMinLOD;
	EClothLODBiasMode ExistingClothLODBiasMode;

	//Streamable member
	FStreamableRenderAssetData ExistingStreamableRenderAssetData;
};

/** 
 * Optional data passed in when importing a skeletal mesh LDO
 */
class FSkelMeshOptionalImportData
{
public:
	/** extra data used for importing extra weight/bone influences */
	FSkeletalMeshImportData RawMeshInfluencesData;
	int32 MaxBoneCountPerChunk = 0;
};

/**
* Data needed for importing an extra set of vertex influences
*/
struct FSkelMeshExtraInfluenceImportData
{
	FReferenceSkeleton		RefSkeleton;
	TArray<SkeletalMeshImportData::FVertInfluence> Influences;
	TArray<SkeletalMeshImportData::FMeshWedge> Wedges;
	TArray<SkeletalMeshImportData::FMeshFace> Faces;
	TArray<FVector> Points;
	int32 MaxBoneCountPerChunk;
};