// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"

struct FExistingSkelMeshData;
struct FReferenceSkeleton;
struct FSkeletalMaterial;
class FSkeletalMeshImportData;
class FSkeletalMeshLODModel;
class USkeletalMesh;
class USkeleton;

namespace SkeletalMeshImportUtils
{
	/** Backups the given SkeletalMesh into a FExistingSkelMeshData */
	UNREALED_API TSharedPtr<FExistingSkelMeshData> SaveExistingSkelMeshData(USkeletalMesh* SourceSkeletalMesh, bool bSaveMaterials, int32 ReimportLODIndex);

	/** Restore a backed up FExistingSkelMeshData into a SkeletalMesh asset */
	UNREALED_API void RestoreExistingSkelMeshData(const TSharedPtr<const FExistingSkelMeshData>& MeshData, USkeletalMesh* SkeletalMesh, int32 ReimportLODIndex, bool bCanShowDialog, bool bImportSkinningOnly, bool bForceMaterialReset);

	UNREALED_API void ProcessImportMeshInfluences(FSkeletalMeshImportData& ImportData, const FString& SkeletalMeshName);
	UNREALED_API void ProcessImportMeshMaterials(TArray<FSkeletalMaterial>& Materials, FSkeletalMeshImportData& ImportData);
	UNREALED_API bool ProcessImportMeshSkeleton(const USkeleton* SkeletonAsset, FReferenceSkeleton& RefSkeleton, int32& SkeletalDepth, FSkeletalMeshImportData& ImportData);
	UNREALED_API void ApplySkinning(USkeletalMesh* SkeletalMesh, FSkeletalMeshLODModel& SrcLODModel, FSkeletalMeshLODModel& DestLODModel);
}