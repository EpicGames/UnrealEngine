// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	StaticMeshImportUtils.h: Static mesh import functions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "Engine/StaticMesh.h"
#include "MeshReductionSettings.h"
#include "UObject/PerPlatformProperties.h"
#include "PerQualityLevelProperties.h"

struct FMeshDescription;
class FPoly;
class UStaticMeshSocket;
class UAssetImportData;
class UThumbnailInfo;
class UModel;
class UBodySetup;
struct FKAggregateGeom;

namespace UnFbx
{
	struct FBXImportOptions;
}

// LOD data to copy over
struct FExistingLODMeshData
{
	FMeshBuildSettings				ExistingBuildSettings;
	FMeshReductionSettings			ExistingReductionSettings;
	uint32							ExisitingMeshTrianglesCount;
	uint32							ExisitingMeshVerticesCount;
	TUniquePtr<FMeshDescription>	ExistingMeshDescription;
	TArray<FStaticMaterial>			ExistingMaterials;
	FPerPlatformFloat				ExistingScreenSize;
	FString							ExistingSourceImportFilename;
};

struct FExistingStaticMeshData
{
	TArray<FStaticMaterial> 	ExistingMaterials;

	FExistingLODMeshData		HiResSourceData;

	FMeshSectionInfoMap			ExistingSectionInfoMap;
	TArray<FExistingLODMeshData>	ExistingLODData;

	TArray<UStaticMeshSocket*>	ExistingSockets;

	bool						ExistingCustomizedCollision;
	bool						bAutoComputeLODScreenSize;

	int32						ExistingLightMapResolution;
	int32						ExistingLightMapCoordinateIndex;

	TWeakObjectPtr<UAssetImportData> ExistingImportData;
	TWeakObjectPtr<UThumbnailInfo> ExistingThumbnailInfo;

	UModel* ExistingCollisionModel;
	UBodySetup* ExistingBodySetup;

	// A mapping of vertex positions to their color in the existing static mesh
	TMap<FVector3f, FColor>		ExistingVertexColorData;

	float						LpvBiasMultiplier;
	bool						bHasNavigationData;
	FName						LODGroup;
	FPerPlatformInt				MinLOD;
	FPerQualityLevelInt			QualityLevelMinLOD;

	int32						ImportVersion;

	bool UseMaterialNameSlotWorkflow;
	//The last import material data (fbx original data before user changes)
	TArray<FName> LastImportMaterialOriginalNameData;
	TArray<TArray<FName>> LastImportMeshLodSectionMaterialData;

	bool						ExistingGenerateMeshDistanceField;
	int32						ExistingLODForCollision;
	float						ExistingDistanceFieldSelfShadowBias;
	bool						ExistingSupportUniformlyDistributedSampling;
	bool						ExistingAllowCpuAccess;
	FVector3f					ExistingPositiveBoundsExtension;
	FVector3f					ExistingNegativeBoundsExtension;

	bool						ExistingSupportPhysicalMaterialMasks;
	bool						ExistingSupportGpuUniformlyDistributedSampling;
	bool						ExistingSupportRayTracing;
	int32						ExistingLODForOccluderMesh;
	bool						ExistingForceMiplevelsToBeResident;
	bool						ExistingNeverStream;
	int32						ExistingNumCinematicMipLevels;

	FMeshNaniteSettings			ExistingNaniteSettings;

	UStaticMesh::FOnMeshChanged	ExistingOnMeshChanged;
	UStaticMesh* ExistingComplexCollisionMesh = nullptr;

	TMap<FName, FString> ExistingUMetaDataTagValues;
};

namespace StaticMeshImportUtils
{
	UNREALED_API bool DecomposeUCXMesh(const TArray<FVector3f>& CollisionVertices, const TArray<int32>& CollisionFaceIdx, UBodySetup* BodySetup);

	/**
	 *	Function for adding a box collision primitive to the supplied collision geometry based on the mesh of the box.
	 *
	 *	We keep a list of triangle normals found so far. For each normal direction,
	 *	we should have 2 distances from the origin (2 parallel box faces). If the
	 *	mesh is a box, we should have 3 distinct normal directions, and 2 distances
	 *	found for each. The difference between these distances should be the box
	 *	dimensions. The 3 directions give us the key axes, and therefore the
	 *	box transformation matrix. This shouldn't rely on any vertex-ordering on
	 *	the triangles (normals are compared +ve & -ve). It also shouldn't matter
	 *	about how many triangles make up each side (but it will take longer).
	 *	We get the centre of the box from the centre of its AABB.
	 */
	UNREALED_API bool AddBoxGeomFromTris(const TArray<FPoly>& Tris, FKAggregateGeom* AggGeom, const TCHAR* ObjName);

	/**
	 *	Function for adding a sphere collision primitive to the supplied collision geometry based on a set of Verts.
	 *
	 *	Simply put an AABB around mesh and use that to generate center and radius.
	 *	It checks that the AABB is square, and that all vertices are either at the
	 *	center, or within 5% of the radius distance away.
	 */
	UNREALED_API bool AddSphereGeomFromVerts(const TArray<FVector3f>& Verts, FKAggregateGeom* AggGeom, const TCHAR* ObjName);

	UNREALED_API bool AddCapsuleGeomFromVerts(const TArray<FVector3f>& Verts, FKAggregateGeom* AggGeom, const TCHAR* ObjName);

	/** Utility for adding one convex hull from the given verts */
	UNREALED_API bool AddConvexGeomFromVertices(const TArray<FVector3f>& Verts, FKAggregateGeom* AggGeom, const TCHAR* ObjName);

	UNREALED_API TSharedPtr<FExistingStaticMeshData> SaveExistingStaticMeshData(UStaticMesh* ExistingMesh, bool bImportMaterials, int32 LodIndex);

	UNREALED_API TSharedPtr<FExistingStaticMeshData> SaveExistingStaticMeshData(UStaticMesh* ExistingMesh, UnFbx::FBXImportOptions* ImportOptions, int32 LodIndex);

	/* This function is call before building the mesh when we do a re-import*/
	UNREALED_API void RestoreExistingMeshSettings(const FExistingStaticMeshData* ExistingMesh, UStaticMesh* NewMesh, int32 LODIndex);

	UNREALED_API void UpdateSomeLodsImportMeshData(UStaticMesh* NewMesh, TArray<int32>* ReimportLodList);

	UNREALED_API void RestoreExistingMeshData(const TSharedPtr<const FExistingStaticMeshData>& ExistingMeshDataPtr, UStaticMesh* NewMesh, int32 LodLevel, bool bCanShowDialog, bool bForceConflictingMaterialReset);
}