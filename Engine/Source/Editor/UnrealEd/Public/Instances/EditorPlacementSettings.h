// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Instances/InstancedPlacementClientInfo.h"

#include "EditorPlacementSettings.generated.h"

class AInstancedPlacementPartitionActor;
struct FISMComponentDescriptor;
class FArchive;
struct FPropertyChangedEvent;

UCLASS()
class UEditorInstancedPlacementSettings : public UInstancedPlacemenClientSettings
{
	GENERATED_BODY()

public:
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostLoad() override;
	virtual void Serialize(FArchive& Ar) override;

	virtual void RegisterISMDescriptors(AInstancedPlacementPartitionActor* ParentPartitionActor, TSortedMap<int32, TArray<FTransform>>& ISMDefinition) const override;
};