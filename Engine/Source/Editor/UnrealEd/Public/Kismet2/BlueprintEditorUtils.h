// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "Misc/Guid.h"
#include "Misc/NotNull.h"
#include "UObject/Class.h"
#include "Templates/SubclassOf.h"
#include "UObject/UnrealType.h"
#include "Engine/Blueprint.h"
#include "Widgets/SWidget.h"
#include "EdGraph/EdGraph.h"
#include "K2Node_EditablePinBase.h"
#include "ClassViewerModule.h"
#include "EdGraphSchema_K2.h"

class AActor;
class ALevelScriptActor;
class FBlueprintEditor;
class FCompilerResultsLog;
class INameValidatorInterface;
class UActorComponent;
class UBlueprintGeneratedClass;
class USimpleConstructionScript;
class UK2Node_Event;
class UK2Node_Variable;
class ULevelScriptBlueprint;
class USCS_Node;
class UTimelineTemplate;
struct FBlueprintCookedComponentInstancingData;
struct FComponentKey;
class UAnimGraphNode_Root;
class UBlueprint;
struct FBPInterfaceDescription;
class UFunction;
class UK2Node_CallFunction;

/** 
  * Flags describing how to handle graph removal
  */
namespace EGraphRemoveFlags
{
	enum Type
	{
		/** No options */
		None = 0x00000000,

		/** If true recompile the blueprint after removing the graph, false if operations are being batched */
		Recompile = 0x00000001,

		/** If true mark the graph as transient, false otherwise */
		MarkTransient = 0x00000002,

		/** Helper enum for most callers */
		Default = Recompile | MarkTransient
	};
};

struct FFunctionFromNodeHelper
{
	UFunction* const Function;
	const UK2Node* const Node;

	static UNREALED_API UFunction* FunctionFromNode(const UK2Node* Node);

	UNREALED_API FFunctionFromNodeHelper(const UObject* Obj);
};

class FBasePinChangeHelper
{
public:
	static bool NodeIsNotTransient(const UK2Node* Node)
	{
		return (NULL != Node)
			&& !Node->HasAnyFlags(RF_Transient) 
			&& (NULL != Cast<UEdGraph>(Node->GetOuter()));
	}

	virtual ~FBasePinChangeHelper() = default;

	virtual void EditCompositeTunnelNode(class UK2Node_Tunnel* TunnelNode) {}

	virtual void EditMacroInstance(class UK2Node_MacroInstance* MacroInstance, UBlueprint* Blueprint) {}

	virtual void EditCallSite(class UK2Node_CallFunction* CallSite, UBlueprint* Blueprint) {}

	virtual void EditDelegates(class UK2Node_BaseMCDelegate* CallSite, UBlueprint* Blueprint) {}

	virtual void EditCreateDelegates(class UK2Node_CreateDelegate* CallSite) {}

	UNREALED_API void Broadcast(UBlueprint* InBlueprint, class UK2Node_EditablePinBase* InTargetNode, UEdGraph* Graph);
};

class FParamsChangedHelper : public FBasePinChangeHelper
{
public:
	TSet<UBlueprint*> ModifiedBlueprints;
	TSet<UEdGraph*> ModifiedGraphs;

	UNREALED_API virtual void EditCompositeTunnelNode(class UK2Node_Tunnel* TunnelNode) override;

	UNREALED_API virtual void EditMacroInstance(class UK2Node_MacroInstance* MacroInstance, UBlueprint* Blueprint) override;

	UNREALED_API virtual void EditCallSite(class UK2Node_CallFunction* CallSite, UBlueprint* Blueprint) override;

	UNREALED_API virtual void EditDelegates(class UK2Node_BaseMCDelegate* CallSite, UBlueprint* Blueprint) override;

	UNREALED_API virtual void EditCreateDelegates(class UK2Node_CreateDelegate* CallSite) override;

};

struct FUCSComponentId
{
public:
	UNREALED_API FUCSComponentId(const class UK2Node_AddComponent* UCSNode);
	FGuid GetAssociatedGuid() const { return GraphNodeGuid; }

private:
	FGuid GraphNodeGuid;
};

DECLARE_CYCLE_STAT_EXTERN(TEXT("Notify Blueprint Changed"), EKismetCompilerStats_NotifyBlueprintChanged, STATGROUP_KismetCompiler, );

struct FCompilerRelevantNodeLink
{
	UK2Node* Node;
	UEdGraphPin* LinkedPin;

	FCompilerRelevantNodeLink(UK2Node* InNode, UEdGraphPin* InLinkedPin)
		: Node(InNode)
		, LinkedPin(InLinkedPin)
	{
	}
};

/** Array type for GetCompilerRelevantNodeLinks() */
typedef TArray<FCompilerRelevantNodeLink, TInlineAllocator<4> > FCompilerRelevantNodeLinkArray;

class FBlueprintEditorUtils
{
public:

	/**
	 * Schedules and refreshes all nodes in the blueprint, making sure that nodes that affect function signatures get regenerated first
	 */
	static UNREALED_API void RefreshAllNodes(UBlueprint* Blueprint);

	/** Event fired after RefreshAllNodes is called */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnRefreshAllNodes, UBlueprint* /*Blueprint*/);
	static UNREALED_API FOnRefreshAllNodes OnRefreshAllNodesEvent;

	/**
	 * Reconstructs all nodes in the blueprint, node reconstruction order determined by FCompareNodePriority.
	 */
	static UNREALED_API void ReconstructAllNodes(UBlueprint* Blueprint);

	/** Event fired after ReconstructAllNodes is called */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnReconstructAllNodes, UBlueprint* /*Blueprint*/);
	static UNREALED_API FOnReconstructAllNodes OnReconstructAllNodesEvent;

	/**
	 * Optimized refresh of nodes that depend on external blueprints.  Refreshes the nodes, but does not recompile the skeleton class
	 */
	static UNREALED_API void RefreshExternalBlueprintDependencyNodes(UBlueprint* Blueprint, UStruct* RefreshOnlyChild = NULL);

	/**
	 * Refresh the nodes of an individual graph.
	 * 
	 * @param	Graph	The graph to refresh.
	 */
	static UNREALED_API void RefreshGraphNodes(const UEdGraph* Graph);

	/**
	 * Replaces any deprecated nodes with new ones
	 */
	static UNREALED_API void ReplaceDeprecatedNodes(UBlueprint* Blueprint);

	/**
	 * Preloads the object and all the members it owns (nodes, pins, etc)
	 */
	static UNREALED_API void PreloadMembers(UObject* InObject);

	/**
	 * Preloads the construction script, and all templates therein, for the given Blueprint object
	 */
	static UNREALED_API void PreloadConstructionScript(UBlueprint* Blueprint);

	/**
	 * Preloads the given construction script, and all templates therein
	 */
	static UNREALED_API void PreloadConstructionScript(USimpleConstructionScript* SimpleConstructionScript);

	/** 
	 * Helper function to patch the new CDO into the linker where the old one existed 
	 */
	static UNREALED_API void PatchNewCDOIntoLinker(UObject* CDO, FLinkerLoad* Linker, int32 ExportIndex, FUObjectSerializeContext* InLoadContext);

	/** 
	 * Procedure used to remove old function implementations and child properties from data only blueprints.
	 */
	static UNREALED_API void RemoveStaleFunctions(UBlueprintGeneratedClass* Class, UBlueprint* Blueprint);

	/**
	 *  Synchronizes Blueprint's GeneratedClass's properties with the NewVariable declarations in the blueprint
	 */
	static UNREALED_API void RefreshVariables(UBlueprint* Blueprint);

	/** Helper function to punch through and honor UAnimGraphNode_Base::PreloadRequiredAssets, which formerly relied on loading assets during compile */
	static UNREALED_API void PreloadBlueprintSpecificData(UBlueprint* Blueprint);
	
	/**
	 * Links external dependencies
	 */
	static UNREALED_API void LinkExternalDependencies(UBlueprint* Blueprint);

	UE_DEPRECATED(5.6, "Use FLinkerLoad::PRIVATE_PatchNewObjectIntoExport instead, which handles the subobjects. And note the comment on that function: both this deprecated function and the LinkerLoad function are coupled to the linker in complicated ways and should only be used in cases of blueprint reinstancing.")
	static UNREALED_API void PatchCDOSubobjectsIntoExport(UObject* PreviousCDO, UObject* NewCDO);

	/** Recreates class meta data */
	static UNREALED_API void RecreateClassMetaData(UBlueprint* Blueprint, UClass* Class, bool bRemoveExistingMetaData);

	/**
	 * Copies the default properties of all parent blueprint classes in the chain to the specified blueprint's skeleton CDO
	 */
	static UNREALED_API void PropagateParentBlueprintDefaults(UClass* ClassToPropagate);

	/** Called on a Blueprint after it has been duplicated */
	static UNREALED_API void PostDuplicateBlueprint(UBlueprint* Blueprint, bool bDuplicateForPIE);

	/** Consigns the blueprint's generated classes to oblivion */
	static UNREALED_API void RemoveGeneratedClasses(UBlueprint* Blueprint);

	/**
	 * Helper function to get the blueprint that ultimately owns a node.
	 *
	 * @param	InNode	Node to find the blueprint for.
	 * @return	The corresponding blueprint or NULL.
	 */
	static UNREALED_API UBlueprint* FindBlueprintForNode(const UEdGraphNode* Node);

	/**
	 * Helper function to get the blueprint that ultimately owns a node.
	 * This will assert if the node is inside a deleted graph.
	 *
	 * @param	InNode	Node to find the blueprint for.
	 * @return	A valid blueprint.
	 */
	static UNREALED_API TNotNull<UBlueprint*> FindBlueprintForNodeChecked(const UEdGraphNode* Node);

	/**
	 * Helper function to get the blueprint that ultimately owns a graph.
	 *
	 * @param	InGraph	Graph to find the blueprint for.
	 * @return	The corresponding blueprint or NULL.
	 */
	static UNREALED_API UBlueprint* FindBlueprintForGraph(const UEdGraph* Graph);

	/**
	 * Helper function to get the blueprint that ultimately owns a graph.
	 * This will assert if the graph is for a deleted function.
	 *
	 * @param	InGraph	Graph to find the blueprint for.
	 * @return	A valid blueprint.
	 */
	static UNREALED_API TNotNull<UBlueprint*> FindBlueprintForGraphChecked(const UEdGraph* Graph);

	/** Helper function to get the SkeletonClass, returns nullptr for UClasses that are not generated by a UBlueprint */
	static UNREALED_API UClass* GetSkeletonClass(UClass* FromClass);
	static UNREALED_API const UClass* GetSkeletonClass(const UClass* FromClass);

	/** Helper function to get the most up to date class , returns FromClass for native types, SkeletonClass for UBlueprint generated classes */
	static UNREALED_API UClass* GetMostUpToDateClass(UClass* FromClass);
	static UNREALED_API const UClass* GetMostUpToDateClass(const UClass* FromClass);
	
	/** Looks at the most up to data class and returns whether the given property exists in it as well */
	static UNREALED_API bool PropertyStillExists(FProperty* Property);

	/** Returns the skeleton version of the property, skeleton classes are often more up to date than the authoritative GeneratedClass */
	static UNREALED_API FProperty* GetMostUpToDateProperty(FProperty* Property);
	static UNREALED_API const FProperty* GetMostUpToDateProperty(const FProperty* Property);

	static UNREALED_API UFunction* GetMostUpToDateFunction(UFunction* Function);
	static UNREALED_API const UFunction* GetMostUpToDateFunction(const UFunction* Function);

	/**
	 * Updates sources of delegates.
	 */
	static UNREALED_API void UpdateDelegatesInBlueprint(UBlueprint* Blueprint);

	/**
	 * Whether or not the blueprint should regenerate its class on load or not.  This prevents macros and other BP types not marked for reconstruction from being recompiled all the time
	 */
	UE_DEPRECATED(5.6, "Replace with IsCompileOnLoadDisabled, ShouldRegenerateBlueprint handles reentrancy which is no longer possible.")
	static UNREALED_API bool ShouldRegenerateBlueprint(UBlueprint* Blueprint);

	/** Returns true if compilation for the given blueprint has been disabled */
	static UNREALED_API bool IsCompileOnLoadDisabled(UBlueprint* Blueprint);

	/**
	 * Blueprint has structurally changed (added/removed functions, graphs, etc...). Performs the following actions:
	 *  - Recompiles the skeleton class.
	 *  - Notifies any observers.
	 *  - Marks the package as dirty.
	 */
	static UNREALED_API void MarkBlueprintAsStructurallyModified(UBlueprint* Blueprint);

	/**
	 * Blueprint has changed in some manner that invalidates the compiled data (link made/broken, default value changed, etc...)
	 *  - Marks the blueprint as status unknown
	 *  - Marks the package as dirty
	 *
	 * @param	Blueprint				The Blueprint to mark as modified
	 * @param	PropertyChangedEvent	Used when marking the blueprint as modified due to a changed property (optional)
	 */
	static UNREALED_API void MarkBlueprintAsModified(UBlueprint* Blueprint, FPropertyChangedEvent PropertyChangedEvent = FPropertyChangedEvent(nullptr));

	/**
	 * Handles regenerating a blueprint class such that it is in a good state for saving, currently this is 
	 * required to get UserDefinedStruct types up to date before saving. It's likely that there may be
	 * other situations that require fixup in this way.
	 */
	static UNREALED_API void RecompileBeforeSaveIfNeeded(UBlueprint* Blueprint);

	/** See whether or not the specified graph name / entry point name is unique */
	static UNREALED_API bool IsGraphNameUnique(UObject* InOuter, const FName& InName);

	/**
	 * Creates a new empty graph.
	 *
	 * @param	ParentScope		The outer of the new graph (typically a blueprint).
	 * @param	GraphName		Name of the graph to add.
	 * @param	SchemaClass		Schema to use for the new graph.
	 *
	 * @return	null if it fails, else.
	 */
	static UNREALED_API class UEdGraph* CreateNewGraph(UObject* ParentScope, const FName& GraphName, TSubclassOf<class UEdGraph> GraphClass, TSubclassOf<class UEdGraphSchema> SchemaClass);

	/**
	 * Creates a new function graph with a signature that matches InNode
	 *
	 * @param InNode        Node to copy signature from
	 * @param InSchemaClass The schema for the new graph
	 */
	static UNREALED_API void CreateMatchingFunction(UK2Node_CallFunction* InNode, TSubclassOf<class UEdGraphSchema> InSchemaClass);

	/**
	 * Creates a function graph, but does not add it to the blueprint.  If bIsUserCreated is true, the entry/exit nodes will be editable. 
	 * SignatureFromObject is used to find signature for entry/exit nodes if using an existing signature.
	 * The template argument SignatureType should be UClass or UFunction.
	 */
	template <typename SignatureType>
	static void CreateFunctionGraph(UBlueprint* Blueprint, class UEdGraph* Graph, bool bIsUserCreated, SignatureType* SignatureFromObject)
	{
		// Give the schema a chance to fill out any required nodes (like the entry node or results node)
		const UEdGraphSchema* Schema = Graph->GetSchema();
		const UEdGraphSchema_K2* K2Schema = Cast<const UEdGraphSchema_K2>(Graph->GetSchema());

		Schema->CreateDefaultNodesForGraph(*Graph);

		if ( K2Schema != NULL )
		{
			K2Schema->CreateFunctionGraphTerminators(*Graph, SignatureFromObject);

			if ( bIsUserCreated )
			{
				// We need to flag the entry node to make sure that the compiled function is callable from Kismet2
				int32 ExtraFunctionFlags = ( FUNC_BlueprintCallable | FUNC_BlueprintEvent | FUNC_Public );
				if ( BPTYPE_FunctionLibrary == Blueprint->BlueprintType )
				{
					ExtraFunctionFlags |= FUNC_Static;
				}
				if ( BPTYPE_Const == Blueprint->BlueprintType )
				{
					ExtraFunctionFlags |= FUNC_Const;
				}
				// We need to mark the function entry as editable so that we can
				// set metadata on it if it is an editor utility blueprint/widget:
				K2Schema->MarkFunctionEntryAsEditable(Graph, true);
				if( IsEditorUtilityBlueprint( Blueprint ))
				{
					if( FKismetUserDeclaredFunctionMetadata* MetaData = GetGraphFunctionMetaData( Graph ))
					{
						MetaData->bCallInEditor = true;
					}
				}
				K2Schema->AddExtraFunctionFlags(Graph, ExtraFunctionFlags);
			}
		}
	}

	/** 
	 * Adds a function graph to this blueprint.  If bIsUserCreated is true, the entry/exit nodes will be editable. 
	 * SignatureFromObject is used to find signature for entry/exit nodes if using an existing signature.
	 * The template argument SignatureType should be UClass or UFunction.
	 */
	template <typename SignatureType>
	static void AddFunctionGraph(UBlueprint* Blueprint, class UEdGraph* Graph, bool bIsUserCreated, SignatureType* SignatureFromObject)
	{
		CreateFunctionGraph(Blueprint, Graph, bIsUserCreated, SignatureFromObject);

		Blueprint->FunctionGraphs.Add(Graph);

		// Potentially adjust variable names for any child blueprints
		ValidateBlueprintChildVariables(Blueprint, Graph->GetFName());

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
	}

	/**
	* Check if the blueprint and function are valid options for conversion to an event (BP is not a function library)
	*
	* @return	True if this function can be converted to a custom event
	*/
	static UNREALED_API bool IsFunctionConvertableToEvent(UBlueprint* const BlueprintObj, UFunction* const Function);

	/**
	* Get the override class of a given function from its name
	* 
	* @param Blueprint		Blueprint to check the function on
	* @param FuncName		Name of the function
	* @param OutFunction	The function that has this name
	* 
	* @return The override class of a given function
	*/
	static UNREALED_API UClass* const GetOverrideFunctionClass(UBlueprint* Blueprint, const FName FuncName, UFunction** OutFunction = nullptr);

	/** Adds a macro graph to this blueprint.  If bIsUserCreated is true, the entry/exit nodes will be editable. SignatureFromClass is used to find signature for entry/exit nodes if using an existing signature. */
	static UNREALED_API void AddMacroGraph(UBlueprint* Blueprint, class UEdGraph* Graph,  bool bIsUserCreated, UClass* SignatureFromClass);

	/** Adds an interface graph to this blueprint */
	static UNREALED_API void AddInterfaceGraph(UBlueprint* Blueprint, class UEdGraph* Graph, UClass* InterfaceClass);

	/** Adds an ubergraph page to this blueprint */
	static UNREALED_API void AddUbergraphPage(UBlueprint* Blueprint, class UEdGraph* Graph);

	/** Returns the name of the Ubergraph Function that the provided blueprint uses */
	static UNREALED_API FName GetUbergraphFunctionName(const UBlueprint* ForBlueprint);

	/** Adds a domain-specific graph to this blueprint */
	static UNREALED_API void AddDomainSpecificGraph(UBlueprint* Blueprint, class UEdGraph* Graph);

	/**
	 * Remove the supplied set of graphs from the Blueprint.
	 *
	 * @param	GraphsToRemove	The graphs to remove.
	 */
	static UNREALED_API void RemoveGraphs( UBlueprint* Blueprint, const TArray<class UEdGraph*>& GraphsToRemove );

	/**
	 * Removes the supplied graph from the Blueprint.
	 *
	 * @param Blueprint			The blueprint containing the graph
	 * @param GraphToRemove		The graph to remove.
	 * @param Flags				Options to control the removal process
	 */
	static UNREALED_API void RemoveGraph( UBlueprint* Blueprint, class UEdGraph* GraphToRemove, EGraphRemoveFlags::Type Flags = EGraphRemoveFlags::Default );

	/**
	 * Tries to rename the supplied graph.
	 * Cleans up function entry node if one exists and marks objects for modification
	 *
	 * @param Graph				The graph to rename.
	 * @param NewName			The new name for the graph
	 */
	static UNREALED_API void RenameGraph(class UEdGraph* Graph, const FString& NewName );

	/**
	 * Renames the graph of the supplied node with a valid name based off of the suggestion.
	 *
	 * @param GraphNode			The node of the graph to rename.
	 * @param DesiredName		The initial form of the name to try
	 */
	static UNREALED_API void RenameGraphWithSuggestion(class UEdGraph* Graph, TSharedPtr<class INameValidatorInterface> NameValidator, const FString& DesiredName );

	/**
	 * Removes the supplied node from the Blueprint.
	 *
	 * @param Node				The node to remove.
	 * @param bDontRecompile	If true, the blueprint will not be marked as modified, and will not be recompiled.  Useful for if you are removing several node at once, and don't want to recompile each time
	 */
	static UNREALED_API void RemoveNode (UBlueprint* Blueprint, UEdGraphNode* Node, bool bDontRecompile=false);

	/**
	 * Returns the graph's top level graph (climbing up the hierarchy until there are no more graphs)
	 *
	 * @param InGraph		The graph to find the parent of
	 *
	 * @return				The top level graph
	 */
	static UNREALED_API UEdGraph* GetTopLevelGraph(const UEdGraph* InGraph);

	/** Determines if the graph is ReadOnly, this differs from editable in that it is never expected to be edited and is in a read-only state */
	static UNREALED_API bool IsGraphReadOnly(UEdGraph* InGraph);

	/** Look to see if an event already exists to override a particular function */
	static UNREALED_API UK2Node_Event* FindOverrideForFunction(const UBlueprint* Blueprint, const UClass* SignatureClass, FName SignatureName);

	/** Find the Custom Event if it already exists in the Blueprint */
	static UNREALED_API UK2Node_Event* FindCustomEventNode(const UBlueprint* Blueprint, FName const CustomName);

	/** Returns all nodes in all graphs of the specified class */
	template< class T > 
	static inline void GetAllNodesOfClass( const UBlueprint* Blueprint, TArray<T*>& OutNodes )
	{
		TArray<UEdGraph*> AllGraphs;
		Blueprint->GetAllGraphs(AllGraphs);
		for(int32 i=0; i<AllGraphs.Num(); i++)
		{
			check(AllGraphs[i] != NULL);
			TArray<T*> GraphNodes;
			AllGraphs[i]->GetNodesOfClass<T>(GraphNodes);
			OutNodes.Append(GraphNodes);
		}
	}

	/** Returns all nodes in all graphs of at least the minimum node type */
	template< class MinNodeType, class ArrayClassType>
	static inline void GetAllNodesOfClassEx(const UBlueprint* Blueprint, TArray<ArrayClassType*>& OutNodes)
	{
		TArray<UEdGraph*> AllGraphs;
		Blueprint->GetAllGraphs(AllGraphs);
		for(UEdGraph* Graph : AllGraphs)
		{
			check(Graph != nullptr);
			Graph->GetNodesOfClassEx<MinNodeType, ArrayClassType>(OutNodes);
		}
	}

	/**
	 * Searches all nodes in a Blueprint and checks for a matching Guid
	 *
	 * @param InBlueprint			The Blueprint to search
	 * @param InNodeGuid			The Guid to check Blueprints against
	 *
	 * @return						Returns a Node with a matching Guid
	 */
	static UEdGraphNode* GetNodeByGUID(const UBlueprint* InBlueprint, const FGuid& InNodeGuid)
	{
		TArray<UEdGraphNode*> GraphNodes;
		GetAllNodesOfClass(InBlueprint, GraphNodes);

		for(UEdGraphNode* GraphNode : GraphNodes)
		{
			if(GraphNode->NodeGuid == InNodeGuid)
			{
				return GraphNode;
			}
		}
		return nullptr;
	}

	/** Gather all bps that Blueprint depends on */
	static UNREALED_API void GatherDependencies(const UBlueprint* Blueprint, TSet<TWeakObjectPtr<UBlueprint>>& OutDependencies, TSet<TWeakObjectPtr<UStruct>>& OutUDSDependencies);

	/** Returns cached a list of loaded Blueprints that are dependent on the given Blueprint. */
	static UNREALED_API void GetDependentBlueprints(UBlueprint* Blueprint, TArray<UBlueprint*>& DependentBlueprints);

	/** Searches the reference graph to find blueprints that are dependent on this BP */
	static UNREALED_API void FindDependentBlueprints(UBlueprint* Blueprint, TArray<UBlueprint*>& DependentBlueprints);

	/** Ensures, that CachedDependencies in BP are up to date */
	static UNREALED_API void EnsureCachedDependenciesUpToDate(UBlueprint* Blueprint);

	/** returns if a graph is an intermediate build product */
	static UNREALED_API bool IsGraphIntermediate(const UEdGraph* Graph);

	/** @return true if the blueprint does not contain any special logic or variables or other elements that require a full compile. */
	static UNREALED_API bool IsDataOnlyBlueprint(const UBlueprint* Blueprint);

	/** Returns whether or not the blueprint is const during execution */
	static UNREALED_API bool IsBlueprintConst(const UBlueprint* Blueprint);

	/** Returns whether or not the blueprint is an editor utility blueprint or widget */
	static UNREALED_API bool IsEditorUtilityBlueprint(const UBlueprint* Blueprint);

	/**
	 * Whether or not this is an actor-based blueprint, and supports features like the uber-graph, components, etc
	 *
	 * @return	Whether or not this is an actor based blueprint
	 */
	static UNREALED_API bool IsActorBased(const UBlueprint* Blueprint);

	/**
	 * @return Whether or not this is a component-based blueprint
	 */
	static UNREALED_API bool IsComponentBased(const UBlueprint* Blueprint);

	/**
	 * Whether or not this blueprint is an interface, used only for defining functions to implement
	 *
	 * @return	Whether or not this is an interface blueprint
	 */
	static UNREALED_API bool IsInterfaceBlueprint(const UBlueprint* Blueprint);

	/**
	* Whether or not this graph is an interface graph (i.e. is from an interface blueprint)
	* 
	* @return	Whether or not this is an interface graph
	*/
	static UNREALED_API bool IsInterfaceGraph(const UEdGraph* Graph);

	/**
	 * Whether or not this blueprint is an interface, used only for defining functions to implement
	 *
	 * @return	Whether or not this is a level script blueprint
	 */
	static UNREALED_API bool IsLevelScriptBlueprint(const UBlueprint* Blueprint);

	/** Returns whether the parent class of the specified blueprint is also a blueprint */
	static UNREALED_API bool IsParentClassABlueprint(const UBlueprint* Blueprint);

	/** Returns whether the parent class of the specified blueprint is an editable blueprint */
	static UNREALED_API bool IsParentClassAnEditableBlueprint(const UBlueprint* Blueprint);

	/**
	 * Whether or not this class represents a class generated by an anonymous actor class stored in a level 
	 *
	 * @return	Whether or not this is an anonymous blueprint
	 */
	static UNREALED_API bool IsAnonymousBlueprintClass(const UClass* Class);

	/**
	 * Checks for events in the argument class
	 * @param Class	The class to check for events.
	 */
	static UNREALED_API bool CanClassGenerateEvents(const UClass* Class);

	/**
	 * Returns true if this blueprint can have a child class created based on it
	 */
	static UNREALED_API bool CanCreateChildBlueprint(const UBlueprint* BP);

	/**
	 * If a blueprint is directly tied to a level (level script and anonymous blueprints), this will return a pointer to that level
	 *
	 * @return	The level, if any, tied to this blueprint
	 */
	static UNREALED_API class ULevel* GetLevelFromBlueprint(const UBlueprint* Blueprint);

	/** Do we support construction scripts */
	static UNREALED_API bool SupportsConstructionScript(const UBlueprint* Blueprint);

	/** Returns the user construction script, if any */
	static UNREALED_API UEdGraph* FindUserConstructionScript(const UBlueprint* Blueprint);

	/** Returns the event graph, if any */
	static UNREALED_API UEdGraph* FindEventGraph(const UBlueprint* Blueprint);

	/** Checks if given graph is an event graph */
	static UNREALED_API bool IsEventGraph(const UEdGraph* InGraph);

	/** Checks if given node is a tunnel instance node */
	static UNREALED_API bool IsTunnelInstanceNode(const UEdGraphNode* InGraphNode);

	/** See if a class is the one generated by this blueprint */
	static UNREALED_API bool DoesBlueprintDeriveFrom(const UBlueprint* Blueprint, UClass* TestClass);

	/** See if a field (property, function etc) is part of the blueprint chain, or  */
	static UNREALED_API bool DoesBlueprintContainField(const UBlueprint* Blueprint, UField* TestField);

	/** Returns whether or not the blueprint supports overriding functions */
	static UNREALED_API bool DoesSupportOverridingFunctions(const UBlueprint* Blueprint);

	/** Returns whether or not the blueprint supports timelines */
	static UNREALED_API bool DoesSupportTimelines(const UBlueprint* Blueprint);

	/** Returns whether or not the blueprint supports event graphs*/
	static UNREALED_API bool DoesSupportEventGraphs(const UBlueprint* Blueprint);

	/** Returns whether or not the blueprint supports implementing interfaces */
	static UNREALED_API bool DoesSupportImplementingInterfaces(const UBlueprint* Blueprint);

	/** Returns whether or not the blueprint supports components */
	static UNREALED_API bool DoesSupportComponents(UBlueprint const* Blueprint);

	/** Returns whether or not the blueprint supports default values (IE has a CDO) */
	static UNREALED_API bool DoesSupportDefaults(UBlueprint const* Blueprint);

	/** Returns whether or not the blueprint graph supports local variables */
	static UNREALED_API bool DoesSupportLocalVariables(UEdGraph const* InGraph);

	// Returns a descriptive name of the type of blueprint passed in
	static UNREALED_API FString GetBlueprintTypeDescription(const UBlueprint* Blueprint);

	/** Constructs a class picker widget for reparenting the specified blueprint(s) */
	static UNREALED_API TSharedRef<SWidget> ConstructBlueprintParentClassPicker( const TArray< UBlueprint* >& Blueprints, const FOnClassPicked& OnPicked);

	/** Try to open reparent menu for specified blueprint */
	static UNREALED_API void OpenReparentBlueprintMenu( UBlueprint* Blueprint, const TSharedRef<SWidget>& ParentContent, const FOnClassPicked& OnPicked);
	static UNREALED_API void OpenReparentBlueprintMenu( const TArray< UBlueprint* >& Blueprints, const TSharedRef<SWidget>& ParentContent, const FOnClassPicked& OnPicked);

	/** Constructs a class picker widget for adding interfaces for the specified blueprint(s) */
	static UNREALED_API TSharedRef<SWidget> ConstructBlueprintInterfaceClassPicker( const TArray< UBlueprint* >& Blueprints, const FOnClassPicked& OnPicked);

	/** return find first native class in the hierarchy */
	static UNREALED_API UClass* FindFirstNativeClass(UClass* Class);
	static UNREALED_API const UClass* FindFirstNativeClass(const UClass* Class);

	/** returns true if this blueprints signature (inc. visibility) was determined by UHT, rather than the blueprint compiler */
	static UNREALED_API bool IsNativeSignature(const UFunction* Fn);

	/**
	 * Gets the names of all graphs in the Blueprint
	 *
	 * @param [in,out]	GraphNames	The graph names will be appended to this array.
	 */
	static UNREALED_API void GetAllGraphNames(const UBlueprint* Blueprint, TSet<FName>& GraphNames);

	/**
	 * Gets the compiler-relevant (i.e. non-ignorable) node links from the given pin.
	 *
	 * @param			FromPin			The pin to start searching from.
	 * @param			OutNodeLinks	Will contain the given pin + owning node if compiler-relevant, or all nodes linked to the owning node at the matching "pass-through" pin that are compiler-relevant. Empty if no compiler-relevant node links can be found from the given pin.
	 */
	static UNREALED_API void GetCompilerRelevantNodeLinks(UEdGraphPin* FromPin, FCompilerRelevantNodeLinkArray& OutNodeLinks);

	/**
	 * Finds the first compiler-relevant (i.e. non-ignorable) node from the given pin.
	 *
	 * @param			FromPin			The pin to start searching from.
	 *
	 * @return			The given pin's owning node if compiler-relevant, or the first node linked to the owning node at the matching "pass-through" pin that is compiler-relevant. May be NULL if no compiler-relevant nodes can be found from the given pin.
	 */
	static UNREALED_API UK2Node* FindFirstCompilerRelevantNode(UEdGraphPin* FromPin);

	/**
	 * Finds the first compiler-relevant (i.e. non-ignorable) node from the given pin and returns the owned pin that's linked.
	 *
	 * @param			FromPin			The pin to start searching from.
	 *
	 * @return			The given pin if its owning node is compiler-relevant, or the first pin linked to the owning node at the matching "pass-through" pin that is owned by a compiler-relevant node. May be NULL if no compiler-relevant nodes can be found from the given pin.
	 */
	static UNREALED_API UEdGraphPin* FindFirstCompilerRelevantLinkedPin(UEdGraphPin* FromPin);

	/**
	 * Removes all local bookmarks that reference the given Blueprint asset.
	 *
	 * @param			ForBlueprint	The Blueprint asset for which to remove local Bookmarks.
	 */
	static UNREALED_API void RemoveAllLocalBookmarks(const UBlueprint* ForBlueprint);

	//////////////////////////////////////////////////////////////////////////
	// Functions

	/**
	 * Gets a list of function names currently in use in the blueprint, based on the skeleton class
	 *
	 * @param			Blueprint		The blueprint to check
	 * @param [in,out]	FunctionNames	List of function names currently in use
	 */
	static UNREALED_API void GetFunctionNameList(const UBlueprint* Blueprint, TSet<FName>& FunctionNames);

	/**
	 * Gets a list of delegates names in the blueprint, based on the skeleton class
	 *
	 * @param			Blueprint		The blueprint to check
	 * @param [in,out]	DelegatesNames	List of function names currently in use
	 */
	static UNREALED_API void GetDelegateNameList(const UBlueprint* Blueprint, TSet<FName>& DelegatesNames);

	/** 
	 * Get a graph for delegate signature with given name, from given blueprint.
	 * 
	 * @param			Blueprint		Blueprint owning the delegate signature graph
	 * @param			DelegateName	Name of delegate.
	 * @return			Graph of delegate-signature function.
	 */
	static UNREALED_API UEdGraph* GetDelegateSignatureGraphByName(UBlueprint* Blueprint, FName DelegateName);

	/** Checks if given graph contains a delegate signature */
	static UNREALED_API bool IsDelegateSignatureGraph(const UEdGraph* Graph);

	/** Checks if given graph is owned by a Math Expression node */
	static UNREALED_API bool IsMathExpressionGraph(const UEdGraph* InGraph);

	/**
	 * Gets a list of pins that should hidden for a given function in a given graph
	 *
	 * @param			Graph			The graph that you're looking to call the function from (some functions hide different pins depending on the graph they're in)
	 * @param			Function		The function to consider
	 * @param [out]		HiddenPins		Set of pins that should be hidden
	 * @param [out]		OutInternalPins	Subset of hidden pins that are marked for internal use only rather than marked as hidden (optional)
	 */
	static UNREALED_API void GetHiddenPinsForFunction(UEdGraph const* Graph, UFunction const* Function, TSet<FName>& HiddenPins, TSet<FName>* OutInternalPins = nullptr);

	/** Makes sure that calls to parent functions are valid, and removes them if not */
	static UNREALED_API void ConformCallsToParentFunctions(UBlueprint* Blueprint);

	//////////////////////////////////////////////////////////////////////////
	// Events

	/** Makes sure that all events we handle exist, and replace with custom events if not */
	static UNREALED_API void ConformImplementedEvents(UBlueprint* Blueprint);

	//////////////////////////////////////////////////////////////////////////
	// Variables

	/**
	 * Checks if pin type stores proper type for a variable or parameter. Especially if the UDStruct is valid.
	 *
	 * @param		Type	Checked pin type.
	 *
	 * @return				if type is valid
	 */
	static UNREALED_API bool IsPinTypeValid(const FEdGraphPinType& Type);

	/**
	* Ensures the validity of each pin connection on the given node. Outputs compiler error if invalid
	* 
	* @param Node			The node to check all linked pins on
	* @param MessageLog		BP compiler results log to output any error messages to
	*/
	static UNREALED_API void ValidatePinConnections(const UEdGraphNode* Node, FCompilerResultsLog& MessageLog);

	/**
	* If the given node is from an editor only module but is placed in a runtime blueprint
	* then place a warning in the message log that it will not be included in a cooked build. 
	* 
	* @param Node			Node to check the outer package on
	* @param MessageLog		BP Compiler results log to output messages to
	*/
	static UNREALED_API void ValidateEditorOnlyNodes(const UK2Node* Node, FCompilerResultsLog& MessageLog);

	/**
	 * Gets the visible class variable list.  This includes both variables introduced here and in all superclasses.
	 *
	 * @param [in,out]	VisibleVariables	The visible variables will be appended to this array.
	 */
	static UNREALED_API void GetClassVariableList(const UBlueprint* Blueprint, TSet<FName>& VisibleVariables, bool bIncludePrivateVars=false);

	/**
	 * Gets variables of specified type
	 *
	 * @param 			FEdGraphPinType	 			Type of variables to look for
	 * @param [in,out]	VisibleVariables			The visible variables will be appended to this array.
	 */
	static UNREALED_API void GetNewVariablesOfType( const UBlueprint* Blueprint, const FEdGraphPinType& Type, TArray<FName>& OutVars);

	/**
	 * Gets local variables of specified type
	 *
	 * @param 			FEdGraphPinType	 			Type of variables to look for
	 * @param [in,out]	VisibleVariables			The visible variables will be appended to this array.
	 */
	static UNREALED_API void GetLocalVariablesOfType( const UEdGraph* Graph, const FEdGraphPinType& Type, TArray<FName>& OutVars);

	/**
	 * Adds a member variable to the blueprint.  It cannot mask a variable in any superclass.
	 *
	 * @param	NewVarName	Name of the new variable.
	 * @param	NewVarType	Type of the new variable.
	 * @param	DefaultValue	Default value stored as string
	 *
	 * @return	true if it succeeds, false if it fails.
	 */
	static UNREALED_API bool AddMemberVariable(UBlueprint* Blueprint, const FName& NewVarName, const FEdGraphPinType& NewVarType, const FString& DefaultValue = FString());

	/**
	 * Duplicates a variable from one Blueprint to another blueprint
	 *
	 * @param InFromBlueprint				The Blueprint the variable can be found in
	 * @param InToBlueprint					The Blueprint the new variable should be added to (can be the same blueprint)
	 * @param InVariableToDuplicate			Variable name to be found and duplicated
	 *
	 * @return								Returns the name of the new variable or NAME_None if failed to duplicate
	 */
	static UNREALED_API FName DuplicateMemberVariable(UBlueprint* InFromBlueprint, UBlueprint* InToBlueprint, FName InVariableToDuplicate);

	/**
	 * Duplicates a variable given its name and Blueprint
	 *
	 * @param InBlueprint					The Blueprint the variable can be found in
	 * @paramInScope						Local variable's scope
	 * @param InVariableToDuplicate			Variable name to be found and duplicated
	 *
	 * @return								Returns the name of the new variable or NAME_None if failed to duplicate
	 */
	static UNREALED_API FName DuplicateVariable(UBlueprint* InBlueprint, const UStruct* InScope, FName InVariableToDuplicate);

	/**
	 * Internal function that deep copies a variable description
	 *
	 * @param InBlueprint					The blueprint to ensure a uniquely named variable in
	 * @param InVariableToDuplicate			Variable description to duplicate
	 *
	 * @return								Returns a copy of the passed in variable description.
	 */
	static UNREALED_API FBPVariableDescription DuplicateVariableDescription(UBlueprint* InBlueprint, FBPVariableDescription& InVariableDescription);

	/**
	 * Removes a member variable if it was declared in this blueprint and not in a base class.
	 *
	 * @param	VarName	Name of the variable to be removed.
	 */
	static UNREALED_API void RemoveMemberVariable(UBlueprint* Blueprint, const FName VarName);
	
	/**
	 * Removes member variables if they were declared in this blueprint and not in a base class.
	 *
	 * @param	VarNames	Names of the variable to be removed.
	 */
	static UNREALED_API void BulkRemoveMemberVariables(UBlueprint* Blueprint, const TArray<FName>& VarNames);

	/**
	 * Removes a field notify variable from the metadata of all other field notify variables and functions.
	 *
	 * @param	VarName	Name of the variable to be removed.
	 */
	static UNREALED_API void RemoveFieldNotifyFromAllMetadata(UBlueprint* Blueprint, const FName VarName);

	/**
	 * Removes all unused member variables.
	 *
	 * @param	OutUsedProperties	The list of used variables in the blueprint
	 * @param	OutUnusedProperties	The list of unused variables in the blueprint
	 */
	static UNREALED_API void GetUsedAndUnusedVariables(UBlueprint* Blueprint, TArray<FProperty*>& OutUsedVariables, TArray<FProperty*>& OutUnusedVariables);

	/**
	 * Finds a member variable Guid using the variable's name
	 *
	 * @param InBlueprint		Blueprint to search for the local variable
	 * @param InVariableGuid	Local variable's name to search for
	 * @return					The Guid associated with the local variable
	 */
	static UNREALED_API FGuid FindMemberVariableGuidByName(UBlueprint* InBlueprint, const FName InVariableName);

	/**
	 * Finds a member variable name using the variable's Guid
	 *
	 * @param InBlueprint		Blueprint to search for the local variable
	 * @param InVariableGuid	Guid to identify the local variable with
	 * @return					Local variable's name
	 */
	static UNREALED_API FName FindMemberVariableNameByGuid(UBlueprint* InBlueprint, const FGuid& InVariableGuid);

	/**
	 * Removes the variable nodes associated with the specified var name
	 *
	 * @param	Blueprint			The blueprint you want variable nodes removed from.
	 * @param	VarName				Name of the variable to be removed.
	 * @param	bForSelfOnly		True if you only want to delete variable nodes that represent ones owned by this blueprint,
	 * @param	LocalGraphScope		Local scope graph of variables
	 *								false if you just want everything with the specified name removed (variables from other classes too).
	 */
	static UNREALED_API void RemoveVariableNodes(UBlueprint* Blueprint, const FName VarName, bool const bForSelfOnly = true, UEdGraph* LocalGraphScope = nullptr);

	/**Rename a member variable*/
	static UNREALED_API void RenameMemberVariable(UBlueprint* Blueprint, const FName OldName, const FName NewName);

	/** Rename a member variable created by a SCS entry */
	static UNREALED_API void RenameComponentMemberVariable(UBlueprint* Blueprint, USCS_Node* Node, const FName NewName);
	
	/** Changes the type of a member variable */
	static UNREALED_API void ChangeMemberVariableType(UBlueprint* Blueprint, const FName VariableName, const FEdGraphPinType& NewPinType);

	/**
	 * Finds the scope's associated graph for local variables (or any passed UFunction)
	 *
	 * @param	InBlueprint			The Blueprint the local variable can be found in
	 * @param	InScope				Local variable's scope
	 */
	static UNREALED_API UEdGraph* FindScopeGraph(const UBlueprint* InBlueprint, const UStruct* InScope);

	/**
	 * Adds a local variable to the function graph.  It cannot mask a member variable or a variable in any superclass.
	 *
	 * @param	NewVarName	Name of the new variable.
	 * @param	NewVarType	Type of the new variable.
	 * @param	DefaultValue	Default value stored as string
	 *
	 * @return	true if it succeeds, false if it fails.
	 */
	static UNREALED_API bool AddLocalVariable(UBlueprint* Blueprint, UEdGraph* InTargetGraph, const FName InNewVarName, const FEdGraphPinType& InNewVarType, const FString& DefaultValue = FString());

	/**
	 * Removes a local variable if it was declared in this blueprint and not in a base class.
	 *
	 * @param	InBlueprint			The Blueprint the local variable can be found in
	 * @param	InScope				Local variable's scope
	 * @param	InVarName			Name of the variable to be removed.
	 */
	static UNREALED_API void RemoveLocalVariable(UBlueprint* InBlueprint, const UStruct* InScope, const FName InVarName);

	/**
	 * Returns a local variable
	 *
	 * @param InBlueprint		Blueprint to search for the local variable
	 * @param InScopeGraph		Local variable's graph
	 * @param InVariableName	Name of the variable to search for
	 * @param OutFunctionEntry	Optional output parameter. If not null, the found function entry is returned.
	 * @return					The local variable description
	 */
	static UNREALED_API FBPVariableDescription* FindLocalVariable(const UBlueprint* InBlueprint, const UEdGraph* InScopeGraph, const FName InVariableName, class UK2Node_FunctionEntry** OutFunctionEntry = nullptr);

	/**
	 * Returns a local variable
	 *
	 * @param InBlueprint		Blueprint to search for the local variable
	 * @param InScope			Local variable's scope
	 * @param InVariableName	Name of the variable to search for
	 * @param OutFunctionEntry	Optional output parameter. If not null, the found function entry is returned.
	 * @return					The local variable description
	 */
	static UNREALED_API FBPVariableDescription* FindLocalVariable(const UBlueprint* InBlueprint, const UStruct* InScope, const FName InVariableName, class UK2Node_FunctionEntry** OutFunctionEntry = nullptr);

	/**
	 * Finds a local variable name using the variable's Guid
	 *
	 * @param InBlueprint		Blueprint to search for the local variable
	 * @param InVariableGuid	Guid to identify the local variable with
	 * @return					Local variable's name
	 */
	static UNREALED_API FName FindLocalVariableNameByGuid(const UBlueprint* InBlueprint, const FGuid& InVariableGuid);

	/**
	 * Finds a local variable Guid using the variable's name
	 *
	 * @param InBlueprint		Blueprint to search for the local variable
	 * @param InScope			Local variable's scope
	 * @param InVariableGuid	Local variable's name to search for
	 * @return					The Guid associated with the local variable
	 */
	static UNREALED_API FGuid FindLocalVariableGuidByName(const UBlueprint* InBlueprint, const UStruct* InScope, const FName InVariableName);

	/**
	 * Finds a local variable Guid using the variable's name
	 *
	 * @param InBlueprint		Blueprint to search for the local variable
	 * @param InScopeGraph		Local variable's graph
	 * @param InVariableGuid	Local variable's name to search for
	 * @return					The Guid associated with the local variable
	 */
	static UNREALED_API FGuid FindLocalVariableGuidByName(const UBlueprint* InBlueprint, const UEdGraph* InScopeGraph, const FName InVariableName);

	/**
	 * Rename a local variable
	 *
	 * @param InBlueprint		Blueprint to search for the local variable
	 * @param InScope			Local variable's scope
	 * @param InOldName			The name of the local variable to change
	 * @param InNewName			The new name of the local variable
	 */
	static UNREALED_API void RenameLocalVariable(UBlueprint* InBlueprint, const UStruct* InScope, const FName InOldName, const FName InNewName);

	/**
	 * Changes the type of a local variable
	 *
	 * @param InBlueprint		Blueprint to search for the local variable
	 * @param InScope			Local variable's scope
	 * @param InVariableName	Name of the local variable to change the type of
	 * @param InNewPinType		The pin type to change the local variable type to
	 */
	static UNREALED_API void ChangeLocalVariableType(UBlueprint* InBlueprint, const UStruct* InScope, const FName InVariableName, const FEdGraphPinType& InNewPinType);

	/** Replaces all variable references in the specified blueprint */
	static UNREALED_API void ReplaceVariableReferences(UBlueprint* Blueprint, const FName OldName, const FName NewName);

	/** Replaces all variable references in the specified blueprint */
	static UNREALED_API void ReplaceVariableReferences(UBlueprint* Blueprint, const FProperty* OldVariable, const FProperty* NewVariable);

	/** Replaces all function references in the specified blueprint */
	static UNREALED_API void ReplaceFunctionReferences(UBlueprint* Blueprint, const FName OldName, const FName NewName);

	/** Check blueprint variable metadata keys/values for validity and make adjustments if needed */
	static UNREALED_API void FixupVariableDescription(UBlueprint* Blueprint, FBPVariableDescription& VarDesc);

	/**
	  * Validate child blueprint component member variables, member variables, and timelines, and function graphs against the given variable name
	  *
	  * @param	InBlueprint					Target blueprint.
	  * @param	InVariableName				Variable or function name to validate child blueprints against.
	  * @param	CustomValidationCallback	Optional callback that allows for doing any post-validation tasks.
	  */
	static UNREALED_API void ValidateBlueprintChildVariables(UBlueprint* InBlueprint, const FName InVariableName,
		TFunction<void(UBlueprint* InChildBP, const FName InVariableName, bool bValidatedVariable)> PostValidationCallback = TFunction<void(UBlueprint*, FName, bool)>());

	/**
	 * Gets AssetData for all child classes of a given blueprint
	 * 
	 * @param InBlueprint    Taget Blueprint
	 * @param OutChildren    AssetData representing the child blueprints
	 * @param bInRecursive   if true, will return classes derived from child classes as well
	 * @return Number of child blueprints found
	 */
	static UNREALED_API int32 GetChildrenOfBlueprint(UBlueprint* InBlueprint, TArray<FAssetData>& OutChildren, bool bInRecursive = true);

	/** Marks all children of a blueprint as modified */
	static UNREALED_API void MarkBlueprintChildrenAsModified(UBlueprint* InBlueprint);

	/** Rename a Timeline. If bRenameNodes is true, will also rename any timeline nodes associated with this timeline */
	static UNREALED_API bool RenameTimeline (UBlueprint* Blueprint, const FName OldVarName, const FName NewVarName);

	/**
	 * Sets the Blueprint edit-only flag on the variable with the specified name
	 *
	 * @param	VarName				Name of the var to set the flag on
	 * @param	bNewBlueprintOnly	The new value to set the bitflag to
	 */
	static UNREALED_API void SetBlueprintOnlyEditableFlag(UBlueprint* Blueprint, const FName& VarName, const bool bNewBlueprintOnly);

	/**
	 * Sets the Blueprint read-only flag on the variable with the specified name
	 *
	 * @param	VarName				Name of the var to set the flag on
	 * @param	bVariableReadOnly	The new value to set the bitflag to
	 */
	static UNREALED_API void SetBlueprintPropertyReadOnlyFlag(UBlueprint* Blueprint, const FName& VarName, const bool bVariableReadOnly);

	/**
	 * Sets the Interp flag on the variable with the specified name to make available to sequencer
	 *
	 * @param	VarName				Name of the var to set the flag on
	 * @param	bInterp	true to make variable available to sequencer, false otherwise
	 */
	static UNREALED_API void SetInterpFlag(UBlueprint* Blueprint, const FName& VarName, const bool bInterp);

	/**
	 * Sets the Transient flag on the variable with the specified name
	 *
	 * @param	InVarName				Name of the var to set the flag on
	 * @param	bInIsTransient			The new value to set the bitflag to
	 */
	static UNREALED_API void SetVariableTransientFlag(UBlueprint* InBlueprint, const FName& InVarName, const bool bInIsTransient);

	/**
	 * Sets the Save Game flag on the variable with the specified name
	 *
	 * @param	InVarName				Name of the var to set the flag on
	 * @param	bInIsSaveGame			The new value to set the bitflag to
	 */
	static UNREALED_API void SetVariableSaveGameFlag(UBlueprint* InBlueprint, const FName& InVarName, const bool bInIsSaveGame);

	/**
	 * Sets the Advanced Display flag on the variable with the specified name
	 *
	 * @param	InVarName				Name of the var to set the flag on
	 * @param	bInIsAdvancedDisplay	The new value to set the bitflag to
	 */
	static UNREALED_API void SetVariableAdvancedDisplayFlag(UBlueprint* InBlueprint, const FName& InVarName, const bool bInIsAdvancedDisplay);

	/**
	 * Sets the Deprecated flag on the variable with the specified name
	 *
	 * @param	InVarName				Name of the var to set the flag on
	 * @param	bInIsDeprecated			The new value to set the bitflag to
	 */
	static UNREALED_API void SetVariableDeprecatedFlag(UBlueprint* InBlueprint, const FName& InVarName, const bool bInIsDeprecated);

	/** Sets a metadata key/value on the specified variable
	 *
	 * @param Blueprint				The Blueprint to find the variable in
	 * @param VarName				Name of the variable
	 * @param InLocalVarScope		Local variable's scope, if looking to modify a local variable
	 * @param MetaDataKey			Key name for the metadata to change
	 * @param MetaDataValue			Value to change the metadata to
	 */
	static UNREALED_API void SetBlueprintVariableMetaData(UBlueprint* Blueprint, const FName& VarName, const UStruct* InLocalVarScope, const FName& MetaDataKey, const FString& MetaDataValue);

	/** Get a metadata key/value on the specified variable, or timeline if it exists, returning false if it does not exist
	 *
	 * @param Blueprint				The Blueprint to find the variable in
	 * @param VarName				Name of the variable
	 * @param InLocalVarScope		Local variable's scope, if looking to modify a local variable
	 * @param MetaDataKey			Key name for the metadata to change
	 * @param OutMetaDataValue		Value of the metadata
	 * @return						TRUE if finding the metadata was successful
	 */
	static UNREALED_API bool GetBlueprintVariableMetaData(const UBlueprint* Blueprint, const FName& VarName, const UStruct* InLocalVarScope, const FName& MetaDataKey, FString& OutMetaDataValue);

	/** Clear metadata key on specified variable, or timeline
	 * @param Blueprint				The Blueprint to find the variable in
	 * @param VarName				Name of the variable
	 * @param InLocalVarScope		Local variable's scope, if looking to modify a local variable
	 * @param MetaDataKey			Key name for the metadata to change
	 */
	static UNREALED_API void RemoveBlueprintVariableMetaData(UBlueprint* Blueprint, const FName& VarName, const UStruct* InLocalVarScope, const FName& MetaDataKey);

	/**
	 * Sets the custom category on the variable with the specified name.
	 * @note: Will not change the category for variables defined via native classes.
	 *
	 * @param	VarName				Name of the variable
	 * @param	InLocalVarScope		Local variable's scope, if looking to modify a local variable
	 * @param	VarCategory			The new value of the custom category for the variable
	 * @param	bDontRecompile		If true, the blueprint will not be marked as modified, and will not be recompiled.  
	 */
	static UNREALED_API void SetBlueprintVariableCategory(UBlueprint* Blueprint, const FName& VarName, const UStruct* InLocalVarScope, const FText& NewCategory, bool bDontRecompile=false);


	/**
	 * Sets the custom category on the function or macro
	 * @note: Will not change the category for functions defined via native classes.
	 *
	 * @param	Graph				Graph associated with the function or macro
	 * @param	NewCategory			The new value of the custom category for the function
	 * @param	bDontRecompile		If true, the blueprint will not be marked as modified, and will not be recompiled.  
	 */
	static UNREALED_API void SetBlueprintFunctionOrMacroCategory(UEdGraph* Graph, const FText& NewCategory, bool bDontRecompile=false);

	/** 
	 * Helper function to grab the root node from an anim graph. 
	 * Asserts if anything other than 1 node is found in an anim graph.
	 * @param	InGraph		The graph to check
	 * @return the one and only root, if this is an anim graph, or null otherwise
	 */
	static UNREALED_API UAnimGraphNode_Root* GetAnimGraphRoot(UEdGraph* InGraph);

	/**
	 * Sets the layer group on the anim graph
	 * @note: Will not change the category for functions defined via native classes.
	 *
	 * @param	InGraph				Graph associated with the layer
	 * @param	InGroupName			The new value of the group for the layer
	 */
	static UNREALED_API void SetAnimationGraphLayerGroup(UEdGraph* InGraph, const FText& InGroupName);

	/** Finds the index of the specified graph (function or macro) in the parent (if it is not reorderable, then we will return INDEX_NONE) */
	static UNREALED_API int32 FindIndexOfGraphInParent(UEdGraph* Graph);

	/** Reorders the specified graph (function or macro) to be at the new index in the parent (moving whatever was there to be after it), assuming it is reorderable and that is a valid index */
	static UNREALED_API bool MoveGraphBeforeOtherGraph(UEdGraph* Graph, int32 NewIndex, bool bDontRecompile);

	/**
	 * Gets the custom category on the variable with the specified name.
	 *
	 * @param	VarName				Name of the variable
	 * @param	InLocalVarScope		Local variable's scope, if looking to modify a local variable
	 * @return						The custom category (None indicates the name will be the same as the blueprint)
	 */
	static UNREALED_API FText GetBlueprintVariableCategory(UBlueprint* Blueprint, const FName& VarName, const UStruct* InLocalVarScope);

	/** Gets pointer to PropertyFlags of variable */
	static UNREALED_API uint64* GetBlueprintVariablePropertyFlags(UBlueprint* Blueprint, const FName& VarName);

	/** Gets the variable linked to a RepNotify function, returns nullptr if not found */
	static UNREALED_API FBPVariableDescription* GetVariableFromOnRepFunction(UBlueprint* Blueprint, FName FuncName);

	/** Get RepNotify function name of variable */
	static UNREALED_API FName GetBlueprintVariableRepNotifyFunc(UBlueprint* Blueprint, const FName& VarName);

	/** Set RepNotify function of variable */
	static UNREALED_API void SetBlueprintVariableRepNotifyFunc(UBlueprint* Blueprint, const FName& VarName, const FName& RepNotifyFunc);

	/** Returns TRUE if the variable was created by the Blueprint */
	static UNREALED_API bool IsVariableCreatedByBlueprint(UBlueprint* InBlueprint, FProperty* InVariableProperty);

	/**
	 * Find the index of a variable first declared in this blueprint. Returns INDEX_NONE if not found.
	 *
	 * @param	InName	Name of the variable to find.
	 *
	 * @return	The index of the variable, or INDEX_NONE if it wasn't introduced in this blueprint.
	 */
	static UNREALED_API int32 FindNewVariableIndex(const UBlueprint* Blueprint, const FName& InName);

	/**
	 * Find the index of a variable first declared in this blueprint or its parents. Returns INDEX_NONE if not found.
	 * 
	 * @param   InBlueprint         Blueprint to begin search in (will search parents as well)
	 * @param	InName	            Name of the variable to find.
	 * @param   OutFoundBlueprint   Blueprint where the variable was eventually found
	 *
	 * @return	The index of the variable, or INDEX_NONE if it wasn't introduced in this blueprint.
	 */
	static UNREALED_API int32 FindNewVariableIndexAndBlueprint(UBlueprint* InBlueprint, FName InName, UBlueprint*& OutFoundBlueprint);

	/**
	 * Find the index of a local variable declared in this blueprint. Returns INDEX_NONE if not found.
	 *
	 * @param	VariableScope	Struct of owning function.
	 *
	 * @param	InVariableName	Name of the variable to find.
	 *
	 * @return	The index of the variable, or INDEX_NONE if it wasn't introduced in this blueprint.
	 */
	static UNREALED_API int32 FindLocalVariableIndex(const UBlueprint* Blueprint, UStruct* VariableScope, const FName& InVariableName);

	/** Change the order of variables in the Blueprint by placing a variable before the target variable. */
	static UNREALED_API bool MoveVariableBeforeVariable(UBlueprint* Blueprint, UStruct* VariableScope, FName VarNameToMove, FName TargetVarName, bool bDontRecompile);

	/** Change the order of variables in the Blueprint by placing a variable after the target variable. */
	static UNREALED_API bool MoveVariableAfterVariable(UBlueprint* Blueprint, UStruct* VariableScope, FName VarNameToMove, FName TargetVarName, bool bDontRecompile);

	/**
	 * Find the index of a timeline first declared in this blueprint. Returns INDEX_NONE if not found.
	 *
	 * @param	InName	Name of the variable to find.
	 *
	 * @return	The index of the variable, or INDEX_NONE if it wasn't introduced in this blueprint.
	 */
	static UNREALED_API int32 FindTimelineIndex(const UBlueprint* Blueprint, const FName& InName);

	/** 
	 * Gets a list of SCS node variable names for the given blueprint.
	 *
	 * @param [in,out]	VariableNames		The list of variable names for the SCS node array.
	 */
	static UNREALED_API void GetSCSVariableNameList(const UBlueprint* Blueprint, TSet<FName>& VariableNames);

	/**
	 * Gets a list of SCS node variable names for the given BPGC.
	 *
	 * @param [in,out]	VariableNames		The list of variable names for the SCS node array.
	 */
	static UNREALED_API void GetSCSVariableNameList(const UBlueprintGeneratedClass* BPGC, TSet<FName>& VariableNames);

	/**
	 * Gets a list of SCS node variable names for the given SCS.
	 *
	 * @param [in,out]	VariableNames		The list of variable names for the SCS node array.
	 */
	static UNREALED_API void GetSCSVariableNameList(const USimpleConstructionScript* SCS, TSet<FName>& VariableNames);

	/** 
	 * Gets a list of function names in blueprints that implement the interface defined by the given blueprint.
	 *
	 * @param [in,out]	Blueprint			The interface blueprint to check.
	 * @param [in,out]	VariableNames		The list of function names for implementing blueprints.
	 */
	static UNREALED_API void GetImplementingBlueprintsFunctionNameList(const UBlueprint* Blueprint, TSet<FName>& FunctionNames);

	/**
	 * Finds an SCSNode by variable name
	 *
	 * @param	InName	Name of the variable to find.
	 * @return	The SCS Node that corresponds to the variable name or null if one is not found
	 */
	static UNREALED_API USCS_Node* FindSCS_Node(const UBlueprint* Blueprint, const FName InName);

	/** Returns whether or not the specified member var is a component */
	static UNREALED_API bool IsVariableComponent(const FBPVariableDescription& Variable);

	/** Indicates if the variable is used on any graphs in this Blueprint */
	static UNREALED_API bool IsVariableUsed(const UBlueprint* VariableBlueprint, const FName& VariableName, const UEdGraph* LocalGraphScope = nullptr);

	/** Indicates if the function is used on any graphs in this Blueprint */
	static UNREALED_API bool IsFunctionUsed(const UBlueprint* FunctionBlueprint, const FName& FunctionName, const UEdGraph* LocalGraphScope = nullptr);

	/** 
	 * Copies the value from the passed in string into a property. ContainerMem points to the Struct or Class containing Property 
	 * NOTE: This function does not work correctly with static arrays.
	 */
	static UNREALED_API bool PropertyValueFromString(const FProperty* Property, const FString& StrValue, uint8* Container, UObject* OwningObject = nullptr, int32 PortFlags = PPF_None);

	/** 
	 * Copies the value from the passed in string into a property. DirectValue is the raw memory address of the property value 
	 * NOTE: This function does not work correctly with static arrays.
	 */
	static UNREALED_API bool PropertyValueFromString_Direct(const FProperty* Property, const FString& StrValue, uint8* DirectValue, UObject* OwningObject = nullptr, int32 PortFlags = PPF_None);

	/** 
	 * Copies the value from a property into the string OutForm. ContainerMem points to the Struct or Class containing Property 
	 * NOTE: This function does not work correctly with static arrays.
	 */
	static UNREALED_API bool PropertyValueToString(const FProperty* Property, const uint8* Container, FString& OutForm, UObject* OwningObject = nullptr, int32 PortFlags = PPF_None);

	/** 
	 * Copies the value from a property into the string OutForm. DirectValue is the raw memory address of the property value 
	 * NOTE: This function does not work correctly with static arrays.
	 */
	static UNREALED_API bool PropertyValueToString_Direct(const FProperty* Property, const uint8* DirectValue, FString& OutForm, UObject* OwningObject = nullptr, int32 PortFlags = PPF_None);

	/** Call PostEditChange() on all Actors based on the given Blueprint */
	static UNREALED_API void PostEditChangeBlueprintActors(UBlueprint* Blueprint, bool bComponentEditChange = false);

	/** @return whether a property is private or not (whether native or via BP metadata) */
	static UNREALED_API bool IsPropertyPrivate(const FProperty* Property);
	
	/** Enumeration of whether a property is writable or if not, why. */
	enum class EPropertyWritableState : uint8
	{
		Writable,
		Private,
		NotBlueprintVisible,
		BlueprintReadOnly
	};

	/** Returns an enumeration indicating if the property can be written to by the given Blueprint */
	static UNREALED_API EPropertyWritableState IsPropertyWritableInBlueprint(const UBlueprint* Blueprint, const FProperty* Property);

	/** Enumeration of whether a property is readable or if not, why. */
	enum class EPropertyReadableState : uint8
	{
		Readable,
		Private,
		NotBlueprintVisible
	};

	/** Returns an enumeration indicating if the property can be read by the given Blueprint */
	static UNREALED_API EPropertyReadableState IsPropertyReadableInBlueprint(const UBlueprint* Blueprint, const FProperty* Property);

	/** Ensures that the CDO root component reference is valid for Actor-based Blueprints */
	static UNREALED_API void UpdateRootComponentReference(UBlueprint* Blueprint);

	/** Determines if this property is associated with a component that would be displayed in the SCS editor */
	static UNREALED_API bool IsSCSComponentProperty(FObjectProperty* MemberProperty);

	/** Attempts to match up the FComponentKey with a ComponentTemplate from the Blueprint's UCS. Will fall back to try matching the given template name if the key cannot be used. */
	static UNREALED_API UActorComponent* FindUCSComponentTemplate(const FComponentKey& ComponentKey, const FName& TemplateName);

	/** Takes the Blueprint's NativizedFlag property and applies it to the authoritative config (does the same for flagged dependencies) */
	UE_DEPRECATED(5.0, "Blueprint Nativization has been removed as a supported feature. This API will eventually be removed.")
	static bool PropagateNativizationSetting(UBlueprint* Blueprint) { return false; }

	/** Retrieves all dependencies that need to be nativized for this to work as a nativized Blueprint */
	UE_DEPRECATED(5.0, "Blueprint Nativization has been removed as a supported feature. This API will eventually be removed.")
	static void FindNativizationDependencies(UBlueprint* Blueprint, TArray<UClass*>& NativizeDependenciesOut) {}

	/** Returns whether or not the given Blueprint should be nativized implicitly, regardless of whether or not the user has explicitly enabled it */
	UE_DEPRECATED(5.0, "Blueprint Nativization has been removed as a supported feature. This API will eventually be removed.")
	static bool ShouldNativizeImplicitly(const UBlueprint* Blueprint) { return false; }

	//////////////////////////////////////////////////////////////////////////
	// Interface

	/** 
	 * Find the interface Guid for a graph if it exists.
	 * 
	 * @param	GraphName		The graph name to find a GUID for.
	 * @param	InterfaceClass	The interface's generated class.
	 */
	static UNREALED_API FGuid FindInterfaceGraphGuid(const FName& GraphName, const UClass* InterfaceClass);

	/** 
	 * Find the interface Guid for a function if it exists.
	 * 
	 * @param	Function		The function to find a graph for.
	 * @param	InterfaceClass	The interface's generated class.
	 */
	static UNREALED_API FGuid FindInterfaceFunctionGuid(const UFunction* Function, const UClass* InterfaceClass);

	/** Add a new interface, and member function graphs to the blueprint */
	UE_DEPRECATED(5.1, "Short class names are no longer supported. Use a version of this function that takes FTopLevelAssetPath.")
	static UNREALED_API bool ImplementNewInterface(UBlueprint* Blueprint, const FName& InterfaceClassName);

	/** Add a new interface, and member function graphs to the blueprint */
	static UNREALED_API bool ImplementNewInterface(UBlueprint* Blueprint, FTopLevelAssetPath InterfaceClassPathName);

	/** Remove an implemented interface, and its associated member function graphs.  If bPreserveFunctions is true, then the interface will move its functions to be normal implemented blueprint functions */
	UE_DEPRECATED(5.1, "Short class names are no longer supported. Use a version of this function that takes FTopLevelAssetPath.")
	static UNREALED_API void RemoveInterface(UBlueprint* Blueprint, const FName& InterfaceClassName, bool bPreserveFunctions = false);
	
	/** Remove an implemented interface, and its associated member function graphs.  If bPreserveFunctions is true, then the interface will move its functions to be normal implemented blueprint functions */
	static UNREALED_API void RemoveInterface(UBlueprint* Blueprint, FTopLevelAssetPath InterfaceClassPathName, bool bPreserveFunctions = false);

	/**
	* Attempt to remove a function from an interfaces list of function graphs.
	* Note that this will NOT remove interface events (i.e. functions with no outputs)
	* 
	* @return	True if the function was removed from the blueprint
	*/
	static UNREALED_API bool RemoveInterfaceFunction(UBlueprint* Blueprint, FBPInterfaceDescription& Interface, UFunction* Function, bool bPreserveFunction);

	/**
	* Promotes a Graph from being an Interface Override to a full member function
	*
	* @param InBlueprint			Blueprint the graph is contained within
	* @param InInterfaceGraph		The graph to promote
	*/
	static UNREALED_API void PromoteGraphFromInterfaceOverride(UBlueprint* InBlueprint, UEdGraph* InInterfaceGraph);

	/** Gets the graphs currently in the blueprint associated with the specified interface */
	UE_DEPRECATED(5.1, "Short class names are no longer supported. Use a version of this function that takes FTopLevelAssetPath.")
	static UNREALED_API void GetInterfaceGraphs(UBlueprint* Blueprint, const FName& InterfaceClassName, TArray<UEdGraph*>& ChildGraphs);

	/** Gets the graphs currently in the blueprint associated with the specified interface */
	static UNREALED_API void GetInterfaceGraphs(UBlueprint* Blueprint, FTopLevelAssetPath InterfaceClassPathName, TArray<UEdGraph*>& ChildGraphs);

	/**
	* Checks if the given function is a part of an interface on this blueprint
	* 
	* @param Blueprint		The blueprint to consider
	* @param Function		Function to check if it is an interface or not
	* @return	True if the given function is implemented as part of an interface
	*/
	static UNREALED_API bool IsInterfaceFunction(UBlueprint* Blueprint, UFunction* Function);

	/**
	* Get the corresponding UFunction pointer to the name given on the blueprint.
	* Searches the given blueprints implemented interfaces first, and then looks 
	* in the parent. 
	* 
	* @param Blueprint		The blueprint to consider
	* @param FuncName		The name of the function to look for
	*
	* @return	Corresponding UFunction pointer to the name given; Nullptr if not 
	*			part of any interfaces
	*/
	static UNREALED_API UFunction* GetInterfaceFunction(UBlueprint* Blueprint, const FName FuncName);

	/** Makes sure that all graphs for all interfaces we implement exist, and add if not */
	static UNREALED_API void ConformImplementedInterfaces(UBlueprint* Blueprint);

	/** Makes sure that all delegate graphs have a corresponding variable declaration, removing the graph if not */
	static UNREALED_API void ConformDelegateSignatureGraphs(UBlueprint* Blueprint);

	/** Makes sure that all function graphs are flagged as bAllowDeletion=true, except for construction script and animgraph: */
	static UNREALED_API void ConformAllowDeletionFlag(UBlueprint* Blueprint);
	
	/** Makes sure that all NULL graph references are removed from SubGraphs and top-level graph arrays */
	static UNREALED_API void PurgeNullGraphs(UBlueprint* Blueprint);

	/** Removes all invalid structure members from the blueprint - e.g. because they were deleted */
	static UNREALED_API void RemoveInvalidStructureMemberVariableFromBlueprint(UBlueprint* Blueprint);

	/** Handle old AnimBlueprints (state machines in the wrong position, transition graphs with the wrong schema, etc...) */
	static UNREALED_API void UpdateOutOfDateAnimBlueprints(UBlueprint* Blueprint);

	/** Handle fixing up composite nodes within the blueprint*/
	static UNREALED_API void UpdateOutOfDateCompositeNodes(UBlueprint* Blueprint);

	/** Handle fixing up composite nodes within the specified Graph of Blueprint, and correctly setting the Outer */
	static UNREALED_API void UpdateOutOfDateCompositeWithOuter(UBlueprint* Blueprint, UEdGraph* Outer );

	/** Handle stale components and ensure correct flags are set */
	static UNREALED_API void UpdateComponentTemplates(UBlueprint* Blueprint);

	/** Handle stale transactional flags on blueprints */
	static UNREALED_API void UpdateTransactionalFlags(UBlueprint* Blueprint);

	/** Handle stale pin watches */
	static UNREALED_API void UpdateStalePinWatches( UBlueprint* Blueprint );

	/** Updates the cosmetic information cache for macros */
	static UNREALED_API void ClearMacroCosmeticInfoCache(UBlueprint* Blueprint);

	/** Returns the cosmetic information for the specified macro graph, caching it if necessary */
	static UNREALED_API FBlueprintMacroCosmeticInfo GetCosmeticInfoForMacro(UEdGraph* MacroGraph);

	/** Return the first function from implemented interface with given name */
	static UNREALED_API UFunction* FindFunctionInImplementedInterfaces(const UBlueprint* Blueprint, const FName& FunctionName, bool* bOutInvalidInterface = nullptr, bool bGetAllInterfaces = false);

	/** 
	 * Build a list of all interface classes either implemented by this blueprint or through inheritance
	 * @param		Blueprint				The blueprint to find interfaces for
	 * @param		bGetAllInterfaces		If true, get all the implemented and inherited. False, just gets the interfaces implemented directly.
	 * @param [out]	ImplementedInterfaces	The interfaces implemented by this blueprint
	 */
	static UNREALED_API void FindImplementedInterfaces(const UBlueprint* Blueprint, bool bGetAllInterfaces, TArray<UClass*>& ImplementedInterfaces);

	/**
	 * Returns true if the interfaces is implemented. It can be implemented directly or inherited.
	 * @param		Blueprint				The blueprint to check interfaces for
	 * @param		bIncludeInherited		If true, check in all the implemented and inherited interfaces. False, check in the interfaces implemented directly.
	 * @param		SomeInterface			The interface to check.
	*/
	static UNREALED_API bool ImplementsInterface(const UBlueprint* Blueprint, bool bIncludeInherited, UClass* SomeInterface);

	/**
	 * Finds a unique name with a base of the passed in string, appending numbers as needed
	 *
	 * @param InBlueprint		The blueprint the kismet object's name needs to be unique in
	 * @param InBaseName		The base name to use
	 * @param InScope			Scope, if any, of the unique kismet name to generate, used for locals
	 *
	 * @return					A unique name that will not conflict in the Blueprint
	 */
	static UNREALED_API FName FindUniqueKismetName(const UBlueprint* InBlueprint, const FString& InBaseName, UStruct* InScope = NULL);
	
	/**
	 * Cleanses a name of invalid characters and replaces them with '_'.
	 * See UE_BLUEPRINT_INVALID_NAME_CHARACTERS for invalid characters.
	 *
	 * @param InBaseName The base name to assign
	 */
	static UNREALED_API void ReplaceInvalidBlueprintNameCharacters(FString& InBaseName);
	
	/** Util version of ReplaceInvalidBlueprintNameCharacters that performs the operation inline. */
	static FString ReplaceInvalidBlueprintNameCharactersInline(FString InBaseName)
	{
		ReplaceInvalidBlueprintNameCharacters(InBaseName);
		return InBaseName;
	}

	/** Finds a unique and valid name for a custom event */
	static UNREALED_API FName FindUniqueCustomEventName(const UBlueprint* Blueprint);

	//////////////////////////////////////////////////////////////////////////
	// Timeline

	/** Finds a name for a timeline that is not already in use */
	static UNREALED_API FName FindUniqueTimelineName(const UBlueprint* Blueprint);

	/** Add a new timeline with the supplied name to the blueprint */
	static UNREALED_API class UTimelineTemplate* AddNewTimeline(UBlueprint* Blueprint, const FName& TimelineVarName);

	/** Remove the timeline from the blueprint 
	 * @note Just removes the timeline from the associated timelist in the Blueprint. Does not remove the node graph
	 * object representing the timeline itself.
	 * @param Timeline			The timeline to remove
	 * @param bDontRecompile	If true, the blueprint will not be marked as modified, and will not be recompiled.  
	 */
	static UNREALED_API void RemoveTimeline(UBlueprint* Blueprint, class UTimelineTemplate* Timeline, bool bDontRecompile=false);

	/** Find the node that owns the supplied timeline template */
	static UNREALED_API class UK2Node_Timeline* FindNodeForTimeline(UBlueprint* Blueprint, UTimelineTemplate* Timeline);

	//////////////////////////////////////////////////////////////////////////
	// LevelScriptBlueprint

	/** Find how many nodes reference the supplied actor */
	static UNREALED_API bool FindReferencesToActorFromLevelScript(ULevelScriptBlueprint* LevelScriptBlueprint, AActor* InActor, TArray<UK2Node*>& ReferencedToActors);

	/** Replace all references of the old actor with the new actor */
	static UNREALED_API void ReplaceAllActorRefrences(ULevelScriptBlueprint* InLevelScriptBlueprint, AActor* InOldActor, AActor* InNewActor);

	/** Function to call modify() on all graph nodes which reference this actor */
	static UNREALED_API void  ModifyActorReferencedGraphNodes(ULevelScriptBlueprint* LevelScriptBlueprint, const AActor* InActor);

	/**
	 * Called after a level script blueprint is changed and nodes should be refreshed for it's new level script actor
	 *
	 * @param	LevelScriptActor	The newly-created level script actor that should be (re-)bound to
	 * @param	ScriptBlueprint		The level scripting blueprint that contains the bound events to try and bind delegates to this actor for
	 */
	static UNREALED_API void FixLevelScriptActorBindings(ALevelScriptActor* LevelScriptActor, const class ULevelScriptBlueprint* ScriptBlueprint);

	/**
	 * Find how many actors reference the supplied actor
	 *
	 * @param InActor The Actor to count references to.
	 * @param InClassesToIgnore An array of class types to ignore, even if there is an instance of one that references the InActor
	 * @param OutReferencingActors An array of actors found that reference the specified InActor
	 */
	static UNREALED_API void FindActorsThatReferenceActor( AActor* InActor, TArray<UClass*>& InClassesToIgnore, TArray<AActor*>& OutReferencingActors );

	/**
	 * Go through the world and build a map of all actors that are referenced by other actors.
	 * @param InWorld The world to scan for Actors.
	 * @param InClassesToIgnore  An array of class types to ignore, even if there is an instance of one that references another Actor
	 * @param OutReferencingActors A map of Actors that are referenced by a list of other Actors.
	*/
	static UNREALED_API void GetActorReferenceMap(UWorld* InWorld, TArray<UClass*>& InClassesToIgnore, TMap<AActor*, TArray<AActor*> >& OutReferencingActors);

	//////////////////////////////////////////////////////////////////////////
	// Diagnostics

	// Diagnostic use only: Lists all of the objects have a direct outer of Package
	static UNREALED_API void ListPackageContents(UPackage* Package, FOutputDevice& Ar);

	// Diagnostic exec commands
	static UNREALED_API bool KismetDiagnosticExec(const TCHAR* Stream, FOutputDevice& Ar);

	/**
	 * Searches the world for any blueprints that are open and do not have a debug instances set and sets one if possible.
	 * It will favor a selected instance over a non selected one
	 */
	static UNREALED_API void FindAndSetDebuggableBlueprintInstances();

	/**
	 * Records node create events for analytics
	 */
	static UNREALED_API void AnalyticsTrackNewNode( UEdGraphNode* NewNode );

	/**
	 * Generates a unique graph name for the supplied blueprint (guaranteed to not 
	 * cause a naming conflict at the time of the call).
	 * 
	 * @param  InOuter		The blueprint/object you want a unique graph name for.
	 * @param  ProposedName		The name you want to give the graph (the result will be some permutation of this string).
	 * @return A unique name intended for a new graph.
	 */
	static UNREALED_API FName GenerateUniqueGraphName(UObject* const InOuter, FString const& ProposedName);

	/* Checks if the passed in selection set causes cycling on compile
	 *
	 * @param InSelectionSet		The selection set to check for a cycle within
	 * @param InMessageLog			Log to report cycling errors to
	 *
	 * @return						Returns TRUE if the selection does cause cycling
	 */
	static UNREALED_API bool CheckIfSelectionIsCycling(const TSet<UEdGraphNode*>& InSelectionSet, FCompilerResultsLog& InMessageLog);
	
	/**
	 * A utility function intended to aid the construction of a specific blueprint 
	 * palette item. Some items can be renamed, so this method determines if that is 
	 * allowed.
	 * 
	 * @param  ActionIn				The action associated with the palette item you're querying for.
	 * @param  BlueprintEditorIn	The blueprint editor owning the palette item you're querying for.
	 * @return True is the item CANNOT be renamed, false if it can.
	 */
	static UNREALED_API bool IsPaletteActionReadOnly(TSharedPtr<FEdGraphSchemaAction> ActionIn, TSharedPtr<class FBlueprintEditor> const BlueprintEditorIn);

	/**
	 * Finds the entry and result nodes for a function or macro graph
	 *
	 * @param InGraph			The graph to search through
	 * @param OutEntryNode		The found entry node for the graph
	 * @param OutResultNode		The found result node for the graph
	 */
	static UNREALED_API void GetEntryAndResultNodes(const UEdGraph* InGraph, TWeakObjectPtr<class UK2Node_EditablePinBase>& OutEntryNode, TWeakObjectPtr<class UK2Node_EditablePinBase>& OutResultNode);


	/**
	 * Finds the entry node for a function or macro graph
	 *
	 * @param InGraph			The graph to search through
	 * @return		The found entry node for the graph
	 */
	static UNREALED_API class UK2Node_EditablePinBase* GetEntryNode(const UEdGraph* InGraph);

	/**
	 * Returns the function meta data block for the graph entry node.
	 *
	 * @param InGraph			The graph to search through
	 * @return					If valid a pointer to the user declared function meta data structure otherwise nullptr.
	 */
	static UNREALED_API FKismetUserDeclaredFunctionMetadata* GetGraphFunctionMetaData(const UEdGraph* InGraph);

	/**
	 * Modifies the graph entry node that contains the function metadata block, used in metadata transactions.
	 *
	 * @param InGraph			The graph to modify
	 */
	static UNREALED_API void ModifyFunctionMetaData(const UEdGraph* InGraph);

	/**
	 * Returns the description of the graph from the metadata
	 *
	 * @param InGraph			Graph to find the description of
	 * @return					The description of the graph
	 */
	static UNREALED_API FText GetGraphDescription(const UEdGraph* InGraph);

	/** Checks if a graph (or any sub-graphs or referenced graphs) have latent function nodes */
	static UNREALED_API bool CheckIfGraphHasLatentFunctions(UEdGraph* InGraph);

	/**
	 * Creates a function result node or returns the current one if one exists
	 *
	 * @param InFunctionEntryNode		The function entry node to spawn the result node for
	 * @return							Spawned result node
	 */
	static UNREALED_API class UK2Node_FunctionResult* FindOrCreateFunctionResultNode(class UK2Node_EditablePinBase* InFunctionEntryNode);

	/** 
	 * Determine the best icon to represent the given pin.
	 *
	 * @param PinType		The pin get the icon for.
	 * @param returns a brush that best represents the icon (or Kismet.VariableList.TypeIcon if none is available )
	 */
	static UNREALED_API const struct FSlateBrush* GetIconFromPin(const FEdGraphPinType& PinType, bool bIsLarge = false);

	/**
	 * Determine the best secondary icon icon to represent the given pin.
	 */
	static UNREALED_API const struct FSlateBrush* GetSecondaryIconFromPin(const FEdGraphPinType& PinType);

	/**
	 * Returns true if this terminal type can be hashed (native types need GetTypeHash, script types are always hashable).
	 */
	static UNREALED_API bool HasGetTypeHash(const FEdGraphPinType& PinType);

	/**
	 * Returns true if this type of FProperty can be hashed. Matches native constructors of FNumericProperty, etc.
	 */
	static UNREALED_API bool PropertyHasGetTypeHash(const FProperty* PropertyType);

	/**
	 * Returns true if the StructType is native and has a GetTypeHash or is non-native and all of its member types are handled by UScriptStruct::GetStructTypeHash
	 */
	static UNREALED_API bool StructHasGetTypeHash(const UScriptStruct* StructType);

	/**
	 * Generate component instancing data (for cooked builds).
	 *
	 * @param ComponentTemplate		The component template to generate instancing data for.
	 * @param OutData				The generated component instancing data.
	 * @param bUseTemplateArchetype	Whether or not to use the template archetype or the template CDO for delta serialization (default is to use the template CDO).
	 * @return						TRUE if component instancing data was built, FALSE otherwise.
	 */
	static UNREALED_API void BuildComponentInstancingData(UActorComponent* ComponentTemplate, FBlueprintCookedComponentInstancingData& OutData, bool bUseTemplateArchetype = false);

	/**
	 * Callback for when a node has been found
	 */
	using FOnNodeFoundOrUpdated = TFunction<void(UBlueprint*, UK2Node*)>;

	/**
	 * Search the blueprints looking for nodes that contain the given script structs
	 */
	static UNREALED_API void FindScriptStructsInNodes(const TSet<UScriptStruct*>& Structs, FOnNodeFoundOrUpdated InOnNodeFoundOrUpdated);

	/**
	 * Search the blueprints looking for nodes that contain the given enumerations
	 */
	static UNREALED_API void FindEnumsInNodes(const TSet<UEnum*>& UEnums, FOnNodeFoundOrUpdated InOnNodeFoundOrUpdated);

	/**
	 * Search the blueprints looking for nodes that contain the given script structs and replace the references
	 */
	static UNREALED_API void UpdateScriptStructsInNodes(const TMap<UScriptStruct*, UScriptStruct*>& Structs, FOnNodeFoundOrUpdated InOnNodeFoundOrUpdated);

	/**
	 * Search the blueprints looking for nodes that contain the given enumerations and replace the references
	 */
	static UNREALED_API void UpdateEnumsInNodes(const TMap<UEnum*, UEnum*>& Structs, FOnNodeFoundOrUpdated InOnNodeFoundOrUpdated);

	/**
	 * Recombine any nested subpins on this node
	 */
	static UNREALED_API void RecombineNestedSubPins(UK2Node* Node);

protected:
	// Removes all NULL graph references from the SubGraphs array and recurses thru the non-NULL ones
	static UNREALED_API void CleanNullGraphReferencesRecursive(UEdGraph* Graph);

	// Removes all NULL graph references in the specified array
	static UNREALED_API void CleanNullGraphReferencesInArray(UBlueprint* Blueprint, TArray<UEdGraph*>& GraphArray);

	/**
	 * Checks that the actor type matches the blueprint type (or optionally is BASED on the same type. 
	 *
	 * @param InActorObject						The object to check
	 * @param InBlueprint						The blueprint to check against
	 * @param bInDisallowDerivedBlueprints		if true will only allow exact type matches, if false derived types are allowed.
	 */
	static UNREALED_API bool IsObjectADebugCandidate( AActor* InActorObject, UBlueprint* InBlueprint , bool bInDisallowDerivedBlueprints );

	/** Validate child blueprint member variables against the given variable name */
	static UNREALED_API bool ValidateAllMemberVariables(UBlueprint* InBlueprint, UBlueprint* InParentBlueprint, const FName InVariableName);

	/** Validate child blueprint component member variables against the given variable name */
	static UNREALED_API bool ValidateAllComponentMemberVariables(UBlueprint* InBlueprint, UBlueprint* InParentBlueprint, const FName& InVariableName);

	/** Validates all timelines of the passed blueprint against the given variable name */
	static UNREALED_API bool ValidateAllTimelines(UBlueprint* InBlueprint, UBlueprint* InParentBlueprint, const FName& InVariableName);

	/** Validates all function graphs of the passed blueprint against the given variable name */
	static UNREALED_API bool ValidateAllFunctionGraphs(UBlueprint* InBlueprint, UBlueprint* InParentBlueprint, const FName& InVariableName);

	/**
	 * Checks if the passed node connects to the selection set
	 *
	 * @param InNode				The node to check
	 * @param InSelectionSet		The selection set to check for a connection to
	 *
	 * @return						Returns TRUE if the node does connect to the selection set
	 */
	static UNREALED_API bool CheckIfNodeConnectsToSelection(UEdGraphNode* InNode, const TSet<UEdGraphNode*>& InSelectionSet);

	/**
	 * Returns an array of variables Get/Set nodes of the current variable
	 *
	 * @param InVarName		Variable to check for being in use
	 * @param InBlueprint	Blueprint to check within
	 * @param InScope		Option scope for local variables
	 * @return				Array of variable nodes
	 */
	static UNREALED_API TArray<UK2Node*> GetNodesForVariable(const FName& InVarName, const UBlueprint* InBlueprint, const UStruct* InScope = nullptr);

	/**
	 * Helper function to warn user of the results of changing var type by displaying a suppressible dialog
	 *
	 * @param InVarName		Variable name to display in the dialog message
	 * @return				TRUE if the user wants to change the variable type
	 */
	static UNREALED_API bool VerifyUserWantsVariableTypeChanged(const FName& InVarName);

	/**
	 * Helper function to warn user that we are about to search for a reference, which can incur an expensive
	 * indexing operation
	 *
	 * @param Reason		The reason the search would be requested, logged if search is suppressed
	 * @param SearchString	The search string that will be used if the user wants to search, logged if search is suppressed
	 * @return				TRUE if the user wants to perform the search operation
	 */
	static UNREALED_API bool VerifyUserWantsToSearch(FText Reason, const FString& SearchString);

	/**
	 * Helper function to warn user of the results of changing a RepNotify variable name by displaying a suppressible dialog
	 *
	 * @param InVarName		Variable name to display in the dialog message
	 * @param InFuncName	Associated OnRep function name to display in the dialog message
	 * @return				TRUE if the user wants to change the variable name
	 */
	static UNREALED_API bool VerifyUserWantsRepNotifyVariableNameChanged(const FName& InVarName, const FName& InFuncName);

	/**
	 * Helper function to get all loaded Blueprints that are children (or using as an interface) the passed Blueprint
	 *
	 * @param InBlueprint		Blueprint to find children of
	 * @param OutBlueprints		Filled out with child Blueprints
	 */
	static UNREALED_API void GetLoadedChildBlueprints(UBlueprint* InBlueprint, TArray<UBlueprint*>& OutBlueprints);

	/**
	 * Validates flags and settings on object pins, keeping them from being given default values and from being in invalid states
	 *
	 * @param InOutVarDesc		The variable description to validate
	 */
	static UNREALED_API void PostSetupObjectPinType(UBlueprint* InBlueprint, FBPVariableDescription& InOutVarDesc);

public:
	/** Event fired after RenameVariableReferences is called */
	DECLARE_MULTICAST_DELEGATE_FourParams(FOnRenameVariableReferences, UBlueprint* /*Blueprint*/, UClass* /*VariableClass*/, const FName& /*OldVarName*/, const FName& /*NewVarName*/);
	static UNREALED_API FOnRenameVariableReferences OnRenameVariableReferencesEvent;

	/**
	 * Looks through the specified blueprint for any references to the specified 
	 * variable, and renames them accordingly.
	 * 
	 * @param  Blueprint		The blueprint that you want to search through.
	 * @param  VariableClass	The class that owns the variable that we're renaming
	 * @param  OldVarName		The current name of the variable we want to replace
	 * @param  NewVarName		The name that we wish to change all references to
	 */
	static UNREALED_API void RenameVariableReferences(UBlueprint* Blueprint, UClass* VariableClass, const FName& OldVarName, const FName& NewVarName);

	/** Event fired after RenameFunctionReferences is called */
	DECLARE_MULTICAST_DELEGATE_FourParams(FOnRenameFunctionReferences, UBlueprint* /*Blueprint*/, UClass* /*FunctionClass*/, const FName& /*OldFuncName*/, const FName& /*NewFuncName*/);
	static UNREALED_API FOnRenameFunctionReferences OnRenameFunctionReferencesEvent;

	/**
	 * Looks through the specified blueprint for any references to the specified 
	 * function, and renames them accordingly.
	 * 
	 * @param  Blueprint		The blueprint that you want to search through.
	 * @param  FunctionClass	The class that owns the function that we're renaming
	 * @param  OldFuncName		The current name of the function we want to replace
	 * @param  NewFuncName		The name that we wish to change all references to
	 */
	static UNREALED_API void RenameFunctionReferences(UBlueprint* Blueprint, UClass* FunctionClass, const FName& OldFuncName, const FName& NewFuncName);

public:
	static UNREALED_API FName GetFunctionNameFromClassByGuid(const UClass* InClass, const FGuid FunctionGuid);
	static UNREALED_API bool GetFunctionGuidFromClassByFieldName(const UClass* InClass, const FName FunctionName, FGuid& FunctionGuid);

	/**
	 * Returns a friendly class display name for the specified class (removing things like _C from the end, may localize the class name).  Class can be nullptr.
	 */
	static UNREALED_API FText GetFriendlyClassDisplayName(const UClass* Class);

	/**
	 * Returns a class name for the specified class that has no automatic suffixes, but is otherwise unmodified.  Class can be nullptr.
	 */
	static UNREALED_API FString GetClassNameWithoutSuffix(const UClass* Class);

	/**
	 * Returns a formatted menu item label for a deprecated variable or function member with the given name.
	 *
	 * @param MemberName		(Required) User-facing name of the deprecated variable or function.
	 */
	static UNREALED_API FText GetDeprecatedMemberMenuItemName(const FText& MemberName);

	/**
	 * Returns a formatted warning message regarding usage of a deprecated variable or function member with the given name.
	 *
	 * @param MemberName		(Required) User-facing name of the deprecated variable or function.
	 * @param DetailedMessage	(Optional) Instructional text or other details from the owner. If empty, a default message will be used.
	 */
	static UNREALED_API FText GetDeprecatedMemberUsageNodeWarning(const FText& MemberName, const FText& DetailedMessage);

	/**
	 * Returns the severity of a deprecation warning by parsing a string loaded from metadata like DeprecatedFunction="Note"
	 *
	 * @param TypeString		(Required) String to parse for a message type
	 */
	static UNREALED_API EEdGraphNodeDeprecationMessageType GetDeprecatedMessageType(const FString& TypeString);

	/**
	 * Remove overridden component templates from instance component handlers when a parent class disables editable when inherited boolean.
	 */
	static UNREALED_API void HandleDisableEditableWhenInherited(UObject* ModifiedObject, TArray<UObject*>& ArchetypeInstances);

	/**
	 * Returns the BPs most derived native parent type:
	 */
	static UNREALED_API UClass* GetNativeParent(const UBlueprint* BP);

	/** Returns the UClass type for an object pin, if any */
	static UNREALED_API UClass* GetTypeForPin(const UEdGraphPin& Pin);

	/**
	 * Returns true if this BP is currently based on a type that returns true for the UObject::ImplementsGetWorld() call:
	 */
	static UNREALED_API bool ImplementsGetWorld(const UBlueprint* BP);

	/**
	 * Returns true if this BP needs to show world context pins, because it does not implement GetWorld or has ShowWorldContextPin set
	 */
	static UNREALED_API bool ShouldShowWorldContextPin(const UBlueprint* BP);

	/*
	 * Checks a function for 'blueprint thread safety'. Note this is not strict thread safety and does not perform any
	 * analysis on the function, it only inspects the metadata. Analysis is performed by the compiler for BP functions
	 * that are marked thread safe and metadata is used both here and in the compiler for native functions that have
	 * been validated for thread safe use in a blueprint.
	 */
	static UNREALED_API bool HasFunctionBlueprintThreadSafeMetaData(const UFunction* InFunction);

public:
	static UNREALED_API bool ShouldOpenWithDataOnlyEditor(const UBlueprint* Blueprint);
};

struct FBlueprintDuplicationScopeFlags
{
	enum EFlags : uint32
	{
		NoFlags = 0,
		NoExtraCompilation = 1 << 0,
		TheSameTimelineGuid = 1 << 1,
		// This flag is needed for C++ backend (while compiler validates graphs). The actual BPGC type is compatible with the original BPGC.
		ValidatePinsUsingSourceClass = 1 << 2,
		TheSameNodeGuid = 1 << 3,
	};

	static UNREALED_API uint32 bStaticFlags;
	static bool HasAnyFlag(uint32 InFlags)
	{
		return 0 != (bStaticFlags & InFlags);
	}

	TGuardValue<uint32> Guard;
	FBlueprintDuplicationScopeFlags(uint32 InFlags) : Guard(bStaticFlags, InFlags) {}
};
struct FMakeClassSpawnableOnScope
{
	UClass* Class;
	bool bIsDeprecated;
	bool bIsAbstract;
	FMakeClassSpawnableOnScope(UClass* InClass)
		: Class(InClass), bIsDeprecated(false), bIsAbstract(false)
	{
		if (Class)
		{
			bIsDeprecated = Class->HasAnyClassFlags(CLASS_Deprecated);
			Class->ClassFlags &= ~CLASS_Deprecated;
			bIsAbstract = Class->HasAnyClassFlags(CLASS_Abstract);
			Class->ClassFlags &= ~CLASS_Abstract;
		}
	}
	~FMakeClassSpawnableOnScope()
	{
		if (Class)
		{
			if (bIsAbstract)
			{
				Class->ClassFlags |= CLASS_Abstract;
			}

			if (bIsDeprecated)
			{
				Class->ClassFlags |= CLASS_Deprecated;
			}
		}
	}
};
