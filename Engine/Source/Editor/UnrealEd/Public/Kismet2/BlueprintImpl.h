// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Blueprint.h"
#include "Kismet2/CompilerResultsLog.h"

template<typename... ArgTypes>
void UBlueprint::Message_Note(const FString& MessageToLog, ArgTypes... Args)
{
	if (CurrentMessageLog)
	{
		CurrentMessageLog->Note(*MessageToLog, Forward<ArgTypes>(Args)...);
	}
	else
	{
		UE_LOGF(LogBlueprint, Log, "[%ls] %ls", *GetName(), *MessageToLog);
	}
}

template<typename... ArgTypes>
void UBlueprint::Message_Warn(const FString& MessageToLog, ArgTypes... Args)
{
	if (CurrentMessageLog)
	{
		CurrentMessageLog->Warning(*MessageToLog, Forward<ArgTypes>(Args)...);
	}
	else
	{
		UE_LOGF(LogBlueprint, Warning, "[%ls] %ls", *GetName(), *MessageToLog);
	}
}

template<typename... ArgTypes>
void UBlueprint::Message_Error(const FString& MessageToLog, ArgTypes... Args)
{
	if (CurrentMessageLog)
	{
		CurrentMessageLog->Error(*MessageToLog, Forward<ArgTypes>(Args)...);
	}
	else
	{
		UE_LOGF(LogBlueprint, Error, "[%ls] %ls", *GetName(), *MessageToLog);
	}
}
