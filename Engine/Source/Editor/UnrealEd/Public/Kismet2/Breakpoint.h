// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPtr.h"

#include "Breakpoint.generated.h"


USTRUCT()
struct FBlueprintBreakpoint
{
	GENERATED_BODY()

private:
	// Is the breakpoint currently enabled?
	UPROPERTY()
	uint8 bEnabled:1;

	// Node that the breakpoint is placed on
	UPROPERTY()
	TSoftObjectPtr<UEdGraphNode> Node = nullptr;

	// Is this breakpoint auto-generated, and should be removed when next hit?
	UPROPERTY()
	uint8 bStepOnce:1;

	UPROPERTY()
	uint8 bStepOnce_WasPreviouslyDisabled:1;

	UPROPERTY()
	uint8 bStepOnce_RemoveAfterHit:1;

public:
	UNREALED_API FBlueprintBreakpoint();

	/** Get the target node for the breakpoint */
	UEdGraphNode* GetLocation() const
	{
		return Node.Get();
	}

	/** @returns true if the breakpoint should be enabled when debugging */
	bool IsEnabled() const
	{
		return bEnabled;
	}

	/** @returns true if the user wanted the breakpoint enabled?  (single stepping, etc... could result in the breakpoint being temporarily enabled) */
	bool IsEnabledByUser() const
	{
		return bEnabled && !(bStepOnce && bStepOnce_WasPreviouslyDisabled);
	}

	bool operator==(const FBlueprintBreakpoint& Other) const
	{
		return Node == Other.Node;
	}

	/** Gets a string that describes the location */
	UNREALED_API FText GetLocationDescription() const;

	friend class FKismetDebugUtilities;
};

