// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/ChildActorComponent.h"

class UToolMenu;
class FSCSEditorTreeNode;

class FChildActorComponentEditorUtils
{
public:
	/** Returns true if the given SCS editor tree node is a child actor node */
	static UNREALED_API bool IsChildActorNode(TSharedPtr<const FSCSEditorTreeNode> InNodePtr);

	/** Returns true if the given SCS editor tree node belongs to a child actor template */
	static UNREALED_API bool IsChildActorSubtreeNode(TSharedPtr<const FSCSEditorTreeNode> InNodePtr);

	/** Returns true if any array element belongs to a child actor template */
	static UNREALED_API bool ContainsChildActorSubtreeNode(const TArray<TSharedPtr<FSCSEditorTreeNode>>& InNodePtrs);

	/** Returns the child actor component root node of a child actor subtree */
	static UNREALED_API TSharedPtr<FSCSEditorTreeNode> GetOuterChildActorComponentNode(TSharedPtr<const FSCSEditorTreeNode> InNodePtr);

	/** Returns whether child actor tree view expansion is enabled in project settings */
	static UNREALED_API bool IsChildActorTreeViewExpansionEnabled();

	/** Returns the default visualization mode for child actors in a component tree view */
	static UNREALED_API EChildActorComponentTreeViewVisualizationMode GetProjectDefaultTreeViewVisualizationMode();

	/** 
	 * Returns the component-specific visualization mode for the given child actor component
	 * @param DefaultVisOverride If different than UseDefault, overrides the default ChildActorComponentTreeViewVisualizationMode from the project settings.
	 */
	static UNREALED_API EChildActorComponentTreeViewVisualizationMode GetChildActorTreeViewVisualizationMode(UChildActorComponent* ChildActorComponent, EChildActorComponentTreeViewVisualizationMode DefaultVisOverride = EChildActorComponentTreeViewVisualizationMode::UseDefault);

	/** 
	 * Whether to expand the given child actor component in a component tree view 
	 * @param DefaultVisOverride If different than UseDefault, overrides the default ChildActorComponentTreeViewVisualizationMode from the project settings and forces child actor tree view expansion to be enabled.
	 */
	static UNREALED_API bool ShouldExpandChildActorInTreeView(UChildActorComponent* ChildActorComponent, EChildActorComponentTreeViewVisualizationMode DefaultVisOverride = EChildActorComponentTreeViewVisualizationMode::UseDefault);

	/** 
	 * Whether the Child Actor should be shown in a component tree view for the given component
	 * @param DefaultVisOverride If different than UseDefault, overrides the default ChildActorComponentTreeViewVisualizationMode from the project settings and forces child actor tree view expansion to be enabled.
	 */
	static UNREALED_API bool ShouldShowChildActorNodeInTreeView(UChildActorComponent* ChildActorComponent, EChildActorComponentTreeViewVisualizationMode DefaultVisOverride = EChildActorComponentTreeViewVisualizationMode::UseDefault);

	/** Populates the given menu with options for the given Child Actor component */
	static UNREALED_API void FillComponentContextMenuOptions(UToolMenu* Menu, UChildActorComponent* ChildActorComponent);

	/** Populates the given menu with additional options if the given SCS editor tree node represents a Child Actor node */
	static UNREALED_API void FillChildActorContextMenuOptions(UToolMenu* Menu, TSharedPtr<const FSCSEditorTreeNode> InNodePtr);
};
