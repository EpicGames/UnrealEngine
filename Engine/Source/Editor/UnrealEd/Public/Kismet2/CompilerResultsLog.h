// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformTime.h"
#include "Internationalization/Text.h"
#include "Misc/CString.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Stats/Stats.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectKey.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtrTemplates.h"

#if WITH_EDITOR
#include "EdGraphToken.h"
#include "Logging/TokenizedMessage.h"
#include "Misc/CompilationResult.h"
#endif

class FDelegateHandle;
class FTokenizedMessage;
class IMessageLogListing;
class UBlueprint;
class UEdGraphNode;
class UEdGraphPin;
class UObject;
struct FObjectKey;

#if WITH_EDITOR

/** This class maps from final objects to their original source object, across cloning, autoexpansion, etc... */
class FBacktrackMap
{
protected:
	// Maps from transient object created during compiling to original 'source code' object
	TMap<UObject const*, UObject*> SourceBacktrackMap;
	// Maps from transient pins created during compiling to original 'source pin' object
	TMap<UEdGraphPin*, UEdGraphPin*> PinSourceBacktrackMap;

public:
	/** Update the source backtrack map to note that NewObject was most closely generated/caused by the SourceObject */
	UNREALED_API void NotifyIntermediateObjectCreation(UObject* NewObject, UObject* SourceObject);

	/** Update the pin source backtrack map to note that NewPin was most closely generated/caused by the SourcePin */
	UNREALED_API void NotifyIntermediatePinCreation(UEdGraphPin* NewPin, UEdGraphPin* SourcePin);

	/** Returns the true source object for the passed in object */
	UNREALED_API UObject* FindSourceObject(UObject* PossiblyDuplicatedObject);
	UNREALED_API UObject const* FindSourceObject(UObject const* PossiblyDuplicatedObject) const;
	UNREALED_API UEdGraphPin* FindSourcePin(UEdGraphPin* PossiblyDuplicatedPin);
	UNREALED_API UEdGraphPin const* FindSourcePin(UEdGraphPin const* PossiblyDuplicatedPin) const;
};

/** This class represents a log of compiler output lines (errors, warnings, and information notes), each of which can be a rich tokenized message */
class FCompilerResultsLog
{
	// Compiler event
	struct FCompilerEvent
	{
		FString Name;
		uint32 Counter;
		double StartTime;
		double FinishTime;
		TSharedPtr<FCompilerEvent> ParentEventScope;
		TArray< TSharedRef<FCompilerEvent> > ChildEvents;

		FCompilerEvent(TSharedPtr<FCompilerEvent> InParentEventScope = nullptr)
			: Name(TEXT(""))
			, Counter(0)
			, StartTime(0.0)
			, FinishTime(0.0)
			, ParentEventScope(InParentEventScope)
		{
		}

		void Start(const TCHAR* InName)
		{
			Name = InName;
			StartTime = FPlatformTime::Seconds();
		}

		void Finish()
		{
			FinishTime = FPlatformTime::Seconds();
		}
	};

	// Current compiler event scope
	TSharedPtr<FCompilerEvent> CurrentEventScope;

public:
	// List of all tokenized messages
	TArray< TSharedRef<FTokenizedMessage> > Messages;

	// Number of error messages
	int32 NumErrors;

	// Number of warnings
	int32 NumWarnings;

	// Should we be silent?
	bool bSilentMode;

	// Should we log only Info messages, or all messages?
	bool bLogInfoOnly;

	// Should nodes mentioned in messages be annotated for display with that message?
	bool bAnnotateMentionedNodes;

	// Should detailed BeginEvent/EndEvent timing information be written to log
	bool bLogDetailedResults;

	// Minimum event time (ms) for events include in detailed results
	int EventDisplayThresholdMs;

	/** Tracks nodes that produced errors/warnings */
	TSet< TWeakObjectPtr<UEdGraphNode> > AnnotatedNodes;

protected:
	// Maps from transient object created during compiling to original 'source code' object
	FBacktrackMap SourceBacktrackMap;

	// Name of the source object being compiled
	FString SourcePath;

	// Map to track intermediate tunnel nodes back to the intermediate expansion tunnel instance.
	TMap<TWeakObjectPtr<const UEdGraphNode>, TWeakObjectPtr<const UEdGraphNode>> IntermediateTunnelNodeToTunnelInstanceMap;

	// Map to track intermediate nodes back to the source macro instance nodes
	TMap<TWeakObjectPtr<const UEdGraphNode>, TWeakObjectPtr<UEdGraphNode>> FullMacroBacktrackMap;

public:
	UNREALED_API FCompilerResultsLog(bool bIsCompatibleWithEvents = true);
	UNREALED_API virtual ~FCompilerResultsLog();

	/** Register this log with the MessageLog module */
	static UNREALED_API void Register();

	/** Unregister this log from the MessageLog module */
	static UNREALED_API void Unregister();

	/** Accessor for the LogName, so it can be opened elsewhere */
	static FName GetLogName(){ return Name; }

	/** Set the source name for the final log summary */
	void SetSourcePath(const FString& InSourcePath)
	{
		SourcePath = InSourcePath;
	}

	/**
	 * Write an error in to the compiler log.
	 * Note: @@ will be replaced by node or pin links for nodes/pins passed via varargs
	 */
	template<typename... ArgTypes>
	TSharedRef<FTokenizedMessage> Error(const TCHAR* Format, ArgTypes... Args)
	{
		++NumErrors;
		TSharedRef<FTokenizedMessage> Line = FTokenizedMessage::Create(EMessageSeverity::Error);
		InternalLogMessage(NAME_None, Format, Line, Args...);
		return Line;
	}

	/**
	 * Write a warning in to the compiler log.
	 * Note: @@ will be replaced by node or pin links for nodes/pins passed via varargs
	 */
	template<typename... ArgTypes>
	TSharedRef<FTokenizedMessage> Warning(const TCHAR* Format, ArgTypes... Args)
	{
		++NumWarnings;
		TSharedRef<FTokenizedMessage> Line = FTokenizedMessage::Create(EMessageSeverity::Warning);
		InternalLogMessage(NAME_None, Format, Line, Args...);
		return Line;
	}

	/**
	 * Write a warning in to the compiler log.
	 * Note: @@ will be replaced by node or pin links for nodes/pins passed via varargs
	 */
	template<typename... ArgTypes>
	void Warning(FName ID, const TCHAR* Format, ArgTypes... Args)
	{
		if (!IsMessageEnabled(ID))
		{
			return;
		}

		++NumWarnings;
		TSharedRef<FTokenizedMessage> Line = FTokenizedMessage::Create(EMessageSeverity::Warning);
		InternalLogMessage(ID, Format, Line, Args...);
		return;
	}

	/**
	 * Write a note in to the compiler log.
	 * Note: @@ will be replaced by node or pin links for nodes/pins passed via varargs
	 */
	template<typename... ArgTypes>
	TSharedRef<FTokenizedMessage> Note(const TCHAR* Format, ArgTypes... Args)
	{
		TSharedRef<FTokenizedMessage> Line = FTokenizedMessage::Create(EMessageSeverity::Info);
		InternalLogMessage(NAME_None, Format, Line, Args...);
		return Line;
	}

	/**
	 * Store a potential error for a given node in the compiler log. All messages for the node can be committed to the log later by calling CommitPotentialMessages
	 * Note: @@ will be replaced by node or pin links for nodes/pins passed via varargs 
	 */
	template<typename... ArgTypes>
	TSharedRef<FTokenizedMessage> StorePotentialError(const UEdGraphNode* Source, const TCHAR* Format, ArgTypes... Args)
	{
		TSharedRef<FTokenizedMessage> Line = FTokenizedMessage::Create(EMessageSeverity::Error);
		TArray<UEdGraphNode*> SourceNodes;
		Tokenize(Format, *Line, SourceNodes, Args...);
		PotentialMessages.FindOrAdd(Source).Add(Line);
		return Line;
	}

	/**
	 * Store a potential warning for a given node in the compiler log. All messages for the node can be committed to the log later by calling CommitPotentialMessages
	 * Note: @@ will be replaced by node or pin links for nodes/pins passed via varargs 
	 */
	template<typename... ArgTypes>
	TSharedRef<FTokenizedMessage> StorePotentialWarning(const UEdGraphNode* Source, const TCHAR* Format, ArgTypes... Args)
	{
		TSharedRef<FTokenizedMessage> Line = FTokenizedMessage::Create(EMessageSeverity::Warning);
		TArray<UEdGraphNode*> SourceNodes;
		Tokenize(Format, *Line, SourceNodes, Args...);
		PotentialMessages.FindOrAdd(Source).Add(Line);
		return Line;
	}

	/**
	 * Store a potential note for a given node in the compiler log. All messages for the node can be committed to the log later by calling CommitPotentialMessages
	 * Note: @@ will be replaced by node or pin links for nodes/pins passed via varargs 
	 */
	template<typename... ArgTypes>
	TSharedRef<FTokenizedMessage> StorePotentialNote(const UEdGraphNode* Source, const TCHAR* Format, ArgTypes... Args)
	{
		TSharedRef<FTokenizedMessage> Line = FTokenizedMessage::Create(EMessageSeverity::Info);
		TArray<UEdGraphNode*> SourceNodes;
		Tokenize(Format, *Line, SourceNodes, Args...);
		PotentialMessages.FindOrAdd(Source).Add(Line);
		return Line;
	}

	/**
	 * Store an already tokenized message.
	 */
	void AddTokenizedMessage(TSharedRef<FTokenizedMessage> InMessage)
	{
		switch (InMessage->GetSeverity())
		{
		case EMessageSeverity::Error:
			++NumErrors;
			break;
		case EMessageSeverity::Warning:
		case EMessageSeverity::PerformanceWarning:
			++NumWarnings;
			break;
		}

		Messages.Add(InMessage);
	}

	/**
	 * Commit all stored potential messages for a given node. Returns true if any messages were written.
	 */
	UNREALED_API bool CommitPotentialMessages(UEdGraphNode* Source);

	/** Update the source backtrack map to note that NewObject was most closely generated/caused by the SourceObject */
	UNREALED_API void NotifyIntermediateObjectCreation(UObject* NewObject, UObject* SourceObject);
	UNREALED_API void NotifyIntermediatePinCreation(UEdGraphPin* NewObject, UEdGraphPin* SourceObject);

	/** Update the expansion map to note that Node was expanded from OuterTunnelInstance, both the node and tunnel instance should be intermediate nodes */
	UNREALED_API void NotifyIntermediateTunnelNode(const UEdGraphNode* Node, const UEdGraphNode* OuterTunnelInstance);

	/** Update the map that tracks nodes created by macro instance nodes */
	UNREALED_API void NotifyIntermediateMacroNode(UEdGraphNode* SourceNode, const UEdGraphNode* IntermediateNode);

	/** Returns the true source object for the passed in object */
	UNREALED_API UObject* FindSourceObject(UObject* PossiblyDuplicatedObject);
	UNREALED_API UObject const* FindSourceObject(UObject const* PossiblyDuplicatedObject) const;
	UNREALED_API UObject* FindSourceMacroInstance(const UEdGraphNode* IntermediateNode) const;

	/** Returns the intermediate tunnel instance that generated the node */
	UNREALED_API const UEdGraphNode* GetIntermediateTunnelInstance(const UEdGraphNode* IntermediateNode) const;

	/** Returns a int32 used to uniquely identify an action for the latent action manager */
	UNREALED_API int32 CalculateStableIdentifierForLatentActionManager( const UEdGraphNode* Node );

	/** Returns the true source object for the passed in object; does type checking on the result */
	template <typename T>
	T* FindSourceObjectTypeChecked(UObject* PossiblyDuplicatedObject)
	{
		return CastChecked<T>(FindSourceObject(PossiblyDuplicatedObject));
	}
	
	template <typename T>
	T const* FindSourceObjectTypeChecked(UObject const* PossiblyDuplicatedObject) const
	{
		return CastChecked<T const>(FindSourceObject(PossiblyDuplicatedObject));
	}

	UNREALED_API UEdGraphPin* FindSourcePin(UEdGraphPin* PossiblyDuplicatedPin);
	UNREALED_API const UEdGraphPin* FindSourcePin(const UEdGraphPin* PossiblyDuplicatedPin) const;
	
	/** Copy errors from an existing log into this one, and optionally write out to log if it was suppressed the first time */
	UNREALED_API void Append(FCompilerResultsLog const& Other, bool bWriteToSystemLog = false);

	/** Begin a new compiler event */
	UNREALED_API void BeginEvent(const TCHAR* InName);

	/** End the current compiler event */
	UNREALED_API void EndEvent();

	UE_DEPRECATED(5.4, "BP-specific perf tracking has been removed, use Insights")
	static FCompilerResultsLog* GetEventTarget()
	{
		return nullptr;
	}

	/** Get the message log listing for this blueprint */
	static UNREALED_API TSharedRef<IMessageLogListing> GetBlueprintMessageLog(UBlueprint* InBlueprint);

	/** ICompilerResultsLog implementation */
	void SetSilentMode(bool bValue) { bSilentMode = bValue; };

protected:
	/** Helper method to add a child event to the given parent event scope */
	UNREALED_API void AddChildEvent(TSharedPtr<FCompilerEvent>& ParentEventScope, TSharedRef<FCompilerEvent>& ChildEventScope);
	void Tokenize(const TCHAR* Text, FTokenizedMessage& OutMessage, TArray<UEdGraphNode*>& OutSourceNode)
	{
		OutMessage.AddToken(FTextToken::Create(FText::FromString(Text)));
	}

	template<typename T, typename... ArgTypes>
	void Tokenize(const TCHAR* Format, FTokenizedMessage& OutMessage, TArray<UEdGraphNode*>& OutSourceNode, T First, ArgTypes... Rest)
	{
		// read to next "@@":
		if (const TCHAR* DelimiterStr = FCString::Strstr(Format, TEXT("@@")))
		{
			int32 TokenLength = UE_PTRDIFF_TO_INT32(DelimiterStr - Format);
			OutMessage.AddToken(FTextToken::Create(FText::FromString(FString::ConstructFromPtrSize(Format, TokenLength))));
			FEdGraphToken::Create(First, this, OutMessage, OutSourceNode);

			const TCHAR* NextChunk = DelimiterStr + FCString::Strlen(TEXT("@@"));
			if (*NextChunk)
			{
				Tokenize(NextChunk, OutMessage, OutSourceNode, Rest...);
			}
		}
		else
		{
			Tokenize(Format, OutMessage, OutSourceNode);
		}
	}

	template<typename... ArgTypes>
	void InternalLogMessage(FName MessageID, const TCHAR* Format, const TSharedRef<FTokenizedMessage>& Message, ArgTypes... Args)
	{
		// Convention for SourceNode established by the original version of the compiler results log
		// was to annotate the error on the first node we can find. I am preserving that behavior
		// for this type safe, variadic version:
		TArray<UEdGraphNode*> SourceNodes;
		Tokenize(Format, *Message, SourceNodes, Args...);
		InternalLogMessage(MessageID, Message, SourceNodes);
	}

	/** Links the UEdGraphNode with the LogLine: */
	UNREALED_API void AnnotateNode(const TArray<UEdGraphNode*>& Nodes, TSharedRef<FTokenizedMessage> LogLine);

	/** Internal method to append the final compiler results summary to the MessageLog */
	UNREALED_API void InternalLogSummary();

	/** Internal helper method to recursively append event details into the MessageLog */
	UNREALED_API void InternalLogEvent(const FCompilerEvent& InEvent, int32 InDepth = 0);

	UNREALED_API void InternalLogMessage(FName MessageID, const TSharedRef<FTokenizedMessage>& Message, const TArray<UEdGraphNode*>& SourceNodes);
	UNREALED_API void FEdGraphToken_Create(const UObject* InObject, FTokenizedMessage& OutMessage, TArray<UEdGraphNode*>& OutSourceNodes);
	UNREALED_API void FEdGraphToken_Create(const UEdGraphPin* InPin, FTokenizedMessage& OutMessage, TArray<UEdGraphNode*>& OutSourceNodes);
	UNREALED_API void FEdGraphToken_Create(const TCHAR* String, FTokenizedMessage& OutMessage, TArray<UEdGraphNode*>& OutSourceNodes);
	UNREALED_API void FEdGraphToken_Create(const FField* InField, FTokenizedMessage& OutMessage, TArray<UEdGraphNode*>& OutSourceNodes);
	void IncrementErrorCount() { ++NumErrors; }
	void IncrementWarningCount() { ++NumWarnings; };
	UNREALED_API bool IsMessageEnabled(FName ID);

private:

	/** Map of stored potential messages indexed by a node. Can be committed to the results log by calling CommitPotentialMessages for that node. */
	TMap< FObjectKey, TArray< TSharedRef<FTokenizedMessage> > > PotentialMessages;

	/** Parses a compiler log dump to generate tokenized output */
	static UNREALED_API TArray< TSharedRef<FTokenizedMessage> > ParseCompilerLogDump(const FString& LogDump);

	/** Goes to an error given a Message Token */
	static UNREALED_API void OnGotoError(const class TSharedRef<IMessageToken>& Token);

	/** Callback function for binding the global compiler dump to open the static compiler log */
	static UNREALED_API void GetGlobalModuleCompilerDump(const FString& LogDump, ECompilationResult::Type CompilationResult, bool bShowLog);
	
	/** Searches a token list for referenced UEdGraphNodes, used to update the nodes when a log is committed */
	static UNREALED_API void GetNodesFromTokens(const TArray<TSharedRef<IMessageToken> >& MessageTokens, TArray<UEdGraphNode*>& OutOwnerNodes);

	/** The log's name, for easy re-use */
	static UNREALED_API const FName Name;
	
	/** Handle to the registered GetGlobalModuleCompilerDump delegate. */
	static UNREALED_API FDelegateHandle GetGlobalModuleCompilerDumpDelegateHandle;
};

class FScopedCompilerEvent
{
public:
	UE_DEPRECATED(5.4, "BP-specific perf tracking has been removed, use Insights")
	FScopedCompilerEvent(const TCHAR* InName)
	{
	}
};

/** Scope wrapper for the blueprint message log. Ensures we dont leak logs that we dont need (i.e. those that have no messages) */
class FScopedBlueprintMessageLog
{
public:
	UNREALED_API FScopedBlueprintMessageLog(UBlueprint* InBlueprint);
	UNREALED_API ~FScopedBlueprintMessageLog();

public:
	/** The listing we wrap */
	TSharedRef<IMessageLogListing> Log;

	/** The generated name of the log */
	FName LogName;
};

#if STATS
#define BP_SCOPED_COMPILER_EVENT_STAT(Stat) \
	SCOPE_CYCLE_COUNTER(Stat); \
	TRACE_CPUPROFILER_EVENT_SCOPE(Stat);
#else
#define BP_SCOPED_COMPILER_EVENT_STAT(Stat) \
	TRACE_CPUPROFILER_EVENT_SCOPE(Stat);
#endif
#endif	//#if WITH_EDITOR
