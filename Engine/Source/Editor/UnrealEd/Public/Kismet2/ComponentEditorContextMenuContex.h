// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/UObjectHash.h"
#include "Components/ActorComponent.h"

//#include "ComponentEditorContextMenuContex.generated.h"
//
//UCLASS()
//class UNREALED_API UComponentEditorContextMenuContext : public UObject
//{
//	GENERATED_BODY()
//public:
//	
//	TArray<UActorComponent*> SelectedComponents;
//};
