// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SubclassOf.h"
#include "Components/ActorComponent.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/Actor.h"
#include "UObject/UObjectHash.h"

class UToolMenu;
class UMaterialInterface;

class FComponentEditorUtils
{
public:
	/** Is the instance component is editable */
	static UNREALED_API bool CanEditComponentInstance(const UActorComponent* ActorComp, const UActorComponent* ParentSceneComp, bool bAllowUserContructionScript);

	/** 
	* Test if the native component is editable. If it is, return a valid pointer to it's FProperty
	* Otherwise, return nullptr. A native component is editable if it is marked as EditAnywhere
	* via meta data tags or is within an editable property container 
	*/
	static UNREALED_API FProperty* GetPropertyForEditableNativeComponent(const UActorComponent* NativeComponent);

	/** Test whether or not the given string is a valid variable name string for the given component instance */
	static UNREALED_API bool IsValidVariableNameString(const UActorComponent* InComponent, const FString& InString);

	/** 
	 * Test whether or not the given string is already the name string of a component on the the owner
	 * Optionally excludes an existing component from the check (ex. a component currently being renamed)
	 * @return True if the InString is an available name for a component of ComponentOwner
	 */
	static UNREALED_API bool IsComponentNameAvailable(const FString& InString, UObject* ComponentOwner, const UActorComponent* ComponentToIgnore = nullptr);
		
	/** Generate a valid variable name string for the given component instance */
	static UNREALED_API FString GenerateValidVariableName(TSubclassOf<UActorComponent> InComponentClass, AActor* ComponentOwner);

	/** Generate a valid variable name string for the given component instance based on the name of the asset referenced by the component */
	static UNREALED_API FString GenerateValidVariableNameFromAsset(UObject* Asset, AActor* ComponentOwner);

	/**
	* Checks whether it is valid to copy the given component
	* @param ComponentToCopy The component to check
	* @return Whether the given component can be copied
	*/
	static UNREALED_API bool CanCopyComponent(const UActorComponent* ComponentToCopy);

	/**
	 * Checks whether it is valid to copy the indicated components
	 * @param ComponentsToCopy The list of components to check
	 * @return Whether the indicated components can be copied
	 */
	static UNREALED_API bool CanCopyComponents(const TArray<UActorComponent*>& ComponentsToCopy);

	/**
	 * Copies the selected components to the clipboard
	 * @param ComponentsToCopy The list of components to copy
	 * @param DestinationData Buffer to fill with the copied data, or null to use the clipboard
	 */
	static UNREALED_API void CopyComponents(const TArray<UActorComponent*>& ComponentsToCopy, FString* DestinationData = nullptr);

	/**
	 * Determines whether the current contents of the clipboard contain paste-able component information
	 * @param RootComponent The root component of the actor being pasted on
	 * @param bOverrideCanAttach Optional override declaring that components can be attached and a check is not needed
	 * @param SourceData Component data to paste, or null to use the clipboard
	 * @return Whether components can be pasted
	 */
	static UNREALED_API bool CanPasteComponents(const USceneComponent* RootComponent, bool bOverrideCanAttach = false, bool bPasteAsArchetypes = false, const FString* SourceData = nullptr);

	/**
	 * Attempts to paste components from the clipboard as siblings of the target component
	 * @param OutPastedComponents List of all the components that were pasted
	 * @param TargetActor The actor to attach the pasted components to
	 * @param TargetComponent The component the paste is targeting (will attempt to paste components as siblings). If null, will attach pasted components to the root.
	 * @param SourceData Component data to paste, or null to use the clipboard
	 */
	static UNREALED_API void PasteComponents(TArray<UActorComponent*>& OutPastedComponents, AActor* TargetActor, USceneComponent* TargetComponent = nullptr, const FString* SourceData = nullptr);

	/**
	 * Gets the copied components from the clipboard without attempting to paste/apply them in any way
	 * @param OutParentMap Contains the child->parent name relationships of the copied components
	 * @param OutNewObjectMap Contains the name->instance object mapping of the copied components
	 */
	static UNREALED_API void GetComponentsFromClipboard(TMap<FName, FName>& OutParentMap, TMap<FName, UActorComponent*>& OutNewObjectMap, bool bGetComponentsAsArchetypes);

	/**
	 * Determines whether the indicated component can be deleted
	 * @param ComponentToDelete The component to determine can be deleted
	 * @return Whether the indicated component can be deleted
	 */
	static UNREALED_API bool CanDeleteComponent(const UActorComponent* ComponentToDelete);

	/**
	 * Determines whether the indicated components can be deleted
	 * @param ComponentsToDelete The list of components to determine can be deleted
	 * @return Whether the indicated components can be deleted
	 */
	static UNREALED_API bool CanDeleteComponents(const TArray<UActorComponent*>& ComponentsToDelete);

	/**
	 * Deletes the indicated components and identifies the component that should be selected following the operation.
	 * Note: Does not take care of the actual selection of a new component. It only identifies which component should be selected.
	 * 
	 * @param ComponentsToDelete The list of components to delete
	 * @param OutComponentToSelect The component that should be selected after the deletion
	 * @return The number of components that were actually deleted
	 */
	static UNREALED_API int32 DeleteComponents(const TArray<UActorComponent*>& ComponentsToDelete, UActorComponent*& OutComponentToSelect);

	/** 
	 * Duplicates a component instance and takes care of attachment and registration.
	 * @param TemplateComponent The component to duplicate
	 * @return The created clone of the provided TemplateComponent. nullptr if the duplication failed.
	 */
	static UNREALED_API UActorComponent* DuplicateComponent(UActorComponent* TemplateComponent);

	/**
	 * Ensures that the selection override delegate is properly bound for the supplied component
	* This includes any attached editor-only primitive components (such as billboard visualizers)
	 * 
	 * @param SceneComponent The component to set the selection override for
	 * @param bBind Whether the override should be bound
	*/
	static UNREALED_API void BindComponentSelectionOverride(USceneComponent* SceneComponent, bool bBind);

	/**
	 * Attempts to apply a material to a component at the specified slot.
	 *
	 * @param SceneComponent The component to which we should attempt to apply the material
	 * @param MaterialToApply The material to apply to the component
	 * @param OptionalMaterialSlot The material slot on the component to which the material should be applied. -1 to apply to all slots on the component.
	 *
	 * @return	True if the material was successfully applied to the component.
	 */
	static UNREALED_API bool AttemptApplyMaterialToComponent( USceneComponent* SceneComponent, UMaterialInterface* MaterialToApply, int32 OptionalMaterialSlot = -1 );

	/** Potentially transforms the delta to be applied to a component into the appropriate space */
	static UNREALED_API void AdjustComponentDelta(const USceneComponent* Component, FVector& Drag, FRotator& Rotation);

	// Given a template and a property, propagates a default value change to all instances (only if applicable)
	template<typename T>
	static void PropagateDefaultValueChange(class USceneComponent* InSceneComponentTemplate, const class FProperty* InProperty, const T& OldDefaultValue, const T& NewDefaultValue, TSet<class USceneComponent*>& UpdatedInstances, int32 PropertyOffset = INDEX_NONE)
	{
		TArray<UObject*> ArchetypeInstances;
		if(InSceneComponentTemplate->HasAnyFlags(RF_ArchetypeObject))
		{
			InSceneComponentTemplate->GetArchetypeInstances(ArchetypeInstances);
			for(int32 InstanceIndex = 0; InstanceIndex < ArchetypeInstances.Num(); ++InstanceIndex)
			{
				USceneComponent* InstancedSceneComponent = static_cast<USceneComponent*>(ArchetypeInstances[InstanceIndex]);
				if(InstancedSceneComponent != nullptr && !UpdatedInstances.Contains(InstancedSceneComponent) && ApplyDefaultValueChange(InstancedSceneComponent, InProperty, OldDefaultValue, NewDefaultValue, PropertyOffset))
				{
					UpdatedInstances.Add(InstancedSceneComponent);
				}
			}
		}
		else if(UObject* Outer = InSceneComponentTemplate->GetOuter())
		{
			Outer->GetArchetypeInstances(ArchetypeInstances);
			for(int32 InstanceIndex = 0; InstanceIndex < ArchetypeInstances.Num(); ++InstanceIndex)
			{
				USceneComponent* InstancedSceneComponent = static_cast<USceneComponent*>(FindObjectWithOuter(ArchetypeInstances[InstanceIndex], InSceneComponentTemplate->GetClass(), InSceneComponentTemplate->GetFName()));
				if(InstancedSceneComponent != nullptr && !UpdatedInstances.Contains(InstancedSceneComponent) && ApplyDefaultValueChange(InstancedSceneComponent, InProperty, OldDefaultValue, NewDefaultValue, PropertyOffset))
				{
					UpdatedInstances.Add(InstancedSceneComponent);
				}
			}
		}
	}

	// Given an instance of a template and a property, set a default value change to the instance (only if applicable)
	template<typename T>
	static bool ApplyDefaultValueChange(class USceneComponent* InSceneComponent, const class FProperty* InProperty, const T& OldDefaultValue, const T& NewDefaultValue, int32 PropertyOffset)
	{
		check(InProperty != nullptr);
		check(InSceneComponent != nullptr);

		ensureMsgf(CastField<FBoolProperty>(InProperty) == nullptr, TEXT("ApplyDefaultValueChange cannot be safely called on a bool property with a non-bool value, becuase of bitfields"));

		T* CurrentValue = PropertyOffset == INDEX_NONE ? InProperty->ContainerPtrToValuePtr<T>(InSceneComponent) : (T*)((uint8*)InSceneComponent + PropertyOffset);
		check(CurrentValue);

		return ApplyDefaultValueChange(InSceneComponent, *CurrentValue, OldDefaultValue, NewDefaultValue);
	}

	// Bool specialization so it can properly handle bitfields
	static bool ApplyDefaultValueChange(class USceneComponent* InSceneComponent, const class FProperty* InProperty, const bool& OldDefaultValue, const bool& NewDefaultValue, int32 PropertyOffset)
	{
		check(InProperty != nullptr);
		check(InSceneComponent != nullptr);
		
		// Only bool properties can have bool values
		const FBoolProperty* BoolProperty = CastField<FBoolProperty>(InProperty);
		check(BoolProperty);

		uint8* CurrentValue = PropertyOffset == INDEX_NONE ? InProperty->ContainerPtrToValuePtr<uint8>(InSceneComponent) : ((uint8*)InSceneComponent + PropertyOffset);
		check(CurrentValue);

		bool CurrentBool = BoolProperty->GetPropertyValue(CurrentValue);
		if (ApplyDefaultValueChange(InSceneComponent, CurrentBool, OldDefaultValue, NewDefaultValue, false))
		{
			BoolProperty->SetPropertyValue(CurrentValue, CurrentBool);

			InSceneComponent->ReregisterComponent();

			return true;
		}

		return false;
	}

	// Given an instance of a template and a current value, propagates a default value change to the instance (only if applicable)
	template<typename T>
	static bool ApplyDefaultValueChange(class USceneComponent* InSceneComponent, T& CurrentValue, const T& OldDefaultValue, const T& NewDefaultValue, bool bReregisterComponent = true)
	{
		check(InSceneComponent != nullptr);

		// Propagate the change only if the current instanced value matches the previous default value (otherwise this could overwrite any per-instance override)
		if(NewDefaultValue != OldDefaultValue && CurrentValue == OldDefaultValue)
		{
			// Ensure that this instance will be included in any undo/redo operations, and record it into the transaction buffer.
			// Note: We don't do this for components that originate from script, because they will be re-instanced from the template after an undo, so there is no need to record them.
			if (!InSceneComponent->IsCreatedByConstructionScript())
			{
				InSceneComponent->SetFlags(RF_Transactional);
				InSceneComponent->Modify();
			}

			// We must also modify the owner, because we'll need script components to be reconstructed as part of an undo operation.
			AActor* Owner = InSceneComponent->GetOwner();
			if(Owner != nullptr)
			{
				Owner->Modify();
			}

			// Modify the value
			CurrentValue = NewDefaultValue;

			if (bReregisterComponent && InSceneComponent->IsRegistered())
			{
				// Re-register the component with the scene so that transforms are updated for display
				if (InSceneComponent->AllowReregistration())
				{
					InSceneComponent->ReregisterComponent();
				}
				else
				{
					InSceneComponent->UpdateComponentToWorld();
				}
			}
			
			return true;
		}

		return false;
	}

	// Try to find the correct variable name for a given native component template or instance (which can have a mismatch)
	static UNREALED_API FName FindVariableNameGivenComponentInstance(const UActorComponent* ComponentInstance);

	/**
	* Populates the given menu with basic options for operations on components in the world.
	* @param Menu Used to register the menu options
	* @param SelectedComponents The selected components to create menu options for
	*/
	static UNREALED_API void FillComponentContextMenuOptions(UToolMenu* Menu, const TArray<UActorComponent*>& SelectedComponents);

	/**
	 * Tries to find a match for ComponentInstance in the ComponentList. First by name and then if multiple Components have a matching name try to match the SceneComponent hierarchy to find the best match.
	 * @param ComponentInstance Component we are trying to match in the ComponentList
	 * @param ComponentList List containing possible matches
	 * @return Valid Component pointer if match was found. nullptr otherwise.
	 */
	static UNREALED_API UActorComponent* FindMatchingComponent(const UActorComponent* ComponentInstance, const TInlineComponentArray<UActorComponent*>& ComponentList);

	/**
	 * Make a FComponentReference from a component pointer.
	 * @param ExpectedComponentOwner The expected component owner. Should be the same as OwningActor from FComponentReference::GetComponent().
	 * @param Component The component we would like to initialize the FComponentReference with.
	 */
	static UNREALED_API FComponentReference MakeComponentReference(const AActor* ExpectedComponentOwner, const UActorComponent* Component);

private:	
	static UNREALED_API USceneComponent* FindClosestParentInList(UActorComponent* ChildComponent, const TArray<UActorComponent*>& ComponentList);

	static UNREALED_API void OnGoToComponentAssetInBrowser(UObject* Asset);
	static UNREALED_API void OnOpenComponentCodeFile(const FString CodeFileName);
	static UNREALED_API void OnEditBlueprintComponent(UObject* Blueprint);
};
