// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandList.h"

class IAssetViewport;
class FToolBarBuilder;
struct FToolMenuSection;

/** This class acts as a generic widget that listens to and process global play world actions */
class SGlobalPlayWorldActions : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SGlobalPlayWorldActions) {}
		SLATE_DEFAULT_SLOT(FArguments, Content)

	SLATE_END_ARGS()

	UNREALED_API void Construct(const FArguments& InArgs);

	UNREALED_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	UNREALED_API virtual bool SupportsKeyboardFocus() const override;
};

//////////////////////////////////////////////////////////////////////////
// FPlayWorldCommands

UE_DECLARE_TCOMMANDS(class FPlayWorldCommands, UNREALED_API)

class FPlayWorldCommands : public TCommands<FPlayWorldCommands>
{
private:

	friend class TCommands<FPlayWorldCommands>;

	UNREALED_API FPlayWorldCommands();

public:

	// TCommands interface
	UNREALED_API virtual void RegisterCommands() override;
	// End of TCommands interface

	/**
	 * Binds all global kismet commands to delegates
	 */
	static UNREALED_API void BindGlobalPlayWorldCommands();

	/** Populates a toolbar with the menu commands for play-world control (pause/resume/stop/possess/eject/step/show current loc) */
	static UNREALED_API void BuildToolbar(FToolMenuSection& InSection);

	/**
	* Return the active widget that processes play world actions for PIE
	*
	*/
	static UNREALED_API TWeakPtr<SGlobalPlayWorldActions> GetActiveGlobalPlayWorldActionsWidget();

	/**
	* Set the active widget that processes play world actions for PIE
	*
	*/
	static UNREALED_API void SetActiveGlobalPlayWorldActionsWidget(TWeakPtr<SGlobalPlayWorldActions> ActiveWidget);

public:

	/** 
	 * A command list that can be passed around and isn't bound to an instance of any tool or editor. 
	 */
	static UNREALED_API TSharedPtr<FUICommandList> GlobalPlayWorldActions;

public:

	/** Start Simulating (SIE) */
	TSharedPtr<FUICommandInfo> Simulate;

	/** Play in editor (PIE) */
	TSharedPtr<FUICommandInfo> RepeatLastPlay;
	TSharedPtr<FUICommandInfo> PlayInViewport;
	TSharedPtr<FUICommandInfo> PlaySimulate;
	TSharedPtr<FUICommandInfo> PlayInEditorFloating;
	TSharedPtr<FUICommandInfo> PlayInVR;
	TSharedPtr<FUICommandInfo> PlayInMobilePreview;
	TSharedPtr<FUICommandInfo> PlayInNewProcess;
	TSharedPtr<FUICommandInfo> PlayInCameraLocation;
	TSharedPtr<FUICommandInfo> PlayInDefaultPlayerStart;
	TSharedPtr<FUICommandInfo> PlayInSettings;
	TSharedPtr<FUICommandInfo> PlayInNetworkSettings;

	/** SIE & PIE controls */
	TSharedPtr<FUICommandInfo> ResumePlaySession;
	TSharedPtr<FUICommandInfo> PausePlaySession;
	TSharedPtr<FUICommandInfo> SingleFrameAdvance;
	TSharedPtr<FUICommandInfo> TogglePlayPauseOfPlaySession;
	TSharedPtr<FUICommandInfo> StopPlaySession;
	TSharedPtr<FUICommandInfo> LateJoinSession;
	TSharedPtr<FUICommandInfo> PossessEjectPlayer;
	TSharedPtr<FUICommandInfo> ShowCurrentStatement;
	TSharedPtr<FUICommandInfo> GetMouseControl;

	/** BP Debugging controls */
	TSharedPtr<FUICommandInfo> AbortExecution;
	TSharedPtr<FUICommandInfo> ContinueExecution;
	TSharedPtr<FUICommandInfo> StepInto;
	TSharedPtr<FUICommandInfo> StepOver;
	TSharedPtr<FUICommandInfo> StepOut;

protected:

	/** A weak pointer to the current active widget that processes PIE actions */
	static UNREALED_API TWeakPtr<SGlobalPlayWorldActions> ActiveGlobalPlayWorldActionsWidget;

	/**
	 * Generates menu content for the PIE combo button drop down menu
	 *
	 * @return	Menu content widget
	 */
	static UNREALED_API TSharedRef< SWidget > GeneratePlayMenuContent( TSharedRef<FUICommandList> InCommandList );
};


//////////////////////////////////////////////////////////////////////////
// FPlayWorldCommandCallbacks

class FPlayWorldCommandCallbacks
{
public:
	/**
	 * Called from the context menu to start previewing the game at the clicked location                   
	 */
	static UNREALED_API void StartPlayFromHere();
	static UNREALED_API void StartPlayFromHere(const TOptional<FVector>& Location, const TOptional<FRotator>& Rotation, const TSharedPtr<IAssetViewport>& ActiveLevelViewport);

	enum EPlayFromHereMode
	{
		/** Provide value for Location, Rotation */
		Custom,
		/** Determines value for Location, Rotation based on click position in viewport */
		Click,
		/** Determines values for Location, Rotation based on the selected actor in level */
		SelectedActor
	};

	struct FStartPlayFromHereOptions
	{
		EPlayFromHereMode Mode = EPlayFromHereMode::Custom;
		TOptional<FVector> Location;
		TOptional<FRotator> Rotation;
		TSharedPtr<IAssetViewport> DestinationSlateViewport;
	};

	static UNREALED_API bool StartPlayFromHere(const FStartPlayFromHereOptions& InOptions);

	static UNREALED_API void ResumePlaySession_Clicked();
	static UNREALED_API void PausePlaySession_Clicked();
	static UNREALED_API void SingleFrameAdvance_Clicked();

	static UNREALED_API bool IsInSIE();
	static UNREALED_API bool IsInPIE();

	static UNREALED_API bool IsInSIE_AndRunning();
	static UNREALED_API bool IsInPIE_AndRunning();

	static UNREALED_API bool HasPlayWorld();
	static UNREALED_API bool HasPlayWorldAndPaused();
	static UNREALED_API bool HasPlayWorldAndRunning();
};
