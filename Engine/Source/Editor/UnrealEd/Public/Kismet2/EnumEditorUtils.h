// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Engine/UserDefinedEnum.h"
#include "HAL/Platform.h"
#include "Kismet2/ListenerManager.h"
#include "UObject/NameTypes.h"

class FArchive;
class FText;
class UEnum;
class UObject;
class UUserDefinedEnum;

class FEnumEditorUtils
{
	static void PrepareForChange(UUserDefinedEnum* Enum);
	static void BroadcastChanges(const UUserDefinedEnum* Enum, const TArray<TPair<FName, int64>>& OldNames, bool bResolveData = true);

	/** copy full enumeratos names from given enum to OutEnumNames, the last '_MAX' enumerator is skipped */
	static void CopyEnumeratorsWithoutMax(const UEnum* Enum, TArray<TPair<FName, int64>>& OutEnumNames);
public:

	enum EEnumEditorChangeInfo
	{
		Changed,
	};

	class FEnumEditorManager : public FListenerManager<UUserDefinedEnum, EEnumEditorChangeInfo>
	{
		FEnumEditorManager() {}
	public:
		UNREALED_API static FEnumEditorManager& Get();

		class ListenerType : public InnerListenerType<FEnumEditorManager>
		{
		};
	};

	typedef FEnumEditorManager::ListenerType INotifyOnEnumChanged;

	//////////////////////////////////////////////////////////////////////////
	// User defined enumerations

	/** Creates new user defined enum in given blueprint. */
	static UNREALED_API UEnum* CreateUserDefinedEnum(UObject* InParent, FName EnumName, EObjectFlags Flags);

	/** return if an enum can be named/renamed with given name*/
	static UNREALED_API bool IsNameAvailebleForUserDefinedEnum(FName Name);

	/** Updates enumerators names after name or path of the Enum was changed */
	static UNREALED_API void UpdateAfterPathChanged(UEnum* Enum);

	/** adds new enumerator (with default unique name) for user defined enum */
	static UNREALED_API void AddNewEnumeratorForUserDefinedEnum(class UUserDefinedEnum* Enum);

	/** Removes enumerator from enum*/
	static UNREALED_API void RemoveEnumeratorFromUserDefinedEnum(class UUserDefinedEnum* Enum, int32 EnumeratorIndex);

	/**
	 * Reorder enumerators in enum. Moves the enumerator at the given initial index to a new target index, shifting other enumerators as needed.
	 * E.g. with enum [A, B, C, D, E], moving index 1 to index 3 results in [A, C, D, B, E].
	 */
	static UNREALED_API void MoveEnumeratorInUserDefinedEnum(class UUserDefinedEnum* Enum, int32 InitialEnumeratorIndex, int32 TargetIndex);

	/** Check if the enumerator-as-bitflags meta data is set */
	static UNREALED_API bool IsEnumeratorBitflagsType(class UUserDefinedEnum* Enum);

	/** Set the state of the enumerator-as-bitflags meta data */
	static UNREALED_API void SetEnumeratorBitflagsTypeState(class UUserDefinedEnum* Enum, bool bBitflagsType);

	/** check if NewName is a short name and is acceptable as name in given enum */
	static UNREALED_API bool IsProperNameForUserDefinedEnumerator(const UEnum* Enum, FString NewName);

	/** Handles necessary notifications when the Enum has had a transaction undone or redone on it. */
	static UNREALED_API void PostEditUndo(UUserDefinedEnum* Enum);

	/*
	 *	Try to update an out-of-date enum index after an enum's change
	 *
	 *	@param Enum - new version of enum
	 *	@param Ar - special archive
	 *	@param EnumeratorIndex - old enumerator index
	 *
	 *	@return new enum 
	 */
	static UNREALED_API int64 ResolveEnumerator(const UEnum* Enum, FArchive& Ar, int64 EnumeratorValue);

	//DISPLAY NAME
	static UNREALED_API bool SetEnumeratorDisplayName(UUserDefinedEnum* Enum, int32 EnumeratorIndex, FText NewDisplayName);
	static UNREALED_API bool IsEnumeratorDisplayNameValid(const UUserDefinedEnum* Enum, int32 EnumeratorIndex, FText NewDisplayName);
	static UNREALED_API void EnsureAllDisplayNamesExist(class UUserDefinedEnum* Enum);
	static UNREALED_API void UpgradeDisplayNamesFromMetaData(class UUserDefinedEnum* Enum);
};
