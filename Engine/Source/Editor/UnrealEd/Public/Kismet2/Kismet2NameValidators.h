// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"

class INameValidatorInterface;
class UBlueprint;

// Eventually we want this to be the same as INVALID_OBJECTNAME_CHARACTERS, except we might allow spaces
// but for now it only includes "." as that has known failure modes (FindObject<T> will try to interpret
// that as a packagename.objectname)
#define UE_BLUEPRINT_INVALID_NAME_CHARACTERS TEXT(".")

//////////////////////////////////////////////////////////////////////////
// FNameValidtorFactory
class FNameValidatorFactory
{
public:
	static UNREALED_API TSharedPtr<class INameValidatorInterface> MakeValidator(class UEdGraphNode* Node);
};

enum class EValidatorResult
{
	/** Name is valid for this object */
	Ok,
	/** The name is already in use and invalid */
	AlreadyInUse,
	/** The entered name is blank */
	EmptyName,
	/* The entered name matches the current name */
	ExistingName,
	/* The entered name is too long */
	TooLong,
	/* The entered name contains invalid characters (see INVALID_OBJECTNAME_CHARACTERS, except for space) */
	ContainsInvalidCharacters,
	/** The entered is in use locally */
	LocallyInUse
};


//////////////////////////////////////////////////////////////////////////
// FNameValidatorInterface

class INameValidatorInterface
{
public:
	virtual ~INameValidatorInterface() = default;

	/** @return true if FName is valid, false otherwise */
	virtual EValidatorResult IsValid (const FName& Name, bool bOriginal = false) = 0;

	/** @return true if FString is valid, false otherwise */
	virtual EValidatorResult IsValid (const FString& Name, bool bOriginal = false) = 0;

	/** @return A text string describing the type of error in ErrorCode for Name */
	static UNREALED_API FText GetErrorText(const FString& Name, EValidatorResult ErrorCode);

	/** @return A string describing the type of error in ErrorCode for Name */
	static FString GetErrorString(const FString& Name, EValidatorResult ErrorCode)
	{
		return GetErrorText(Name, ErrorCode).ToString();
	}

	/** @return Ok if was valid and doesn't require anything, AlreadyInUse if was already in use and is replaced with new one */
	UNREALED_API EValidatorResult FindValidString(FString& InOutName);

	/** 
	 * Helper method to see if object exists with this name in the blueprint. Useful for 
	 * testing for name conflicts with objects create with Blueprint as their outer
	 *
	 * @param Blueprint	The blueprint to check
	 * @param Name		The name to check for conflicts
	 *
	 * @return True if name is not used by object in the blueprint, False otherwise
	 */
	static UNREALED_API bool BlueprintObjectNameIsUnique(class UBlueprint* Blueprint, const FName& Name);
};

/////////////////////////////////////////////////////
// FKismetNameValidator
class FKismetNameValidator : public INameValidatorInterface
{
public:
	UNREALED_API FKismetNameValidator(const class UBlueprint* Blueprint, FName InExistingName = NAME_None, const UStruct* InScope = nullptr);
	~FKismetNameValidator() = default;

	/** Return the name validator maximum string length */
	static UNREALED_API int32 GetMaximumNameLength();

	// Begin FNameValidatorInterface
	UNREALED_API virtual EValidatorResult IsValid( const FString& Name, bool bOriginal = false) override;
	UNREALED_API virtual EValidatorResult IsValid( const FName& Name, bool bOriginal = false) override;
	// End FNameValidatorInterface
private:
	/** Name set to validate */
	TSet<FName> Names;
	/** The blueprint to check for validity within */
	const UBlueprint* BlueprintObject;
	/** The current name of the object being validated */
	FName ExistingName;
	/** Scope to check against local variables (or other members) */
	const UStruct* Scope;
};

/////////////////////////////////////////////////////
// FStringSetNameValidator

class FStringSetNameValidator : public INameValidatorInterface
{
public:
	// Begin FNameValidatorInterface
	UNREALED_API virtual EValidatorResult IsValid(const FString& Name, bool bOriginal) override;
	UNREALED_API virtual EValidatorResult IsValid(const FName& Name, bool bOriginal) override;
	// End FNameValidatorInterface

	// This class is a base class for anything that just needs to validate a string is unique
	FStringSetNameValidator(const FString& InExistingName)
		: ExistingName(InExistingName)
	{
	}

protected:
	/* Name set to validate */
	TSet<FString> Names;
	FString ExistingName;
};

/////////////////////////////////////////////////////
// FAnimStateTransitionNodeSharedRulesNameValidator

class FAnimStateTransitionNodeSharedRulesNameValidator : public FStringSetNameValidator
{
public:
	UNREALED_API FAnimStateTransitionNodeSharedRulesNameValidator(class UAnimStateTransitionNode* InStateTransitionNode);
};

/////////////////////////////////////////////////////
// FAnimStateTransitionNodeSharedCrossfadeNameValidator

class FAnimStateTransitionNodeSharedCrossfadeNameValidator : public FStringSetNameValidator
{
public:
	UNREALED_API FAnimStateTransitionNodeSharedCrossfadeNameValidator(class UAnimStateTransitionNode* InStateTransitionNode);
};

/////////////////////////////////////////////////////
// FDummyNameValidator

// Always returns the same value
class FDummyNameValidator : public INameValidatorInterface
{
public:
	FDummyNameValidator(EValidatorResult InReturnValue) : ReturnValue(InReturnValue) {}

	// Begin FNameValidatorInterface
	virtual EValidatorResult IsValid(const FString& Name, bool bOriginal) override { return ReturnValue; }
	virtual EValidatorResult IsValid(const FName& Name, bool bOriginal) override { return ReturnValue; }
	// End FNameValidatorInterface

private:
	EValidatorResult ReturnValue;
};
