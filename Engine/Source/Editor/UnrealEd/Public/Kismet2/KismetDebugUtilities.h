// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UObject/Object.h"
#include "UObject/Class.h"
#include "Engine/Selection.h"

#include "KismetDebugUtilities.generated.h"

static_assert(DO_BLUEPRINT_GUARD, "KismetDebugUtilities assumes BP exception tracking is enabled");

class UBlueprint;
class UEdGraphPin;
struct FBlueprintBreakpoint;
struct FBlueprintWatchedPin;
template<typename ElementType> class TSimpleRingBuffer;

DECLARE_LOG_CATEGORY_EXTERN(LogBlueprintDebug, Log, All);

//////////////////////////////////////////////////////////////////////////
// EBlueprintBreakpointReloadMethod

/** Indicates how to restore breakpoints when a Blueprint asset is reloaded. */
UENUM()
enum class EBlueprintBreakpointReloadMethod
{
	/** Restore all breakpoints and keep their saved enabled/disabled state. */
	RestoreAll,
	/** Restore all breakpoints and disable on reload. */
	RestoreAllAndDisable,
	/** Discard all breakpoints on reload. */
	DiscardAll
};

//////////////////////////////////////////////////////////////////////////
// FKismetTraceSample

struct FKismetTraceSample
{
	TWeakObjectPtr<class UObject> Context;
	TWeakObjectPtr<class UFunction> Function;
	int32 Offset;
	double ObservationTime;
};

//////////////////////////////////////////////////////////////////////////
// FDebugInfo

// call FKismetDebugUtilities::GetDebugInfo to construct
struct FPropertyInstanceInfo : TSharedFromThis<FPropertyInstanceInfo>
{
	/**
	 * used to determine whether an object's property has
	 * been visited yet when generating FDebugInfo
	 */
	struct FPropertyInstance
	{
		const FProperty* Property;
		const void* Value;
	};
	
	/**
	* Helper constructor. call FKismetDebugUtilities::GetDebugInfo or
	* FKismetDebugUtilities::GetDebugInfoInternal instead
	*/
    FPropertyInstanceInfo(FPropertyInstance PropertyInstance, const TSharedPtr<FPropertyInstanceInfo>& Parent = nullptr);

	/**
	 * Constructs a Shared FPropertyInstanceInfo
	 */
	static TSharedPtr<FPropertyInstanceInfo> Make(FPropertyInstance PropertyInstance, const TSharedPtr<FPropertyInstanceInfo>& Parent);

	/**
	* populates a FDebugInfo::Children with sub-properties (non-recursive)
	*/
	void PopulateChildren(FPropertyInstance PropertyInstance);

	/** Resolves the PathToProperty treating this PropertyInstance as the head of the path */
	UNREALED_API TSharedPtr<FPropertyInstanceInfo> ResolvePathToProperty(const TArray<FName>& InPathToProperty);

	/** Returns the watch text for info popup bubbles on the graph */
	UNREALED_API FString GetWatchText() const;

	/** Returns children of this node (generating them if necessary) */
	UNREALED_API const TArray<TSharedPtr<FPropertyInstanceInfo>>& GetChildren();

	FText Name;
	FText DisplayName;
	FText Value;
	FText Type;
	TWeakObjectPtr<UObject> Object = nullptr; // only filled if property is a UObject
	TFieldPath<const FProperty> Property;
	bool bIsInContainer = false;
	int32 ContainerIndex = INDEX_NONE;

private:
	// Warning: only the head ValueAddress in the tree is guaranteed to be valid. Call GetPropertyInstance() instead of
	// using ValueAddress directly.
	const void* ValueAddress;
	FPropertyInstance GetPropertyInstance() const;
	
	TWeakPtr<FPropertyInstanceInfo> Parent;
	TArray<TSharedPtr<FPropertyInstanceInfo>> Children;
};

inline uint32 GetTypeHash(const FPropertyInstanceInfo::FPropertyInstance& PropertyInstance)
{
	return HashCombine(GetTypeHash(PropertyInstance.Property), GetTypeHash(PropertyInstance.Value));
}

inline bool operator==(const FPropertyInstanceInfo::FPropertyInstance& A, const FPropertyInstanceInfo::FPropertyInstance& B)
{
	return A.Property == B.Property && A.Value == B.Value;
}


//////////////////////////////////////////////////////////////////////////
// FObjectsBeingDebuggedIterator

// Helper class to iterate over all objects that should be visible in the debugger
struct FObjectsBeingDebuggedIterator
{
public:
	UNREALED_API FObjectsBeingDebuggedIterator();

	/** @name Element access */
	//@{
	UNREALED_API UObject* operator* () const;
	UNREALED_API UObject* operator-> () const;
	//@}

	/** Advances iterator to the next element in the container. */
	UNREALED_API FObjectsBeingDebuggedIterator& operator++();

	/** conversion to "bool" returning true if the iterator has not reached the last element. */
	FORCEINLINE explicit operator bool() const
	{ 
		return IsValid(); 
	}
	/** inverse of the "bool" operator */
	FORCEINLINE bool operator !() const 
	{
		return !(bool)*this;
	}

private:
	FSelectionIterator SelectedActorsIter;
	int32 LevelScriptActorIndex;
private:
	UNREALED_API void FindNextLevelScriptActor();
	UNREALED_API bool IsValid() const;
	UNREALED_API UWorld* GetWorld() const;
};



//////////////////////////////////////////////////////////////////////////
// FObjectsBeingDebuggedIterator

// Helper class to iterate over all objects that should be visible in the debugger
struct FBlueprintObjectsBeingDebuggedIterator
{
public:
	UNREALED_API FBlueprintObjectsBeingDebuggedIterator(UBlueprint* InBlueprint);

	/** @name Element access */
	//@{
	UNREALED_API UObject* operator* () const;
	UNREALED_API UObject* operator-> () const;
	//@}

	/** Advances iterator to the next element in the container. */
	UNREALED_API FBlueprintObjectsBeingDebuggedIterator& operator++();

	/** conversion to "bool" returning true if the iterator has not reached the last element. */
	FORCEINLINE explicit operator bool() const
	{ 
		return IsValid(); 
	}
	/** inverse of the "bool" operator */
	FORCEINLINE bool operator !() const 
	{
		return !(bool)*this;
	}

private:
	UBlueprint* Blueprint;
private:
	UNREALED_API bool IsValid() const;
};

//////////////////////////////////////////////////////////////////////////
// FKismetDebugUtilities

class FKismetDebugUtilities
{
public:
	// Delegate for when pins are added or removed from the watchlist
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnWatchedPinsListChanged, class UBlueprint*);

	static UNREALED_API void OnScriptException(const UObject* ActiveObject, const FFrame& StackFrame, const FBlueprintExceptionInfo& Info);

	/** Returns the current instruction; if a debugging session has started; otherwise none */
	static UNREALED_API class UEdGraphNode* GetCurrentInstruction();

	/** Returns the most recent hit breakpoint; if a debugging session has started; otherwise none */
	static UNREALED_API class UEdGraphNode* GetMostRecentBreakpointHit();

	/** Returns the most recent hit breakpoint; if a debugging session has started in PIE/SIE; otherwise none */
	static UNREALED_API UWorld* GetCurrentDebuggingWorld();

	/** Request abort the current frame execution */
	static UNREALED_API void RequestAbortingExecution();

	/** Request an attempt to single-step to the next node */
	static UNREALED_API void RequestSingleStepIn();
	
	/** Request an attempt to step over to the next node in this graph or the calling graph */
	static UNREALED_API void RequestStepOver();

	/** Request an attempt to step out of the current graph */
	static UNREALED_API void RequestStepOut();

	/** Called on terminatation of the current script execution so we can reset any break conditions */
	static UNREALED_API void EndOfScriptExecution(const FBlueprintContextTracker& BlueprintContext);

	// The maximum number of trace samples to gather before overwriting old ones
	enum { MAX_TRACE_STACK_SAMPLES = 1024 };

	// Get the trace stack
	static UNREALED_API const TSimpleRingBuffer<FKismetTraceSample>& GetTraceStack();

	// Find the node that resulted in code at the specified location in the Object, or NULL if there was a problem (e.g., no debugging information was generated)
	static UNREALED_API class UEdGraphNode* FindSourceNodeForCodeLocation(const UObject* Object, const UFunction* Function, int32 DebugOpcodeOffset, bool bAllowImpreciseHit = false);

	// Return proper class for breakpoint
	static UNREALED_API const UClass* FindClassForNode(const UObject* Object, const UFunction* Function);

	// Notify the debugger of the start of the game frame
	static UNREALED_API void NotifyDebuggerOfStartOfGameFrame(UWorld* CurrentWorld);

	// Notify the debugger of the end of the game frame
	static UNREALED_API void NotifyDebuggerOfEndOfGameFrame(UWorld* CurrentWorld);

	// Whether or not we are single stepping
	static UNREALED_API bool IsSingleStepping();

	// Breakpoint utils

	/** Is the node a valid breakpoint target? (i.e., the node is impure and ended up generating code) */
	static UNREALED_API bool IsBreakpointValid(const FBlueprintBreakpoint& Breakpoint);

	/** Set the node that the breakpoint should focus on */
	static UNREALED_API void SetBreakpointLocation(FBlueprintBreakpoint& Breakpoint, UEdGraphNode* NewNode);

	/** Set or clear the enabled flag for the breakpoint */
	static UNREALED_API void SetBreakpointEnabled(FBlueprintBreakpoint& Breakpoint, bool bIsEnabled);
	static UNREALED_API void SetBreakpointEnabled(const UEdGraphNode* OwnerNode, const UBlueprint* OwnerBlueprint, bool bIsEnabled);

	/** Sets this breakpoint up as a single-step breakpoint (will disable or delete itself after one go if the breakpoint wasn't already enabled) */
	static UNREALED_API void SetBreakpointEnabledForSingleStep(FBlueprintBreakpoint& Breakpoint, bool bDeleteAfterStep);

	/** Reapplies the breakpoint (used after recompiling to ensure it is set if needed) */
	static UNREALED_API void ReapplyBreakpoint(FBlueprintBreakpoint& Breakpoint);

	/** Start the process of deleting this breakpoint */
	static UNREALED_API void RemoveBreakpointFromNode(const UEdGraphNode* OwnerNode, const UBlueprint* OwnerBlueprint);

	/** Update the internal state of the breakpoint when it got hit */
	static UNREALED_API void UpdateBreakpointStateWhenHit(const UEdGraphNode* OwnerNode, const UBlueprint* OwnerBlueprint);

	/** Returns the installation site(s); don't cache these pointers! */
	static UNREALED_API void GetBreakpointInstallationSites(const FBlueprintBreakpoint& Breakpoint, TArray<uint8*>& InstallSites);

	/** Install/uninstall the breakpoint into/from the script code for the generated class that contains the node */
	static UNREALED_API void SetBreakpointInternal(FBlueprintBreakpoint& Breakpoint, bool bShouldBeEnabled);

	/** Returns the set of valid macro source node breakpoint location(s) for the given macro instance node. The set may be empty. */
	static UNREALED_API void GetValidBreakpointLocations(const class UK2Node_MacroInstance* MacroInstanceNode, TArray<const UEdGraphNode*>& BreakpointLocations);

	/** Adds a breakpoint to the provided node */
	static UNREALED_API void CreateBreakpoint(const UBlueprint* Blueprint, UEdGraphNode* Node, bool bIsEnabled = true);

	/**
	 * Performs a task on every breakpoint in the provided blueprint
	 * @param Blueprint The owning blueprint of the breakpoints to iterate
	 * @param Task function to be called on every element
	 */
	static UNREALED_API void ForeachBreakpoint(const UBlueprint* Blueprint, TFunctionRef<void(FBlueprintBreakpoint &)> Task);

	/**
	* Removes any breakpoint that matches the provided predicate
	* @param Blueprint The owning blueprint of the breakpoints to iterate
	* @param Predicate function that returns true if a breakpoint should be removed
	*/
	static UNREALED_API void RemoveBreakpointsByPredicate(const UBlueprint* Blueprint, const TFunctionRef<bool(const FBlueprintBreakpoint&)> Predicate);

	/**
	* Returns the first breakpoint that matches the provided predicate or nullptr if nothing matched
	* @param Blueprint The owning blueprint of the breakpoints to iterate
	* @param Predicate function that returns true for the found breakpoint
	*/
	static UNREALED_API FBlueprintBreakpoint* FindBreakpointByPredicate(const UBlueprint* Blueprint, const TFunctionRef<bool(const FBlueprintBreakpoint&)> Predicate);

	/** Queries whether a blueprint has breakpoints in it */
	static UNREALED_API bool BlueprintHasBreakpoints(const UBlueprint* Blueprint);

	/** Handles breakpoint validation/restoration after loading the given Blueprint */
	static UNREALED_API void RestoreBreakpointsOnLoad(const UBlueprint* Blueprint);
	
	// Blueprint utils 

	/** Duplicates debug data from original blueprint to new duplicated blueprint */
	static UNREALED_API void PostDuplicateBlueprint(UBlueprint* SrcBlueprint, UBlueprint* DupBlueprint, const TArray<UEdGraphNode*>& DupNodes);

	// Looks thru the debugging data for any class variables associated with the pin
	static UNREALED_API class FProperty* FindClassPropertyForPin(UBlueprint* Blueprint, const UEdGraphPin* Pin);

	// Looks thru the debugging data for any class variables associated with the node (e.g., temporary variables or timelines)
	static UNREALED_API class FProperty* FindClassPropertyForNode(UBlueprint* Blueprint, const UEdGraphNode* Node);

	// Is there debugging data available for this blueprint?
	static UNREALED_API bool HasDebuggingData(const UBlueprint* Blueprint);

	/** Determines if the given pin's value can be inspected */
	static UNREALED_API bool CanInspectPinValue(const UEdGraphPin* Pin);

	/** Returns the breakpoint associated with a node, or NULL */
	static UNREALED_API FBlueprintBreakpoint* FindBreakpointForNode(const UEdGraphNode* OwnerNode, const UBlueprint* OwnerBlueprint, bool bCheckSubLocations = false);

	/** Deletes all breakpoints in this blueprint */
	static UNREALED_API void ClearBreakpoints(const UBlueprint* Blueprint);
	static UNREALED_API void ClearBreakpointsForPath(const FString &BlueprintPath);

	// Notifies listeners when a watched pin is added or removed
	static UNREALED_API FOnWatchedPinsListChanged WatchedPinsListChangedEvent;

	/**
	 * Returns whether a pin property can be watched
	 * @param Blueprint The blueprint that owns the pin
	 * @param Pin The Pin to check if it can be watched
	 * @param InPathToProperty Path to the property on the pin to watch
	 */
	static UNREALED_API bool CanWatchPin(const UBlueprint* Blueprint, const UEdGraphPin* Pin, const TArray<FName>& InPathToProperty = TArray<FName>());

	/** 
	 * Returns whether a pin property is being watched
	 * @param Blueprint The blueprint that owns the pin
	 * @param Pin The Pin to check if it's being watched
	 * @param InPathToProperty Path to the property on the pin to watch
	 */
	static UNREALED_API bool IsPinBeingWatched(const UBlueprint* Blueprint, const UEdGraphPin* Pin, const TArray<FName>& InPathToProperty = TArray<FName>());
	
	/**
	 * Returns whether there are any watched properties for a given pin 
	 * @param Blueprint The blueprint that owns the pin
	 * @param Pin The Pin to check if it has any watches
	 */
	static UNREALED_API bool DoesPinHaveWatches(const UBlueprint* Blueprint, const UEdGraphPin* Pin);

	/**
	 * Toggles whether a pin is being watched 
	 * @param Blueprint The blueprint that owns the pin
	 * @param Pin The Pin to watch
	 */
	static UNREALED_API void TogglePinWatch(const UBlueprint* Blueprint, const UEdGraphPin* Pin);

	/**
	 * Removes a pin property Watch 
	 * @param Blueprint The blueprint that owns the pin
	 * @param Pin The Pin to stop watching
	 * @param InPathToProperty Path to the property on the pin to stop watching
	 * @return true if a watch was found and removed
	 */
	static UNREALED_API bool RemovePinWatch(const UBlueprint* Blueprint, const UEdGraphPin* Pin, const TArray<FName>& InPathToProperty = TArray<FName>());

	/** 
	 * Adds a pin property watch 
	 * @param Blueprint The blueprint that owns the pin
	 * @param WatchedPin The Pin to watch
	 */
	static UNREALED_API void AddPinWatch(const UBlueprint* Blueprint, FBlueprintWatchedPin&& WatchedPin);

	/** Removes all Watched pins from a blueprint */
	static UNREALED_API void ClearPinWatches(const UBlueprint* Blueprint);

	/** Returns whether any pins are watched for a Blueprint */
	static UNREALED_API bool BlueprintHasPinWatches(const UBlueprint* Blueprint);

	/**
	* Performs a task on every watched pin in the provided blueprint
	* @param Blueprint The owning blueprint of the watched pins to iterate
	* @param Task function to be called on every element
	*/
	static UNREALED_API void ForeachPinWatch(const UBlueprint* Blueprint, TFunctionRef<void(UEdGraphPin*)> Task);
	
	/**
	* Performs a task on every watched pin in the provided blueprint
	* @param Blueprint The owning blueprint of the watched pins to iterate
	* @param Task function to be called on every element
	*/
	static UNREALED_API void ForeachPinPropertyWatch(const UBlueprint* Blueprint, TFunctionRef<void(FBlueprintWatchedPin&)> Task);

	/**
	* Removes any watched pin that matches the provided predicate
	* @param Blueprint The owning blueprint of the watched pins to iterate
	* @param Predicate function that returns true if a watched pin should be removed
	*/
	static UNREALED_API bool RemovePinWatchesByPredicate(const UBlueprint* Blueprint, const TFunctionRef<bool(const UEdGraphPin*)> Predicate);
	
	/**
	* Removes any watched pin that matches the provided predicate
	* @param Blueprint The owning blueprint of the watched pins to iterate
	* @param Predicate function that returns true if a watched pin should be removed
	*/
	static UNREALED_API bool RemovePinPropertyWatchesByPredicate(const UBlueprint* Blueprint, const TFunctionRef<bool(const FBlueprintWatchedPin&)> Predicate);

	/**
	* Returns the first watched pin that matches the provided predicate or nullptr if nothing matched
	* @param Blueprint The owning blueprint of the watched pins to iterate
	* @param Predicate function that returns true for the found watched pin
	*/
	static UNREALED_API UEdGraphPin* FindPinWatchByPredicate(const UBlueprint* Blueprint, const TFunctionRef<bool(const UEdGraphPin*)> Predicate);

	enum EWatchTextResult
	{
		// The property was valid and the value has been returned as a string
		EWTR_Valid,

		// The property is a local of a function that is not on the current stack
		EWTR_NotInScope,

		// There is no debug object selected
		EWTR_NoDebugObject,

		// There is no property related to the pin
		EWTR_NoProperty
	};

	// Gets the watched tooltip for a specified site
	static UNREALED_API EWatchTextResult GetWatchText(FString& OutWatchText, UBlueprint* Blueprint, UObject* ActiveObject, const UEdGraphPin* WatchPin);

	// Gets the debug info for a specified site
	static UNREALED_API EWatchTextResult GetDebugInfo(TSharedPtr<FPropertyInstanceInfo> &OutDebugInfo, UBlueprint* Blueprint, UObject* ActiveObject, const UEdGraphPin* WatchPin);

	// Retrieves Debug info from a Property and a pointer to it's associated data
	static UNREALED_API void GetDebugInfoInternal(TSharedPtr<FPropertyInstanceInfo> &DebugInfo, const FProperty* Property, const void* PropertyValue);

	//@TODO: Pretty lame way to handle this messaging, ideally the entire Info object gets pushed into the editor when intraframe debugging is triggered!
	// This doesn't work properly if there is more than one blueprint editor open at once either (one will consume it, the others will be left in the cold)
	static UNREALED_API FText GetAndClearLastExceptionMessage();
protected:
	static UNREALED_API void CheckBreakConditions(UEdGraphNode* NodeStoppedAt, bool bHitBreakpoint, int32 BreakpointOffset, bool& InOutBreakExecution);
	static UNREALED_API void AttemptToBreakExecution(UBlueprint* BlueprintObj, const UObject* ActiveObject, const FFrame& StackFrame, const FBlueprintExceptionInfo& Info, UEdGraphNode* NodeStoppedAt, int32 DebugOpcodeOffset);

	static UNREALED_API void GetDebugInfo_InContainer(int32 Index, TSharedPtr<FPropertyInstanceInfo> &DebugInfo, const FProperty* Property, const void* Data);

	/**
	* @brief	Helper function for converting between blueprint and debuggable data
	*			output params are only valid if the return result is EWatchTextResult::EWTR_Valid
	* 
	* @param 	Blueprint		Active blueprint that is being debugged
	* @param 	ActiveObject	Instance of the object that is being debugged
	* @param 	WatchPin		The pin where this debug breakpoint is from
	* @param 	OutProperty		Property of interest
	* @param 	OutData			Populated with the property address of interest
	* @param 	OutDelta		Populated with the same thing as OutData
	* @param 	OutParent		Populated with the active object
	* @param 	SeenObjects		Used to track what objects have been traversed to find the OutProperty address
	* @param 	bOutIsDirectPtr	True if OutData/OutDelta point directly at the property rather than its base address
	* @return	EWTR_Valid if the debug data could be found, otherwise an appropriate error code
	*/
	static UNREALED_API EWatchTextResult FindDebuggingData(UBlueprint* Blueprint, UObject* ActiveObject, const UEdGraphPin* WatchPin, const FProperty*& OutProperty, const void*& OutData, const void*& OutDelta, UObject*& OutParent, TArray<UObject*>& SeenObjects, bool* bOutIsDirectPtr = nullptr);

	/**	Retrieve the user settings associated with a blueprint.
	*	returns null if the blueprint has default settings (no breakpoints and no watches) */
	static UNREALED_API struct FPerBlueprintSettings* GetPerBlueprintSettings(const UBlueprint* Blueprint);

	/**	Retrieve the Array of breakpoints associated with a blueprint.
	*	returns null if there are no breakpoints associated with this blueprint */
	static UNREALED_API TArray<FBlueprintBreakpoint>* GetBreakpoints(const UBlueprint* Blueprint);

	/**	Retrieve the Array of watched pins associated with a blueprint.
	*	returns null if there are no watched pins associated with this blueprint */
	static UNREALED_API TArray<FBlueprintWatchedPin>* GetWatchedPins(const UBlueprint* Blueprint);
	
	/** Save any modifications made to breakpoints */
	static UNREALED_API void SaveBlueprintEditorSettings();

	static UNREALED_API void CleanupBreakpoints(const UBlueprint* Blueprint);
	static UNREALED_API void CleanupWatches(const UBlueprint* Blueprint);
	static UNREALED_API void RemoveEmptySettings(const FString& BlueprintPath);

	/** Looks along the outer chain to find the owning world and disallowing breaking on tracepoints for editor preview and inactive worlds. */
	static UNREALED_API bool TracepointBreakAllowedOnOwningWorld(const UObject* ObjOuter);

private:
	FKismetDebugUtilities() {}
};

//////////////////////////////////////////////////////////////////////////
