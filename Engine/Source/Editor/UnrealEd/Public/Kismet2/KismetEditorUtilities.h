// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "Templates/SubclassOf.h"
#include "GameFramework/Actor.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"

class UToolMenu;
class IBlueprintEditor;
class UEdGraph;
class USCS_Node;
struct Rect;

enum class EBlueprintBytecodeRecompileOptions
{
	None = 0x0,

	// in batch compile mode we don't 'BroadcastCompiled/BroadcastBlueprintCompiled'
	BatchCompile           = 0x1,
	// normally we create a REINST_ version even when doing the bytecode compilation
	// this flag can be used of the blueprints GeneratedClass is being reinstanced by 
	// calling code:
	SkipReinstancing	   = 0x2  
};

ENUM_CLASS_FLAGS(EBlueprintBytecodeRecompileOptions)

enum class EBlueprintCompileOptions
{
	None = 0x0,

	/** This flag has several effects, but its behavior is to 'make things work' when regenerating a blueprint on load */
	IsRegeneratingOnLoad = 0x1,
	/** Skips garbage collection at the end of compile, useful if caller will collect garbage themselves */
	SkipGarbageCollection = 0x2,
	/** Prevents intermediate products from being garbage collected, useful for debugging macro/node expansion */
	SaveIntermediateProducts = 0x4,
	/** Indicates that the skeleton is up to date, and therefore the skeleton compile pass can be skipped */
	SkeletonUpToDate = 0x8,
	/** Indicates this is a batch compile and that BroadcastCompiled and BroadcastBlueprintCompiled should be skipped */
	BatchCompile = 0x10,
	/** Skips saving blueprints even if save on compile is enabled */
	SkipSave = 0x20,
	/** Skips creating a reinstancer and running reinstancing routines - useful if calling code is performing reinstancing */
	SkipReinstancing = 0x40,
	/** Simply regenerates the skeleton class */
	RegenerateSkeletonOnly = 0x80,
	/** Skips class-specific validation of the default object - in some cases we may not have a fully-initialized CDO after reinstancing */
	SkipDefaultObjectValidation = 0x100,
	/** Skips Find-in-Blueprint search data update - in some cases (e.g. creating new assets) this is being deferred until after compilation */
	SkipFiBSearchMetaUpdate = 0x200,
	/** Allow the delta serialization during FBlueprintCompileReinstancer::CopyPropertiesForUnrelatedObjects */
	UseDeltaSerializationDuringReinstancing = 0x400,
	/** Skips the new variable defaults detection - in some cases we do not want to use the defaults from the generated class such as during a reparent */
	SkipNewVariableDefaultsDetection = 0x800,
	/** Directs reinstancing to find and replace external references to the regenerated CDO during reference replacement - in general, this is not needed */
	IncludeCDOInReferenceReplacement = 0x1000,
};

ENUM_CLASS_FLAGS(EBlueprintCompileOptions)

//////////////////////////////////////////////////////////////////////////
// FKismetEditorUtilities

class FKismetEditorUtilities
{
public:
	/** 
	 * Event that's broadcast anytime a Blueprint is created
	 */
	DECLARE_DELEGATE_OneParam(FOnBlueprintCreated, UBlueprint* /*InBlueprint*/);

	/** Manages the TargetClass and EventName to use for spawning default "ghost" nodes in a new Blueprint */
	struct FDefaultEventNodeData
	{
		/** If the new Blueprint is a child of the TargetClass an event will be attempted to be spawned.
		 *	Hiding the category and other things can prevent the event from being placed
		 */
		UClass* TargetClass;

		/** Event Name to spawn a node for */
		FName EventName;
	};

	/** Manages the TargetClass and EventName to use for spawning default "ghost" nodes in a new Blueprint */
	struct FOnBlueprintCreatedData
	{
		/** If the new Blueprint is a child of the TargetClass, the callback will be executed */
		UClass* TargetClass;

		/** Callback to execute */
		FOnBlueprintCreated OnBlueprintCreated;
	};

	/**
	 * Create a new Blueprint and initialize it to a valid state.
	 *
	 * @param ParentClass					the parent class of the new blueprint
	 * @param Outer							the outer object of the new blueprint
	 * @param NewBPName						the name of the new blueprint
	 * @param BlueprintType					the type of the new blueprint (normal, const, etc)
	 * @param CallingContext				the name of the calling method or module used to identify creation methods to engine analytics/usage stats (default None will be ignored)
	 * @return								the new blueprint
	 */
	static UNREALED_API UBlueprint* CreateBlueprint(UClass* ParentClass, UObject* Outer, const FName NewBPName, enum EBlueprintType BlueprintType, FName CallingContext = NAME_None);

	/**
	 * Create a new Blueprint and initialize it to a valid state.
	 *
	 * @param ParentClass					the parent class of the new blueprint
	 * @param Outer							the outer object of the new blueprint
	 * @param NewBPName						the name of the new blueprint
	 * @param BlueprintType					the type of the new blueprint (normal, const, etc)
	 * @param BlueprintClassType			the actual class of the blueprint asset (UBlueprint or a derived type)
	 * @param BlueprintGeneratedClassType	the actual generated class of the blueprint asset (UBlueprintGeneratedClass or a derived type)
	 * @param CallingContext				the name of the calling method or module used to identify creation methods to engine analytics/usage stats (default None will be ignored)
	 * @return								the new blueprint
	 */
	static UNREALED_API UBlueprint* CreateBlueprint(UClass* ParentClass, UObject* Outer, const FName NewBPName, enum EBlueprintType BlueprintType, TSubclassOf<UBlueprint> BlueprintClassType, TSubclassOf<UBlueprintGeneratedClass> BlueprintGeneratedClassType, FName CallingContext = NAME_None);

	/**
	 * Creates a user construction script graph for the blueprint.
	 *
	 * @param Blueprint					the blueprint
	 * @return							the new UCS Graph, does not register it.
	 */
	static UNREALED_API UEdGraph* CreateUserConstructionScript(UBlueprint* Blueprint);

	/** 
	 * Event that's broadcast anytime a blueprint is unloaded, and becomes 
	 * invalid (with calls to ReplaceBlueprint(), for example).
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnBlueprintUnloaded, UBlueprint*);
	static UNREALED_API FOnBlueprintUnloaded OnBlueprintUnloaded;

	/** Event that's broadcast anytime a blueprint generated class is unloaded */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnBlueprintGeneratedClassUnloaded, UBlueprintGeneratedClass*);
	static UNREALED_API FOnBlueprintGeneratedClassUnloaded OnBlueprintGeneratedClassUnloaded;

	/** 
	 * Unloads the specified Blueprint (marking it pending-kill, and removing it 
	 * from its outer package). Then proceeds to replace all references with a
	 * copy of the one passed.
	 *
	 * @param  Target		The Blueprint you want to unload and replace.
	 * @param  Replacement	The Blueprint you cloned and used to replace Target.
	 * @return The duplicated replacement Blueprint.
	 */
	static UNREALED_API UBlueprint* ReplaceBlueprint(UBlueprint* Target, UBlueprint const* Replacement);

	/** 
	 * Determines if the specified blueprint is referenced currently in the undo 
	 * buffer.
	 *
	 * @param  Blueprint	The Blueprint you want to query about.
	 * @return True if the Blueprint is saved in the undo buffer, false if not.
	 */
	static UNREALED_API bool IsReferencedByUndoBuffer(UBlueprint* Blueprint);

	/** Create the correct event graphs for this blueprint */
	static UNREALED_API void CreateDefaultEventGraphs(UBlueprint* Blueprint);

	/** Tries to compile a blueprint, updating any actors in the editor who are using the old class, etc... */
	static UNREALED_API void CompileBlueprint(UBlueprint* BlueprintObj, EBlueprintCompileOptions CompileFlags = EBlueprintCompileOptions::None, class FCompilerResultsLog* pResults = nullptr );

	/** Generates a blueprint skeleton only.  Minimal compile, no notifications will be sent, no GC, etc.  Only successful if there isn't already a skeleton generated */
	static UNREALED_API bool GenerateBlueprintSkeleton(UBlueprint* BlueprintObj, bool bForceRegeneration = false);

	/** Tries to make sure that a data-only blueprint is conformed to its native parent, in case any native class flags have changed */
	static UNREALED_API void ConformBlueprintFlagsAndComponents(UBlueprint* BlueprintObj);

	/** @return true is it's possible to create a blueprint from the specified class */
	static UNREALED_API bool CanCreateBlueprintOfClass(const UClass* Class);

	/** 
	 * Given an array of Actors, identifies which of those Actors are roots of attachment hierarchies (and implicitly which are attached to another actor in the array)
	 * Optionally will also populate an attachment map that indicates which actors in the array are attached to each other actor (potentially indirectly)
	 * For example if A is attached to B is attached to C and E is attached to D, and A, C, and E are in the Actors array, C and E will be in the RootActors, 
	 * and the AttachmentMap will indicate that C has A as an attachment 
	 */
	static UNREALED_API void IdentifyRootActors(const TArray<AActor*>& Actors, TArray<AActor*>& RootActors, TMap<AActor*, TArray<AActor*>>* AttachmentMap = nullptr);

	enum class EAddComponentToBPHarvestMode : uint8
	{
		/* Not harvesting */
		None,

		/* Harvesting, use the component name for the new component directly */
		Harvest_UseComponentName,

		/* Harvesting, new component name will be OwnerName_ComponentName */
		Havest_AppendOwnerName
	};

	/** Parameter struct for customizing calls to AddComponentsToBlueprint */
	struct FAddComponentsToBlueprintParams
	{
		FAddComponentsToBlueprintParams()
			: HarvestMode(EAddComponentToBPHarvestMode::None)
			, bKeepMobility(false)
			, OptionalNewRootNode(nullptr)
			, OutNodes(nullptr)
		{
		}

		/** Which harvest mode to use when harvesting the components in to the blueprint */
		EAddComponentToBPHarvestMode HarvestMode;

		/** Whether the components should keep their mobility or be adjusted to the new parent */
		bool bKeepMobility;

		/** Which SCSNode to attach the new components to, if null attachment will be to Root */
		USCS_Node* OptionalNewRootNode;

		/** Optional pointer to an array for the caller to get a list of the created SCSNodes */
		TArray<USCS_Node*>* OutNodes;
	};

	/** Take a list of components that belong to a single Actor and add them to a blueprint as SCSNodes */
	static UNREALED_API void AddComponentsToBlueprint(UBlueprint* Blueprint, const TArray<UActorComponent*>& Components, const FAddComponentsToBlueprintParams& Params = FAddComponentsToBlueprintParams());

	/** Parameter struct for customizing calls to AddActorsToBlueprint */
	struct FAddActorsToBlueprintParams
	{
		FAddActorsToBlueprintParams()
			: bReplaceActors(true)
			, bDeferCompilation(false)
			, AttachNode(nullptr)
			, RelativeToInstance(nullptr)
		{
		}

		/** Whether the Actors being added to the blueprint should be deleted */
		bool bReplaceActors;

		/** Puts off compilation of the blueprint as additional manipulation is going to be done before it compiles */
		bool bDeferCompilation;

		/** Which SCSNode the ChildActorComponents should be attached to. If null, attachment will be to the Root */
		USCS_Node* AttachNode;

		/** An Actor in the level to use as the pivot point when setting the component's relative transform */
		AActor* RelativeToInstance;

		/** 
		 * If RelativeToInstance is null, RelativeToTransform is the WorldLocation Pivot
		 * If RelativeToInstance is non-null, RelativeToTransform is a relative transform to the instances WorldLocation
		 */
		FTransform RelativeToTransform;
	};

	/** Take a list of actors and add them to a blueprint as Child Actor Components */
	static UNREALED_API void AddActorsToBlueprint(UBlueprint* Blueprint, const TArray<AActor*>& Actors, const FAddActorsToBlueprintParams& Params = FAddActorsToBlueprintParams());

	/** Parameter struct for customizing calls to CreateBlueprintFromActor */
	struct FCreateBlueprintFromActorParams
	{
		FCreateBlueprintFromActorParams()
			: bReplaceActor(true)
			, bKeepMobility(false)
			, bDeferCompilation(false)
			, bOpenBlueprint(true)
			, ParentClassOverride(nullptr)
		{
		}

		/** If true, replace the actor in the scene with one based on the created blueprint */
		bool bReplaceActor;

		/** If true, The mobility of each actor components will be copied */
		bool bKeepMobility;

		/** Puts off compilation of the blueprint as additional manipulation is going to be done before it compiles */
		bool bDeferCompilation;

		/** Whether the newly created blueprint should be opened in the editor */
		bool bOpenBlueprint;

		/** The parent class to use when creating the blueprint. If null, the class of Actor will be used.  If specified, must be a subclass of the Actor's class */
		UClass* ParentClassOverride;
	};

	/**
	 * Take an Actor and generate a blueprint based on it. Uses the Actors type as the parent class.
	 * @param Path					The path to use when creating the package for the new blueprint
	 * @param Actor					The actor to use as the template for the blueprint
	 * @param Params				The parameter struct of additional behaviors
	 * @return The blueprint created from the actor
	 */
	static UNREALED_API UBlueprint* CreateBlueprintFromActor(const FString& Path, AActor* Actor, const FCreateBlueprintFromActorParams& Params = FCreateBlueprintFromActorParams());

	/** 
	 * Take an Actor and generate a blueprint based on it. Uses the Actors type as the parent class. 
	 * @param BlueprintName			The name to use for the Blueprint
	 * @param Outer					The outer object to create the blueprint within
	 * @param Actor					The actor to use as the template for the blueprint
	 * @param Params				The parameter struct of additional behaviors
	 * @return The blueprint created from the actor
	 */
	static UNREALED_API UBlueprint* CreateBlueprintFromActor(const FName BlueprintName, UObject* Outer, AActor* Actor, const FCreateBlueprintFromActorParams& Params = FCreateBlueprintFromActorParams());

	/** Parameter struct for customizing calls to CreateBlueprintFromActors */
	struct FCreateBlueprintFromActorsParams
	{
		FCreateBlueprintFromActorsParams(const TArray<AActor*>& Actors)
			: RootActor(nullptr)
			, AdditionalActors(Actors)
			, bReplaceActors(true)
			, bDeferCompilation(false)
			, bOpenBlueprint(true)
			, ParentClass(AActor::StaticClass())
		{
		}

		FCreateBlueprintFromActorsParams(AActor* RootActor, const TArray<AActor*>& ChildActors)
			: RootActor(RootActor)
			, AdditionalActors(ChildActors)
			, bReplaceActors(true)
			, bDeferCompilation(false)
			, bOpenBlueprint(true)
			, ParentClass(RootActor->GetClass())
		{
		}

		/** Optional Actor to use as the template for the blueprint */
		AActor* RootActor;

		/** The actors to use when creating child actor components */
		const TArray<AActor*>& AdditionalActors;

		/** If true, replace the actors in the scene with one based on the created blueprint */
		bool bReplaceActors;

		/** Puts off compilation of the blueprint as additional manipulation is going to be done before it compiles */
		bool bDeferCompilation;

		/** Whether the newly created blueprint should be opened in the editor */
		bool bOpenBlueprint;

		/** The parent class to use when creating the blueprint. If a RootActor is specified, the class must be a subclass of the RootActor's class */
		UClass* ParentClass;
	};

	/**
	 * Take a collection of Actors and generate a blueprint based on it
	 * @param Path					The path to use when creating the package for the new blueprint
	 * @param Params				The parameter struct containing actors and additional behavior definitions
	 * @return The blueprint created from the actor
	 */
	static UNREALED_API UBlueprint* CreateBlueprintFromActors(const FString& Path, const FCreateBlueprintFromActorsParams& Params);

	/**
	 * Take a collection of Actors and generate a blueprint based on it
	 * @param BlueprintName			The name to use for the Blueprint
	 * @param Package				The package to create the blueprint within
	 * @param Params				The parameter struct containing actors and additional behavior definitions
	 * @return The blueprint created from the actor
	 */
	static UNREALED_API UBlueprint* CreateBlueprintFromActors(const FName BlueprintName, UPackage* Package, const FCreateBlueprintFromActorsParams& Params);

	/** Parameter struct for customizing calls to CreateBlueprintFromActor */
	struct FHarvestBlueprintFromActorsParams
	{
		FHarvestBlueprintFromActorsParams()
			: bReplaceActors(true)
			, bOpenBlueprint(true)
			, ParentClass(AActor::StaticClass())
		{
		}

		/** If true, replace the actors in the scene with one based on the created blueprint */
		bool bReplaceActors;

		/** Whether the newly created blueprint should be opened in the editor */
		bool bOpenBlueprint;

		/** The parent class to use when creating the blueprint. If a RootActor is specified, the class must be a subclass of the RootActor's class */
		UClass* ParentClass;
	};

	/**
	 * Take a list of Actors and generate a blueprint by harvesting the components they have. 
	 * @param Path					The path to use when creating the package for the new blueprint
	 * @param Actors				The actor list to use as the template for the new blueprint, typically this is the currently selected actors
	 * @param Params				The parameter struct containing actors and additional behavior definitions
	 * @return The blueprint created from the actors
	 */
	static UNREALED_API UBlueprint* HarvestBlueprintFromActors(const FString& Path, const TArray<AActor*>& Actors, const FHarvestBlueprintFromActorsParams& Params = FHarvestBlueprintFromActorsParams());

	/**
	 * Take a list of Actors and generate a blueprint by harvesting the components they have.
	 * @param BlueprintName			The name to use for the Blueprint
	 * @param Parackage				The package to create the blueprint within
	 * @param Actors				The actor list to use as the template for the new blueprint, typically this is the currently selected actors
	 * @param Params				The parameter struct containing actors and additional behavior definitions
	 * @return The blueprint created from the actors
	 */
	static UNREALED_API UBlueprint* HarvestBlueprintFromActors(const FName BlueprintName, UPackage* Package, const TArray<AActor*>& Actors, const FHarvestBlueprintFromActorsParams& Params = FHarvestBlueprintFromActorsParams());

	/**
	 * Take a list of Actors and update an existing Blueprint by harvesting the components they have. Essentially HarvestBlueprintFromActors, but 
	 * updates an existing Blueprint rather than creating a new one.
	 * @param Path					The path to the existing Blueprint
	 * @param Actors				The actor list to use as the template for the blueprint.
	 * @return The updated blueprint, or null if it failed somehow
	 */
	static UNREALED_API UBlueprint* UpdateExistingBlueprintFromActors(const FString& Path, const TArray<AActor*>& Actors);

	/**
	 * Updates this Actor's blueprint based on the actor itself. 
	 * @return The number of properties that changes in the blueprint.
	 */
	static UNREALED_API int32 ApplyInstanceChangesToBlueprint(AActor* Actor);

	/** 
	 * Creates a new blueprint instance and replaces the provided actor list with the new actor
	 * @param Blueprint             The blueprint class to create an actor instance from
	 * @param SelectedActors        The list of currently selected actors in the editor
	 * @param Location              The location of the newly created actor
	 * @param Rotator               The rotation of the newly created actor
	 * @param AttachParent          The actor the newly created instance should be attached to if any
	 */
	static UNREALED_API AActor* CreateBlueprintInstanceFromSelection(class UBlueprint* Blueprint, const TArray<AActor*>& SelectedActors, const FVector& Location, const FRotator& Rotator, AActor* AttachParent = nullptr);

	/** 
	 * Create a new Blueprint from the supplied base class. Pops up window to let user select location and name.
	 *
	 * @param InWindowTitle			The window title
	 * @param InParentClass			The class to create a Blueprint based on
	 */
	static UNREALED_API UBlueprint* CreateBlueprintFromClass(FText InWindowTitle, UClass* InParentClass, FString NewNameSuggestion = TEXT(""));

	/** Create a new Actor Blueprint and add the supplied asset to it. */
	static UNREALED_API UBlueprint* CreateBlueprintUsingAsset(UObject* Asset, bool bOpenInEditor);

	/** Open a Kismet window, focusing on the specified object (either a node, or a graph).  Prefers existing windows, but will open a new application if required. */
	static UNREALED_API void BringKismetToFocusAttentionOnObject(const UObject* ObjectToFocusOn, bool bRequestRename=false);

	/** Open a Kismet window, focusing on the specified pin.  Prefers existing windows, but will open a new application if required. */
	static UNREALED_API void BringKismetToFocusAttentionOnPin(const UEdGraphPin* PinToFocusOn );

	/** Open level script kismet window and show any references to the selected actor */
	static UNREALED_API void ShowActorReferencesInLevelScript(const AActor* Actor);

	/** Upgrade any cosmetically stale information in a blueprint (done when edited instead of PostLoad to make certain operations easier)
		@returns True if blueprint modified, False otherwise */
	static UNREALED_API void UpgradeCosmeticallyStaleBlueprint(UBlueprint* Blueprint);

	/** Create a new event node in the level script blueprint, for the supplied Actor and event (multicast delegate property) name */
	static UNREALED_API void CreateNewBoundEventForActor(AActor* Actor, FName EventName, bool bShouldJumpToNode = true);

	/** Create a new event node in the  blueprint, for the supplied component, event name and blueprint */
	static UNREALED_API void CreateNewBoundEventForComponent(UObject* Component, FName EventName, UBlueprint* Blueprint, FObjectProperty* ComponentProperty, bool bShouldJumpToNode = true);

	/** Create a new event node in the  blueprint, for the supplied class, event name and blueprint */
	static UNREALED_API void CreateNewBoundEventForClass(UClass* Class, FName EventName, UBlueprint* Blueprint, FObjectProperty* ComponentProperty, bool bShouldJumpToNode = true);

	/** Can we paste to this graph? */
	static UNREALED_API bool CanPasteNodes(const class UEdGraph* Graph);

	/** Perform paste on graph, at location */
	static UNREALED_API void  PasteNodesHere( class UEdGraph* Graph, const UE::Slate::FDeprecateVector2DParameter& Location);

	/** Attempt to get the bounds for currently selected nodes
		@returns false if no nodes are selected */
	static UNREALED_API bool GetBoundsForSelectedNodes(const class UBlueprint* Blueprint, class FSlateRect& Rect, float Padding = 0.0f);

	static UNREALED_API int32 GetNumberOfSelectedNodes(const class UBlueprint* Blueprint);

	/** Find the event node for this actor with the given event name */
	static UNREALED_API const class UK2Node_ActorBoundEvent* FindBoundEventForActor(AActor const* Actor, FName EventName);

	/** Find the event node for the component property with the given event name */
	static UNREALED_API const class UK2Node_ComponentBoundEvent* FindBoundEventForComponent(const UBlueprint* Blueprint, FName EventName, FName PropertyName);

	/** Finds all bound component nodes for the given property on this blueprint */
	static UNREALED_API void FindAllBoundEventsForComponent(const UBlueprint* Blueprint, FName PropertyName, TArray<UK2Node_ComponentBoundEvent*>& OutNodes);

	/** Returns true if the given property name has any bound component events in any blueprint graphs */
	static UNREALED_API bool PropertyHasBoundEvents(const UBlueprint* Blueprint, FName PropertyName);

	/** Checks to see if the class is an interface class of any type, including native interfaces that are blueprint accessible */
	static UNREALED_API bool IsClassABlueprintInterface(const UClass* Class);

	/** Checks to see if a given class is implementable by any blueprints, if false a native class needs to implement it */
	static UNREALED_API bool IsClassABlueprintImplementableInterface(const UClass* Class);

	/** Checks to see if a blueprint can implement the specified class as an interface */
	static UNREALED_API bool CanBlueprintImplementInterface(UBlueprint const* Blueprint, UClass const* Class);

	/** Check to see if a given class is blueprint skeleton class. */
	static UNREALED_API bool IsClassABlueprintSkeleton (const UClass* Class);

	/** Check to see if a given class is blueprint spawnable component class. */
	static UNREALED_API bool IsClassABlueprintSpawnableComponent(const UClass* Class);

	/** Check to see if a given class is a blueprint macro library */
	static UNREALED_API bool IsClassABlueprintMacroLibrary(const UClass* Class);

	/** Run over the components in the blueprint, and then remove any that fall outside this blueprint's scope (e.g. components brought over after reparenting from another class) */
	static UNREALED_API void StripExternalComponents(class UBlueprint* Blueprint);

	/** Whether or not the specified actor is a valid target for bound events */
	static UNREALED_API bool IsActorValidForLevelScript(const AActor* Actor);

	/** 
	 *	if bCouldAddAny is true it returns if any event can be bound in LevelScript for given Actor
	 *	else it returns if there exists any event in level script bound with the actor
	 */
	static UNREALED_API bool AnyBoundLevelScriptEventForActor(AActor* Actor, bool bCouldAddAny);

	/** It lists bounded LevelScriptEvents for given actor */
	static UNREALED_API void AddLevelScriptEventOptionsForActor(UToolMenu* Menu, TWeakObjectPtr<AActor> ActorPtr, bool bExistingEvents, bool bNewEvents, bool bOnlyEventName);
	
	/** Return information about the given macro graph */
	static UNREALED_API void GetInformationOnMacro(UEdGraph* MacroGraph, /*out*/ class UK2Node_Tunnel*& EntryNode, /*out*/ class UK2Node_Tunnel*& ExitNode, bool& bIsPure);
	
	/** 
	 * Add information about any interfaces that have been implemented to the OutTags array
	 *
	 * @param	Blueprint		Blueprint to harvest interface data from
	 * @param	OutTags			Array to add tags to
	 */
	static UNREALED_API void AddInterfaceTags(const UBlueprint* Blueprint, TArray<UObject::FAssetRegistryTag>& OutTags);

	/**
	 * Add a default event node to the graph, this node will also be in a disabled state and will spawn
	 * with a call to it's parent if available
	 *
	 * @param InBlueprint		Blueprint this event will be a part of
	 * @param InGraph			The graph to spawn the event node in
	 * @param InEventName		The name of the event function
	 * @param InEventClass		The class this event can be found in
	 * @param InOutNodePosY		Position to spawn the node at, will return with an offset more suitable to offset the next node
	 * @return					The K2Node_Event will be returned
	 */
	static UNREALED_API class UK2Node_Event* AddDefaultEventNode(UBlueprint* InBlueprint, UEdGraph* InGraph, FName InEventName, UClass* InEventClass, int32& InOutNodePosY);

	/**
	 * Will add an event to the list of default event nodes to be auto-generated for the class or a child of the class
	 *
	 * @param InOwner			Method of look-up so these registrations can later be removed when unregistering the Owner.
	 * @param InTargetClass		If a new Blueprint is a child of the target class, the event will attempt to be placed in the main event graph
	 * @param InEventName		Name of event to place
	 */
	static UNREALED_API void RegisterAutoGeneratedDefaultEvent(void* InOwner, UClass* InTargetClass, FName InEventName);

	/**
	 * Will add an event to a list of callbacks to occur post Blueprint creation if the Blueprint is a child of the class
	 *
	 * @param InOwner							Method of look-up so these registrations can later be removed when unregistering the Owner.
	 * @param InTargetClass						If a new Blueprint is a child of the target class, the event will attempt to be placed in the main event graph
	 * @param InOnBlueprintCreatedCallback		Callback to call when the Blueprint is created
	 */
	static UNREALED_API void RegisterOnBlueprintCreatedCallback(void* InOwner, UClass* InTargetClass, FOnBlueprintCreated InOnBlueprintCreatedCallback);

	/** Unregisters a class from having AutoGeneratedDefaultEvent nodes or callbacks for OnBlueprintCreated */
	static UNREALED_API void UnregisterAutoBlueprintNodeCreation(void* InOwner);

	/** Add InNode to selection of editor */
	static UNREALED_API void AddToSelection(const class UEdGraph* Graph, UEdGraphNode* InNode);

	/** Get IBlueprintEditor for given object, if it exists */
	static UNREALED_API TSharedPtr<class IBlueprintEditor> GetIBlueprintEditorForObject(const UObject* ObjectToFocusOn, bool bOpenEditor);
private:
	/** Stores whether we are already listening for kismet clicks */
	static UNREALED_API bool bIsListeningForClicksOnKismetLog;

	/** List of blueprint parent class names cached by IsTrackedBlueprintParent() */
	static UNREALED_API TArray<FString> TrackedBlueprintParentList;

	/** Mapping of classes to names of Events that should be automatically spawned */
	static UNREALED_API TMultiMap<void*, FDefaultEventNodeData> AutoGeneratedDefaultEventsMap;

	/** Mapping of classes to delegate callbacks when a Blueprint is created, occurs post Event node creation */
	static UNREALED_API TMultiMap<void*, FOnBlueprintCreatedData> OnBlueprintCreatedCallbacks;
private:
	
	/**
	 * Attempts to decide whether a blueprint's parent class is suitable for tracking via analytics.
	 *
	 * @param ParentClass	The parent class to check
	 * 
	 * @return	True if the parent class is one we wish to track by reporting creation of children to analytics, otherwise false
	 */
	static UNREALED_API bool IsTrackedBlueprintParent(const UClass* ParentClass);

	FKismetEditorUtilities() {}
};

