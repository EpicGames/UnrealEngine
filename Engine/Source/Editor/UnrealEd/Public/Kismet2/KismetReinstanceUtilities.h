// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "Templates/SubclassOf.h"
#include "Components/ActorComponent.h"
#include "UObject/GCObject.h"

class FReinstanceFinalizer;
class UBlueprint;
class FEditorCacheArchetypeManager;
struct FObjectInstancingGraph;

DECLARE_STATS_GROUP(TEXT("Kismet Reinstancer"), STATGROUP_KismetReinstancer, STATCAT_Advanced);

enum class EBlueprintCompileReinstancerFlags
{
	None = 0x0,

	BytecodeOnly			= 0x1,
	AutoInferSaveOnCompile	= 0x2,
	AvoidCDODuplication		= 0x4,
	UseDeltaSerialization	= 0x8,
};

ENUM_CLASS_FLAGS(EBlueprintCompileReinstancerFlags)

struct FReplaceInstancesOfClassParameters
{
	UE_NONCOPYABLE(FReplaceInstancesOfClassParameters)

	FReplaceInstancesOfClassParameters() = default;

	UE_DEPRECATED(5.2, "Please use the default constructor instead.")
	FReplaceInstancesOfClassParameters(UClass* InOldClass, UClass* InNewClass) {}

	/** The old class, instances of which will be replaced */
	UE_DEPRECATED(5.2, "This member is no longer in use.")
	UClass* OldClass = nullptr;

	/** The new class, used to create new instances to replace instances of the old class */
	UE_DEPRECATED(5.2, "This member is no longer in use.")
	UClass* NewClass = nullptr;

	/** OriginalCDO, use if OldClass->ClassDefaultObject has been overwritten (non-batch only, legacy) */
	UObject* OriginalCDO = nullptr;

	/* Mapping of all the replaced CDOs and archetypes*/
	TMap<UClass*, TMap<UObject*, UObject*>>* OldToNewTemplates = nullptr;

	/** Set of objects that should not have their references updated if they refer to instances that are replaced */
	TSet<UObject*>* ObjectsThatShouldUseOldStuff = nullptr;

	/** Set of objects for which new objects should not be created, useful if client has created new instances themself */
	const TSet<UObject*>* InstancesThatShouldUseOldClass = nullptr;

	/** Hashes identifying original class's construction version, if there's a match we can skip reconstruction */
	TMap<TObjectPtr<const UClass>, FBlake3Hash>* ConstructionVersioningData = nullptr;

	/** Set to true if class object has been replaced (non-batch only, legacy) */
	bool bClassObjectReplaced = false; 

	/** Defaults to true, indicates whether root components should be preserved */
	bool bPreserveRootComponent = true;

	bool bArchetypesAreUpToDate = false;

	/**
	 * Blueprints reuses its UClass* from compile to compile, but it's more
	 * intuitive to just replace a UClass* with a new instance (e.g. from a
	 * package reload). This flag tells the reinstancer to replace references
	 * to old classes with references to new classes.
	 */
	bool bReplaceReferencesToOldClasses = false;

	/**
	 * Indicates whether CDOs should be included in reference replacement.
	 * Disabled by default since this can increase search cost, because this
	 * effectively means replacement will require us to do a referencer search
	 * for at least one instance of every remapped class (i.e. - the CDO). In
	 * most cases (e.g. load time), incurring this cost is unnecessary because
	 * the CDO is not expected to be referenced outside of its own class type,
	 * so we treat it as an opt-in behavior rather than enabling it by default.
	 * 
	 * Note: This flag is implied to be 'true' if 'OriginalCDO' is non-NULL. In
	 * that case, the value of this flag will not be used in order to maintain
	 * backwards-compatibility with existing code paths.
	 */
	bool bReplaceReferencesToOldCDOs = false;
};

struct UNREALED_API UE_DEPRECATED(5.2, "This type is no longer in use.") FBatchReplaceInstancesOfClassParameters
{
	bool bArchetypesAreUpToDate = false;

	/**
	 * Blueprints reuses its UClass* from compile to compile, but it's more
	 * intuitive to just replace a UClass* with a new instance (e.g. from a
	 * package reload). This flag tells the reinstancer to replace references
	 * to old classes with references to new classes.
	 */
	bool bReplaceReferencesToOldClasses = false;
};

class FBlueprintCompileReinstancer : public TSharedFromThis<FBlueprintCompileReinstancer>, public FGCObject
{
protected:
	friend struct FBlueprintCompilationManagerImpl;
	friend struct FRigVMBlueprintCompilationManagerImpl;
	friend struct FReinstancingJob;

	static UNREALED_API TSet<TWeakObjectPtr<UBlueprint>> CompiledBlueprintsToSave;

	static UNREALED_API UClass* HotReloadedOldClass;
	static UNREALED_API UClass* HotReloadedNewClass;

	/** Reference to the class we're actively reinstancing */
	UClass* ClassToReinstance;

	/** Reference to the duplicate of ClassToReinstance, which all previous instances are now instances of */
	TObjectPtr<UClass> DuplicatedClass;

	/** The original CDO object for the class being actively reinstanced */
	TObjectPtr<UObject>	OriginalCDO;

	/** The original Sparse Class Data object for the class being actively reinstanced */
	void* OriginalSCD;

	/** The original SDO Struct for the class being actively reinstanced */
	TObjectPtr<UScriptStruct> OriginalSCDStruct;

	/** A snapshot of the SCD, currently delta serialized from its archetype - taken before ownership of SCDs is taken */
	TArray<uint8> SCDSnapshot;

	/** Children of this blueprint, which will need to be recompiled and relinked temporarily to maintain the class layout */
	TArray<UBlueprint*> Children;

	/** Mappings from old fields before recompilation to their new equivalents */
	TMap<FName, FProperty*> PropertyMap;
	TMap<FName, TObjectPtr<UFunction>> FunctionMap;

	/** Whether or not this resinstancer has already reinstanced  */
	bool bHasReinstanced;

	/** Cached value, mostly used to determine if we're explicitly targeting the skeleton class or not */
	enum EReinstClassType
	{
		RCT_Unknown,
		RCT_BpSkeleton,
		RCT_BpGenerated,
		RCT_Native,
	};
	EReinstClassType ReinstClassType;

	uint32 ClassToReinstanceDefaultValuesCRC;

	/** Objects that should keep reference to old class */
	TSet<UObject*> ObjectsThatShouldUseOldStuff;

	/** TRUE if this is the root reinstancer that all other active reinstancing is spawned from */
	bool bIsRootReinstancer;

	/** TRUE if this reinstancer should resave compiled Blueprints if the user has requested it */
	bool bAllowResaveAtTheEndIfRequested;

	/** TRUE if delta serialization should be forced during FBlueprintCompileReinstancer::CopyPropertiesForUnrelatedObjects */
	bool bUseDeltaSerializationToCopyProperties;

public:
	// FSerializableObject interface
	UNREALED_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FBlueprintCompileReinstancer");
	}
	// End of FSerializableObject interface

	static UNREALED_API void OptionallyRefreshNodes(UBlueprint* BP);

	UNREALED_API virtual void BlueprintWasRecompiled(UBlueprint* BP, bool bBytecodeOnly);

	static TSharedPtr<FBlueprintCompileReinstancer> Create(UClass* InClassToReinstance, EBlueprintCompileReinstancerFlags Flags = EBlueprintCompileReinstancerFlags::AutoInferSaveOnCompile)
	{
		return MakeShareable(new FBlueprintCompileReinstancer(InClassToReinstance, Flags));
	}

	UNREALED_API virtual ~FBlueprintCompileReinstancer();

	/** Saves a mapping of field names to their UField equivalents, so we can remap any bytecode that references them later */
	UNREALED_API void SaveClassFieldMapping(UClass* InClassToReinstance);

	/** Helper to gather mappings from the old class's fields to the new class's version */
	UNREALED_API void GenerateFieldMappings(TMap<FFieldVariant, FFieldVariant>& FieldMapping);

	/** Reinstances all objects in the ObjectReinstancingMap */
	UNREALED_API void ReinstanceObjects(bool bForceAlwaysReinstance = false);

	/** Updates references to properties and functions of the class that has in the bytecode of dependent blueprints */
	UNREALED_API void UpdateBytecodeReferences( TSet<UBlueprint*>& OutDependentBlueprints, TMap<FFieldVariant, FFieldVariant>& OutFieldMapping);

	/** Populates SCDSnapshot, for use via PropagateSparseClassDataToNewClass */
	UNREALED_API void SaveSparseClassData(const UClass* ForClass);

	/** Instructs the reinstancer to take ownership of the sparse class data - doing so will make it difficult to identify archetype data */ 
	UNREALED_API void TakeOwnershipOfSparseClassData(UClass* ForClass);

	/** Copies an owned sparse class data instance to a new class and frees any owned SCD */
	UNREALED_API void PropagateSparseClassDataToNewClass(UClass* NewClass);

	/** Consumes the set and map populated by calls to UpdateBytecodeReferences */
	static UNREALED_API void FinishUpdateBytecodeReferences( const TSet<UBlueprint*>& DependentBPs, const TMap<FFieldVariant, FFieldVariant>& FieldMappings);

	// @todo_deprecated - To be removed in a future release.
	UE_DEPRECATED(5.2, "Please use the version that takes an FReplaceInstancesOfClassParameters input instead.")
	static UNREALED_API void ReplaceInstancesOfClass(UClass* OldClass, UClass* NewClass, UObject* OriginalCDO = nullptr, TSet<UObject*>* ObjectsThatShouldUseOldStuff = nullptr, bool bClassObjectReplaced = false, bool bPreserveRootComponent = true);
	UE_DEPRECATED(5.2, "Please use ReplaceInstancesOfClass() instead.")
	static UNREALED_API void ReplaceInstancesOfClassEx(const FReplaceInstancesOfClassParameters& Parameters );
	UE_DEPRECATED(5.2, "Please use the version that takes an FReplaceInstancesOfClassParameters input instead.")
	static UNREALED_API void BatchReplaceInstancesOfClass(TMap<UClass*, UClass*>& InOldToNewClassMap,
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const FBatchReplaceInstancesOfClassParameters& BatchParams = FBatchReplaceInstancesOfClassParameters()
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	);

	/** Replace all instances of OldClass with a new instance of NewClass */
	static UNREALED_API void ReplaceInstancesOfClass(UClass* OldClass, UClass* NewClass, const FReplaceInstancesOfClassParameters& Params);

	/** Batch replaces a mapping of one or more classes to their new class */
	static UNREALED_API void BatchReplaceInstancesOfClass(const TMap<UClass*, UClass*>& InOldToNewClassMap, const FReplaceInstancesOfClassParameters& Params);
	
	/** Function used to safely discard a CDO, so that the class can have its layout changed, callers must move parent CDOs aside before moving child CDOs aside: */
	static UNREALED_API UClass* MoveCDOToNewClass(UClass* OwnerClass, const TMap<UClass*, UClass*>& OldToNewMap, bool bAvoidCDODuplication);

	/**
	* Moves CDOs aside to immutable versions of classes(`REINST`) so that the CDO's can safely be GC'd.
	* These `REINST` classes will be re-parented to a native parent that we know will not be churning
	* through this function again later, so we avoid O(N^2) processing of REINST classes.
	* Maps each given `SKEL` class to its appropriate `REINST` version of itself
	*/
	static UNREALED_API void MoveDependentSkelToReinst(UClass* OwnerClass, TMap<UClass*, UClass*>& OldToNewMap);

	/** Gathers the full class Hierarchy of the ClassToSearch, sorted top down (0 index being UObject, n being the subclasses) */
	static UNREALED_API void GetSortedClassHierarchy(UClass* ClassToSearch, TArray<UClass*>& OutHierarchy, UClass** OutNativeParent);

	/** Returns true if the given class is a REINST class (starts with the 'REINST_' prefix) */
	static UNREALED_API bool IsReinstClass(const UClass* Class);

	/**
	 * When re-instancing a component, we have to make sure all instance owners' 
	 * construction scripts are re-ran (in-case modifying the component alters 
	 * the construction of the actor).
	 * 
	 * @param  ComponentClass    Identifies the component that was altered (used to find all its instances, and thusly all instance owners).
	 */
	static UNREALED_API void ReconstructOwnerInstances(TSubclassOf<UActorComponent> ComponentClass);
	
	/** Verify that all instances of the duplicated class have been replaced and collected */
	UNREALED_API void VerifyReplacement();

	virtual bool IsClassObjectReplaced() const { return false; }

	UNREALED_API void FinalizeFastReinstancing(TArray<UObject*>& ObjectsToReplace);
protected:

	UNREALED_API TSharedPtr<FReinstanceFinalizer> ReinstanceInner(bool bForceAlwaysReinstance);

	UNREALED_API TSharedPtr<FReinstanceFinalizer> ReinstanceFast();

	UNREALED_API void CompileChildren();

	bool IsReinstancingSkeleton() const { return (ReinstClassType == RCT_BpSkeleton); }

	/** Default constructor, can only be used by derived classes */
	FBlueprintCompileReinstancer()
		: ClassToReinstance(NULL)
		, DuplicatedClass(nullptr)
		, OriginalCDO(nullptr)
		, bHasReinstanced(false)
		, ReinstClassType(RCT_Unknown)
		, ClassToReinstanceDefaultValuesCRC(0)
		, bIsRootReinstancer(false)
	{}

	/** 
	 * Sets the reinstancer up to work on every object of the specified class
	 *
	 * @param InClassToReinstance		Class being reinstanced
	 * @param bIsBytecodeOnly			TRUE if only the bytecode is being recompiled
	 * @param bSkipGC					TRUE if garbage collection should be skipped
	 * @param bAutoInferSaveOnCompile	TRUE if the reinstancer should infer whether or not save on compile should occur, FALSE if it should never save on compile
	 */
	UNREALED_API FBlueprintCompileReinstancer(UClass* InClassToReinstance, EBlueprintCompileReinstancerFlags Flags = EBlueprintCompileReinstancerFlags::AutoInferSaveOnCompile);

	/** Reparents the specified blueprint or class to be the duplicated class in order to allow properties to be copied from the previous CDO to the new one */
	UNREALED_API void ReparentChild(UBlueprint* ChildBP);
	UNREALED_API void ReparentChild(UClass* ChildClass);

	/** Determine whether reinstancing actors should preserve the root component of the new actor */
	virtual bool ShouldPreserveRootComponentOfReinstancedActor() const { return true; }

public:
	/**
	* Attempts to copy as many properties as possible from the old object to the new. 
	* Use during BP compilation to copy properties from the old CDO to the new one.
	* 
	* @param OldObject						The old object to copy properties from
	* @param NewObject						The new Object to copy properties to
	* @param bClearExternalReferences		If true then attempt to replace references to old classes and instances on this object with the corresponding new ones
	* @param bForceDeltaSerialization		If true the delta serialization will be used when copying
	* @param bOnlyHandleDirectSubObjects	If true will only copy/handle immediate subobjects
	* @param OldToNewInstanceMap			If != null it will be used to replace any references found in this object
	* @param OldToNewClassMap				if != null it will be used to replace any class references found in this object
	*/
	static UNREALED_API void CopyPropertiesForUnrelatedObjects(UObject* OldObject, UObject* NewObject, bool bClearExternalReferences, bool bForceDeltaSerialization = false, bool bOnlyHandleDirectSubObjects = false, TMap<UObject*, UObject*>* OldToNewInstanceMap =nullptr, const TMap<UClass*,UClass*>* OldToNewClassMap =nullptr);

	/**
	 * This method will pre-create all non-default sub object needed for a re-instantiation, what is left is to CopyPropertiesForUnrelatedObjects on the created instances map to finish the re-instancing
	 * If the re-instancing is done in a big batch and part to the sub object might already be re-instantiated, you will need to provide those via the OldToNewInstanceMap
	 * 
	 * @param OldToNewClassMap in case there are subobject that will need new class type
	 * @param OldObject to pre-create its non-default sub objects
	 * @param NewUObject where to store those pre-created sub objects
	 * @param CreatedInstanceMap in/out of the result of all of the pre-created objects
	 * @param OldToNewInstanceMap optional parameter of the possible re-instanced sub objects if any
	 * @param OrderedListOfObjectToCopy List of object to copy in order
	 * @param InstancesThatShouldUseOldClass Subobjects that should keep their old class during reinstantiation, even if a new one is provided by OldToNewClassMap. This shouldn't be used for simple reinstantiation as the old class would be dead. One use-case would be changing the class of an object.
	 */
	static UNREALED_API void PreCreateSubObjectsForReinstantiation(const TMap<UClass*, UClass*>& OldToNewClassMap, UObject* OldObject, UObject* NewUObject, TMap<UObject*, UObject*>& CreatedInstanceMap, const TMap<UObject*, UObject*>* OldToNewInstanceMap = nullptr, TArray< TTuple<UObject*, UObject*>>* OrderedListOfObjectToCopy = nullptr, const TSet<UObject*>* InstancesThatShouldUseOldClass = nullptr);

private:
	struct FReinstancingState
	{
		TArray<UObject*>& ObjectsToReplaceScratch;
		TSet<UObject*>& CachedArchetypeObjects;
		TMap<UObject*, UObject*>& OldToNewInstanceMap;
		const TMap<UClass*, UClass*>& InOldToNewClassMap;
		TMap<UObject*, FName>& OldToNewNameMap;
		TMap<UClass*, TMap<UObject*, UObject*>>* OldToNewTemplates = nullptr;
		TArray<UObject*>& PotentialEditorsForRefreshing;
		TSet<AActor*>& OwnersToRerunConstructionScript;
		TArray<struct FActorReplacementHelper>& ReplacementActors;
		TMap<FSoftObjectPath, UObject*>& ReinstancedObjectsWeakReferenceMap;
		TMap<UObject*, FObjectInstancingGraph>& GraphCache;
		TMap<TObjectPtr<const UClass>, FBlake3Hash>* ConstructionVersioningData;
		TArray<UObject*>& ObjectsReplaced;
		TSet<UPackage*>& CleanPackageList;
		bool& bSelectionChanged;
		bool& bFixupSCS;
	};

	/** Helper to replace an instance of an object*/
	static UNREALED_API void ReplaceObjectHelper(const FReinstancingState& State, UObject*& OldObject, UClass* OldClass, UObject*& NewUObject, UClass* NewClass, int32 OldObjIndex, TArray<UObject*>& ObjectsToReplace, TFunctionRef<TArray<TObjectPtr<USceneComponent>>&(USceneComponent*)> GetAttachChildrenArray, bool bIsComponent, bool bArchetypesAreUpToDate, const TSet<UObject*>* InstancesThatShouldUseOldClass);

	/**
	 * Pre-populate instancing graph for a specified instance if needed
	 *
	 * @param GraphCache Cache of precomputed instance graphs for each top level object
	 * @param InstancingGraph The instancing graph to populate
	 * @param OldInstance Object that is about to be reallocated
	 * @param InArchetype Archetype of ObjectInstance
	 * @param OldToNewInstanceMap Possible instance mapping of instances
	 * @return false if the instancing graph is not needed
	 */
	static UNREALED_API bool PrePopulateInstancingGraphForArchetype(TMap<UObject*,FObjectInstancingGraph>& GraphCache, FObjectInstancingGraph& InstancingGraph, TNotNull<UObject*> OldInstance, UObject* InArchetype, TMap<UObject*, UObject*>* OldToNewInstanceMap = nullptr);


	/** Handles the sub object pre-creation recursively */
	static UNREALED_API void PreCreateSubObjectsForReinstantiation_Inner(const TSet<UObject*>* OldInstancedSubObjects, const TMap<UClass*, UClass*>& OldToNewClassMap, UObject* OldObject, UObject* NewUObject, TMap<UObject*, UObject*>& CreatedInstanceMap, const TMap<UObject*, UObject*>* OldToNewInstanceMap, TArray< TTuple<UObject*, UObject*>>* OrderedListOfObjectToCopy, const TSet<UObject*>* InstancesThatShouldUseOldClass);

	/** Handles the work of ReplaceInstancesOfClass, handling both normal replacement of instances and batch */
	static UNREALED_API void ReplaceInstancesOfClass_Inner(const TMap<UClass*, UClass*>& InOldToNewClassMap, const FReplaceInstancesOfClassParameters& Params);

	/** Returns true if A is higher up the class hierarchy  */
	static UNREALED_API bool ReinstancerOrderingFunction(UClass* A, UClass* B);
	
	/** Helper for caching archetypes before objects are reconstructed */
	static UNREALED_API void CacheArchetypes(	
		const TArray<UObject*>& ObjectsToReplace, 
		FEditorCacheArchetypeManager& CacheManager, 
		TSet<UObject*>& CachedArchetypeObjects );
	/** Returns a hash identifying the result of constructing an instance of the provided Class */
	static UNREALED_API FBlake3Hash CalculateConstructionHash(const UClass* ForClass);
	/** Returns the above, but only if there are instances of the class that would require reinstancing */
	static UNREALED_API FBlake3Hash CalculateConstructionHashIfNecessary(const UClass* ForClass);

	static void ReinstanceWithVersioningData(const FReinstancingState& ReinstancingState, const TSet<UObject*>* InstancesThatShouldUseOldClass);
	static void CategorizeClassesAndInstances(
		const TMap<UClass*, UClass*>& InOldToNewClassMap
		, TMap<UClass*, UClass*>& UnchangedClasses
		, TMap<UClass*, UClass*>& NormalReinstancingClasses
		, TMap<UClass*, UClass*>& ActiveActorReinstancingClasses
		, const FReinstancingState& ReinstancingState);
	static void RestoreClass(TMap<UClass*, UClass*>& OldToNewClassMap, const FReinstancingState& ReinstancingState);
	static void ReinstanceNormalObjects(TMap<UClass*, UClass*>& OldToNewClassMap, const FReinstancingState& ReinstancingState, const TSet<UObject*>* InstancesThatShouldUseOldClass);
	static void ReinstanceActors(TMap<UClass*, UClass*>& OldToNewClassMap, const FReinstancingState& ReinstancingState, const TSet<UObject*>* InstancesThatShouldUseOldClass);
	static void GetObjectsToReinstance(const UClass* OldClass, TArray<UObject*>& ObjectsToReplace);
	static void UpdateObjectBeingDebugged(UObject* InOldObject, UObject* InNewObject);
	static void CheckAndSaveOuterPackageToCleanList(TSet<UPackage*>& CleanPackageList, const UObject* InObject);
	static void CheckAndSaveAttachedActorsOuterPackageToCleanList(TSet<UPackage*>& CleanPackageList, const AActor* Actor);
	static bool ShouldReinstanceObject(
		const UObject* OldObject, const TMap<UClass*, UClass*>& InOldToNewClassMap, const bool bIsComponent);
	static bool ShouldReinstanceActor(const AActor* OldActor);
};
