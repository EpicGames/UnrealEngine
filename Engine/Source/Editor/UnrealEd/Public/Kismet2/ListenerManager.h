// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

template<typename TType, typename TChangeInfo>
class INotifyOnChanged
{
public:
	virtual void PreChange(const TType* Changed, TChangeInfo ChangedType) = 0;
	virtual void PostChange(const TType* Changed, TChangeInfo ChangedType) = 0;
};

template<typename TType, typename TChangeInfo>
class FListenerManager
{
public:
	typedef INotifyOnChanged<TType, TChangeInfo> BaseNotifyOnChanged;

	template<typename TManagerType>
	class InnerListenerType : public BaseNotifyOnChanged
	{
	public:
		InnerListenerType()
		{
			TManagerType::Get().AddListener(this);
		}
		virtual ~InnerListenerType()
		{
			TManagerType::Get().RemoveListener(this);
		}
	};

private:
	TSet<BaseNotifyOnChanged*> Listeners;

public:
	void AddListener(BaseNotifyOnChanged* Listener)
	{
		if (Listener)
		{
			Listeners.Add(Listener);
		}
	}

	void RemoveListener(BaseNotifyOnChanged* Listener)
	{
		Listeners.Remove(Listener);
	}

	void PreChange(const TType* Changed, TChangeInfo Info)
	{
		// Note: Not using range-based iteration here in case the listener mutates the list via add/remove.
		for (auto ListenerIt = Listeners.CreateConstIterator(); ListenerIt; ++ListenerIt)
		{
			(*ListenerIt)->PreChange(Changed, Info);
		}
	}

	void PostChange(const TType* Changed, TChangeInfo Info)
	{
		// Note: Not using range-based iteration here in case the listener mutates the list via add/remove.
		for (auto ListenerIt = Listeners.CreateConstIterator(); ListenerIt; ++ListenerIt)
		{
			(*ListenerIt)->PostChange(Changed, Info);
		}
	}
	
	const TSet<BaseNotifyOnChanged*>& GetListeners() const
	{
		return Listeners;
	}
};
