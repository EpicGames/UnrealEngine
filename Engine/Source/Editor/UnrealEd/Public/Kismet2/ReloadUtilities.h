// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ReloadUtilities.h: Helper utilities and classes for reloading
=============================================================================*/

#pragma once

#include "UObject/Reload.h"
#include "Containers/StringFwd.h"

class UBlueprint;
class UK2Node;

class FReload : public IReload
{
public:
	struct FNativeFuncPtrMapKeyFuncs : TDefaultMapKeyFuncs<FNativeFuncPtr, FNativeFuncPtr, false>
	{
		static FORCEINLINE uint32 GetKeyHash(FNativeFuncPtr Key)
		{
			return *(uint32*)&Key;
		}
	};

	using TFunctionRemap = TMap<FNativeFuncPtr, FNativeFuncPtr, FDefaultSetAllocator, FNativeFuncPtrMapKeyFuncs>;

public:

	UNREALED_API FReload(EActiveReloadType InType, const TCHAR* InPrefix, const TArray<UPackage*>& InPackages, FOutputDevice& InAr);
	UNREALED_API FReload(EActiveReloadType InType, const TCHAR* InPrefix, FOutputDevice& InAr);

	UNREALED_API virtual ~FReload();

	// IReload interface 
	virtual EActiveReloadType GetType() const override { return Type; }
	virtual const TCHAR* GetPrefix() const override { return Prefix; };
	UNREALED_API virtual bool GetEnableReinstancing(bool bHasChanged) const override;
	UNREALED_API virtual void NotifyFunctionRemap(FNativeFuncPtr NewFunctionPointer, FNativeFuncPtr OldFunctionPointer) override;
	UNREALED_API virtual void NotifyChange(UClass* New, UClass* Old) override;
	UNREALED_API virtual void NotifyChange(UEnum* New, UEnum* Old) override;
	UNREALED_API virtual void NotifyChange(UScriptStruct* New, UScriptStruct* Old) override;
	UNREALED_API virtual void NotifyChange(UPackage* New, UPackage* Old) override;
	UNREALED_API virtual void Reinstance() override;
	UNREALED_API virtual UObject* GetReinstancedCDO(UObject* CDO) override;
	UNREALED_API virtual const UObject* GetReinstancedCDO(const UObject* CDO) override;

	virtual const TMap<UClass*, UClass*>& GetReinstancedClasses() override
	{
		return ReinstancedClasses;
	}

	const TArray<UClass*>& GetNewClasses() override
	{
		return NewClasses;
	}

	/**
	 * If you wish to reuse the same reload object, invoke this method to reset the state
	 */
	UNREALED_API void Reset();

	/**
	 * Perform any finalize processing for reloading.
	 */
	UNREALED_API void Finalize(bool bRunGC = true);

	/**
	 * Set the sending of the complete notification
	 */
	void SetSendReloadCompleteNotification(bool bSend)
	{
		bSendReloadComplete = bSend;
	}

	/**
	 * Enable/Disable the support for reinstancing
	 */
	void SetEnableReinstancing(bool bInEnableReinstancing)
	{
		bEnableReinstancing = bInEnableReinstancing;
	}

	/**
	 * Return true if anything was re-instanced
	 */
	bool HasReinstancingOccurred() const
	{
		return bHasReinstancingOccurred;
	}

private:

	struct FReinstanceStats
	{
		int32 New = 0;
		int32 Changed = 0;
		int32 Unchanged = 0;

		bool HasValues() const
		{
			return New + Changed + Unchanged != 0;
		}

		bool HasReinstancingOccurred() const
		{
			return New + Changed != 0;
		}
	};

	struct FBlueprintUpdateInfo
	{
		TSet<UK2Node*> Nodes;
	};

	/**
	 * Finds all references to old CDOs and replaces them with the new ones.
	 * Skipping UBlueprintGeneratedClass::OverridenArchetypeForCDO as it's the
	 * only one needed.
	 */
	UNREALED_API void ReplaceReferencesToReconstructedCDOs();

	/**
	 * Based on the pointers, update the given stat
	 */
	UNREALED_API void UpdateStats(FReinstanceStats& Stats, void* New, void* Old);

	/**
	 * Helper method to format all the stats
	 */
	static UNREALED_API void FormatStats(FStringBuilderBase& Out, const TCHAR* Singular, const TCHAR* Plural, const FReinstanceStats& Stats);

	/**
	 * Helper method to format a specific stat
	 */
	static UNREALED_API void FormatStat(FStringBuilderBase& Out, const TCHAR* Singular, const TCHAR* Plural, const TCHAR* What, int32 Value);

	/** Type of the active reload */
	EActiveReloadType Type = EActiveReloadType::None;

	/** Prefix applied when renaming objects */
	const TCHAR* Prefix = nullptr;

	/** List of packages affected by the reload */
	TSet<UPackage*> Packages;
	
	/** Output device for any logging */
	FOutputDevice& Ar;

	/** Map from old function pointer to new function pointer for hot reload. */
	TFunctionRemap FunctionRemap;

	/** Map of the reconstructed CDOs during the reinstancing process */
	TMap<UObject*, UObject*> ReconstructedCDOsMap;

	/** Map of new CDOs where classes where changed */
	TMap<UObject*, UObject*> ReinstancedCDOsMap;

	/** Map from old class to new class.  New class may be null */
	TMap<UClass*, UClass*> ReinstancedClasses;

	/** Array of new classes */
	TArray<UClass*> NewClasses;

	/** Map from old struct to new struct.  New struct may be null */
	TMap<UScriptStruct*, UScriptStruct*> ReinstancedStructs;

	/** Map from old enum to new enum.  New enum may be null */
	TMap<UEnum*, UEnum*> ReinstancedEnums;

	/** If true, we have to collect the package list from the context */
	bool bCollectPackages;

	/** If true, send reload complete notification */
	bool bSendReloadComplete = true;

	/** If true, reinstancing is enabled */
	bool bEnableReinstancing = true;

	FReinstanceStats ClassStats;
	FReinstanceStats EnumStats;
	FReinstanceStats StructStats;
	FReinstanceStats PackageStats;
	int32 NumFunctionsRemapped = 0;
	int32 NumScriptStructsRemapped = 0;
	mutable bool bEnabledMessage = false;
	mutable bool bHasReinstancingOccurred = false;
};
