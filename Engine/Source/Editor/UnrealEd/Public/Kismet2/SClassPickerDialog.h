// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "ClassViewerModule.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Preferences/UnrealEdOptions.h"

class ITableRow;
class SClassViewer;
class STableViewBase;
class SWindow;

//////////////////////////////////////////////////////////////////////////
// SClassPickerDialog

class SClassPickerDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SClassPickerDialog )
		: _AssetType(NULL)
	{}

		SLATE_ARGUMENT( TSharedPtr<SWindow>, ParentWindow )
		SLATE_ARGUMENT( FClassViewerInitializationOptions, Options )
		SLATE_ARGUMENT( UClass*, AssetType )

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	UNREALED_API void Construct(const FArguments& InArgs);

	UNREALED_API static bool PickClass(const FText& TitleText, const FClassViewerInitializationOptions& ClassViewerOptions, UClass*& OutChosenClass, UClass* AssetType);

private:
	/** Handler for when a class is picked in the class picker */
	void OnClassPicked(UClass* InChosenClass);

	/** Creates the default class widgets */
	TSharedRef<ITableRow> GenerateListRow(TSharedPtr<FClassPickerDefaults> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Handler for when any of the default buttons are pressed **/
	FReply OnDefaultClassPicked(UClass* InChosenClass);

	/** Handler for when "Ok" we selected in the class viewer */
	FReply OnClassPickerConfirmed();

	/** Handler for when "Cancel" we selected in the class viewer */
	FReply OnClassPickerCanceled();

	/** Handler for the custom button to hide/unhide the default class viewer */
	void OnDefaultAreaExpansionChanged(bool bExpanded);

	/** Handler for the custom button to hide/unhide the class viewer */
	void OnCustomAreaExpansionChanged(bool bExpanded);

	/** select button visibility delegate */
	EVisibility GetSelectButtonVisibility() const;

	/** Overridden from SWidget: Called when a key is pressed down - capturing copy */
	FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent );

private:
	/** A pointer to the window that is asking the user to select a parent class */
	TWeakPtr<SWindow> WeakParentWindow;

	/** A pointer to a class viewer **/
	TSharedPtr<class SClassViewer> ClassViewer;

	/** The class that was last clicked on */
	UClass* ChosenClass;

	/** A flag indicating that Ok was selected */
	bool bPressedOk;

	/**  An array of default classes used in the dialog **/
	TArray< TSharedPtr<FClassPickerDefaults>  > AssetDefaultClasses;

	/** Determines if selecting no class is permitted **/
	bool bAllowNone;
};
