// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Kismet2/ListenerManager.h"

struct FEdGraphPinType;
struct FStructVariableDescription;
class UBlueprint;
class UUserDefinedStruct;

class FStructureEditorUtils
{
public:
	enum EStructureEditorChangeInfo
	{
		Unknown,
		AddedVariable,
		RemovedVariable,
		RenamedVariable,
		VariableTypeChanged,
		MovedVariable,
		DefaultValueChanged,
	};

	class FStructEditorManager : public FListenerManager<UUserDefinedStruct, EStructureEditorChangeInfo>
	{
		FStructEditorManager();
	public:
		UNREALED_API static FStructEditorManager& Get();

		class ListenerType : public InnerListenerType<FStructEditorManager>
		{
		};

		/** The current reason why a structure is being updated */
		UNREALED_API static EStructureEditorChangeInfo ActiveChange;
	};

	typedef FStructEditorManager::ListenerType INotifyOnStructChanged;

	template<class TElement>
	struct FFindByNameHelper
	{
		const FName Name;

		FFindByNameHelper(FName InName) : Name(InName) { }

		bool operator() (const TElement& Element) const
		{
			return (Name == Element.VarName);
		}
	};

	template<class TElement>
	struct FFindByGuidHelper
	{
		const FGuid Guid;

		FFindByGuidHelper(FGuid InGuid) : Guid(InGuid) { }

		bool operator() (const TElement& Element) const
		{
			return (Guid == Element.VarGuid);
		}
	};

	//STRUCTURE
	static UNREALED_API UUserDefinedStruct* CreateUserDefinedStruct(UObject* InParent, FName Name, EObjectFlags Flags);

	static UNREALED_API void CompileStructure(UUserDefinedStruct* Struct);

	static UNREALED_API FString GetTooltip(const UUserDefinedStruct* Struct);

	static UNREALED_API bool ChangeTooltip(UUserDefinedStruct* Struct, const FString& InTooltip);

	//VARIABLE
	static UNREALED_API bool AddVariable(UUserDefinedStruct* Struct, const FEdGraphPinType& VarType);

	static UNREALED_API bool RemoveVariable(UUserDefinedStruct* Struct, FGuid VarGuid);

	static UNREALED_API bool RenameVariable(UUserDefinedStruct* Struct, FGuid VarGuid, const FString& NewDisplayNameStr);

	static UNREALED_API bool RenameVariable(UUserDefinedStruct* Struct, const FString& OldDisplayNameStr, const FString& NewDisplayNameStr);

	static UNREALED_API bool ChangeVariableType(UUserDefinedStruct* Struct, FGuid VarGuid, const FEdGraphPinType& NewType);

	static UNREALED_API bool ChangeVariableDefaultValue(UUserDefinedStruct* Struct, FGuid VarGuid, const FString& NewDefaultValue);

	static UNREALED_API bool IsUniqueVariableFriendlyName(const UUserDefinedStruct* Struct, const FString& DisplayName);

	static UNREALED_API FString GetVariableFriendlyName(const UUserDefinedStruct* Struct, FGuid VarGuid);

	static UNREALED_API FString GetVariableFriendlyNameForProperty(const UUserDefinedStruct* Struct, const FProperty* Property);

	static UNREALED_API FProperty* GetPropertyByFriendlyName(const UUserDefinedStruct* Struct, FString DisplayName);

	static UNREALED_API FString GetVariableTooltip(const UUserDefinedStruct* Struct, FGuid VarGuid);
	
	static UNREALED_API bool ChangeVariableTooltip(UUserDefinedStruct* Struct, FGuid VarGuid, const FString& InTooltip);

	static UNREALED_API bool ChangeEditableOnBPInstance(UUserDefinedStruct* Struct, FGuid VarGuid, bool bInIsEditable);

	static UNREALED_API bool ChangeSaveGameEnabled(UUserDefinedStruct* Struct, FGuid VarGuid, bool bInSaveGame);

	enum EMovePosition
	{
		PositionAbove,
		PositionBelow,
	};

	/**
	 * Move the variable with the given guid in the struct to be immediately above or below another variable.
	 * 
	 * @param Struct         The struct containing the variable to move.
	 * @param MoveVarGuid    The guid of the variable being moved.
	 * @param RelativeToGuid The guid of the variable above/below which MoveVarGuid is being moved.
	 * @param Position       Whether to put MoveVarGuid above or below RelativeToGuid.
	 * 
	 * @return Whether the variable was actually moved.
	 */
	static UNREALED_API bool MoveVariable(UUserDefinedStruct* Struct, FGuid MoveVarGuid, FGuid RelativeToGuid, EMovePosition Position);

	/** Checks whether MoveVariable can actually move the variable with the given guid. */
	static UNREALED_API bool CanMoveVariable(UUserDefinedStruct* Struct, FGuid MoveVarGuid, FGuid RelativeToGuid, EMovePosition Position);

	//Multi-line text
	static UNREALED_API bool CanEnableMultiLineText(const UUserDefinedStruct* Struct, FGuid VarGuid);

	static UNREALED_API bool ChangeMultiLineTextEnabled(UUserDefinedStruct* Struct, FGuid VarGuid, bool bIsEnabled);

	static UNREALED_API bool IsMultiLineTextEnabled(const UUserDefinedStruct* Struct, FGuid VarGuid);

	//3D Widget
	static UNREALED_API bool CanEnable3dWidget(const UUserDefinedStruct* Struct, FGuid VarGuid);

	static UNREALED_API bool Change3dWidgetEnabled(UUserDefinedStruct* Struct, FGuid VarGuid, bool bIsEnabled);

	static UNREALED_API bool Is3dWidgetEnabled(const UUserDefinedStruct* Struct, FGuid VarGuid);

	//Value Range
	static UNREALED_API bool CanEditValueRange(const UUserDefinedStruct* Struct, FGuid VarGuid);

	//META DATA
	static UNREALED_API bool SetMetaData(UUserDefinedStruct* Struct, FGuid VarGuid, FName Key, const FString& Value);
	static UNREALED_API const FString* GetMetaData(const UUserDefinedStruct* Struct, FGuid VarGuid, FName Key);

	//GUID AND VAR DESC
	static UNREALED_API TArray<FStructVariableDescription>& GetVarDesc(UUserDefinedStruct* Struct);

	static UNREALED_API const TArray<FStructVariableDescription>& GetVarDesc(const UUserDefinedStruct* Struct);

	static UNREALED_API TArray<FStructVariableDescription>* GetVarDescPtr(UUserDefinedStruct* Struct);

	static UNREALED_API const TArray<FStructVariableDescription>* GetVarDescPtr(const UUserDefinedStruct* Struct);

	static UNREALED_API FStructVariableDescription* GetVarDescByGuid(UUserDefinedStruct* Struct, FGuid VarGuid);

	static UNREALED_API const FStructVariableDescription* GetVarDescByGuid(const UUserDefinedStruct* Struct, FGuid VarGuid);

	static UNREALED_API FGuid GetGuidForProperty(const FProperty* Property);

	static UNREALED_API FProperty* GetPropertyByGuid(const UUserDefinedStruct* Struct, FGuid VarGuid);

	static UNREALED_API FGuid GetGuidFromPropertyName(FName Name);

	//MISC
	static UNREALED_API void ModifyStructData(UUserDefinedStruct* Struct);

	static UNREALED_API bool UserDefinedStructEnabled();

	static UNREALED_API void RemoveInvalidStructureMemberVariableFromBlueprint(UBlueprint* Blueprint);

	//DEFAULT VALUE
	static UNREALED_API void RecreateDefaultInstanceInEditorData(UUserDefinedStruct* Struct);

	//VALIDATION
	static UNREALED_API bool CanHaveAMemberVariableOfType(const UUserDefinedStruct* Struct, const FEdGraphPinType& VarType, FString* OutMsg = NULL);

	enum EStructureError
	{
		Ok, 
		Recursion,
		FallbackStruct,
		NotCompiled,
		NotBlueprintType,
		NotSupportedType,
		EmptyStructure
	};

	/** Can the structure be a member variable for a BPGClass or BPGStruct */
	static UNREALED_API EStructureError IsStructureValid(const UScriptStruct* Struct, const UStruct* RecursionParent = NULL, FString* OutMsg = NULL);

	/** called after UDS was changed by editor*/
	static UNREALED_API void OnStructureChanged(UUserDefinedStruct* Struct, EStructureEditorChangeInfo ChangeReason = EStructureEditorChangeInfo::Unknown);

	static UNREALED_API void BroadcastPreChange(UUserDefinedStruct* Struct);
	static UNREALED_API void BroadcastPostChange(UUserDefinedStruct* Struct);
};

