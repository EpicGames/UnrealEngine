// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphPin.h"
#include "HAL/Platform.h"
#include "Misc/Guid.h"
#include "Templates/TypeHash.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPtr.h"

#include "WatchedPin.generated.h"

class UEdGraphNode;

/** Contains information about a watched pin in a Blueprint graph for local settings data.
 */
USTRUCT()
struct FBlueprintWatchedPin
{
	GENERATED_BODY()

	UNREALED_API FBlueprintWatchedPin();
	UNREALED_API FBlueprintWatchedPin(const UEdGraphPin* Pin);
	UNREALED_API FBlueprintWatchedPin(const UEdGraphPin* Pin, TArray<FName>&& InPathToProperty);

	/** Returns a reference to the underlying graph pin */
	UNREALED_API UEdGraphPin* Get() const;

	/** Returns a reference to the path to the property we're watching on this pin */
	const TArray<FName>& GetPathToProperty() const { return PathToProperty; }

	/** Resets the pin watch to the given graph pin */
	UNREALED_API void SetFromPin(const UEdGraphPin* Pin);

	/** Move another watched pin struct into this one */
	UNREALED_API void SetFromWatchedPin(FBlueprintWatchedPin&& Other);

	bool operator==(const FBlueprintWatchedPin& Other) const
	{
		return PinId == Other.PinId && OwningNode == Other.OwningNode && PathToProperty == Other.PathToProperty;
	}

	friend uint32 GetTypeHash(const FBlueprintWatchedPin& WatchedPin);

private:
	/** Node that owns the pin that the watch is placed on */
	UPROPERTY()
	TSoftObjectPtr<UEdGraphNode> OwningNode;

	/** Unique ID of the pin that the watch is placed on */
	UPROPERTY()
	FGuid PinId;

	/** Path from the pin to a nested property, empty if just watching the Pin
	 *  NOTE: each segment of the path is Property->GetAuthoredName
	 */
	UPROPERTY()
	TArray<FName> PathToProperty;

	/** Holds a cached reference to the underlying pin object. We don't save this directly to settings data,
	 *  because it internally maintains a weak object reference to the owning node that it will then try to
	 *  load after parsing the underlying value from the user's local settings file. To avoid issues and
	 *	overhead of trying to load referenced assets when reading the config file at editor startup, we
	 *  maintain our own soft object reference for the settings data instead. Additionally, we can add more
	 *  context this way without affecting other parts of the engine that rely on the pin reference type.
	 */
	mutable FEdGraphPinReference CachedPinRef;
};

FORCEINLINE uint32 GetTypeHash(const FBlueprintWatchedPin& WatchedPin)
{
	uint32 PathHash = 0;
	for (FName PathName : WatchedPin.PathToProperty)
	{
		PathHash = HashCombine(PathHash, GetTypeHash(PathName));
	}

	return HashCombine(GetTypeHash(WatchedPin.PinId), PathHash);
}
