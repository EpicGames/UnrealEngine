// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "UObject/NameTypes.h"

/**
* A collection of utilities for creating, using, and changing the value of wildcard pin types
* throughout the graph editor.
*/
class FWildcardNodeUtils
{
public:
	
	/** 
	* Gets the default wildcard pin type. Useful for doing comparisons on other pin types 
	* and checks during compilation of nodes.
	*/
	static UNREALED_API FEdGraphPinType GetDefaultWildcardPinType();
	
	/** 
	 * Checks if the pin has any wildcard components in its type. IsWildcardPin( const UEdGraphPin*) 
	 * will return false for pins that have only wildcard TMap Values, this will not.
	 * This function or IsWildcardPin(const FEdGraphTerminalType&) should be preferred.
	 */
	static UNREALED_API bool HasAnyWildcards(const UEdGraphPin* Pin);

	/**
	 * Returns true if the pin has any non wild cards (primary or secondary type)
	 */
	static UNREALED_API bool HasAnyNonWildcards(const UEdGraphPin* Pin);

	/**
	 * Returns a non wildcard pin from ForPin's LinkedTo list. Preferring pins
	 * with no wildcards to pins with some wildcards.
	 */
	static UNREALED_API const UEdGraphPin* FindInferrableLinkedPin( const UEdGraphPin* ForPin);

	/**
	* Checks if the given pin is in a wildcard state
	*
	* @param	Pin		The pin the consider
	* @return	True if the given pin is a Wildcard pin
	*/
	static UNREALED_API bool IsWildcardPin(const UEdGraphPin* const Pin);
	static UNREALED_API bool IsWildcardPin(const FEdGraphTerminalType& Terminal);

	/**
	* Checks if the given pin is linked to any wildcard pins
	* 
	* @return	True if the given pin is linked to any wildcard pins
	*/
	static UNREALED_API bool IsLinkedToWildcard(const UEdGraphPin* const Pin);

	/**
	* Add a default wildcard pin to the given node
	* 
	* @param Node				The node to add this pin to
	* @param PinName			Name of the given wildcard pin
	* @param Direction			
	* @param ContainerType		
	* @return	The newly created pin or nullptr if failed
	*/
	static UNREALED_API UEdGraphPin* CreateWildcardPin(UEdGraphNode* Node, const FName PinName, const EEdGraphPinDirection Direction, const EPinContainerType ContainerType = EPinContainerType::None);

	/**
	* Check this node for any wildcard pins
	*
	* @return	True if the given node has any wildcard pins on it
	*/
	static UNREALED_API bool NodeHasAnyWildcards(const UEdGraphNode* const Node);
	
	/** 
	 * Utility functions for overwriting a wildcard type with an inferred type. 
	 * These will leave the pin's container information unchanged - e.g. so that
	 * a macro node can take in an 'array of wildcards'. 
	 */
	static UNREALED_API void InferType(UEdGraphPin* ToPin, const FEdGraphPinType& Type);
	static UNREALED_API void InferType(FEdGraphPinType& ToType, const FEdGraphPinType& Type);
	static UNREALED_API void InferType(FEdGraphPinType& ToType, const FEdGraphTerminalType& Type);

	/** Utility functions for resetting a pin to wildcard, again, leaving container information unchanged */
	static UNREALED_API void ResetToWildcard(UEdGraphPin* Pin);
	static UNREALED_API void ResetToWildcard(FEdGraphPinType& PinType);
};
