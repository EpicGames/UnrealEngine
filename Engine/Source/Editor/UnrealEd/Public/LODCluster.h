// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Set.h"

class AActor;
class ALODActor;
class ULevel;
class UStaticMeshComponent;

/**
 *
 *	This is a LOD cluster struct that holds list of actors with relevant information
 *
 *	http://deim.urv.cat/~rivi/pub/3d/icra04b.pdf
 *
 *	This is used by Hierarchical LOD Builder to generates list of clusters 
 *	that are together in vicinity and build as one actor
 *
 **/
struct FLODCluster
{
	// constructors
	FLODCluster(const FLODCluster& Other);
	FLODCluster(FLODCluster&& Other);

	FLODCluster(AActor* Actor1);
	FLODCluster(AActor* Actor1, AActor* Actor2);
	FLODCluster();

	/** Cluster operators */
	FLODCluster operator+(const FLODCluster& Other) const;
	FLODCluster& operator+=(const FLODCluster& Other);
	FLODCluster operator-(const FLODCluster& Other) const;
	FLODCluster& operator-=(const FLODCluster& Other);
	FLODCluster& operator=(const FLODCluster & Other);
	FLODCluster& operator=(FLODCluster&& Other);

	bool operator==(const FLODCluster& Other) const;

	/** Invalidates this cluster */
	inline void Invalidate() { bValid = false; }
	
	/** Returns whether or not this cluster is valid */
	inline bool IsValid() const { return bValid; }

	/** Return cost of the cluster, lower is better */
	inline double GetCost() const
	{
		return ClusterCost;
	}

	/** Return cost of the union of this cluster & the other cluster, lower is better */
	double GetMergedCost(const FLODCluster& Other) const;

	/** Compare clusters and returns true when this contains any of Other's actors */
	bool Contains(FLODCluster& Other) const;
	
	/** Returns data/info for this Cluster as a string */
	FString ToString() const;
	
	// member variable
	/** List of Actors that this cluster contains */
	TSet<AActor*, DefaultKeyFuncs<AActor*>, TInlineSetAllocator<2>> Actors;
	/** Cluster bounds */
	FSphere	Bound;
	/** Filling factor for this cluster, determines how much of the cluster's bounds/area is occupied by the contained actors*/
	double FillingFactor;
	/** Cached cluster cost, FMath::Pow(Bound.W, 3) / FillingFactor */
	double ClusterCost;

private:
	/**
	* Merges this cluster with Other by combining the actor arrays and updating the bounds, filling factor and cluster cost
	*
	* @param Other - Other Cluster to merge with	
	*/
	void MergeClusters(const FLODCluster& Other);

	/**
	* Subtracts the Other cluster from this cluster by removing Others actors and updating the bounds, filling factor and cluster cost
	* This will invalidate the cluster if Actors.Num result to be 0
	*
	* @param Other - Other cluster to subtract
	*/
	void SubtractCluster(const FLODCluster& Other);

	/**
	* Adds a new actor to this cluster and updates the bounds accordingly, the filling factor is NOT updated
	*
	* @param NewActor - Actor to be added to the cluster	
	*/
	FSphere AddActor(AActor* NewActor);

	/** Bool flag whether or not this cluster is valid */
	bool bValid;
};
