// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

#include "LayersColumns.generated.h"

/**
 * Tag that indicates this row in TEDS represents a ULayer
 */
USTRUCT(meta = (DisplayName = "Layer Tag"))
struct FEditorDataStorageLayerTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

/**
 * Column containing the name of a ULayer
 */
USTRUCT(meta = (DisplayName = "Layer Name Column"))
struct FEditorDataStorageLayerNameColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	
	UPROPERTY(meta = (Searchable, Sortable))
	FName LayerName;
};

/**
 * Column on actor rows that contains the layers this actor belongs to
 */
USTRUCT(meta = (DisplayName = "Layers"))
struct FEditorDataStorageActorLayersColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UE::Editor::DataStorage::FRowHandleArray Layers;
};

namespace UE::Editor::Layers
{
	using FLayerTag = FEditorDataStorageLayerTag;
	using FLayerNameColumn = FEditorDataStorageLayerNameColumn;
	using FActorLayersColumn = FEditorDataStorageActorLayersColumn;
}