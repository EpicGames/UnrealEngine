// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorSubsystem.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

#include "LayersSubsystem.generated.h"

class AActor;
class FLayersBroadcast;
class FLevelEditorViewportClient;
class UEditorEngine;
class ULayer;
class ULevel;
class UWorld;
template< typename TItemType > class IFilter;

namespace ELayersAction
{
	enum Type
	{
		/**	The specified ChangedLayer is a newly created ULayer, if ChangedLayer is invalid then multiple Layers were added */
		Add,

		/**
		 *	The specified ChangedLayer was just modified, if ChangedLayer is invalid then multiple Layers were modified.
		 *  ChangedProperty specifies what field on the ULayer was changed, if NAME_None then multiple fields were changed
		 */
		Modify,

		/**	A ULayer was deleted */
		Delete,

		/**	The specified ChangedLayer was just renamed */
		Rename,

		/**	A large amount of changes have occurred to a number of Layers. A full rebind will be required. */
		Reset,
	};
}

UCLASS(MinimalAPI)
class ULayersSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	typedef IFilter< const TWeakObjectPtr< AActor >& > ActorFilter;

	/**
	 *	Prepares for use
	 */
	UNREALED_API virtual void Initialize(FSubsystemCollectionBase& Collection) override final;

	/**
	 *	Internal cleanup
	 */
	UNREALED_API virtual void Deinitialize() override final;

	/**
	 *	Destructor
	 */
	UNREALED_API virtual ~ULayersSubsystem();

	/** Broadcasts whenever one or more Layers are modified*/
	DECLARE_EVENT_ThreeParams(ULayersSubsystem, FOnLayersChanged, const ELayersAction::Type /*Action*/, const TWeakObjectPtr< ULayer >& /*ChangedLayer*/, const FName& /*ChangedProperty*/);
	virtual FOnLayersChanged& OnLayersChanged() final { return LayersChanged; }

	/** Broadcasts whenever one or more Actors changed layers*/
	DECLARE_EVENT_OneParam(ULayersSubsystem, FOnActorsLayersChanged, const TWeakObjectPtr< AActor >& /*ChangedActor*/);
	virtual FOnActorsLayersChanged& OnActorsLayersChanged() final { return ActorsLayersChanged; }

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	// Operations on Levels

	/**
	 *	Aggregates any information regarding layers associated with the level and it contents
	 *
	 *	@param	Level	The process
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API virtual void AddLevelLayerInformation(ULevel* Level) final;
	/**
	 *	Purges any information regarding layers associated with the level and it contents
	 *
	 *	@param	Level	The process
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API virtual void RemoveLevelLayerInformation(ULevel* Level) final;

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	// Operations on an individual actor.

	/**
	 *	Checks to see if the specified actor is in an appropriate state to interact with layers
	 *
	 *	@param	Actor	The actor to validate
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API virtual bool IsActorValidForLayer(AActor* Actor) final;
	/**
	 *	Synchronizes an newly created Actor's layers with the layer system
	 *
	 *	@param	Actor	The actor to initialize
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API virtual bool InitializeNewActorLayers(AActor* Actor) final;
	/**
	 *	Disassociates an Actor's layers from the layer system, general used before deleting the Actor
	 *
	 *	@param	Actor	The actor to disassociate from the layer system
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API virtual bool DisassociateActorFromLayers(AActor* Actor) final;

	/**
	 * Adds the actor to the named layer.
	 *
	 * @param	Actor		The actor to add to the named layer
	 * @param	LayerName	The name of the layer to add the actor to
	 * @return				true if the actor was added.  false is returned if the actor already belongs to the layer.
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API virtual bool AddActorToLayer(AActor* Actor, const FName& LayerName) final;
	/**
	 * Adds the provided actor to the named layers.
	 *
	 * @param	Actor		The actor to add to the provided layers
	 * @param	LayerNames	A valid list of layer names.
	 * @return				true if the actor was added to at least one of the provided layers.
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API virtual bool AddActorToLayers(AActor* Actor, const TArray< FName >& LayerNames) final;

	/**
	 * Removes an actor from the specified layer.
	 *
	 * @param	Actor			The actor to remove from the provided layer
	 * @param	LayerToRemove	The name of the layer to remove the actor from
	 * @return					true if the actor was removed from the layer.  false is returned if the actor already belonged to the layer.
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API virtual bool RemoveActorFromLayer(AActor* Actor, const FName& LayerToRemove, const bool bUpdateStats = true) final;
	/**
	 * Removes the provided actor from the named layers.
	 *
	 * @param	Actor		The actor to remove from the provided layers
	 * @param	LayerNames	A valid list of layer names.
	 * @return				true if the actor was removed from at least one of the provided layers.
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API virtual bool RemoveActorFromLayers(AActor* Actor, const TArray< FName >& LayerNames, const bool bUpdateStats = true) final;


	/////////////////////////////////////////////////
	// Operations on multiple actors.

	/**
	 * Add the actors to the named layer
	 *
	 * @param	Actors		The actors to add to the named layer
	 * @param	LayerName	The name of the layer to add to
	 * @return				true if at least one actor was added to the layer.  false is returned if all the actors already belonged to the layer.
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API bool AddActorsToLayer(const TArray< AActor* >& Actors, const FName& LayerName);
	/**
	 * Add the actors to the named layer
	 *
	 * @param	Actors		The actors to add to the named layer
	 * @param	LayerName	The name of the layer to add to
	 * @return				true if at least one actor was added to the layer.  false is returned if all the actors already belonged to the layer.
	 */
	UNREALED_API virtual bool AddActorsToLayer(const TArray< TWeakObjectPtr< AActor > >& Actors, const FName& LayerName) final;
	/**
	 * Add the actors to the named layers
	 *
	 * @param	Actors		The actors to add to the named layers
	 * @param	LayerNames	A valid list of layer names.
	 * @return				true if at least one actor was added to at least one layer.  false is returned if all the actors already belonged to all specified layers.
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API bool AddActorsToLayers(const TArray< AActor* >& Actors, const TArray< FName >& LayerNames);
	/**
	 * Add the actors to the named layers
	 *
	 * @param	Actors		The actors to add to the named layers
	 * @param	LayerNames	A valid list of layer names.
	 * @return				true if at least one actor was added to at least one layer.  false is returned if all the actors already belonged to all specified layers.
	 */
	UNREALED_API virtual bool AddActorsToLayers(const TArray< TWeakObjectPtr< AActor > >& Actors, const TArray< FName >& LayerNames) final;

	/**
	*	Disassociates actors from the layer system, generally used before deleting the Actors
	*
	*	@param	Actors	The actors to disassociate from the layer system
	*/
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API virtual bool DisassociateActorsFromLayers(const TArray< AActor* >& Actors) final;
	
	/**
	 * Removes the actors from the specified layer.
	 *
	 * @param	Actors			The actors to remove from the provided layer
	 * @param	LayerToRemove	The name of the layer to remove the actors from
	 * @return					true if at least one actor was removed from the layer.  false is returned if all the actors already belonged to the layer.
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API bool RemoveActorsFromLayer(const TArray< AActor* >& Actors, const FName& LayerName, const bool bUpdateStats = true);
	/**
	 * Removes the actors from the specified layer.
	 *
	 * @param	Actors			The actors to remove from the provided layer
	 * @param	LayerToRemove	The name of the layer to remove the actors from
	 * @return					true if at least one actor was removed from the layer.  false is returned if all the actors already belonged to the layer.
	 */
	UNREALED_API virtual bool RemoveActorsFromLayer(const TArray< TWeakObjectPtr< AActor > >& Actors, const FName& LayerName, const bool bUpdateStats = true) final;
	/**
	 * Remove the actors to the named layers
	 *
	 * @param	Actors		The actors to remove to the named layers
	 * @param	LayerNames	A valid list of layer names.
	 * @return				true if at least one actor was removed from at least one layer.  false is returned if none of the actors belonged to any of the specified layers.
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API bool RemoveActorsFromLayers(const TArray< AActor* >& Actors, const TArray< FName >& LayerNames, const bool bUpdateStats = true);
	/**
	 * Remove the actors to the named layers
	 *
	 * @param	Actors		The actors to remove to the named layers
	 * @param	LayerNames	A valid list of layer names.
	 * @return				true if at least one actor was removed from at least one layer.  false is returned if none of the actors belonged to any of the specified layers.
	 */
	UNREALED_API virtual bool RemoveActorsFromLayers(const TArray< TWeakObjectPtr< AActor > >& Actors, const TArray< FName >& LayerNames, const bool bUpdateStats = true) final;

	
	/////////////////////////////////////////////////
	// Operations on selected actors.

	/**
	 * Find and return the selected actors.
	 *
	 * @return				The selected AActor's as a TArray.
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API TArray< AActor* > GetSelectedActors() const;


	/**
	 * Adds selected actors to the named layer.
	 *
	 * @param	LayerName	A layer name.
	 * @return				true if at least one actor was added.  false is returned if all selected actors already belong to the named layer.
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API virtual bool AddSelectedActorsToLayer(const FName& LayerName) final;
	/**
	 * Adds selected actors to the named layers.
	 *
	 * @param	LayerNames	A valid list of layer names.
	 * @return				true if at least one actor was added.  false is returned if all selected actors already belong to the named layers.
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API virtual bool AddSelectedActorsToLayers(const TArray< FName >& LayerNames) final;

	/**
	 * Removes the selected actors from the named layer.
	 *
	 * @param	LayerName	A layer name.
	 * @return				true if at least one actor was added.  false is returned if all selected actors already belong to the named layer.
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API virtual bool RemoveSelectedActorsFromLayer(const FName& LayerName) final;
	/**
	 * Removes selected actors from the named layers.
	 *
	 * @param	LayerNames	A valid list of layer names.
	 * @return				true if at least one actor was removed.
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API virtual bool RemoveSelectedActorsFromLayers(const TArray< FName >& LayerNames) final;


	/////////////////////////////////////////////////
	// Operations on actors in layers
	
	/**
	 * Selects/de-selects actors belonging to the named layer.
	 *
	 * @param	LayerName						A valid layer name.
	 * @param	bSelect							If true actors are selected; if false, actors are deselected.
	 * @param	bNotify							If true the Editor is notified of the selection change; if false, the Editor will not be notified.
	 * @param	bSelectEvenIfHidden	[optional]	If true even hidden actors will be selected; if false, hidden actors won't be selected.
	 * @return									true if at least one actor was selected/deselected.
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API bool SelectActorsInLayer(const FName& LayerName, const bool bSelect, const bool bNotify, const bool bSelectEvenIfHidden = false);
	/**
	 * Selects/de-selects actors belonging to the named layer.
	 *
	 * @param	LayerName						A valid layer name.
	 * @param	bSelect							If true actors are selected; if false, actors are deselected.
	 * @param	bNotify							If true the Editor is notified of the selection change; if false, the Editor will not be notified.
	 * @param	bSelectEvenIfHidden	[optional]	If true even hidden actors will be selected; if false, hidden actors won't be selected.
	 * @param	Filter	[optional]				Actor that don't pass the specified filter restrictions won't be selected.
	 * @return									true if at least one actor was selected/deselected.
	 */
	UNREALED_API virtual bool SelectActorsInLayer(const FName& LayerName, const bool bSelect, const bool bNotify, const bool bSelectEvenIfHidden, const TSharedPtr< ActorFilter >& Filter) final;
	/**
	 * Selects/de-selects actors belonging to the named layers.
	 *
	 * @param	LayerNames						A valid list of layer names.
	 * @param	bSelect							If true actors are selected; if false, actors are deselected.
	 * @param	bNotify							If true the Editor is notified of the selection change; if false, the Editor will not be notified
	 * @param	bSelectEvenIfHidden	[optional]	If true even hidden actors will be selected; if false, hidden actors won't be selected.
	 * @return									true if at least one actor was selected/deselected.
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API bool SelectActorsInLayers(const TArray< FName >& LayerNames, const bool bSelect, const bool bNotify, const bool bSelectEvenIfHidden = false);
	/**
	 * Selects/de-selects actors belonging to the named layers.
	 *
	 * @param	LayerNames						A valid list of layer names.
	 * @param	bSelect							If true actors are selected; if false, actors are deselected.
	 * @param	bNotify							If true the Editor is notified of the selection change; if false, the Editor will not be notified
	 * @param	bSelectEvenIfHidden	[optional]	If true even hidden actors will be selected; if false, hidden actors won't be selected.
	 * @param	Filter	[optional]				Actor that don't pass the specified filter restrictions won't be selected.
	 * @return									true if at least one actor was selected/deselected.
	 */
	UNREALED_API virtual bool SelectActorsInLayers(const TArray< FName >& LayerNames, const bool bSelect, const bool bNotify, const bool bSelectEvenIfHidden, const TSharedPtr< ActorFilter >& Filter) final;


	/////////////////////////////////////////////////
	// Operations on actor viewport visibility regarding layers

	/**
	 * Updates the visibility for all actors for all views.
	 *
	 * @param LayerThatChanged  If one layer was changed (toggled in view pop-up, etc), then we only need to modify actors that use that layer.
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API virtual void UpdateAllViewVisibility(const FName& LayerThatChanged) final;
	/**
	 * Updates the per-view visibility for all actors for the given view
	 *
	 * @param ViewportClient				The viewport client to update visibility on
	 * @param LayerThatChanged [optional]	If one layer was changed (toggled in view pop-up, etc), then we only need to modify actors that use that layer
	 */
	UNREALED_API virtual void UpdatePerViewVisibility(FLevelEditorViewportClient* ViewportClient, const FName& LayerThatChanged = NAME_Skip) final;

	/**
	 * Updates per-view visibility for the given actor in the given view
	 *
	 * @param ViewportClient				The viewport client to update visibility on
	 * @param Actor								Actor to update
	 * @param bReregisterIfDirty [optional]		If true, the actor will reregister itself to give the rendering thread updated information
	 */
	UNREALED_API virtual void UpdateActorViewVisibility(FLevelEditorViewportClient* ViewportClient, AActor* Actor, const bool bReregisterIfDirty = true) final;
	/**
	 * Updates per-view visibility for the given actor for all views
	 *
	 * @param Actor		Actor to update
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API virtual void UpdateActorAllViewsVisibility(AActor* Actor) final;

	/**
	 * Removes the corresponding visibility bit from all actors (slides the later bits down 1)
	 *
	 * @param ViewportClient	The viewport client to update visibility on
	 */
	UNREALED_API virtual void RemoveViewFromActorViewVisibility(FLevelEditorViewportClient* ViewportClient) final;

	/**
	 * Updates the provided actors visibility in the viewports
	 *
	 * @param	Actor						Actor to update
	 * @param	bOutSelectionChanged [OUT]	Whether the Editors selection changed
	 * @param	bOutActorModified [OUT]		Whether the actor was modified
	 * @param	bNotifySelectionChange		If true the Editor is notified of the selection change; if false, the Editor will not be notified
	 * @param	bRedrawViewports			If true the viewports will be redrawn; if false, they will not
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API virtual bool UpdateActorVisibility(AActor* Actor, bool& bOutSelectionChanged, bool& bOutActorModified, const bool bNotifySelectionChange, const bool bRedrawViewports) final;
	/**
	 * Updates the visibility of all actors in the viewports
	 *
	 * @param	bNotifySelectionChange		If true the Editor is notified of the selection change; if false, the Editor will not be notified
	 * @param	bRedrawViewports			If true the viewports will be redrawn; if false, they will not
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API virtual bool UpdateAllActorsVisibility(const bool bNotifySelectionChange, const bool bRedrawViewports) final;


	/////////////////////////////////////////////////
	// Operations on layers

	/**
	 *	Appends all the actors associated with the specified layer.
	 *
	 *	@param	LayerName			The layer to find actors for.
	 *	@param	InOutActors			The list to append the found actors to.
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API void AppendActorsFromLayer(const FName& LayerName, TArray< AActor* >& InOutActors) const;
	/**
	 *	Appends all the actors associated with the specified layer.
	 *
	 *	@param	LayerName			The layer to find actors for.
	 *	@param	InOutActors			The list to append the found actors to.
	 *  @param	Filter	[optional]	Actor that don't pass the specified filter restrictions won't be selected.
	 */
	UNREALED_API void AppendActorsFromLayer(const FName& LayerName, TArray< AActor* >& InOutActors, const TSharedPtr< ActorFilter >& Filter) const;
	/**
	 *	Appends all the actors associated with the specified layer.
	 *
	 *	@param	LayerName			The layer to find actors for.
	 *	@param	InOutActors			The list to append the found actors to.
	 *  @param	Filter	[optional]	Actor that don't pass the specified filter restrictions won't be selected.
	 */
	UNREALED_API virtual void AppendActorsFromLayer(const FName& LayerName, TArray< TWeakObjectPtr< AActor > >& InOutActors, const TSharedPtr< ActorFilter >& Filter = TSharedPtr< ActorFilter >(nullptr)) const final;
	/**
	 *	Appends all the actors associated with ANY of the specified layers.
	 *
	 *	@param	LayerNames			The layers to find actors for.
	 *	@param	InOutActors			The list to append the found actors to.
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API void AppendActorsFromLayers(const TArray< FName >& LayerNames, TArray< AActor* >& InOutActors) const;
	/**
	 *	Appends all the actors associated with ANY of the specified layers.
	 *
	 *	@param	LayerNames			The layers to find actors for.
	 *	@param	InOutActors			The list to append the found actors to.
	 *  @param	Filter	[optional]	Actor that don't pass the specified filter restrictions won't be selected.
	 */
	UNREALED_API void AppendActorsFromLayers(const TArray< FName >& LayerNames, TArray< AActor* >& InOutActors, const TSharedPtr< ActorFilter >& Filter) const;
	/**
	 *	Appends all the actors associated with ANY of the specified layers.
	 *
	 *	@param	LayerNames			The layers to find actors for.
	 *	@param	InOutActors			The list to append the found actors to.
	 *  @param	Filter	[optional]	Actor that don't pass the specified filter restrictions won't be selected.
	 */
	UNREALED_API virtual void AppendActorsFromLayers(const TArray< FName >& LayerNames, TArray< TWeakObjectPtr< AActor > >& InOutActors, const TSharedPtr< ActorFilter >& Filter = TSharedPtr< ActorFilter >(nullptr)) const final;

	/**
	 *	Gets all the actors associated with the specified layer. Analog to AppendActorsFromLayer but it returns rather than appends the actors.
	 *
	 *	@param	LayerName			The layer to find actors for.
	 *	@return						The list to assign the found actors to.
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API TArray< AActor* > GetActorsFromLayer(const FName& LayerName) const;
	/**
	 *	Gets all the actors associated with the specified layer. Analog to AppendActorsFromLayer but it returns rather than appends the actors.
	 *
	 *	@param	LayerName			The layer to find actors for.
	 *  @param	Filter	[optional]	Actor that don't pass the specified filter restrictions won't be selected.
	 *	@return						The list to assign the found actors to.
	 */
	UNREALED_API TArray< AActor* > GetActorsFromLayer(const FName& LayerName, const TSharedPtr< ActorFilter >& Filter) const;
	/**
	 *	Gets all the actors associated with ANY of the specified layers. Analog to AppendActorsFromLayers but it returns rather than appends the actors.
	 *
	 *	@param	LayerNames			The layers to find actors for.
	 *	@return						The list to assign the found actors to.
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API TArray< AActor* > GetActorsFromLayers(const TArray< FName >& LayerNames) const;
	/**
	 *	Gets all the actors associated with ANY of the specified layers. Analog to AppendActorsFromLayers but it returns rather than appends the actors.
	 *
	 *	@param	LayerNames			The layers to find actors for.
	 *  @param	Filter	[optional]	Actor that don't pass the specified filter restrictions won't be selected.
	 *	@return						The list to assign the found actors to.
	 */
	UNREALED_API TArray< AActor* > GetActorsFromLayers(const TArray< FName >& LayerNames, const TSharedPtr< ActorFilter >& Filter) const;

	/**
	 * Changes the named layer's visibility to the provided state
	 *
	 * @param	LayerName	The name of the layer to affect.
	 * @param	bIsVisible	If true the layer will be visible; if false, the layer will not be visible.
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API virtual void SetLayerVisibility(const FName& LayerName, const bool bIsVisible) final;
	/**
	 * Changes visibility of the named layers to the provided state
	 *
	 * @param	LayerNames	The names of the layers to affect
	 * @param	bIsVisible	If true the layers will be visible; if false, the layers will not be visible
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API virtual void SetLayersVisibility(const TArray< FName >& LayerNames, const bool bIsVisible) final;

	/**
	 * Toggles the named layer's visibility
	 *
	 * @param LayerName	The name of the layer to affect
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API virtual void ToggleLayerVisibility(const FName& LayerName) final;
	/**
	 * Toggles the visibility of all of the named layers
	 *
	 * @param	LayerNames	The names of the layers to affect
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API virtual void ToggleLayersVisibility(const TArray< FName >& LayerNames) final;

	/**
	 * Set the visibility of all layers to true
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API virtual void MakeAllLayersVisible() final;

	/**
	 * Gets the ULayer Object of the named layer
	 *
	 * @param	LayerName	The name of the layer whose ULayer Object is returned
	 * @return				The ULayer Object of the provided layer name
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API ULayer* GetLayer(const FName& LayerName) const;
	/**
	 * Checks whether the ULayer Object of the provided layer name exists.
	 *
	 * @param	LayerName		The name of the layer whose ULayer Object to retrieve
	 * @return					If true a valid ULayer Object was found; if false, a valid ULayer object was not found
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API bool IsLayer(const FName& LayerName);
	/**
	 * Attempts to get the ULayer Object of the provided layer name.
	 *
	 * @param	LayerName		The name of the layer whose ULayer Object to retrieve
	 * @param	OutLayer[OUT] 	Set to the ULayer Object of the named layer. Set to Invalid if no ULayer Object exists.
	 * @return					If true a valid ULayer Object was found and set to OutLayer; if false, a valid ULayer object was not found and invalid set to OutLayer
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API virtual bool TryGetLayer(const FName& LayerName, ULayer*& OutLayer) final;

	/**
	 * Gets all known layers and appends their names to the provide array
	 *
	 * @param OutLayers[OUT] Output array to store all known layers
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API virtual void AddAllLayerNamesTo(TArray< FName >& OutLayerNames) const final;
	/**
	 * Gets all known layers and appends them to the provided array
	 *
	 * @param OutLayers[OUT] Output array to store all known layers
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API virtual void AddAllLayersTo(TArray< ULayer* > & OutLayers) const final;
	/**
	 * Gets all known layers and appends them to the provided array
	 *
	 * @param OutLayers[OUT] Output array to store all known layers
	 */
	UNREALED_API virtual void AddAllLayersTo(TArray< TWeakObjectPtr< ULayer > >& OutLayers) const final;

	/**
	 * Creates a ULayer Object for the named layer
	 *
	 * @param	LayerName	The name of the layer to create
	 * @return				The newly created ULayer Object for the named layer
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API ULayer* CreateLayer(const FName& LayerName);

	/**
	 * Deletes all of the provided layers, disassociating all actors from them
	 *
	 * @param LayersToDelete	A valid list of layer names.
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API virtual void DeleteLayers(const TArray< FName >& LayersToDelete) final;
	/**
	 * Deletes the provided layer, disassociating all actors from them
	 *
	 * @param LayerToDelete		A valid layer name
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API virtual void DeleteLayer(const FName& LayerToDelete) final;

	/**
	 * Renames the provided originally named layer to the provided new name
	 *
	 * @param	OriginalLayerName	The name of the layer to be renamed
	 * @param	NewLayerName		The new name for the layer to be renamed
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API virtual bool RenameLayer(const FName& OriginalLayerName, const FName& NewLayerName) final;

	/**
	 * Get the current UWorld object.
	 *
	 * @return						The UWorld* object
	 */
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API UWorld* GetWorld() const; // Fallback to GWorld

	/**
	 * Delegate handler for FEditorDelegates::MapChange. It internally calls LayersChanged.Broadcast.
	 **/
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API void EditorMapChange();
	/**
	 * Delegate handler for FEditorDelegates::RefreshLayerBrowser. It internally calls UpdateAllActorsVisibility to refresh the actors of each layer.
	 **/
	UFUNCTION(BlueprintCallable, Category = Layers)
	UNREALED_API void EditorRefreshLayerBrowser();
	/**
	 * Delegate handler for FEditorDelegates::PostUndoRedo. It internally calls LayersChanged.Broadcast and UpdateAllActorsVisibility to refresh the actors of each layer.
	 **/
	UNREALED_API void PostUndoRedo();

private:
	UNREALED_API void AddActorToStats(ULayer* Layer, AActor* Actor);
	UNREALED_API void RemoveActorFromStats(ULayer* Layer, AActor* Actor);
	
	// Update the layers column on the given actor's row in TEDS
	void AddActorToLayerInTeds(ULayer* Layer, AActor* Actor);
	void RemoveActorFromLayerInTeds(ULayer* Layer, AActor* Actor);
	
	// Internal function to update all actors visibility and also optionally update the TEDS state. This is only needed in specific 
	// situations and not every time UpdateAllActorsVisibility is called, but we also don't want to iterate all actors twice so the internal function
	// can be called manually after things like undo/redo when TEDS needs to be updated
	bool UpdateAllActorsVisibility_Internal(const bool bNotifySelectionChange, const bool bRedrawViewports, bool bUpdateTedsState = false);
	
	// Force sync the layers state for the given actor from the world to TEDS (stomping the old state in TEDS)
	void SyncActorLayersToTeds(AActor* Actor);


	/**
	 *	Checks to see if the named layer exists, and if it doesn't creates it.
	 *
	 * @param	LayerName	A valid layer name
	 * @return				The ULayer Object of the named layer
	 */
	UNREALED_API ULayer* EnsureLayerExists(const FName& LayerName);

	/**	Fires whenever one or more layer changes */
	FOnLayersChanged LayersChanged;

	/**	Fires whenever one or more actor layer changes */
	FOnActorsLayersChanged ActorsLayersChanged;

	/**
	 * Auxiliary class that sets the callback function to FEditorDelegates::MapChange.Broadcast() and FEditorDelegates::RefreshLayerBrowser.Broadcast().
	 * Note that UClasses objects do not allow broadcast, that is why we need this auxiliary class.
	 */
	TSharedPtr<class FLayersBroadcast> LayersBroadcast;
	
	/**	Cached Teds interfaces */
	UE::Editor::DataStorage::ICoreProvider* DataStorage = nullptr;
	UE::Editor::DataStorage::ICompatibilityProvider* DataStorageCompat = nullptr;
	
	/**	Delegate run when TEDS is first initialized */
	FDelegateHandle OnDataStorageInitialized;
};
