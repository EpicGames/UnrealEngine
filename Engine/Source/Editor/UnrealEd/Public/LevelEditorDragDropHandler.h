// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Internationalization/Text.h"
#include "UObject/GCObject.h"
#include "LevelEditorDragDropHandler.generated.h"

class FViewport;
struct FAssetData;

struct FLevelEditorDragDropWorldSurrogateReferencingObject : public FGCObject
{
public:
	FLevelEditorDragDropWorldSurrogateReferencingObject(const UObject* InSurrogateObject) { SurrogateObject = InSurrogateObject; }
	bool IsValid() const { return GetValue() != nullptr; }
	const UObject* GetValue() const { return SurrogateObject; }
	virtual bool OnPreDropObjects(UWorld* World, const TArray<UObject*>& DroppedObjects) { return true; }
	virtual bool OnPostDropObjects(UWorld* World, const TArray<UObject*>& DroppedObjects) { return true; }

	//~ Begin FGCObject Interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override { Collector.AddReferencedObject(SurrogateObject); }
	virtual FString GetReferencerName() const override { return TEXT("FLevelEditorDragDropWorldSurrogateReferencingObject"); }
	//~ End FGCObject Interface

protected:
	TObjectPtr<const UObject> SurrogateObject;
};

UCLASS(transient, MinimalAPI)
class ULevelEditorDragDropHandler : public UObject
{
public:
	GENERATED_BODY()
	
public:
	UNREALED_API ULevelEditorDragDropHandler();
	
	UNREALED_API virtual bool PreviewDropObjectsAtCoordinates(int32 MouseX, int32 MouseY, UWorld* World, FViewport* Viewport, const FAssetData& AssetData);
	UNREALED_API virtual bool PreDropObjectsAtCoordinates(int32 MouseX, int32 MouseY, UWorld* World, FViewport* Viewport, const TArray<UObject*>& DroppedObjects, TArray<AActor*>& OutNewActors);
	UNREALED_API virtual bool PostDropObjectsAtCoordinates(int32 MouseX, int32 MouseY, UWorld* World, FViewport* Viewport, const TArray<UObject*>& DroppedObjects);

	bool GetCanDrop() const { return bCanDrop; }
	FText GetPreviewDropHintText() const { return HintText; }

	DECLARE_DELEGATE_RetVal_TwoParams(TUniquePtr<FLevelEditorDragDropWorldSurrogateReferencingObject>, FOnLevelEditorDragDropWorldSurrogateReferencingObject, UWorld*, const FSoftObjectPath&);
	FOnLevelEditorDragDropWorldSurrogateReferencingObject& OnLevelEditorDragDropWorldSurrogateReferencingObject() { return OnLevelEditorDragDropWorldSurrogateReferencingObjectDelegate; }
	
protected:
	bool PassesFilter(UWorld* World, const FAssetData& AssetData, TUniquePtr<FLevelEditorDragDropWorldSurrogateReferencingObject>* OutWorldSurrogateReferencingObject = nullptr);

	bool bRunAssetFilter = true;

	/** True if it's valid to drop the object at the location queried */
	bool bCanDrop = false;

	/** Optional hint text that may be returned to the user. */
	FText HintText;

	TUniquePtr<FLevelEditorDragDropWorldSurrogateReferencingObject> WorldSurrogateReferencingObject;
	FOnLevelEditorDragDropWorldSurrogateReferencingObject OnLevelEditorDragDropWorldSurrogateReferencingObjectDelegate;
};
