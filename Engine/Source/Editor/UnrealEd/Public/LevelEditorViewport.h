// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "AssetSelection.h" // FExtraPlaceAssetOptions
#include "InputCoreTypes.h"
#include "GameFramework/Actor.h"
#include "Camera/CameraComponent.h"
#include "UnrealWidgetFwd.h"
#include "EditorViewportClient.h"
#include "UObject/ObjectKey.h"
#include "UnrealEdMisc.h"
#include "Elements/Framework/TypedElementListFwd.h"

struct FAssetData;
struct FMinimalViewInfo;
class FCanvas;
class FDragTool;
class HModel;
class IAssetFactoryInterface;
class ILevelEditor;
class SLevelViewport;
class UActorFactory;
class UModel;
class UTypedElementSelectionSet;
struct FWorldContext;
struct FTypedElementHandle;
struct FGizmoState;

/** Describes an object that's currently hovered over in the level viewport */
struct FViewportHoverTarget
{
	/** The actor we're drawing the hover effect for, or NULL */
	TObjectPtr<AActor> HoveredActor;

	/** The BSP model we're drawing the hover effect for, or NULL */
	TObjectPtr<UModel> HoveredModel;

	/** Surface index on the BSP model that currently has a hover effect */
	uint32 ModelSurfaceIndex;


	/** Construct from an actor */
	FViewportHoverTarget( AActor* InActor )
		: HoveredActor( InActor ),
			HoveredModel( nullptr ),
			ModelSurfaceIndex( INDEX_NONE )
	{
	}

	/** Construct from an BSP model and surface index */
	FViewportHoverTarget( UModel* InModel, int32 InSurfaceIndex )
		: HoveredActor( nullptr ),
			HoveredModel( InModel ),
			ModelSurfaceIndex( InSurfaceIndex )
	{
	}

	/** Equality operator */
	bool operator==( const FViewportHoverTarget& RHS ) const
	{
		return RHS.HoveredActor == HoveredActor &&
				RHS.HoveredModel == HoveredModel &&
				RHS.ModelSurfaceIndex == ModelSurfaceIndex;
	}

	friend uint32 GetTypeHash( const FViewportHoverTarget& Key )
	{
		return Key.HoveredActor ? GetTypeHash(Key.HoveredActor) : GetTypeHash(Key.HoveredModel)+Key.ModelSurfaceIndex;
	}
};

struct FTrackingTransaction
{
	/** State of this transaction */
	struct ETransactionState
	{
		enum Enum
		{
			Inactive,
			Active,
			Pending,
		};
	};

	UNREALED_API FTrackingTransaction();
	UNREALED_API ~FTrackingTransaction();

	/**
	 * Initiates a transaction.
	 */
	UNREALED_API void Begin(const FText& Description, AActor* AdditionalActor = nullptr);

	UNREALED_API void End();

	UNREALED_API void Cancel();

	/** Begin a pending transaction, which won't become a real transaction until PromotePendingToActive is called */
	UNREALED_API void BeginPending(const FText& Description);

	/** Promote a pending transaction (if any) to an active transaction */
	UNREALED_API void PromotePendingToActive();

	bool IsActive() const { return TrackingTransactionState == ETransactionState::Active; }

	bool IsPending() const { return TrackingTransactionState == ETransactionState::Pending; }
	
	int32 TransCount = 0;

private:

	UNREALED_API const UTypedElementSelectionSet* GetSelectionSet() const;
	UNREALED_API UTypedElementSelectionSet* GetMutableSelectionSet() const;

	/** Editor selection changed delegate handler */	
	UNREALED_API void OnEditorSelectionChanged(const UTypedElementSelectionSet* InSelectionSet);

	/** The current transaction. */
	class FScopedTransaction* ScopedTransaction = nullptr;

	/** This is set to Active if TrackingStarted() has initiated a transaction, Pending if a transaction will begin before the next delta change */
	ETransactionState::Enum TrackingTransactionState = ETransactionState::Inactive;

	/** The description to use if a pending transaction turns into a real transaction */
	FText PendingDescription;

	/** Initial package dirty states for the Actors within the transaction */
	TMap<UPackage*, bool> InitialPackageDirtyStates;
	
};

/** Interface for objects who want to lock the viewport to an actor. */
struct FLevelViewportActorLock
{
	/** Represents no lock. */
	static UNREALED_API const FLevelViewportActorLock None;

	/** Creates a new instance of FLevelViewportActorLock. */
	FLevelViewportActorLock() 
		: LockedActor(nullptr) {}

	/** Creates a new instance of FLevelViewportActorLock. */
	FLevelViewportActorLock(AActor* InActor) 
		: LockedActor(InActor) {}

	/** Creates a new instance of FLevelViewportActorLock. */
	FLevelViewportActorLock(AActor* InActor, TOptional<EAspectRatioAxisConstraint> InAspectRatioAxisConstraint) 
		: LockedActor(InActor), AspectRatioAxisConstraint(InAspectRatioAxisConstraint) {}

	/** Returns whether the locked actor is valid. */
	bool HasValidLockedActor() const { return LockedActor.IsValid(); }

	/** Gets the locked actor. */
	AActor* GetLockedActor() const { return LockedActor.Get(); }

	/** The actor the viewport should be locked to. */
	TWeakObjectPtr<AActor> LockedActor;

	/** An optional aspect ratio axis constraint to use when resizing the viewport. */
	TOptional<EAspectRatioAxisConstraint> AspectRatioAxisConstraint;
};

/** */
class FLevelEditorViewportClient : public FEditorViewportClient
{
	friend class FActorElementLevelEditorViewportInteractionCustomization;
	friend class FComponentElementLevelEditorViewportInteractionCustomization;

public:

	//~ TODO: UE_DEPRECATED(5.4, "Use GetDropPreviewElements instead.")
	/** @return Returns the current global drop preview actor, or a NULL pointer if we don't currently have one */
	static const TArray< TWeakObjectPtr<AActor> >& GetDropPreviewActors()
	{
		return DropPreviewActors;
	}

	/** Returns the current drop preview elements, or a null pointer if there are none. */
	const TSharedPtr<FTypedElementList>& GetDropPreviewElements() const
	{
		return DropPreviewElements;
	}

	FVector2D GetDropPreviewLocation() const { return FVector2D( DropPreviewMouseX, DropPreviewMouseY ); }

	/**
	 * Constructor
	 */
	UNREALED_API FLevelEditorViewportClient(const TSharedPtr<class SLevelViewport>& InLevelViewport);

	/**
	 * Destructor
	 */
	UNREALED_API virtual ~FLevelEditorViewportClient();

	////////////////////////////
	// FViewElementDrawer interface
	UNREALED_API virtual void Draw(const FSceneView* View,FPrimitiveDrawInterface* PDI) override;
	// End of FViewElementDrawer interface
	
	UNREALED_API virtual FSceneView* CalcSceneView(FSceneViewFamily* ViewFamily, const int32 StereoViewIndex = INDEX_NONE) override;

	////////////////////////////
	// FEditorViewportClient interface
	UNREALED_API virtual void DrawCanvas( FViewport& InViewport, FSceneView& View, FCanvas& Canvas ) override;
	UNREALED_API virtual bool InputKey(const FInputKeyEventArgs& InEventArgs) override;
	UNREALED_API virtual bool InputAxis(const FInputKeyEventArgs& Args) override;
	UNREALED_API virtual EMouseCursor::Type GetCursor(FViewport* Viewport,int32 X,int32 Y) override;
	UNREALED_API virtual void CapturedMouseMove(FViewport* InViewport, int32 InMouseX, int32 InMouseY) override;
	UNREALED_API virtual void MouseMove(FViewport* InViewport, int32 x, int32 y) override;
	UNREALED_API virtual void Tick(float DeltaSeconds) override;
	UNREALED_API virtual bool InputWidgetDelta( FViewport* Viewport, EAxisList::Type CurrentAxis, FVector& Drag, FRotator& Rot, FVector& Scale ) override;
	UNREALED_API virtual TSharedPtr<FDragTool> MakeDragTool( EDragTool::Type DragToolType ) override;
	virtual bool IsLevelEditorClient() const override { return ParentLevelEditor.IsValid(); }
	UNREALED_API virtual void TrackingStarted( const struct FInputEventState& InInputState, bool bIsDraggingWidget, bool bNudge ) override;
	UNREALED_API virtual void TrackingStopped() override;
	UNREALED_API virtual void AbortTracking() override;
	UNREALED_API virtual UE::Widget::EWidgetMode GetWidgetMode() const override;
	UNREALED_API virtual FVector GetWidgetLocation() const override;
	UNREALED_API virtual FMatrix GetWidgetCoordSystem() const override;
	UNREALED_API virtual void SetupViewForRendering( FSceneViewFamily& ViewFamily, FSceneView& View ) override;
	UNREALED_API virtual FLinearColor GetBackgroundColor() const override;
	UNREALED_API virtual const FEditorViewportCameraSpeedSettings& GetCameraSpeedSettings() const override;
	UNREALED_API virtual void SetCameraSpeedSettings(const FEditorViewportCameraSpeedSettings& InCameraSpeedSettings) override;
	UNREALED_API virtual void ReceivedFocus(FViewport* InViewport) override;
	UNREALED_API virtual void LostFocus(FViewport* InViewport) override;
	UNREALED_API virtual void ProcessClick(FSceneView& View, HHitProxy* HitProxy, FKey Key, EInputEvent Event, uint32 HitX, uint32 HitY) override;
	UNREALED_API virtual void SummonContextMenu(const FTypedElementHandle& HitProxyElement = FTypedElementHandle());
	UNREALED_API virtual UWorld* GetWorld() const override;
	UNREALED_API virtual void BeginCameraMovement(bool bHasMovement) override;
	UNREALED_API virtual void EndCameraMovement() override;
	UNREALED_API virtual void SetVREditView(bool bGameViewEnable) override;
	UNREALED_API virtual bool GetPivotForOrbit(FVector& Pivot) const override;
	UNREALED_API virtual bool ShouldScaleCameraSpeedByDistance() const override;

	UNREALED_API virtual bool OverrideHighResScreenshotCaptureRegion(FIntRect& OutCaptureRegion) override;

	UNREALED_API virtual bool BeginTransform(const FGizmoState& InState) override;
	UNREALED_API virtual bool EndTransform(const FGizmoState& InState) override;
	UNREALED_API virtual bool OnTransformCanceled(const FGizmoState& InState) override; 
	

	/** Sets a flag for this frame indicating that the camera has been cut, and temporal effects (such as motion blur) should be reset */
	void SetIsCameraCut()
	{
		bEditorCameraCut = true;
		bWasEditorCameraCut = false;
	}
	bool GetIsCameraCut() const
	{
		return bEditorCameraCut;
	}

	/** 
	 * Initialize visibility flags
	 */
	UNREALED_API void InitializeVisibilityFlags();

	/**
	 * Initialize viewport interaction
	 */
	UNREALED_API void InitializeViewportInteraction();

	/**
	 * Reset the camera position and rotation.  Used when creating a new level.
	 */
	UNREALED_API void ResetCamera();

	/**
	 * Reset the view for a new map 
	 */
	UNREALED_API void ResetViewForNewMap();

	/**
	 * Stores camera settings that may be adversely affected by PIE, so that they may be restored later
	 */
	UNREALED_API void PrepareCameraForPIE();

	/**
	 * Restores camera settings that may be adversely affected by PIE
	 */
	UNREALED_API void RestoreCameraFromPIE();

	/**
	 * Updates the audio listener for this viewport 
	 *
	 * @param View	The scene view to use when calculate the listener position
	 */
	UE_DEPRECATED(5.8, "View is redundant so use method with no parameters")
	UNREALED_API void UpdateAudioListener( const FSceneView& View );
	
	/**
	 * Updates the audio listener for this viewport.
	 */
	UNREALED_API void UpdateAudioListener();

	/** Determines if the new MoveCanvas movement should be used */
	UNREALED_API bool ShouldUseMoveCanvasMovement (void);

	/** 
	 * Returns true if the passed in volume is visible in the viewport (due to volume actor visibility flags)
	 *
	 * @param VolumeActor	The volume to check
	 */
	UNREALED_API bool IsVolumeVisibleInViewport( const AActor& VolumeActor ) const;

	/**
	 * Updates or resets view properties such as aspect ratio, FOV, location etc to match that of any actor we are locked to
	 */
	UNREALED_API void UpdateViewForLockedActor(float DeltaTime=0.f);

	/**
	 * Returns the horizontal axis for this viewport.
	 */
	UNREALED_API EAxisList::Type GetHorizAxis() const;

	/**
	 * Returns the vertical axis for this viewport.
	 */
	UNREALED_API EAxisList::Type GetVertAxis() const;

	UNREALED_API virtual void NudgeSelectedObjects( const struct FInputEventState& InputState ) override;

	/**
	 * Moves the viewport camera according to the locked actors location and rotation
	 * @param bInvalidateHitProxies Whether to invalidate hit proxies when refreshing the viewport. Set to false during mouse-look to avoid unnecessary rebuilds.
	 */
	UNREALED_API void MoveCameraToLockedActor(const bool bInvalidateHitProxies = true);

	/**
	 * Check to see if this actor is locked by the viewport
	 */
	UNREALED_API bool IsActorLocked(const TWeakObjectPtr<const AActor> InActor) const;

	/**
	 * Check to see if any actor is locked by the viewport
	 */
	UNREALED_API bool IsAnyActorLocked() const;

	UNREALED_API void ApplyDeltaToActors( const FVector& InDrag, const FRotator& InRot, const FVector& InScale );
	UNREALED_API void ApplyDeltaToActor( AActor* InActor, const FVector& InDeltaDrag, const FRotator& InDeltaRot, const FVector& InDeltaScale );
	UNREALED_API void ApplyDeltaToComponent(USceneComponent* InComponent, const FVector& InDeltaDrag, const FRotator& InDeltaRot, const FVector& InDeltaScale);
	
	UNREALED_API void ApplyDeltaToSelectedElements(const FTransform& InDeltaTransform);
	UNREALED_API void ApplyDeltaToElement(const FTypedElementHandle& InElementHandle, const FTransform& InDeltaTransform);

	UNREALED_API void MirrorSelectedActors(const FVector& InMirrorScale);
	UNREALED_API void MirrorSelectedElements(const FVector& InMirrorScale);

	UNREALED_API bool GetFocusBounds(FTypedElementListConstRef InElements, FBoxSphereBounds& OutBounds);

	/**
	 * Get the elements (from the current selection set) that this viewport can manipulate (eg, via the transform gizmo).
	 */
	UNREALED_API FTypedElementListConstRef GetElementsToManipulate(const bool bForceRefresh = false);

	UNREALED_API virtual void SetIsSimulateInEditorViewport( bool bInIsSimulateInEditorViewport ) override;

	/**
	 *	Draw the texture streaming bounds.
	 */
	UNREALED_API void DrawTextureStreamingBounds(const FSceneView* View,FPrimitiveDrawInterface* PDI);

	/** GC references. */
	UNREALED_API void AddReferencedObjects( FReferenceCollector& Collector ) override;
	
	/**
	 * Copies layout and camera settings from the specified viewport
	 *
	 * @param InViewport The viewport to copy settings from
	 */
	UNREALED_API void CopyLayoutFromViewport( const FLevelEditorViewportClient& InViewport );

	/**
	 * Returns whether the provided unlocalized sprite category is visible in the viewport or not
	 *
	 * @param	InSpriteCategory	Sprite category to get the index of
	 *
	 * @return	true if the specified category is visible in the viewport; false if it is not
	 */
	UNREALED_API bool GetSpriteCategoryVisibility( const FName& InSpriteCategory ) const;

	/**
	 * Returns whether the sprite category specified by the provided index is visible in the viewport or not
	 *
	 * @param	Index	Index of the sprite category to check
	 *
	 * @return	true if the category specified by the index is visible in the viewport; false if it is not
	 */
	UNREALED_API bool GetSpriteCategoryVisibility( int32 Index ) const;

	/**
	 * Sets the visibility of the provided unlocalized category to the provided value
	 *
	 * @param	InSpriteCategory	Sprite category to get the index of
	 * @param	bVisible			true if the category should be made visible, false if it should be hidden
	 */
	UNREALED_API void SetSpriteCategoryVisibility( const FName& InSpriteCategory, bool bVisible );

	/**
	 * Sets the visibility of the category specified by the provided index to the provided value
	 *
	 * @param	Index		Index of the sprite category to set the visibility of
	 * @param	bVisible	true if the category should be made visible, false if it should be hidden
	 */
	UNREALED_API void SetSpriteCategoryVisibility( int32 Index, bool bVisible );

	/**
	 * Sets the visibility of all sprite categories to the provided value
	 *
	 * @param	bVisible	true if all the categories should be made visible, false if they should be hidden
	 */
	UNREALED_API void SetAllSpriteCategoryVisibility( bool bVisible );

	UNREALED_API void SetReferenceToWorldContext(FWorldContext& WorldContext);

	UNREALED_API void RemoveReferenceToWorldContext(FWorldContext& WorldContext);

	//~ TODO: UE_DEPRECATED(5.4, "Use HasDropPreviewElements instead.")
	/** Returns true if a placement dragging actor exists */
	UNREALED_API virtual bool HasDropPreviewActors() const override;

	//~ TODO: UE_DEPRECATED(5.4, "Use UpdateDropPreviewElements instead.")
	/**
	 * If dragging an actor for placement, this function updates its position.
	 *
	 * @param MouseX						The position of the mouse's X coordinate
	 * @param MouseY						The position of the mouse's Y coordinate
	 * @param DroppedObjects				The Objects that were used to create preview objects
	 * @param out_bDroppedObjectsVisible	Output, returns if preview objects are visible or not
	 *
	 * Returns true if preview actors were updated
	 */
	UNREALED_API virtual bool UpdateDropPreviewActors(int32 MouseX, int32 MouseY, const TArray<UObject*>& DroppedObjects, bool& out_bDroppedObjectsVisible, class UActorFactory* FactoryToUse = NULL) override;

	//~ TODO: UE_DEPRECATED(5.4, "Use DestroyDropPreviewElements instead.")
	/**
	 * If dragging an actor for placement, this function destroys the actor.
	 */
	UNREALED_API virtual void DestroyDropPreviewActors() override;

	UNREALED_API virtual bool HasDropPreviewElements() const override;
	UNREALED_API virtual bool UpdateDropPreviewElements(int32 MouseX, int32 MouseY, 
		const TArray<UObject*>& DroppedObjects, bool& out_bDroppedObjectsVisible, 
		TScriptInterface<IAssetFactoryInterface> Factory = nullptr) override;
	UNREALED_API virtual void DestroyDropPreviewElements() override;

	/**
	 * Checks the viewport to see if the given object can be dropped using the given mouse coordinates local to this viewport
	 *
	 * @param MouseX			The position of the mouse's X coordinate
	 * @param MouseY			The position of the mouse's Y coordinate
	 * @param AssetInfo			Asset in question to be dropped
	 */
	UNREALED_API virtual FDropQuery CanDropObjectsAtCoordinates(int32 MouseX, int32 MouseY, const FAssetData& AssetInfo) override;

	/**
	 * Attempts to intelligently drop the given objects in the viewport, using the given mouse coordinates local to this viewport
	 *
	 * @param MouseX			The position of the mouse's X coordinate
	 * @param MouseY			The position of the mouse's Y coordinate
	 * @param DroppedObjects	The asset objects to be placed into the editor via this viewport
	 * @param OutNewItems		The new items that were created
	 * @param Options			Additional options
	 */
	UNREALED_API virtual bool DropObjectsAtCoordinates(int32 MouseX, int32 MouseY, const TArray<UObject*>& DroppedObjects, 
		TArray<FTypedElementHandle>& OutNewItems, const FDropObjectOptions& Options = FDropObjectOptions()) override;

	//~ TODO: UE_DEPRECATED(5.4, "Use the overload that uses FDropObjectOptions instead.")
	UNREALED_API virtual bool DropObjectsAtCoordinates(int32 MouseX, int32 MouseY, const TArray<UObject*>& DroppedObjects, TArray<AActor*>& OutNewActors, bool bOnlyDropOnTarget = false, bool bCreateDropPreview = false, bool bSelectActors = true, UActorFactory* FactoryToUse = NULL) override;


	/**
	 * Sets GWorld to the appropriate world for this client
	 * 
	 * @return the previous GWorld
	 */
	UNREALED_API virtual UWorld* ConditionalSetWorld() override;

	/**
	 * Restores GWorld to InWorld
	 *
	 * @param InWorld	The world to restore
	 */
	UNREALED_API virtual void ConditionalRestoreWorld( UWorld* InWorld  ) override;

	/**
	 *	Called to check if a material can be applied to an object, given the hit proxy
	 */
	UNREALED_API bool CanApplyMaterialToHitProxy( const HHitProxy* HitProxy ) const;

	/**
	 * Static: Adds a hover effect to the specified object
	 *
	 * @param	InHoverTarget	The hoverable object to add the effect to
	 */
	static UNREALED_API void AddHoverEffect( const struct FViewportHoverTarget& InHoverTarget );

	/**
	 * Static: Removes a hover effect to the specified object
	 *
	 * @param	InHoverTarget	The hoverable object to remove the effect from
	 */
	static UNREALED_API void RemoveHoverEffect( const struct FViewportHoverTarget& InHoverTarget );

	/**
	 * Static: Clears viewport hover effects from any objects that currently have that
	 */
	static UNREALED_API void ClearHoverFromObjects();

	/** Set the global ptr to the current viewport */
	UNREALED_API void SetCurrentViewport();

	/** Set the global ptr to the last viewport to receive a key press */
	UNREALED_API void SetLastKeyViewport();

	/** 
	 * Access the 'active' actor lock.
	 *
	 * This returns the actor lock (as per GetActorLock) if that is the currently active lock. It is *not* the currently
	 * active lock if there's a valid cinematic lock actor (as per GetCinematicActorLock), since cinematics take
	 * precedence.
	 * 
	 * @return  The actor currently locked to the viewport and actively linked to the camera movements.
	 */
	TWeakObjectPtr<AActor> GetActiveActorLock() const
	{
		if (ActorLocks.CinematicActorLock.HasValidLockedActor())
		{
			return TWeakObjectPtr<AActor>();
		}
		return ActorLocks.ActorLock.LockedActor;
	}
	
	/**
	 * Find a view component to use for the specified actor. Prioritizes selected 
	 * components first, followed by camera components (then falls through to the first component that implements GetEditorPreviewInfo)
	 */
	static UNREALED_API UActorComponent* FindViewComponentForActor(AActor const* Actor);

	/** 
	 * Find the camera component that is driving this viewport, in the following order of preference:
	 *		1. Cinematic locked actor
	 *		2. User actor lock (if (bLockedCameraView is true)
	 * 
	 * @return  Pointer to a camera component to use for this viewport's view
	 */
	UCameraComponent* GetCameraComponentForView() const
	{
		const FLevelViewportActorLock& ActorLock = ActorLocks.GetLock(bLockedCameraView);
		return Cast<UCameraComponent>(FindViewComponentForActor(ActorLock.GetLockedActor()));
	}

	/**
	 * Gets the actor lock. This is the actor locked to the viewport via the viewport menus.
	 */
	const FLevelViewportActorLock& GetActorLock() const
	{
		return ActorLocks.ActorLock;
	}

	/**
	 * Gets the actor lock. This is the actor locked to the viewport via the viewport menus.
	 */
	FLevelViewportActorLock& GetActorLock()
	{
		return ActorLocks.ActorLock;
	}

	/** 
	 * Set the actor lock. This is the actor locked to the viewport via the viewport menus.
	 */
	UNREALED_API void SetActorLock(AActor* Actor);

	/** 
	 * Set the actor lock. This is the actor locked to the viewport via the viewport menus.
	 */
	UNREALED_API void SetActorLock(const FLevelViewportActorLock& InActorLock);

	/**
	 * Get the actor locked to the viewport by cinematic tools like Sequencer.
	 */
	const FLevelViewportActorLock& GetCinematicActorLock() const
	{
		return ActorLocks.CinematicActorLock;
	}

	/**
	 * Get the actor locked to the viewport by cinematic tools like Sequencer.
	 */
	FLevelViewportActorLock& GetCinematicActorLock()
	{
		return ActorLocks.CinematicActorLock;
	}

	/**
	 * Set the actor locked to the viewport by cinematic tools like Sequencer.
	 */
	UNREALED_API void SetCinematicActorLock(AActor* Actor);

	/**
	 * Set the actor locked to the viewport by cinematic tools like Sequencer.
	 */
	UNREALED_API void SetCinematicActorLock(const FLevelViewportActorLock& InActorLock);

	/**
	 * Gets the previous actor lock. This is the actor locked to the viewport via the viewport menus.
	 */
	const FLevelViewportActorLock& GetPreviousActorLock() const
	{
		return PreviousActorLocks.ActorLock;
	}

	/**
	 * Get the previous actor locked to the viewport by cinematic tools like Sequencer.
	 */
	const FLevelViewportActorLock& GetPreviousCinematicActorLock() const
	{
		return PreviousActorLocks.CinematicActorLock;
	}

	/** 
	 * Check whether this viewport is locked to the specified actor
	 */
	bool IsLockedToActor(AActor* Actor) const
	{
		return ActorLocks.HasActorLocked(Actor);
	}

	/**
	 * Check whether this viewport is locked to display a cinematic camera, like a Sequencer camera.
	 */
	bool IsLockedToCinematic() const
	{
		return ActorLocks.CinematicActorLock.HasValidLockedActor();
	}

	UNREALED_API void UpdateHoveredObjects( const TSet<FViewportHoverTarget>& NewHoveredObjects );

	/**
	 * Calling SetViewportType from Dragtool_ViewportChange
	 */
	UE_DEPRECATED(5.6, "Call FLevelEditorViewportClient::SetViewportType instead")
	UNREALED_API void SetViewportTypeFromTool(ELevelViewportType InViewportType);

	/**
	 * Static: Attempts to place the specified asset object in the level, returning one or more 
	 * newly-created objects by their FTypedElementHandles if successful.
	 */
	static UNREALED_API TArray<FTypedElementHandle> TryPlacingAssetObject(ULevel* InLevel, UObject* AssetObject,
		const UE::AssetPlacementUtil::FExtraPlaceAssetOptions& AdditionalParams,
		const FViewportCursorLocation* CursorInformation = nullptr);

	//~TODO: UE_DEPRECATED(5.4, "Use TryPlacingAssetObject instead")
	/**
	 * Static: Attempts to place the specified object in the level, returning one or more newly-created actors if successful.
	 * IMPORTANT: The placed actor's location must be first set using GEditor->ClickLocation and GEditor->ClickPlane.
	 *
	 * @param	InLevel			Level in which to drop actor
	 * @param	ObjToUse		Asset to attempt to use for an actor to place
	 * @param	CursorLocation	Location of the cursor while dropping
	 * @param	bSelectActors	If true, select the newly dropped actors (defaults: true)
	 * @param	ObjectFlags		The flags to place on the actor when it is spawned
	 * @param	FactoryToUse	The preferred actor factory to use (optional)
	 * @param	Cursor			Optional pre-calculated cursor location
	 *
	 * @return	true if the object was successfully used to place an actor; false otherwise
	 */
	static UNREALED_API TArray<AActor*> TryPlacingActorFromObject( ULevel* InLevel, UObject* ObjToUse, bool bSelectActors,
		EObjectFlags ObjectFlags, UActorFactory* FactoryToUse, 
		const FName Name = NAME_None, const FViewportCursorLocation* Cursor = nullptr);

	/** 
	 * Returns true if creating a preview actor in the viewport. 
	 */
	static bool IsDroppingPreviewActor()
	{
		return bIsDroppingPreviewActor;
	}

	/**
	 * Static: Given a texture, returns a material for that texture, creating a new asset if necessary.  This is used
	 * for dragging and dropping assets into the scene
	 *
	 * @param	UnrealTexture	Texture that we need a material for
	 *
	 * @return	The material that uses this texture, or null if we couldn't find or create one
	 */
	static UNREALED_API UObject* GetOrCreateMaterialFromTexture( UTexture* UnrealTexture );

	virtual bool UseAppTime() const override { return false; }

	/**
	 * Informs the renderer that the view is being interactively edited. (ex. rotation/translation gizmo).
	 * This state is reset on tick.
	 */
	UNREALED_API void SetEditingThroughMovementWidget();

	/**
	 * Duplicates the current Level Editor selection. Newly created elements can be selected after creation or not.
	 * E.g. Alt + Gizmo Drag creates a copy and then selects it, while pressing Alt while Gizmo Drag is being performed
	 * "stamps" a duplicate of the dragged selection, without selecting it.
	 *
	 * @param bInSelectNewElements whether the duplicates should be selected or not
	 */
	UNREALED_API void DuplicateSelection(bool bInSelectNewElements);

protected:
	/**
	* Sets the state of creating a preview actor in the viewport.
	*/
	static void SetIsDroppingPreviewActor(bool bNewIsDroppingPreviewActor)
	{
		bIsDroppingPreviewActor = bNewIsDroppingPreviewActor;
	}


	/** 
	 * Checks the viewport to see if the given blueprint asset can be dropped on the viewport.
	 * @param AssetInfo		The blueprint Asset in question to be dropped
	 *
	 * @return true if asset can be dropped, false otherwise
	 */
	UNREALED_API bool CanDropBlueprintAsset ( const struct FSelectedAssetInfo& );

	/** Called when the widget mode changes. */
	void OnWidgetModeChanged(UE::Widget::EWidgetMode NewMode);

	/** Called when editor cleanse event is triggered */
	UNREALED_API void OnEditorCleanse();

	/** Called before the editor tries to begin PIE */
	UNREALED_API void OnPreBeginPIE(const bool bIsSimulating);

	/** Callback for when an editor user setting has changed */
	UNREALED_API void HandleViewportSettingChanged(FName PropertyName);

	/** Callback for when a map is created or destroyed */
	UNREALED_API void OnMapChanged(UWorld* InWorld, EMapChangeType MapChangeType);

	/** Delegate handler for ActorMoved events */
	UNREALED_API void OnActorMoved(AActor* InActor);

public:
	/** FEditorViewportClient Interface*/
	UNREALED_API virtual void UpdateLinkedOrthoViewports(bool bInvalidate = false) override;
	UNREALED_API virtual ELevelViewportType GetViewportType() const override;
	UNREALED_API virtual void SetViewportType(ELevelViewportType InViewportType) override;
	UNREALED_API virtual void RotateViewportType() override;
	UNREALED_API virtual void OverridePostProcessSettings(FSceneView& View) override;
	UNREALED_API virtual bool ShouldLockPitch() const override;
	UNREALED_API virtual void CheckHoveredHitProxy(HHitProxy* HoveredHitProxy) override;

protected:

	UNREALED_API virtual void PerspectiveCameraMoved() override;
	UNREALED_API virtual bool GetActiveSafeFrame(float& OutAspectRatio) const override;
	UNREALED_API virtual void RedrawAllViewportsIntoThisScene() override;

private:
	// Currently used by a couple friended classes 
	// TODO: Should probably either rewrite those to not use it, or make some element-based version public
	UNREALED_API FTransform GetPreDragActorTransform(const AActor* InActor) const;

	/**
	 * Checks to see the viewports locked actor need updating
	 * @param InActor The actor to check against each viewport's locked actor.
	 * @param bCheckRealtime If true, only update viewports that have realtime enabled.
	 * @param bInvalidateHitProxies Whether to invalidate hit proxies when refreshing the viewport. Set to false during mouse-look to avoid unnecessary rebuilds.
	 */
	UNREALED_API void UpdateLockedActorViewports(const AActor* InActor, const bool bCheckRealtime, const bool bInvalidateHitProxies = true);

	/**
	 * Checks to see the viewports locked actor need updating
	 * @param InActor The actor to check against this viewport's locked actor.
	 * @param bCheckRealtime If true, only update this viewport if realtime is enabled.
	 * @param bInvalidateHitProxies Whether to invalidate hit proxies when refreshing the viewport. Set to false during mouse-look to avoid unnecessary rebuilds.
	 */
	UNREALED_API void UpdateLockedActorViewport(const AActor* InActor, const bool bCheckRealtime, const bool bInvalidateHitProxies = true);

	/**
	 * Moves the locked actor according to the viewport cameras location and rotation
	 */
	UNREALED_API void MoveLockedActorToCamera();
	
	/** @return	Returns true if the delta tracker was used to modify any selected actors or BSP.  Must be called before EndTracking(). */
	UNREALED_API bool HaveSelectedObjectsBeenChanged() const;

	/** Cache the list of elements to manipulate based on the current selection set. */
	UNREALED_API void CacheElementsToManipulate(const bool bForceRefresh = false);

	/** Reset the list of elements to manipulate */
	UNREALED_API void ResetElementsToManipulate(const bool bClearList = true);

	/** Reset the list of elements to manipulate, because the selection set they were cached from has changed */
	UNREALED_API void ResetElementsToManipulateFromSelectionChange(const UTypedElementSelectionSet* InSelectionSet);

	/** Reset the list of elements to manipulate, because the typed element registry is about to process deferred deletion */
	UNREALED_API void ResetElementsToManipulateFromProcessingDeferredElementsToDestroy();

	/** Get the selection set that associated with our level editor. */
	UNREALED_API const UTypedElementSelectionSet* GetSelectionSet() const;
	UNREALED_API UTypedElementSelectionSet* GetMutableSelectionSet() const;

	/**
	 * Called when to attempt to apply an object to a BSP surface
	 *
	 * @param	ObjToUse			The object to attempt to apply
	 * @param	ModelHitProxy		The hitproxy of the BSP model whose surface the user is clicking on
	 * @param	Cursor				Mouse cursor location
	 *
	 * @return	true if the object was applied to the object
	 */
	UNREALED_API bool AttemptApplyObjAsMaterialToSurface( UObject* ObjToUse, class HModel* ModelHitProxy, FViewportCursorLocation& Cursor );

	/**
	 * Called when an asset is dropped onto the blank area of a viewport.
	 *
	 * @param	Cursor			Mouse cursor location
	 * @param	DroppedObjects	Array of objects dropped into the viewport
	 * @param	ObjectFlags		The object flags to place on the actors that this function spawns.
	 * @param	OutNewItems		The list of actors created while dropping
	 * @param	Options			Additional options
	 *
	 * @return	true if the drop operation was successfully handled; false otherwise
	 */
	UNREALED_API bool DropObjectsOnBackground(struct FViewportCursorLocation& Cursor, const TArray<UObject*>& DroppedObjects, 
		EObjectFlags ObjectFlags, TArray<FTypedElementHandle>& OutNewItems, const FDropObjectOptions& Options);
	
	//~ TODO: UE_DEPRECATED(5.4, "Use the overload that uses FDropObjectOptions instead.")
	UNREALED_API bool DropObjectsOnBackground(struct FViewportCursorLocation& Cursor, const TArray<UObject*>& DroppedObjects, EObjectFlags ObjectFlags, TArray<AActor*>& OutNewActors, bool bCreateDropPreview = false, bool bSelectActors = true, class UActorFactory* FactoryToUse = NULL);

	/**
	* Called when an asset is dropped upon an existing actor.
	*
	* @param	Cursor				Mouse cursor location
	* @param	DroppedObjects		Array of objects dropped into the viewport
	* @param	DroppedUponActor	The actor that we are dropping upon
	* @param    DroppedUponSlot     The material slot/submesh that was identified as the drop location.  If unknown use -1.
	* @param	ObjectFlags			The object flags to place on the actors that this function spawns.
	* @param	OutNewItems			The list of items created while dropping
	* @param	Options				Additional options. Note that bOnlyDropOnTarget is ignored, since this function only drops on the actor
	*
	* @return	true if the drop operation was successfully handled; false otherwise
	*/
	UNREALED_API bool DropObjectsOnActor(struct FViewportCursorLocation& Cursor, const TArray<UObject*>& DroppedObjects,
		AActor* DroppedUponActor, int32 DroppedUponSlot, EObjectFlags ObjectFlags,
		TArray<FTypedElementHandle>& OutNewItems, const FDropObjectOptions& Options);

	//~ TODO: UE_DEPRECATED(5.4, "Use the overload that uses FDropObjectOptions instead.")
	UNREALED_API bool DropObjectsOnActor(struct FViewportCursorLocation& Cursor, const TArray<UObject*>& DroppedObjects, 
		AActor* DroppedUponActor, int32 DroppedUponSlot, EObjectFlags ObjectFlags, 
		TArray<AActor*>& OutNewActors, bool bCreateDropPreview = false, 
		bool bSelectActors = true, class UActorFactory* FactoryToUse = NULL);

	/**
	 * Called when an asset is dropped upon a BSP surface.
	 *
	 * @param	View				The SceneView for the dropped-in viewport
	 * @param	Cursor				Mouse cursor location
	 * @param	DroppedObjects		Array of objects dropped into the viewport
	 * @param	TargetProxy			Hit proxy representing the dropped upon model
	 * @param	ObjectFlags			The object flags to place on the actors that this function spawns.
	 * @param	OutNewActors		The list of actors created while dropping
	 * @param	bCreateDropPreview	If true, the actor being dropped is a preview actor (defaults: false)
	 * @param	bSelectActors		If true, select the newly dropped actors (defaults: true)
	 * @param	FactoryToUse		The preferred actor factory to use (optional)
	 *
	 * @return	true if the drop operation was successfully handled; false otherwise
	 */
	UNREALED_API bool DropObjectsOnBSPSurface(FSceneView* View, struct FViewportCursorLocation& Cursor, 
		const TArray<UObject*>& DroppedObjects, HModel* TargetProxy, EObjectFlags ObjectFlags,
		TArray<FTypedElementHandle>& OutNewItems, const FDropObjectOptions& Options);

	//UE_DEPRECATED(5.4, "Use the overload that uses FDropObjectOptions instead.")
	UNREALED_API bool DropObjectsOnBSPSurface(FSceneView* View, struct FViewportCursorLocation& Cursor, const TArray<UObject*>& DroppedObjects, HModel* TargetProxy, EObjectFlags ObjectFlags, TArray<AActor*>& OutNewActors, bool bCreateDropPreview = false, bool bSelectActors = true, UActorFactory* FactoryToUse = NULL);

	/**
	 * Called when an asset is dropped upon a manipulation widget.
	 *
	 * @param	View				The SceneView for the dropped-in viewport
	 * @param	Cursor				Mouse cursor location
	 * @param	DroppedObjects		Array of objects dropped into the viewport
	 * @param	bCreateDropPreview	If true, the actor being dropped is a preview actor (defaults: false)
	 *
	 * @return	true if the drop operation was successfully handled; false otherwise
	 */
	UNREALED_API bool DropObjectsOnWidget(FSceneView* View, struct FViewportCursorLocation& Cursor, const TArray<UObject*>& DroppedObjects, bool bCreateDropPreview = false);

	/** Project the specified elements into the world according to the current drag parameters */
	void ProjectSelectedElementsIntoWorld(FViewport* Viewport, const FVector& Drag, const FRotator& Rot);

	/** Draw additional details for brushes in the world */
	UNREALED_API void DrawBrushDetails(const FSceneView* View, FPrimitiveDrawInterface* PDI);

	/** Internal function for public FindViewComponentForActor, which finds a view component to use for the specified actor. */
	static UNREALED_API UActorComponent* FindViewComponentForActor(AActor const* Actor, TSet<AActor const*>& CheckedActors);

public:
	/** Static: List of objects we're hovering over */
	static UNREALED_API TSet< FViewportHoverTarget > HoveredObjects;
	
	/** Parent level editor that owns this viewport.  Currently, this may be null if the parent doesn't happen to be a level editor. */
	TWeakPtr< class ILevelEditor > ParentLevelEditor;

	/** List of layers that are hidden in this view */
	TArray<FName>			ViewHiddenLayers;

	/** Special volume actor visibility settings. Each bit represents a visibility state for a specific volume class. 1 = visible, 0 = hidden */
	TBitArray<>				VolumeActorVisibility;

	/** The viewport location that is restored when exiting PIE */
	FVector					LastEditorViewLocation;
	/** The viewport orientation that is restored when exiting PIE */
	FRotator				LastEditorViewRotation;

	FVector					ColorScale;

	FLinearColor			FadeColor;

	float					FadeAmount;

	bool					bEnableFading;

	bool					bEnableColorScaling;

	/** Indicates whether, of not, the base attachment volume should be drawn for this viewport. */
	bool bDrawBaseInfo;

	/**
	 * Used for drag duplication. Set to true on Alt+LMB so that the selected
	 * objects (components or actors) will be duplicated as soon as the widget is displaced.
	 */
	bool					bDuplicateOnNextDrag;

	/**
	* bDuplicateActorsOnNextDrag will not be set again while bDuplicateActorsInProgress is true.
	* The user needs to release ALT and all mouse buttons to clear bDuplicateActorsInProgress.
	*/
	bool					bDuplicateActorsInProgress;

	/**
	 * true when a brush is being transformed by its Widget
	 */
	bool					bIsTrackingBrushModification;

	/**
	 * true if only the pivot position has been moved
	 */
	bool					bOnlyMovedPivot;

	/** True if this viewport is to change its view (aspect ratio, post processing, FOV etc) to match that of the currently locked camera, if applicable */
	bool					bLockedCameraView;

	/** true if the viewport needs to restore the flag when tracking ends */
	bool					bNeedToRestoreComponentBeingMovedFlag;

	/** true if gizmo manipulation was started from a tracking event */
	bool					bHasBegunGizmoManipulation;

	/** Whether this viewport recently received focus. Used to determine whether component selection is permissible. */
	bool bReceivedFocusRecently;

	/** When enabled, the Unreal transform widget will become visible after an actor is selected, even if it was turned off via a show flag */
	bool bAlwaysShowModeWidgetAfterSelectionChanges;

private:
	//TODO: UE_DEPRECATED(5.4, TEXT("Use DropPreviewElements instead"))
	static UNREALED_API TArray< TWeakObjectPtr< AActor > > DropPreviewActors;
		
	/** The viewport clients share a list of drop preview elements.However we still want the elements
	  to be destroyed if all the viewports are destroyed, hence a static weak pointer and private shared
	  pointers. */
	static UNREALED_API TWeakPtr<FTypedElementList> StaticDropPreviewElements;
	TSharedPtr<FTypedElementList> DropPreviewElements;

	/** If currently creating a preview actor. */
	static UNREALED_API bool bIsDroppingPreviewActor;

	/** Map of element locations before a drag operation, keyed by their handle. */
	TMap<FTypedElementHandle, FTransform> PreDragElementTransforms;

	/** The elements (from the current selection set) that this viewport can manipulate (eg, via the transform gizmo) */
	bool bHasCachedElementsToManipulate = false;
	FTypedElementListRef CachedElementsToManipulate;

	/** Bit array representing the visibility of every sprite category in the current viewport */
	TBitArray<>	SpriteCategoryVisibility;

	UWorld* World;

	/** Global shared transaction for all mouse interactions. */
	FTrackingTransaction TrackingTransaction;

	/** Cached transform of pilot actor before transaction, used to allow other transactions while piloting by performing a single end-transaction */
	TOptional<FTransform> CachedPilotTransform;

	/** Represents the last known drop preview mouse position. */
	int32 DropPreviewMouseX;
	int32 DropPreviewMouseY;

	/** If this view was controlled by another view this/last frame, don't update itself */
	bool bWasControlledByOtherViewport;

	/** Whether the user is currently using the rotation / translation widget */
	bool bCurrentlyEditingThroughMovementWidget;

	/**
	 * When locked to an actor this view will be positioned in the same location and rotation as the actor.
	 * If the actor has a camera component the view will also inherit camera settings such as aspect ratio, 
	 * FOV, post processing settings, and the like.
	 *
	 * This structure allows us to keep track of two actor locks: a normal actor lock, and a lock specifically
	 * for cinematic tools like Sequencer. A viewport locked to an actor by cinematics will always take 
	 * precedent over any other.
	 */
	struct FActorLockStack
	{
		/** Get the active lock info. Cinematics take precedence. */
		const FLevelViewportActorLock& GetLock(bool bAllowActorLock = true) const
		{
			if (CinematicActorLock.LockedActor.IsValid())
			{
				return CinematicActorLock;
			}
			return bAllowActorLock ? ActorLock : FLevelViewportActorLock::None;
		}

		/** Returns whether the given actor is used as one of our locks. */
		bool HasActorLocked(const AActor* InActor) const
		{
			return CinematicActorLock.LockedActor.Get() == InActor || ActorLock.LockedActor.Get() == InActor;
		}

		FLevelViewportActorLock CinematicActorLock;
		FLevelViewportActorLock ActorLock;
	};
	FActorLockStack ActorLocks;
	FActorLockStack PreviousActorLocks;

	/** Caching for expensive FindViewComponentForActor. Invalidated once per Tick. */
	static UNREALED_API TMap<TObjectKey<AActor>, TWeakObjectPtr<UActorComponent>> ViewComponentForActorCache;

	/** If true, we switched between two different cameras. Set by cinematics, used by the motion blur to invalidate this frames motion vectors */
	bool					bEditorCameraCut;

	/** Stores the previous frame's value of bEditorCameraCut in order to reset it back to false on the next frame */
	bool					bWasEditorCameraCut;

	bool					bApplyCameraSpeedScaleByDistance;

	/** Handle to a timer event raised in ::ReceivedFocus*/
	FTimerHandle			FocusTimerHandle;
};
