// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Misc/CoreMiscDefines.h"

// Moved to UnrealEd module
UE_DEPRECATED_HEADER(5.7, "Use ViewportSelectionUtilities.h instead of LevelViewportClickHandlers.h.")
#include "ViewportSelectionUtilities.h"
