// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FLocalizationSCC;
class IAssetRegistry;
class UObject;
class UPackage;
struct FAssetData;
struct FTopLevelAssetPath;

struct FLocalizedAssetSCCUtil
{
	static bool SaveAssetWithSCC(const TSharedPtr<FLocalizationSCC>& InSourceControlInfo, UObject* InAsset);
	static bool SaveAssetWithSCC(const TSharedPtr<FLocalizationSCC>& InSourceControlInfo, UObject* InAsset, const FString& InFilename);

	static bool SavePackageWithSCC(const TSharedPtr<FLocalizationSCC>& InSourceControlInfo, UPackage* InPackage);
	static bool SavePackageWithSCC(const TSharedPtr<FLocalizationSCC>& InSourceControlInfo, UPackage* InPackage, const FString& InFilename);

	static bool DeleteAssetWithSCC(const TSharedPtr<FLocalizationSCC>& InSourceControlInfo, UObject* InAsset);

	static bool DeletePackageWithSCC(const TSharedPtr<FLocalizationSCC>& InSourceControlInfo, UPackage* InPackage);

	typedef TFunctionRef<bool(const FString&)> FSaveFileCallback;
	static bool SaveFileWithSCC(const TSharedPtr<FLocalizationSCC>& InSourceControlInfo, const FString& InFilename, const FSaveFileCallback& InSaveFileCallback);
};

struct FLocalizedAssetUtil
{
	static bool GetAssetsByPathAndClass(IAssetRegistry& InAssetRegistry, const FName InPackagePath, const FTopLevelAssetPath& InClassName, const bool bIncludeLocalizedAssets, TArray<FAssetData>& OutAssets);
	static bool GetAssetsByPathAndClass(IAssetRegistry& InAssetRegistry, const TArray<FName>& InPackagePaths, const FTopLevelAssetPath& InClassName, const bool bIncludeLocalizedAssets, TArray<FAssetData>& OutAssets);
};
