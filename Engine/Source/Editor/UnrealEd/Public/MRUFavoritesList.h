// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MRUFavoritesList : Helper class for handling MRU and favorited maps

=============================================================================*/

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "MRUList.h"

/** Simple class to represent a combined MRU and favorite map list */
class FMainMRUFavoritesList : public FMRUList
{
public:
	/** Constructor */
	UNREALED_API FMainMRUFavoritesList();

	UE_DEPRECATED(5.4, "Use the constructor that takes in an override section for the favorites as well")
	UNREALED_API FMainMRUFavoritesList(const FString& IniSectionOverride, const int32 InitMaxItems = 12);

	UNREALED_API FMainMRUFavoritesList(const FString& IniSectionOverride, const FString& IniFavoritesSectionOverride, const int32 InitMaxItems = 12);
	
	/** Destructor */
	UNREALED_API ~FMainMRUFavoritesList();
	
	/** Populate MRU/Favorites list by reading saved values from the relevant INI file */
	UNREALED_API virtual void ReadFromINI();

	/** Save off the state of the MRU and favorites lists to the relevant INI file */
	UNREALED_API virtual void WriteToINI() const;

	/**
	 * Returns the number of favorites items
	 *
	 * @return	Number of favorites
	 */
	int32 GetNumFavorites() const
	{
		return FavoriteItems.Num();
	}

	/**
	 * Add a new file item to the favorites list
	 *
	 * @param	Item	Filename of the item to add to the favorites list
	 */
	UNREALED_API void AddFavoritesItem( const FString& Item );
	
	/**
	 * Remove a file from the favorites list
	 *
	 * @param	Item	Filename of the item to remove from the favorites list
	 */
	UNREALED_API void RemoveFavoritesItem( const FString& Item );

	/**
	 * Returns whether a filename is favorited or not
	 *
	 * @param	Item	Filename of the item to check
	 *
	 * @return	true if the provided item is in the favorite's list; false if it is not
	 */
	UNREALED_API bool ContainsFavoritesItem( const FString& Item ) const;

	/**
	 * Return the favorites item specified by the provided index
	 *
	 * @param	ItemIndex	Index of the favorites item to return
	 *
	 * @return	The favorites item specified by the provided index
	 */
	UNREALED_API FString GetFavoritesItem( int32 ItemIndex ) const;

	/**
	 * Verifies that the favorites item specified by the provided index still exists. If it does not, the item
	 * is removed from the favorites list and the user is notified.
	 *
	 * @param	ItemIndex	Index of the favorites item to check
	 *
	 * @return	true if the item specified by the index was verified and still exists; false if it does not
	 */
	UNREALED_API bool VerifyFavoritesFile( int32 ItemIndex );

	/**
	 * Moves the specified favorites item to the head of the list
	 *
	 * @param	Item	Filename of the item to move
	 */
	UNREALED_API void MoveFavoritesItemToHead( const FString& Item );

	DECLARE_DELEGATE_RetVal_OneParam(bool, FDoesMRUFavoritesItemPassFilter, const FString& MRUFavoritesItem);
	/**
	* Supplies an optional delegate that can be used to filter a given MRUFavorites item
	* Useful for dynamically verifying which items should be utilized at a given time
	*
	* @param DoesMRUFavoritesItemPassFilterDelegate The delegate to use
	*/
	UNREALED_API void RegisterDoesMRUFavoritesItemPassFilterDelegate(FDoesMRUFavoritesItemPassFilter DoesMRUFavoritesItemPassFilterDelegate);

	/**
	* Unregisters the optional filter delegate
	*/
	UNREALED_API void UnregisterDoesMRUFavoritesItemPassFilterDelegate();

	/**
	* Checks the favorites item specified by the provided index against the optional 'DoesMRUFavoritesItemPassFilterDelegate'.
	*
	* @param ItemIndex Index of the favorites item to check
	*
	* @return true if the item specified by the index passes the filter or if no filter has been provided; false if it does not pass the filter
	*/
	UNREALED_API bool FavoritesItemPassesCurrentFilter(int32 ItemIndex) const;

	/**
	* Checks the MRU item specified by the provided index against the optional 'DoesMRUFavoritesItemPassFilterDelegate'.
	*
	* @param ItemIndex Index of the MRU item to check
	*
	* @return true if the item specified by the index passes the filter or if no filter has been provided; false if it does not pass the filter
	*/
	UNREALED_API bool MRUItemPassesCurrentFilter(int32 ItemIndex) const;

private:

	/** Filter delegate */
	FDoesMRUFavoritesItemPassFilter DoesMRUFavoritesItemPassFilter;

	/** Favorited items */
	TArray<FString> FavoriteItems;

	/** INI section to read/write favorite items to */
	FString INIFavoritesSection;
};
