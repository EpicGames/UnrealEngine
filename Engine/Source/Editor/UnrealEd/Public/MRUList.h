// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MRUList : Helper class for handling MRU lists

=============================================================================*/

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"

/**
 * An MRU list of files.
 */
class FMRUList
{
public:
	UNREALED_API FMRUList(const FString& InINISection, const int32 InitMaxItems = 12 );
	UNREALED_API virtual ~FMRUList();

	UNREALED_API virtual void ReadFromINI();
	UNREALED_API virtual void WriteToINI() const;
	
	/**
	 * Returns the number of MRU items
	 *
	 * @return	Number of MRU items
	 */
	int32 GetNumItems() const
	{
		return Items.Num();
	}

	/**
	 * Adds an item to the MRU list, moving it to the top.
	 *
	 * @param	Item		The long package name of the item to add.
	 */
	UNREALED_API void AddMRUItem(const FString& Item);
	
	/**
	 * Returns the index of the specified item, or INDEX_NONE if not found.
	 *
	 * @param	Item		The item to query.
	 * @return				The index of the specified item.
	 */
	UNREALED_API int32 FindMRUItemIdx(const FString& Item) const;

	/**
	 * Removes the specified item from the list if it exists.
	 *
	 * @param	Item		The item to remove.
	 */
	UNREALED_API void RemoveMRUItem(const FString& Item);

	/**
	 * Removes the item at the specified index.
	 *
	 * @param	ItemIndex	Index of the item to remove.  Must be in [0, GetMaxItems()-1].
	 */
	UNREALED_API void RemoveMRUItem(int32 ItemIndex);

	/**
	 * Clears the list of MRU items. 
	 */
	UNREALED_API void ClearMRUItems();

	/**
	 * Checks to make sure the file specified still exists.  If it does, it is
	 * moved to the top of the MRU list and we return true.  If not, we remove it
	 * from the list and return false.
	 *
	 * @param	ItemIndex		Index of the item to query
	 * @param	OutPackageName	Long package name after following redirector
	 * @return					true if the item exists, false if it doesn't
	 */
	UNREALED_API bool VerifyMRUFile(int32 ItemIndex, FString& OutPackageName);

	/**
	 * Accessor.
	 *
	 * @param	ItemIndex		Index of the item to access.
	 * @return					The filename at that index.
	 */
	const FString& GetMRUItem(int32 ItemIndex) const { return Items[ItemIndex]; }

	/**
	 * @return					The maximum number of items this MRUList can contain
	 */
	inline int32 GetMaxItems() {	return MaxItems; }

private:
	/**
	 * Make sure we don't have more than GetMaxItems() in the list.
	 */
	UNREALED_API void Cull();

	/**
	 * Moves the specified item to the top of the MRU list.
	 *
	 * @param	ItemIndex		Index of the item to bring to top.
	 */
	UNREALED_API void MoveToTop(int32 ItemIndex);

protected:

	/**
	 * Internal helper function to populate a provided array with data from the provided ini section using the provided ini key as a
	 * base, searching for the number of provided elements at a maximum
	 *
	 * @param	OutItems	Array to populate with items found in the INI file
	 * @param	INISection	Section of the INI file to check for items
	 * @param	INIKeyBase	Base string to use to search for individual items; this value will have a number appended to it while searching
	 * @param	NumElements	Maximum number of elements to search in the INI for
	 */
	static UNREALED_API void InternalReadINI( TArray<FString>& OutItems, const FString& INISection, const FString& INIKeyBase, int32 NumElements );
	
	/**
	 * Internal helper function to write out the provided array of data to the provided INI section, using the provided key base as a prefix
	 * for each item written to the INI file
	 *
	 * @param	InItems		Array of filenames to output to the INI file
	 * @param	INISection	Section of the INI file to write to
	 * @param	INIKeyBase	Base prefix to use for each item written to the INI file
	 */
	static UNREALED_API void InternalWriteINI( const TArray<FString>& InItems, const FString& INISection, const FString& INIKeyBase );

	/** Maximum MRU items */
	const int32 MaxItems;
	
	/** The filenames. */
	TArray<FString> Items;

	/** The INI section we read/write the filenames to. */
	FString INISection;
};
