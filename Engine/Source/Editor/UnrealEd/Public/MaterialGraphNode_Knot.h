// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "MaterialGraph/MaterialGraphNode.h"
#include "MaterialGraphNode_Knot.generated.h"

UCLASS(MinimalAPI)
class UMaterialGraphNode_Knot : public UMaterialGraphNode
{
	GENERATED_UCLASS_BODY()

public:
	// UEdGraphNode interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual bool ShouldOverridePinNames() const override;
	virtual FText GetPinNameOverride(const UEdGraphPin& Pin) const override;
	virtual void OnRenameNode(const FString& NewName) override;
	virtual TSharedPtr<class INameValidatorInterface> MakeNameValidator() const override;
	virtual bool CanSplitPin(const UEdGraphPin* Pin) const override;
	virtual bool IsCompilerRelevant() const override { return false; }
	virtual UEdGraphPin* GetPassThroughPin(const UEdGraphPin* FromPin) const override;
	virtual bool ShouldDrawNodeAsControlPointOnly(int32& OutInputPinIndex, int32& OutOutputPinIndex) const override { OutInputPinIndex = 0;  OutOutputPinIndex = 1; return true; }
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	// End of UEdGraphNode interface
	
	UEdGraphPin* GetInputPin() const
	{
		return UMaterialGraphNode::GetInputPin(0);
	}

	UEdGraphPin* GetOutputPin() const
	{
		return UMaterialGraphNode::GetOutputPin(0);
	}
};
