// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinition.h"
#include "MergeUtils.generated.h"

namespace MergeUtils
{
	UNREALED_API EAssetCommandResult Merge(const FAssetAutomaticMergeArgs& MergeArgs);
	UNREALED_API EAssetCommandResult Merge(const FAssetManualMergeArgs& MergeArgs);

	// download and load a temp package at a source control file path
	UNREALED_API UPackage* LoadPackageForMerge(const FString& SCFile, const FString& Revision, const UPackage* LocalPackage);
	// download a source control file path and return it's local path
	UNREALED_API FString LoadSCFileForMerge(const FString& SCFile, const FString& Revision);
}

UCLASS()
class UUndoableResolveHandler : public UObject
{
public:
	GENERATED_BODY()
	void SetManagedObject(UObject* Object);
	void MarkResolved();

	virtual void PostEditUndo() override;

private:
	FString BaseRevisionNumber;
	FString CurrentRevisionNumber;
	FString BackupFilepath;
	TWeakObjectPtr<UObject> ManagedObject;
	TSharedPtr<class ISourceControlChangelist> CheckinIdentifier;
	
	UPROPERTY()
	bool bShouldBeResolved = false;
};