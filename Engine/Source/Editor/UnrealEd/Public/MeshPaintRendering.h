// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================
	MeshPaintRendering.h: Mesh texture paint brush rendering
================================================================================*/

#pragma once

#include "CoreMinimal.h"

class FRHICommandList;
class UTextureRenderTarget2D;
class FGraphicsPipelineStateInitializer;
namespace ERHIFeatureLevel { enum Type : int; }

namespace MeshPaintRendering
{
	/** Batched element parameters for mesh paint shaders */
	struct FMeshPaintShaderParameters
	{

	public:

		// @todo MeshPaint: Should be serialized no?
		UTextureRenderTarget2D* CloneTexture;
		UTextureRenderTarget2D* PaintBrushTexture;

		FMatrix WorldToBrushMatrix;
		FVector2f PaintBrushDirectionVector;

		float BrushRadius;
		float BrushRadialFalloffRange;
		float BrushDepth;
		float BrushDepthFalloffRange;
		float BrushStrength;
		float PaintBrushRotationOffset;
		FLinearColor BrushColor;
		bool RedChannelFlag;
		bool BlueChannelFlag;
		bool GreenChannelFlag;
		bool AlphaChannelFlag;
		bool GenerateMaskFlag;
		bool bRotateBrushTowardsDirection;
		bool bUseFillBucket = false;
	};


	/** Batched element parameters for mesh paint dilation shaders used for seam painting */
	struct FMeshPaintDilateShaderParameters
	{

	public:

		UTextureRenderTarget2D* Texture0;
		UTextureRenderTarget2D* Texture1;
		UTextureRenderTarget2D* Texture2;

		float WidthPixelOffset;
		float HeightPixelOffset;

	};


	/** Binds the mesh paint vertex and pixel shaders to the graphics device */
	void UNREALED_API SetMeshPaintShaders(FRHICommandList& RHICmdList,
											FGraphicsPipelineStateInitializer& GraphicsPSOInit,
											ERHIFeatureLevel::Type InFeatureLevel, 
											const FMatrix& InTransform,
											const float InGamma,
											const FMeshPaintShaderParameters& InShaderParams );

	/** Binds the mesh paint dilation vertex and pixel shaders to the graphics device */
	void UNREALED_API SetMeshPaintDilateShaders(FRHICommandList& RHICmdList, 
													FGraphicsPipelineStateInitializer& GraphicsPSOInit,
													ERHIFeatureLevel::Type InFeatureLevel, 
													const FMatrix& InTransform,
													const float InGamma,
													const FMeshPaintDilateShaderParameters& InShaderParams );

}
