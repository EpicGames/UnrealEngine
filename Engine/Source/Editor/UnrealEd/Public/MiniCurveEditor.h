// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/Docking/TabManager.h"
#include "Subsystems/AssetEditorSubsystem.h"

class FCurveOwnerInterface;
class SCurveEditor;

class SMiniCurveEditor
	: public SCompoundWidget
	, public IAssetEditorInstance
{
public:
	SLATE_BEGIN_ARGS( SMiniCurveEditor )
		: _CurveOwner(nullptr)
		, _OwnerObject(nullptr)
		{}
		SLATE_ARGUMENT( FCurveOwnerInterface*, CurveOwner )
		SLATE_ARGUMENT( UObject*, OwnerObject )
		SLATE_ARGUMENT( TWeakPtr<SWindow>, ParentWindow )
	SLATE_END_ARGS()

	UNREALED_API void Construct(const FArguments& InArgs);
	UNREALED_API virtual ~SMiniCurveEditor() override;

	// IAssetEditorInstance interface
	UNREALED_API virtual FName GetEditorName() const override;
	UNREALED_API virtual void FocusWindow(UObject* ObjectToFocusOn) override;
	UNREALED_API virtual bool CloseWindow() override;
	UNREALED_API virtual bool CloseWindow(EAssetEditorCloseReason InCloseReason) override;
	virtual bool IsPrimaryEditor() const override { return true; };
	virtual void InvokeTab(const struct FTabId& TabId) override {}
	UNREALED_API virtual TSharedPtr<class FTabManager> GetAssociatedTabManager() override;
	UNREALED_API virtual double GetLastActivationTime() override;
	UNREALED_API virtual void RemoveEditingAsset(UObject* Asset) override;

private:
	/** Sometimes false when this is a transient or popout window, indicating it shouldn't be tracked as an asset editor. */
	bool bHasOwner = true;

	float ViewMinInput = 0.0f;
	float ViewMaxInput = 1.0f;

	TSharedPtr<class SCurveEditor> TrackWidget;

	float GetViewMinInput() const { return ViewMinInput; }
	float GetViewMaxInput() const { return ViewMaxInput; }

	/** Return length of timeline */
	UNREALED_API float GetTimelineLength() const;

	UNREALED_API void SetInputViewRange(float InViewMinInput, float InViewMaxInput);

protected:
	TWeakPtr<SWindow> WidgetWindow;
};
