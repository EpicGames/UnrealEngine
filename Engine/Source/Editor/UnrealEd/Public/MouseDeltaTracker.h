// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "UnrealWidgetFwd.h"

class FCanvas;
class FDragTool;
class FEditorViewportClient;
class FPrimitiveDrawInterface;
class FSceneView;
struct FInputEventState;

/**
 * Keeps track of mouse movement deltas in the viewports.
 */
class FMouseDeltaTracker
{
public:

	UNREALED_API FMouseDeltaTracker();
	UNREALED_API ~FMouseDeltaTracker();

	/**
	 * Begin tracking at the specified location for the specified viewport.
	 */
	UNREALED_API void StartTracking(FEditorViewportClient* InViewportClient, const int32 InX, const int32 InY, const FInputEventState& InInputState, bool bNudge = false, bool bResetDragToolState = true);

	/**
	 * Called when a mouse button has been released.  If there are no other
	 * mouse buttons being held down, the internal information is reset.
	 */
	UNREALED_API bool EndTracking(FEditorViewportClient* InViewportClient);

	/**
	 * Adds delta movement into the tracker.
	 */
	UNREALED_API void AddDelta(FEditorViewportClient* InViewportClient, FKey InKey, const int32 InDelta, bool InNudge);

	/**
	* Returns the current delta.
	*/
	UNREALED_API const FVector GetDelta() const;

	/**
	 * Returns the current snapped delta.
	 */
	UNREALED_API const FVector GetDeltaSnapped() const;

	/**
	* Returns the absolute delta since dragging started.
	*/
	UNREALED_API const FVector GetAbsoluteDelta() const;

	/**
	* Returns the absolute snapped delta since dragging started. 
	*/
	UNREALED_API const FVector GetAbsoluteDeltaSnapped() const;

	/**
	 * Returns the screen space delta since dragging started.
	 */
	UNREALED_API const FVector GetScreenDelta() const;

	/**
	 * Returns the raw mouse delta in pixels since dragging started.
	 */
	UNREALED_API const FVector GetRawDelta() const;

	/**
	 * Returns the unsnapped start position of the current mouse drag. (This will be zero if there is no drag in progress)
	 */
	UNREALED_API const FVector GetDragStartPos() const;

	/**
	 * Returns if the user used a modifier to drag a selected item. (Rather than using a widget handle)
	 */
	UNREALED_API const bool GetUsedDragModifier() const;

	/**
	 * Resets the UsedDragModifier flag
	 */
	UNREALED_API void ResetUsedDragModifier();

	/**
	 * Converts the delta movement to drag/rotation/scale based on the viewport type or widget axis.
	 */
	UNREALED_API void ConvertMovementDeltaToDragRot(FSceneView* InView, FEditorViewportClient* InViewportClient, FVector& InDragDelta, FVector& OutDrag, FRotator& OutRotation, FVector& OutScale) const;
	/**
	 * Absolute Translation conversion from mouse position on the screen to widget axis movement/rotation.
	 */
	UNREALED_API void AbsoluteTranslationConvertMouseToDragRot(FSceneView* InView, FEditorViewportClient* InViewportClient, FVector& OutDrag, FRotator& OutRotation, FVector& OutScale ) const;

	/**
	 * Subtracts the specified value from End and EndSnapped.
	 */
	UNREALED_API void ReduceBy(const FVector& In);

	/**
	 * @return		true if a drag tool is being used by the tracker, false otherwise.
	 */
	UNREALED_API bool UsingDragTool() const;

	/**
	 * @return True if any mouse movement has happened since tracking started
	 */
	bool HasReceivedDelta() const { return bHasReceivedAddDelta; }

	/**
	 * Marks that something caused the equivalent of mouse dragging, but with other means (keyboard short cut, mouse wheel).  Allows suppression of context menus with flight camera, etc
	 */
	void SetExternalMovement (bool InExternalMovement = true)			{ bExternalMovement = InExternalMovement;};
	/**
	 * @return		true if something caused external movement of the mouse (keyboard, mouse wheel)
	 */
	bool WasExternalMovement (void) const { return bExternalMovement; }

	/**
	 * Renders the drag tool.  Does nothing if no drag tool exists.
	 */
	UNREALED_API void Render3DDragTool(const FSceneView* View, FPrimitiveDrawInterface* PDI);

	/**
	 * Renders the drag tool.  Does nothing if no drag tool exists.
	 */
	UNREALED_API void RenderDragTool(const FSceneView* View, FCanvas* Canvas);

	/** 
	 * @return 		the widget mode last time this tracker object started tracking
	 */
	UE::Widget::EWidgetMode GetTrackingWidgetMode() const { return TrackingWidgetMode; }
private:
	/**
	 * Starts using a drag tool if needed
	 */
	UNREALED_API void ConditionalBeginUsingDragTool( FEditorViewportClient* InViewportClient );
private:
	/** The unsnapped start position of the current mouse drag. */
	FVector Start;
	/** The snapped start position of the current mouse drag. */
	FVector StartSnapped;
	/** The screen space start position of the current mouse drag (may be scaled or rotated according to the ortho zoom or view). */
	FVector StartScreen;
	/** The unsnapped end position of the current mouse drag. */
	FVector End;
	/** The snapped end position of the current mouse drag. */
	FVector EndSnapped;
	/** The screen space end position of the current mouse drag (may be scaled or rotated according to the ortho zoom or view). */
	FVector EndScreen;
	/** The raw unscaled mouse delta in pixels */
	FVector RawDelta;

	/** The amount that the End vectors have been reduced by since dragging started, this is added to the deltas to get an absolute delta. */
	FVector ReductionAmount;

	/**
	 * If there is a dragging tool being used, this will point to it.
	 * Gets newed/deleted in StartTracking/EndTracking.
	 */
	TSharedPtr<FDragTool> DragTool;


	/** Keeps track of whether AddDelta has been called since StartTracking. */
	bool bHasReceivedAddDelta;

	/** True if we attempted to use a drag tool since StartTracking was called */
	bool bHasAttemptedDragTool;

	/** Tracks whether keyboard/mouse wheel/etc have caused simulated mouse movement.  Reset on StartTracking */
	bool bExternalMovement;

	/** Tracks if the user used a modifier to drag a selected item. (Rather than using a widget handle).Reset on StartTracking */
	bool bUsedDragModifier;

	/** Tracks whether the drag tool is in the process of being deleted (to protect against reentrancy) */
	bool bIsDeletingDragTool;

	/** Stores the widget mode active when the tracker begins tracking to help stop it change mid-track */
	UE::Widget::EWidgetMode TrackingWidgetMode;

	/**
	 * Sets the current axis of the widget for the specified viewport.
	 *
	 * @param	InViewportClient		The viewport whose widget axis is to be set.
	 */
	UNREALED_API void DetermineCurrentAxis(FEditorViewportClient* InViewportClient);

};

static const float MOUSE_CLICK_DRAG_DELTA = 4.0f*4.0f;
