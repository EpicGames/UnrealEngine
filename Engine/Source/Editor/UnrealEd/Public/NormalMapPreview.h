// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	NormalMapPreview.h: Definitions for previewing normal maps.
==============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "BatchedElements.h"

/**
 * Batched element parameters for previewing normal maps.
 *
 * Deprecated, do not use.  Use FBatchedElementTexture2DPreviewParameters instead with normal map flag. 
 */
//UE_DEPRECATED(5.5, "Use FBatchedElementTexture2DPreviewParameters instead with normal map flag")
class FNormalMapBatchedElementParameters : public FBatchedElementParameters
{
	/** Binds vertex and pixel shaders for this element */
	UNREALED_API virtual void BindShaders(
		FRHICommandList& RHICmdList,
		FGraphicsPipelineStateInitializer& GraphicsPSOInit,
		ERHIFeatureLevel::Type InFeatureLevel,
		const FMatrix& InTransform,
		const float InGamma,
		const FMatrix& ColorWeights,
		const FTexture* Texture) override;
};
