// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ObjectTools.h: Object-related utilities

=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Serialization/ArchiveUObject.h"
#include "AssetRegistry/AssetData.h"
#include "UObject/GCObject.h"
#include "CollectionManagerTypes.h"

class FNamePermissionList;
class FTextureRenderTargetResource;
class ICollectionContainer;
class SWindow;
class UExporter;
class UFactory;
class USoundWave;

DECLARE_DELEGATE_RetVal_ThreeParams(bool, FLoadThumbnailsFromPackage, const FString& /*InPackageFileName*/, const TSet<FName>& /*InObjectFullNames*/, FThumbnailMap& /*InOutThumbnails*/);

namespace ObjectTools
{
	/** A simple struct to represent the package group name triplet */
	struct FPackageGroupName
	{
		FString PackageName;
		FString GroupName;
		FString ObjectName;
	};

	/** Returns true if the specified object can be displayed in a content browser */
	UNREALED_API bool IsObjectBrowsable( UObject* Obj );

	/**
	 * An archive for collecting object references that are top-level objects.
	 */
	class FArchiveTopLevelReferenceCollector : public FArchiveUObject
	{
	public:
		FArchiveTopLevelReferenceCollector(
			TArray<UObject*>* InObjectArray,
			const TArray<UObject*>& InIgnoreOuters,
			const TArray<UClass*>& InIgnoreClasses );


		/** @return		true if the specified objects should be serialized to determine asset references. */
		FORCEINLINE bool ShouldSearchForAssets(const UObject* Object) const
		{
			// Discard class default objects.
			if ( Object->IsTemplate(RF_ClassDefaultObject) )
			{
				return false;
			}
			// Check to see if we should be based on object class.
			if ( IsAnIgnoreClass(Object) )
			{
				return false;
			}
			// Discard sub-objects of outer objects to ignore.
			if ( IsInIgnoreOuter(Object) )
			{
				return false;
			}
			return true;
		}

		/** @return		true if the specified object is an 'IgnoreClasses' type. */
		FORCEINLINE bool IsAnIgnoreClass(const UObject* Object) const
		{
			for ( int32 ClassIndex = 0 ; ClassIndex < IgnoreClasses.Num() ; ++ClassIndex )
			{
				if ( Object->IsA(IgnoreClasses[ClassIndex]) )
				{
					return true;
				}
			}
			return false;
		}

		/** @return		true if the specified object is not a subobject of one of the IngoreOuters. */
		FORCEINLINE bool IsInIgnoreOuter(const UObject* Object) const
		{
			for ( int32 OuterIndex = 0 ; OuterIndex < IgnoreOuters.Num() ; ++OuterIndex )
			{
				if( ensure( IgnoreOuters[ OuterIndex ] != NULL ) )
				{
					if ( Object->IsIn(IgnoreOuters[OuterIndex]) )
					{
						return true;
					}
				}
			}
			return false;
		}

	private:
		/**
		 * UObject serialize operator implementation
		 *
		 * @param Object	reference to Object reference
		 * @return reference to instance of this class
		 */
		FArchive& operator<<( UObject*& Obj );

		/** Stored pointer to array of objects we add object references to */
		TArray<UObject*>*		ObjectArray;

		/** Only objects not within these objects will be considered.*/
		const TArray<UObject*>&	IgnoreOuters;

		/** Only objects not of these types will be considered.*/
		const TArray<UClass*>&	IgnoreClasses;
	};

	/** Target package and object name for moving an asset. */
	class FMoveInfo
	{
	public:
		FString FullPackageName;
		FString NewObjName;

		void Set(const TCHAR* InFullPackageName, const TCHAR* InNewObjName);

		/** @return		true once valid (non-empty) move info exists. */
		bool IsValid() const;
	};

	enum class EAllowCancelDuringDelete : uint8
	{
		AllowCancel,
		CancelNotAllowed
	};

	enum class EAllowCancelDuringPrivatize : uint8
	{
		AllowCancel,
		CancelNotAllowed
	};

	/**
	 * Handles fully loading packages for a set of passed in objects.
	 *
	 * @param	Objects				Array of objects whose packages need to be fully loaded
	 * @param	OperationString		Localization key for a string describing the operation; appears in the warning string presented to the user.
	 *
	 * @return true if all packages where fully loaded, false otherwise
	 */
	bool HandleFullyLoadingPackages( const TArray<UObject*>& Objects, const FText& OperationText );


	/** Duplicates a list of objects
	 *
	 * @param	SelectedObjects			The objects to delete.
	 * @param	SourcePath				A path to use to form a relative path for the copied objects.
	 * @param	DestinationPath			A path to use as a default destination for the copied objects.
	 * @param	OpenDialog				If true, a dialog will open to prompt the user for path information.
	 * @param	OutNewObjects	If non-NULL, returns the list of duplicated objects.
	 */
	UNREALED_API void DuplicateObjects( const TArray<UObject*>& SelectedObjects, const FString& SourcePath = TEXT(""), const FString& DestinationPath = TEXT(""), bool bOpenDialog = true, TArray<UObject*>* OutNewObjects = NULL );

	/** Duplicates a single object
	 *
	 * @param	Object									The objects to delete.
	 * @param	PGN										The new package, group, and name of the object.
	 * @param	InOutPackagesUserRefusedToFullyLoad		A set of packages the user opted out of fully loading. This is used internally to prevent asking multiple times.
	 * @param	bPromptToOverwrite						If true the user will be prompted to overwrite if duplicating to an existing object.  If false, the duplication will always happen
	 * @param	DuplicatedObjects						If non-null, the map is filled with all objects (including sub-objects) that were duplicated with their source object as key
	 * @retun	The duplicated object or NULL if a failure occurred.
	 */
	UNREALED_API UObject* DuplicateSingleObject(UObject* Object, const FPackageGroupName& PGN, TSet<UPackage*>& InOutPackagesUserRefusedToFullyLoad, bool bPromptToOverwrite = true, TMap<TSoftObjectPtr<UObject>, TSoftObjectPtr<UObject>>* DuplicatedObjects = nullptr);

	/** Helper struct to detail the results of a consolidation operation */
	struct FConsolidationResults : public FGCObject
	{
		/** FGCObject interface; Serialize any object references */
		virtual void AddReferencedObjects( FReferenceCollector& Collector ) override
		{
			Collector.AddReferencedObjects( DirtiedPackages );
			Collector.AddReferencedObjects( InvalidConsolidationObjs );
			Collector.AddReferencedObjects( FailedConsolidationObjs );
		}
		virtual FString GetReferencerName() const override
		{
			return TEXT("ObjectTools::FConsolidationResults");
		}

		/** Packages dirtied by a consolidation operation */
		TArray<TObjectPtr<UPackage>>	DirtiedPackages;

		/** Objects which were not valid for consolidation */
		TArray<TObjectPtr<UObject>>	InvalidConsolidationObjs;

		/** Objects which failed consolidation (partially consolidated) */
		TArray<TObjectPtr<UObject>>	FailedConsolidationObjs;
	};

	/** Helper struct for batch replacements where Old references get replaced with New */
	struct FReplaceRequest
	{
		UObject* New = nullptr;
		TArrayView<UObject*> Old; 
	};

	/**
	 * Consolidates objects by replacing all references/uses of the provided "objects to consolidate" with references to the "object to consolidate to." This is
	 * useful for situations such as when a particular asset is duplicated in multiple places and it would be handy to allow all uses to point to one particular copy
	 * of the asset. When executed, the function first attempts to directly replace all relevant references located within objects that are already loaded and in memory.
	 * Next, it deletes the "objects to consolidate," leaving behind object redirectors to the "object to consolidate to" in their wake.
	 *
	 * @param	ObjectToConsolidateTo	Object to which all references of the "objects to consolidate" will instead refer to after this operation completes
	 * @param	ObjectsToConsolidate	Objects which all references of which will be replaced with references to the "object to consolidate to"; each will also be deleted
	 * @param	Requests				Batch of consolidations. All objects consilidated to, i.e. FReplaceRequest::New, must be non-null.
	 *
	 * @note	This function performs NO type checking, by design. It is potentially dangerous to replace references of one type with another, so utilize caution.
	 * @note	The "objects to consolidate" are DELETED by this function.
	 *
	 * @return	Structure of consolidation results, specifying which packages were dirtied, which objects failed consolidation (if any), etc.
	 */
	UNREALED_API FConsolidationResults ConsolidateObjects(UObject* ObjectToConsolidateTo, TArray<UObject*>& ObjectsToConsolidate, bool bShowDeleteConfirmation = true );
	UNREALED_API FConsolidationResults ConsolidateObjects(UObject* ObjectToConsolidateTo, TArray<UObject*>& ObjectsToConsolidate, TSet<UObject*>& ObjectsToConsolidateWithin, TSet<UObject*>& ObjectsToNotConsolidateWithin, bool bShouldDeleteAfterConsolidate, bool bWarnAboutRootSet = true);
	UNREALED_API FConsolidationResults ConsolidateObjects(TArrayView<FReplaceRequest> Requests, TSet<UObject*>& ObjectsToConsolidateWithin, TSet<UObject*>& ObjectsToNotConsolidateWithin, bool bShouldDeleteAfterConsolidate, bool bWarnAboutRootSet = true);

	
	UNREALED_API void CompileBlueprintsAfterRefUpdate(const TArray<UObject*>& ObjectsConsolidatedWithin);
	/**
	 * Copies references for selected generic browser objects to the clipboard.
	 */
	UNREALED_API void CopyReferences( const TArray< UObject* >& SelectedObjects ); // const

	UNREALED_API void ShowReferencers( const TArray< UObject* >& SelectedObjects ); // const

	/**
	 * Displays a tree(currently) of all assets which reference the passed in object.
	 *
	 * @param ObjectToGraph		The object to find references to.
	 * @param InBrowsableTypes	A mapping of classes to browsable types.  The tool only shows browsable types or actors
	 */
	UNREALED_API void ShowReferenceGraph( UObject* ObjectToGraph );
	/**
	 * Displays all of the objects the passed in object references
	 *
	 * @param	Object			Object whose references should be displayed
	 */
	UNREALED_API void ShowReferencedObjs(UObject* Object);
	/**
	 * Displays all of the objects the passed in object references
	 *
	 * @param	Object			Object whose references should be displayed
	 * @param	CollectionName	Name of a collection that needs to be made with these referenced objects
	 * @param	ShareType		The share type of any created collection
	 */
	UE_DEPRECATED(5.6, "Use the ICollectionContainer overload instead.")
	UNREALED_API void ShowReferencedObjs(UObject* Object, const FString& CollectionName, ECollectionShareType::Type ShareType = ECollectionShareType::CST_Private);
	/**
	 * Displays all of the objects the passed in object references
	 *
	 * @param	Object				Object whose references should be displayed
	 * @param	CollectionContainer	The collection container for any created collection
	 * @param	CollectionName		Name of a collection that needs to be made with these referenced objects
	 * @param	ShareType			The share type of any created collection
	 */
	UNREALED_API void ShowReferencedObjs(UObject* Object, ICollectionContainer* CollectionContainer, FName CollectionName, ECollectionShareType::Type ShareType);

	/**
	 * Select the object referencers in the level
	 *
	 * @param	Object			Object whose references are to be selected
	 *
	 */
	UNREALED_API void SelectActorsInLevelDirectlyReferencingObject( UObject* RefObj );

	/**
	 * Select the object and it's external referencers' referencers in the level.
	 * This function calls AccumulateObjectReferencersForObjectRecursive to
	 * recursively build a list of objects to check for referencers in the level
	 *
	 * @param	Object				Object whose references are to be selected
	 * @param	bRecurseMaterial	Whether or not we're allowed to recurse the material
	 *
	 */
	UNREALED_API void SelectObjectAndExternalReferencersInLevel( UObject* Object, const bool bRecurseMaterial );

	/**
	 * Recursively add the objects referencers to a single array
	 *
	 * @param	Object				Object whose references are to be selected
	 * @param	Referencers			Array of objects being referenced in level
	 * @param	bRecurseMaterial	Whether or not we're allowed to recurse the material
	 *
	 */
	UNREALED_API void AccumulateObjectReferencersForObjectRecursive( UObject* Object, TArray<UObject*>& Referencers, const bool bRecurseMaterial );

	/**
	 * Shows a confirmation dialog asking the user if it is ok to delete the packages containing the supplied objects
	 *
	 * @param	ObjectsToDelete		The list of objects to delete
	 *
	 * @return true if the user accepted the dialog or no dialog was necessary
	 */
	bool ShowDeleteConfirmationDialog ( const TArray<UObject*>& ObjectsToDelete );

	/**
	 * Collects garbage and marks truely empty packages for delete
	 *
	 * @param	ObjectsDeletedSuccessfully		The list of objects that were recently deleted but not yet cleaned up
	 */
	UNREALED_API void CleanupAfterSuccessfulDelete ( const TArray<UPackage*>& ObjectsDeletedSuccessfully, bool bPerformReferenceCheck = true );

	/**
	 * Deletes the list of objects
	 *
	 * @param	ObjectsToDelete		The list of objects to delete
	 * @param	bShowConfirmation	True when a dialog should prompt the user that they are about to delete something
	 *
	 * @return The number of objects successfully deleted
	 */
	UNREALED_API int32 DeleteObjects( const TArray< UObject* >& ObjectsToDelete, bool bShowConfirmation = true, EAllowCancelDuringDelete AllowCancelDuringDelete = EAllowCancelDuringDelete::AllowCancel);

	struct FDeleteItemsParameters
	{
		TArray<UObject*> ObjectsToDelete;
		TArray<FString> FoldersToDelete;
		bool bShowConfirmation = true;
		EAllowCancelDuringDelete AllowCancelDuringDelete = EAllowCancelDuringDelete::AllowCancel;
	};

	struct FDeleteItemsResult
	{
		int32 ObjectsDeleted = 0;
		int32 FoldersDeleted = 0;
	};
	/** 
	 * Deletes items provided (objects, folders)
	 * @param InOptions The payload and options for the deletion operation
	 * @return FDeleteItemsResult The items successfully deleted
	 */ 
	UNREALED_API FDeleteItemsResult DeleteItems(const FDeleteItemsParameters& InOptions);

	/**
	* Privatizes the list of objects (marks their packages as NotExternallyReferencable)
	* 
	* @param InObjectsToPrivatize The list of objects to privatize
	* @param bShowConfirmation True when the dialog should prompt the user that they are about to privatize something and doing so would break references
	* @param AllowCancelDuringPrivatize Whether or not canceling is allowed when not showing the confirmation dialog
	* 
	* @return The number of objects successfully privatized
	*/
	UNREALED_API int32 PrivatizeObjects(const TArray<UObject*>& InObjectsToPrivatize, bool bShowConfirmation = true, EAllowCancelDuringPrivatize AllowCancelDuringPrivatize = EAllowCancelDuringPrivatize::AllowCancel, const EAssetAccessSpecifier InAssetAccessSpecifier = EAssetAccessSpecifier::Private);

	/**
	* Deletes the list of objects without checking if they are still being used.  This should not be called directly
	* this is primarily used by the delete system after it has done the work of making sure it's safe to delete.
	*
	* @param	ObjectsToDelete		The list of objects to delete
	*
	* @return The number of objects successfully deleted
	*/
	UNREALED_API int32 DeleteObjectsUnchecked( const TArray< UObject* >& ObjectsToDelete );

	/**
	* Deletes the list of objects
	*
	* @param	AssetsToDelete		The list of assets to delete
	* @param	bShowConfirmation	True when a dialog should prompt the user that they are about to delete something
	*
	* @return The number of assets successfully deleted
	*/
	UNREALED_API int32 DeleteAssets( const TArray<FAssetData>& AssetsToDelete, bool bShowConfirmation = true );
	
	/**
	* Privatizes the list of Assets (marks their packages as NotExternallyReferenceable)
	* 
	* @param AssetsToPrivatize The list of assets to privatize
	* @param bShowConfirmation True when a dialog should prompt the user that they are about to privatize something and going to break references
	* 
	* @return The number of assets successfully privatized
	*/
	UNREALED_API int32 PrivatizeAssets(const TArray<FAssetData>& AssetsToPrivatize, bool bShowConfirmation = true, const EAssetAccessSpecifier InAssetAccessSpecifier = EAssetAccessSpecifier::Private);

	/**
	 * Delete a single object
	 *
	 * @param	ObjectToDelete		The object to delete
	 *
	 * @return If the object was successfully
	 */
	UNREALED_API bool DeleteSingleObject( UObject* ObjectToDelete, bool bPerformReferenceCheck = true );

	/**
	 * Deletes the list of objects
	 *
	 * @param	ObjectsToDelete		The list of objects to delete
	 * @param   ShowConfirmation    Show the confirmation dialog.
	 *
	 * @return The number of objects successfully deleted
	 */
	UNREALED_API int32 ForceDeleteObjects( const TArray< UObject* >& ObjectsToDelete, bool ShowConfirmation = true );

	/**
	 * Forcefully replaces references to passed in objects
	 *
	 * @param ObjectToReplaceWith	Any references found to 'ObjectsToReplace' will be replaced with this object.  If the object is nullptr references will be nulled.
	 * @param ObjectsToReplace		An array of objects that should be replaced with 'ObjectToReplaceWith'
	 * @param Requests				Batch of replacements where all FReplaceRequest::Old are replaced with FReplaceRequest::New for each request
	 */
	UNREALED_API void ForceReplaceReferences(UObject* ObjectToReplaceWith, TArray<UObject*>& ObjectsToReplace);
	UNREALED_API void ForceReplaceReferences(UObject* ObjectToReplaceWith, TArray<UObject*>& ObjectsToReplace, TSet<UObject*>& ObjectsToReplaceWithin);
	UNREALED_API void ForceReplaceReferences(TArrayView<FReplaceRequest> Requests, TSet<UObject*>& ObjectsToReplaceWithin);

	/**
	 * Gathers additional objects to delete such as map built data
	 *
	 * @param	ObjectsToDelete		List of objects to delete that is appended by additional objects that should be deleted
	 *
	 */
	UNREALED_API void AddExtraObjectsToDelete(TArray< UObject* >& ObjectsToDelete);

	/**
	 * Gathers additional localized variants to delete
	 *
	 * @param	InAssetsToDelete				Initial list of assets to delete
	 * @param	OutAssetsWithVariantsToDelete	Final list of assets to delete (initial + variants if the user requested it)
	 *
	 * @return Whether or not the function succeeded. If it returns false, the delete process should be interrupted.
	 */
	UNREALED_API bool AddExtraLocalizedVariantsToDelete(const TArray<FAssetData>& InAssetsToDelete, TArray<FAssetData>& OutAssetsWithVariantsToDelete);

	/**
	 * Utility function to compose a string list of referencing objects
	 *
	 * @param References			Array of references to the relevant object
	 * @param RefObjNames			String list of all objects
	 * @param DefObjNames			String list of all objects referenced in default properties
	 *
	 * @return Whether or not any objects are in default properties
	 */
	UNREALED_API bool ComposeStringOfReferencingObjects( TArray<FReferencerInformation>& References, FString& RefObjNames, FString& DefObjNames );

	/** Information that can be gathered from the move dialog. */
	struct FMoveDialogInfo
	{
		FPackageGroupName PGN;
		bool bOkToAll;
		bool bSavePackages;
		bool bPromptForRenameOnConflict;

		FMoveDialogInfo()
			: bOkToAll(0)
			, bSavePackages(0)
			, bPromptForRenameOnConflict(1)
		{}
	};

	/** Sends the redirector to the deleted redirectors package where it will be cleaned up later */
	UNREALED_API void DeleteRedirector(UObjectRedirector* Redirector);

	/**
	 * Helper function for RenameObjectsInternal. This function Updates a FMoveDialogInfo with information from the user. If OkToAll was pressed, it simply updates the relevant information.
	 *
	 * @param DialogTitle				The title text for the dialog, if it opens
	 * @param Object					The object for which to provide move information.
	 * @param bUniqueDefaultName		If true, when the user is prompted for a name the default supplied name will be unique.
	 * @param SourcePath				A path to use to form a relative path for the renamed objects.
	 * @param DestinationPath			The default target path for the objects. If empty string, the default will be based on the the package name.
	 * @param InOutInfo					The information gathered from the move dialog.
	 * @return true if the move information was successfully extracted from the dialog.
	 */
	UNREALED_API bool GetMoveDialogInfo(const FText& DialogTitle, UObject* Object, bool bUniqueDefaultName, const FString& SourcePath, const FString& DestinationPath, FMoveDialogInfo& InOutInfo);

	/**
	 * Internal implementation of rename objects with refs
	 *
	 * @param Objects					The objects to rename
	 * @param bLocPackages				If true, the objects belong in localized packages
	 * @param ObjectToLanguageExtMap	A mapping of object to matching language (for fixing up localization if the objects are moved ).  Note: Not used if bLocPackages is false
	 * @param SourcePath				A path to use to form a relative path for the renamed objects.
	 * @param DestinationPath			The default target path for the objects. If empty string, the default will be based on the the package name.
	 * @param	OpenDialog				If true, a dialog will open to prompt the user for path information.
	 * @return true when all objects were renamed successfully
	 */
	UNREALED_API bool RenameObjectsInternal( const TArray<UObject*>& Objects, bool bLocPackages, const TMap< UObject*, FString >* ObjectToLanguageExtMap, const FString& SourcePath, const FString& DestinationPath, bool bOpenDialog );

	/**
	 * Renames a single object
	 *
	 * @param Object								The object to rename
	 * @param PGN									The new package, group, and name of the object.
	 * @param InOutPackagesUserRefusedToFullyLoad	A set of packages the user opted out of fully loading. This is used internally to prevent asking multiple times.
	 * @param InOutErrorMessage						A string with any errors that occurred during the rename.
	 * @param ObjectToLanguageExtMap				A mapping of object to matching language (for fixing up localization if the objects are moved ).  Note: Not used if bLocPackages is false
	 * @param bLeaveRedirector						If true, a redirector will be left to allow unloaded assets to maintain a reference to the renamed object
	 * @return true when the object was renamed successfully
	 */
	UNREALED_API bool RenameSingleObject(UObject* Object, FPackageGroupName& PGN, TSet<UPackage*>& InOutPackagesUserRefusedToFullyLoad, FText& InOutErrorMessage, const TMap< UObject*, FString >* ObjectToLanguageExtMap = NULL, bool bLeaveRedirector = true);

	/**
	 * Finds all language variants for the passed in sound wave
	 *
	 * @param OutObjects	A list of found localized sound wave objects
	 * @param OutObjectToLanguageExtMap	A mapping of sound wave objects to their language extension
	 * @param Wave	The sound wave to search for
	 */
	UNREALED_API void AddLanguageVariants( TArray<UObject*>& OutObjects, TMap< UObject*, FString >& OutObjectToLanguageExtMap, USoundWave* Wave );


	/**
	 * Renames an object and leaves redirectors so other content that references it does not break
	 * Also renames all loc instances of the same asset
	 * @param Objects					The objects to rename
	 * @param bIncludeLocInstances		If true, the objects belong in localized packages
	 * @param SourcePath				A path to use to form a relative path for the renamed objects.
	 * @param DestinationPath			The default target path for the objects. If empty string, the default will be based on the the package name.
	 * @param OpenDialog				If true, a dialog will open to prompt the user for path information.
	 * @return true when all objects were renamed successfully
	 */
	UNREALED_API bool RenameObjects( const TArray< UObject* >& SelectedObjects, bool bIncludeLocInstances, const FString& SourcePath = TEXT(""), const FString& DestinationPath = TEXT(""), bool bOpenDialog = true );

	/** Converts all invalid object name characters to _ */
	UNREALED_API FString SanitizeObjectName(const FString& InObjectName);
	/** Converts all invalid object path characters to _ */
	UNREALED_API FString SanitizeObjectPath(const FString& InObjectPath);
	/** Converts all specified invalid characters to _ */
	UNREALED_API FString SanitizeInvalidChars(const FString& InText, const FString& InvalidChars);
	/** Converts all specified invalid characters to _ */
	UNREALED_API FString SanitizeInvalidChars(const FString& InText, const TCHAR* InvalidChars);
	/** Converts all specified invalid characters to _ */
	UNREALED_API void SanitizeInvalidCharsInline(FString& InText, const TCHAR* InvalidChars);

	/**
	 * Populates two strings with all of the file types and extensions the provided factory supports.
	 *
	 * @param	InFactory		Factory whose supported file types and extensions should be retrieved
	 * @param	out_Filetypes	File types supported by the provided factory, concatenated into a string
	 * @param	out_Extensions	Extensions supported by the provided factory, concatenated into a string
	 * @param   SupportedExtensions			If not null only extension in the list can be added
	 */
	UNREALED_API void GenerateFactoryFileExtensions( const UFactory* InFactory
		, FString& out_Filetypes
		, FString& out_Extensions
		, TMultiMap<uint32, UFactory*>& out_FilterIndexToFactory);

	/**
	 * Populates two strings with all of the file types and extensions the provided factories support.
	 *
	 * @param	InFactories		Factories whose supported file types and extensions should be retrieved
	 * @param	out_Filetypes	File types supported by the provided factory, concatenated into a string
	 * @param	out_Extensions	Extensions supported by the provided factory, concatenated into a string
	 * @param   SupportedExtensions			If not null only extension in the list can be added
	 */
	UNREALED_API void GenerateFactoryFileExtensions( const TArray<UFactory*>& InFactories
		, FString& out_Filetypes
		, FString& out_Extensions
		, TMultiMap<uint32, UFactory*>& out_FilterIndexToFactory);

	/**
	 * Generates a list of file types for a given class.
	 * @param   SupportedExtensions			If not null only extension in the list can be added
	 */
	UNREALED_API void AppendFactoryFileExtensions( UFactory* InFactory
		, FString& out_Filetypes
		, FString& out_Extensions);

	/**
	 * Populates two strings with all of the file types and extensions the format list provides.
	 *
	 * @param	InFormats		Array of supported file types. Each entry needs to be of the form "ext;Description" where ext is the file extension. 
	 * @param	out_FileTypes	File types supported by the provided array of formats, concatenated into a string
	 * @param	out_Extensions	Extensions supported by the provided array of formats, concatenated into a string
	 * @param   SupportedExtensions			If not null only extension in the list can be added
	 */
	UNREALED_API void AppendFormatsFileExtensions(const TArray<FString>& InFormats
		, FString& out_FileTypes
		, FString& out_Extensions);

	/**
	 * Populates two strings with all of the file types and extensions the format list provides.
	 *
	 * @param	InFormats		Array of supported file types. Each entry needs to be of the form "ext;Description" where ext is the file extension.
	 * @param	out_FileTypes	File types supported by the provided array of formats, concatenated into a string
	 * @param	out_Extensions	Extensions supported by the provided array of formats, concatenated into a string
	 * @param	out_FilterIndexToFactory	Add INDEX_NONE entry for all provided Formats
	 * @param   SupportedExtensions			If not null only extension in the list can be added
	 */
	UNREALED_API void AppendFormatsFileExtensions(const TArray<FString>& InFormats
		, FString& out_FileTypes
		, FString& out_Extensions
		, TMultiMap<uint32, UFactory*>& out_FilterIndexToFactory);

	/**
	 * Iterates over all classes and assembles a list of non-abstract UExport-derived type instances.
	 */
	UNREALED_API void AssembleListOfExporters(TArray<UExporter*>& OutExporters);

	/**
	 * Assembles a path from the outer chain of the specified object.
	 */
	UNREALED_API void GetDirectoryFromObjectPath(const UObject* Obj, FString& OutResult);

	/** Options for in use object tagging */
	enum EInUseSearchOption
	{
		SO_CurrentLevel, // Searches for in use objects refrenced by the current level
		SO_VisibleLevels, // Searches for in use objects referenced by visible levels
		SO_LoadedLevels // Searches for in use objects referenced by all loaded levels
	};

	enum class EInUseSearchFlags : uint32
	{
		None = 0,
		SkipCompilingAssets = 1, // Skip serialization of assets still being compiled, some data might be missing.
	};
	ENUM_CLASS_FLAGS(EInUseSearchFlags);

	/**
	 * Tags objects which are in use by levels specified by the search option
	 *
	 * @param SearchOption                  The search option for finding in use objects
	 * @param bShouldSkipCompilingAssets    Whether to avoid stalls on assets still being compiled.
	 */
	UNREALED_API void TagInUseObjects( EInUseSearchOption SearchOption, EInUseSearchFlags InUseSearchFlags = EInUseSearchFlags::None);

	/**
	 * Opens a property window for the selected objects
	 *
	 * @param SelectedObjects	The objects to view in the property window
	 * @return Pointer to the new properties window.
	 */
	UNREALED_API TSharedPtr<SWindow> OpenPropertiesForSelectedObjects( const TArray<UObject*>& SelectedObjects );

	/**
	 * Removes deleted objects from open property windows
	 *
	 * @param DeletedObjects	The objects to remove
	 */
	UNREALED_API void RemoveDeletedObjectsFromPropertyWindows( TArray<UObject*>& DeletedObjects );

	UE_DEPRECATED(5.1, "No longer used.")
	UNREALED_API bool IsAssetValidForPlacing(UWorld* InWorld, const FString& ObjectPath);

	/**
	 * Determines if the class is placeable in a world.
	 *
	 * @param InClass	Class to test.
	 *
	 * @return true if the class can be placed in the world.
	 */
	UNREALED_API bool IsClassValidForPlacing(const UClass* InClass);

	/**
	 * Determines if a given class is a redirector.
	 *
	 * @param Class	The class
	 *
	 * @return true if the class is a redirector, otherwise false.
	 */
	UNREALED_API bool IsClassRedirector( const UClass* Class );

	/**
	 * Determines if an array of objects are all of interchangeable types.
	 *
	 * @param InProposedObjects	The objects to check.
	 *
	 * @return true if all objects are interchangeable, otherwise false.
	 */
	UNREALED_API bool AreObjectsOfEquivalantType( const TArray<UObject*>& InProposedObjects );

	/**
	 * Determines if two classes are interchangeable.  This would tell you if you could substitute one object
	 * reference for another.  For example, Material and MaterialInstances are interchangeable.
	 *
	 * @param ClassA	The first class
	 * @param ClassB	The second class
	 *
	 * @return true if they are interchangeable otherwise false.
	 */
	UNREALED_API bool AreClassesInterchangeable( const UClass* ClassA, const UClass* ClassB );

	/**
	 * Find referencers of an object to be deleted.
	 *
	 * @param InObject                          The object to be deleted.
	 * @param bOutIsReferenced	                Set if the object is currently referenced
	 * @param bOutIsReferencedByUndo	        Set if the object is currently referenced by an undo transaction
	 * @param OutMemoryReferences               Optional pointer if specific information is required about referencers
	 * @param bInRequireReferencingProperties   Whether referencing properties information should be filled. (Opt-in only has it can degrade performance)
	 */
	UNREALED_API void GatherObjectReferencersForDeletion(UObject* InObject, bool& bOutIsReferenced, bool& bOutIsReferencedByUndo, FReferencerInformationList* OutMemoryReferences = nullptr, bool bInRequireReferencingProperties = false);

	/** 
	* Find any subobjects that might also need references replaced for a deep copy
	* @param InObjects					The objects that are going to have their references replaced
	* @param ObjectsToExclude			Any objects that should not have their references replaced
	* @param OutObjectsAndSubobjects	The complete list of objects and any subobjects that should have references replaced
	*/
	UNREALED_API void GatherSubObjectsForReferenceReplacement(TSet<UObject*>& InObjects, TSet<UObject*>& ObjectsToExclude, TSet<UObject*>& OutObjectsAndSubObjects);

	/**
	 * Given a SourceObject and ObjectToSearchFor will attempt to find the ObjectToSearchFor on the given object.
	 * If the ObjectToSearchFor is found this will report back properties needed to traverse to find this pointer.
	 * This is useful to find out where a reference is being pulled in from.
	 * 
	 * Uses the CVar ObjectTools.MaxTimesToCheckSameObject to configure how many times the same object should be checked and traversed. This is to help avoid circular dependencies.
	 * 
	 * @param SourceObject - which object we will scan the properties on to attempt to find the given ObjectToSearchFor
	 * @param ObjectToSearchFor - which object we will look for on each property of the SourceObject
	 * @param OutFoundPropertyChains - an array of strings for which properties to traverse to find the ObjectToSearchFor
	 * @return true when the ObjectToSearchFor is found, otherwise false.
	 */
	UNREALED_API bool GatherPropertyChainsToObject(const UObject* SourceObject, const UObject* ObjectToSearchFor, TArray<FString>& OutFoundPropertyChains);


	/**
	 * Batch version of UObject::GetArchetypeInstances.  Can be considerably faster in large worlds when querying multiple objects with the
	 * same UClass, as it only traverses the world once per UClass.  Null pointers are allowed for InObjects, returning an empty output.
	 *
	 * @param	InObjects		list of objects to query for archetype instances
	 * @param	OutInstances	per InObjects array element, receives the list of objects which have the given object in its archetype chain
	 */
	UNREALED_API void BatchGetArchetypeInstances(TArrayView<UObject*> InObjects, TArray<TArray<UObject*>>& OutInstances);


	/**
	 * Gets the user-facing name for the given UFunction.
	 * @param Function				the function to find the name for
	 * @param bAllowFriendlyNames	set to true if you will allow a FriendlyName (name with spaces which does not match the Function's Name)
	 * @return the name as localized text
	 */
	UNREALED_API FText GetUserFacingFunctionName(const UFunction* Function, bool bAllowFriendlyNames = false);


	/**
	 * Gets the non-specific tooltip for the given UFunction.
	 * @param Function the function to find the tooltip for
	 * @param bIncludeTarget if true, appends a "Target is [ClassName]" line identifying the function's owning class
	 * @return the tooltip as a string
	 */
	UNREALED_API FString GetDefaultTooltipForFunction(const UFunction* Function, bool bIncludeTarget = true);

	/**
	 * Gets the non-specific tooltip for the given UFunction parameter.
	 * @param Function the function to parse the param tooltip from
	 * @param ParamName the name of the parameter
	 * @return the tooltip as a string
	 */
	UNREALED_API FString GetDefaultTooltipForFunctionParam(const UFunction* Function, const FString& ParamName);

	/**
	 * Gets the non-specific tooltip for the given UFunction's return value.
	 * @param Function the function to parse the return tooltip from
	 * @return the tooltip as a string
	 */
	UNREALED_API FString GetDefaultTooltipForFunctionReturn(const UFunction* Function);
}


class FRHIGPUTextureReadback;

namespace ThumbnailTools
{
	/** Thumbnail texture flush mode */
	namespace EThumbnailTextureFlushMode
	{
		enum Type
		{
			/** Don't flush texture streaming at all */
			NeverFlush = 0,

			/** Aggressively stream resources before rendering thumbnail to avoid blurry textures */
			AlwaysFlush,
		};
	}

	/** Finds the file path/name of an existing package for the specified object full name, or false if the package is not valid.  */
	UNREALED_API bool QueryPackageFileNameForObject( const FString& InFullName, FString& OutPackageFileName );

	/**
	 * Renders a thumbnail for the specified object.
	 *
	 * @param	InObject				the rendered thumbnail will represent this object
	 * @param	InImageWidth			the maximum width of the thumbnail. It may be smaller when supplying a OutThumbnail.
	 * @param	InImageHeight			the maximum height of the thumbnail. It may be smaller when supplying a OutThumbnail.
	 * @param	InFlushMode				Flushes or not texture streaming before rendering
	 * @param	InRenderTargetResource	if non-NULL, the render target that will receive the thumbnail.
	 * @param	OutThumbnail			if non-NULL, the FObjectThumbnail that will receive the thumbnail data. This would also force a synchronization with the RenderThread
	 */
	UNREALED_API void RenderThumbnail( UObject* InObject, const uint32 InImageWidth, const uint32 InImageHeight, EThumbnailTextureFlushMode::Type InFlushMode,
									   FTextureRenderTargetResource* InRenderTargetResource = NULL, FObjectThumbnail* OutThumbnail = NULL );

	/**
	 * Async process to get a texture copied to an object thumbnail
	 * 1. Wait for the TextureReadback to be ready
	 * 2. Once TextureReadback is ready, enqueue a render command to copy its data to the object thumbnail
	 * 3. Once the render command completes, ObjectThumbnail can be read
	 */
	struct FAsyncObjectThumbnail
	{
		explicit FAsyncObjectThumbnail(int32 InWidth, int32 InHeight, TSharedRef<FRHIGPUTextureReadback> InTextureReadback);

		// Checks if the texture data is available to be readback
		bool IsTextureReadbackReady() const;

		// Enqueues a render command to copy the texture readback data to the ObjectThumbnail
		void EnqueueReadbackDataCopy();

		// ObjectThumbnail is available
		bool IsObjectThumbnailReady() const;

		/**
		 * Returns a copy of ObjectThumbnail and empties the Future
		 * IsObjectThumbnailReady must be true before calling this function
		 */
		FObjectThumbnail ConsumeObjectThumbnailUnchecked() { return ObjectThumbnailFuture->Consume(); }

	private:
		int32 ImageWidth, ImageHeight;

		TSharedRef<FRHIGPUTextureReadback> TextureReadback;
		TSharedPtr<TPromise<FObjectThumbnail>> ObjectThumbnailPromise;
		TOptional<TFuture<FObjectThumbnail>> ObjectThumbnailFuture;
	};

	/**
	 * Renders a thumbnail for the specified object.
	 * A FObjectThumbnail will be filled once the rendering is complete.
	 *
	 * @param	InObject					the rendered thumbnail will represent this object
	 * @param	InImageWidth				the maximum width of the thumbnail. It may be smaller when supplying a OutThumbnail.
	 * @param	InImageHeight				the maximum height of the thumbnail. It may be smaller when supplying a OutThumbnail.
	 * @param	InFlushMode					Flushes or not texture streaming before rendering
	 * @param	InRenderTargetResource		if non-NULL, the render target that will receive the thumbnail.
	 */
	UNREALED_API TOptional<FAsyncObjectThumbnail> RenderThumbnailAsync( UObject* InObject, const uint32 InImageWidth, const uint32 InImageHeight, EThumbnailTextureFlushMode::Type InFlushMode,
																			FTextureRenderTargetResource* InRenderTargetResource = nullptr );

	/** Generates a thumbnail for the specified object and caches it. Used to generate the thumbnail that gets saved in a package. */
	UNREALED_API FObjectThumbnail* GenerateThumbnailForObjectToSaveToDisk( UObject* InObject );

	/**
	 * Caches a thumbnail into a package's thumbnail map.
	 *
	 * @param	ObjectFullName	the full name for the object to associate with the thumbnail
	 * @param	Thumbnail		the thumbnail to cache; specify NULL to remove the current cached thumbnail
	 * @param	DestPackage		the package that will hold the cached thumbnail
	 *
	 * @return	pointer to the thumbnail data that was cached into the package
	 */
	UNREALED_API FObjectThumbnail* CacheThumbnail( const FString& ObjectFullName, FObjectThumbnail* Thumbnail, UPackage* DestPackage );

	/**
	 * Caches an empty thumbnail entry
	 *
	 * @param	ObjectFullName	the full name for the object to associate with the thumbnail
	 * @param	DestPackage		the package that will hold the cached thumbnail
	 */
	UNREALED_API void CacheEmptyThumbnail( const FString& ObjectFullName, UPackage* DestPackage );

	/** Searches for an object's thumbnail in memory and returns it if found */
	UNREALED_API const FObjectThumbnail* FindCachedThumbnail( const FString& InFullName );

	/** Returns the thumbnail for the specified object or NULL if one doesn't exist yet */
	UNREALED_API FObjectThumbnail* GetThumbnailForObject( UObject* InObject );

	/** Loads the thumbnail of an asset from the specified package file name (or from the external thumbnail cache file if it exists) */
	UNREALED_API bool LoadThumbnailFromPackage(const FAssetData& AssetData, FObjectThumbnail& OutThumbnail);

	/** Loads thumbnails from the specified package file name (or from the external thumbnail cache file if it exists) */
	UNREALED_API bool LoadThumbnailsFromPackage( const FString& InPackageFileName, const TSet<FName>& InObjectFullNames, FThumbnailMap& InOutThumbnails );

	/** Loads thumbnails from the specified package file name (without using external thumbnail cache or delegates, only attempts to read from InPackageFileNameToOpen provided) */
	UNREALED_API bool LoadThumbnailsFromPackageDirectly( const FString& InPackageFileName, const TSet<FName>& InObjectFullNames, FThumbnailMap& InOutThumbnails, const FString& InPackageFileNameToOpen);

	/** Loads thumbnails from a package unless they're already cached in that package's thumbnail map */
	UNREALED_API bool ConditionallyLoadThumbnailsFromPackage( const FString& InPackageFileName, const TSet< FName >& InObjectFullNames, FThumbnailMap& InOutThumbnails );

	/** Loads thumbnails for the specified objects (or copies them from a cache, if they're already loaded.) */
	UNREALED_API bool ConditionallyLoadThumbnailsForObjects( const TArray< FName >& InObjectFullNames, FThumbnailMap& InOutThumbnails );

	/** Standard thumbnail height setting used by generation */
	inline const int32 DefaultThumbnailSize=256;

	/** Returns true if the given asset has a custom thumbnail cached or on the disk. */
	UNREALED_API bool AssetHasCustomThumbnail(const FString& InAssetDataFullName);
	UNREALED_API bool AssetHasCustomThumbnail(const FString& InAssetDataFullName, FObjectThumbnail& OutThumbnail);
	/** Returns true if the given asset has a custom thumbnail cached or on the disk and if the thumbnail was captured from a viewport. */
	UNREALED_API bool AssetHasCustomCreatedThumbnail(const FString& InAssetDataFullName);

	/** Add delegate to handle loading of thumbnail from package. */
	UNREALED_API FDelegateHandle AddLoadThumbnailsFromPackageDelegate(FLoadThumbnailsFromPackage InDelegate);

	/** Remove delegate to handle loading of thumbnail from package. */
	UNREALED_API void RemoveLoadThumbnailsFromPackageDelegate(const FDelegateHandle InHandle);

	/** Loads thumbnails from the specified package file name */
	UNREALED_API bool LoadThumbnailsFromPackageUsingExternalFolders(const FString& InPackageFileName, const TSet<FName>& InObjectFullNames, FThumbnailMap& InOutThumbnails, TConstArrayView<FString> ExternalFolders);
}
