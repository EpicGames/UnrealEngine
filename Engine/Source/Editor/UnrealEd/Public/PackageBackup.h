// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PackageBackup.h: Utility class for backing up a package.
=============================================================================*/

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "UObject/Package.h"

/**
 * Class that houses various static utility functions to allow for packages
 * to be backed-up to a special auto save/backup directory. Backups are traditionally
 * made before a save occurs to a package, to help guard against corruption/data loss.
 */
class UE_DEPRECATED(5.1, "Backups are no longer used in Unreal Package Saves") FAutoPackageBackup
{
public:
	
	/**
	 * Create a backup of the specified package. A backup is only created if the specified
	 * package meets specific criteria, as outlined in the comments for ShouldBackupPackage().
	 *
	 * @param	InPackage	Package which should be backed up
	 *
	 * @see		FAutoPackageBackup::ShouldBackupPackage()
	 *
	 * @return	true if the package was successfully backed up; false if it was not
	 */
	static bool BackupPackage( const UPackage& InPackage );

	/**
	 * Create a backup of the specified packages. A backup is only created if a specified
	 * package meets specific criteria, as outlined in the comments for ShouldBackupPackage().
	 *
	 * @param	InPackage	Package which should be backed up
	 *
	 * @see		FAutoPackageBackup::ShouldBackupPackage()
	 *
	 * @return	true if all provided packages were successfully backed up; false if one or more
	 *			were not
	 */
	static bool BackupPackages( const TArray<UPackage*>& InPackages );

private:
	/**
	 * Helper function designed to determine if the provided package should be backed up or not.
	 * The function checks for many conditions, such as if the package is too large to backup,
	 * if the package has a particular attribute that should prevent it from being backed up (such
	 * as being marked for PIE-use), if cooking is in progress, etc.
	 *
	 * @param	InPackage		Package which should be checked to see if its valid for backing-up
	 * @param	OutFileName		File name of the package on disk if the function determines the package
	 *							already existed
	 *
	 * @return	true if the package is valid for backing-up; false otherwise
	 */
	static bool ShouldBackupPackage( const UPackage& InPackage, FString& OutFilename );

	/**
	 * Helper function that returns whether the user has package backups enabled or not. The value
	 * is determined by a configuration INI setting.
	 *
	 * @return	true if package backups are enabled; false otherwise
	 */
	static bool IsPackageBackupEnabled();

	/**
	 * Helper function that returns the maximum amount of space the user has designated to allow for
	 * package backups. This value is determined by a configuration INI setting.
	 *
	 * @return	The maximum amount of space allowed, in bytes
	 */
	static int32 GetMaxAllowedBackupSpace();

	/**
	 * Helper function that returns the time in between backups of a package before another backup of
	 * the same package should be considered valid. This value is determined by a configuration INI setting,
	 * and prevents a package from being backed-up over and over again in a small time frame.
	 *
	 * @return	The interval to wait before allowing another backup of the same package, in seconds
	 */
	static int32 GetBackupInterval();

	/**
	 * Helper function that returns the directory to store package backups in.
	 *
	 * @return	String containing the directory to store package backups in.
	 */
	static FString GetBackupDirectory();

	/**
	 * Deletes old backed-up package files until the provided amount of space (in bytes)
	 * is available to use in the backup directory. Fails if the provided amount of space
	 * is more than the amount of space the user has allowed for backups or if enough space
	 * could not be made.
	 *
	 * @param	InBackupFiles		File info of the files in the backup directory
	 * @param	InSpaceUsed			The amount of space, in bytes, the files in the provided array take up
	 * @param	InSpaceRequired		The amount of space, in bytes, to assure is available
	 *								for the purposes of package backups
	 *
	 * @return	true if the space was successfully provided, false if not or if the requested space was
	 *			greater than the max allowed space by the user
	 */
	static bool PerformBackupSpaceMaintenance( TArray<struct FBackupFileInfo>& InBackupFiles, int64 InSpaceUsed, int64 InSpaceRequired );

	// Copy constructor, Destructor, and Self-assignment operator all left private and un-implemented in
	// order to prevent the class from being instantiated
	FAutoPackageBackup( const FAutoPackageBackup& );
	~FAutoPackageBackup();
	FAutoPackageBackup& operator=( const FAutoPackageBackup& );
};
