// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * File to hold common package helper functions.
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "Misc/PackageName.h"
#include "Commandlets/Commandlet.h"
#include "Misc/FeedbackContext.h"
#include "FileHelpers.h"
#include "CollectionManagerTypes.h"

class Error;
class ICollectionContainer;

UNREALED_API DECLARE_LOG_CATEGORY_EXTERN(LogPackageHelperFunctions, Log, All);

/**
 * Flags which modify the way that NormalizePackageNames works.
 */
enum EPackageNormalizationFlags
{
	/** reset the linker for any packages currently in memory that are part of the output list */
	NORMALIZE_ResetExistingLoaders		= 0x01,
	/** do not include map packages in the result array; only relevant if the input array is empty */
	NORMALIZE_ExcludeMapPackages		= 0x02,
	/** do not include content packages in the result array; only relevant if the input array is empty */
	NORMALIZE_ExcludeContentPackages	= 0x04,
	/** do not include packages inside developer folders in the result array; only relevant if the input array is empty */
	NORMALIZE_ExcludeDeveloperPackages  = 0x08,
	/** do not include packages outside developer folders in the result array; only relevant if the input array is empty */
	NORMALIZE_ExcludeNonDeveloperPackages  = 0x10,
	/** do not include packages inside the Engine/Content folders in the result array; only relevant if the input array is empty */
	NORMALIZE_ExcludeEnginePackages		= 0x20,
	/** do not include packages inside NoRedist, NotForLicensees, or LimitedAccess folders */
	NORMALIZE_ExcludeNoRedistPackages	= 0x40,
	/** do not include localized packages */
	NORMALIZE_ExcludeLocalizedPackages	= 0x80,
	/** Combo flags */
	NORMALIZE_DefaultFlags				= NORMALIZE_ResetExistingLoaders,
};

void SearchDirectoryRecursive( const FString& SearchPathMask, TArray<FString>& out_PackageNames, TArray<FString>& out_PackageFilenames );

/**
 * Takes an array of package names (in any format) and converts them into relative pathnames for each package.
 *
 * @param	PackageNames		the array of package names to normalize.  If this array is empty, the complete package list will be used.
 * @param	PackagePathNames	will be filled with the complete relative path name for each package name in the input array
 * @param	PackageWildcard		if specified, allows the caller to specify a wildcard to use for finding package files
 * @param	PackageFilter		allows the caller to limit the types of packages returned.
 *
 * @return	true if packages were found successfully, false otherwise.
 */
bool UNREALED_API NormalizePackageNames( TArray<FString> PackageNames, TArray<FString>& PackagePathNames, const FString& PackageWildcard=FString(TEXT("*.*")), uint8 PackageFilter=NORMALIZE_DefaultFlags );


/** 
 * Helper function to save a package that may or may not be a map package
 *
 * @param	Package		The package to save
 * @param	Filename	The location to save the package to
 * @param	KeepObjectFlags	Objects with any these flags will be kept when saving even if unreferenced.
 * @param	ErrorDevice	the output device to use for warning and error messages
 *
 * @return true if successful
 */
bool UNREALED_API SavePackageHelper(UPackage* Package, FString Filename,  EObjectFlags KeepObjectFlags = RF_Standalone,
	FOutputDevice* ErrorDevice=GWarn, ESaveFlags SaveFlags = SAVE_None);
UE_DEPRECATED(5.0, "LinkerToConformAgainst is no longer implemented; call function overload that does not take LinkerToConformAgainst.")
bool UNREALED_API SavePackageHelper(UPackage* Package, FString Filename, EObjectFlags KeepObjectFlags,
	FOutputDevice* ErrorDevice, FLinkerNull* LinkerToConformAgainst, ESaveFlags SaveFlags = SAVE_None);


/**
 *	Collection helper
 *	Used to create and update ContentBrowser collections
 *
 */
class FContentHelper
{
public:
	UE_DEPRECATED(5.6, "Use the ICollectionContainer overload instead.")
	FContentHelper();

	explicit FContentHelper(const TSharedRef<ICollectionContainer>& InCollectionContainer);

	~FContentHelper()
	{
		if (bInitialized == true)
		{
			Shutdown();
		}
	}

	/**
	 *	Initialize the Collection helper
	 *	
	 *	@return	bool					true if successful, false if failed
	 */
	UNREALED_API bool Initialize();

	/**
	 *	Shutdown the collection helper
	 */
	UNREALED_API void Shutdown();
	
	/**
	 *	Create a new tag
	 *	
	 *	@param	CollectionName		The name of the tag to create
	 *
	 *	@return	bool			true if successful, false if failed
	 */
	bool CreateCollection( FName CollectionName, ECollectionShareType::Type InType );

	/**
	 *	Clear the given collection
	 *	
	 *	@param	InCollectionName		The name of the collection to clear
	 *	@param	InType					Type of collection
	 *
	 *	@return	bool					true if successful, false if failed
	 */
	bool ClearCollection(FName InCollectionName, ECollectionShareType::Type InType);

	/**
	 *	Fill the given collection with the given list of assets
	 *
	 *	@param	InCollectionName	The name of the collection to fill
	 *	@param	InType				Type of collection
	 *	@param	InAssetList			The list of items to fill the collection with (can be empty)
	 *
	 *	@return	bool				true if successful, false if not.
	 */
	bool SetCollection(FName InCollectionName, ECollectionShareType::Type InType, const TArray<FSoftObjectPath>& InAssetList);

	/**
	 *	Update the given collection with the lists of adds/removes
	 *
	 *	@param	InCollectionName	The name of the collection to update
	 *	@param	InType				Type of collection
	 *	@param	InAddList			The list of items to ADD to the collection (can be empty)
	 *	@param	InRemoveList		The list of items to REMOVE from the collection (can be empty)
	 *
	 *	@return	bool				true if successful, false if not.
	 */
	bool UpdateCollection(FName InCollectionName, ECollectionShareType::Type InType, const TArray<FSoftObjectPath>& InAddList, const TArray<FSoftObjectPath>& InRemoveList);

	/**
	 *	Retrieve the assets contained in the given collection.
	 *
	 *	@param	InCollectionName	Name of collection to query
	 *	@param	InType				Type of collection
	 *	@param	OutAssetPaths		The assets contained in the collection
	 * 
	 *	@return True if collection was created successfully
	 */
	UNREALED_API bool QueryAssetsInCollection(FName InCollectionName, ECollectionShareType::Type InType, TArray<FSoftObjectPath>& OutAssetPaths);

protected:
	/** Creates a new collection */
	template <class AssetSetPolicy>	bool CreateAssetSet ( FName InSetName, ECollectionShareType::Type InSetType );

	/** Clears the content of a Tag or Collection */
	template <class AssetSetPolicy>	bool ClearAssetSet ( FName InSetName, ECollectionShareType::Type InSetType );
	
	/** Sets the contents of a Tag or Collection to be the InAssetList. Assets not mentioned in the list will be untagged. */
	template <class AssetSetPolicy>	bool AssignSetContent( FName InSetName, ECollectionShareType::Type InType, const TArray<FSoftObjectPath>& InAssetList );
	
	/** Add and remove assets for the specified Tag or Connection. Assets from InAddList are added; assets from InRemoveList are removed. */
	template <class AssetSetPolicy>	bool UpdateSetContent( FName InSetName, ECollectionShareType::Type InType, const TArray<FSoftObjectPath>& InAddList, const TArray<FSoftObjectPath>& InRemoveList );
	
	/** Get the list of all assets in the specified Collection or Tag */
	template <class AssetSetPolicy>	bool QuerySetContent( FName InCollectionName, ECollectionShareType::Type InType, TArray<FSoftObjectPath>& OutAssetPaths );

	bool bInitialized;
	TSharedPtr<ICollectionContainer> CollectionContainer;
};

/**
 * This is our Functional "Do an Action to all Packages" Template.  Basically it handles all
 * of the boilerplate code which normally gets copy pasted around.  So now we just pass in
 * the OBJECTYPE  (e.g. Texture2D ) and then the Functor which will do the actual work.
 *
 * @see UFindMissingPhysicalMaterialsCommandlet
 * @see UFindTexturesWhichLackLODBiasOfTwoCommandlet
 **/
template< typename OBJECTYPE, typename FUNCTOR >
void DoActionToAllPackages( UCommandlet* Commandlet, const FString& Params )
{
	// Parse command line.
	TArray<FString> Tokens;
	TArray<FString> Switches;

	UE_LOGF(LogPackageHelperFunctions, Warning, "%ls", *Params);

	const TCHAR* Parms = *Params;
	UCommandlet::ParseCommandLine(Parms, Tokens, Switches);

	const bool bVerbose = Switches.Contains(TEXT("VERBOSE"));
	const bool bLoadMaps = Switches.Contains(TEXT("LOADMAPS"));
	const bool bOverrideLoadMaps = Switches.Contains(TEXT("OVERRIDELOADMAPS"));
	const bool bOnlyLoadMaps = Switches.Contains(TEXT("ONLYLOADMAPS"));
	const bool bSkipReadOnly = Switches.Contains(TEXT("SKIPREADONLY"));
	const bool bOverrideSkipOnly = Switches.Contains(TEXT("OVERRIDEREADONLY"));
	const bool bGCEveryPackage = Switches.Contains(TEXT("GCEVERYPACKAGE"));

	TArray<FString> FilesInPath;
	FEditorFileUtils::FindAllPackageFiles(FilesInPath);

	int32 GCIndex = 0;
	for( int32 FileIndex = 0; FileIndex < FilesInPath.Num(); FileIndex++ )
	{
		const FString& Filename = FilesInPath[FileIndex];

		const bool bIsAutoSave = Filename.Contains( TEXT("AUTOSAVES") );
		
		// See if we should skip read only packages
		if( bSkipReadOnly && !bOverrideSkipOnly )
		{
			const bool bIsReadOnly = IFileManager::Get().IsReadOnly( *Filename );
			if( bIsReadOnly )
			{
				UE_LOGF(LogPackageHelperFunctions, Warning, "Skipping %ls (read-only)", *Filename);			
				continue;
			}
		}

		// if we don't want to load maps for this
		if( ((!bLoadMaps && !bOnlyLoadMaps) || bOverrideLoadMaps) && ( FPaths::GetExtension(Filename, true) == FPackageName::GetMapPackageExtension() ) )
		{
			continue;
		}

		// if we only want to load maps for this
		if( ( bOnlyLoadMaps == true ) && ( FPaths::GetExtension(Filename, true) != FPackageName::GetMapPackageExtension() ) )
		{
			continue;
		}

		if( bVerbose == true )
		{
			UE_LOGF(LogPackageHelperFunctions, Warning, "Loading %ls", *Filename );
		}

		UPackage* Package = LoadPackage( NULL, *Filename, LOAD_None );
		if( Package != NULL )
		{
			FUNCTOR TheFunctor;
			TheFunctor.template DoIt<OBJECTYPE>( Commandlet, Package, Tokens, Switches );
		}
		else
		{
			UE_LOGF(LogPackageHelperFunctions, Error, "Error loading %ls!", *Filename );
		}

		if( ( (++GCIndex % 10) == 0 ) || ( bGCEveryPackage == true ) )
		{
			CollectGarbage(RF_NoFlags);
		}
	}
}
