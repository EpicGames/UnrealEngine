// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "SourceControlHelpers.h"

//This class should not be used as a base class because it can force a connection to the SCM through FScopedSourceControl. Connection errors can result in failing automation processes that are unrelated. 
//The same can happen if the class is used as a member of another class. It should only be used as a local variable in a method so the connection is only attempted if required. 
class FPackageSourceControlHelper final
{
public:
	UNREALED_API bool UseSourceControl() const;
	UNREALED_API bool Delete(const FString& PackageName) const;
	UNREALED_API bool Delete(const TArray<FString>& PackageNames, bool bErrorsAsWarnings = false) const;
	UNREALED_API bool Delete(UPackage* Package) const;
	UNREALED_API bool Delete(const TArray<UPackage*>& Packages) const;
	UNREALED_API bool AddToSourceControl(UPackage* Package) const;
	UNREALED_API bool AddToSourceControl(const TArray<UPackage*>& Packages, bool bErrorsAsWarnings = false) const;
	UNREALED_API bool AddToSourceControl(const TArray<FString>& PackageNames, bool bErrorsAsWarnings = false) const;
	UNREALED_API bool Checkout(UPackage* Package) const;
	UNREALED_API bool Checkout(const TArray<FString>& PackageNames, bool bErrorsAsWarnings = false) const;
	UNREALED_API bool GetDesiredStatesForModification(const TArray<FString>& PackageNames, TArray<FString>& OutPackagesToCheckout, TArray<FString>& OutPackagesToAdd, bool bErrorsAsWarnings = false) const;
	UNREALED_API bool GetMarkedForDeleteFiles(const TArray<FString>& PackageFilenames, TArray<FString>& OutPackageFilenames, bool bErrorsAsWarnings = false) const;
private:
	ISourceControlProvider& GetSourceControlProvider() const;
	FScopedSourceControl SourceControl;	
};
