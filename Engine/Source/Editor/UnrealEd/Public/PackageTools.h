// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PackageTools.h: Object-related utilities

=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "PackageTools.generated.h"

class ULevel;

class FPackageReloadedEvent;
enum class EPackageReloadPhase : uint8;

UENUM()
enum class EReloadPackagesInteractionMode : uint8
{
	/** Interactive, ask the user what to do */
	Interactive,

	/** Non-interactive, assume a positive response */
	AssumePositive,

	/** Non-interactive, assume a negative response */
	AssumeNegative,
};

UCLASS(Abstract, MinimalAPI)
class UPackageTools : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	using EReloadPackagesInteractionMode = ::EReloadPackagesInteractionMode;

	/**
	 * Filters the global set of packages.
	 *
	 * @param	OutGroupPackages			The map that receives the filtered list of group packages.
	 * @param	OutPackageList				The array that will contain the list of filtered packages.
	 */
	static UNREALED_API void GetFilteredPackageList(TSet<UPackage*>& OutFilteredPackageMap);
	
	/**
	 * Fills the OutObjects list with all valid objects that are supported by the current
	 * browser settings and that reside withing the set of specified packages.
	 *
	 * @param	InPackages			Filters objects based on package.
	 * @param	OutObjects			[out] Receives the list of objects
	 */
	static UNREALED_API void GetObjectsInPackages( const TArray<UPackage*>* InPackages, TArray<UObject*>& OutObjects );

	/**
	 * Handles fully loading passed in packages.
	 *
	 * @param	TopLevelPackages	Packages to be fully loaded.
	 * @param	OperationText		Text describing the operation; appears in the warning string presented to the user.
	 * 
	 * @return true if all packages where fully loaded, false otherwise
	 */
	static UNREALED_API bool HandleFullyLoadingPackages( const TArray<UPackage*>& TopLevelPackages, const FText& OperationText );


	/**
	 * Loads the specified package file (or returns an existing package if it's already loaded.)
	 *
	 * @param	InFilename	File name of package to load
	 * @param	InLoadFlags	Flags controlling loading behavior, from the ELoadFlags enum

	 *
	 * @return	The loaded package (or NULL if something went wrong.)
	 */
	static UNREALED_API UPackage* LoadPackage( FString InFilename, uint32 InLoadFlags = 0);

	/**
	 * Helper function that attempts to unload the specified top-level packages.
	 *
	 * @param	PackagesToUnload	the list of packages that should be unloaded
	 *
	 * @return	true if the set of loaded packages was changed
	 */
	static UNREALED_API bool UnloadPackages( const TArray<UPackage*>& PackagesToUnload );

	/**
	 * Helper function that attempts to unload the specified top-level packages.
	 *
	 * @param	PackagesToUnload		the list of packages that should be unloaded
	 * @param	OutErrorMessage			An error message specifying any problems with unloading packages
	 * @param	bUnloadDirtyPackages	Whether to unload packages that are dirty (that need to be saved)
	 *
	 * @return	true if the set of loaded packages was changed
	 */
	static UNREALED_API bool UnloadPackages( const TArray<UPackage*>& PackagesToUnload, FText& OutErrorMessage, bool bUnloadDirtyPackages = false);

	/**
	 * struct containing list of packages to unload and other params.
	 */
	struct FUnloadPackageParams
	{
		FUnloadPackageParams(const TArray<UPackage*>& PackagesToUnload) : Packages(PackagesToUnload) {}

		/** Packages to unload */
		const TArray<UPackage*> Packages;
		/** Whether to unload packages that are dirty (that need to be saved) */
		bool bUnloadDirtyPackages = false;
		/** Whether to reset the transaction buffer to allow unloading packages that might be referenced by it */
		bool bResetTransBuffer = true;
		
		/** Result: An error message specifying any problems with unloading packages */
		FText OutErrorMessage;
	};

	/**
	 * Helper function that attemps to unload the specified top-level packages.
	 * 
	 * @param struct containg list of packages that should be unloaded and other behavior params.
	 * 
	 * @return true if the set of loaded packages was changed
	 */
	static UNREALED_API bool UnloadPackages(FUnloadPackageParams& Params);

	/**
	 * Helper function that attempts to reload the specified top-level packages.
	 *
	 * @param	PackagesToReload	The list of packages that should be reloaded
	 *
	 * @return	true if the set of loaded packages was changed
	 */
	static UNREALED_API bool ReloadPackages( const TArray<UPackage*>& PackagesToReload );

	/**
	 * Helper function that attempts to reload the specified top-level packages.
	 *
	 * @param	PackagesToReload	The list of packages that should be reloaded
	 * @param	OutErrorMessage		An error message specifying any problems with reloading packages
	 * @param	bInteractive		Whether the function is allowed to ask the user questions (such as whether to reload dirty packages)
	 *
	 * @return	true if the set of loaded packages was changed
	 */
	UE_DEPRECATED(4.21, "ReloadPackages taking bInteractive is deprecated. Use the version taking EReloadPackagesInteractionMode instead.")
	static UNREALED_API bool ReloadPackages( const TArray<UPackage*>& PackagesToReload, FText& OutErrorMessage, const bool bInteractive = true );

	/**
	 * Helper function that attempts to reload the specified top-level packages.
	 *
	 * @param	PackagesToReload	The list of packages that should be reloaded
	 * @param	OutErrorMessage		An error message specifying any problems with reloading packages
	 * @param	InteractionMode		Whether the function is allowed to ask the user questions (such as whether to reload dirty packages)
	 *
	 * @return	true if the set of loaded packages was changed
	 */
	static UNREALED_API bool ReloadPackages( const TArray<UPackage*>& PackagesToReload, FText& OutErrorMessage, const EReloadPackagesInteractionMode InteractionMode = EReloadPackagesInteractionMode::Interactive );

	/**
	 *	Exports the given packages to files.
	 *
	 * @param	PackagesToExport		The set of packages to export.
	 * @param	ExportPath				receives the value of the path the user chose for exporting.
	 * @param	bUseProvidedExportPath	If true and ExportPath is specified, use ExportPath as the user's export path w/o prompting for a directory, where applicable
	 */
	static UNREALED_API void ExportPackages( const TArray<UPackage*>& PackagesToExport, FString* ExportPath=NULL, bool bUseProvidedExportPath = false );

	/**
	 * Wrapper method for multiple objects at once.
	 *
	 * @param	TopLevelPackages		the packages to be export
	 * @param	LastExportPath			the path that the user last exported assets to
	 * @param	FilteredClasses			if specified, set of classes that should be the only types exported if not exporting to single file
	 * @param	bUseProvidedExportPath	If true, use LastExportPath as the user's export path w/o prompting for a directory, where applicable
	 *
	 * @return	the path that the user chose for the export.
	 */
	static UNREALED_API FString DoBulkExport(const TArray<UPackage*>& TopLevelPackages, FString LastExportPath, const TSet<UClass*>* FilteredClasses = NULL, bool bUseProvidedExportPath = false );

	/** Helper function that attempts to check out the specified top-level packages. */
	static UNREALED_API void CheckOutRootPackages( const TArray<UPackage*>& Packages );


	/**
	 * Checks if the passed in path is in an external directory. I.E Ones not found automatically in the content directory
	 *
	 * @param	PackagePath	Path of the package to check, relative or absolute
	 * @return	true if PackagePath points to an external location
	 */
	static UNREALED_API bool IsPackagePathExternal(const FString& PackagePath);

	/**
	 * Checks if the passed in package's filename is in an external directory. I.E Ones not found automatically in the content directory
	 *
	 * @param	Package	The package to check
	 * @return	true if the package points to an external filename
	 */
	static UNREALED_API bool IsPackageExternal(const UPackage& Package);

	/** Saves all the dirty packages for the specified objects*/
	static UNREALED_API bool SavePackagesForObjects(const TArray<UObject*>& ObjectsToSave);

	/**
	 * Checks if the package has only one asset which shares its name
	 *
	 * @param Package The package to check
	 * @return true if the package has only one asset which shares the name of the package
	 */
	static UNREALED_API bool IsSingleAssetPackage (const FString& Package);

	/** Replaces all invalid package name characters with _ */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Package Tools")
	static UNREALED_API FString SanitizePackageName(const FString& InPackageName);

	/** 
	 * Converts a long package name to a file name.
	 * This can be called on package paths as well, provide no extension in that case.
	 * Will return an empty string if it fails.
	 * @param PackageName Long Package Name
	 * @param Extension Package extension.
	 * @return Package filename, or empty if it failed.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Package Tools", meta=(AdvancedDisplay=1))
	static UNREALED_API FString PackageNameToFilename(const FString& PackageName, const FString& Extension = TEXT(""));
	
	/** 
	 * Tries to convert a given relative or absolute filename to a long package name or path starting with a root like /Game
	 * This works on both package names and directories, and it does not validate that it actually exists on disk.
	 * @param Filename Filename to convert.
	 * @return Resulting long package name if the supplied filename properly maps to a long package root, empty string otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Package Tools")
	static UNREALED_API FString FilenameToPackageName(const FString& Filename);

	/**
	 * Find or create a package at the desired path
	 * If a package already exist but was made for another type of asset,
	 * this function will create another one with a modified unique name
	 * @param LongPackageName The package path in unreal asset path
	 * @param AssetClass The class of asset the package should be for.
	 */
	static UNREALED_API UPackage* FindOrCreatePackageForAssetType(const FName LongPackageName, UClass* AssetClass);

	/**
	 * Utility function that gathers all async compilable objects from given packages
	 * and flush them to make sure there is no remaining async work trying to load data
	 * from said packages.
	 * @param Packages List of packages from which to gather compilable assets to flush.
	 */
	static UNREALED_API void FlushAsyncCompilation(TArrayView<UPackage* const> Packages);

private:
	static void RestoreStandaloneOnReachableObjects();

	static void HandlePackageReloaded(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent);

	static TSet<UPackage*>* PackagesBeingUnloaded;
	static TSet<UObject*> ObjectsThatHadFlagsCleared;
	static FDelegateHandle ReachabilityCallbackHandle;
};

UE_DEPRECATED(4.21, "PackageTools namespace has been deprecated. Please use UPackageTools instead.") 
typedef UPackageTools PackageTools;
