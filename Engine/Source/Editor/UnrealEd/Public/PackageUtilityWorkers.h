// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PackageUtilityWorkers.cpp: Declarations for structs and classes used by package commandlets.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Misc/FeedbackContext.h"

/**
 * These bit flag values represent the different types of information that can be reported about a package
 */
enum EPackageInfoFlags
{
	PKGINFO_None			=0x00,
	PKGINFO_Names			=0x01,
	PKGINFO_Imports			=0x02,
	PKGINFO_Exports			=0x04,
	PKGINFO_Compact			=0x08,
	PKGINFO_Depends			=0x20,
	PKGINFO_Paths			=0x40,
	PKGINFO_Thumbs			=0x80,
	PKGINFO_Lazy			=0x100,
	PKGINFO_AssetRegistry	=0x200,
	PKGINFO_Text			=0x400,

	PKGINFO_All			= PKGINFO_Names|PKGINFO_Imports|PKGINFO_Exports|PKGINFO_Depends|PKGINFO_Paths|PKGINFO_Thumbs|PKGINFO_Lazy|PKGINFO_AssetRegistry|PKGINFO_Text,
};

enum EPackageInfoDisplayFlags
{
	PKGINFODISPLAY_None					= 0x00,
	PKGINFODISPLAY_HideOffsets			= 0x01, // If present, FObjectExport::SerialOffset will not be included in the output; useful when generating a report for comparison against another version of the same package.
	PKGINFODISPLAY_HideProcessUnstable	= 0x02, // If present, values that are unstable across different process invocations will be not included, such as FName.GetComparisonIndex().ToUnstableInt()
	PKGINFODISPLAY_HideSaveUnstable		= 0x04, // If present, values that are recalculated during save such as the Package guid will not be included

	PKGINFODISPLAY_HideAllUnstable		= PKGINFODISPLAY_HideOffsets | PKGINFODISPLAY_HideProcessUnstable | PKGINFODISPLAY_HideSaveUnstable,
	PKGINFODISPLAY_All					= PKGINFODISPLAY_HideAllUnstable
};
/**
 * Base for classes which generate output for the PkgInfo commandlet
 */
struct FPkgInfoReporter
{
	/** Constructors */
	FPkgInfoReporter() 
		: FPkgInfoReporter(PKGINFO_None, (EPackageInfoDisplayFlags)PKGINFODISPLAY_None, nullptr)
	{}
	UE_DEPRECATED(4.25, "Use The EPackageInfoDisplayFlags constructor instead") FPkgInfoReporter(uint32 InInfoFlags, bool bInHideOffsets, FLinkerLoad* InLinker = NULL)
		: FPkgInfoReporter(InInfoFlags, bInHideOffsets ? PKGINFODISPLAY_HideOffsets : PKGINFODISPLAY_None, InLinker)
	{}
	FPkgInfoReporter(uint32 InInfoFlags, EPackageInfoDisplayFlags InDisplayFlags, FLinkerLoad* InLinker = NULL)
		: InfoFlags(InInfoFlags), DisplayFlags(InDisplayFlags), Linker(InLinker), PackageCount(0)
	{}
	FPkgInfoReporter( const FPkgInfoReporter& Other )
	: InfoFlags(Other.InfoFlags), DisplayFlags(Other.DisplayFlags), Linker(Other.Linker), PackageCount(Other.PackageCount)
	{}

	/** Destructor */
	virtual ~FPkgInfoReporter() = default;

	/**
	 * Performs the actual work - generates a report containing information about the linker.
	 *
	 * @param	InLinker	if specified, changes this reporter's Linker before generating the report.
	 */
	virtual void GeneratePackageReport( class FLinkerLoad* InLinker = nullptr, FOutputDevice& Out = *GWarn)=0;

	/**
	 * Changes the target linker for this reporter.  Useful when generating reports for multiple packages.
	 */
	void SetLinker( class FLinkerLoad* NewLinker )
	{
		Linker = NewLinker;
	}

protected:
	/**
	 * A bitmask of PKGINFO_ flags that determine the categories of information included in the report.
	 */
	uint32 InfoFlags;

	/*
	 * A bitmask of EPackageInfoDisplayFlags that determine the display of information in the report.
	 */
	uint32 DisplayFlags;

	/**
	 * The linker of the package to generate the report for
	 */
	class FLinkerLoad* Linker;

	/**
	 * The number of packages evaluated by this reporter so far.  Must be incremented by child classes.
	 */
	int32 PackageCount;

	bool IsHideOffsets() const { return (DisplayFlags & PKGINFODISPLAY_HideOffsets) != 0; }
	bool IsHideProcessUnstable() const { return (DisplayFlags & PKGINFODISPLAY_HideProcessUnstable) != 0; }
	bool IsHideSaveUnstable() const { return (DisplayFlags & PKGINFODISPLAY_HideSaveUnstable) != 0; }
};

struct FPkgInfoReporter_Log : public FPkgInfoReporter
{
	using FPkgInfoReporter::FPkgInfoReporter;

	/**
	 * Writes information about the linker to the log.
	 *
	 * @param	InLinker	if specified, changes this reporter's Linker before generating the report.
	 */
	virtual void GeneratePackageReport( class FLinkerLoad* InLinker = nullptr, FOutputDevice& Out = *GWarn);
};
