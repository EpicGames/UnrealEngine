// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// PhysicalMaterialMaskImport
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "PhysicalMaterials/PhysicalMaterialMask.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Factories/Factory.h"
#include "EditorReimportHandler.h"

class FPhysicalMaterialMaskImport 
{
public:
	// Create physical material mask texture
	static UNREALED_API void ImportMaskTexture(UPhysicalMaterialMask* PhysMatMask);

	// Reimport physical material mask texture
	static UNREALED_API EReimportResult::Type ReimportMaskTexture(UPhysicalMaterialMask* PhysMatMask);

	// Reimport physical material mask texture with new file
	static UNREALED_API EReimportResult::Type ReimportMaskTextureWithNewFile(UPhysicalMaterialMask* PhysMatMask);

private:
	// Opens file dialog for texture selection
	static FString OpenMaskFileDialog();

	// Imports physical material mask texture
	static UTexture* ImportMaskTextureFile(UPhysicalMaterialMask* PhysMatMask, const FString& TextureFilename);

	// Returns file formats supported for mask texture. 
	static void GetSupportedTextureFileTypes(TArray<FString>& OutFileFormats);

	// Returns texture source formats supported for mask texture. 
	static void GetSupportedTextureSourceFormats(TArray<ETextureSourceFormat>& OutSourceFormats);

	// Helper to get dialog file type filter string 
	static void GetFileTypeFilterString(FString& OutFileTypeFilter);
};
