// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PhysicsAssetUtils.h"
#include "PhysicsAssetGenerationSettings.generated.h"

UCLASS(MinimalAPI, config=EditorPerProjectUserSettings)
class UPhysicsAssetGenerationSettings : public UObject
{
	GENERATED_BODY()

public:
	UPhysicsAssetGenerationSettings()
	{
	}

	UPROPERTY(EditAnywhere, Category="Body Creation", meta=(ShowOnlyInnerProperties))
	FPhysAssetCreateParams CreateParams;
};