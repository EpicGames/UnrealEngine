// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "EditorModeRegistry.h"
#include "EdMode.h"

class FEditorModeTools;
class FEditorViewportClient;
class FViewport;
class UEdMode;

class FPhysicsManipulationEdModeFactory : public IEditorModeFactory
{
	/** Called when the selection changes in the editor */
	virtual void OnSelectionChanged(FEditorModeTools& Tools, UObject* ItemUndergoingChange) const override;

	/** Gets the information pertaining to the mode type that this factory creates */
	virtual FEditorModeInfo GetModeInfo() const override;

	/** Create a new instance of our mode */
	virtual TSharedRef<FEdMode> CreateMode() const override;
};

/** 
 * Editor mode to manipulate physics objects in level editor viewport simulation
 **/
class FPhysicsManipulationEdMode : public FEdMode
{
public:
	FPhysicsManipulationEdMode();
	~FPhysicsManipulationEdMode();

	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FPhysicsManipulationEdMode");
	}

	virtual void Enter() override;
	virtual void Exit() override;

	virtual bool InputDelta(FEditorViewportClient* InViewportClient,FViewport* InViewport,FVector& InDrag,FRotator& InRot,FVector& InScale) override;
	virtual bool StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	virtual bool EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;

	virtual bool ShouldDrawWidget() const override		{ return true; };
	virtual bool UsesTransformWidget() const override	{ return true; };

private:
	TObjectPtr<class UPhysicsHandleComponent> HandleComp;

	FVector HandleTargetLocation;
	FRotator HandleTargetRotation;

};
