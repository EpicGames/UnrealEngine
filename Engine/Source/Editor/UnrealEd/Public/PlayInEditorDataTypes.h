// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Optional.h"
#include "Engine/GameInstance.h"
#include "UObject/GCObject.h"
#include "PlayInEditorDataTypes.generated.h"

USTRUCT()
struct FSlatePlayInEditorInfo
{
	GENERATED_BODY()
public:
	/** The spawned player for updating viewport location from player when pie closes */
	TWeakObjectPtr<class ULocalPlayer>	EditorPlayer;

	/** The current play in editor SWindow if playing in a floating window */
	TWeakPtr<class SWindow>				SlatePlayInEditorWindow;
	
	/** The current play in editor rendering and I/O viewport if playing in a floating window*/
	TSharedPtr<class FSceneViewport>	SlatePlayInEditorWindowViewport;
	
	/** The slate viewport that should be used for play in viewport */
	TWeakPtr<class IAssetViewport>		DestinationSlateViewport;

	FSlatePlayInEditorInfo()
	: SlatePlayInEditorWindow(NULL), DestinationSlateViewport(NULL)
	{}
};

/**
 * Data structure for storing PIE login credentials
 */
USTRUCT()
struct FPIELoginInfo
{
public:

	GENERATED_BODY()

	/** Type of account. Needed to identity the auth method to use (epic, internal, facebook, etc) */
	UPROPERTY()
	FString Type;
	/** Id of the user logging in (email, display name, facebook id, etc) */
	UPROPERTY()
	FString Id;
	/** Credentials of the user logging in (password or auth token) */
	UPROPERTY()
	FString Token;
};

/**
 * Holds various data to pass to the post login delegate for PIE logins
 */
struct FPieLoginStruct
{
	/** World context handle for this login */
	FName WorldContextHandle;

	/** X location for window positioning */
	int32 NextX;
	/** Y location for window positioning */
	int32 NextY;
	/** Passthrough start time of PIE */
	double PIEStartTime;

	/** A copy of the PIE Parameters this instance should use. */
	FGameInstancePIEParameters GameInstancePIEParameters;

	/** Which index is the instance of this Play in Editor session that will be created. */
	int32 PIEInstanceIndex;

	FPieLoginStruct(const FPieLoginStruct& InOther)
	{
		WorldContextHandle = InOther.WorldContextHandle;
		NextX = InOther.NextX;
		NextY = InOther.NextY;
		PIEStartTime = InOther.PIEStartTime;
		GameInstancePIEParameters = InOther.GameInstancePIEParameters;
		PIEInstanceIndex = InOther.PIEInstanceIndex;
	}

	FPieLoginStruct()
		: WorldContextHandle(NAME_None)
		, NextX(0)
		, NextY(0)
		, PIEStartTime(0)
		, PIEInstanceIndex(-1)
	{
	}
};

enum class EPlaySessionDestinationType : uint8
{
	/* Run this as a window within the current process. */
	InProcess,
	/* Run this as a new process on the local machine. */
	NewProcess,
	/* Run this through the Launcher (UAT) on a new process, possibly on a remote machine. */
	Launcher
};

enum class EPlaySessionWorldType : uint8
{
	/* Play in Editor. This spawns the Player Controller and is equivalent to playing the game. */
	PlayInEditor,

	/* Simulate in Editor. This skips spawning a Player Controller and just simulates the world. */
	SimulateInEditor
};

enum class EPlaySessionPreviewType : uint8
{
	/* No preview mode is applied. */
	NoPreview,
	/* Emulate Mobile ES3 Rendering. */
	MobilePreview,
	/* Emulate Vulkan Rendering. */
	VulkanPreview UE_DEPRECATED(5.7, "VulkanPreview is no longer supported, please use MobilePreview instead"),
	/* Attempt to launch in a HMD (Not used for VREditor) */
	VRPreview,
};

struct FRequestPlaySessionParams
{
public:
	// Default Constructor
	FRequestPlaySessionParams()
		: SessionDestination(EPlaySessionDestinationType::InProcess)
		, WorldType(EPlaySessionWorldType::PlayInEditor)
		, EditorPlaySettings(nullptr)
		, bAllowOnlineSubsystem(true)
	{
	}

	struct FLauncherDeviceInfo
	{
		/** The id of the Device selected in the Launch drop-down to launch on. Platform name is to the left of any @ symbol. */
		FString DeviceId;
		/** The name of the Device selected in the Launch drop-down to launch on. */
		FString DeviceName;
		/** If True, a remote play session will attempt to update the flash/software on the target device if it's out of date */
		bool bUpdateDeviceFlash = false;
		/** If True, the launch device is a Simulator */
		bool bIsSimulator = false;
		/** If set, the architecture used for building and deploying to the device */
		FString Architecture;
	};

	/** Where should the session be launched? May be local or remote. */
	EPlaySessionDestinationType SessionDestination;

	/** Is this PIE or SIE */
	EPlaySessionWorldType WorldType;

	/** If set, preview an emulated version of some rendering specifications. SessionDestination should be set to match. */
	TOptional<EPlaySessionPreviewType> SessionPreviewTypeOverride;

	/** A ULevelEditorPlaySettings instance that the session should be started with. nullptr means use the CDO. */
	TObjectPtr<ULevelEditorPlaySettings> EditorPlaySettings;

	/** If this is set the Play session will start from this location instead of using the GameMode to find a Player Spawn. */
	TOptional<FVector> StartLocation;

	/** If StartLocation is set, this can be used to optionally override the default (FRotator::ZeroRotator) rotation. */
	TOptional<FRotator> StartRotation;

	/** 
	* If specified, the Play Session will be created inside of this Slate Viewport.
	* Only supported on EPlaySessionDestinationType::InProcess destinations.
	*/
	TOptional<TWeakPtr<class IAssetViewport>> DestinationSlateViewport;

	/** 
	* If specified, the play session will attempt to launch on a remote device and 
	* not in the local editor. Requires EPlaySessionDestinationType::Launcher.
	*/
	TOptional<FLauncherDeviceInfo> LauncherTargetDevice;

	/**
	* If specified, this will be appended to the parameters being passed to the stand 
	* executable. Requires EPlaySessionDestinationType::NewProcess.
	*/
	TOptional<FString> AdditionalStandaloneCommandLineParameters;

	/** If specified, the PIE instance will be created inside this user-provided window instead of creating a default. */
	TWeakPtr<SWindow> CustomPIEWindow;

	TSubclassOf<AGameModeBase> GameModeOverride;

	/** Override which map is loaded for the Play session. This overrides both offline & servers (server only can be overridden in ULevelEditorPlaySettings) */
	FString GlobalMapOverride;
	
	/** If false, then PIE won't try to ask the Online Subsystem/"Use Online Logins" feature if it should authenticate the PIE sessions. */
	bool bAllowOnlineSubsystem;

	bool HasPlayWorldPlacement() const
	{
		return WorldType != EPlaySessionWorldType::SimulateInEditor && StartLocation.IsSet();
	}
};

/**
* This stores transient information about the current Play Session for the duration of the
* session. This allows us to cache information across async processes and hold useful
* information for clients who want to Late Join.
*/
struct FPlayInEditorSessionInfo
{
public:
	FPlayInEditorSessionInfo()
		: PlayRequestStartTime(0.0)
		, PlayRequestStartTime_StudioAnalytics(0.0)
		, PIEInstanceCount(0)
		, NumViewportInstancesCreated(0)
		, NumClientInstancesCreated(0)
		, NumOutstandingPIELogins(0)
		, bStartedInSpectatorMode(false)
		, bUsingOnlinePlatform(false)
		, bAnyBlueprintErrors(false)
		, bServerWasLaunched(false)
		, bLateJoinRequested(false)
	{
	}

	double PlayRequestStartTime;
	double PlayRequestStartTime_StudioAnalytics;

	// M
	FRequestPlaySessionParams OriginalRequestParams;

	struct FWindowSizeAndPos
	{
		FWindowSizeAndPos()
			: Size(0,0)
			, Position(0,0){}

		FIntPoint Size;
		FIntPoint Position;
	};

	/** How many PIE Instances have we opened. External dedicated servers don't count (but internal ones do). */
	int32 PIEInstanceCount;

	/** How many instances have we created viewports for. This is used for the window title (client number) and saving/loading window pos settings.*/
	int32 NumViewportInstancesCreated;

	/** 
	* How many clients have we created? This is different than PIEInstanceCount as we only count clients, 
	* and different from NumViewportInstancesCreated as this counts in-editor viewport clients as well.
	*/
	int32 NumClientInstancesCreated;
	
	/** How many PIE instances are we waiting to finish logging in? Used to track when we've finished getting PIE started. */
	int32 NumOutstandingPIELogins;

	/** Did we start in Spectator Mode? */
	bool bStartedInSpectatorMode;

	/** Are we using an online platform to do log-in authentication? */
	bool bUsingOnlinePlatform;

	/** Were there any Blueprint Errors when we started this PIE session? */
	bool bAnyBlueprintErrors;

	/** Have we launched a server for this PIE session yet? */
	bool bServerWasLaunched;

	/** If true, a late join client will be added on the next tick. */
	bool bLateJoinRequested;

	/** Transient information about window sizes/positions. This gets loaded from settings at start and saved at end. */
	TArray<FWindowSizeAndPos> CachedWindowInfo;

	/** Position of the last opened window. This allows us to spawn windows adjacent to each other instead of overlapping. */
	FWindowSizeAndPos LastOpenedWindowInfo;
};