// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/Platform.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateStructs.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableRow.h"

class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
class FText;
class FWidgetStyle;
class ITableRow;
class SButton;
class SWidget;
struct FGeometry;
struct FSlateBrush;


/**
* A collection of widgets and helpers used for the style of the trees in SKismetDebuggingView
*/
namespace PropertyInfoViewStyle
{

	/**
	 * Used to indent within stylized details tree to achieve a layered effect
	 * see KismetDebugViewStyle::SIndent for usage
	 * @param IndentLevel depth of the tree
	 * @param IsHovered will give a lighter color if this line in the tree is hovered
	 * @return the color to set the indent to
	 */
	UNREALED_API FSlateColor GetIndentBackgroundColor(int32 IndentLevel, bool IsHovered);

	/**
	 * calls KismetDebugViewStyle::GetIndentBackgroundColor using the indent level
	 * and hover state of the provided table row
	 * @param Row a table row - likely nested within a tree
	 * @return the color to set the background of the row to
	 */
	UNREALED_API FSlateColor GetRowBackgroundColor(ITableRow* Row);


	/**
	 * SIndent is a widget used to indent trees in a layered
	 * style. It's based on SDetailRowIndent but generalized to allow for use with
	 * any ITableRow
	 * 
	 * see SDebugLineItem::GenerateWidgetForColumn (SKismetDebuggingView.cpp) for
	 * use example
	 */
	class SIndent : public SCompoundWidget
	{ 
	public:
			
		SLATE_BEGIN_ARGS(SIndent) {}
		SLATE_END_ARGS()

		UNREALED_API void Construct(const FArguments& InArgs, TSharedRef<ITableRow> DetailsRow);

	private:
		UNREALED_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry,
		                      const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements,
		                      int32 LayerId, const FWidgetStyle& InWidgetStyle,
		                      bool bParentEnabled) const override;

		UNREALED_API virtual FOptionalSize GetIndentWidth() const;

		UNREALED_API virtual FSlateColor GetRowBackgroundColor(int32 IndentLevel) const;

	private:
		TWeakPtr<ITableRow> Row;
		float TabSize = 16.0f;
	};

	/**
	 * SExpanderArrow is a widget intended to be used alongside SIndent.
	 * It's based on SDetailExpanderArrow but generalized to allow for use with
	 * any ITableRow
	 * 
	 * see SDebugLineItem::GenerateWidgetForColumn (SKismetDebuggingView.cpp) for
	 * use example
	 */
	class SExpanderArrow : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SExpanderArrow) {}
			SLATE_ATTRIBUTE(bool, HasChildren)
		SLATE_END_ARGS()

		UNREALED_API void Construct(const FArguments& InArgs, TSharedRef<ITableRow> DetailsRow);

	private:
		EVisibility GetExpanderVisibility() const;

		const FSlateBrush* GetExpanderImage() const;

		FReply OnExpanderClicked() const;

	private:
		TWeakPtr<ITableRow> Row;
		TSharedPtr<SButton> ExpanderArrow;
		TAttribute<bool> HasChildren;
	};

	/**
	 * allows you to highlight any arbitrary text based widget.
	 * See FTraceStackChildItem::GenerateValueWidget (SKismetDebuggingView.cpp)
	 * for an example.
	 *
	 * note: the text in Content must be scaled, positioned, etc exactly
	 * the same as a standard STextBlock would be in order for the highlight
	 * to match up properly
	 */
	class STextHighlightOverlay : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(STextHighlightOverlay) {}
			SLATE_DEFAULT_SLOT( FArguments, Content )
			SLATE_ATTRIBUTE(FText, FullText)
			SLATE_ATTRIBUTE(FText, HighlightText)
		SLATE_END_ARGS()

		UNREALED_API void Construct(const FArguments& InArgs);
	};
}
