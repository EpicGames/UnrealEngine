// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/ArchiveUObject.h"
#include "UObject/GCObject.h"

typedef TMap< TObjectPtr<UObject>, TSet<TObjectPtr<UObject>> >	ObjectReferenceGraph;
typedef TMap< UObject*, FString >			ObjectNameMap;

/**
 * Data container to hold information about what is referencing a given
 * set of assets.
 */
struct FReferencedAssets
{
	/**
	 * The object that holding a reference to the set of assets
	 */
	UObject* Referencer;

	/**
	 * The set of assets that are being referenced
	 */
	TSet<UObject*> AssetList;

	/** Default ctor */
	FReferencedAssets() :
		Referencer(NULL)
	{}

	/** 
	 * Sets the name of the referencer
	 */
	FReferencedAssets(UObject* InReferencer) :
		Referencer(InReferencer)
	{
	}

	// serializer
	friend FArchive& operator<<( FArchive& Ar, FReferencedAssets& Asset )
	{
		return Ar << Asset.Referencer << Asset.AssetList;
	}
};

/**
 * This archive searches objects for assets. It determines the set of
 * assets by whether they support thumbnails or not. Possibly, not best
 * but it displays everything as thumbnails, so...
 */
class FFindAssetsArchive : public FArchiveUObject
{
	/**
	 * The root object that was used to being serialization for this archive
	 */
	UObject* StartObject;

	/**
	 * The object currently being serialized.
	 */
	UObject* CurrentObject;

	/** The array to add any found assets too */
	TSet<UObject*>& AssetList;

	/**
	* Set when the global asset list is updated.  Used to prevent the reference graph from being
	* polluted by calls to the public version of BuildAssetList.
	*/
	ObjectReferenceGraph* CurrentReferenceGraph;

	/**
	 * if false, ignore all assets referenced only through script
	 */
	bool bIncludeScriptRefs;

	/**
	 * if false, ignore all assets referenced only through archetype/class default objects
	 */
	bool bIncludeDefaultRefs;

	/**
	 * Maximum depth to recursively serialize objects; 0 indicates no limit to recursion
	 */
	const int32 MaxRecursionDepth;

	/**
	 * Current Recursion Depth
	 */
	int32 CurrentDepth;

	/**
	 * If true, the reference graph map key-value pairs will be stored as child-parent instead of parent-child
	 */
	bool bUseReverseReferenceGraph;

	/**
	 * Manually serializes the class and archetype for the specified object so that assets which are referenced
	 * through the object's class/archetype can be differentiated.
	 */
	void HandleReferencedObject( UObject* Obj );

	/**
	 * Retrieves the referenced assets list for the specified object.
	 */
	TSet<TObjectPtr<UObject>>* GetAssetList( UObject* Referencer );

public:
	/**
	 * Functor that starts the serialization process
	 */
	UNREALED_API FFindAssetsArchive( 
		UObject* InSearch, 
		TSet<UObject*>& OutAssetList, 
		ObjectReferenceGraph* ReferenceGraph=NULL, 
		int32 MaxRecursion=0, 
		bool bIncludeClasses=true, 
		bool bIncludeDefaults=false,
		bool bReverseReferenceGraph=false );

	/**
	 * Adds the object refence to the asset list if it supports thumbnails.
	 * Recursively searches through its references for more assets
	 *
	 * @param Obj the object to inspect
	 */
	FArchive& operator<<(class UObject*& Obj);
};

class FFindReferencedAssets : 
	public FGCObject
{
public:

	UNREALED_API FFindReferencedAssets();
	UNREALED_API virtual ~FFindReferencedAssets();

	/* === FGCObject interface === */
	UNREALED_API virtual void AddReferencedObjects( FReferenceCollector& Collector );
	virtual FString GetReferencerName() const override
	{
		return TEXT("FFindReferencedAssets");
	}

	/**
	 * Checks an object to see if it should be included for asset searching
	 *
	 * @param Object the object in question
	 * @param ClassesToIgnore the list of classes to skip
	 * @param PackagesToIgnore the list of packages to skip
	 * @param bIncludeDefaults specify true to include content referenced through defaults
	 *
	 * @return true if it should be searched, false otherwise
	 */
	UNREALED_API static bool ShouldSearchForAssets( const UObject* Object, const TArray<UClass*>& ClassesToIgnore, const TArray<UPackage*>& PackagesToIgnore, bool bIncludeDefaults=false );

	/**
	 * Returns a list of assets referenced by the specified UObject.
	 * @param Object the object used to build the references
	 * @param IgnoreClasses the list of classes to skip
	 * @param IgnorePackages the list of packages to skip
	 * @param bIncludeDefaultRefs specify true to include content referenced through defaults
	 * @param bOnlyDirectReferences specify true to only include direct references
	 * 
	 * @return list of assets referenced by the specified UObject.
	 */
	UNREALED_API static void BuildAssetList(UObject *Object, const TArray<UClass*>& IgnoreClasses, const TArray<UPackage*>& IgnorePackages, TSet<UObject*>& ReferencedAssets, bool bIncludeDefaultRefs=false, bool bOnlyDirectReferences=false);

protected:
	/**
	 * Listens for MapChange events.  Clears all references to actors in the current level.
	 */
	void OnEditorMapChange(uint32 Flag);

	/**
	 * This is a list of classes that should be ignored when building the
	 * asset list as they are always loaded and therefore not pertinent
	 */
	TArray<TObjectPtr<UClass>> IgnoreClasses;

	/**
	 * This is a list of packages that should be ignored when building the
	 * asset list as they are always loaded and therefore not pertinent
	 */
	TArray<TObjectPtr<UPackage>> IgnorePackages;

	/**
	 * Holds the list of assets that are being referenced by the current 
	 * selection
	 */
	TArray<FReferencedAssets> Referencers;

	/**
	 * The object graph for the assets referenced by the currently selected actors.
	 */
	ObjectReferenceGraph ReferenceGraph;

	/** Handle to the registered OnEditorMapChange delegate. */
	FDelegateHandle OnEditorMapChangeDelegateHandle;
};
