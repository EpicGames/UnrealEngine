// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class SActorEditorContext : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SActorEditorContext) {}
	SLATE_ARGUMENT(UWorld*, World)
	SLATE_END_ARGS()

	UNREALED_API void Construct(const FArguments& InArgs);
	UNREALED_API ~SActorEditorContext();
	UNREALED_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	static UNREALED_API bool IsVisible(UWorld* InWorld);
private:

	void OnEditorMapChange(uint32 MapChangeFlags = 0) { Rebuild(); }
	void OnActorEditorContextSubsystemChanged();
	void Rebuild();
	
	bool bNeedsRebuild;
	bool bIsContextExpanded;
	UWorld* World;
};
