// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "SEditorViewport.h"

class FEditorViewportClient;
class FViewportTabContent;
class FMenuBuilder;
struct FAssetEditorViewportConstructionArgs;

class SAssetEditorViewport : public SEditorViewport
{
public:

	SLATE_BEGIN_ARGS(SAssetEditorViewport)
		{
		}

		SLATE_ATTRIBUTE(FVector2D, ViewportSize);
		SLATE_ARGUMENT(TSharedPtr<FEditorViewportClient>, EditorViewportClient)
	SLATE_END_ARGS()

	UNREALED_API void Construct(const FArguments& InArgs, const FAssetEditorViewportConstructionArgs& InViewportConstructionArgs);
	UNREALED_API void OnSetViewportConfiguration(FName ConfigurationName);
	UNREALED_API bool IsViewportConfigurationSet(FName ConfigurationName) const;

	UNREALED_API void GenerateLayoutMenu(FMenuBuilder& MenuBuilder) const;

protected:
	UNREALED_API virtual void BindCommands() override;

	UNREALED_API virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	FName ConfigKey;
	TWeakPtr<FViewportTabContent> ParentTabContent;
};
