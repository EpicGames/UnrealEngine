// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * UI to pick options when importing a data table
 */

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Curves/RealCurve.h"
#include "Delegates/Delegate.h"
#include "Factories/CSVImportFactory.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Misc/Optional.h"
#include "Serialization/Archive.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"

class SWidget;
class SWindow;
class UDataTable;
class UScriptStruct;

enum class ECSVImportOptionDlgResponse : uint8
{
	Import,
	ImportAll,
	Cancel
};

class SCSVImportOptions : public SCompoundWidget
{
private:
	/** Typedef for curve enum pointers */
	typedef TSharedPtr<ERichCurveInterpMode>		CurveInterpModePtr;

public:
	SLATE_BEGIN_ARGS(SCSVImportOptions)
		: _WidgetWindow()
		, _FullPath()
		, _TempImportDataTable()
		, _bReimportingCurveTable()
		{}

		SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
		SLATE_ARGUMENT(FText, FullPath)
		SLATE_ARGUMENT(UDataTable*, TempImportDataTable)
		SLATE_ARGUMENT(bool, bReimportingCurveTable)
	SLATE_END_ARGS()

	SCSVImportOptions()
		: UserDlgResponse(ECSVImportOptionDlgResponse::Cancel)
		, SelectedImportType(ECSVImportType::ECSV_DataTable)
		, SelectedStruct(nullptr)
		, TempImportDataTable(nullptr)
		{}

	UNREALED_API void Construct(const FArguments& InArgs);

	/** If we should import */
	UNREALED_API bool ShouldImport();

	/** If the current settings should be applied to all items being imported */
	UNREALED_API bool ShouldImportAll();

	/** Get the row struct we selected */
	UNREALED_API UScriptStruct* GetSelectedRowStruct();

	/** Get the import type we selected */
	UNREALED_API ECSVImportType GetSelectedImportType();

	/** Get the interpolation mode we selected */
	UNREALED_API ERichCurveInterpMode GetSelectedCurveIterpMode();

	/** Whether to show table row options */
	UNREALED_API EVisibility GetTableRowOptionVis() const;

	/** Whether to show curve type options */
	UNREALED_API EVisibility GetCurveTypeVis() const;

	/** Whether to show details panel */
	UNREALED_API EVisibility GetDetailsPanelVis() const;

	UNREALED_API FString GetImportTypeText(TSharedPtr<ECSVImportType> Type) const;

	/** Called to create a widget for each struct */
	UNREALED_API TSharedRef<SWidget> MakeImportTypeItemWidget(TSharedPtr<ECSVImportType> Type);

	/** Called when import type changes */
	UNREALED_API void OnImportTypeSelected(TSharedPtr<ECSVImportType> Selection, ESelectInfo::Type SelectionType);

	/** Called when datatable row is selected */
	UNREALED_API void OnStructSelected(UScriptStruct* NewStruct);

	UNREALED_API FString GetCurveTypeText(CurveInterpModePtr InterpMode) const;

	/** Called to create a widget for each curve interpolation enum */
	UNREALED_API TSharedRef<SWidget> MakeCurveTypeWidget(CurveInterpModePtr InterpMode);

	/** Called when 'Apply' button is pressed */
	UNREALED_API FReply OnImport();

	/** Do we have all of the data we need to import this asset? */
	UNREALED_API bool CanImport() const;

	/** Called when 'Cancel' button is pressed */
	UNREALED_API FReply OnCancel();

	UNREALED_API FText GetSelectedItemText() const;

	UNREALED_API FText GetSelectedCurveTypeText() const;

private:
	UNREALED_API FReply HandleImport();
	UNREALED_API FReply OnImportAll();

	/** Whether we should go ahead with import */
	ECSVImportOptionDlgResponse					UserDlgResponse;

	/** Window that owns us */
	TWeakPtr< SWindow >							WidgetWindow;

	/** Whether we are reimporting a curve table */
	bool										bReimportingCurveTable;

	// Import type

	/** List of import types to pick from, drives combo box */
	TArray< TSharedPtr<ECSVImportType> >						ImportTypes;

	/** The combo box */
	TSharedPtr< SComboBox< TSharedPtr<ECSVImportType> > >		ImportTypeCombo;

	/** Indicates what kind of asset we want to make from the CSV file */
	ECSVImportType												SelectedImportType;


	// Row type

	/** The row struct combo box */
	TSharedPtr< SWidget >							RowStructCombo;

	/** The selected row struct */
	UScriptStruct*									SelectedStruct;

	/** Temp DataTable to hold import options */
	TWeakObjectPtr< UDataTable >					TempImportDataTable;

	/** The curve interpolation combo box */
	TSharedPtr< SComboBox<CurveInterpModePtr> >		CurveInterpCombo;

	/** A property view to edit advanced options */
	TSharedPtr< class IDetailsView >				PropertyView;

	/** All available curve interpolation modes */
	TArray< CurveInterpModePtr >					CurveInterpModes;

	/** The selected curve interpolation type */
	ERichCurveInterpMode							SelectedCurveInterpMode;
};
