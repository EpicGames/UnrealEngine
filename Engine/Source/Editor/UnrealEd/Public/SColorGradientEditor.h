// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/SlateRect.h"
#include "Layout/Geometry.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Rendering/RenderingCommon.h"
#include "Curves/KeyHandle.h"
#include "Curves/RichCurve.h"
#include "Widgets/SLeafWidget.h"

class FCurveOwnerInterface;
class FPaintArgs;
class FSlateWindowElementList;
struct FColorGradientEditorGeometrySolver;

struct FGradientStopMark
{
public:

	UNREALED_API FGradientStopMark();
	UNREALED_API FGradientStopMark( float InTime, FKeyHandle InRedKeyHandle, FKeyHandle InGreenKeyHandle, FKeyHandle InBlueKeyHandle, FKeyHandle InAlphaKeyHandle = FKeyHandle() );


	UNREALED_API bool operator==( const FGradientStopMark& Other ) const;
	UNREALED_API bool IsValid( FCurveOwnerInterface& CurveOwner );
	UNREALED_API bool IsValidAlphaMark( const TArray<FRichCurveEditInfo>& Curves ) const;
	UNREALED_API bool IsValidColorMark( const TArray<FRichCurveEditInfo>& Curves ) const;
	UNREALED_API FLinearColor GetColor( FCurveOwnerInterface& CurveOwner ) const;
	UNREALED_API float GetTime(FCurveOwnerInterface& CurveOwner) const;
	UNREALED_API void SetColor( const FLinearColor& InColor, FCurveOwnerInterface& CurveOwner );
	UNREALED_API void SetTime(float NewTime, FCurveOwnerInterface& CurveOwner);

public:
	float Time;
	FKeyHandle RedKeyHandle;
	FKeyHandle GreenKeyHandle;
	FKeyHandle BlueKeyHandle;
	FKeyHandle AlphaKeyHandle;

};

class SColorGradientEditor : public SLeafWidget
{
public:
	SLATE_BEGIN_ARGS( SColorGradientEditor ) 
		: _ViewMinInput(0.0f)
		, _ViewMaxInput(0.0f)
		, _IsEditingEnabled( true )
		, _ClampStopsToViewRange( false )
		, _WithAlphaChannel( true )
		, _DrawColorAndAlphaSeparate( true )
	{}
		SLATE_ATTRIBUTE( float, ViewMinInput )
		SLATE_ATTRIBUTE( float, ViewMaxInput )
		SLATE_ATTRIBUTE( bool, IsEditingEnabled )
		SLATE_ATTRIBUTE(bool, HideStops)
		SLATE_ARGUMENT( bool, ClampStopsToViewRange )
		SLATE_ARGUMENT( bool, WithAlphaChannel )
		SLATE_ARGUMENT(bool, DrawColorAndAlphaSeparate )
		SLATE_ATTRIBUTE(TOptional<FVector2D>, DesiredSizeOverride)
	SLATE_END_ARGS()

	UNREALED_API SColorGradientEditor();
	inline virtual ~SColorGradientEditor() = default;

	UNREALED_API void Construct( const FArguments& InArgs );

	/** SWidget Interface */
	virtual bool SupportsKeyboardFocus() const override { return true; }
	UNREALED_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	UNREALED_API virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UNREALED_API virtual FReply OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override;
	UNREALED_API virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UNREALED_API virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UNREALED_API virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	UNREALED_API virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	UNREALED_API virtual FVector2D ComputeDesiredSize(float) const override;
	UNREALED_API virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override;

	/**
	 * Sets the curve that is being edited by this editor
	 *
	 * @param InCurveOwner Interface to the curves to edit
	 */
	UNREALED_API void SetCurveOwner( FCurveOwnerInterface* InCurveOwner );

private:
	/**
	 * Opens a context menu with options for how we display the gradient
	 *
	 * @param MouseEvent	The mouse event that raised this handler (containing the location and widget path of the click)
	 */
	UNREALED_API void OpenGradientOptionsMenu(const FPointerEvent& MouseEvent);

	/**
	 * Opens a context menu with options for the selected gradient stop
	 *
	 * @param MouseEvent	The mouse event that raised this handler (containing the location and widget path of the click)
	 */
	UNREALED_API void OpenGradientStopContextMenu(const FPointerEvent& MouseEvent);

	/**
	 * Opens a color picker to change the color of the selected stop
	 */
	UNREALED_API void OpenGradientStopColorPicker();

	/**
	 * Called when the selected stop color changes from the color picker
	 *
	 * @param InNewColor	The new color
	 */
	UNREALED_API void OnSelectedStopColorChanged( FLinearColor InNewColor );

	/**
	 * Called when canceling out of the color picker
	 *
	 * @param PreviousColor	The color before anything was changed
	 */
	UNREALED_API void OnCancelSelectedStopColorChange( FLinearColor PreviousColor );

	/**
	 * Called right before a user begins using the slider to change the alpha value of a stop
	 */
	UNREALED_API void OnBeginChangeAlphaValue();

	/**
	 * Called right after a user ends using the slider to change the alpha value of a stop
	 */
	UNREALED_API void OnEndChangeAlphaValue( float NewValue );

	/**
	 * Called when the alpha value of a stop changes
	 */
	UNREALED_API void OnAlphaValueChanged( float NewValue );

	/**
	 * Called when the alpha value of a stop changes by typing into the sliders text box
	 */
	UNREALED_API void OnAlphaValueCommitted( float NewValue, ETextCommit::Type );

	/**
	 * Called to remove the selected gradient stop
	 */
	UNREALED_API void OnRemoveSelectedGradientStop();

	/**
	 * Called when the gradient stop time is changed by typing into the text box
	 */
	UNREALED_API void OnTimeValueCommited( float NewValue, ETextCommit::Type Type );

	/**
	 * Draws a single gradient stop
	 *
	 * @param Mark				The stop mark to draw
	 * @param Geometry			The geometry of the area to draw in
	 * @param XPos				The local X location where the stop should be placed at
	 * @param Color				The color of the stop
	 * @param DrawElements		List of draw elements to add to
	 * @param LayerId			The layer to start drawing on
	 * @param InClippingRect	The clipping rect 
	 * @param DrawEffects		Any draw effects to apply
	 * @param bColor			If true RGB of the Color param will be used, otherwise the A value will be used
	 */
	UNREALED_API void DrawGradientStopMark( const FGradientStopMark& Mark, const FGeometry& Geometry, float XPos, const FLinearColor& Color, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FSlateRect& InClippingRect, ESlateDrawEffect DrawEffects, bool bColor, const FWidgetStyle& InWidgetStyle ) const;

	/**
	 * Gets the gradient stop (if any) of at the current mouse position
	 *
	 * @param MousePos		The screen space mouse position to check
	 * @param MyGeometry	The geometry that was clicked in
	 * @return				The stop mark at MousePos or invalid if none was found
	 */
	UNREALED_API FGradientStopMark GetGradientStopAtPoint( const FVector2D& MousePos, const FGeometry& MyGeometry );

	/**
	 * Get all gradient stop marks on the curve
	 */
	UNREALED_API void GetGradientStopMarks( TArray<FGradientStopMark>& OutColorMarks, TArray<FGradientStopMark>& OutAlphaMarks ) const;

	/**
	 * Removes a gradient stop
	 *
	 * @param InMark	The stop to remove
	 */
	UNREALED_API void DeleteStop( const FGradientStopMark& InMark );

	/**
	 * Adds a gradient stop
	 *
	 * @param Position		The position to add the stop at
	 * @param MyGeometry	The geometry of the stop area
	 * @param bColorStop	If true a color stop is added, otherwise an alpha stop is added
	 */
	UNREALED_API FGradientStopMark AddStop( const FVector2D& Position, const FGeometry& MyGeometry, bool bColorStop, TOptional<FLinearColor> Color );

	/**
	 * Moves a gradient stop to a new time
	 *
	 * @param Mark		The gradient stop to move
	 * @param NewTime	The new time to move to
	 */
	UNREALED_API void MoveStop( FGradientStopMark& Mark, float NewTime );

	bool ShouldHideStops() const;

private:
	/** The currently selected stop */
	FGradientStopMark SelectedStop;
	/** interface to the curves being edited */
	FCurveOwnerInterface* CurveOwner;
	/** Current min input value that is visible */
	TAttribute<float> ViewMinInput;
	/** Current max input value that is visible */
	TAttribute<float> ViewMaxInput;
	/** Whether or not the gradient is editable or just viewed */
	TAttribute<bool> IsEditingEnabled;
	/** Whether or not to clamp the time value of stops to the view range. */
	bool bClampStopsToViewRange;
	/** Cached position where context menus should appear */
	FVector2D ContextMenuPosition;
	/** Whether or not the color gradient stop area is hovered */
	bool bColorAreaHovered;
	/** Whether or not the alpha gradient stop area is hovered */
	bool bAlphaAreaHovered;
	/** Current distance dragged since we captured the mouse */
	float DistanceDragged;
	/** True if an alpha value is being dragged */
	bool bDraggingAlphaValue;
	/** True if a gradient stop is being dragged */
	bool bDraggingStop;
	/** Do we draw a gradient for color and alpha separate or combined? */
	bool bDrawColorAndAlphaSeparate;
	/** True to request hide stops (will only be fullfilled when IsEditingEnabled = false) */
	TAttribute<bool> bRequestHideStops;
	/** desired size override */
	TSlateAttribute<TOptional<FVector2D>, EInvalidateWidgetReason::Layout> DesiredSizeOverride;

	/** Solver for the geometry in this widget */
	TPimplPtr<FColorGradientEditorGeometrySolver> GeometrySolver;
};
