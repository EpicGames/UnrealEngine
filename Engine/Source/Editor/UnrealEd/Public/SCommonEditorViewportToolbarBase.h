// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SViewportToolBar.h"
#include "IPreviewProfileController.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "ViewportToolbar/UnrealEdViewportToolbarContext.h"

#include "SCommonEditorViewportToolbarBase.generated.h"

#define UE_API UNREALED_API

class SComboButton;

// This is the interface that the host of a SCommonEditorViewportToolbarBase must implement
class ICommonEditorViewportToolbarInfoProvider
{
public:
	// Get the viewport widget
	virtual TSharedRef<class SEditorViewport> GetViewportWidget() = 0;

// 	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
// 	TSharedPtr<FExtender> LevelEditorExtenders = LevelEditorModule.GetMenuExtensibilityManager()->GetAllExtenders();
	virtual TSharedPtr<FExtender> GetExtenders() const = 0;

	// Called to inform the host that a button was clicked (typically used to focus on a particular viewport in a multi-viewport setup)
	virtual void OnFloatingButtonClicked() = 0;
};

namespace CommonEditorViewportUtils
{
	struct FShowMenuCommand
	{
		TSharedPtr<FUICommandInfo> ShowMenuItem;
		FText LabelOverride;

		UE_DEPRECATED(5.5, "Use the version of the show flags builder in FShowFlagMenuCommands")
		FShowMenuCommand(TSharedPtr<FUICommandInfo> InShowMenuItem, const FText& InLabelOverride)
			: ShowMenuItem(InShowMenuItem)
			, LabelOverride(InLabelOverride)
		{
		}

		UE_DEPRECATED(5.5, "Use the version of the show flags builder in FShowFlagMenuCommands")
		FShowMenuCommand(TSharedPtr<FUICommandInfo> InShowMenuItem)
			: ShowMenuItem(InShowMenuItem)
		{
		}
	};

	UE_DEPRECATED(5.5, "Use the version of the show flags builder in FShowFlagMenuCommands::BuildShowFlagsMenu which takes a UToolMenu instead")
	static inline void FillShowMenu(class FMenuBuilder& MenuBuilder, TArray<FShowMenuCommand> MenuCommands, int32 EntryOffset)
	{
		// Generate entries for the standard show flags
		// Assumption: the first 'n' entries types like 'Show All' and 'Hide All' buttons, so insert a separator after them
		for (int32 EntryIndex = 0; EntryIndex < MenuCommands.Num(); ++EntryIndex)
		{
			MenuBuilder.AddMenuEntry(MenuCommands[EntryIndex].ShowMenuItem, NAME_None, MenuCommands[EntryIndex].LabelOverride);
			if (EntryIndex == EntryOffset - 1)
			{
				MenuBuilder.AddMenuSeparator();
			}
		}
	}
}

UCLASS(MinimalAPI)
class UCommonViewportToolbarBaseMenuContext : public UUnrealEdViewportToolbarContext
{
	GENERATED_BODY()

public:
	TWeakPtr<const class SCommonEditorViewportToolbarBase> ToolbarWidget;
	
	UE_API virtual TSharedPtr<IPreviewProfileController> GetPreviewProfileController() const override;
};

class SPreviewSceneProfileSelector : public SCompoundWidget
{
public:
	virtual ~SPreviewSceneProfileSelector() override = default;

	SLATE_BEGIN_ARGS(SPreviewSceneProfileSelector)
	{}
	SLATE_ARGUMENT(TSharedPtr<IPreviewProfileController>, PreviewProfileController)
	SLATE_END_ARGS()
	
	UE_API void Construct(const FArguments& InArgs);

protected:
	UE_DEPRECATED(5.5, "Unused")
	void UpdateAssetViewerProfileList()
	{
	}

	UE_DEPRECATED(5.5, "Unused")
	void UpdateAssetViewerProfileSelection()
	{
	}

	UE_DEPRECATED(5.5, "Unused")
	void OnSelectionChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type /*SelectInfo*/)
	{
	}

	/** Creates and returns the asset viewer profile combo box.*/
	UE_API TSharedRef<SWidget> MakeAssetViewerProfileComboBox();

private:
	/** Interface to set/get/list the preview profiles. */
	TSharedPtr<IPreviewProfileController> PreviewProfileController;

	/** Displays/Selects the active advanced viewer profile. */
	TSharedPtr<SComboButton> AssetViewerProfileComboButton;

	/** Builds the drop-down list for selecting a viewer profile. */
	UE_API TSharedRef<SWidget> BuildComboMenu();
};

/**
 * A viewport toolbar widget for an asset or level editor that is placed in a viewport
 */
class SCommonEditorViewportToolbarBase : public SViewportToolBar
{
public:
	SLATE_BEGIN_ARGS(SCommonEditorViewportToolbarBase)
		: _AddRealtimeButton(false)
		, _PreviewProfileController(nullptr)
		{}

		SLATE_ARGUMENT(bool, AddRealtimeButton)
		SLATE_ARGUMENT(TSharedPtr<IPreviewProfileController>, PreviewProfileController) // Should be null if the Preview doesn't require profile.
	SLATE_END_ARGS()

	virtual ~SCommonEditorViewportToolbarBase()= default;

	UNREALED_API void Construct(const FArguments& InArgs, TSharedPtr<class ICommonEditorViewportToolbarInfoProvider> InInfoProvider);

	UE_DEPRECATED(5.6, "Use the version taking UToolMenu as argument: UE::UnrealEd::ConstructScreenPercentageMenu(InMenu)")
	static UNREALED_API void ConstructScreenPercentageMenu(FMenuBuilder& MenuBuilder, class FEditorViewportClient* ViewportClient);

	/** Gets the preview profile controller explicitly set to this toolbar */
	const TSharedPtr<IPreviewProfileController>& GetPreviewProfileController() const { return PreviewProfileController; }

private:
	/**
	 * Returns the label for the "Camera" tool bar menu, which changes depending on the viewport type
	 *
	 * @return	Label to use for this menu label
	 */
	UNREALED_API FText GetCameraMenuLabel() const;
	
	UNREALED_API FSlateIcon GetCameraMenuIcon() const;

	/**
	 * Generates the toolbar options menu content 
	 *
	 * @return The widget containing the options menu content
	 */
	UNREALED_API TSharedRef<SWidget> GenerateOptionsMenu() const;

	/**
	 * Generates the toolbar camera menu content 
	 *
	 * @return The widget containing the view menu content
	 */
	UNREALED_API TSharedRef<SWidget> GenerateCameraMenu() const;

	/**
	 * Generates the toolbar view menu content 
	 *
	 * @return The widget containing the view menu content
	 */
	UNREALED_API TSharedRef<SWidget> GenerateViewMenu() const;

	/**
	 * Generates the toolbar show menu content 
	 *
	 * @return The widget containing the show menu content
	 */
	UNREALED_API virtual TSharedRef<SWidget> GenerateShowMenu() const;

	/**
	 * Returns the initial visibility of the view mode options widget 
	 *
	 * @return The visibility value
	 */
	UNREALED_API EVisibility GetViewModeOptionsVisibility() const;

	/**
	 * Generates the toolbar view param menu content 
	 *
	 * @return The widget containing the show menu content
	 */
	UNREALED_API TSharedRef<SWidget> GenerateViewModeOptionsMenu() const;

	/** Called by the FOV slider in the perspective viewport to get the FOV value */
	UNREALED_API float OnGetFOVValue() const;

	/** Called by the far view plane slider in the perspective viewport to get the far view plane value */
	UNREALED_API float OnGetFarViewPlaneValue() const;

	/** Called when the far view plane slider is adjusted in the perspective viewport */
	UNREALED_API void OnFarViewPlaneValueChanged( float NewValue );

	/** Called when we click the realtime warning */
	UNREALED_API FReply OnRealtimeWarningClicked();
	/** Called to determine if we should show the realtime warning */
	UNREALED_API EVisibility GetRealtimeWarningVisibility() const;

protected:
	/**
	 * @return The widget containing the perspective only FOV window.
	 */
	UNREALED_API TSharedRef<SWidget> GenerateFOVMenu() const;
	/**
	 * @return The widget containing the far view plane slider.
	 */
	UNREALED_API TSharedRef<SWidget> GenerateFarViewPlaneMenu() const;

	// Merges the extender list from the host with the specified extender and returns the results
	UNREALED_API TSharedPtr<FExtender> GetCombinedExtenderList(TSharedRef<FExtender> MenuExtender) const;

	/** Gets the extender for the view menu */
	UNREALED_API virtual TSharedPtr<FExtender> GetViewMenuExtender() const;

	UNREALED_API void CreateViewMenuExtensions(FMenuBuilder& MenuBuilder);

	/** Extension allowing derived classes to add to the options menu.*/	
	UNREALED_API virtual void ExtendOptionsMenu(FMenuBuilder& OptionsMenuBuilder) const;

	/** Extension allowing derived classes to add to left-aligned portion of the toolbar slots.*/
	UNREALED_API virtual void ExtendLeftAlignedToolbarSlots(TSharedPtr<SHorizontalBox> MainBoxPtr, TSharedPtr<SViewportToolBar> ParentToolBarPtr) const;

	UNREALED_API virtual void FillShowFlagsMenu(class UToolMenu* InMenu) const;

	// Returns the info provider for this viewport
	UNREALED_API ICommonEditorViewportToolbarInfoProvider& GetInfoProvider() const;

	// Get the viewport client
	UNREALED_API class FEditorViewportClient& GetViewportClient() const;

	// Creates the view menu widget (override point for children)
	UNREALED_API virtual TSharedRef<class SEditorViewportViewMenu> MakeViewMenu();

	UNREALED_API FText GetScalabilityWarningLabel() const;
	UNREALED_API EVisibility GetScalabilityWarningVisibility() const;
	UNREALED_API TSharedRef<SWidget> GetScalabilityWarningMenuContent() const;
	virtual bool GetShowScalabilityMenu() const
	{
		return false;
	}
	/** Called when the FOV slider is adjusted in the perspective viewport */
	UNREALED_API virtual void OnFOVValueChanged(float NewValue) const;

	/** Called when the ScreenPercentage slider is adjusted in the viewport */
	UNREALED_API void OnScreenPercentageValueChanged(int32 NewValue);

private:
	/** The viewport that we are in */
	TWeakPtr<class ICommonEditorViewportToolbarInfoProvider> InfoProviderPtr;
	TSharedPtr<IPreviewProfileController> PreviewProfileController;
	
	TSharedPtr<class SEditorViewportViewMenu> BlankViewMenu;
	
	/// Automatic Legacy Upgrade Support
	// True when generating the tool menu widget for the first time.
	// Used to avoid calling menu generation functions right on construction, as by default Tool Menus could call those function earlier than before. 
	bool bIsGeneratingToolMenuWidget = false;
	// Allows functions to modulate behavior depending on whether the toolbar is in the new context.
	mutable bool bIsBuildingToolMenu = false;
	mutable bool bHasExtendedSettingsMenu = true;
	mutable bool bHasExtendedLeftSide = true;
	mutable bool bUsesDefaultViewMenu = false;
	TSharedPtr<SWidget> MakeLegacyShowMenu() const;
	bool ShouldCreateOptionsMenu() const;
};

#undef UE_API
