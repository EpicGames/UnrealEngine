// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Templates/SubclassOf.h"
#include "Components/ActorComponent.h"
#include "SubobjectData.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"

class FComponentClassComboEntry;
class SToolTip;
class FTextFilterExpressionEvaluator;
class IClassViewerFilter;
class IUnloadedBlueprintData;
class FClassViewerFilterFuncs;
class FClassViewerFilterOption;
class FClassViewerInitializationOptions;

typedef TSharedPtr<class FComponentClassComboEntry> FComponentClassComboEntryPtr;

//////////////////////////////////////////////////////////////////////////

namespace EComponentCreateAction
{
	enum Type
	{
		/** Create a new C++ class based off the specified ActorComponent class and then add it to the tree */
		CreateNewCPPClass,
		/** Create a new blueprint class based off the specified ActorComponent class and then add it to the tree */
		CreateNewBlueprintClass,
		/** Spawn a new instance of the specified ActorComponent class and then add it to the tree */
		SpawnExistingClass,
	};
}


DECLARE_DELEGATE_OneParam(FOnSubobjectCreated, FSubobjectDataHandle);

DECLARE_DELEGATE_RetVal_ThreeParams( UActorComponent*, FComponentClassSelected, TSubclassOf<UActorComponent>, EComponentCreateAction::Type, UObject*);
DECLARE_DELEGATE_RetVal_ThreeParams( FSubobjectDataHandle, FSubobjectClassSelected, TSubclassOf<UActorComponent>, EComponentCreateAction::Type, UObject*);

struct FComponentEntryCustomizationArgs
{
	/** Specific asset to use instead of the selected asset in the content browser */
	TWeakObjectPtr<UObject> AssetOverride;
	/** Custom name to display */
	FString ComponentNameOverride;
	
	/** Callback when a new subobject is created */
	FOnSubobjectCreated OnSubobjectCreated;
	/** Brush icon to use instead of the class icon */
	FName IconOverrideBrushName;
	/** Custom sort priority to use (smaller means sorted first) */
	int32 SortPriority;

	FComponentEntryCustomizationArgs()
		: AssetOverride( nullptr )
		, ComponentNameOverride()
		, IconOverrideBrushName( NAME_None )
		, SortPriority(0)
	{
	
	}
};
class FComponentClassComboEntry: public TSharedFromThis<FComponentClassComboEntry>
{
public:
	FComponentClassComboEntry( const FString& InHeadingText, TSubclassOf<UActorComponent> InComponentClass, bool InIncludedInFilter, EComponentCreateAction::Type InComponentCreateAction, FComponentEntryCustomizationArgs InCustomizationArgs = FComponentEntryCustomizationArgs() )
		: ComponentClass(InComponentClass)
		, IconClass(InComponentClass)
		, ComponentName()
		, ComponentPath()
		, HeadingText(InHeadingText)
		, bIncludedInFilter(InIncludedInFilter)
		, ComponentCreateAction(InComponentCreateAction)
		, CustomizationArgs(InCustomizationArgs)
	{}

	FComponentClassComboEntry(const FString& InHeadingText, const FString& InComponentName, FTopLevelAssetPath InComponentPath, const UClass* InIconClass, bool InIncludedInFilter)
		: ComponentClass(nullptr)
		, IconClass(InIconClass)
		, ComponentName(InComponentName)
		, ComponentPath(InComponentPath)
		, HeadingText(InHeadingText)
		, bIncludedInFilter(InIncludedInFilter)
		, ComponentCreateAction(EComponentCreateAction::SpawnExistingClass)
	{}

	FComponentClassComboEntry( const FString& InHeadingText )
		: ComponentClass(nullptr)
		, IconClass(nullptr)
		, ComponentName()
		, ComponentPath()
		, HeadingText(InHeadingText)
		, bIncludedInFilter(false)
	{}

	FComponentClassComboEntry()
		: ComponentClass(nullptr)
		, IconClass(nullptr)
	{}


	TSubclassOf<UActorComponent> GetComponentClass() const { return ComponentClass; }

	const UClass* GetIconClass() const { return IconClass; }

	const FString& GetHeadingText() const { return HeadingText; }

	bool IsHeading() const
	{
		return ((ComponentClass == NULL && ComponentName == FString()) && !HeadingText.IsEmpty());
	}
	bool IsSeparator() const
	{
		return ((ComponentClass == NULL && ComponentName == FString()) && HeadingText.IsEmpty());
	}
	
	bool IsClass() const
	{
		return (ComponentClass != NULL || ComponentName != FString());
	}
	
	bool IsIncludedInFilter() const
	{
		return bIncludedInFilter;
	}
	
	const FString& GetComponentNameOverride() const
	{
		return CustomizationArgs.ComponentNameOverride;
	}

	EComponentCreateAction::Type GetComponentCreateAction() const
	{
		return ComponentCreateAction;
	}
	
	FOnSubobjectCreated& GetOnSubobjectCreated()
    {
    	return CustomizationArgs.OnSubobjectCreated;
    }
	
	FString GetClassDisplayName() const;
	FString GetClassName() const;
	FString GetComponentPath() const { return ComponentPath.ToString(); }

	UObject* GetAssetOverride()
	{
		return CustomizationArgs.AssetOverride.Get();
	}

	FName GetIconOverrideBrushName() const { return CustomizationArgs.IconOverrideBrushName; }

	int32 GetSortPriority() const { return CustomizationArgs.SortPriority; }

	void AddReferencedObjects(FReferenceCollector& Collector);
	bool OnBlueprintGeneratedClassUnloaded(UBlueprintGeneratedClass* BlueprintGeneratedClass);

	// If the component type is not loaded, this stores data that can be used for class filtering.
	TSharedPtr<IUnloadedBlueprintData> GetUnloadedBlueprintData() const { return UnloadedBlueprintData; }

	// Can optionally be called to set unloaded component type data to assist with class filtering.
	void SetUnloadedBlueprintData(TSharedPtr<IUnloadedBlueprintData> InUnloadedBlueprintData)
	{
		UnloadedBlueprintData = InUnloadedBlueprintData;
	}

private:
	TSubclassOf<UActorComponent> ComponentClass;
	TObjectPtr<const UClass> IconClass;
	// For components that are not loaded we just keep the name of the component,
	// loading occurs when the blueprint is spawned, which should also trigger a refresh
	// of the component list:
	FString ComponentName;
	FTopLevelAssetPath ComponentPath;
	FString HeadingText;
	bool bIncludedInFilter;
	EComponentCreateAction::Type ComponentCreateAction;
	FComponentEntryCustomizationArgs CustomizationArgs;
	TSharedPtr<IUnloadedBlueprintData> UnloadedBlueprintData;
};

//////////////////////////////////////////////////////////////////////////

class SComponentClassCombo : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SComponentClassCombo )
		: _IncludeText(true)
	{}

		SLATE_ATTRIBUTE(bool, IncludeText)
		SLATE_EVENT(FComponentClassSelected, OnComponentClassSelected)
		SLATE_EVENT(FSubobjectClassSelected, OnSubobjectClassSelected)
		SLATE_ARGUMENT(TArray<TSharedRef<IClassViewerFilter>>, CustomClassFilters)

	SLATE_END_ARGS()

	UNREALED_API void Construct(const FArguments& InArgs);

	UNREALED_API ~SComponentClassCombo();

	/** Clear the current combo list selection */
	UNREALED_API void ClearSelection();

	UNREALED_API FText GetCurrentSearchString() const;

	/**
	 * Called when the user changes the text in the search box.
	 * @param InSearchText The new search string.
	 */
	UNREALED_API void OnSearchBoxTextChanged( const FText& InSearchText );

	/** Callback when the user commits the text in the searchbox */
	UNREALED_API void OnSearchBoxTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo);

	UNREALED_API void OnAddComponentSelectionChanged( FComponentClassComboEntryPtr InItem, ESelectInfo::Type SelectInfo );

	UNREALED_API TSharedRef<ITableRow> GenerateAddComponentRow( FComponentClassComboEntryPtr Entry, const TSharedRef<STableViewBase> &OwnerTable ) const;

	/** Update list of component classes */
	UNREALED_API void UpdateComponentClassList();

	/** Returns a component name without the substring "Component" and sanitized for display */
	static UNREALED_API FString GetSanitizedComponentName(FComponentClassComboEntryPtr Entry);

protected:
	/** Internal data used to facilitate component class filtering */
	struct FComponentClassFilterData
	{
		TSharedPtr<FClassViewerInitializationOptions> InitOptions;
		TSharedPtr<IClassViewerFilter> ClassFilter;
		TSharedPtr<FClassViewerFilterFuncs> FilterFuncs;
	};

	/**
	 * Updates the filtered list of component names.
	 */
	UNREALED_API void GenerateFilteredComponentList();

private:
	
	FText GetFriendlyComponentName(FComponentClassComboEntryPtr Entry) const;

	TSharedRef<SToolTip> GetComponentToolTip(FComponentClassComboEntryPtr Entry) const;

	bool IsComponentClassAllowed(FComponentClassComboEntryPtr Entry) const;

	void GetComponentClassFilterOptions(TArray<TSharedRef<FClassViewerFilterOption>>& OutFilterOptions) const;

	TSharedRef<SWidget> GetFilterOptionsMenuContent();

	void ToggleFilterOption(TSharedRef<FClassViewerFilterOption> FilterOption);

	bool IsFilterOptionEnabled(TSharedRef<FClassViewerFilterOption> FilterOption) const;

	EVisibility GetFilterOptionsButtonVisibility() const;
	
	FComponentClassSelected OnComponentClassSelected;

	FSubobjectClassSelected OnSubobjectClassSelected;
	
	/** List of component class names used by combo box */
	TArray<FComponentClassComboEntryPtr>* ComponentClassList;

	/** List of component class names, filtered by the current search string */
	TArray<FComponentClassComboEntryPtr> FilteredComponentClassList;

	/** The current search string */
	TSharedPtr<FTextFilterExpressionEvaluator> TextFilter;

	/** The search box control - part of the combo drop down */
	TSharedPtr<SSearchBox> SearchBox;

	/** The Add combo button. */
	TSharedPtr<class SPositiveActionButton> AddNewButton;

	/** The component list control - part of the combo drop down */
	TSharedPtr< SListView<FComponentClassComboEntryPtr> > ComponentClassListView;

	/** Internal data that facilitates custom class filtering */
	FComponentClassFilterData ComponentClassFilterData;

	/** Cached selection index used to skip over unselectable items */
	int32 PrevSelectedIndex;
};
