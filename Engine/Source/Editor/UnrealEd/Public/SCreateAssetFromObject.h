// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"

class SEditableTextBox;
class SWindow;

//////////////////////////////////////////////////////////////////////////
// SCreateAssetFromActor

DECLARE_DELEGATE_OneParam(FOnPathChosen, const FString&);

class SCreateAssetFromObject : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCreateAssetFromObject)
		: _AssetFilenameSuffix()
		, _HeadingText()
		, _CreateButtonText()
	{}

	/** The default suffix to use for the asset filename */
	SLATE_ARGUMENT(FString, AssetFilenameSuffix)

	/** The text to display at the top of the dialog */
	SLATE_ARGUMENT(FText, HeadingText)

	/** The label for the create button */
	SLATE_ARGUMENT(FText, CreateButtonText)

	/** The path we begin naviagation at */
	SLATE_ARGUMENT(FString, AssetPath)

	SLATE_ARGUMENT(FText, DefaultNameOverride)

	/** Action to perform when create clicked */
	SLATE_EVENT(FOnPathChosen, OnCreateAssetAction)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	UNREALED_API void Construct(const FArguments& InArgs, TSharedPtr<SWindow> InParentWindow);

private:

	/** Callback when the "create asset" button is clicked. */
	FReply OnCreateAssetFromActorClicked();

	/** Callback when the selected asset path has changed. */
	void OnSelectAssetPath(const FString& Path);

	/** Destroys the window when the operation is cancelled. */
	FReply OnCancelCreateAssetFromActor();

	/** Callback when level selection has changed, will destroy the window. */
	void OnLevelSelectionChanged(UObject* InObjectSelected);

	/** Callback when the user changes the filename for the Blueprint */
	void OnFilenameChanged(const FText& InNewName);

	/** Callback to see if creating an asset is enabled */
	bool IsCreateAssetFromActorEnabled() const;

	/** Rquest to destroy the parent window */
	void RequestDestroyParentWindow();

private:
	/** The window this widget is nested in */
	TWeakPtr<SWindow> ParentWindow;

	/** The selected path to create the asset */
	FString AssetPath;

	/** The resultant actor instance label, based on the original actor labels */
	FString ActorInstanceLabel;

	/** The default suffix to use for the asset filename */
	FString AssetFilenameSuffix;

	/** The text to display as a heading for the dialog */
	FText HeadingText;

	/** The label to be displayed on the create button */
	FText CreateButtonText;

	/** Filename textbox widget */
	TSharedPtr<SEditableTextBox> FileNameWidget;

	/** True if an error is currently being reported */
	bool bIsReportingError;

	/** Called when the create button is clicked */
	FOnPathChosen OnCreateAssetAction;

	/** Used to keep track of the delegate in the selection event */
	FDelegateHandle SelectionDelegateHandle;
};
