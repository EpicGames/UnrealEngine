// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Templates/SubclassOf.h"
#include "Templates/SharedPointer.h"
#include "Layout/Geometry.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Curves/KeyHandle.h"
#include "Widgets/SWidget.h"
#include "SColorGradientEditor.h"
#include "UObject/GCObject.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/SlateDelegates.h"
#include "Curves/CurveBase.h"
#include "EditorUndoClient.h"

class FPaintArgs;
class FSlateWindowElementList;
class FUICommandList;
class IPropertyHandle;
class IPropertyUtilities;
class IMenu;
class SBox;
class SErrorText;
class SToolTip;
class UCurveEditorSettings;
class UCurveFactory;
enum class ECheckBoxState : uint8;

UNREALED_API DECLARE_LOG_CATEGORY_EXTERN(LogCurveEditor, Log, All);

//////////////////////////////////////////////////////////////////////////
// FTrackScaleInfo

/** Utility struct for converting between curve space and local/absolute screen space.  The input domain is traditionally
 *  the time axis of the curve, and the output domain is traditionally the value axis.
 */
struct FTrackScaleInfo
{
	float ViewMinInput;
	float ViewMaxInput;
	float ViewInputRange;
	float PixelsPerInput;

	float ViewMinOutput;
	float ViewMaxOutput;
	float ViewOutputRange;
	float PixelsPerOutput;

	FVector2f WidgetSize;

	FTrackScaleInfo(float InViewMinInput, float InViewMaxInput, float InViewMinOutput, float InViewMaxOutput, const FVector2f& InWidgetSize)
	{
		WidgetSize = InWidgetSize;

		ViewMinInput = InViewMinInput;
		ViewMaxInput = InViewMaxInput;
		ViewInputRange = ViewMaxInput - ViewMinInput;
		PixelsPerInput = ViewInputRange > 0.f ? ( WidgetSize.X / ViewInputRange ) : 0.f;

		ViewMinOutput = InViewMinOutput;
		ViewMaxOutput = InViewMaxOutput;
		ViewOutputRange = InViewMaxOutput - InViewMinOutput;
		PixelsPerOutput = ViewOutputRange > 0.f ? ( WidgetSize.Y / ViewOutputRange ) : 0.f;
	}

	UE_DEPRECATED(5.6, "Use the FTrackScaleInfo constructor accepting FVector2f instead of FVector2D")
	FTrackScaleInfo(float InViewMinInput, float InViewMaxInput, float InViewMinOutput, float InViewMaxOutput, const FVector2D InWidgetSize)
		: FTrackScaleInfo(InViewMinInput
		, InViewMaxInput
		, InViewMinOutput
		, InViewMaxOutput
		, UE::Slate::CastToVector2f(InWidgetSize))
	{
	}

	/** Local Widget Space -> Curve Input domain. */
	float LocalXToInput(float ScreenX) const
	{
		return PixelsPerInput != 0.0f ? (ScreenX/PixelsPerInput) + ViewMinInput : 0.0f;
	}

	/** Curve Input domain -> local Widget Space */
	float InputToLocalX(float Input) const
	{
		return (Input - ViewMinInput) * PixelsPerInput;
	}

	/** Local Widget Space -> Curve Output domain. */
	float LocalYToOutput(float ScreenY) const
	{
		return PixelsPerOutput != 0.0f ? (ViewOutputRange + ViewMinOutput) - (ScreenY/PixelsPerOutput) : 0.0f;
	}

	/** Curve Output domain -> local Widget Space */
	float OutputToLocalY(float Output) const
	{
		return (ViewOutputRange - (Output - ViewMinOutput)) * PixelsPerOutput;
	}

	float GetTrackCenterY() const
	{
		return (0.5f * WidgetSize.Y);
	}


};

/** Represents UI state for a curve displayed in the curve editor. */
class FCurveViewModel
{
public:
	/** The curve info for the curve being edited. */
	FRichCurveEditInfo CurveInfo;
	/** The color which should be used to draw the curve and it's label in the UI. */
	FLinearColor Color;
	/** Whether or not the curve should be displayed in the UI. */
	bool bIsVisible;
	/** Whether or not the curve is locked from editing. */
	bool bIsLocked;
	/** Whether or not the curve is selected. */
	bool bIsSelected;

	FCurveViewModel(FRichCurveEditInfo InCurveInfo, FLinearColor InColor, bool bInIsLocked)
	{
		CurveInfo = InCurveInfo;
		Color = InColor;
		bIsVisible = true;
		bIsLocked = bInIsLocked;
		bIsSelected = true;
	}
};

//////////////////////////////////////////////////////////////////////////
// SCurveEditor

DECLARE_DELEGATE_TwoParams( FOnSetInputViewRange, float, float )
DECLARE_DELEGATE_TwoParams( FOnSetOutputViewRange, float, float )
DECLARE_DELEGATE_OneParam( FOnSetAreCurvesVisible, bool )

class SCurveEditor : 
	public SCompoundWidget,
	public FGCObject,
	public FEditorUndoClient
{
private:

	/** Represents the different states of a drag operation */
	enum class EDragState
	{
		/** The user has clicked a mouse button, but hasn't moved more then the drag threshold. */
		PreDrag,
		/** The user is dragging the selected keys. */
		DragKey,
		/** The user is free dragging the selected keys. */
		FreeDrag,
		/** The user is dragging a selected tangent handle. */
		DragTangent,
		/** The user is performing a marquee selection of keys. */
		MarqueeSelect,
		/** The user is panning the curve view. */
		Pan,
		/** The user is zooming the curve view. */
		Zoom,
		/** There is no active drag operation. */
		None
	};

	enum class EMovementAxisLock
	{
		/** Lock movement to horizontal axis */
		AxisLock_Horizontal,
		/** Lock movement to vertical axis */
		AxisLock_Vertical,
		/** Don't lock movement */
		None
	};

public:
	SLATE_BEGIN_ARGS( SCurveEditor )
		: _ViewMinInput(0.0f)
		, _ViewMaxInput(10.0f)
		, _ViewMinOutput(0.0f)
		, _ViewMaxOutput(1.0f)
		, _InputSnap(0.1f)
		, _OutputSnap(0.05f)
		, _InputSnappingEnabled(false)
		, _OutputSnappingEnabled(false)
		, _ShowTimeInFrames(false)
		, _TimelineLength(5.0f)
		, _DesiredSize(FVector2D::ZeroVector)
		, _DrawCurve(true)
		, _HideUI(true)
		, _AllowZoomOutput(true)
		, _AlwaysDisplayColorCurves(false)
		, _AlwaysHideGradientEditor(false)
		, _ZoomToFitVertical(true)
		, _ZoomToFitHorizontal(true)
		, _ShowZoomButtons(true)
		, _XAxisName()
		, _YAxisName()
		, _ShowInputGridNumbers(true)
		, _ShowOutputGridNumbers(true)
		, _ShowCurveSelector(true)
		, _GridColor(FLinearColor(0.0f, 0.0f, 0.0f, 0.3f))
		{
			_Clipping = EWidgetClipping::ClipToBounds;
		}

		SLATE_ATTRIBUTE( float, ViewMinInput )
		SLATE_ATTRIBUTE( float, ViewMaxInput )
		SLATE_ATTRIBUTE( TOptional<float>, DataMinInput )
		SLATE_ATTRIBUTE( TOptional<float>, DataMaxInput )
		SLATE_ATTRIBUTE( float, ViewMinOutput )
		SLATE_ATTRIBUTE( float, ViewMaxOutput )
		SLATE_ATTRIBUTE( float, InputSnap )
		SLATE_ATTRIBUTE( float, OutputSnap )
		SLATE_ATTRIBUTE( bool, InputSnappingEnabled )
		SLATE_ATTRIBUTE( bool, OutputSnappingEnabled )
		SLATE_ATTRIBUTE( bool, ShowTimeInFrames )
		SLATE_ATTRIBUTE( float, TimelineLength )
		SLATE_ATTRIBUTE( FVector2D, DesiredSize )
		SLATE_ATTRIBUTE( bool, AreCurvesVisible )
		SLATE_ARGUMENT( bool, DrawCurve )
		SLATE_ARGUMENT( bool, HideUI )
		SLATE_ARGUMENT( bool, AllowZoomOutput )
		SLATE_ARGUMENT( bool, AlwaysDisplayColorCurves )
		SLATE_ARGUMENT( bool, AlwaysHideGradientEditor )
		SLATE_ARGUMENT( bool, ZoomToFitVertical )
		SLATE_ARGUMENT( bool, ZoomToFitHorizontal )
		SLATE_ARGUMENT( bool, ShowZoomButtons )
		SLATE_ARGUMENT( TOptional<FString>, XAxisName )
		SLATE_ARGUMENT( TOptional<FString>, YAxisName )
		SLATE_ARGUMENT( bool, ShowInputGridNumbers )
		SLATE_ARGUMENT( bool, ShowOutputGridNumbers )
		SLATE_ARGUMENT( bool, ShowCurveSelector )
		SLATE_ARGUMENT( FLinearColor, GridColor )
		SLATE_EVENT( FOnSetInputViewRange, OnSetInputViewRange )
		SLATE_EVENT( FOnSetOutputViewRange, OnSetOutputViewRange )
		SLATE_EVENT( FOnSetAreCurvesVisible, OnSetAreCurvesVisible )
		SLATE_EVENT( FSimpleDelegate, OnCreateAsset )
	SLATE_END_ARGS()

	UNREALED_API void Construct(const FArguments& InArgs);

	/** Destructor */
	UNREALED_API ~SCurveEditor();

	/** Set the curve that is being edited by this track widget. Also provide an option to enable/disable editing */
	UNREALED_API void SetCurveOwner(FCurveOwnerInterface* InCurveOwner, bool bCanEdit = true);
	
	void SetPropertyUtils(const TSharedPtr<IPropertyUtilities>& InPropertyUtils)
	{
		PropertyUtilitiesWeak = InPropertyUtils;
	}

	UNREALED_API void RegisterToPropertyChangedEvent(const TSharedPtr<IPropertyHandle>& InRootPropertyHandle);

	/** Set new zoom to fit **/
	UNREALED_API void SetZoomToFit(bool bNewZoomToFitVertical, bool bNewZoomToFitHorizontal);

	/** Get the currently edited curve */
	UNREALED_API FCurveOwnerInterface* GetCurveOwner() const;

	/** Construct an object of type UCurveFactory and return it's reference*/
	UNREALED_API UCurveFactory* GetCurveFactory( );

	/**
	* Function to create a curve object and return its reference.
	 * 
	 * @param 1: Type of curve to create
	 * @param 2: Reference to the package in which the asset has to be created
	 * @param 3: Name of asset
	 *
	 * @return: Reference to the newly created curve object
	 */
	UNREALED_API UObject* CreateCurveObject( TSubclassOf<UCurveBase> CurveType, UObject* PackagePtr, FName& AssetName );

	/**
	 * Since we create a UFactory object, it needs to be serialized
	 *
	 * @param Ar The archive to serialize with
	 */
	UNREALED_API virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SCurveEditor");
	}

	/** Gets a list of the commands handled by this control */
	UNREALED_API TSharedPtr<FUICommandList> GetCommands();

	/** Gets or sets whether autoframing is allowed */
	bool GetAllowAutoFrame() const { return bAllowAutoFrame; }
	void SetAllowAutoFrame(bool bInAllowAutoFrame) { bAllowAutoFrame = bInAllowAutoFrame; }

	/** Gets whether autoframe will be invoked (combination of allow auto frame and curve editor auto frame setting) */
	UNREALED_API bool GetAutoFrame() const;

	/* Get the curves to will be used during a fit operation */
	UNREALED_API TArray<FRealCurve*> GetCurvesToFit() const;

	/** Zoom to fit */
	UNREALED_API void ZoomToFitHorizontal(const bool bZoomToFitAll = false);
	UNREALED_API void ZoomToFitVertical(const bool bZoomToFitAll = false);
	UNREALED_API void ZoomToFit(const bool bZoomToFitAll = false);

	/* Set flag that allows scrolling up/down over the widget from the outside without it handling the scroll wheel event */
	UNREALED_API void SetRequireFocusToZoom(bool bInRequireFocusToZoom);

	UNREALED_API virtual TOptional<bool> OnQueryShowFocus(const EFocusCause InFocusCause) const override;

private:
	/** Used to track a key and the curve that owns it */
	struct FSelectedCurveKey
	{
		FSelectedCurveKey(FRealCurve* InCurve, FKeyHandle InKeyHandle)
			: Curve(InCurve), KeyHandle(InKeyHandle)
		{}

		/** If this is a valid Curve/Key */
		bool			IsValid() const
		{
			return Curve != NULL && Curve->IsKeyHandleValid(KeyHandle);
		}
		/** Does the curve match? */ 
		bool			IsSameCurve(const FSelectedCurveKey& Key) const 
		{ 
			return Curve == Key.Curve;
		}

		/** Does the curve and the key match ?*/
		bool			operator == (const FSelectedCurveKey& Other) const 
		{
			return (Curve == Other.Curve) && (KeyHandle == Other.KeyHandle);
		}

		/** Are the curve and the key different ?*/
		bool			operator != (const FSelectedCurveKey& Other) const
		{
			return (Curve != Other.Curve) && (KeyHandle != Other.KeyHandle);
		}

		FRealCurve*		Curve;
		FKeyHandle		KeyHandle;
	};

	/** Used to track a key and tangent*/
	struct FSelectedTangent
	{
		FSelectedTangent(): Key(NULL,FKeyHandle::Invalid()),bIsArrival(false)
		{
		}

		FSelectedTangent(FSelectedCurveKey InKey) : Key(InKey)
		{
		}

		/** Does the key and the arrival match ?*/
		bool			operator == (const FSelectedTangent& Other) const 
		{
			return (Key == Other.Key) && (bIsArrival == Other.bIsArrival);
		}

		/** If this is a valid Curve/Key */
		bool			IsValid() const;

		/** The Key for the tangent */
		FSelectedCurveKey Key;

		/** Indicates if it is the arrival tangent, or the leave tangent */
		bool			  bIsArrival;
	};

private:
	/** Adds a new key to the curve. */
	void AddNewKey(FGeometry InMyGeometry, FVector2D ScreenPosition, TSharedPtr<TArray<TSharedPtr<FCurveViewModel>>> CurvesToAddKeysTo, bool bAddKeysInline);

	/** Test if the curve is exists, and if it being displayed on this widget */
	bool		IsValidCurve(FRealCurve* Curve) const;

	/** Util to get a curve by index */
	FRealCurve* GetCurve(int32 CurveIndex) const;

	/** Called when new value for a key is entered */
	void NewValueEntered( const FText& NewText, ETextCommit::Type CommitInfo );

	void NewHorizontalGridScaleEntered(const FString& NewText, bool bCommitFromEnter );
	void NewVerticalGridScaleEntered(const FString& NewText, bool bCommitFromEnter );

	/** Called by delete command */
	void DeleteSelectedKeys();

	/** Test a screen space location to find which key was clicked on */
	FSelectedCurveKey HitTestKeys(const FGeometry& InMyGeometry, const FVector2D& HitScreenPosition);

	/** Test a screen space location to find if any cubic tangents were clicked on */
	FSelectedTangent HitTestCubicTangents(const FGeometry& InMyGeometry, const FVector2D& HitScreenPosition);

	/** Get screen space tangent positions for a given key */
	void GetTangentPoints( FTrackScaleInfo &ScaleInfo,const FSelectedCurveKey &Key, FVector2D& Arrive, FVector2D& Leave) const;

	/** Get the set of keys within a rectangle in local space */
	TArray<FSelectedCurveKey> GetEditableKeysWithinMarquee(const FGeometry& InMyGeometry, FVector2D MarqueeTopLeft, FVector2D MarqueeBottomRight) const;

	/** Get the set of tangents within a rectangle in local space */
	TArray<FSelectedTangent> GetEditableTangentsWithinMarquee(const FGeometry& InMyGeometry, FVector2D MarqueeTopLeft, FVector2D MarqueeBottomRight) const;

	/** Empty key selection set */
	void EmptyKeySelection();
	/** Add a key to the selection set */
	void AddToKeySelection(FSelectedCurveKey Key);
	/** Remove a key from the selection set */
	void RemoveFromKeySelection(FSelectedCurveKey Key);
	/** See if a key is currently selected */
	bool IsKeySelected(FSelectedCurveKey Key) const;
	/** See if any keys are selected */
	bool AreKeysSelected() const;

	/** Empty tangent selection set */
	void EmptyTangentSelection();
	/** Add a tangent to the selection set */
	void AddToTangentSelection(FSelectedTangent Tangent);
	/** Remove a tangent from the selection set */
	void RemoveFromTangentSelection(FSelectedTangent Tangent);
	/** See if a tangent is currently selected */
	bool IsTangentSelected(FSelectedTangent Tangent) const;
	/** See if any tangents are selected */
	bool AreTangentsSelected() const;

	/** Is the tangent visible? */
	bool IsTangentVisible(FRichCurve*, FKeyHandle, bool& IsTangentSelected, bool& IsArrivalSelected, bool& IsLeaveSelected) const;

	/** Empty key and tangent selection set */
	void EmptyAllSelection();

	/** Get the value of the desired key as text */
	TOptional<float> GetKeyValue(FSelectedCurveKey Key) const;
	/** Get the time of the desired key */
	TOptional<float> GetKeyTime(FSelectedCurveKey Key) const;

	/** Move the selected keys */
	void MoveSelectedKeys(FVector2D Delta);

	/** Function to check whether the current track is editable */
	bool IsEditingEnabled() const;

	FReply ZoomToFitHorizontalClicked();
	FReply ZoomToFitVerticalClicked();

	void ToggleInputSnapping();
	void ToggleOutputSnapping();
	bool IsInputSnappingEnabled() const;
	bool IsOutputSnappingEnabled() const;
	bool ShowTimeInFrames() const;

	TOptional<float> OnGetTime() const;
	void OnTimeComitted(float NewValue, ETextCommit::Type CommitType);
	void OnTimeChanged(float NewValue);

	TOptional<int32> OnGetTimeInFrames() const;
	void OnTimeInFramesComitted(int32 NewValue, ETextCommit::Type CommitType);
	void OnTimeInFramesChanged(int32 NewValue);

	void UpdateCurveTimeSingleKey(FSelectedCurveKey Key, float NewTime, bool bSetFromFrame = false);
	void UpdateCurveTimeSingleKey(FSelectedCurveKey Key, int32 NewFrame);
	void LogAndToastCurveTimeWarning(FRealCurve* Curve);

	TOptional<float> OnGetValue() const;
	void OnValueComitted(float NewValue, ETextCommit::Type CommitType);
	void OnValueChanged(float NewValue);

	void OnBeginSliderMovement(FText TransactionName);
	void OnEndSliderMovement(float NewValue);
	void OnEndSliderMovement(int32 NewValue);

	EVisibility GetCurveAreaVisibility() const;
	EVisibility GetCurveSelectorVisibility() const;
	EVisibility GetEditVisibility() const;
	EVisibility GetColorGradientVisibility() const;
	EVisibility GetZoomButtonVisibility() const;
	EVisibility GetTimeEditVisibility() const;
	EVisibility GetFrameEditVisibility() const;

	bool GetInputEditEnabled() const;

	/** Function to create context menu on mouse right click*/
	void CreateContextMenu(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent);

	/** Callback function called when item is select in the context menu */
	void OnCreateExternalCurveClicked();

	/** Called when "Show Curves" is selected from the context menu */
	void OnShowCurveToggled();

	/** Called when "Show Gradient" is selected from the context menu */
	void OnShowGradientToggled() { bIsGradientEditorVisible = !bIsGradientEditorVisible; }

	/** Function pointer to execute callback function when user select 'Create external curve'*/
	FSimpleDelegate OnCreateAsset;

	//~ Begin SWidget Interface
	UNREALED_API virtual FVector2D ComputeDesiredSize(float) const override;

	/** Paint a curve */
	void PaintCurve(TSharedPtr<FCurveViewModel> CurveViewModel, const FGeometry &AllottedGeometry, FTrackScaleInfo &ScaleInfo, FSlateWindowElementList &OutDrawElements, 
					int32 LayerId, const FSlateRect& MyCullingRect, ESlateDrawEffect DrawEffects, const FWidgetStyle &InWidgetStyle, bool bAnyCurveViewModelsSelected) const;

	/** Paint the keys that make up a curve */
	void PaintKeys(TSharedPtr<FCurveViewModel> CurveViewModel, FTrackScaleInfo &ScaleInfo, FSlateWindowElementList &OutDrawElements, int32 LayerId, int32 SelectedLayerId, const FGeometry &AllottedGeometry, const FSlateRect& MyCullingRect, ESlateDrawEffect DrawEffects, const FWidgetStyle &InWidgetStyle, bool bAnyCurveViewModelsSelected ) const;

	/** Paint the tangent for a key with cubic curves */
	void PaintTangent( TSharedPtr<FCurveViewModel> CurveViewModel, FTrackScaleInfo &ScaleInfo, FRichCurve* Curve, FKeyHandle KeyHandle, FVector2D KeyLocation, FSlateWindowElementList &OutDrawElements, int32 LayerId, 
					   const FGeometry &AllottedGeometry, const FSlateRect& MyCullingRect, ESlateDrawEffect DrawEffects, int32 LayerToUse, const FWidgetStyle &InWidgetStyle, bool bTangentSelected, bool bIsArrivalSelected, bool bIsLeaveSelected, bool bAnyCurveViewModelsSelected ) const;

	/** Paint Grid lines, these make it easier to visualize relative distance */
	void PaintGridLines( const FGeometry &AllottedGeometry, FTrackScaleInfo &ScaleInfo, FSlateWindowElementList &OutDrawElements, int32 LayerId, const FSlateRect& MyCullingRect, ESlateDrawEffect DrawEffects )const;

	/** Paints the marquee for selection */
	void PaintMarquee(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;

	/** Gets the delta value for the input value numeric entry box. */
	float GetInputNumericEntryBoxDelta() const;

	/** Gets the delta value for the output value numeric entry box. */
	float GetOutputNumericEntryBoxDelta() const;

protected:
	//~ Begin SWidget Interface
	UNREALED_API virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UNREALED_API virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UNREALED_API virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UNREALED_API virtual FReply OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UNREALED_API virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	UNREALED_API virtual void   OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override;

private:
	/** Attempts to start a drag operation when the mouse moves. */
	void TryStartDrag(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/** Processes an ongoing drag operation when the mouse moves. */
	void ProcessDrag(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/** Completes an ongoing drag operation on mouse up. */
	void EndDrag(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/** Handles a mouse click operation on mouse up. */
	void ProcessClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	virtual bool SupportsKeyboardFocus() const override
	{
		return true;
	}

	/** Zoom the view. */
	void ZoomView(FVector2D Delta);

	/* Generates the line(s) for rendering between KeyIndex and the following key. */
	void CreateLinesForSegment( FRealCurve* Curve, ERichCurveInterpMode InterpMode, const TPair<float,float>& Key1_TimeValue, const TPair<float,float>& Key2_TimeValue, TArray<FVector2D>& Points, FTrackScaleInfo &ScaleInfo) const;
	
	/** Detect if user is clicking on a curve */
	TSharedPtr<FCurveViewModel> HitTestCurves(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent);

	/* user is moving the tangents */
	void MoveTangents(FTrackScaleInfo& ScaleInfo, FVector2D Delta);

	//Curve Selection interface 

	/** Construct widget that allows user to select which curve to edit if tthere are multiple */
	TSharedRef<SWidget> CreateCurveSelectionWidget() const;

	/** Create context Menu for waring menu*/
	void	PushWarningMenu(FVector2D Position, const FText& Message);

	/** Create context Menu for key interpolation settings*/
	void	PushKeyMenu(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent);

	/** Called when the user selects the interpolation mode */
	void	OnSelectInterpolationMode(ERichCurveInterpMode InterpMode, ERichCurveTangentMode TangentMode);

	bool IsInterpolationModeSelected(ERichCurveInterpMode InterpMode, ERichCurveTangentMode TangentMode);

	bool HasRichCurves() const;

	/** Flatten or straighten tangents */
	void OnFlattenOrStraightenTangents(bool bFlattenTangents);

	/** Called when user selects bake or reduce curve */
	void OnBakeCurve();
	void OnBakeCurveSampleRateCommitted(const FText& InText, ETextCommit::Type CommitInfo);
	void OnReduceCurve();
	void OnReduceCurveToleranceCommitted(const FText& InText, ETextCommit::Type CommitInfo);

	/** Called when the user selects the extrapolation type */
	void OnSelectPreInfinityExtrap(ERichCurveExtrapolation Extrapolation);
	bool IsPreInfinityExtrapSelected(ERichCurveExtrapolation Extrapolation);
	void OnSelectPostInfinityExtrap(ERichCurveExtrapolation Extrapolation);
	bool IsPostInfinityExtrapSelected(ERichCurveExtrapolation Extrapolation);

	/** Begin a transaction for dragging a key or tangent */
	void	BeginDragTransaction();

	/** End a transaction for dragging a key or tangent */
	void	EndDragTransaction();

	/** Calculate the distance between grid lines: determines next lowest power of 2, works with fractional numbers */
	static float CalcGridLineStepDistancePow2(double RawValue);

	/** Perform undo */
	void	UndoAction();
	/** Perform redo*/
	void	RedoAction();

	/** Refresh the Details View and respawn this widget when Curve Data goes stale */
	void RefreshDetailsView();

	//~ Begin FEditorUndoClient Interface
	UNREALED_API virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
	// End of FEditorUndoClient

	bool AreCurvesVisible() const { return bAlwaysDisplayColorCurves || bAreCurvesVisible.Get(); }
	bool IsGradientEditorVisible() const { return !bAlwaysHideGradientEditor && bIsGradientEditorVisible; }
	bool IsLinearColorCurve() const;

	bool IsCurveSelectable(TSharedPtr<FCurveViewModel> CurveViewModel) const;

	FVector2D SnapLocation(FVector2D InLocation);

	FText GetIsCurveVisibleToolTip(TSharedPtr<FCurveViewModel> CurveViewModel) const;
	ECheckBoxState IsCurveVisible(TSharedPtr<FCurveViewModel> CurveViewModel) const;
	void OnCurveIsVisibleChanged(ECheckBoxState NewCheckboxState, TSharedPtr<FCurveViewModel> CurveViewModel);

	FText GetIsCurveLockedToolTip(TSharedPtr<FCurveViewModel> CurveViewModel) const;
	ECheckBoxState IsCurveLocked(TSharedPtr<FCurveViewModel> CurveViewModel) const;
	void OnCurveIsLockedChanged(ECheckBoxState NewCheckboxState, TSharedPtr<FCurveViewModel> CurveViewModel);

	void RemoveCurveKeysFromSelection(TSharedPtr<FCurveViewModel> CurveViewModel);

	FText GetCurveToolTipNameText() const;
	FText GetCurveToolTipInputText() const;
	FText GetCurveToolTipOutputText() const;

	FText GetInputAxisName() const;

	void UpdateCurveToolTip( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent );

	TSharedPtr<FCurveViewModel> GetViewModelForCurve(FRealCurve* InCurve);

	void OnObjectPropertyChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent);
	void OnRootPropertyChanged(const FPropertyChangedEvent& PropertyChangedEvent);

	void HandlePackageReloaded(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent);

protected:

	/** Set Default output values when range is too small **/
	UNREALED_API virtual void SetDefaultOutput(const float MinZoomRange);
	/** Get Time Step for vertical line drawing **/
	UNREALED_API virtual float GetTimeStep(FTrackScaleInfo &ScaleInfo) const;
	
	//~ Begin SWidget Interface
	UNREALED_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, 
		FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;

	/** Update view range */
	UNREALED_API void SetInputMinMax(float NewMin, float NewMax);
	UNREALED_API void SetOutputMinMax(float NewMin, float NewMax);

	/** Access the user-supplied settings object */
	UCurveEditorSettings* GetSettings() const { return Settings; }

	/** Empty all the curve view models */
	void EmptyAllCurveViewModels();

	/** Clear the selected curve view models */
	UNREALED_API void ClearSelectedCurveViewModels();

	/** Set the selected curve view model that matches the rich curve */
	UNREALED_API void SetSelectedCurveViewModel(FRealCurve* Curve);

	/** Return whether any curve view models are selected */
	UNREALED_API bool AnyCurveViewModelsSelected() const;

	/** Ensure that selected keys and tangents are still valid */
	UNREALED_API void ValidateSelection();

	/** Modeless Version of the String Entry Box */
	void GenericTextEntryModeless(const FText& DialogText, const FText& DefaultText, FOnTextCommitted OnTextComitted);
	
	/** Closes the popup created by GenericTextEntryModeless*/
	void CloseEntryPopupMenu();

	/** Convert time to frames and vice versa */
	int32 TimeToFrame(float InTime) const;
	float FrameToTime(int32 InFrame) const;

private:

	/** User-supplied object for this curve editor */
	TObjectPtr<UCurveEditorSettings> Settings;

	/** Curve selection*/
	TWeakPtr<SBox>		CurveSelectionWidget;

	/** Text block used to display warnings related to curves */
	TSharedPtr< SErrorText > WarningMessageText;

	/** Interface for curve supplier */
	FCurveOwnerInterface*		CurveOwner;

	/** If we should draw the curve */
	bool				bDrawCurve;
	/**If we should hide the UI when the mouse leaves the control */
	bool				bHideUI;
	/**If we should allow zoom for output */
	bool				bAllowZoomOutput;
	/** If we always show the color curves or allow the user to toggle this */
	bool				bAlwaysDisplayColorCurves;
	/** If we always hide the gradient editor */
	bool				bAlwaysHideGradientEditor;

	/** Whether or not to draw the numbers for the input grid. */
	bool bDrawInputGridNumbers;
	/** Whether or not to draw the numbers for the output grid. */
	bool bDrawOutputGridNumbers;

	/** Array of selected keys*/
	TArray<FSelectedCurveKey> SelectedKeys;

	/** Array of selected tangents */
	TArray<FSelectedTangent> SelectedTangents;

	/** Minimum input of data range  */
	TAttribute< TOptional<float> >	DataMinInput;
	/** Maximum input of data range  */
	TAttribute< TOptional<float> >	DataMaxInput;

	/** Editor Size **/
	TAttribute<FVector2D>	DesiredSize;

	/** Handler for adjust timeline panning viewing */
	FOnSetInputViewRange	SetInputViewRangeHandler;

	/** Handler for adjust timeline panning viewing */
	FOnSetOutputViewRange	SetOutputViewRangeHandler;

	/** Handler for setting whether or not curves are being displayed. */
	FOnSetAreCurvesVisible SetAreCurvesVisibleHandler;

	/** Index for the current transaction if any */
	int32					TransactionIndex;

	// Command stuff

	TSharedPtr< FUICommandList > Commands;

	/**Flag to enable/disable track editing*/
	bool		bCanEditTrack;
	
	/** True if the gradient editor is being displayed */
	bool bIsGradientEditorVisible;

	/** Reference to curve factor instance*/
	TObjectPtr<UCurveFactory> CurveFactory;

	/** Gradient editor */
	TSharedPtr<class SColorGradientEditor> GradientViewer;

	/** Flag to allow auto framing */
	bool bAllowAutoFrame;

	/** Flag to allow scrolling up/down over the widget from the outside without it handling the scroll wheel event */
	bool bRequireFocusToZoom;

protected:
	/** Minimum input of view range  */
	TAttribute<float>	ViewMinInput;
	/** Maximum input of view range  */
	TAttribute<float>	ViewMaxInput;
	/** How long the overall timeline is */
	TAttribute<float>	TimelineLength;

	/** Max output view range */
	TAttribute<float>	ViewMinOutput;
	/** Min output view range */
	TAttribute<float>	ViewMaxOutput;

	/** The snapping value for the input domain. */
	TAttribute<float> InputSnap;

	/** The snapping value for the output domain. */
	TAttribute<float> OutputSnap;

	/** Whether or not input snapping is enabled. */
	TAttribute<bool> bInputSnappingEnabled;

	/** Whether or not output snapping is enabled. */
	TAttribute<bool> bOutputSnappingEnabled;

	/** Show time in frames. */
	TAttribute<bool> bShowTimeInFrames;

	/** Whether or not curves are being displayed */
	TAttribute<bool> bAreCurvesVisible;

	/** True if you want the curve editor to fit to zoom **/
	bool bZoomToFitVertical;

	/** True if you want the curve editor to fit to zoom **/
	bool bZoomToFitHorizontal;

	/** True if the sliders are being used to adjust point values **/
	bool bIsUsingSlider;

	/** True if the internal zoom buttons should be visible. */
	bool bShowZoomButtons;

	/** Whether or not to show the curve selector widgets. */
	bool bShowCurveSelector;

	/** The location of mouse during the last OnMouseButtonDown callback in widget local coordinates. */
	FVector2D MouseDownLocation;

	/** The location of the mouse during the last OnMouseMove callback in widget local coordinates. */
	FVector2D MouseMoveLocation;

	/** The state of the current drag operation happening with the widget, if any. */
	EDragState DragState;

	/** The movement axis lock state */
	EMovementAxisLock MovementAxisLock;

	/** The number of pixels which the mouse must move before a drag operation starts. */
	float DragThreshold;

	/** A handle to the key which was clicked to start a key drag operation. */
	FKeyHandle DraggedKeyHandle = FKeyHandle::Invalid();

	/** A map of selected key handle to their starting locations at the beginning of a drag operation. */
	TMap<FKeyHandle, FVector2D> PreDragKeyLocations;

	/** A map of selected key handles to their tangent values at the beginning of a drag operation. */
	TMap<FKeyHandle, FVector2D> PreDragTangents;

	/** The text to display for the input axis. */
	FText InputAxisName;
	/** The text to display for the input (frame) axis. */
	FText InputFrameAxisName;
	/** The text to display for the output axis. */
	FText OutputAxisName;

	/** The view models for the curves. */
	TArray<TSharedPtr<FCurveViewModel>> CurveViewModels;

	/** The tooltip control for the curves. */
	TSharedPtr<SToolTip> CurveToolTip;

	/** The text for the name portion of the tooltip. */
	FText CurveToolTipNameText;
	/** The text for the input portion of the tooltip. */
	FText CurveToolTipInputText;
	/** The text for the output portion of the tooltip. */
	FText CurveToolTipOutputText;

	/** The color used to draw the grid lines. */
	FLinearColor GridColor;

	/** The tolerance to use when reducing curves */
	float ReduceTolerance;

	/** Generic Popup Entry */
	TWeakPtr<IMenu> EntryPopupMenu;

private:
	bool bIsPendingRebuilt;
	TWeakPtr<IPropertyUtilities> PropertyUtilitiesWeak;
	TWeakPtr<IPropertyHandle> RootPropertyHandleWeak;
};
