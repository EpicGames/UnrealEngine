// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Styling/SlateColor.h"
#include "Input/Reply.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SViewport.h"
#include "UnrealWidgetFwd.h"
#include "EditorViewportClient.h"
#include "Viewports/SDepthBar.h"

class FActiveTimerHandle;
class FSceneViewport;
class FUICommandList;
class IPreviewProfileController;
class SViewport;
class UModeManagerInteractiveToolsContext;
class UViewportInteractionsBehaviorSource;
struct FSlateBrush;
struct FToolMenuContext;
class SDepthBar;
class SEditorViewportGridPanel;

class SEditorViewport
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SEditorViewport)
		: _ViewportSize(SViewport::FArguments::GetDefaultViewportSize())
	{ }
	SLATE_ATTRIBUTE(FVector2D, ViewportSize);
	SLATE_END_ARGS()
	
	UNREALED_API SEditorViewport();
	UNREALED_API virtual ~SEditorViewport();

	UNREALED_API void Construct( const FArguments& InArgs );

	UNREALED_API virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	UNREALED_API virtual bool SupportsKeyboardFocus() const override;
	UNREALED_API virtual FReply OnFocusReceived( const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent ) override;
	UNREALED_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	
	/**
	 * @return True if the viewport is being updated in realtime
	 */
	UNREALED_API bool IsRealtime() const;

	/** @return True if the viewport is currently visible */
	UNREALED_API virtual bool IsVisible() const;

	/** 
	 * Invalidates the viewport to ensure it is redrawn during the next tick. 
	 * This is implied every frame while the viewport IsRealtime().
	 */
	UNREALED_API void Invalidate();

	/** Toggles realtime on/off for the viewport. Slate tick/paint is ensured when realtime is on. */
	UNREALED_API void OnToggleRealtime();

	/**
	 * Sets whether this viewport can render directly to the back buffer.  Advanced use only
	 * 
	 * @param	bInRenderDirectlyToWindow	Whether we should be able to render to the back buffer
	 */
	UNREALED_API void SetRenderDirectlyToWindow( const bool bInRenderDirectlyToWindow );

	/**
	 * Sets whether stereo rendering is allowed for this viewport.  Advanced use only
	 * 
	 * @param	bInEnableStereoRendering	Whether stereo rendering should be allowed for this viewport
	 */
	UNREALED_API void EnableStereoRendering( const bool bInEnableStereoRendering );

	/**
	 * @return true if the specified coordinate system the active one active
	 */
	UNREALED_API virtual bool IsCoordSystemActive( ECoordSystem CoordSystem ) const;

	/**
	 * Cycles between world and local coordinate systems
	 */
	UNREALED_API virtual void OnCycleCoordinateSystem();
	
	/** @return The viewport command list */
	const TSharedPtr<FUICommandList> GetCommandList() const { return CommandList; }

	TSharedPtr<FEditorViewportClient> GetViewportClient() const { return Client; }

	/**
	 * @return The current FSceneViewport shared pointer
	 */
	TSharedPtr<FSceneViewport> GetSceneViewport() { return SceneViewport; }

	/** Adds a widget overlaid over the viewport */
	UNREALED_API virtual void AddOverlayWidget(TSharedRef<SWidget> OverlaidWidget, int32 ZOrder = INDEX_NONE);

	/** Removes a widget that was previously overlaid on to this viewport */
	UNREALED_API virtual void RemoveOverlayWidget(TSharedRef<SWidget> OverlaidWidget);

	/**
	 * Controls the visibility of the widget transform toolbar, if there is an associated toolbar
	 */
	UNREALED_API virtual EVisibility GetTransformToolbarVisibility() const;

	/** Build the exposure menu using EV100 settings */
	UNREALED_API TSharedRef<SWidget> BuildFixedEV100Menu()  const;
	
	/** Build the slider to adjust wireframe opacity */
	UNREALED_API TSharedRef<SWidget> BuildWireframeMenu()  const;

	/**
 * Called when the user wants to show the in-viewport context menu
 */
	virtual void ToggleInViewportContextMenu() {}
	virtual void HideInViewportContextMenu() {}
	UNREALED_API virtual void UpdateInViewportMenuLocation(const FVector2D InLocation);
	virtual bool CanToggleInViewportContextMenu() { return false; }

	UNREALED_API bool IsPreviewingScreenPercentage() const;
	UNREALED_API void TogglePreviewingScreenPercentage();
	UNREALED_API void OnOpenViewportPerformanceProjectSettings();
	UNREALED_API void OnOpenViewportPerformanceEditorPreferences();

	UNREALED_API TSharedPtr<IPreviewProfileController> GetPreviewProfileController();

	/**
	 * A legacy patch built to allow SCommonEditorViewportToolbarBase to automatically switch to the new viewport style.
	 * This function will go away in the future.
	 */
	void MarkLegacyToolbarChildAsAutomaticallyUpgradable(const TSharedRef<SWidget>& ExpectedChild);

///////////////////////////////////////////////////////////////////////////////
// begin shader platform control functions block
///////////////////////////////////////////////////////////////////////////////
private:
	/** Called to get the shader platform preview text */
	UE_DEPRECATED(5.7, "GetCurrentFeatureLevelPreviewText is deprecated, please use GetCurrentShaderPlatformPreviewText")
	UNREALED_API FText GetCurrentFeatureLevelPreviewText(bool bDrawOnlyLabel) const;
	UNREALED_API FText GetCurrentShaderPlatformPreviewText(bool bDrawOnlyLabel) const;

	/** Helper function that, for some FeatureLevel argument, will retrieve the required shader platform */
	UNREALED_API EShaderPlatform GetShaderPlatformHelper(const ERHIFeatureLevel::Type InFeatureLevel) const;

protected:
	/** @return The visibility of the current shader platform preview text display */
	UE_DEPRECATED(5.7, "GetCurrentFeatureLevelPreviewTextVisibility is deprecated, please use GetCurrentShaderPlatformPreviewTextVisibility")
	UNREALED_API EVisibility GetCurrentFeatureLevelPreviewTextVisibility() const;
	UNREALED_API EVisibility GetCurrentShaderPlatformPreviewTextVisibility() const;

	/** @return true if realtime can be toggled (it cannot be toggled directly if there is an override in place) */
	UNREALED_API bool CanToggleRealtime() const;

	/** call this function to build a 'text' widget that can display the current Shader Platform */
	UE_DEPRECATED(5.7, "BuildFeatureLevelWidget is deprecated, please use BuildShaderPlatformWidget")
	UNREALED_API TSharedRef<SWidget> BuildFeatureLevelWidget() const;
	UNREALED_API TSharedRef<SWidget> BuildShaderPlatformWidget() const;
///////////////////////////////////////////////////////////////////////////////
// end shader platform control functions block
///////////////////////////////////////////////////////////////////////////////

	/** Called by the fixed EV100 slider to get the fixed EV100 value */
	UNREALED_API float OnGetFixedEV100Value() const;

	/** Called when fixed EV100 slider is adjusted */
	UNREALED_API void OnFixedEV100ValueChanged( float NewValue );

	/** Called to know whether the fixed EV100 slider is enabled. */
	UNREALED_API bool IsFixedEV100Enabled() const;

	/** Adjustable wireframe opacity */
	UNREALED_API void OnWireframeOpacityChanged(float Opacity);
	UNREALED_API float OnGetWireframeOpacity() const;

	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() = 0;

	// Implement this to add a viewport toolbar above the viewport that does not cover the viewport.
	virtual TSharedPtr<SWidget> BuildViewportToolbar() { return TSharedPtr<SWidget>(nullptr); }

	// Implement this to add a viewport toolbar to the inside top of the viewport
	UE_DEPRECATED(5.7, "Please use BuildViewportToolbar() to create a toolbar that lives above the viewport instead. Alternatively, insert the toolbar in PopulateViewportOverlays()")
	virtual TSharedPtr<SWidget> MakeViewportToolbar() final { return TSharedPtr<SWidget>(nullptr); }

	// Implement this to add an arbitrary set of toolbars or other overlays to the inside of the viewport
	virtual void PopulateViewportOverlays(TSharedRef<SOverlay> Overlay) { }

	virtual TSharedPtr<IPreviewProfileController> CreatePreviewProfileController() { return nullptr; }

	UNREALED_API virtual void BindCommands();
	UNREALED_API virtual void BindShowCommands( FUICommandList& OutCommandList );
	virtual const FSlateBrush* OnGetViewportBorderBrush() const { return nullptr; }
	virtual FSlateColor OnGetViewportBorderColorAndOpacity() const { return FLinearColor::Black; }
	
	/**
	 * @return The visibility of widgets in the viewport (e.g, menus).  Note this is not the visibility of the scene rendered in the viewport                                                              
	 */
	UNREALED_API virtual EVisibility OnGetViewportContentVisibility() const;

	/**
	 * @return The visibility of the viewport focus indicator.                                                             
	 */
	virtual EVisibility OnGetFocusedViewportIndicatorVisibility() const { return EVisibility::Collapsed; }

	/** UI Command delegate bindings */
	UNREALED_API void OnToggleStats();

	/**
	 * Toggles Stat command visibility in this viewport
	 *
	 * @param CommandName				Name of the command
	 */
	UNREALED_API virtual void ToggleStatCommand(FString CommandName);

	/**
	 * Checks if Stat command is visible in this viewport
	 *
	 * @param CommandName				Name of the command
	 */
	UNREALED_API virtual bool IsStatCommandVisible(FString CommandName) const;

	/**
	 * Toggles a show flag in this viewport
	 *
	 * @param EngineShowFlagIndex	the ID to toggle
	 */
	UNREALED_API void ToggleShowFlag( uint32 EngineShowFlagIndex );

	/**
	 * Checks if a show flag is enabled in this viewport
	 *
	 * @param EngineShowFlagIndex	the ID to check
	 * @return true if the show flag is enabled, false otherwise
	 */
	UNREALED_API bool IsShowFlagEnabled( uint32 EngineShowFlagIndex ) const;

	/** Changes the auto exposure setting for this viewport */
	UNREALED_API void ChangeExposureSetting();

	/** Checks if auto exposure setting is selected */
	UNREALED_API bool IsExposureSettingSelected() const;
	
	UNREALED_API virtual void OnScreenCapture();
	UNREALED_API virtual void OnScreenCaptureForProjectThumbnail();
	virtual bool DoesAllowScreenCapture() { return true; }
	
	/**
	 * Changes the snapping grid size
	 */
	virtual void OnIncrementPositionGridSize() {};
	virtual void OnDecrementPositionGridSize() {};
	virtual void OnIncrementRotationGridSize() {};
	virtual void OnDecrementRotationGridSize() {};

	/**
	 * @return true if the specified widget mode is active
	 */
	UNREALED_API virtual bool IsWidgetModeActive( UE::Widget::EWidgetMode Mode ) const;

	/**
	 * @return true if the translate/rotate mode widget is visible 
	 */
	UNREALED_API virtual bool IsTranslateRotateModeVisible() const;

	/**
	* @return true if the 2d mode widget is visible
	*/
	UNREALED_API virtual bool Is2DModeVisible() const;

	/**
	 * Moves between widget modes
	 */
	UNREALED_API virtual void OnCycleWidgetMode();

	/**
	 * Manipulate gizmo size
	 */
	UNREALED_API virtual void OnIncreaseGizmoSize();
	UNREALED_API virtual void OnDecreaseGizmoSize();
	UNREALED_API virtual void OnResetGizmoSize();
	
	/**
	 * Called when the user wants to focus the viewport to the current selection
	 */
	virtual void OnFocusViewportToSelection(){}

	/** Gets the world this viewport is for */
	UNREALED_API virtual UWorld* GetWorld() const;

	/**
	 * Override this function to add the desired viewport interactions for this editor.
	 * Requires virtual function GetModeManagerInteractiveToolsContext to return a valid UModeManagerInteractiveToolsContext
	 */
	UE_DEPRECATED(5.8, "Viewport interactions should be managed from the mode manager layer rather than from individual viewports.")
	virtual void OnRegisterViewportInteractions(UViewportInteractionsBehaviorSource* InBehaviorSource)
	{
	}

	/**
	 * Override to return the Mode Manager Interactive Tools Context for your specific SEditorViewport. Required by
	 * OnRegisterViewportInteractions in order to obtain a valid UViewportInteractionsBehaviorSource
	 */
	virtual UModeManagerInteractiveToolsContext* GetModeManagerInteractiveToolsContext()
	{
		return nullptr;
	}

	/**
	 * Called when surface snapping has been enabled/disabled
	 */
	static UNREALED_API void OnToggleSurfaceSnap();

	/**
	 * Called to test whether surface snapping is enabled or not
	 */
	static UNREALED_API bool OnIsSurfaceSnapEnabled();

	/**
	 * Called when rotation matching surface normal has been enabled/disabled
	 */
	static UNREALED_API void OnToggleRotateToSurfaceNormal();

	/**
	 * Called to test whether rotation matching surface normal is enabled or not
	 */
	static UNREALED_API bool IsRotateToSurfaceNormalEnabled();

	/**
	 * Called to enable/disable distance-based perspective camera speed scaling
	 */
	static UNREALED_API void OnToggleDistanceBasedCameraSpeed();

	/**
	 * Called to test whether distance-based perspective camera speed scaling is enabled
	 */
	static UNREALED_API bool IsDistanceBasedCameraSpeedEnabled();

	/**
	 * Creates a widget showing whether the viewport is active or not
	 * @param InVisibility visibility attribute, can be used to dynamically hide the widget
	 */
	static UNREALED_API TSharedPtr<SWidget> CreateViewportIndicatorWidget(const TAttribute<EVisibility>& InVisibility);

	UNREALED_API void SetAspectRatio(float AspectRatioIn);
	UNREALED_API void SetUsePlatformAspectRatio();
	UNREALED_API void UnsetAspectRatio();

protected:
	TSharedPtr<SOverlay> ViewportOverlay;

	TSharedPtr<SEditorViewportGridPanel> ViewportAspectRatioManager;

	/** Viewport that renders the scene provided by the viewport client */
	TSharedPtr<FSceneViewport> SceneViewport;
	
	/** Widget where the scene viewport is drawn in */
	TSharedPtr<SViewport> ViewportWidget;
	
	/** The client responsible for setting up the scene */
	TSharedPtr<FEditorViewportClient> Client;
	
	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	/** The depth bar used by this viewport */
	UE_EXPERIMENTAL(5.7, "Experimental interface to the Depth bar controls")
	TSharedPtr<SDepthBar> DepthBar;
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
	
	/** Commandlist used in the viewport (Maps commands to viewport specific actions) */
	TSharedPtr<FUICommandList> CommandList;

	/** Handles Preview Profiles */
	TSharedPtr<IPreviewProfileController> PreviewProfileController;

	/** The last time the viewport was ticked (for visibility determination) */
	double LastTickTime;

	FVector2D InViewportContextMenuLocation;

private:
	/** Ensures a Slate tick/paint pass when the viewport is realtime or was invalidated this frame */
	UNREALED_API EActiveTimerReturnType EnsureTick( double InCurrentTime, float InDeltaTime );

	/** Gets the visibility of the active viewport border */
	UNREALED_API EVisibility GetActiveBorderVisibility() const;

	/** Called when the user wants to focus the viewport to the current selection and clip the near plane to the bounds of that selection */
	void OnFocusAndClipViewportToSelection();

	/**
	 * Creates the viewport depth bar and any containing widgets
	 */
	TSharedPtr<SWidget> BuildViewportDepthBar();
	
	/**
	 * Called when enabling or disabling the depth bar.
	 */
	void OnToggleUseDepthBar();

	/**
	 * @return True if the depth bar can be enabled. 
	 */
	bool CanUseDepthBar() const;

	/**
	 * @return True if the depth bar is active 
	 */
	bool IsDepthBarActive() const;
	
	/** @return The visibility of the viewport depth bar. */
	EVisibility OnGetDepthBarVisibility() const;

	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS	
	/** @return The current mode of the depth bar. */
	SDepthBar::EMode GetDepthBarMode() const;

	/** @return The current depth space of the depth bar. */
	TOptional<SDepthBar::FDepthSpace> GetDepthBarSpace() const;
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
	
	/** @return The current near plane for the depth bar. */
	TOptional<double> GetDepthBarNearPlane() const;
	
	/** @return The current near plane for the depth bar. */
	TOptional<double> GetDepthBarFarPlane() const;
	
	/** Called to change the depth bar near plane. */
	void OnDepthBarNearPlaneChanged(const TOptional<double>& InNearPlane);
	
	/** Called to change the depth bar far plane. */
	void OnDepthBarFarPlaneChanged(const TOptional<double>& InFarPlane);
	
	/** Reset the depth bar back to the default state. */
	void DismissDepthBar();

private:
	/** The handle to the active timer */
	TWeakPtr<FActiveTimerHandle> ActiveTimerHandle;

	/** Whether the viewport needs to update, even without input or realtime (e.g. inertial camera movement) */
	bool bInvalidated;
	
	/** Whether the depth bar should be displayed, if possible */
	bool bShowDepthBar = false;
	
	/** Whether there is enough space to display the depth bar */
	bool bHasSpaceForDepthBar = false;
	
	/** For automatic legacy toolbar upgrades. Registers the child widget to expect in order to allow an automatic upgrade. */
	TWeakPtr<SWidget> AutoUpgradeWidgetChild = nullptr;
	/** A flag indicating that the automatic toolbar upgrade has been approved */
	bool bLegacyToolbarIsAutomaticallyUpgradable = false;

	/**
	 * Storing Interactive Tools Context so that we can unbind delegates in the class constructor. Once the new ITF
	 * based gizmos, drag tools and viewport interactions will be the default, this can be removed.
	 */
	TWeakObjectPtr<UModeManagerInteractiveToolsContext> ModeManagerInteractiveToolsContextWeak;
};
