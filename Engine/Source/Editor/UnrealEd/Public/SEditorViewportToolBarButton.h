// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/SlateDelegates.h"
#include "HAL/Platform.h"
#include "Misc/Attribute.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Styling/ToolBarStyle.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FName;
class SWidget;
struct FButtonStyle;
struct FCheckBoxStyle;
struct FSlateBrush;

enum class ECheckBoxState : uint8;

/**
 * A simple class that represents a toolbar button in an editor viewport toolbar
 */
class SEditorViewportToolBarButton : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SEditorViewportToolBarButton)
		: _ButtonType(EUserInterfaceActionType::Button)
		, _ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("EditorViewportToolBar").ButtonStyle)
		, _CheckBoxStyle(&FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("EditorViewportToolBar").ToggleButton)
		, _IsChecked(false)
		{}

		/** Called when the button is clicked */
	SLATE_EVENT(FOnClicked, OnClicked)

		/** The button type to use */
		SLATE_ARGUMENT(EUserInterfaceActionType, ButtonType)

		/** Style to use when this is a regular button type */
		SLATE_STYLE_ARGUMENT(FButtonStyle, ButtonStyle)

		/** Style to use when this is a check box type */
		SLATE_STYLE_ARGUMENT(FCheckBoxStyle, CheckBoxStyle)

		/** Checked state of the button */
		SLATE_ATTRIBUTE(bool, IsChecked)

		/** Style name of an image to use. Simple two state images are supported.  An image can be different depending on checked/unchecked state */
		SLATE_ATTRIBUTE(FName, Image)
		/** Any custom content to show in the button in place of other content */
		SLATE_DEFAULT_SLOT( FArguments, Content )
	SLATE_END_ARGS()

	UNREALED_API void Construct( const FArguments& Declaration );
private:
	/** 
	 * Called when the button check state changes
	 * 
	 * @param NewCheckedState	The new state of the check box
	 */
	void OnCheckStateChanged( ECheckBoxState NewCheckedState );

	/**
	 * Called when we need to get the image to show in the button
	 *
	 * @return The brush defining the image to use
	 */
	const FSlateBrush* OnGetButtonImage() const;

	/**
	 * Called when we need to get the state of the check box button
	 *
	 * @return The state of the check box button
	 */
	ECheckBoxState OnIsChecked() const;

private:
	/** Attribute used to get the state of a checkbox */
	TAttribute<bool> IsChecked;
	/** Delegate to call when the button is clicked */
	FOnClicked OnClickedDelegate;
	/** Cached brush to use when the button is checked */
	const FSlateBrush* CheckedBrush;
	/** Cached brush to use when the button is unchecked */
	const FSlateBrush* NormalBrush;
};

