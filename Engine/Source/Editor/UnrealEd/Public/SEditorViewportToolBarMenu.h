// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "Framework/SlateDelegates.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "SViewportToolBar.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SMenuAnchor;
struct FGeometry;
struct FPointerEvent;
struct FSlateBrush;

namespace EMenuItemType
{
	enum Type
	{
		Default,
		Header,
		Separator,
	};
}


/**
 * Widget that opens a menu when clicked
 */
class SEditorViewportToolbarMenu : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SEditorViewportToolbarMenu)
		: _MenuStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("EditorViewportToolBar.Button"))
		, _ForegroundColor(FSlateColor::UseStyle())
	{}
		/** We need to know about the toolbar we are in */
		SLATE_ARGUMENT(TSharedPtr<class SViewportToolBar>, ParentToolBar);
		/** Style to use */
		SLATE_STYLE_ARGUMENT(FButtonStyle, MenuStyle)
		/** The label to show in the menu */
		SLATE_ATTRIBUTE(FText, Label)
		/** Optional icon to display next to the label */
		SLATE_ATTRIBUTE(const FSlateBrush*, LabelIcon)
		/** The image to show in the menu.  If both the label and image are valid, the button image is used.  Note that if this image is used, the label icon will not be displayed. */
		SLATE_ARGUMENT(FName, Image)
		/** Content to show in the menu */
		SLATE_EVENT(FOnGetContent, OnGetMenuContent)
		/** The foreground color of the content */
		SLATE_ATTRIBUTE(FSlateColor, ForegroundColor)
		/** Where the menu is spawned relative to the menu button */
		SLATE_ARGUMENT_DEFAULT(EMenuPlacement, MenuPlacement) = EMenuPlacement::MenuPlacement_BelowAnchor;
	SLATE_END_ARGS()

	/**
	 * Constructs the menu
	 */
	UNREALED_API void Construct( const FArguments& Declaration );

	/**
	 * Returns parent tool bar
	 */
	UNREALED_API TWeakPtr<class SViewportToolBar> GetParentToolBar() const;

	/**
	 * @return true if this menu is open
	 */
	UNREALED_API bool IsMenuOpen() const;

private:
	/**
	 * Called when the menu button is clicked.  Will toggle the visibility of the menu content                   
	 */
	FReply OnMenuClicked();

	/**
	 * Called when the mouse enters a menu button.  If there was a menu previously opened
	 * we open this menu automatically
	 */
	UNREALED_API void OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent );

	EVisibility GetLabelIconVisibility() const;

protected:
	/** Parent tool bar for querying other open menus */
	TWeakPtr<class SViewportToolBar> ParentToolBar;

	/** Name of tool menu */
	FName MenuName;

	/**
	 * Called to query the tool tip text for this widget, but will return an empty text for menu bar items
	 * when a menu for that menu bar is already open
	 *
	 * @param	ToolTipText	Tool tip text to display, if possible
	 *
	 * @return	Tool tip text, or an empty text if filtered out
	 */
	UNREALED_API FText GetFilteredToolTipText(TAttribute<FText> ToolTipText) const;

private:
	/** Our menus anchor */
	TSharedPtr<SMenuAnchor> MenuAnchor;
	TAttribute<const FSlateBrush*> LabelIconBrush;
};
