// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "SEditorViewportToolBarMenu.h"
#include "Styling/SlateTypes.h"

class FExtender;
class SEditorViewport;
class SWidget;
class UToolMenu;
struct FSlateBrush;

class SEditorViewportViewMenu : public SEditorViewportToolbarMenu
{
public:
	SLATE_BEGIN_ARGS( SEditorViewportViewMenu ){}
		SLATE_ARGUMENT( TSharedPtr<class FExtender>, MenuExtenders )
	SLATE_END_ARGS()

	UNREALED_API void Construct( const FArguments& InArgs, TSharedRef<SEditorViewport> InViewport, TSharedRef<class SViewportToolBar> InParentToolBar );

private:
	FText GetViewMenuLabel() const;
	const FSlateBrush* GetViewMenuLabelIcon() const;
	void FillViewMenu(UToolMenu* Menu) const;

protected:
	UNREALED_API virtual TSharedRef<SWidget> GenerateViewMenuContent() const;
	UNREALED_API virtual void RegisterMenus() const;

	TWeakPtr<SEditorViewport> Viewport;
	TSharedPtr<class FExtender> MenuExtenders;

	static UNREALED_API const FName BaseMenuName;
};
