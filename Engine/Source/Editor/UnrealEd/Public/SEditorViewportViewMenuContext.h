// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "SEditorViewportViewMenuContext.generated.h"

class SEditorViewportViewMenu;

UCLASS(MinimalAPI)
class UEditorViewportViewMenuContext : public UObject
{
	GENERATED_BODY()
public:

	TWeakPtr<const SEditorViewportViewMenu> EditorViewportViewMenu;
};
