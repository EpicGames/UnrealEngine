// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "InputCoreTypes.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "SlateFwd.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

class FKeyTreeInfo;
class ITableRow;
class SComboButton;
class SSearchBox;
class SWidget;
struct FAnalogInputEvent;
struct FGeometry;
struct FKeyEvent;
struct FPointerEvent;
struct FSlateBrush;

DECLARE_DELEGATE_OneParam(FOnKeyChanged, TSharedPtr<FKey>)

//////////////////////////////////////////////////////////////////////////
// SKeySelector

typedef TSharedPtr<FKeyTreeInfo> FKeyTreeItem;
typedef STreeView<FKeyTreeItem> SKeyTreeView;

/** Widget for selecting an input key */
class SKeySelector : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SKeySelector )
		: _CurrentKey(FKey())
		, _TreeViewWidth(300.f)
		, _TreeViewHeight(400.f)
		, _Font( FAppStyle::GetFontStyle( TEXT("NormalFont") ) )
		, _FilterBlueprintBindable( true )
		, _AllowClear( true )
		, _AllowKeyChange( true )
		{}
		SLATE_ATTRIBUTE( TOptional<FKey>, CurrentKey )
		SLATE_ATTRIBUTE( FOptionalSize, TreeViewWidth )
		SLATE_ATTRIBUTE( FOptionalSize, TreeViewHeight )
		SLATE_EVENT( FOnKeyChanged, OnKeyChanged )
		SLATE_ATTRIBUTE( FSlateFontInfo, Font )
		SLATE_ARGUMENT( bool, FilterBlueprintBindable )
		SLATE_ARGUMENT( bool, AllowClear )
		SLATE_ARGUMENT( bool, AllowKeyChange )
	SLATE_END_ARGS()
public:
	UNREALED_API void Construct(const FArguments& InArgs);

	/** Sets bool to produce tooltip notifying this key selector it was disabled from KeyStructCustomization */
	void SetEnabledFromKeyStructCustomization(bool bIsEnabled)
	{
		bEnabledFromKeyStructCustomization = bIsEnabled;
	}

	/** Gets bEnabledFromKeyStructCustomization bool */
	bool GetSetEnabledFromKeyStructCustomization() const
	{
		return bEnabledFromKeyStructCustomization;
	}
	
	/** Sets tooltip on the KeySelector when it is disabled */
	void SetDisabledKeySelectorToolTip(const FText& InToolTip)
	{
		DisabledSelectorToolTip = InToolTip;
	}
	
	/** Gets tooltip on the KeySelector when it is disabled */
	FText GetDisabledKeySelectorToolTip() const
	{
		return DisabledSelectorToolTip;
	}
	
protected:
	/** Gets the icon for the key being manipulated */
	UNREALED_API const FSlateBrush* GetKeyIconImage() const;
	/** Toggles the icon's color when in listen mode */
	UNREALED_API FSlateColor GetKeyIconColor() const;

	/** Gets a succinct description for the key being manipulated */
	UNREALED_API FText GetKeyDescription() const;
	/** Gets a description tooltip for the key being manipulated */
	UNREALED_API FText GetKeyDescriptionToolTip() const;
	/** Gets a tooltip for the selected key */
	UNREALED_API FText GetKeyTooltip() const;
	
	/** Tooltip to display on the selector when the selector is disabled*/
	FText DisabledSelectorToolTip = FText::FromString(TEXT("Key Selector Disabled"));

	/** Treeview support functions */
	UNREALED_API virtual TSharedRef<ITableRow> GenerateKeyTreeRow(FKeyTreeItem InItem, const TSharedRef<STableViewBase>& OwnerTree);
	UNREALED_API void OnKeySelectionChanged(FKeyTreeItem Selection, ESelectInfo::Type SelectInfo);
	UNREALED_API void GetKeyChildren(FKeyTreeItem InItem, TArray<FKeyTreeItem>& OutChildren);

	/**
	 * Returns true if this key selector can actually change the that it's assigned to or not
	 */
	UNREALED_API bool CanChangeKey() const;

	/** Gets the Menu Content, setting it up if necessary */
	UNREALED_API virtual TSharedRef<SWidget>	GetMenuContent();

	/** Key searching support */
	UNREALED_API void OnFilterTextChanged(const FText& NewText);
	UNREALED_API void OnFilterTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo);
	UNREALED_API void GetSearchTokens(const FString& SearchString, TArray<FString>& OutTokens) const;

	/** Helper to generate the filtered list of keys, based on the search string matching */
	UNREALED_API bool GetChildrenMatchingSearch(const TArray<FString>& SearchTokens, const TArray<FKeyTreeItem>& UnfilteredList, TArray<FKeyTreeItem>& OutFilteredList);

	/** Start listening for the next key press */
	UNREALED_API FReply ListenForInput();
	/** Assigns the heard input as the current key */
	UNREALED_API FReply ProcessHeardInput(FKey KeyHeard);

	virtual bool SupportsKeyboardFocus() const override { return bListenForNextInput; }

	/** Input listeners */
	UNREALED_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	UNREALED_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UNREALED_API virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UNREALED_API virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UNREALED_API virtual FReply OnAnalogValueChanged(const FGeometry& MyGeometry, const FAnalogInputEvent& InAnalogInputEvent) override;

	/** 
	 * Determine the best icon to represent the given key.
	 *
	 * @param Key		The key to get the icon for.
	 * @param returns a brush that best represents the icon
	 */
	UNREALED_API const FSlateBrush* GetIconFromKey(FKey Key) const;

protected:
	/** Combo Button that shows current key and icon */
	TSharedPtr<SComboButton>	KeyComboButton;

	/** Reference to the menu content that's displayed when the key button is clicked on */
	TSharedPtr<SWidget>			MenuContent;
	TSharedPtr<SSearchBox>		FilterTextBox;
	TSharedPtr<SKeyTreeView>	KeyTreeView;
	FText						SearchText;

	/** The key attribute that we're modifying with this widget, or an empty optional if the key contains multiple values */
	TAttribute<TOptional<FKey>>	CurrentKey;

	/** Delegate that is called every time the key changes. */
	FOnKeyChanged				OnKeyChanged;

	/** If true, then we are allowed to change the key in this widget. Otherwise, this should be used for display only. */
	TAttribute<bool> AllowChangeKey;

	/** Desired width of the tree view widget */
	TAttribute<FOptionalSize>	TreeViewWidth;
	/** Desired height of the tree view widget */
	TAttribute<FOptionalSize>	TreeViewHeight;

	/** Font used for category tree entries */
	FSlateFontInfo				CategoryFont;
	/** Font used for key tree entries */
	FSlateFontInfo				KeyFont;

	/** Array containing the unfiltered list of all values this key could possibly have */
	TArray<FKeyTreeItem>		KeyTreeRoot;
	/** Array containing a filtered list, according to the text in the searchbox */
	TArray<FKeyTreeItem>		FilteredKeyTreeRoot;

	bool bListenForNextInput = false;
	bool bEnabledFromKeyStructCustomization = true;
};
