// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Events.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Input/SSearchBox.h"

/** This is a container widget to help refocus to a listview widget from a searchbox or other text box widgets that are used in conjunction.
	Will refocus when the up or down arrows are pressed, and will commit a selection when enter is pressed regardless of where focus is. */
template <typename ListType>
class SListViewSelectorDropdownMenu : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SListViewSelectorDropdownMenu )
		{}
		SLATE_DEFAULT_SLOT( FArguments, Content )
	SLATE_END_ARGS()
public:
	/**
	 * @param InDefaultfocusWidget		The default widget to give focus to when the listview does not handle an action
	 * @param InTargetListView			If the SListViewSelectorDropdownMenu is handling the KeyDown event, focus will be applied to the TargetListView for certain keys that it can handle
	 */
	void Construct(const FArguments& InArgs, TSharedPtr<SSearchBox> InDefaultFocusWidget, TSharedPtr< SListView< ListType > > InTargetListView)
	{
		check(InTargetListView.IsValid());

		TargetListView = InTargetListView;
		DefaultFocusWidget = InDefaultFocusWidget;

		if (DefaultFocusWidget.IsValid())
		{
			// We have some overrides for OnKeyDown handling of the SSearchBox to provide the seamless functionality
			InDefaultFocusWidget->SetOnKeyDownHandler(FOnKeyDown::CreateSP(this, &SListViewSelectorDropdownMenu<ListType>::OnKeyDown));
		}

		this->ChildSlot
		[
			InArgs._Content.Widget
		];
	}

	// SWidget interface
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& KeyEvent) override
	{
		TSharedPtr< SListView< ListType > > TargetListViewPtr = TargetListView.Pin();

		if (KeyEvent.GetKey() == EKeys::Up || KeyEvent.GetKey() == EKeys::Down)
		{
			// Deliver focus to the tree view, so the user can use the arrow keys to move through the items
			return TargetListViewPtr->OnKeyDown(FindChildGeometry(MyGeometry, TargetListViewPtr.ToSharedRef()), KeyEvent);
		}
		else if(KeyEvent.GetKey() == EKeys::Enter)
		{
			// If there is anything selected, re-select it "direct" so that the menu will act upon the selection
			if(TargetListViewPtr->GetNumItemsSelected() > 0)
			{
				TargetListViewPtr->SetSelection(TargetListViewPtr->GetSelectedItems()[0]);
			}
			return FReply::Handled();
		}
		else if (DefaultFocusWidget.IsValid())
		{
			TSharedPtr< SWidget > DefaultFocusWidgetPtr = DefaultFocusWidget.Pin();
			FReply Reply = DefaultFocusWidgetPtr->OnKeyDown(FindChildGeometry(MyGeometry, TargetListViewPtr.ToSharedRef()), KeyEvent);

			if (!DefaultFocusWidgetPtr->HasKeyboardFocus())
			{
				Reply.SetUserFocus(DefaultFocusWidgetPtr.ToSharedRef(), EFocusCause::OtherWidgetLostFocus);
			}
			return Reply;
		}
		return FReply::Unhandled();
	}
	// End of SWidget interface

private:
	/** The type tree view widget this is handling keyboard input for */
	TWeakPtr< SListView<ListType> > TargetListView;
	/** Widget to revert focus back to when this widget does not handle (or forward) a key input */
	TWeakPtr< SSearchBox > DefaultFocusWidget;
};
