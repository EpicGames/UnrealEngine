// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Scalability.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Internationalization/Text.h"

enum class ECheckBoxState : uint8;

/**
 * Scalability settings configuration widget                                                                  
 **/

class SScalabilitySettings : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( SScalabilitySettings )
	{}

	SLATE_END_ARGS()

	// Widget construction
	UNREALED_API void Construct( const FArguments& InArgs );

	UNREALED_API ~SScalabilitySettings();
private:

	static bool IsPlayInEditor();
	static FFormatNamedArguments GetScreenPercentageFormatArguments(const class UGameViewportClient* ViewportClient);

	// Checks cached quality levels to see if the specified group is at the specified quality level
	ECheckBoxState IsGroupQualityLevelSelected(const TCHAR* InGroupName, int32 InQualityLevel) const;

	// Callback for when a particular scalability group has its quality level changed
	void OnGroupQualityLevelChanged(ECheckBoxState NewState, const TCHAR* InGroupName, int32 InQualityLevel);

	// Callback for when the resolution scale slider changes
	void OnResolutionScaleChanged(float InValue);

	// Callback to retrieve current resolution scale
	float GetResolutionScale() const;

	// Callback to retrieve current resolution scale as a display string
	FText GetResolutionScaleString() const;

	// Callback for when a resolution preset is clicked
	FReply OnResolutionPresetClicked(int32 Preset);

	// Whether the resolution can be modified.
	static bool IsResolutionScaleEditable();

	// Makes a button widget for the group quality levels
	TSharedRef<SWidget> MakeButtonWidget(const FText& InName, const TCHAR* InGroupName, int32 InQualityLevel, const FText& InToolTip);

	// Makes a general quality level header button widget
	TSharedRef<SWidget> MakeHeaderButtonWidget(const FText& InName, int32 InQualityLevel, const FText& InToolTip, Scalability::EQualityLevelBehavior Behavior);

	// Makes the auto benchmark button
	TSharedRef<SWidget> MakeAutoButtonWidget();
	
	// Makes the default button
	TSharedRef<SWidget> MakeDefaultButtonWidget();

	// Callback for when a quality level header button is pressed
	void OnHeaderClicked(ECheckBoxState CheckState, int32 InQualityLevel, Scalability::EQualityLevelBehavior Behavior);
	ECheckBoxState GetHeaderChecked(int32 InQualityLevel, Scalability::EQualityLevelBehavior Behavior) const;

	// Callback for auto benchmark button
	FReply OnAutoClicked();
	
	// Callback for default button
	void OnDefaultClicked(ECheckBoxState CheckState);
	ECheckBoxState GetDefaultCheckState() const;

	// Create a gridslot for the group quality level with all the required formatting
	SGridPanel::FSlot::FSlotArguments MakeGridSlot(int32 InCol, int32 InRow, int32 InColSpan = 1, int32 InRowSpan = 1);

	/** Called to get the "Show notification" check box state */
	ECheckBoxState IsMonitoringPerformance() const;

	/** Called when the state of the "Show notification" check box changes */
	void OnMonitorPerformanceChanged(ECheckBoxState NewState);

	/** Adds buttons for one settings strip to the grid */
	void AddButtonsToGrid(int32 X0, int32 Y0, TSharedRef<SGridPanel> ButtonMatrix, const FText* FiveNameArray, int32 ButtonCount, const TCHAR* GroupName, const FText& TooltipShape);

private:
	/* The state of scalability settings at the point of opening the menu*/
	Scalability::FQualityLevels InitialQualityLevels;

	/** The state of quality levels as they are changed in this widget */
	Scalability::FQualityLevels CachedQualityLevels;
};
