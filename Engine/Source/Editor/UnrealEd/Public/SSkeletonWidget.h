// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/Skeleton.h"
#include "CoreMinimal.h"
#include "IContentBrowserSingleton.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Styling/AppStyle.h"
#include "AssetRegistry/AssetData.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "PreviewScene.h"
#include "EditorViewportClient.h"
#include "SEditorViewport.h"
#include "EditorAnimUtils.h"

class UAnimSet;
class USkeletalMesh;

using namespace EditorAnimUtils;

/**
 * This below code is to select skeleton from the list 
 */
#define LOCTEXT_NAMESPACE "SkeletonWidget" 

struct FBoneTrackPair 
{
	FName Bone1;
	FName Bone2;

	FBoneTrackPair(FName InBone1, FName InBone2)
		: Bone1(InBone1)
		, Bone2(InBone2)
	{
	}
};

class SBonePairRow : public SMultiColumnTableRow< TSharedPtr<FBoneTrackPair> >
{
public:
	SLATE_BEGIN_ARGS(SBonePairRow){}
		SLATE_ARGUMENT( TSharedPtr<FBoneTrackPair>, BonePair)
	SLATE_END_ARGS()

	/**
	 * Construct child widgets that comprise this widget.
	 *
	 * @param InArgs  Declaration from which to construct this widget
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView )
	{
		this->BonePair = InArgs._BonePair;

		SMultiColumnTableRow< TSharedPtr<FBoneTrackPair> >::Construct( FSuperRowType::FArguments(), InOwnerTableView );
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override
	{
		TSharedPtr<SBorder> Border1, Border2;

		if (ColumnName == TEXT("Curretly Selected"))
		{
			return SAssignNew(Border1, SBorder) .Padding(2.f)
				.Content()
				[
					SNew(STextBlock)
					.Text(FText::FromName(BonePair->Bone1))
				];
		}
		else 
		{
			if (BonePair->Bone2 == NAME_None)
			{
				return SAssignNew(Border2, SBorder) .Padding(2.f)
					.ColorAndOpacity(FLinearColor(1.f, 0.f, 0.f))
					.Content()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("MissingBone", "Missing"))
					];
			}
			else
			{
				return SAssignNew(Border2, SBorder) .Padding(2.f)
					.Content()
					[
						SNew(STextBlock)
						.Text(FText::FromName(BonePair->Bone2))
					];
			}
		}
	}
private:
	TSharedPtr<FBoneTrackPair> BonePair;
};

class SSkeletonWidget : public SCompoundWidget
{
public:

	USkeleton* GetSelectedSkeleton() const
	{ 
		return CurSelectedSkeleton; 
	}
protected:
	USkeleton* CurSelectedSkeleton;
};

/** 1 columns - just show bone list **/
class SSkeletonListWidget : public SSkeletonWidget
{
public:
	SLATE_BEGIN_ARGS( SSkeletonListWidget ){}
		SLATE_ARGUMENT(bool, ShowBones)
		SLATE_ARGUMENT(EAssetViewType::Type, InitialViewType)
	SLATE_END_ARGS()
public:
	// WIDGETS

	void Construct(const FArguments& InArgs);

	void SkeletonSelectionChanged(const FAssetData& AssetData);

	TSharedRef<ITableRow> GenerateSkeletonRow( USkeleton* InSkeleton, const TSharedRef<STableViewBase>& OwnerTable )
	{
		return
			SNew( STableRow< USkeleton* >, OwnerTable )
			. Content()
			[
				SNew(STextBlock)
				.Text(FText::FromString(InSkeleton->GetFullName()))
			];
	}


	TSharedRef<ITableRow> GenerateSkeletonBoneRow( TSharedPtr<FName> InBoneName, const TSharedRef<STableViewBase>& OwnerTable )
	{
		return
			SNew( STableRow< TSharedPtr<FName> >, OwnerTable )
			. Content()
			[
				SNew(STextBlock)
				.Text(FText::FromName(*InBoneName))
			];
	}

private:
	bool bShowBones = true;
	EAssetViewType::Type InitialViewType = EAssetViewType::Column;
	SVerticalBox::FSlot* BoneListSlot;
	TArray< TSharedPtr<FName> > BoneList;
};

/** 2 columns - bone pair widget **/
class SSkeletonCompareWidget : public SSkeletonWidget
{
public:
	SLATE_BEGIN_ARGS( SSkeletonCompareWidget )
		: _Object(NULL)
		{}

		SLATE_ARGUMENT( UObject*, Object )
		SLATE_ARGUMENT( TArray<FName>*, BoneNames )
	SLATE_END_ARGS()
public:
	// WIDGETS

	void Construct(const FArguments& InArgs);

	void SkeletonSelectionChanged(const FAssetData& AssetData);

	TSharedRef<ITableRow> GenerateBonePairRow( TSharedPtr<FBoneTrackPair> InBonePair, const TSharedRef<STableViewBase>& OwnerTable )
	{
		return
			SNew( SBonePairRow, OwnerTable )
			.BonePair(InBonePair);
	}

private:
	TArray<FName>	BoneNames;
	SVerticalBox::FSlot * BonePairSlot;
	TArray< TSharedPtr<FBoneTrackPair> >	BonePairList;
};

class SSkeletonSelectorWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SSkeletonSelectorWindow )
		: _Object(NULL)
		{}

		SLATE_ARGUMENT( UObject*, Object )
		SLATE_ARGUMENT( TSharedPtr<SWindow>, WidgetWindow )
	SLATE_END_ARGS()
public:
	UNREALED_API void Construct(const FArguments& InArgs);

	void ConstructWindowFromAnimSet(UAnimSet* InAnimSet);

	void ConstructWindowFromMesh(USkeletalMesh* InSkeletalMesh);

	void ConstructWindowFromAnimBlueprint(UAnimBlueprint* AnimBlueprint);

	void ConstructWindow();

	void ConstructButtons(TSharedRef<SVerticalBox> ParentBox)
	{
		ParentBox->AddSlot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Bottom)
		[
			SNew(SUniformGridPanel)
			.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
			.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
			.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
			+SUniformGridPanel::Slot(0,0)
			[
				SNew(SButton)
				.Text(LOCTEXT("Accept", "Accept"))
				.HAlign(HAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.OnClicked_Raw( this, &SSkeletonSelectorWindow::OnAccept )
			]
			+SUniformGridPanel::Slot(1,0)
			[
				SNew(SButton)
				.Text(LOCTEXT("Cancel", "Cancel"))
				.HAlign(HAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.OnClicked_Raw( this, &SSkeletonSelectorWindow::OnCancel )
			]
		];
	}

	FReply OnAccept()
	{
		SelectedSkeleton = SkeletonWidget->GetSelectedSkeleton();

		if (SelectedSkeleton!=NULL && WidgetWindow.IsValid())
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	FReply OnCancel()
	{
		if ( WidgetWindow.IsValid() )
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	USkeleton* GetSelectedSkeleton()
	{
		return SelectedSkeleton;
	}

private:
	TSharedPtr<SSkeletonWidget>			SkeletonWidget;
	TWeakPtr<SWindow>					WidgetWindow;
	USkeleton *							SelectedSkeleton;
};


//////////////////////////
class SBasePoseViewport: public /*SCompoundWidget*/SEditorViewport
{
public:
	SLATE_BEGIN_ARGS(SBasePoseViewport)
	{}

	SLATE_ARGUMENT(USkeleton*, Skeleton)
SLATE_END_ARGS()

public:
	SBasePoseViewport();

	void Construct(const FArguments& InArgs);
	void SetSkeleton(USkeleton* Skeleton);

protected:
	/** SEditorViewport interface */
	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;

private:
	/** Skeleton */
	USkeleton* TargetSkeleton;

	FPreviewScene PreviewScene;

	class UDebugSkelMeshComponent* PreviewComponent;

	bool IsVisible() const override;
};

//////////////////////////////////////////////////////////////////////////

class FDisplayedAssetEntryInfo
{
public:
	USkeleton* NewSkeleton;
	UObject* AnimAsset;
	UObject* RemapAsset;

	static TSharedRef<FDisplayedAssetEntryInfo> Make(UObject* InAsset, USkeleton* InNewSkeleton);

protected:

	FDisplayedAssetEntryInfo(){};
	FDisplayedAssetEntryInfo(UObject* InAsset, USkeleton* InNewSkeleton);
};


/////////////////////////////////////////////
/** 
 * Slate panel for choose displaying bones to remove
 */
class SSkeletonBoneRemoval : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SSkeletonBoneRemoval){}

		/** The bones to remove (for list display) */
		SLATE_ARGUMENT( TArray<FName>, BonesToRemove )

		/** The window this panel has been placed in */
		SLATE_ARGUMENT( TSharedPtr<SWindow>, WidgetWindow )

		/** Message to display to the user */
		SLATE_ARGUMENT( FText, WarningMessage )

	SLATE_END_ARGS()	

	/**
	 * Constructs this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	void Construct( const FArguments& InArgs );

	/** Reference to our window */
	TWeakPtr<SWindow> WidgetWindow;

	/** Button Handlers */
	FReply OnOk();
	FReply OnCancel();

	/** Handle closing to dialog window */
	void CloseWindow();

	/** Create an individual row for the bone name list */
	TSharedRef<ITableRow> GenerateSkeletonBoneRow( TSharedPtr<FName> InBoneName, const TSharedRef<STableViewBase>& OwnerTable )
	{
		return
		SNew( STableRow< TSharedPtr<FName> >, OwnerTable )
		. Content()
		[
			SNew(STextBlock)
			.Text(FText::FromName(*InBoneName))
		];
	}

	/**
	 *  Show Modal window
	 *
	 * @param BonesToRemove		List of bones that will be removed
	 * @param WarningMessage	Message to display to the user so they know what is going on
	 *
	 * @return true if successfully selected new skeleton
	 */
	static UNREALED_API bool ShowModal(const TArray<FName> BonesToRemove, const FText& WarningMessage);

	/** Did the user choose to continue */
	bool bShouldContinue;

	/** List of bone names that will be removed */
	TArray< TSharedPtr<FName> > BoneNames;
};

class SSelectFolderDlg: public SWindow
{
public:
	SLATE_BEGIN_ARGS(SSelectFolderDlg)
	{
	}

	SLATE_ARGUMENT(FText, DefaultAssetPath)
	SLATE_END_ARGS()

		SSelectFolderDlg()
		: UserResponse(EAppReturnType::Cancel)
	{
		}

	void Construct(const FArguments& InArgs);

public:
	/** Displays the dialog in a blocking fashion */
	EAppReturnType::Type ShowModal();

	/** Gets the resulting asset path */
	FString GetAssetPath();

protected:
	void OnPathChange(const FString& NewPath);
	FReply OnButtonClick(EAppReturnType::Type ButtonID);

	EAppReturnType::Type UserResponse;
	FText AssetPath;
};

class SReplaceMissingSkeletonDialog : public SWindow
{
	
public:
	
	SLATE_BEGIN_ARGS(SReplaceMissingSkeletonDialog)
	{
	}

	SLATE_ARGUMENT(TArray<TWeakObjectPtr<UObject>>, AnimAssets)
	SLATE_END_ARGS()

	void UNREALED_API Construct(const FArguments& InArgs);
	
	SReplaceMissingSkeletonDialog()
		: UserResponse(EAppReturnType::Cancel)
		, bWasSkeletonReplaced(false)
	{
	}

	bool UNREALED_API ShowModal();

protected:
	void OnSkeletonSelected(const FAssetData& Replacement);
	FReply OnButtonClick(EAppReturnType::Type ButtonID);
	EAppReturnType::Type UserResponse;
	FAssetData SelectedAsset;
	TArray<TWeakObjectPtr<UObject>> AssetsToReplaceSkeletonOn;
	bool bWasSkeletonReplaced;
};

#undef LOCTEXT_NAMESPACE
