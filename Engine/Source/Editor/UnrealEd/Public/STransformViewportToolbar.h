// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/Input/SSpinBox.h"
#include "Textures/SlateIcon.h"
#include "SViewportToolBar.h"
#include "Settings/LevelEditorViewportSettings.h"

class FExtender;
class FUICommandList;
class SEditorViewport;
class SSlider;
class SEditorViewportToolbarMenu;
enum class ECheckBoxState : uint8;

/** A delegate that is executed when adding menu content. */
DECLARE_DELEGATE_OneParam(FOnCamSpeedChanged, int32);
DECLARE_DELEGATE_OneParam(FOnCamSpeedScalarChanged, float);

/**
 * Viewport toolbar containing transform, grid snapping, local to world and camera speed controls.
 */
class STransformViewportToolBar  : public SViewportToolBar
{

public:
	SLATE_BEGIN_ARGS( STransformViewportToolBar ){}
		SLATE_ARGUMENT( TSharedPtr<class SEditorViewport>, Viewport )
		SLATE_ARGUMENT( TSharedPtr<FUICommandList>, CommandList )
		SLATE_ARGUMENT( TSharedPtr<FExtender>, Extenders )

		SLATE_EVENT( FOnCamSpeedChanged, OnCamSpeedChanged )
		SLATE_EVENT( FOnCamSpeedScalarChanged, OnCamSpeedScalarChanged )
	SLATE_END_ARGS()

	UNREALED_API void Construct( const FArguments& InArgs );
	/**
	 * Static: Creates a widget for the main tool bar
	 *
	 * @return	New widget
	 */
	UNREALED_API TSharedRef<SWidget> MakeTransformToolBar(const TSharedPtr< FExtender > InExtenders);

private:
	FSlateIcon GetLocalToWorldIcon() const;
	FSlateIcon GetSurfaceSnappingIcon() const;

	/** Callback to toggle between local and world space */
	FReply OnCycleCoordinateSystem();

	/** Camera speed menu construction callback */
	TSharedRef<SWidget> FillCameraSpeedMenu();

	/** Camera speed Label callback */
	FText GetCameraSpeedLabel() const;

	/** Returns the current camera speed setting */
	float GetCamSpeedSliderPosition() const;

	/** 
	 * Sets new camera speed 
	 *
	 * @Param	NewValue	Value to set camera speed too
	 */
	void OnSetCamSpeed(float NewValue);

	/** Called when the camera speed is changed */
	FOnCamSpeedChanged OnCamSpeedChanged;
	FOnCamSpeedScalarChanged OnCamSpeedScalarChanged;

	/** Grid snap label callbacks */
	FText GetLayer2DLabel() const;

	/** GridSnap menu construction callbacks */
	TSharedRef<SWidget> FillLocationGridSnapMenu();
	TSharedRef<SWidget> FillRotationGridSnapMenu();
	TSharedRef<SWidget> FillLayer2DSnapMenu();
	TSharedRef<SWidget> FillScaleGridSnapMenu();

	/** Grid snap setting callbacks */
	static void SetGridSize( int32 InIndex );
	static void SetRotationGridSize( int32 InIndex, ERotationGridMode InGridMode );
	static void SetScaleGridSize( int32 InIndex );
	static void SetLayer2D(int32 Layer2DIndex);

	/** Grid snap is checked callbacks for the menu values */
	static bool IsGridSizeChecked( int32 GridSizeIndex );
	static bool IsRotationGridSizeChecked( int32 GridSizeIndex, ERotationGridMode GridMode );
	static bool IsScaleGridSizeChecked( int32 GridSizeIndex );
	static bool IsLayer2DSelected(int32 Later2DIndex);

	/** Callbacks for preserving non-uniform scaling when snapping */
	static void TogglePreserveNonUniformScale();
	static bool IsPreserveNonUniformScaleChecked();

	/** Make the surface snapping toolbar checkbox button */
	TSharedRef< SWidget > MakeSurfaceSnappingButton();
	TSharedRef<SWidget> GenerateSurfaceSnappingMenu();
	FSlateColor GetSurfaceSnappingForegroundColor() const;

	/** Grid Snap checked state callbacks */
	ECheckBoxState IsLocationGridSnapChecked() const;
	ECheckBoxState IsRotationGridSnapChecked() const;
	ECheckBoxState IsLayer2DSnapChecked() const;
	ECheckBoxState IsScaleGridSnapChecked() const;

	EVisibility IsLayer2DSnapVisible() const;

	/** Grid snap toggle handlers */
	void HandleToggleLocationGridSnap(ECheckBoxState InState);
	void HandleToggleRotationGridSnap(ECheckBoxState InState);
	void HandleToggleLayer2DSnap(ECheckBoxState InState);
	void HandleToggleScaleGridSnap(ECheckBoxState InState);

private:
	TSharedPtr<SEditorViewportToolbarMenu> SurfaceSnappingMenu;

	/** The editor viewport that we are in */
	TWeakPtr<class SEditorViewport> Viewport;

	/** Command list */
	TSharedPtr<FUICommandList> CommandList;
};
