// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Animation/CurveSequence.h"
#include "CoreMinimal.h"
#include "Editor/UnrealEdTypes.h"
#include "Engine/EngineBaseTypes.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SMenuAnchor;
struct FSlateBrush;

/**
 * A level viewport toolbar widget that is placed in a viewport
 */
class SViewportToolBar : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SViewportToolBar ){}
	SLATE_END_ARGS()

	UNREALED_API void Construct( const FArguments& InArgs );

	/**
	 * @return The currently open pull down menu if there is one
	 */
	UNREALED_API TWeakPtr<SMenuAnchor> GetOpenMenu() const;

	/**
	 * Sets the open menu to a new menu and closes any currently opened one
	 *
	 * @param NewMenu The new menu that is opened
	 */
	UNREALED_API void SetOpenMenu( TSharedPtr< SMenuAnchor >& NewMenu );


	/** @return Whether the given viewmode is supported. */ 
	UNREALED_API virtual bool IsViewModeSupported(EViewModeIndex ViewModeIndex) const;

protected:
	/**
	* Returns the label for the "Camera" tool bar menu based on based viewport type
	*
	* @param ViewportType	The Viewport type we want the label for
	*
	* @return	Label to use for this menu label
	*/
	UE_DEPRECATED(5.6, "Please use UE::UnrealEd::GetCameraSubmenuLabelFromViewportType instead.")
	UNREALED_API virtual FText GetCameraMenuLabelFromViewportType(const ELevelViewportType ViewportType) const;

	/**
	* Returns the label icon for the "Camera" tool bar menu, which changes depending on the viewport type
	*
	* @param ViewportType	The Viewport type we want the icon for
	*
	* @return	Label icon to use for this menu label
	*/
	UNREALED_API virtual const FSlateBrush* GetCameraMenuLabelIconFromViewportType(const ELevelViewportType ViewportType) const;

private:
	/** The pulldown menu that is open if any */
	TWeakPtr< SMenuAnchor > OpenedMenu;
};

