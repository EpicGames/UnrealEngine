// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/SlateDelegates.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "SViewportToolBar.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SCompoundWidget.h"

class FName;
class SMenuAnchor;
struct FGeometry;
struct FPointerEvent;
struct FSlateIcon;

/**
 * Custom widget to display a toggle/drop down menu. 
 *	Displayed as shown:
 * +--------+-------------+
 * | Toggle | Menu button |
 * +--------+-------------+
 */
class SViewportToolBarComboMenu : public SCompoundWidget 
{
public:
	SLATE_BEGIN_ARGS(SViewportToolBarComboMenu) : _BlockLocation(EMultiBlockLocation::None), _MinDesiredButtonWidth(-1.0f) {}
	
		/** We need to know about the toolbar we are in */
		SLATE_ARGUMENT( TSharedPtr<class SViewportToolBar>, ParentToolBar );

		/** Content for the drop down menu  */
		SLATE_EVENT( FOnGetContent, OnGetMenuContent )

		/** Called upon state change with the value of the next state */
		SLATE_EVENT( FOnCheckStateChanged, OnCheckStateChanged )
			
		/** Sets the current checked state of the checkbox */
		SLATE_ATTRIBUTE( ECheckBoxState, IsChecked )

		/** Icon shown on the toggle button */
		SLATE_ATTRIBUTE( FSlateIcon, Icon )

		/** Label shown on the menu button */
		SLATE_ATTRIBUTE( FText, Label )

		/** Overall style */
		SLATE_ATTRIBUTE_DEPRECATED( FName, Style, 5.3, "The Style attribute is deprecated, styles are specified through stylesheets now" )

		/** ToolTip shown on the menu button */
		SLATE_ATTRIBUTE( FText, MenuButtonToolTip )

		/** ToolTip shown on the toggle button */
		SLATE_ATTRIBUTE( FText, ToggleButtonToolTip )

		/** The button location */
		SLATE_ARGUMENT_DEPRECATED( EMultiBlockLocation::Type, BlockLocation, 5.3, "The BlockLocation argument is deprecated and not in use anymore" )

		/** The minimum desired width of the menu button contents */
		SLATE_ARGUMENT( float, MinDesiredButtonWidth )

	SLATE_END_ARGS( )

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	UNREALED_API void Construct( const FArguments& InArgs );

protected:
	/**
	 * Called to query the tool tip text for this widget, but will return an empty text for toolbar items
	 * when a menu for that toolbar is already open
	 *
	 * @param	ToolTipText	Tool tip text to display, if possible
	 *
	 * @return	Tool tip text, or an empty text if filtered out
	 */
	UNREALED_API FText GetFilteredToolTipText(TAttribute<FText> ToolTipText) const;

private:
	/**
	 * Called when the menu button is clicked.  Will toggle the visibility of the menu content                   
	 */
	FReply OnMenuClicked();

	/**
	 * Called when the mouse enters a menu button.  If there was a menu previously opened
	 * we open this menu automatically
	 */
	UNREALED_API void OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent );

private:
	/** Our menus anchor */
	TSharedPtr<SMenuAnchor> MenuAnchor;

	/** Parent tool bar for querying other open menus */
	TWeakPtr<class SViewportToolBar> ParentToolBar;
};
