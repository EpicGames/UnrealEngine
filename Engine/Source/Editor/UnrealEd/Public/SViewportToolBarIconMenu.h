// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/SlateDelegates.h"
#include "Input/Reply.h"
#include "Misc/Attribute.h"
#include "SViewportToolBar.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FName;
class FText;
class SMenuAnchor;
struct FGeometry;
struct FPointerEvent;
struct FSlateIcon;

/**
 * Custom widget to display an icon/drop down menu. 
 *	Displayed as shown:
 * +--------+-------------+
 * | Icon     Menu button |
 * +--------+-------------+
 */
class SViewportToolBarIconMenu : public SCompoundWidget 
{
public:
	SLATE_BEGIN_ARGS( SViewportToolBarIconMenu ){}
	
		/** We need to know about the toolbar we are in */
		SLATE_ARGUMENT( TSharedPtr<class SViewportToolBar>, ParentToolBar );

		/** Content for the drop down menu  */
		SLATE_EVENT( FOnGetContent, OnGetMenuContent )

		/** Icon shown */
		SLATE_ATTRIBUTE( FSlateIcon, Icon )

		/** Label shown on the menu button */
		SLATE_ATTRIBUTE( FText, Label )

		/** Overall style */
		SLATE_ATTRIBUTE( FName, Style )

	SLATE_END_ARGS( )

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	UNREALED_API void Construct( const FArguments& InArgs );
	
private:
	/**
	 * Called when the menu button is clicked.  Will toggle the visibility of the menu content                   
	 */
	FReply OnMenuClicked();

	/**
	 * Called when the mouse enters a menu button.  If there was a menu previously opened
	 * we open this menu automatically
	 */
	UNREALED_API void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

private:
	/** Our menus anchor */
	TSharedPtr<SMenuAnchor> MenuAnchor;

	/** Parent tool bar for querying other open menus */
	TWeakPtr<class SViewportToolBar> ParentToolBar;
};
