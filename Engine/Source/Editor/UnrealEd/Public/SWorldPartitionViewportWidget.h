// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class IWorldPartitionEditorModule;

class SWorldPartitionViewportWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SWorldPartitionViewportWidget) {}
	SLATE_ARGUMENT(bool, Clickable)
	SLATE_END_ARGS()

	UNREALED_API void Construct(const FArguments& InArgs);
	UNREALED_API ~SWorldPartitionViewportWidget();

	UNREALED_API EVisibility GetVisibility(UWorld* InWorld);

private:
	bool bClickable;
	FText Message;
	FText Tooltip;
	FName InvokeTab;
	IWorldPartitionEditorModule* WorldPartitionEditorModule;
};
