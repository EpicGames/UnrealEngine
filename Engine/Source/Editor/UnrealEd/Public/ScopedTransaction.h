// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Delineates a transactable block; Begin()s a transaction when entering scope,
 * and End()s a transaction when leaving scope.
 */
class FScopedTransaction
{
public:
	/**
	 * Construct an FScopedTransaction with full context 
	 * @param TransactionContext		The context for this transaction, ie editor module or producing system
	 * @param SessionName				The description of this transaction, ie what is occurring
	 * @param PrimaryObject				The main object being edited (if known)
	 * @param bShouldActuallyTransact	If false the transaction is ignored. Good for creating optional transactions based on condition
	 */
	UE_NODISCARD_CTOR UNREALED_API FScopedTransaction(const TCHAR* TransactionContext, const FText& SessionName, UObject* PrimaryObject, const bool bShouldActuallyTransact = true);

	/**
	 * Construct an FScopedTransaction with minimal context 
	 * @param SessionName				The description of this transaction, ie what is occurring
	 * @param bShouldActuallyTransact	If false the transaction is ignored. Good for creating optional transactions based on condition 
	 */
	UE_NODISCARD_CTOR UNREALED_API FScopedTransaction(const FText& SessionName, const bool bShouldActuallyTransact = true);
	UNREALED_API ~FScopedTransaction();

	/**
	 * Cancels the transaction.  Reentrant.
	 */
	UNREALED_API void Cancel();

	/**
	 * @return	true if the transaction is still outstanding (that is, has not been canceled).
	 */
	UNREALED_API bool IsOutstanding() const;

private:
	/** Helper method to share construction of the transaction */
	void Construct (const TCHAR* TransactionContext, const FText& SessionName, UObject* PrimaryObject, const bool bShouldActuallyTransact = true);

	/** Stores the transaction index, which is used to cancel the transaction. */
	int32 Index;
};
