// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Elements/Framework/TypedElementList.h"
#include "Elements/Framework/TypedElementListFwd.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "Selection.generated.h"

class FArchive;
class ISelectionElementBridge;
class UTypedElementSelectionSet;
struct FTypedElementHandle;

/**
 * Manages selections of objects.
 * Used in the editor for selecting objects in the various browser windows.
 */
UCLASS(transient, MinimalAPI)
class USelection : public UObject
{
	GENERATED_BODY()

private:
	template<typename SelectionFilter>
	friend class TSelectionIterator;

public:
	static UNREALED_API USelection* CreateObjectSelection(UObject* InOuter = GetTransientPackage(), FName InName = NAME_None, EObjectFlags InFlags = RF_NoFlags);
	static UNREALED_API USelection* CreateActorSelection(UObject* InOuter = GetTransientPackage(), FName InName = NAME_None, EObjectFlags InFlags = RF_NoFlags);
	static UNREALED_API USelection* CreateComponentSelection(UObject* InOuter = GetTransientPackage(), FName InName = NAME_None, EObjectFlags InFlags = RF_NoFlags);

	/** Params: UObject* NewSelection */
	DECLARE_EVENT_OneParam(USelection, FOnSelectionChanged, UObject*);

	/** Event fired when the typed element selection set for a selection is changed */
	DECLARE_EVENT_ThreeParams(USelection, FOnSelectionElementSelectionPtrChanged, USelection* /*Selection*/, UTypedElementSelectionSet* /*OldSelectionSet*/, UTypedElementSelectionSet* /*NewSelectionSet*/);

	/** Called when selection in editor has changed */
	static UNREALED_API FOnSelectionChanged SelectionChangedEvent;
	/** Called when an object has been selected (generally an actor) */
	static UNREALED_API FOnSelectionChanged SelectObjectEvent;
	/** Called to deselect everything */
	static UNREALED_API FSimpleMulticastDelegate SelectNoneEvent;
	/** Called when the assigned typed element selection pointer set for a selection is changed */
	static UNREALED_API FOnSelectionElementSelectionPtrChanged SelectionElementSelectionPtrChanged;

	/**
	 * Set the element selection set instance for this selection set.
	 * @note Also sets the element list instance.
	 */
	UNREALED_API void SetElementSelectionSet(UTypedElementSelectionSet* InElementSelectionSet);

	/**
	 * Get the element selection set instance for this selection set, if any.
	 */
	UNREALED_API UTypedElementSelectionSet* GetElementSelectionSet() const;

	/**
	 * Returns the number of objects in the selection set.  This function is used by clients in
	 * conjunction with op::() to iterate over selected objects.  Note that some of these objects
	 * may be NULL, and so clients should use CountSelections() to get the true number of
	 * non-NULL selected objects.
	 * 
	 * @return		Number of objects in the selection set.
	 */
	UNREALED_API int32 Num() const;

	/**
	 * @return	The Index'th selected objects.  May be NULL.
	 */
	UNREALED_API UObject* GetSelectedObject(const int32 InIndex) const;

	/**
	 * Call before beginning selection operations
	 */
	UNREALED_API void BeginBatchSelectOperation();

	/**
	 * Should be called when selection operations are complete.  If all selection operations are complete, notifies all listeners
	 * that the selection has been changed.
	 */
	UNREALED_API void EndBatchSelectOperation(bool bNotify = true);

	/**
	 * @return	Returns whether or not the selection object is currently in the middle of a batch select block.
	 */
	UNREALED_API bool IsBatchSelecting() const;

	/**
	 * Selects the specified object.
	 *
	 * @param	InObject	The object to select/deselect.  Must be non-NULL.
	 */
	UNREALED_API void Select(UObject* InObject);

	/**
	 * Deselects the specified object.
	 *
	 * @param	InObject	The object to deselect.  Must be non-NULL.
	 */
	UNREALED_API void Deselect(UObject* InObject);

	/**
	 * Selects or deselects the specified object, depending on the value of the bSelect flag.
	 *
	 * @param	InObject	The object to select/deselect.  Must be non-NULL.
	 * @param	bSelect		true selects the object, false deselects.
	 */
	UNREALED_API void Select(UObject* InObject, bool bSelect);

	/**
	 * Toggles the selection state of the specified object.
	 *
	 * @param	InObject	The object to select/deselect.  Must be non-NULL.
	 */
	UNREALED_API void ToggleSelect(UObject* InObject);

	/**
	 * Deselects all objects of the specified class, if no class is specified it deselects all objects.
	 *
	 * @param	InClass		The type of object to deselect.  Can be NULL.
	 */
	UNREALED_API void DeselectAll( UClass* InClass = NULL );

	/**
	 * If batch selection is active, sets flag indicating something actually changed.
	 */
	UNREALED_API void ForceBatchDirty();

	/**
	 * Manually invoke a selection changed notification for this set.
	 */
	UNREALED_API void NoteSelectionChanged();

	/**
	 * Manually invoke a selection changed notification for no specific set.
	 * @note Legacy BSP code only!
	 */
	static UNREALED_API void NoteUnknownSelectionChanged();

	/**
	 * Returns the first selected object of the specified class.
	 *
	 * @param	InClass				The class of object to return.  Must be non-NULL.
	 * @param	RequiredInterface	[opt] Interface this class must implement to be returned.  May be NULL.
	 * @param	bArchetypesOnly		[opt] true to only return archetype objects, false otherwise
	 * @return						The first selected object of the specified class.
	 */
	UObject* GetTop(const UClass* InClass, const UClass* RequiredInterface=nullptr, bool bArchetypesOnly=false)
	{
		check( InClass );
		for( int32 i=0; i<Num(); ++i )
		{
			UObject* SelectedObject = GetSelectedObject(i);
			if (SelectedObject)
			{
				// maybe filter out non-archetypes
				if ( bArchetypesOnly && !SelectedObject->HasAnyFlags(RF_ArchetypeObject) )
				{
					continue;
				}

				if ( InClass->HasAnyClassFlags(CLASS_Interface) )
				{
					//~ Begin InClass is an Interface, and we want the top object that implements it
					if ( SelectedObject->GetClass()->ImplementsInterface(InClass) )
					{
						return SelectedObject;
					}
				}
				else if ( SelectedObject->IsA(InClass) )
				{
					//~ Begin InClass is a class, so we want the top object of that class that implements the required Interface, if specified
					if ( !RequiredInterface || SelectedObject->GetClass()->ImplementsInterface(RequiredInterface) )
					{
						return SelectedObject;
					}
				}
			}
		}
		return nullptr;
	}

	/**
	* Returns the last selected object of the specified class.
	*
	* @param	InClass		The class of object to return.  Must be non-NULL.
	* @return				The last selected object of the specified class.
	*/
	UObject* GetBottom(const UClass* InClass)
	{
		check( InClass );
		for( int32 i = Num()-1 ; i > -1 ; --i )
		{
			UObject* SelectedObject = GetSelectedObject(i);
			if( SelectedObject && SelectedObject->IsA(InClass) )
			{
				return SelectedObject;
			}
		}
		return nullptr;
	}

	/**
	 * Returns the first selected object.
	 *
	 * @return				The first selected object.
	 */
	template< class T > T* GetTop()
	{
		UObject* Selected = GetTop(T::StaticClass());
		return Selected ? CastChecked<T>(Selected) : nullptr;
	}

	/**
	* Returns the last selected object.
	*
	* @return				The last selected object.
	*/
	template< class T > T* GetBottom()
	{
		UObject* Selected = GetBottom(T::StaticClass());
		return Selected ? CastChecked<T>(Selected) : nullptr;
	}

	/**
	 * Returns true if the specified object is non-NULL and selected.
	 *
	 * @param	InObject	The object to query.  Can be NULL.
	 * @return				true if the object is selected, or false if InObject is unselected or NULL.
	 */
	UNREALED_API bool IsSelected(const UObject* InObject) const;

	/**
	 * Returns the number of selected objects of the specified type.
	 *
	 * @param	bIgnorePendingKill	specify true to count only those objects which are not pending kill (marked for garbage collection)
	 * @return						The number of objects of the specified type.
	 */
	template< class T >
	int32 CountSelections( bool bIgnorePendingKill=false )
	{
		return CountSelections(T::StaticClass(), bIgnorePendingKill);
	}

	/**
	 * Untemplated version of CountSelections.
	 */
	int32 CountSelections(UClass *ClassToCount, bool bIgnorePendingKill=false)
	{
		int32 Count = 0;
		for( int32 i=0; i<Num(); ++i )
		{
			UObject* SelectedObject = GetSelectedObject(i);
			if( SelectedObject && SelectedObject->IsA(ClassToCount) && !(bIgnorePendingKill && !IsValidChecked(SelectedObject)) )
			{
				++Count;
			}
		}
		return Count;
	}

	UNREALED_API bool IsClassSelected(UClass* Class) const;

	//~ Begin UObject Interface
	UNREALED_API virtual void Serialize(FArchive& Ar) override;
	UNREALED_API virtual bool Modify( bool bAlwaysMarkDirty=true) override;
	//~ End UObject Interface


	/**
	 * Fills in the specified array with all selected objects of the desired type.
	 * 
	 * @param	OutSelectedObjects		[out] Array to fill with selected objects of type T
	 * @return							The number of selected objects of the specified type.
	 */
	template< class T > 
	int32 GetSelectedObjects(TArray<T*> &OutSelectedObjects)
	{
		OutSelectedObjects.Empty(Num());
		for (int32 Idx = 0; Idx < Num(); ++Idx)
		{
			UObject* SelectedObject = GetSelectedObject(Idx);
			if (SelectedObject && SelectedObject->IsA(T::StaticClass()))
			{
				OutSelectedObjects.Add((T*)SelectedObject);
			}
		}
		return OutSelectedObjects.Num();
	}

	int32 GetSelectedObjects( TArray<TWeakObjectPtr<UObject>>& OutSelectedObjects )
	{
		OutSelectedObjects.Empty(Num());
		for (int32 Idx = 0; Idx < Num(); ++Idx)
		{
			UObject* SelectedObject = GetSelectedObject(Idx);
			if (SelectedObject)
			{
				OutSelectedObjects.Add(SelectedObject);
			}
		}
		return OutSelectedObjects.Num();
	}

	int32 GetSelectedObjects(UClass *FilterClass, TArray<UObject*> &OutSelectedObjects)
	{
		OutSelectedObjects.Empty(Num());
		for (int32 Idx = 0; Idx < Num(); ++Idx)
		{
			UObject* SelectedObject = GetSelectedObject(Idx);
			if (SelectedObject && SelectedObject->IsA(FilterClass))
			{
				OutSelectedObjects.Add(SelectedObject);
			}
		}
		return OutSelectedObjects.Num();
	}

private:
	/** Initializes the selection set with its typed element bridge */
	UNREALED_API void Initialize(TSharedRef<ISelectionElementBridge>&& InSelectionElementBridge);

	UNREALED_API bool IsValidObjectToSelect(const UObject* InObject) const;
	UNREALED_API UObject* GetObjectForElementHandle(const FTypedElementHandle& InElementHandle) const;

	UNREALED_API void OnElementListSyncEvent(const FTypedElementList& InElementList, FTypedElementList::FLegacySync::ESyncType InSyncType, const FTypedElementHandle& InElementHandle, bool bIsWithinBatchOperation);

	/** Bridge from UObjects to their corresponding typed elements. */
	TSharedPtr<ISelectionElementBridge> SelectionElementBridge;

	/** Underlying element selection set (if any). */
	UPROPERTY()
	TObjectPtr<UTypedElementSelectionSet> ElementSelectionSet = nullptr;

private:
	// Hide IsSelected(), as calling IsSelected() on a selection set almost always indicates
	// an error where the caller should use IsSelected(UObject* InObject).
	bool IsSelected() const
	{
		return UObject::IsSelected();
	}
};


/** A filter for generic selection sets.  Simply allows objects which are non-null */
class FGenericSelectionFilter
{
public:
	bool IsObjectValid( const UObject* InObject ) const
	{
		return InObject != nullptr;
	}
};

/**
 * Manages selections of objects.  Used in the editor for selecting
 * objects in the various browser windows.
 */
template<typename SelectionFilter>
class TSelectionIterator
{
public:
	TSelectionIterator(USelection& InSelection)
		: Selection( InSelection )
		, Filter( SelectionFilter() )
	{
		Reset();
	}

	/** Advances iterator to the next valid element in the container. */
	void operator++()
	{
		while ( true )
		{
			++Index;

			// Halt if the end of the selection set has been reached.
			if ( !IsIndexValid() )
			{
				return;
			}

			// Halt if at a valid object.
			if ( IsObjectValid() )
			{
				return;
			}
		}
	}

	/** Element access. */
	UObject* operator*() const
	{
		return GetCurrentObject();
	}

	/** Element access. */
	UObject* operator->() const
	{
		return GetCurrentObject();
	}

	/** Returns true if the iterator has not yet reached the end of the selection set. */
	explicit operator bool() const
	{
		return IsIndexValid();
	}

	/** Resets the iterator to the beginning of the selection set. */
	void Reset()
	{
		Index = -1;
		++( *this );
	}

	/** Returns an index to the current element. */
	int32 GetIndex() const
	{
		return Index;
	}

private:
	UObject* GetCurrentObject() const
	{
		return Selection.GetSelectedObject(Index);
	}

	bool IsObjectValid() const
	{
		return Filter.IsObjectValid( GetCurrentObject() );
	}

	bool IsIndexValid() const
	{
		return Index >= 0 && Index < Selection.Num();
	}

	USelection&	Selection;
	SelectionFilter Filter;
	int32			Index;
};


class FSelectionIterator : public TSelectionIterator<FGenericSelectionFilter>
{
public:
	FSelectionIterator(USelection& InSelection)
		: TSelectionIterator<FGenericSelectionFilter>( InSelection )
	{}
};

/** A filter for only iterating through editable components */
class FSelectedEditableComponentFilter
{
public:
	bool IsObjectValid(const UObject* Object) const
	{
		if (const UActorComponent* Comp = Cast<UActorComponent>( Object ))
		{
			return Comp->IsEditableWhenInherited();
		}
		return false;
	}
};

/**
 * An iterator used to iterate through selected components that are editable (i.e. not created in a blueprint)
 */
class FSelectedEditableComponentIterator : public TSelectionIterator<FSelectedEditableComponentFilter>
{
public:
	FSelectedEditableComponentIterator(USelection& InSelection)
		: TSelectionIterator<FSelectedEditableComponentFilter>(InSelection)
	{}
};

class FDeselectedActorsEvent
{
public:
	FDeselectedActorsEvent(const TArray<AActor*>& InDeselectedActors)
		: DeselectedActors(InDeselectedActors)
	{}

	UNREALED_API ~FDeselectedActorsEvent();

private:
	const TArray<AActor*>& DeselectedActors;
};
