// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/PackageWriterToSharedBuffer.h"

/**
 * Package writer to write a cooked file to a given directory with a given name.
 * BaseFilename is the filename of the package to write. The extension is ignored.
 * Then extension of each file is specified to match Unreal's loading expectations.
 */
class FDefaultCookedFilePackageWriter : public TPackageWriterToSharedBuffer<FBasePackageWriter>
{
public:
	using Super = TPackageWriterToSharedBuffer<FBasePackageWriter>;

	FDefaultCookedFilePackageWriter(const FString& InBaseFilename);

	void WritePackageData(const IPackageWriter::FPackageInfo& Info, FLargeMemoryWriter& ExportsArchive, const TArray<FFileRegion>& FileRegions) override;
	void WriteBulkData(const IPackageWriter::FBulkDataInfo& Info, const FIoBuffer& BulkData, const TArray<FFileRegion>& FileRegions) override;
	
protected:
	void CommitPackageInternal(FPackageWriterRecords::FPackage&& BaseRecord, const FCommitPackageInfo& Info) override;

	FPackageWriterRecords::FPackage* ConstructRecord() override;

private:
	FString BaseFilename;
};
