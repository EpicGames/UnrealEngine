// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IO/IoChunkId.h"
#include "Memory/CompositeBuffer.h"
#include "Memory/SharedBuffer.h"
#include "Serialization/PackageWriterToSharedBuffer.h"

/** 
 * Class containing functions to save cooked packages in separate .uasset,.uexp,.ubulk files.
 */
class FFilePackageWriterUtil
{
public:

	/** Version of the superclass's per-package record that includes our class-specific data. */
	struct FRecord : public FPackageWriterRecords::FPackage
	{
		bool bCompletedExportsArchiveForDiff = false;
	};

	struct FWritePackageParameters
	{
		FRecord& Record;
		const IPackageWriter::FCommitPackageInfo& Info;
		UE::PackageWriter::Private::ICookerInterface* Cooker = nullptr;
		TMap<FName, TRefCountPtr<FPackageHashes>>* AllPackageHashes = nullptr;
		FCriticalSection* PackageHashesLock = nullptr;
		bool bProvidePerPackageResult = false;

		UE_DEPRECATED(5.7, "Use constructor that takes only the non-null parameters, and set other parameters individually.")
		FWritePackageParameters(FRecord& InRecord,
			const IPackageWriter::FCommitPackageInfo& InInfo,
			TMap<FName, TRefCountPtr<FPackageHashes>>* InAllPackageHashes,
			FCriticalSection* InPackageHashesLock,
			bool bInProvidePerPackageResult);

		FWritePackageParameters(FRecord& InRecord,
			const IPackageWriter::FCommitPackageInfo& InInfo);
	};

	static void WritePackage(FWritePackageParameters& Parameters);

private:

	/** Buffers that are combined into the HeaderAndExports file (which is then split into .uasset + .uexp or .uoasset + .uoexp). */
	struct FExportBuffer
	{
		FSharedBuffer Buffer;
		TArray<FFileRegion> Regions;
	};

	/** Stack data for the helper functions of CommitPackageInternal. */
	struct FCommitContext
	{
		const IPackageWriter::FCommitPackageInfo& Info;
		TArray<TArray<FExportBuffer>> ExportsBuffers;
		TArray<UE::PackageWriter::Private::FWriteFileData> OutputFiles;
		UE::PackageWriter::Private::ICookerInterface* Cooker = nullptr;
	};

	static void AsyncSave(FWritePackageParameters& Parameters);
	static void CollectForSavePackageData(FRecord& Record, FCommitContext& Context);
	static void CollectForSaveBulkData(FRecord& Record, FCommitContext& Context);
	static void CollectForSaveLinkerAdditionalDataRecords(FRecord& Record, FCommitContext& Context);
	static void CollectForSaveAdditionalFileRecords(FRecord& Record, FCommitContext& Context);
	static void CollectForSaveExportsFooter(FRecord& Record, FCommitContext& Context);
	static void CollectForSaveExportsPackageTrailer(FRecord& Record, FCommitContext& Context);
	static void CollectForSaveExportsBuffers(FRecord& Record, FCommitContext& Context);
	static void AsyncSaveOutputFiles(FCommitContext& Context,
		TMap<FName,TRefCountPtr<FPackageHashes>>* AllPackageHashes, FCriticalSection* PackageHashesLock,
		bool bProvidePerPackageResult);
};
