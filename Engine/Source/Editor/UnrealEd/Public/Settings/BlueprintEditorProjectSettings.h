// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/DeveloperSettings.h"
#include "Components/ChildActorComponent.h"

#include "BlueprintEditorProjectSettings.generated.h"


UCLASS(config=Editor, meta=(DisplayName="Blueprint Project Settings"), defaultconfig, MinimalAPI)
class UBlueprintEditorProjectSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

public:
	/**
	 * [DEPRECATED] Flag to disable faster compiles for individual blueprints if they have no function signature
	 * changes.
	 * 
	 * If you still require this functionality for debugging purposes, it has moved to a console variable that can be
	 * temporarily enabled during an editor session to bypass the fast path: 'BP.bForceAllDependenciesToRecompile 1'.
	 */
	UE_DEPRECATED(5.1, "Fast path dependency compilation is now the standard behavior. As a result, this setting is no longer in use, and will eventually be removed.")
	UPROPERTY(config)
	uint8 bForceAllDependenciesToRecompile : 1;

	/** If enabled, the editor will load packages to look for soft references to actors when deleting/renaming them. This can be slow in large projects so disable this to improve performance but increase the chance of breaking blueprints/sequences that use soft actor references */
	UPROPERTY(EditAnywhere, config, Category=Actors)
	uint8 bValidateUnloadedSoftActorReferences : 1;

	/**
	 * Enable the option to expand child actor components within component tree views (experimental).
	 */
	UPROPERTY(EditAnywhere, config, Category = Experimental)
	uint8 bEnableChildActorExpansionInTreeView : 1;

	/**
	 * In Blueprint graphs, allows conversion of impure nodes to pure ones.
	 * While this allows full flexibility of graph node visualization, this could potentially introduce unwanted performance hits.
	 * Specifically, pure nodes are evaluated for each connected output. If an expensive function node were converted to pure, the user
	 * might not realize that there's now added overhead.
	 */
	UPROPERTY(EditAnywhere, config, Category = Blueprints, DisplayName = "Allow conversion of impure nodes to pure ones")
	uint8 bAllowImpureToPureNodeConversion : 1;

	/**
	 * Default view mode to use for child actor components in a Blueprint actor's component tree hierarchy (experimental).
	 */
	UPROPERTY(EditAnywhere, config, Category = Experimental, meta = (EditCondition = "bEnableChildActorExpansionInTreeView"))
	EChildActorComponentTreeViewVisualizationMode DefaultChildActorTreeViewMode;

	// A list of namespace identifiers that all Blueprint assets in the project should import by default. Requires Blueprint namespace features to be enabled in editor preferences. Editing this list will also cause any visible Blueprint editor windows to be closed.
	UPROPERTY(EditAnywhere, config, Category = Blueprints, DisplayName = "Global Namespace Imports (Shared)")
	TArray<FString> NamespacesToAlwaysInclude;

	/** 
	 * List of compiler messages that have been suppressed outside of full, interactive editor sessions for 
	 * the current project - useful for silencing warnings that were added to the engine after 
	 * project inception and are going to be addressed as they are found by content authors
	 */
	UPROPERTY(EditAnywhere, config, Category= Blueprints, DisplayName = "Compiler Messages Disabled Except in Editor")
	TArray<FName> DisabledCompilerMessagesExceptEditor;
	
	/** 
	 * List of compiler messages that have been suppressed completely - message suppression is only 
	 * advisable when using blueprints that you cannot update and are raising innocuous warnings. 
	 * If useless messages are being raised prefer to contact support rather than disabling messages
	 */
	UPROPERTY(EditAnywhere, config, Category= Blueprints, DisplayName = "Compiler Messages Disabled Entirely")
	TArray<FName> DisabledCompilerMessages;

	/**
	 * List of deprecated UProperties/UFunctions to suppress warning messages for - useful for source changes
	 * that would otherwise cause content warnings
	 * The easiest way to populate this list is using the context menu on nodes with deprecated references
	 */
	UPROPERTY(EditAnywhere, config, Category = Blueprints, DisplayName = "Deprecated Symbols to Suppress")
	TArray<FString> SuppressedDeprecationMessages;

	/**
	 * Any blueprint deriving from one of these base classes will be allowed to recompile during Play-in-Editor
	 * (This setting exists both as an editor preference and project setting, and will be allowed if listed in either place) 
	 */
	UE_DEPRECATED(5.6, "This is now the default behavior (true). If you have classes that should not compile during PIE add them to BaseClassesToDisallowRecompilingDuringPlayInEditor.")
	UPROPERTY(config)
	TArray<TSoftClassPtr<UObject>> BaseClassesToAllowRecompilingDuringPlayInEditor;
	
	/** 
	 * Any blueprint deriving from one of these base classes will be disallowed to recompile during Play-in-Editor
	 * (This setting exists both as an editor preference and project setting, and will be allowed if listed in either place) 
	 */
	UPROPERTY(EditAnywhere, config, Category=Play, meta=(AllowAbstract, ShowObjectRootClass))
	TArray<TSoftClassPtr<UObject>> BaseClassesToDisallowRecompilingDuringPlayInEditor;

	/**
	 * Disallow call-in-editor functions in Editor Utility Blueprint assets from adding buttons to the Details view when an instance of the Blueprint is selected.
	 * 
	 * @todo - Eventually "Blutility" functions are no longer to appear in the Details view by default, in favor of a more flexible solution. At that point, we can
	 * expose this option and allow users to opt the project back into the legacy path until it can be fully deprecated.
	 */
	UPROPERTY(config)
	bool bDisallowEditorUtilityBlueprintFunctionsInDetailsView;

public:
	// UObject interface
	UNREALED_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UObject interface
};
