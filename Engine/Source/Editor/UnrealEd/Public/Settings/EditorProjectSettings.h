// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/DeveloperSettings.h"
#include "Math/UnitConversion.h"
#include "LegacyScreenPercentageDriver.h"

#include "EditorProjectSettings.generated.h"

/** UENUM to define the specific set of allowable unit types */
UENUM()
enum class EUnitDisplay : uint8
{
	None,
	Metric,
	Imperial,
	Invalid
};

/** UENUM to define the specific set of allowable default units */
UENUM()
enum class EDefaultLocationUnit : uint8
{
	Micrometers,
	Millimeters,
	Centimeters,
	Meters,
	Kilometers,

	Inches,
	Feet,
	Yards,
	Miles,
		
	Invalid
};

UENUM()
enum class EReferenceViewerSettingMode : uint8
{
	// Use the editor default setting
	NoPreference,

	// Show this kind of reference by default (it can be toggled off in the reference viewer)
	ShowByDefault,

	// Hide this kind of reference by default (it can be toggled back on in the reference viewer)
	HideByDefault
};

/**
 * Editor project appearance settings. Stored in default config, per-project
 */
UCLASS(config=Editor, defaultconfig, meta=(DisplayName="Appearance"), MinimalAPI)
class UEditorProjectAppearanceSettings : public UDeveloperSettings
{
public:
	GENERATED_BODY()
	UNREALED_API UEditorProjectAppearanceSettings(const FObjectInitializer&);

protected:
	/** Called when a property on this object is changed */
	UNREALED_API virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent ) override;
	UNREALED_API virtual void PostInitProperties() override;

public:

	UPROPERTY(EditAnywhere, config, Category=Units, meta=(DisplayName="Display Units on Applicable Properties", Tooltip="Whether to display units on editor properties where the property has units set."))
	bool bDisplayUnits;

	UPROPERTY(EditAnywhere, config, Category = Units, meta = (EditCondition="bDisplayUnits", DisplayName = "Display Units on Component Transforms", Tooltip = "Whether to display units on component transform properties"))
	bool bDisplayUnitsOnComponentTransforms;

	UPROPERTY(EditAnywhere, config, Category=Units, AdvancedDisplay, meta=(DisplayName="Distance/Length", Tooltip="Choose a set of units in which to display distance/length values."))
	TArray<EUnit> DistanceUnits;

	UPROPERTY(EditAnywhere, config, Category=Units, AdvancedDisplay, meta=(DisplayName="Mass", Tooltip="Choose a set of units in which to display masses."))
	TArray<EUnit> MassUnits;

	UPROPERTY(EditAnywhere, config, Category=Units, AdvancedDisplay, meta=(DisplayName="Time", Tooltip="Choose the units in which to display time."))
	TArray<EUnit> TimeUnits;
	
	UPROPERTY(EditAnywhere, config, Category=Units, AdvancedDisplay, meta=(DisplayName="Angles", Tooltip="Choose the units in which to display angles.", ValidEnumValues="Degrees, Radians"))
	EUnit AngleUnits;

	UPROPERTY(EditAnywhere, config, Category=Units, AdvancedDisplay, meta=(DisplayName="Speed/Velocity", Tooltip="Choose the units in which to display speeds and velocities.", ValidEnumValues="CentimetersPerSecond, MetersPerSecond, KilometersPerHour, MilesPerHour"))
	EUnit SpeedUnits;

	UPROPERTY(EditAnywhere, config, Category=Units, AdvancedDisplay, meta=(DisplayName="Angular Speed", Tooltip="Choose the units in which to display angular speeds.", ValidEnumValues="DegreesPerSecond, RadiansPerSecond"))
	EUnit AngularSpeedUnits;
	
	UPROPERTY(EditAnywhere, config, Category=Units, AdvancedDisplay, meta=(DisplayName="Acceleration", Tooltip="Choose the units in which to display acceleration.", ValidEnumValues="CentimetersPerSecondSquared, MetersPerSecondSquared"))
	EUnit AccelerationUnits;

	UPROPERTY(EditAnywhere, config, Category=Units, AdvancedDisplay, meta=(DisplayName="Temperature", Tooltip="Choose the units in which to display temperatures.", ValidEnumValues="Celsius, Farenheit, Kelvin"))
	EUnit TemperatureUnits;

	UPROPERTY(EditAnywhere, config, Category=Units, AdvancedDisplay, meta=(DisplayName="Force", Tooltip="Choose the units in which to display forces.", ValidEnumValues="Newtons, PoundsForce, KilogramsForce, KilogramCentimetersPerSecondSquared"))
	EUnit ForceUnits;

	UPROPERTY(EditAnywhere, config, Category = Units, AdvancedDisplay, meta = (DisplayName = "Torque", Tooltip = "Choose the units in which to display torques.", ValidEnumValues = "NewtonMeters, KilogramCentimetersSquaredPerSecondSquared"))
	EUnit TorqueUnits;

	UPROPERTY(EditAnywhere, config, Category = Units, AdvancedDisplay, meta = (DisplayName = "Impulse", Tooltip = "Choose the units in which to display impulses.", ValidEnumValues = "NewtonSeconds"))
	EUnit ImpulseUnits;

	UPROPERTY(EditAnywhere, config, Category = Units, AdvancedDisplay, meta = (DisplayName = "PositionalImpulse", Tooltip = "Choose the units in which to display positional impulses.", ValidEnumValues = "KilogramMeters, KilogramCentimeters"))
	EUnit PositionalImpulseUnits;

	// Should the Reference Viewer have 'Show Searchable Names' checked by default when opened in this project
	UPROPERTY(EditAnywhere, config, Category=ReferenceViewer)
	EReferenceViewerSettingMode ShowSearchableNames;

	// The default maximum search breadth for the reference viewer when opened
	UPROPERTY(EditAnywhere, config, Category = ReferenceViewer, meta=(DisplayName="Default Max Search Breadth", ClampMin=1, ClampMax=1000, UIMin=1, UIMax=50))
	int32 ReferenceViewerDefaultMaxSearchBreadth = 20;

public:
	/** Deprecated properties that didn't live very long */

	UPROPERTY(config)
	EUnitDisplay UnitDisplay_DEPRECATED;

	UPROPERTY(config)
	EDefaultLocationUnit DefaultInputUnits_DEPRECATED;
};
/**
* 2D layer settings
*/
USTRUCT()
struct FMode2DLayer
{
	GENERATED_USTRUCT_BODY()

	FMode2DLayer()
	: Name(TEXT("Default"))
	, Depth(0)
	{ }

	FMode2DLayer(FString InName, float InDepth)
		: Name(InName)
		, Depth(InDepth)
	{ }

	/** A descriptive name for this snap layer. */
	UPROPERTY(EditAnywhere, config, Category = Layer)
	FString Name;

	/** The position of this snap layer's plane along the Snap Axis. */
	UPROPERTY(EditAnywhere, config, Category = Layer)
	float Depth;
};

UENUM()
enum class ELevelEditor2DAxis : uint8
{
	X,
	Y,
	Z
};

/**
 * Configure settings for the 2D Level Editor
 */
UCLASS(config=Editor, meta=(DisplayName="2D"), defaultconfig, MinimalAPI)
class ULevelEditor2DSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

public:
	/** If enabled, shows the 2D combined translate and rotate tool in the viewport toolbar. */
	UPROPERTY(EditAnywhere, config, Category=General, meta=(DisplayName="Enable 2D combined translate + rotate widget"))
	bool bEnable2DWidget;

	/** If enabled, shows the 2D layer snapping controls in the viewport toolbar. */
	UPROPERTY(EditAnywhere, config, Category=LayerSnapping)
	bool bEnableSnapLayers;

	/** Sets the world space axis for 2D snap layers. */
	UPROPERTY(EditAnywhere, config, Category=LayerSnapping, meta=(EditCondition=bEnableSnapLayers))
	ELevelEditor2DAxis SnapAxis;

	/** Snap layers that are displayed in the viewport toolbar. */
	UPROPERTY(EditAnywhere, config, Category=LayerSnapping, meta=(EditCondition=bEnableSnapLayers))
	TArray<FMode2DLayer> SnapLayers;

public:
	// UObject interface
	UNREALED_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UObject interface

};

/**
 * Configure per-project settings for the Level Editor
 */
UCLASS(config=Editor, meta=(DisplayName="Level Editor"), defaultconfig, MinimalAPI)
class ULevelEditorProjectSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, config, Category=Editing, meta=(
		DisplayName="Enable viewport static mesh instance selection",
		ConsoleVariable="TypedElements.EnableViewportSMInstanceSelection"))
	bool bEnableViewportSMInstanceSelection;

public:
	UNREALED_API ULevelEditorProjectSettings(const class FObjectInitializer& ObjectInitializer);
	// UObject interface
	UNREALED_API virtual void PostInitProperties() override;
	UNREALED_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UObject interface
};

/**
 * Configure per-project performance settings for the Editor
 */
UCLASS(config=Editor, meta=(DisplayName="Performance"), defaultconfig, MinimalAPI)
class UEditorPerformanceProjectSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

public:
	
	UPROPERTY(EditAnywhere, config, Category=ViewportResolution, meta=(
		DisplayName="Default screen percentage mode for realtime editor viewports using desktop renderer."))
	EScreenPercentageMode RealtimeScreenPercentageMode;
	
	UPROPERTY(EditAnywhere, config, Category=ViewportResolution, meta=(
		DisplayName="Default screen percentage mode for realtime editor viewports using mobile renderer."))
	EScreenPercentageMode MobileScreenPercentageMode;
	
	UPROPERTY(EditAnywhere, config, Category=ViewportResolution, meta=(
		DisplayName="Default screen percentage mode for VR editor viewports."))
	EScreenPercentageMode VRScreenPercentageMode;
	
	UPROPERTY(EditAnywhere, config, Category=ViewportResolution, meta=(
		DisplayName="Default screen percentage mode for path traced editor viewports."))
	EScreenPercentageMode PathTracerScreenPercentageMode;

	UPROPERTY(EditAnywhere, config, Category=ViewportResolution, meta=(
		DisplayName="Default screen percentage mode for non-realtime editor viewports."))
	EScreenPercentageMode NonRealtimeScreenPercentageMode;

	UPROPERTY(EditAnywhere, config, Category=ViewportResolution, meta=(
		EditCondition="RealtimeScreenPercentageMode == EScreenPercentageMode::Manual || NonRealtimeScreenPercentageMode == EScreenPercentageMode::Manual",
		DisplayName="Manual screen percentage to be set by default for editor viewports."))
	float ManualScreenPercentage;

	UPROPERTY(EditAnywhere, config, Category=ViewportResolution, meta=(
		DisplayName="Minimum default rendering resolution to use for editor viewports."))
	int32 MinViewportRenderingResolution;

	UPROPERTY(EditAnywhere, config, Category=ViewportResolution, meta=(
		DisplayName="Maximum default rendering resolution to use for editor viewports."))
	int32 MaxViewportRenderingResolution;


	static UNREALED_API void ExportResolutionValuesToConsoleVariables();

public:
	// UObject interface
	UNREALED_API virtual void PostInitProperties() override;
	UNREALED_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UObject interface

};

/**
 * Settings for how developers interact with assets. Stored in default config, per-project
 */
UCLASS(config = Editor, defaultconfig, meta = (DisplayName = "Assets"), MinimalAPI)
class UEditorProjectAssetSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	/**
	 * When performing cleanup operations on redirectors (such as resaving their referencers),
	 * prompt the user to delete unreferenced redirectors.
	 */
	UPROPERTY(EditAnywhere, config, Category = Redirectors)
	bool bPromptToDeleteUnreferencedRedirectors = true;
	
	/**
	 * Manage (rename/move/delete) localized variants alongside their source asset
	 */
	UPROPERTY(EditAnywhere, DisplayName = "Manage Localized Variants Alongside Source Asset", config, Category = Internationalization, meta = (ToolTip = "If the project uses Localization, localized variants of assets will be moved/renamed/deleted alongside their source asset."))
	bool bRenameLocalizedVariantsAlongsideSourceAsset = true;
};

UCLASS(config = Editor, meta = (DisplayName = "Derived Data"), defaultconfig, MinimalAPI)
class UDDCProjectSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UDDCProjectSettings() { }

	/**
	 *
	 */
	UPROPERTY(EditAnywhere, config, Category = Warnings)
	bool EnableWarnings = true;

	/**
	 * 
	 */
	UPROPERTY(EditAnywhere, config, Category= Warnings)
	bool RecommendEveryoneSetupAGlobalLocalDDCPath=false;

	/**
	 * 
	 */
	UPROPERTY(EditAnywhere, config, Category= Warnings)
	bool RecommendEveryoneSetupAGlobalSharedDDCPath=false;

	/**
	 * 
	 */
	UPROPERTY(EditAnywhere, config, Category= Warnings)
	bool RecommendEveryoneSetupAGlobalS3DDCPath = false;

	/**
	 *
	 */
	UPROPERTY(EditAnywhere, config, Category = Warnings)
	bool RecommendEveryoneEnableS3DDC = false;

	/**
	 *
	 */
	UPROPERTY(EditAnywhere, config, Category = Warnings)
	bool RecommendEveryoneUseUnrealCloudDDC = false;
};
