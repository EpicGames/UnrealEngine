// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "Misc/DateTime.h"
#include "Misc/Guid.h"
#include "Scalability.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UObjectGlobals.h"

#include "EditorSettings.generated.h"

class FProperty;


USTRUCT()
struct FRecentProjectFile
{
	GENERATED_BODY()

	/** Path to the project */
	UPROPERTY(config)
	FString ProjectName;

	/** Timestamp of the last time the editor opened this project */
	UPROPERTY(config)
	FDateTime LastOpenTime;

	FRecentProjectFile()
	{}

	FRecentProjectFile(const FString& InProjectName, FDateTime InLastOpenTime)
		: ProjectName(InProjectName)
		, LastOpenTime(InLastOpenTime)
	{}

	bool operator==(const FRecentProjectFile& Other) const
	{
		return ProjectName == Other.ProjectName;
	}

	bool operator==(const FString& OtherProjectName) const
	{
		return ProjectName == OtherProjectName;
	}
};

DECLARE_MULTICAST_DELEGATE_OneParam(FOnMostRecentProjectSettingChanged, bool /** bInShouldUseMostRecentProject */)

UCLASS(config=EditorSettings, MinimalAPI)
class UEditorSettings : public UObject
{
	GENERATED_UCLASS_BODY()

	// Derived Data Cache Settings
	// =====================================================================

	/**
	 * Adjusts the Local Cache location. This affects every project and workspace used from your user profile that accepts the UE-LocalDataCachePath environment variable override.
	 * This is usually the first location to query for previously built data.
	 */
	UPROPERTY(EditAnywhere, Category = DerivedDataCache, meta = (DisplayName = "Local DDC Path", ConfigRestartRequired = true))
	FDirectoryPath GlobalLocalDDCPath;

	/**
	 * Adjusts the Shared Cache location. This affects every project and workspace used from your user profile that accepts the UE-SharedDataCachePath environment variable override.
	 * The Shared Cache location is usually queried if we don't find previously built data in the Local cache. Colleagues should point to the same shared location so that work can be distributed. 
	 */
	UPROPERTY(EditAnywhere, Category = DerivedDataCache, meta = (DisplayName = "Shared DDC Path", ConfigRestartRequired = true))
	FDirectoryPath GlobalSharedDDCPath;

	/**
	 * Project specific override for the Local Cache location. The editor must be restarted for changes to take effect.
	 * This will override the 'Local DDC Path'.
	 */
	UPROPERTY(EditAnywhere, config, Category= DerivedDataCache, AdvancedDisplay, meta = (DisplayName = "Project Local DDC Path", ConfigRestartRequired = true))
	FDirectoryPath LocalDerivedDataCache;

	/**
	 * Project specific override for the Shared Cache location. The editor must be restarted for changes to take effect.
	 * This will override the 'Shared DDC Path'.
	 */
	UPROPERTY(EditAnywhere, config, Category= DerivedDataCache, AdvancedDisplay, meta = (DisplayName = "Project Shared DDC Path", ConfigRestartRequired = true))
	FDirectoryPath SharedDerivedDataCache;

	/** Whether to enable any DDC Notifications */
	UPROPERTY(EditAnywhere, config, Category = "Derived Data Cache Notifications", meta = (DisplayName = "Enable Notifications", ConfigRestartRequired = false))
	bool bEnableDDCNotifications = true;

	/** Whether to enable the Unreal Cloud DDC notification */
	UPROPERTY(EditAnywhere, config, Category = "Derived Data Cache Notifications", meta = (DisplayName = "Notify Use Unreal Cloud DDC", ConfigRestartRequired = false, EditCondition = "bEnableDDCNotifications"))
	bool bNotifyUseUnrealCloudDDC = true;

	/** Whether to enable the DDC path notification */
	UPROPERTY(EditAnywhere, config, Category = "Derived Data Cache Notifications", meta = (DisplayName = "Notify Setup DDC Path", ConfigRestartRequired = false, EditCondition = "bEnableDDCNotifications"))
	bool bNotifySetupDDCPath = true;

	/** Whether to enable the DDC path notification */
	UPROPERTY(EditAnywhere, config, Category = "Derived Data Cache Notifications", meta = (DisplayName = "Notify Enable S3 DDC", ConfigRestartRequired = false, EditCondition = "bEnableDDCNotifications"))
	bool bNotifyEnableS3DD = true;

	/** Whether to enable the S3 derived data cache backend */
	UPROPERTY(EditAnywhere, config, Category="Derived Data Cache S3", meta = (DisplayName = "Enable AWS S3 Cache", ConfigRestartRequired = true))
	bool bEnableS3DDC = true;

	/**
	 * Adjusts the Local Cache location for AWS/S3 downloaded package bundles.
	 * This affects every project and workspace used from your user profile that accepts the UE-S3DataCachePath environment variable override.
	 */
	UPROPERTY(EditAnywhere, Category="Derived Data Cache S3", meta = (DisplayName = "Local S3DDC Path", ConfigRestartRequired = true, EditCondition = "bEnableS3DDC"))
	FDirectoryPath GlobalS3DDCPath;

	// Horde Settings
	// =====================================================================

	/** URL of the Horde server */
	UPROPERTY(EditAnywhere, Category = "Horde", meta = (DisplayName = "Horde Server URL", ConfigRestartRequired = true))
	FString HordeUrl;

	// =====================================================================

	UE_DEPRECATED(5.7, "Use Load At Startup setting")
	UPROPERTY(meta = (DeprecatedProperty, DeprecatedMessage="Use Load At Startup setting"))
	bool bLoadTheMostRecentlyLoadedProjectAtStartup; // Note that this property is NOT config since it is not necessary to save the value to ini. It is determined at startup in UEditorEngine::InitEditor().
	FOnMostRecentProjectSettingChanged& OnMostRecentProjectSettingChanged() { return OnMostRecentProjectSettingChangedDelegate; }

	// =====================================================================
	// The following options are NOT exposed in the preferences Editor
	// (usually because there is a different way to set them interactively!)

	/** Game project files that were recently opened in the editor */
	UPROPERTY(config)
	TArray<FRecentProjectFile> RecentlyOpenedProjectFiles;

	/** The paths of projects created with the new project wizard. This is used to populate the "Path" field of the new project dialog. */
	UPROPERTY(config)
	TArray<FString> CreatedProjectPaths;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.6, "Ability to add Starter Content has been removed")
	UPROPERTY(config, meta = (DeprecatedProperty, DeprecationMessage = "Ability to add Starter Content has been removed"))
	bool bCopyStarterContentPreference_DEPRECATED = false;
#endif // WITH_EDITORONLY_DATA

	/** The id's of the surveys completed */
	UPROPERTY(config)
	TArray<FGuid> CompletedSurveys;

	/** The id's of the surveys currently in-progress */
	UPROPERTY(config)
	TArray<FGuid> InProgressSurveys;

	UPROPERTY(config)
	float AutoScalabilityWorkScaleAmount;

	/** Engine scalability benchmark results */
	Scalability::FQualityLevels EngineBenchmarkResult;

	/** Load the engine scalability benchmark results. Performs a benchmark if not yet valid. */
	UNREALED_API void LoadScalabilityBenchmark();

	/** Auto detects and applies the scalability benchmark */
	UNREALED_API void AutoApplyScalabilityBenchmark();

	/** @return true if the scalability benchmark is valid */
	UNREALED_API bool IsScalabilityBenchmarkValid() const;

	//~ Begin UObject Interface
	UNREALED_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	UNREALED_API virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface

private:
	FOnMostRecentProjectSettingChanged OnMostRecentProjectSettingChangedDelegate;
};
