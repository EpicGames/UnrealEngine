// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorViewportSettings.generated.h"

USTRUCT(MinimalAPI)
struct FEditorViewportCameraSpeedSettings
{
	GENERATED_BODY()
	
	FEditorViewportCameraSpeedSettings() {}
	UNREALED_API FEditorViewportCameraSpeedSettings(float Speed);
	UNREALED_API FEditorViewportCameraSpeedSettings(float Speed, float Min, float Max);
	
public:
	/** Provides a speed range where the min & max allow for the specified UI Min and Max. */
	UNREALED_API static FEditorViewportCameraSpeedSettings FromUIRange(float Speed, float UIMin, float UIMax);

	UNREALED_API float GetCurrentSpeed() const;
	UNREALED_API void SetCurrentSpeed(float NewSpeed);
	
	UNREALED_API float GetAbsoluteMinSpeed() const;
	UNREALED_API float GetAbsoluteMaxSpeed() const;
	
	UNREALED_API float GetMinUISpeed() const;
	UNREALED_API float GetMaxUISpeed() const;
	
	/**
	 * Sets the minimum & maximum speed.
	 * Adjusts the current speed & UI speed range as necessary.
	 */
	UNREALED_API void SetAbsoluteSpeedRange(float NewMinSpeed, float NewMaxSpeed);
	
	/**
     * Sets the minimum & maximum UI speed.
     * Constrained by the current Min & Max.
     */
	UNREALED_API void SetUISpeedRange(float NewMinSpeed, float NewMaxSpeed);
	
	/** Get the camera speed relative to the min & max, ranging from 0..1 */
	UNREALED_API float GetRelativeSpeed() const;
	
	/** Set the camera speed with a 0..1 value relative to the min & max */
	UNREALED_API void SetRelativeSpeed(float RelativeSpeed);
	
	/** Get the camera speed relative to the min & max, ranging from 0..1 */
	UNREALED_API float GetRelativeUISpeed() const;
	
	/** Set the camera speed with a 0..1 value relative to the min & max */
	UNREALED_API void SetRelativeUISpeed(float RelativeUISpeed);
	
	/** Get the threshold used to warn the user about low camera speed */
	UNREALED_API float GetLowSpeedWarningThreshold() const;
	
private:
	/** The speed of the camera */
	UPROPERTY(Meta = (ClampMin = 0.0001f))
	float CurrentSpeed = 1.0f;
	
	/** The slowest possible speed of the camera */
	UPROPERTY(EditAnywhere, Category = Camera, Meta = (ClampMin = 0.0001f))
	float AbsoluteMinSpeed = 0.00001f;
	
	/** The fastest possible speed of the  camera */
	UPROPERTY(EditAnywhere, Category = Camera, Meta = (ClampMin = 1.0f))
	float AbsoluteMaxSpeed = 10000.0f;
	
	/** The minimum speed of the camera when dragging with a slider */
	UPROPERTY(EditAnywhere, Category = Camera, DisplayName="Min Slider Speed", Meta = (ClampMin = 0.0001f))
	float MinUISpeed = 0.33f;
	
	/** The maximum speed of the camera when dragging with a slider */
	UPROPERTY(EditAnywhere, Category = Camera, DisplayName="Max Slider Speed", Meta = (ClampMin = 0.0001f))
	float MaxUISpeed = 32.0f;

	/** A threshold used to warn the user about low camera speed */
	UPROPERTY()
	float LowSpeedWarningThreshold = 0.01f;
};
