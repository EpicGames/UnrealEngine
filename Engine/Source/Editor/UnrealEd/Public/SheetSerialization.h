// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editor/EditorSubsystem/Public/EditorSubsystem.h"

#include "SheetSerialization.generated.h"

/*
 * The goal of these classes is to enable copying & pasting between Excel/Sheets and DataTables by flattening the row structure.
 * Any property type can be customized, but by default:
 * - Structs are expanded horizontally per property.
 * - Arrays are expanded vertically with an added index column.
 * - Sets are expanded vertically with an added index column.
 * - Maps are expanded vertically with separate key and value columns.
 * 
 * As an example, imagine a DataTable with 3 rows based on a struct with the following variables:
 *	Description		String (Multi-line)		Single
 *	Level			Integer					Single
 *	SpawnLocs		Vector					Array
 *	Names			Name					Array
 *	Shouts			String					Set
 *	SurnamePerMap	String/Name				Map
 *
 * Here's an ASCII representation of what that DataTable would look like in spreadsheet form:
_________________________________________________________________________________________________________________________________________________
|           |This is the main one.      |   |   |           |           |           |   |       |   |                       |           |       |
|Primary____|It's_pretty_cool.__________|15_|0__|1.726383___|166.280151_|0.000000___|0__|Alex___|0__|Hello,_there,_"friend"!|ThreeFort__|Tanner_|
|___________|___________________________|___|___|___________|___________|___________|1__|Chris__|1__|I'm_here!______________|Canyon_____|Smith__|
|___________|___________________________|___|___|___________|___________|___________|2__|Riley__|2__|Look_at_me!____________|___________|_______|
|           |This isn't the main one.   |   |   |           |           |           |   |       |   |                       |           |       |
|Secondary__|It's_still_pretty_cool.____|25_|0__|158.941282_|-6.327449__|0.536000___|0__|Drew___|0__|Well,_then!____________|Desert_____|Cooper_|
|___________|___________________________|___|1__|9.060441___|41.501978__|-10.015190_|1__|Jamie__|2__|I_like_games!__________|Plains_____|King___|
|___________|___________________________|___|2__|-126.515899|1566.867684|5.362432___|2__|Pat____|___|_______________________|LavaFields_|Baker__|
|___________|___________________________|___|___|___________|___________|___________|3__|Reese__|___|_______________________|___________|_______|
|Tertiary___|I guess this is fine.______|40_|0__|1091.647622|-160.268072|321.305607_|0__|Jordan_|0__|I'm spawnin' here!_____|Crypt______|Mason__|
|___________|___________________________|___|1__|1.145088___|9.301404___|0.000000___|1__|Morgan_|1__|Don't look at me!______|___________|_______|
|___________|___________________________|___|___|___________|___________|___________|2__|Quinn__|2__|Why am I here?_________|___________|_______|
|___________|___________________________|___|___|___________|___________|___________|3__|Sam____|___|_______________________|___________|_______|
 * 
 * Here's a JSON export of the same DataTable:
 * [
	{
		"Name": "Primary",
		"Description": "This is the main one.\r\nIt's pretty cool.",
		"Level": 15,
		"SpawnLocs": [
			{
				"X": 1.726383,
				"Y": 166.28015099999999,
				"Z": 0
			}
		],
		"Names": [
			"Alex",
			"Chris",
			"Riley"
		],
		"Shouts": [
			"Hello, there, \"friend\"!",
			"I'm here!",
			"Look at me!"
		],
		"SurnamePerMap":
		{
			"ThreeFort":
			{
				"SurnamePerMap_31": "Tanner"
			},
			"Canyon":
			{
				"SurnamePerMap_31": "Smith"
			}
		}
	},
	{
		"Name": "Secondary",
		"Description": "This isn't the main one.\r\nIt's still pretty cool.",
		"Level": 25,
		"SpawnLocs": [
			{
				"X": 158.941282,
				"Y": -6.3274489999999997,
				"Z": 0.53600000000000003
			},
			{
				"X": 9.0604410000000009,
				"Y": 41.501978000000001,
				"Z": -10.01519
			},
			{
				"X": -126.515899,
				"Y": 1566.8676840000001,
				"Z": 5.3624320000000001
			}
		],
		"Names": [
			"Drew",
			"Jamie",
			"Pat",
			"Reese"
		],
		"Shouts": [
			"Well, then!",
			"I like games!"
		],
		"SurnamePerMap":
		{
			"Desert":
			{
				"SurnamePerMap_31": "Cooper"
			},
			"Plains":
			{
				"SurnamePerMap_31": "King"
			},
			"LavaFields":
			{
				"SurnamePerMap_31": "Baker"
			}
		}
	},
	{
		"Name": "Tertiary",
		"Description": "I guess this is fine.",
		"Level": 40,
		"SpawnLocs": [
			{
				"X": 1091.647622,
				"Y": -160.26807199999999,
				"Z": 321.30560700000001
			},
			{
				"X": 1.1450880000000001,
				"Y": 9.3014039999999998,
				"Z": 0
			}
		],
		"Names": [
			"Jordan",
			"Morgan",
			"Quinn",
			"Sam"
		],
		"Shouts": [
			"I'm spawnin' here!",
			"Don't look at me!",
			"Why am I here?"
		],
		"SurnamePerMap":
		{
			"Crypt":
			{
				"SurnamePerMap_31": "Mason"
			}
		}
	}
]
 */

class UDataTable;

// Spreadsheet Data
struct FSheetData
{
	TArray<FString> Cells;
	int32 ColumnCount = 0;
	TMap<const FProperty*, int32> ColumnsPerProperty;

	UNREALED_API int32 GetRowCount() const;

	// Gets the string representing the cell at the given row and column, adding cells if necessary to do so
	UNREALED_API FString& GetStringForCell(const int32 Column, const int32 Row);

	// Get the number of rows used by an object in the sheet data at the given starting column and row
	UNREALED_API int32 GetObjectRowCount(const int32 StartingColumn, const int32 StartingRow, const int32 MaxRows);

	// Fill the sheet data based on data copied from a spreadsheet
	UNREALED_API bool FillFromSheet(const FString& InSheetText, FOutputDevice* ErrorText);
};

// Information required to operate on a row from a DataTable
struct FDataTableRowData
{
	FName Name;
	uint8* DataPtr;
};

// Delegate for filling the sheet data with headers for the given property based on the default property name built up from its parentage, returning the number of columns used
using FFillHeadersFromProperty = TDelegate<int32(FSheetData& SheetData, const FProperty* InProperty, const FString& DefaultPropName)>;

// Delegate for filling the sheet data with cell values for the given property starting at the specified starting position, returning the number of rows used
using FFillCellsFromProperty = TDelegate<int32(FSheetData& SheetData, const FIntPoint StartingPosition, void* InObject, UObject* Parent, const FProperty* InProperty)>;

// Delegate for setting the value of the given property based on cells from the sheet data at the specified starting position
using FSetPropertyFromCells = TDelegate<void(FSheetData& SheetData, const FIntPoint StartingPosition, int32 ObjectRowCount, void* InObject, UObject* Parent, const FProperty* InProperty)>;

//////////////////////////////////////////////////////////////////////////
// USheetSerializationSubSystem

/*
 * Used to serialize data for copying/pasting to/from spreadsheets (in Excel or Sheets).
 * Properties can be customized to be processed in a specific fashion.
 */
UCLASS(MinimalAPI)
class USheetSerializationSubSystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	// Gets the singleton
	static UNREALED_API USheetSerializationSubSystem* Get();

	UNREALED_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UNREALED_API virtual void Deinitialize() override;

	// Register the function to get column headers from the given property class
	UNREALED_API virtual void RegisterHeaderGetter(const FString& PropertyClassName, const FFillHeadersFromProperty& GetHeaderFromProperty);

	// Register the function to get cell values from the given property class
	UNREALED_API virtual void RegisterCellGetter(const FString& PropertyClassName, const FFillCellsFromProperty& GetCellsFromProperty);

	// Register the function to set the value of the given property class from cell values
	UNREALED_API virtual void RegisterPropertySetter(const FString& PropertyClassName, const FSetPropertyFromCells& SetPropertyFromCells);

	// Unregister the function to get column headers from the given property class
	UNREALED_API virtual void UnregisterHeaderGetter(const FString& PropertyClassName);

	// Unregister the function to get cell values from the given property class
	UNREALED_API virtual void UnregisterCellGetter(const FString& PropertyClassName);

	// Unregister the function to set the value of the given property class from cell values
	UNREALED_API virtual void UnregisterPropertySetter(const FString& PropertyClassName);


	// Serialize the specified rows from the given data table into a format able to be pasted into a spreadsheet, optionally including headers
	UNREALED_API FString ExportForSheet(TArray<FDataTableRowData>& Rows, UDataTable* DataTable, const bool IncludeHeaders = false);

	// Deserialize clipboard text from a spreadsheet into the given data table, replacing all of its contents with that from the spreadsheet
	UNREALED_API bool ImportFromSheet(const FString& SheetText, UDataTable* DataTable, FOutputDevice* ErrorText);

	// Deserialize clipboard text from a spreadsheet into a single row of the given data table, replacing the row's contents with that from the spreadsheet
	// Does not change the row's name, but outputs the new desired row name so that it may be changed separately.
	UNREALED_API bool ImportRowFromSheet(const FString& SheetText, UDataTable* DataTable, FName& OutRowName, uint8* RowPtr, FOutputDevice* ErrorText);


	// Escape the cell data for proper spreadsheet formatting
	static UNREALED_API FString& EscapeIfNecessary(FString& CellData, bool* OutModified = nullptr);

	// Unescape the cell data coming from a spreadsheet
	static UNREALED_API FString& UnEscapeIfNecessary(FString& CellData, bool* OutModified = nullptr);


	// Fill headers into the sheet data for the given struct, returning the number of columns used
	UNREALED_API int32 FillHeadersFromStruct(FSheetData& SheetData, const UStruct* InStruct, const FString* ParentName = nullptr);

	// Fill headers into the sheet data for the given property, returning the number of columns used
	UNREALED_API int32 FillHeadersFromProperty(FSheetData& SheetData, const FProperty* InProperty, const FProperty* ParentProperty, const FString* ParentName = nullptr);


	// Fill cell values into the sheet data for the given object, returning the number of rows used
	UNREALED_API int32 FillCellsFromObject(FSheetData& SheetData, const FIntPoint StartingPosition, void* InObject, UObject* Parent, const UStruct* RowStruct);

	// Fill cell values into the sheet data for the given property, returning the number of rows used
	UNREALED_API int32 FillCellsFromProperty(FSheetData& SheetData, const FIntPoint StartingPosition, void* InObject, UObject* Parent, const FProperty* InProperty);


	// Sets the value for the object based on cells from the sheet data at the given starting position
	UNREALED_API void SetObjectFromCells(FSheetData& SheetData, const FIntPoint StartingPosition, int32 ObjectRowCount, void* InObject, UObject* Parent, const UStruct* RowStruct);

	// Sets the value for the property of the object based on cells from the sheet data at the given starting position
	UNREALED_API void SetPropertyFromCells(FSheetData& SheetData, const FIntPoint StartingPosition, int32 ObjectRowCount, void* InObject, UObject* Parent, const FProperty* InProperty);

private:
	// Fill headers into the sheet data for the given FStructProperty, returning the number of columns used
	int32 FillHeadersFromStructProperty(FSheetData& SheetData, const FProperty* InProperty, const FString& DefaultPropName);

	// Fill headers into the sheet data for the given FArrayProperty, returning the number of columns used
	int32 FillHeadersFromArrayProperty(FSheetData& SheetData, const FProperty* InProperty, const FString& DefaultPropName);

	// Fill headers into the sheet data for the given FSetProperty, returning the number of columns used
	int32 FillHeadersFromSetProperty(FSheetData& SheetData, const FProperty* InProperty, const FString& DefaultPropName);

	// Fill headers into the sheet data for the given FMapProperty, returning the number of columns used
	int32 FillHeadersFromMapProperty(FSheetData& SheetData, const FProperty* InProperty, const FString& DefaultPropName);


	// Fill cell values into the sheet data for the given FStructProperty, returning the number of rows used
	int32 FillCellsFromStructProperty(FSheetData& SheetData, const FIntPoint StartingPosition, void* InObject, UObject* Parent, const FProperty* InProperty);

	// Fill cell values into the sheet data for the given FArrayProperty, returning the number of rows used
	int32 FillCellsFromArrayProperty(FSheetData& SheetData, const FIntPoint StartingPosition, void* InObject, UObject* Parent, const FProperty* InProperty);

	// Fill cell values into the sheet data for the given FSetProperty, returning the number of rows used
	int32 FillCellsFromSetProperty(FSheetData& SheetData, const FIntPoint StartingPosition, void* InObject, UObject* Parent, const FProperty* InProperty);

	// Fill cell values into the sheet data for the given FMapProperty, returning the number of rows used
	int32 FillCellsFromMapProperty(FSheetData& SheetData, const FIntPoint StartingPosition, void* InObject, UObject* Parent, const FProperty* InProperty);

	// Fill cell values into the sheet data for the given FStrProperty, returning the number of rows used
	int32 FillCellsFromStringProperty(FSheetData& SheetData, const FIntPoint StartingPosition, void* InObject, UObject* Parent, const FProperty* InProperty);


	// Sets the value for the FStructProperty of the object based on cells from the sheet data at the given starting position
	void SetStructPropertyFromCells(FSheetData& SheetData, const FIntPoint StartingPosition, int32 ObjectRowCount, void* InObject, UObject* Parent, const FProperty* InProperty);

	// Sets the value for the FArrayProperty of the object based on cells from the sheet data at the given starting position
	void SetArrayPropertyFromCells(FSheetData& SheetData, const FIntPoint StartingPosition, int32 ObjectRowCount, void* InObject, UObject* Parent, const FProperty* InProperty);

	// Sets the value for the FSetProperty of the object based on cells from the sheet data at the given starting position
	void SetSetPropertyFromCells(FSheetData& SheetData, const FIntPoint StartingPosition, int32 ObjectRowCount, void* InObject, UObject* Parent, const FProperty* InProperty);

	// Sets the value for the FMapProperty of the object based on cells from the sheet data at the given starting position
	void SetMapPropertyFromCells(FSheetData& SheetData, const FIntPoint StartingPosition, int32 ObjectRowCount, void* InObject, UObject* Parent, const FProperty* InProperty);

	// Sets the value for the FStrProperty of the object based on cells from the sheet data at the given starting position
	void SetStringPropertyFromCells(FSheetData& SheetData, const FIntPoint StartingPosition, int32 ObjectRowCount, void* InObject, UObject* Parent, const FProperty* InProperty);

	TMap <FString, FFillHeadersFromProperty> HeaderGetters;
	TMap <FString, FFillCellsFromProperty> CellGetters;
	TMap <FString, FSetPropertyFromCells> PropertySetters;
};

