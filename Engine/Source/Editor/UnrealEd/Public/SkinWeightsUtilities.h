// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/SkeletalMesh.h"
#include "Framework/Commands/UIAction.h"

namespace UnFbx
{
	struct FBXImportOptions;
}

//////////////////////////////////////////////////////////////////////////
// FSkinWeightsUtilities

class FSkinWeightsUtilities
{
public:
	/**
	* This function import a new set of skin weights for a specified LOD. Return true if the weights are successfully updates.
	* If it return false, nothing in the skeletal skin weights was modified.
	*
	* @param SkeletalMesh - The skeletal mesh to operate on.
	* @param Path - The file path to import the weight from
	* @Param TargetLODIndex - The LODIndex to imp[ort the skin weight
	* @Param ProfileName - The name of the profile to associate the imported skin weight
	* @param bReregisterComponent - if true the component using the skeletal mesh will all be re register.
	*/
	static UNREALED_API bool ImportAlternateSkinWeight(USkeletalMesh* SkeletalMesh, const FString& Path, int32 TargetLODIndex, const FName& ProfileName, const bool bIsReimport);
	
	/**
	* This function reimport all skin weights profile for a specified LOD. Return true if the weights are successfully updates.
	* If it return false, nothing in the skeletal skin weights was modified.
	*
	* @param SkeletalMesh - The skeletal mesh to operate on.
	* @Param TargetLODIndex - The LODIndex to imp[ort the skin weight
	* @param bReregisterComponent - if true the component using the skeletal mesh will all be re register.
	*/
	static UNREALED_API bool ReimportAlternateSkinWeight(USkeletalMesh* SkeletalMesh, int32 TargetLODIndex);

	static UNREALED_API bool RemoveSkinnedWeightProfileData(USkeletalMesh* SkeletalMesh, const FName& ProfileName, int32 LODIndex);

	/*
	 * Ask user a file path for a particular LOD
	 */
	static UNREALED_API FString PickSkinWeightPath(int32 LODIndex, USkeletalMesh* SkeletalMesh);
private:
	FSkinWeightsUtilities() {}
};
