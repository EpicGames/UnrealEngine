// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FEditorViewportSnapping;
class FEditorViewportClient;
class FLevelEditorViewportClient;
class FPrimitiveDrawInterface;
class FSceneView;

class FSnappingUtils
{
public:
	/**
	 * @return true if any snapping is active and should be applied
	 */
	static UNREALED_API bool IsSnappingActive();

	/**
	 * @return true if snapping (translation) to the grid is enabled
	 */
	static UNREALED_API bool IsSnapToGridEnabled();

	/**
	 * @return true if *any* snapping that affects translation is enabled
	 */
	static UNREALED_API bool IsTranslationSnappingEnabled(const bool bInRequireSnappingActive = true);

	/**
	 * @return true if orientation snapping to angular grid is enabled
	 */
	static UNREALED_API bool IsRotationSnapEnabled();

	/**
	 * @return true if *any* snapping that affects orientation is enabled
	 */
	static UNREALED_API bool IsRotationSnappingEnabled(const bool bInRequireSnappingActive = true);

	/**
	 * @return true if scale snapping to grid is enabled
	 */
	static UNREALED_API bool IsScaleSnapEnabled();

	/**
	 * @return true if *any* snapping that affects scale is enabled
	 */
	static UNREALED_API bool IsScaleSnappingEnabled(const bool bInRequireSnappingActive = true);

	/**
	 * @return true if snapping actors to other actors is enabled
	 */
	static UNREALED_API bool IsSnapToActorEnabled();

	/** Set user setting for actor snap. */
	static UNREALED_API void EnableActorSnap(bool bEnable);

	/** Access user setting for distance. Fractional 0.0->100.0 */
	static UNREALED_API float GetActorSnapDistance(bool bScalar = false);

	/** Set user setting for distance. Fractional 0.0->100.0 */
	static UNREALED_API void SetActorSnapDistance(float Distance);

	/**
	 * Attempts to snap the selected actors to the nearest other actor
	 *
	 * @param DragDelta			The current world space drag amount
	 * @param ViewportClient	The viewport client the user is dragging in
	 */
	static UNREALED_API bool SnapActorsToNearestActor( FVector& DragDelta, FLevelEditorViewportClient* ViewportClient );

	/**
	 * Attempts to snap the selected actors to the nearest other actor
	 *
	 * @param DragDelta			The current world space drag amount
	 * @param InWorld			The world the user is dragging in
	 * @return the location delta to the nearest actor, if any
	 */
	static UNREALED_API TOptional<FVector> GetDeltaToNearestActor( FVector& DragDelta, const UWorld* InWorld );

	/**
	 * Snaps actors to the nearest vertex on another actor
	 *
	 * @param DragDelta			The current world space drag amount that will be modified to account for snapping to a vertex
	 * @param ViewportClient	The viewport client the user is dragging in
	 * @return true if anything was snapped
	 */
	static UNREALED_API bool SnapDraggedActorsToNearestVertex( FVector& DragDelta, FEditorViewportClient* ViewportClient );

	/**
	 * Snaps a delta drag movement to the nearest vertex
	 *
	 * @param BaseLocation		Location that should be snapped before any drag is applied
	 * @param DragDelta			Delta drag movement that should be snapped.  This value will be updated such that BaseLocation+DragDelta is the nearest snapped verted
	 * @param ViewportClient	The viewport client being dragged in.
	 * @return true if anything was snapped
	 */
	static UNREALED_API bool SnapDragLocationToNearestVertex( const FVector& BaseLocation, FVector& DragDelta, FEditorViewportClient* ViewportClient, bool bIsPivot = false );

	/**
	 * Snaps a location to the nearest vertex
	 *
	 * @param Location			The location to snap
	 * @param MouseLocation		The current 2d mouse location.  Vertices closer to the mouse are favored
	 * @param ViewportClient	The viewport client being used
	 * @param OutVertexNormal	The normal at the closest vertex
	 * @return true if anything was snapped
	 */
	static UNREALED_API bool SnapLocationToNearestVertex( FVector& Location, const FVector2D& MouseLocation, FEditorViewportClient* ViewportClient,
		FVector& OutVertexNormal, bool bDrawVertHelpers, const TSet<TWeakObjectPtr<class AActor>>& ActorsToIgnore );
	static UNREALED_API bool SnapLocationToNearestVertex( FVector& Location, const FVector2D& MouseLocation, FLevelEditorViewportClient* ViewportClient, FVector& OutVertexNormal, bool bDrawVertHelpers );

	/**
	 * Snaps a scale value to the scale grid
	 *
	 * @param Point	The point to snap.  This value will be modified to account for snapping
	 * @param GridBase	Base grid offset
	 */
	static UNREALED_API void SnapScale( FVector& Point, const FVector& GridBase );

	/**
	 * Adjusts the given scale value for percentage based scaling, if enabled
	 * The input should be 0-100, and the output will be adjusted to be 0-1
	 *
	 * @param InOutScale	The scale value to adjust
	 */
	static UNREALED_API void AdjustPercentageBasedScale( FVector& InOutScale );

	/**
	 * Snaps a point value to the positional grid
	 *
	 * @param Point	The point to snap.  This value will be modified to account for snapping
	 * @param GridBase	Base grid offset
	 */
	static UNREALED_API void SnapPointToGrid( FVector& Point, const FVector& GridBase );

	/**
	 * Snaps a rotator to the rotational grid
	 *
	 * @param Rotation	The rotator to snap.  This value will be modified to account for snapping
	 */
	static UNREALED_API void SnapRotatorToGrid( FRotator& Rotation );

	static UNREALED_API bool SnapToBSPVertex( FVector& Location, FVector GridBase, FRotator& Rotation );

	/**
	 * Clears all vertices being drawn to help a user snap
	 *
	 * @param bClearImmediatley	true to clear helpers without fading them out
	 */
	static UNREALED_API void ClearSnappingHelpers( bool bClearImmediately = false);

	/**
	 * Draws snapping helpers 
	 *
	 * @param View	The current view of the scene
	 * @param PDI	Drawing interface
	 */
	static UNREALED_API void DrawSnappingHelpers(const FSceneView* View,FPrimitiveDrawInterface* PDI); 

	/**
	 * Initialize the snapping system
	 */
	static UNREALED_API void InitEditorSnappingTools();

private:
	/** Built-in editor snap implementation */
	static TSharedPtr<class FEditorViewportSnapping> EditorViewportSnapper;

};
