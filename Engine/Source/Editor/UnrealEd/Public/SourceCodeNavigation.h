// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Textures/SlateIcon.h"
#include "UObject/WeakObjectPtr.h"

class AActor;

DECLARE_LOG_CATEGORY_EXTERN(LogSelectionDetails, Log, All);

DECLARE_DELEGATE_OneParam(FOnIDEInstallerDownloadComplete, bool /*bWasSuccessful*/);

/**
 * Singleton holding database of module names and disallowed header names in the engine and current project.
 */
class FSourceFileDatabase
{
public:
	/** Constructs database */
	FSourceFileDatabase();

	/** Destructs database */
	~FSourceFileDatabase();

	/** Return array of module names used by the engine and current project, including those in plugins */
	const TArray<FString>& GetModuleNames() const { return ModuleNames; }

	/** Return set of public header names used by engine modules, which are disallowed as project header names */
	const TSet<FString>& GetDisallowedHeaderNames() const { return DisallowedHeaderNames; }

	/** Update the list of known modules and disallowed header names if they've become stale */
	void UpdateIfNeeded();

private:

	/** Return array of filenames matching the given wildcard, recursing into subdirectories if no results are yielded from the base directory */
	void FindRootFilesRecursive(TArray<FString> &FileNames, const FString &BaseDirectory, const FString &Wildcard);

	/** Called when a new .Build.cs file is added to the current project */
	void OnNewModuleAdded(FName InModuleName);

	bool bIsDirty;
	TArray<FString> ModuleNames;
	TSet<FString> DisallowedHeaderNames;
};

/**
* Handles source code navigation for custom scripting languages.
*
* Should be registered with FSourceCodeNavigation::AddNavigationHandler and unregistered
* using FSourceCodeNavigation::Remove.NavigationHandler
*/
class ISourceCodeNavigationHandler : public TSharedFromThis<ISourceCodeNavigationHandler>
{
public:
	virtual ~ISourceCodeNavigationHandler() = default;

	/**
	* Determines whether it is possible to navigate to the UClass using this handler.
	*
	* @param	InClass			Class to inspect
	* @return					True if the class can be handled by this handler.
	*/
	virtual bool CanNavigateToClass(const UClass* InClass) { return false; }

	/**
	* Asynchronously navigates to a UClass in an IDE or text editor.
	*
	* @param	InClass			Class to which to navigate.
	* @return					True if the class can be handled by this handler.
	*/
	virtual bool NavigateToClass(const UClass* InClass) { return false; }

	/**
	* Determines whether it is possible to navigate to the UScriptStruct using this handler.
	*
	* @param	InStruct		Struct to inspect
	* @return					True if the struct can be handled by this handler.
	*/
	virtual bool CanNavigateToStruct(const UScriptStruct* InStruct) { return false; }

	/**
	* Asynchronously navigates to a UScriptStruct in an IDE or text editor.
	*
	* @param	InStruct		Struct to which to navigate.
	* @return					True if the struct can be handled by this handler.
	*/
	virtual bool NavigateToStruct(const UScriptStruct* InStruct) { return false; }

	/**
	* Determines whether it is possible to navigate to the UFunction using this handler.
	*
	* @param	InFunction		Function to inspect
	* @return					True if the function can be handled by this handler.
	*/
	virtual bool CanNavigateToFunction(const UFunction* InFunction) { return false; }

	/**
	* Asynchronously navigates to a UFunction in an IDE or text editor.
	*
	* @param	InFunction		Function to which to navigate.
	* @return					True if the function can be handled by this handler.
	*/
	virtual bool NavigateToFunction(const UFunction* InFunction) { return false; }

	/**
	* Determines whether it is possible to navigate to the FProperty using this handler.
	*
	* @param	InProperty		Property to inspect
	* @return					True if the property can be handled by this handler.
	*/
	virtual bool CanNavigateToProperty(const FProperty* InProperty) { return false; }

	/**
	* Asynchronously navigates to a FProperty in an IDE or text editor.
	*
	* @param	InProperty		Property to which to navigate.
	* @return					True if the property can be handled by this handler.
	*/
	virtual bool NavigateToProperty(const FProperty* InProperty) { return false; }

	/**
	* Determines whether it is possible to navigate to the UStruct using this handler.
	*
	* @param	InStruct		Struct to inspect
	* @return					True if the struct can be handled by this handler.
	*/
	virtual bool CanNavigateToStruct(const UStruct* InStruct) { return false; };

	/**
	* Asynchronously navigates to a UStruct in an IDE or text editor.
	*
	* @param	InStruct		Struct in which to navigate.
	* @return					True if the struct can be handled by this handler.
	*/
	virtual bool NavigateToStruct(const UStruct* InStruct) { return false; };
};

/**
 * Source code navigation functionality
 */
class FSourceCodeNavigation
{

public:
	/** Holds useful information about a function's symbols */
	struct FFunctionSymbolInfo
	{
		/** Function symbol string */
		FString SymbolName;

		/** The name of the UObject class this function is in, if any */
		FString ClassName;

		/** The name of the module the function is in (or empty string if not known) */
		FString ModuleName;
	};


	/** Allows function symbols to be organized by class */
	struct FEditCodeMenuClass 
	{
		/** Name of the class */
		FString Name;

		/** Module name this class resides in */
		FString ModuleName;

		/** All of the functions in this class */
		TArray< FFunctionSymbolInfo > Functions;

		/** True if the list of functions is a complete list, or false if we still do not have all data digested yet (or
		    the user only asked for a list of classes, and we've determined the class may have relevant functions.) */
		bool bIsCompleteList;

		/** Referenced object (e.g., a blueprint for a kismet graph symbol) */
		TWeakObjectPtr<UObject> ReferencedObject;
	};


	/**
	 * Initializes FSourceCodeNavigation static class
	 */
	UNREALED_API static void Initialize();

	/**
	 * Retrieves the SourceFileDatabase instance
	 */
	UNREALED_API static const FSourceFileDatabase& GetSourceFileDatabase();

	/**
	 * Asynchronously locates the source file and line for a specific function in a specific module and navigates an external editing to that source line
	 *
	 * @param	FunctionSymbolName	The function to navigate tool (e.g. "MyClass::MyFunction")
	 * @param	FunctionModuleName	The module to search for this function's symbols (e.g. "GameName-Win64-Debug")
	 * @param	bIgnoreLineNumber	True if we should just open the file and not go to a specific line number
	 */
	UNREALED_API static void NavigateToFunctionSourceAsync( const FString& FunctionSymbolName, const FString& FunctionModuleName, const bool bIgnoreLineNumber );

	/** Gather modes for GatherFunctionsForActors() */
	struct EGatherMode
	{
		enum Type
		{
			/** Gather classes and all functions in those classes */
			ClassesAndFunctions,

			/** Only gather the class names (much faster!) */
			ClassesOnly
		};
	};

	/**
	 * Finds all of the functions in classes for the specified list of actors
	 *
	 * @param	Actors			List of actors to gather functions for
	 * @param	GatherMode		Whether to gather all classes and functions, or only classes
	 * @param	Classes	(Out)	Sorted list of function symbols organized by class for actor classes
	 */
	UNREALED_API static void GatherFunctionsForActors( TArray< AActor* >& Actors, const EGatherMode::Type GatherMode, TArray< FEditCodeMenuClass >& Classes );

	/**
	* Deprecated, use NavigateToFunction
	*/
	UNREALED_API static bool NavigateToFunctionAsync(UFunction* InFunction);

	/**
	* Determines whether it is possible to navigate to the UClass in the IDE
	*
	* @param	InClass			UClass to navigate to in source code
	* @return					Whether the navigation is likely to be successful or not
	*/
	UNREALED_API static bool CanNavigateToClass(const UClass* InClass);

	/**
	* Navigates asynchronously to the UClass in the IDE
	*
	* @param	InClass			UClass to navigate to in source code
	* @return					Whether the navigation is likely to be successful or not
	*/
	UNREALED_API static bool NavigateToClass(const UClass* InClass);

	/**
	* Determines whether it is possible to navigate to the UScriptStruct in the IDE
	*
	* @param	InStruct		UScriptStruct to navigate to in source code
	* @return					Whether the navigation is likely to be successful or not
	*/
	UNREALED_API static bool CanNavigateToStruct(const UScriptStruct* InStruct);

	/**
	* Navigates asynchronously to the UScriptStruct in the IDE
	*
	* @param	InStruct		UScriptStruct to navigate to in source code
	* @return					Whether the navigation is likely to be successful or not
	*/
	UNREALED_API static bool NavigateToStruct(const UScriptStruct* InStruct);

	/**
	* Determines whether it is possible to navigate to the UFunction in the IDE
	*
	* @param	InFunction		UFunction to navigate to in source code
	* @return					Whether the navigation is likely to be successful or not
	*/
	UNREALED_API static bool CanNavigateToFunction(const UFunction* InFunction);

	/**
	 * Navigates asynchronously to the UFunction in IDE
	 *
	 * @param	InFunction		UFunction to navigate to in source code
	 * @return					Whether the navigation was successful or not
	 */
	UNREALED_API static bool NavigateToFunction(const UFunction* InFunction);

	/**
	* Determines whether it is possible to navigate to the FProperty in the IDE
	*
	* @param	InProperty		FProperty to navigate to in source code
	* @return					Whether the navigation is likely to be successful or not
	*/
	UNREALED_API static bool CanNavigateToProperty(const FProperty* InProperty);

	/**
	 * Navigates asynchronously to the FProperty in the IDE
	 *
	 * @param	InProperty		FProperty to navigate to in source code
	 * @return					Whether the navigation was successful or not
	 */
	UNREALED_API static bool NavigateToProperty(const FProperty* InProperty);

	/**
	 * Determines whether it is possible to navigate to the UStruct in the IDE
	 *
	 * @param	InStruct		UStruct to navigate to in source code
	 * @return					Whether the navigation is likely to be successful or not
 	 */
	UNREALED_API static bool CanNavigateToStruct(const UStruct* InStruct);

	/**
	 * Navigates asynchronously to the UStruct in the IDE
	 *
	 * @param	InStruct		UStruct to navigate to in source code
	 * @return					Whether the navigation was successful or not
	 */
	UNREALED_API static bool NavigateToStruct(const UStruct* InStruct);

	/** Delegate that's triggered when any symbol query has completed */
	DECLARE_MULTICAST_DELEGATE( FOnSymbolQueryFinished );

	/** Call this to access the multi-cast delegate that you can register a callback with */
	UNREALED_API static FOnSymbolQueryFinished& AccessOnSymbolQueryFinished();

	/** Returns the name of the suggested IDE, based on platform */
	UNREALED_API static FText GetSuggestedSourceCodeIDE(bool bShortIDEName = false);

	/** Returns the name of the selected IDE */
	UNREALED_API static FText GetSelectedSourceCodeIDE();

	/** Returns the IDE specific representative icon to open source code*/
	UNREALED_API static FSlateIcon GetOpenSourceCodeIDEIcon();

	/** Returns the IDE specific representative icon to refresh source code*/
	UNREALED_API static FSlateIcon GetRefreshSourceCodeIDEIcon();

	/** Returns the url to the location where the suggested IDE can be downloaded */
	UNREALED_API static FString GetSuggestedSourceCodeIDEDownloadURL();

	/** Returns whether the suggested source code IDE for the current platform can be installed directly (vs. requiring that the user download it manually) */
	UNREALED_API static bool GetCanDirectlyInstallSourceCodeIDE();

	/** Downloads and installs the suggested IDE (currently only works for Windows) */
	UNREALED_API static void DownloadAndInstallSuggestedIDE(FOnIDEInstallerDownloadComplete OnDownloadComplete);

	/** Refresh the state of compiler availability.  Don't call unless you know you might be doing something to cause that. */
	UNREALED_API static void RefreshCompilerAvailability();

	/** Returns true if the compiler for the current platform is available for use */
	static bool IsCompilerAvailable() { return bCachedIsCompilerAvailable; }

	/** Finds the base directory for a given module name. Does not rely on symbols; finds matching .build.cs files. */
	UNREALED_API static bool FindModulePath( const FString& InModuleName, FString& OutModulePath );

	/** Finds the base directory for a given module. Does not rely on symbols; finds matching .build.cs files. */
	UNREALED_API static bool FindModulePath( const UPackage* InModulePackage, FString& OutModulePath );

	/** Finds the path to a given class header. Does not rely on symbols; finds matching .build.cs files. */
	UNREALED_API static bool FindClassHeaderPath( const UField* InField, FString &OutClassHeaderPath );

	/** Finds the path to a given class source. Does not rely on symbols; finds matching .build.cs files. */
	UNREALED_API static bool FindClassSourcePath( const UField* InField, FString &OutClassSourcePath );

	/** Opens a single source file */
	UNREALED_API static bool OpenSourceFile( const FString& AbsoluteSourcePath, int32 LineNumber = 0, int32 ColumnNumber = 0 );

	/** Opens a multiple source files */
	UNREALED_API static bool OpenSourceFiles(const TArray<FString>& AbsoluteSourcePaths);

	/** Opens a single source file asynchronously, showing a notification while the IDE loads. */
	UNREALED_API static void OpenSourceFileAsync(const FString& AbsoluteSourcePath, int32 LineNumber = 0, int32 ColumnNumber = 0);

	/** Opens multiple source files asynchronously, showing a notification while the IDE loads. */
	UNREALED_API static void OpenSourceFilesAsync(const TArray<FString>& AbsoluteSourcePaths);

	/** Add multiple source files to the current solution/project/workspace */
	UNREALED_API static bool AddSourceFiles(const TArray<FString>& AbsoluteSourcePaths);

	/** Open the current source code solution */
	UNREALED_API static bool OpenModuleSolution();

	/** Open the source code solution for the project at the given location */
	UNREALED_API static bool OpenProjectSolution(const FString& InProjectPath);

	/** Query if the current source code solution exists */
	UNREALED_API static bool DoesModuleSolutionExist();

	/** Attempt to locate fully qualified class module name */
	UNREALED_API static bool FindClassModuleName( UClass* InClass, FString& ModuleName );

	/** Delegate that's triggered when any symbol query has completed */
	DECLARE_MULTICAST_DELEGATE( FOnCompilerNotFound );

	/** Call this to access the multi-cast delegate that you can register a callback with */
	UNREALED_API static FOnCompilerNotFound& AccessOnCompilerNotFound();

	/** Delegate that's triggered when a new module (.Build.cs file) has been added */
	DECLARE_MULTICAST_DELEGATE_OneParam( FOnNewModuleAdded, FName );

	/** Call this to access the multi-cast delegate that you can register a callback with */
	UNREALED_API static FOnNewModuleAdded& AccessOnNewModuleAdded();

	/** Add a navigation handler */
	UNREALED_API static void AddNavigationHandler(ISourceCodeNavigationHandler* Handler);

	/** Remove a navigation handler */
	UNREALED_API static void RemoveNavigationHandler(ISourceCodeNavigationHandler* Handler);

	/** Set a new preferred accessor and refresh compiler status */
	UNREALED_API static void SetPreferredAccessor(const TCHAR* Name);

private:

	/** Critical section for locking access to the source file database. */
	static FCriticalSection CriticalSection;

	/** Source file database instance. */
	static FSourceFileDatabase Instance;

	/** Cached result of check for compiler availability. Speeds up performance greatly since BlueprintEditor is checking this on draw. */
	static bool UNREALED_API bCachedIsCompilerAvailable;
};
