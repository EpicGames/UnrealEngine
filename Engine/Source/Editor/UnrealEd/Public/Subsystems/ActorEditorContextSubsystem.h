// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorSubsystem.h"

#include "ActorEditorContextSubsystem.generated.h"

struct IActorEditorContextClient;
class AActor;
class UActorEditorContextStateCollection;

DECLARE_MULTICAST_DELEGATE(FOnActorEditorContextSubsystemChanged);

/**
* UActorEditorContextSubsystem 
*/
UCLASS(MinimalAPI)
class UActorEditorContextSubsystem : public UEditorSubsystem
{
public:

	GENERATED_BODY()

	static UNREALED_API UActorEditorContextSubsystem* Get();

	UNREALED_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UNREALED_API virtual void Deinitialize() override;

	UNREALED_API void RegisterClient(IActorEditorContextClient* Client);
	UNREALED_API void UnregisterClient(IActorEditorContextClient* Client);
	UNREALED_API void ResetContext();
	UNREALED_API void ResetContext(IActorEditorContextClient* Client);
	UNREALED_API void PushContext(bool bDuplicateContext = false);
	UNREALED_API void PopContext();
	UNREALED_API void InitializeContextFromActor(AActor* Actor);
	UNREALED_API void CaptureContext(UActorEditorContextStateCollection* InStateCollection);
	UNREALED_API void RestoreContext(const UActorEditorContextStateCollection* InStateCollection) const;
	UNREALED_API TArray<IActorEditorContextClient*> GetDisplayableClients() const;
	FOnActorEditorContextSubsystemChanged& OnActorEditorContextSubsystemChanged() { return ActorEditorContextSubsystemChanged; }

private:

	UNREALED_API UWorld* GetWorld() const;
	UNREALED_API void OnActorEditorContextClientChanged(IActorEditorContextClient* Client);	
	UNREALED_API void ApplyContext(AActor* InActor);

	void OnPasteActorsBegin();
	void OnPasteActorsEnd(const TArray<AActor*>& InActors);

	FOnActorEditorContextSubsystemChanged ActorEditorContextSubsystemChanged;
	TArray<IActorEditorContextClient*> Clients;
	TArray<TArray<IActorEditorContextClient*>> PushedContextsStack;
	bool bIsApplyEnabled = true;

	friend class UActorEditorContextEditorState;
};