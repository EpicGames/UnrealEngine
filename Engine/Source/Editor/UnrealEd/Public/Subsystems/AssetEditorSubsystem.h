// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"

#include "Toolkits/IToolkit.h"

#include "Containers/Ticker.h"
#include "Tools/Modes.h"
#include "Misc/NamePermissionList.h"
#include "AssetTypeActivationOpenedMethod.h"
#include "MRUFavoritesList.h"
#include "AssetDefinition.h"

#include "AssetEditorSubsystem.generated.h"

class UAssetEditor;
class UEdMode;
class UObject;
class UClass;
struct FAssetEditorRequestOpenAsset;
class FEditorModeTools;
class IMessageContext;
class UToolMenu;
struct FToolMenuSection;

/** The way that editors were requested to close */
enum class EAssetEditorCloseReason : uint8
{
	/* NOTE: All close reasons can be passed into OnRequestClose() and CloseWindow(), but only some are broadcast by
	 * UAssetEditorSubsystem::AssetEditorRequestCloseEvent currently while others are for the asset editors themselves
	 */
	
	// Close reasons broadcast by UAssetEditorSubsystem
	
	CloseAllEditorsForAsset,   // All asset editors operating on a specific asset are being requested to close
	CloseOtherEditors,         // An asset editor is requesting all asset editors using an asset except itself to close
	RemoveAssetFromAllEditors, // An asset is being removed from all asset editors (which may or may not actually close)
	CloseAllAssetEditors,      // Every single Asset Editor is being requested to close 

	// Close reasons for individual asset editors
	
	AssetEditorHostClosed,     // The "default" reason for an asset editor to close, e.g when the close button is clicked
	AssetUnloadingOrInvalid,   // The asset being edited is being unloaded or is no longer valid
	EditorRefreshRequested,    // The asset editor wants to close and re-open to edit the same asset
	AssetForceDeleted          // The asset being edited has been force deleted
};

/**
 * This class keeps track of a currently open asset editor; allowing it to be
 * brought into focus, closed, etc..., without concern for how the editor was
 * implemented.
 */
class IAssetEditorInstance
{
public:

	virtual FName GetEditorName() const = 0;
	virtual void FocusWindow(UObject* ObjectToFocusOn = nullptr) = 0;

	UE_DEPRECATED(5.3, "Use CloseWindow that takes in an EAssetEditorCloseReason instead")
	virtual bool CloseWindow() { return true; }
	
	virtual bool CloseWindow(EAssetEditorCloseReason InCloseReason)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return CloseWindow();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** If false, the asset being edited will not be included in reopen assets prompt on restart */
	UE_DEPRECATED(5.4, "Use the override that takes in an UObject instead")
	virtual bool IncludeAssetInRestoreOpenAssetsPrompt() const { return true; }

	virtual bool IncludeAssetInRestoreOpenAssetsPrompt(UObject* Asset) const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return IncludeAssetInRestoreOpenAssetsPrompt();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	virtual bool IsPrimaryEditor() const = 0;
	virtual void InvokeTab(const struct FTabId& TabId) = 0;
	UE_DEPRECATED(5.0, "Toolbar tab no longer exists and tab ID will return None; do not add it to layouts")
	virtual FName GetToolbarTabId() const { return NAME_None; }
	virtual FName GetEditingAssetTypeName() const { return NAME_None; }
	virtual TSharedPtr<class FTabManager> GetAssociatedTabManager() = 0;
	virtual double GetLastActivationTime() = 0;
	virtual void RemoveEditingAsset(UObject* Asset) = 0;
	virtual EAssetOpenMethod GetOpenMethod() const { return EAssetOpenMethod::Edit; }

protected:
	// protected destructor since IAssetEditorInstance isn't meant to be used to delete derived class instances
	~IAssetEditorInstance() =  default;
};

struct FRegisteredModeInfo
{
	TWeakObjectPtr<UClass> ModeClass;
	FEditorModeInfo ModeInfo;
};
using RegisteredModeInfoMap = TMap<FEditorModeID, FRegisteredModeInfo>;

DECLARE_MULTICAST_DELEGATE_OneParam(FAssetEditorOpenDebugObject, UObject* /* DebugObject */);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FAssetEditorOpenDebugObjectDynamic, UObject*, DebugObject);

/**
 * UAssetEditorSubsystem
 */
UCLASS(MinimalAPI)
class UAssetEditorSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	UNREALED_API UAssetEditorSubsystem();

	UNREALED_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UNREALED_API virtual void Deinitialize() override;

	UNREALED_API bool IsAssetEditable(const UObject* Asset);

	/** Opens an asset by path */
	UNREALED_API void OpenEditorForAsset(const FString& AssetPathName, const EAssetTypeActivationOpenedMethod OpenedMethod = EAssetTypeActivationOpenedMethod::Edit);
	UNREALED_API void OpenEditorForAsset(const FSoftObjectPath& AssetPath, const EAssetTypeActivationOpenedMethod OpenedMethod = EAssetTypeActivationOpenedMethod::Edit);

	/**
	 * Tries to open an editor for the specified asset.  Returns true if the asset is open in an editor.
	 * If the file is already open in an editor, it will not create another editor window but instead bring it to front
	 */
	UNREALED_API bool OpenEditorForAsset(UObject* Asset, const EToolkitMode::Type ToolkitMode = EToolkitMode::Standalone, TSharedPtr<IToolkitHost> OpenedFromLevelEditor = TSharedPtr<IToolkitHost>(), const bool bShowProgressWindow = true, EAssetTypeActivationOpenedMethod OpenedMethod = EAssetTypeActivationOpenedMethod::Edit);
	
	template<typename ObjectType>
	bool OpenEditorForAsset(TObjectPtr<ObjectType> Asset, const EToolkitMode::Type ToolkitMode = EToolkitMode::Standalone, TSharedPtr<IToolkitHost> OpenedFromLevelEditor = TSharedPtr<IToolkitHost>(), const bool bShowProgressWindow = true)
	{
		return OpenEditorForAsset(ToRawPtr(Asset), ToolkitMode, OpenedFromLevelEditor, bShowProgressWindow);
	}

	/**
	 * Tries to open an editor for all of the specified assets.
	 * If any of the assets are already open, it will not create a new editor for them.
	 * If all assets are of the same type, the supporting AssetTypeAction (if it exists) is responsible for the details of how to handle opening multiple assets at once.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset Tools")
	UNREALED_API bool OpenEditorForAssets(const TArray<UObject*>& Assets, const EAssetTypeActivationOpenedMethod OpenedMethod = EAssetTypeActivationOpenedMethod::Edit);
	UNREALED_API bool OpenEditorForAssets_Advanced(const TArray<UObject*>& InAssets, const EToolkitMode::Type ToolkitMode = EToolkitMode::Standalone, TSharedPtr<IToolkitHost> OpenedFromLevelEditor = TSharedPtr<IToolkitHost>(), const EAssetTypeActivationOpenedMethod OpenedMethod = EAssetTypeActivationOpenedMethod::Edit);

	/** Opens editors for the supplied assets (via OpenEditorForAsset) */
	UNREALED_API void OpenEditorsForAssets(const TArray<FString>& AssetsToOpen, const EAssetTypeActivationOpenedMethod OpenedMethod = EAssetTypeActivationOpenedMethod::Edit);
	UNREALED_API void OpenEditorsForAssets(const TArray<FName>& AssetsToOpen, const EAssetTypeActivationOpenedMethod OpenedMethod = EAssetTypeActivationOpenedMethod::Edit);
	UNREALED_API void OpenEditorsForAssets(const TArray<FSoftObjectPath>& AssetsToOpen);

	/** Check whether the given asset can be opened in the given open method */
	UNREALED_API bool CanOpenEditorForAsset(UObject* Asset, const EAssetTypeActivationOpenedMethod OpenedMethod, FText* OutErrorMsg); 

	/** Returns the primary editor if one is already open for the specified asset.
	 * If there is one open and bFocusIfOpen is true, that editor will be brought to the foreground and focused if possible.
	 */
	UNREALED_API IAssetEditorInstance* FindEditorForAsset(UObject* Asset, bool bFocusIfOpen);

	/** Returns all editors currently opened for the specified asset */
	UNREALED_API TArray<IAssetEditorInstance*> FindEditorsForAsset(UObject* Asset);

	/** Returns all editors currently opened for the specified asset or any of its subobjects */
	UNREALED_API TArray<IAssetEditorInstance*> FindEditorsForAssetAndSubObjects(UObject* Asset);

	/** Close all active editors for the supplied asset and return the number of asset editors that were closed */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset Tools")
	UNREALED_API int32 CloseAllEditorsForAsset(UObject* Asset);

	/** Close any editor which is not this one */
	UNREALED_API void CloseOtherEditors(UObject* Asset, IAssetEditorInstance* OnlyEditor);

	/** Remove given asset from all open editors */
	UNREALED_API void RemoveAssetFromAllEditors(UObject* Asset);

	/** Event called specifically when an external system requests an asset editor to be closed (e.g by calling CloseAllAssetEditors())
	 *  If you want an event that is called anytime an asset is removed from an editor (which may or may not close the editor) - use
	 *  OnAssetClosedInEditor()
	 */
	DECLARE_EVENT_TwoParams(UAssetEditorSubsystem, FAssetEditorRequestCloseEvent, UObject*, EAssetEditorCloseReason);
	virtual FAssetEditorRequestCloseEvent& OnAssetEditorRequestClose() { return AssetEditorRequestCloseEvent; }

	/** Get all assets currently being tracked with open editors */
	UNREALED_API TArray<UObject*> GetAllEditedAssets();

	/** Returns all editors that are currently alive */
	UNREALED_API TArray<IAssetEditorInstance*> GetAllOpenEditors() const;

	/** Notify the asset editor manager that an asset was opened */
	UNREALED_API void NotifyAssetOpened(UObject* Asset, IAssetEditorInstance* Instance);
	UNREALED_API void NotifyAssetsOpened(const TArray< UObject* >& Assets, IAssetEditorInstance* Instance);

	/** Called when an asset has been opened in an editor */
	DECLARE_EVENT_TwoParams(UAssetEditorSubsystem, FOnAssetOpenedInEditorEvent, UObject*, IAssetEditorInstance*);
	virtual FOnAssetOpenedInEditorEvent& OnAssetOpenedInEditor() { return AssetOpenedInEditorEvent; }

	/** Notify the asset editor manager that an asset editor is being opened and before widgets are constructed */
	UNREALED_API void NotifyEditorOpeningPreWidgets(const TArray< UObject* >& Assets, IAssetEditorInstance* Instance);

	/** Called when an asset editor is opening and before widgets are constructed */
	DECLARE_EVENT_TwoParams(UAssetEditorSubsystem, FOnAssetsOpenedInEditorEvent, const TArray<UObject*>&, IAssetEditorInstance*);
	virtual FOnAssetsOpenedInEditorEvent& OnEditorOpeningPreWidgets() { return EditorOpeningPreWidgetsEvent; }

	/** Notify the asset editor manager that an asset editor is done editing an asset */
	UNREALED_API void NotifyAssetClosed(UObject* Asset, IAssetEditorInstance* Instance);

	/** Called when an editor is done editing an asset */
	DECLARE_EVENT_TwoParams(UAssetEditorSubsystem, FOnAssetClosedInEditorEvent, UObject*, IAssetEditorInstance*);
	virtual FOnAssetClosedInEditorEvent& OnAssetClosedInEditor() { return AssetClosedInEditorEvent; }

	/** Notify the asset editor manager that an asset was closed */
	UNREALED_API void NotifyEditorClosed(IAssetEditorInstance* Instance);

	/** Close all open asset editors */
	UNREALED_API bool CloseAllAssetEditors();

	/** Called when an asset editor is requested to be opened */
	DECLARE_EVENT_OneParam(UAssetEditorSubsystem, FAssetEditorRequestOpenEvent, UObject*);
	virtual FAssetEditorRequestOpenEvent& OnAssetEditorRequestedOpen() { return AssetEditorRequestOpenEvent; }

	/** Called when an asset editor is actually opened */
	DECLARE_EVENT_OneParam(UAssetEditorSubsystem, FAssetEditorOpenEvent, UObject*);
	FAssetEditorOpenEvent& OnAssetEditorOpened() { return AssetEditorOpenedEvent; }

	/** Request notification to restore the assets that were previously open when the editor was last closed */
	UNREALED_API void RequestRestorePreviouslyOpenAssets();

	UNREALED_API void RegisterUAssetEditor(UAssetEditor* NewAssetEditor);
	UNREALED_API void UnregisterUAssetEditor(UAssetEditor* RemovedAssetEditor);

	/**
	 * Attempts to open a debug object in an already-open asset editor. Will not open the asset editor responsible for this
	 * type of object, only cause the editor to debug the specified object.
	 * Returns true if an asset editor accepted the debug object.
	 */
	UNREALED_API bool OpenDebugObjectInEditor(UObject* DebugObject);

	/**
	 * Returns the debug object delegate for a specific class.
	 */
	UNREALED_API FAssetEditorOpenDebugObject* GetDebugObjectDelegateForClass(UClass* InClass);
	
	/**
	 * Creates a scriptable editor mode based on ID name, which will be owned by the given Owner, if that name exists in the map of editor modes found at system startup.
	 * @param ModeID	ID of the mode to create.
	 * @param Owner		The tools ownership context that the mode should be created under.
	 *
	 * @return 			A pointer to the created UEdMode or nullptr, if the given ModeID does not exist in the set of known modes.
	 */
	UNREALED_API UEdMode* CreateEditorModeWithToolsOwner(FEditorModeID ModeID, FEditorModeTools& Owner);
	
	/**
	 * Returns information about an editor mode, based on the given ID.
	 * @param ModeID		ID of the editor mode.
	 * @param OutModeInfo	The out struct where the mode information should be stored.
	 *
	 * @return 				True if OutModeInfo was filled out successfully, otherwise false.
	 */
	UNREALED_API bool FindEditorModeInfo(const FEditorModeID& InModeID, FEditorModeInfo& OutModeInfo) const;
	
	/**
	 * Creates an array of all known FEditorModeInfos, sorted by their priority, from greatest to least.
	 *
	 * @return 			The sorted array of FEditorModeInfos.
	 */
	UNREALED_API TArray<FEditorModeInfo> GetEditorModeInfoOrderedByPriority() const;

	/**
	 * Event that is triggered whenever a mode is registered or unregistered
	 */
	UNREALED_API FRegisteredModesChangedEvent& OnEditorModesChanged();

	/**
	 * Event that is triggered whenever a mode is registered
	 */
	UNREALED_API FOnModeRegistered& OnEditorModeRegistered();

	/**
	 * Event that is triggered whenever a mode is unregistered
	 */
	UNREALED_API FOnModeUnregistered& OnEditorModeUnregistered();

	/** Get the permission list that controls which editor modes are exposed */
	UNREALED_API FNamePermissionList& GetAllowedEditorModes();

	/** Fill in the given section with the recent assets for the given asset editor instance */
	UNREALED_API void CreateRecentAssetsMenuForEditor(const IAssetEditorInstance* InAssetEditorInstance, FToolMenuSection& InSection);

	FMainMRUFavoritesList* GetRecentlyOpenedAssets() const
	{
		return RecentAssetsList.Get();
	}

private:

	/** Handles FAssetEditorRequestOpenAsset messages. */
	UNREALED_API void HandleRequestOpenAssetMessage(const FAssetEditorRequestOpenAsset& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Handles ticks from the ticker. */
	UNREALED_API bool HandleTicker(float DeltaTime);

	/** Spawn a notification asking the user if they want to restore their previously open assets */
	UNREALED_API void SpawnRestorePreviouslyOpenAssetsNotification(const bool bCleanShutdown, const TArray<FString>& AssetsToOpen);

	/** Handler for when the "Restore Now" button is clicked on the RestorePreviouslyOpenAssets notification */
	UNREALED_API void OnConfirmRestorePreviouslyOpenAssets(TArray<FString> AssetsToOpen);

	/** Handler for when the "Don't Restore" button is clicked on the RestorePreviouslyOpenAssets notification */
	UNREALED_API void OnCancelRestorePreviouslyOpenAssets();

	/** Register extensions to the Level Editor "File" Menu */
	UNREALED_API void RegisterLevelEditorMenuExtensions();

	/**
	 * Fill in the given menu with entries from the recent assets list, clicking on which will open up the asset
	 * @param InMenu The menu to fill in
	 * @param InAssetEditorName If provided, only assets in the list that were opened in this asset editor will be added to the menu
	 */
	UNREALED_API void CreateRecentAssetsMenu(UToolMenu* InMenu, const FName InAssetEditorName = FName());

	UNREALED_API void OnAssetRemoved(const FAssetData& AssetData);

	UNREALED_API void OnAssetRenamed(const FAssetData& AssetData, const FString& AssetOldName);
public:

	/**
	 * Saves a list of open asset editors so they can be restored on editor restart.
	 * @param bOnShutdown If true, this is handled as if the engine is shutting down right now.
	 */

	UNREALED_API void SaveOpenAssetEditors(const bool bOnShutdown);
	
	/**
	 * Saves a list of open asset editors so they can be restored on editor restart.
	 * @param bOnShutdown If true, this is handled as if the engine is shutting down right now.
	 * @param bCancelIfDebugger If true, don't save a list of assets to restore if we are running under a debugger.
	 */
	UE_DEPRECATED(5.0, "Please use the version of SaveOpenAssetEditors with only one argument, bOnShutdown.")
	UNREALED_API void SaveOpenAssetEditors(const bool bOnShutdown, const bool bCancelIfDebugger);

	/** Restore the assets that were previously open when the editor was last closed. */
	UNREALED_API void RestorePreviouslyOpenAssets();

	/** Sets bAutoRestoreAndDisableSaving and sets bRequestRestorePreviouslyOpenAssets to false to avoid running RestorePreviouslyOpenAssets() twice. */
	UE_DEPRECATED(5.0, "Please use the SetAutoRestoreAndDisableSavingOverride instead to set/unset the override")
	UNREALED_API void SetAutoRestoreAndDisableSaving(const bool bInAutoRestoreAndDisableSaving);

	/** Set/Unset the bAutoRestoreAndDisableSaving override, along with setting bRequestRestorePreviouslyOpenAssets to false to avoid running RestorePreviouslyOpenAssets() twice */
	UNREALED_API void SetAutoRestoreAndDisableSavingOverride(TOptional<bool> bInAutoRestoreAndDisableSaving);

	/** Get the current value of bAutoRestoreAndDisableSaving */
	UNREALED_API TOptional<bool> GetAutoRestoreAndDisableSavingOverride() const;

	UNREALED_API void SetRecentAssetsFilter(const FMainMRUFavoritesList::FDoesMRUFavoritesItemPassFilter& InFilter);

	/** These functions are used by the Asset Editor Toolkit to query the method in which the given assets are being opened
	 *  These are only valid when the asset is in the process of being opened, i.e while we are in OpenEditorForAsset.
	 *  After which, you should query the asset editor itself to get the open method
	 */
	TOptional<EAssetOpenMethod> GetAssetBeingOpenedMethod(TObjectPtr<UObject> Asset);
	TOptional<EAssetOpenMethod> GetAssetsBeingOpenedMethod(TArray<TObjectPtr<UObject>> Assets);

	/* Functionality to add a filter that is run to check if an asset is allowed to be opened in read only mode
	 * (If the asset type supports it)
	 */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FReadOnlyAssetFilter, const FString&);
	UNREALED_API void AddReadOnlyAssetFilter(const FName& Owner, const FReadOnlyAssetFilter& ReadOnlyAssetFilter);
	UNREALED_API void RemoveReadOnlyAssetFilter(const FName& Owner);

private:

	/** Handles a package being reloaded */
	UNREALED_API void HandlePackageReloaded(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent);

	/** Callback for when the Editor closes, before Slate shuts down all the windows. */
	UNREALED_API void OnEditorClose();

	UNREALED_API void RegisterEditorModes();
	UNREALED_API void UnregisterEditorModes();
	UNREALED_API void OnSMInstanceElementsEnabled();

	UNREALED_API bool IsEditorModeAllowed(const FName ModeId) const;

	UNREALED_API void InitializeRecentAssets();
	UNREALED_API void SaveRecentAssets(const bool bOnShutdown = false);
	UNREALED_API void CullRecentAssetEditorsMap();

	UNREALED_API bool ShouldShowRecentAsset(const FString& AssetName, int32 RecentAssetIndex,  const FName& InAssetEditorName) const;
	UNREALED_API bool ShouldShowRecentAssetsMenu(const FName& InAssetEditorName) const;

	UNREALED_API UObject* FindOrLoadAssetForOpening(const FSoftObjectPath& AssetPath);

private:

	/** struct used by OpenedEditorTimes map to store editor names and times */
	struct FOpenedEditorTime
	{
		FName EditorName;
		FDateTime OpenedTime;
	};

	/** struct used to track total time and # of invocations during an overall UnrealEd session */
	struct FAssetEditorAnalyticInfo
	{
		FTimespan SumDuration;
		int32 NumTimesOpened;

		FAssetEditorAnalyticInfo()
			: SumDuration(0)
			, NumTimesOpened(0)
		{
		}
	};

	/**
	 * Utility struct for assets book keeping.
	 * We'll index assets using the raw pointer, as it should be possible to clear an entry after an asset has been garbage collected
	 * but any access to the UObject should go through the weak object pointer.
	 */
	struct FAssetEntry
	{
		// Implicit constructor
		FAssetEntry(UObject* InRawPtr)
			: RawPtr(InRawPtr)
			, ObjectPtr(InRawPtr)
		{
		}

		UObject* RawPtr;
		FWeakObjectPtr ObjectPtr;

		bool operator==(const FAssetEntry& Other) const { return RawPtr == Other.RawPtr; }

		inline friend uint32 GetTypeHash(const FAssetEntry& AssetEntry)
		{
			return GetTypeHash(AssetEntry.RawPtr);
		}
	};

	/** Holds the opened assets. */
	TMultiMap<FAssetEntry, IAssetEditorInstance*> OpenedAssets;

	/** Holds the opened editors. */
	TMultiMap<IAssetEditorInstance*, FAssetEntry> OpenedEditors;

	/** Holds the times that editors were opened. */
	TMap<IAssetEditorInstance*, FOpenedEditorTime> OpenedEditorTimes;

	/** Holds the cumulative time editors have been open by type. */
	TMap<FName, FAssetEditorAnalyticInfo> EditorUsageAnalytics;

private:
	/** Holds a delegate to be invoked when the widget ticks. */
	FTickerDelegate TickDelegate;

	/** Call to request closing editors for an asset */
	FAssetEditorRequestCloseEvent AssetEditorRequestCloseEvent;

	/** Called when an asset has been opened in an editor */
	FOnAssetOpenedInEditorEvent AssetOpenedInEditorEvent;

	/** Called when editor is opening and before widgets are constructed */
	FOnAssetsOpenedInEditorEvent EditorOpeningPreWidgetsEvent;

	/** Called when an editor is done editing an asset */
	FOnAssetClosedInEditorEvent AssetClosedInEditorEvent;

	/** Multicast delegate executed when an asset editor is requested to be opened */
	FAssetEditorRequestOpenEvent AssetEditorRequestOpenEvent;

	/** Multicast delegate executed when an asset editor is actually opened */
	FAssetEditorOpenEvent AssetEditorOpenedEvent;

	/** Flag whether we are currently shutting down */
	bool bSavingOnShutdown;
	
	/**
	 * Flag whether to disable SaveOpenAssetEditors() and enable auto-restore on RestorePreviouslyOpenAssets().
	 * Useful e.g., to allow LayoutsMenu.cpp re-load layouts on-the-fly and reload the previously opened assets.
	 * If true, SaveOpenAssetEditors() will not save any asset editor and RestorePreviouslyOpenAssets() will automatically open them without asking the user.
	 * If false, SaveOpenAssetEditors() will not save any asset editor and RestorePreviouslyOpenAssets() will never open them.
	 * If unset, default behavior of both SaveOpenAssetEditors() and RestorePreviouslyOpenAssets().
	 */
	TOptional<bool> bAutoRestoreAndDisableSaving;

	/** Flag whether there has been a request to notify whether to restore previously open assets */
	bool bRequestRestorePreviouslyOpenAssets;
	
	/** True if the "Remember my choice" checkbox on the restore assets notification is checked */
	bool bRememberMyChoiceChecked = false;

	/** A pointer to the notification used by RestorePreviouslyOpenAssets */
	TWeakPtr<SNotificationItem> RestorePreviouslyOpenAssetsNotificationPtr;
	
	UPROPERTY(Transient)
	TArray<TObjectPtr<UAssetEditor>> OwnedAssetEditors;
	
	/** Map of FEditorModeId to EditorModeInfo for all known UEdModes when the subsystem initialized */
	RegisteredModeInfoMap EditorModes;

	/** Event that is triggered whenever a mode is registered or unregistered. */
	FRegisteredModesChangedEvent OnEditorModesChangedEvent;

	/** Event that is triggered whenever a mode is registered. Includes the mode's ID. */
	FOnModeRegistered OnEditorModeRegisteredEvent;

	/** Event that is triggered whenever a mode is unregistered. Includes the mode's ID. */
	FOnModeUnregistered OnEditorModeUnregisteredEvent;
	
	/**
	 * Which FEditorModeInfo data should be returned when queried, filtered by the mode's ID.
	 * Note that this does not disable or unregister disallowed modes, it simply removes them from the query results.
	 */
	FNamePermissionList AllowedEditorModes;

	/** MRU list of recently opened assets */
	TUniquePtr<FMainMRUFavoritesList> RecentAssetsList;
	
	/** Map keeping track of the asset editor each recent asset was opened in */
	TMap<FString, FString> RecentAssetToAssetEditorMap;

	/** The filter run through any assets before they are shown in the recents menu or restore prompt */
	FMainMRUFavoritesList::FDoesMRUFavoritesItemPassFilter RecentAssetsFilter;

	/** The max number of recent assets to show in the menu */
	int32 MaxRecentAssetsToShowInMenu = 20;

	/** The method any assets being opened are requested to open in, only valid during the open process (OpenEditorForAsset)
	 *  Since there is no way to generically get this information to FAssetEditorToolkit::InitAssetEditor
	 */
	TMap<FString, EAssetOpenMethod> AssetOpenMethodCache;

	/** External filters that are run to check if an asset is openable in read only mode */
	TMap<FName, FReadOnlyAssetFilter> ReadOnlyAssetFilters;

	/** Callbacks to open specific debug objects based on their type. */
	TMap<FName, FAssetEditorOpenDebugObject> DebugObjectDelegates;

	/** Callbacks to open specific debug objects exposed to blueprint. */
	UPROPERTY(BlueprintAssignable, meta = (AllowPrivateAccess = "true", Category = "Debug"))
	FAssetEditorOpenDebugObjectDynamic DebugObjectDelegate;
};
