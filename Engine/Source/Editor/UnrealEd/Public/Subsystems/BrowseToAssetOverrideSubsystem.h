// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "EditorSubsystem.h"
#include "UObject/TopLevelAssetPath.h"

#include "BrowseToAssetOverrideSubsystem.generated.h"

struct FAssetData;

using FBrowseToAssetOverrideDelegate = TDelegate<FName(const UObject*)>;

using FGetBrowseToAssetOverrideDelegate = TDelegate<FTopLevelAssetPath(const UObject*)>;

UCLASS(MinimalAPI)
class UBrowseToAssetOverrideSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	static UNREALED_API UBrowseToAssetOverrideSubsystem* Get();

	/**
	 * Given an object, see if it has a "browse to asset" override.
	 * @return Returns true if there is an override.
	 */
	UNREALED_API bool HasBrowseToAssetOverride(const UObject* Object) const;

	/**
	 * Given an object, see if it has a "browse to asset" override.
	 * @return Returns true if there is an override.
	 */
	UNREALED_API bool TryGetBrowseToAssetOverride(const UObject* Object, FTopLevelAssetPath& OutAssetPath) const;

	/**
	 * Given an object, see if it has a "browse to asset" override.
	 * @return Returns true if there is an override.
	 */
	UNREALED_API bool TryGetBrowseToAssetOverride(const UObject* Object, FAssetData& OutAssetData) const;

	/**
	 * Given an object, see if it has a "browse to asset" package name override.
	 * @return The package name of override, or None if there is no override.
	 */
	UE_DEPRECATED(5.8, "Call TryGetBrowseToAssetOverride instead.")
	UNREALED_API FName GetBrowseToAssetOverride(const UObject* Object);

	/**
	 * Register a per-class override for the "browse to asset" resolution.
	 * @note Callback should return a valid top level asset path if there is an override.
	 */
	UNREALED_API void RegisterBrowseToAssetOverrideForClass(const FTopLevelAssetPath& Class, FGetBrowseToAssetOverrideDelegate&& Callback);

	template <typename ClassType>
	void RegisterBrowseToAssetOverrideForClass(FGetBrowseToAssetOverrideDelegate&& Callback)
	{
		RegisterBrowseToAssetOverrideForClass(ClassType::StaticClass()->GetClassPathName(), MoveTemp(Callback));
	}

	/**
	 * Register a per-class override for the "browse to asset" resolution.
	 * @note Callback should return a package name, or None if there is no override.
	 */
	UE_DEPRECATED(5.8, "Use FGetBrowseToAssetOverrideDelegate overload instead.")
	UNREALED_API void RegisterBrowseToAssetOverrideForClass(const FTopLevelAssetPath& Class, FBrowseToAssetOverrideDelegate&& Callback);

	template <typename ClassType>
	UE_DEPRECATED(5.8, "Use FGetBrowseToAssetOverrideDelegate overload instead.")
	void RegisterBrowseToAssetOverrideForClass(FBrowseToAssetOverrideDelegate&& Callback)
	{
		RegisterBrowseToAssetOverrideForClass(ClassType::StaticClass()->GetClassPathName(), MoveTemp(Callback));
	}
	
	/**
	 * Unregister a per-class override for the "browse to asset" resolution.
	 */
	UNREALED_API void UnregisterBrowseToAssetOverrideForClass(const FTopLevelAssetPath& Class);

	template <typename ClassType>
	void UnregisterBrowseToAssetOverrideForClass()
	{
		UnregisterBrowseToAssetOverrideForClass(ClassType::StaticClass()->GetClassPathName());
	}

	/**
	 * Register a per-interface override for the "browse to asset" resolution.
	 * @note Callback should return a valid top level asset path if there is an override.
	 */
	UNREALED_API void RegisterBrowseToAssetOverrideForInterface(const FTopLevelAssetPath& Interface, FGetBrowseToAssetOverrideDelegate&& Callback);

	template <typename InterfaceType>
	void RegisterBrowseToAssetOverrideForInterface(FGetBrowseToAssetOverrideDelegate&& Callback)
	{
		RegisterBrowseToAssetOverrideForInterface(InterfaceType::UClassType::StaticClass()->GetClassPathName(), MoveTemp(Callback));
	}

	/**
	 * Register a per-interface override for the "browse to asset" resolution.
	 * @note Callback should return a package name, or None if there is no override.
	 */
	UE_DEPRECATED(5.8, "Use FGetBrowseToAssetOverrideDelegate overload instead.")
	UNREALED_API void RegisterBrowseToAssetOverrideForInterface(const FTopLevelAssetPath& Interface, FBrowseToAssetOverrideDelegate&& Callback);

	template <typename InterfaceType>
	UE_DEPRECATED(5.8, "Use FGetBrowseToAssetOverrideDelegate overload instead.")
	void RegisterBrowseToAssetOverrideForInterface(FBrowseToAssetOverrideDelegate&& Callback)
	{
		RegisterBrowseToAssetOverrideForInterface(InterfaceType::UClassType::StaticClass()->GetClassPathName(), MoveTemp(Callback));
	}

	/**
	 * Unregister a per-interface override for the "browse to asset" resolution.
	 */
	UNREALED_API void UnregisterBrowseToAssetOverrideForInterface(const FTopLevelAssetPath& Interface);

	template <typename InterfaceType>
	void UnregisterBrowseToAssetOverrideForInterface()
	{
		UnregisterBrowseToAssetOverrideForInterface(InterfaceType::UClassType::StaticClass()->GetClassPathName());
	}
	
private:
	TMap<FTopLevelAssetPath, FGetBrowseToAssetOverrideDelegate> PerClassOverrides;
	TMap<FTopLevelAssetPath, FGetBrowseToAssetOverrideDelegate> PerInterfaceOverrides;
};
