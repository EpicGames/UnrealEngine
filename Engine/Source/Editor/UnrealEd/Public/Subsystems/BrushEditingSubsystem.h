// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"

#include "BrushEditingSubsystem.generated.h"

class FLevelEditorViewportClient;
class FEditorViewportClient;
class FEditorModeTools;
class HHitProxy;
class ABrush;
struct FViewportClick;

UCLASS(abstract, MinimalAPI)
class UBrushEditingSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	UNREALED_API UBrushEditingSubsystem();

	UE_DEPRECATED(5.7, "Use ProcessClickOnBrushGeometry taking FEditorViewportClient as argument.")
	virtual bool ProcessClickOnBrushGeometry(FLevelEditorViewportClient* ViewportClient, HHitProxy* InHitProxy, const FViewportClick& Click) final { return false; }
	virtual bool ProcessClickOnBrushGeometry(FEditorViewportClient* ViewportClient, HHitProxy* InHitProxy, const FViewportClick& Click) { return false; }


	virtual void UpdateGeometryFromSelectedBrushes() {}
	virtual void UpdateGeometryFromBrush(ABrush* Brush) {}

	virtual bool IsGeometryEditorModeActive() const { return false; }

	virtual void DeselectAllEditingGeometry() {};

	virtual bool HandleActorDelete() { return false; };
private:

};
