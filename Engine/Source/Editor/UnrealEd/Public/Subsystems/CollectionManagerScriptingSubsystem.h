// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CollectionManagerScriptingTypes.h"
#include "EditorSubsystem.h"
#include "UObject/Package.h"

#include "CollectionManagerScriptingSubsystem.generated.h"

#define UE_API UNREALED_API

USTRUCT(BlueprintType, DisplayName="Collection Container Source", meta=(ScriptName="CollectionContainerSource"))
struct FCollectionScriptingContainerSource
{
	GENERATED_BODY()

	/** The name of the container. Defaults to the base game's container. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="CollectionManager")
	FName Name = NAME_Game;

	/** Friendly title of the container. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="CollectionManager")
	FText Title = NSLOCTEXT("CollectionScriptingContainerSource", "ProjectCollectionSource_Name", "Collections");
};

USTRUCT(BlueprintType, DisplayName="Collection", meta=(ScriptName="Collection"))
struct FCollectionScriptingRef
{
	GENERATED_BODY()

	/** The name (not title) of the container that owns this collection. Defaults to the base game's container. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="CollectionManager")
	FName Container = NAME_Game;

	/** Friendly name of the collection. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="CollectionManager")
	FName Name;

	/** Share type of this collection, which controls its visibility. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="CollectionManager")
	ECollectionScriptingShareType ShareType = ECollectionScriptingShareType::Local;
};

UCLASS(MinimalAPI, DisplayName="Collection Manager Subsystem", meta=(ScriptName="CollectionManagerSubsystem"))
class UCollectionManagerScriptingSubsystem final : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	/**
	 * Gets all available collection containers.
	 *
	 * @return Array of all collection containers.
	 */
	UFUNCTION(BlueprintCallable, Category="CollectionManager")
	UE_API TArray<FCollectionScriptingContainerSource> GetCollectionContainers();

	/**
	 * Create a new collection with the given name and share type in the provided collection container.
	 *
	 * @param Container The container the collection should be created in. "None" defaults to the base game's container.
	 * @param Collection Name to give to the collection.
	 * @param ShareType Whether the collection should be local, private, or shared?
	 * @param OutNewCollection The newly created collection.
	 *
	 * @return True if the collection was created, false otherwise (see the output log for details on error).
	 */
	UFUNCTION(BlueprintCallable, Category="CollectionManager")
	UE_API bool CreateCollection(const FCollectionScriptingContainerSource Container, const FName Collection, const ECollectionScriptingShareType ShareType, FCollectionScriptingRef& OutNewCollection);

	/**
	 * Create a new collection with the given name and share type in the provided collection container if it doesn't already exist, or empty the existing collection if it does.
	 *
	 * @param Container The container the collection should be created or found in. "None" defaults to the base game's container.
	 * @param Collection Name of the collection to create or empty.
	 * @param ShareType Whether the collection should be local, private, or shared?
	 * @param OutNewOrEmptyCollection The collection that was created or emptied.
	 *
	 * @return True if the collection was created or emptied, false otherwise (see the output log for details on error).
	 */
	UFUNCTION(BlueprintCallable, Category="CollectionManager")
	UE_API bool CreateOrEmptyCollection(const FCollectionScriptingContainerSource Container, const FName Collection, const ECollectionScriptingShareType ShareType, FCollectionScriptingRef& OutNewOrEmptyCollection);

	/**
	 * Get all available collections in the specified container.
	 *
	 * @param Container The container to retrieve collections from. "None" defaults to the base game's container.
	 * @param OutCollections The collections found in the container.
	 *
	 * @return True if OutCollections contains at least one collection, false otherwise (see the output log for details on error).
	 */
	UFUNCTION(BlueprintCallable, Category="CollectionManager")
	UE_API bool GetCollections(const FCollectionScriptingContainerSource Container, TArray<FCollectionScriptingRef>& OutCollections);

	/**
	 * Destroy the given collection.
	 *
	 * @param Collection The collection to destroy.
	 *
	 * @return True if the collection was destroyed, false otherwise (see the output log for details on error).
	 */
	UFUNCTION(BlueprintCallable, Category="CollectionManager")
	UE_API bool DestroyCollection(const FCollectionScriptingRef& Collection);

	/**
	 * Rename the given collection.
	 *
	 * @param Collection The collection to rename.
	 * @param NewName New name to give to the collection.
	 * @param NewShareType New share type to give to the collection (local, private, or shared).
	 *
	 * @return True if the collection was renamed, false otherwise (see the output log for details on error).
	 */
	UFUNCTION(BlueprintCallable, Category="CollectionManager")
	UE_API bool RenameCollection(const FCollectionScriptingRef& Collection, const FName NewName, const ECollectionScriptingShareType NewShareType);

	/**
	 * Re-parent the given collection.
	 *
	 * @param Collection The collection to re-parent.
	 * @param NewParentCollection The new parent collection, or "None" to have the collection become a root collection.
	 *
	 * @return True if the collection was re-parented, false otherwise (see the output log for details on error).
	 */
	UFUNCTION(BlueprintCallable, Category="CollectionManager")
	UE_API bool ReparentCollection(const FCollectionScriptingRef& Collection, const FCollectionScriptingRef NewParentCollection);

	/**
	 * Remove all assets from the given collection.
	 *
	 * @param Collection The collection to modify.
	 *
	 * @return True if the collection was modified, false otherwise (see the output log for details on error).
	 */
	UFUNCTION(BlueprintCallable, Category="CollectionManager")
	UE_API bool EmptyCollection(const FCollectionScriptingRef& Collection);

	/**
	 * Add the given asset to the given collection.
	 *
	 * @param Collection The collection to modify. Will be created if it does not exist.
	 * @param AssetPath Asset to add (its path name, eg) /Game/MyFolder/MyAsset.MyAsset).
	 *
	 * @return True if the collection was modified, false otherwise (see the output log for details on error).
	 */
	UFUNCTION(BlueprintCallable, Category="CollectionManager")
	UE_API bool AddAssetToCollection(const FCollectionScriptingRef& Collection, const FSoftObjectPath& AssetPath);

	/**
	 * Add the given asset to the given collection.
	 *
	 * @param Collection The collection to modify. Will be created if it does not exist.
	 * @param AssetData Asset to add.
	 *
	 * @return True if the collection was modified, false otherwise (see the output log for details on error).
	 */
	UFUNCTION(BlueprintCallable, Category="CollectionManager")
	UE_API bool AddAssetDataToCollection(const FCollectionScriptingRef& Collection, const FAssetData& AssetData);

	/**
	 * Add the given asset to the given collection.
	 *
	 * @param Collection The collection to modify. Will be created if it does not exist.
	 * @param AssetPtr Asset to add.
	 *
	 * @return True if the collection was modified, false otherwise (see the output log for details on error).
	 */
	UFUNCTION(BlueprintCallable, Category="CollectionManager")
	UE_API bool AddAssetPtrToCollection(const FCollectionScriptingRef& Collection, const UObject* AssetPtr);

	/**
	 * Add the given assets to the given collection.
	 *
	 * @param Collection The collection to modify. Will be created if it does not exist.
	 * @param AssetPaths Assets to add (their paths, eg) /Game/MyFolder/MyAsset.MyAsset).
	 *
	 * @return True if the collection was modified, false otherwise (see the output log for details on error).
	 */
	UFUNCTION(BlueprintCallable, Category="CollectionManager")
	UE_API bool AddAssetsToCollection(const FCollectionScriptingRef& Collection, const TArray<FSoftObjectPath>& AssetPaths);

	/**
	 * Add the given assets to the given collection.
	 *
	 * @param Collection The collection to modify. Will be created if it does not exist.
	 * @param AssetDatas Assets to add.
	 *
	 * @return True if the collection was modified, false otherwise (see the output log for details on error).
	 */
	UFUNCTION(BlueprintCallable, Category="CollectionManager")
	UE_API bool AddAssetDatasToCollection(const FCollectionScriptingRef& Collection, const TArray<FAssetData>& AssetDatas);

	/**
	 * Add the given assets to the given collection.
	 *
	 * @param Collection The collection to modify. Will be created if it does not exist.
	 * @param AssetPtrs Assets to add.
	 *
	 * @return True if the collection was modified, false otherwise (see the output log for details on error).
	 */
	UFUNCTION(BlueprintCallable, Category="CollectionManager")
	UE_API bool AddAssetPtrsToCollection(const FCollectionScriptingRef& Collection, const TArray<UObject*>& AssetPtrs);

	/**
	 * Remove the given asset from the given collection.
	 *
	 * @param Collection The collection to modify.
	 * @param AssetPath Asset to remove (its path, eg) /Game/MyFolder/MyAsset.MyAsset).
	 *
	 * @return True if the collection was modified, false otherwise (see the output log for details on error).
	 */
	UFUNCTION(BlueprintCallable, Category="CollectionManager")
	UE_API bool RemoveAssetFromCollection(const FCollectionScriptingRef& Collection, const FSoftObjectPath& AssetPath);

	/**
	 * Remove the given asset from the given collection.
	 *
	 * @param Collection The collection to modify.
	 * @param AssetData Asset to remove.
	 *
	 * @return True if the collection was modified, false otherwise (see the output log for details on error).
	 */
	UFUNCTION(BlueprintCallable, Category="CollectionManager")
	UE_API bool RemoveAssetDataFromCollection(const FCollectionScriptingRef& Collection, const FAssetData& AssetData);

	/**
	 * Remove the given asset from the given collection.
	 *
	 * @param Collection The collection to modify.
	 * @param AssetPtr Asset to remove.
	 *
	 * @return True if the collection was modified, false otherwise (see the output log for details on error).
	 */
	UFUNCTION(BlueprintCallable, Category="CollectionManager")
	UE_API bool RemoveAssetPtrFromCollection(const FCollectionScriptingRef& Collection, const UObject* AssetPtr);

	/**
	 * Remove the given assets from the given collection.
	 *
	 * @param Collection The collection to modify.
	 * @param AssetPaths Assets to remove (their paths, eg) /Game/MyFolder/MyAsset.MyAsset).
	 *
	 * @return True if the collection was modified, false otherwise (see the output log for details on error).
	 */
	UFUNCTION(BlueprintCallable, Category="CollectionManager")
	UE_API bool RemoveAssetsFromCollection(const FCollectionScriptingRef& Collection, const TArray<FSoftObjectPath>& AssetPaths);

	/**
	 * Remove the given assets from the given collection.
	 *
	 * @param Collection The collection to modify.
	 * @param AssetDatas Assets to remove.
	 *
	 * @return True if the collection was modified, false otherwise (see the output log for details on error).
	 */
	UFUNCTION(BlueprintCallable, Category="CollectionManager")
	UE_API bool RemoveAssetDatasFromCollection(const FCollectionScriptingRef& Collection, const TArray<FAssetData>& AssetDatas);

	/**
	 * Remove the given assets from the given collection.
	 *
	 * @param Collection The collection to modify.
	 * @param AssetPtrs Assets to remove.
	 *
	 * @return True if the collection was modified, false otherwise (see the output log for details on error).
	 */
	UFUNCTION(BlueprintCallable, Category="CollectionManager")
	UE_API bool RemoveAssetPtrsFromCollection(const FCollectionScriptingRef& Collection, const TArray<UObject*>& AssetPtrs);

	/**
	 * Check whether the given collection exists in the given container (matching both name and share type).
	 *
	 * @param Container The container to search. "None" defaults to the base game's container.
	 * @param Collection The name of the collection to look for.
	 * @param ShareType The share type of the collection to look for.
	 *
	 * @return True if the collection exists, false otherwise (if false is due to an error, see the output log for details).
	 */
	UFUNCTION(BlueprintCallable, Category="CollectionManager")
	UE_API bool CollectionExists(const FCollectionScriptingContainerSource Container, const FName Collection, const ECollectionScriptingShareType ShareType);

	/**
	 * Gets the given collections in the given container (matching only by name).
	 *
	 * @param Container The container to search. "None" defaults to the base game's container.
	 * @param Collection The collection to look for.
	 * @param OutCollections The collections found.
	 *
	 * @return True if OutCollections contains at least one collection, false otherwise (see the output log for details on error).
	 */
	UFUNCTION(BlueprintCallable, Category = "CollectionManager")
	UE_API bool GetCollectionsByName(const FCollectionScriptingContainerSource Container, const FName Collection, TArray<FCollectionScriptingRef>& OutCollections);

	/**
	 * Get the assets in the given collection.
	 *
	 * @param Collection The collection from which to retrieve assets.
	 * @param OutAssets The assets found in the collection.
	 *
	 * @return True if the collection exists, false otherwise (see the output log for details on error).
	 */
	UFUNCTION(BlueprintCallable, Category="CollectionManager")
	UE_API bool GetAssetsInCollection(const FCollectionScriptingRef& Collection, TArray<FAssetData>& OutAssets);

	/**
	 * Get the color of the given collection.
	 *
	 * @param Collection The collection to get the color of.
	 * @param OutColor The collection's color.
	 *
	 * @return True if the collection exists and has a color set, false otherwise (see the output log for details on error).
	 */
	UFUNCTION(BlueprintCallable, Category = "CollectionManager")
	UE_API bool GetCollectionColor(const FCollectionScriptingRef& Collection, FLinearColor& OutColor);

	/**
	 * Set the color of the given collection.
	 *
	 * @param Collection The collection to set the color of.
	 * @param Color The new collection color.
	 *
	 * @return True if the collection exists and the color was set, false otherwise (see the output log for details on error).
	 */
	UFUNCTION(BlueprintCallable, Category = "CollectionManager")
	UE_API bool SetCollectionColor(const FCollectionScriptingRef& Collection, const FLinearColor& Color);

	/**
	 * Clear the color of the given collection.
	 *
	 * @param Collection The collection to clear the color of.
	 *
	 * @return True if the collection exists and the color was cleared, false otherwise (see the output log for details on error).
	 */
	UFUNCTION(BlueprintCallable, Category = "CollectionManager")
	UE_API bool ClearCollectionColor(const FCollectionScriptingRef& Collection);

	/**
	 * Get the collections in the specified container that contain the given asset.
	 *
	 * @param Container Container to search for collections containing the given asset. "None" defaults to the base game's container.
	 * @param AssetPath Asset to test (its path name, eg) /Game/MyFolder/MyAsset.MyAsset).
	 * @param OutCollections Array of the collections that contain the asset.
	 *
	 * @return True if the container exists, false otherwise (see the output log for details on error).
	 */
	UFUNCTION(BlueprintCallable, Category="CollectionManager")
	UE_API bool GetCollectionsContainingAsset(const FCollectionScriptingContainerSource Container, const FSoftObjectPath& AssetPath, TArray<FCollectionScriptingRef>& OutCollections);

	/**
	 * Get the collections in the specified container that contain the given asset.
	 *
	 * @param Container Container to search for collections containing the given asset. "None" defaults to the base game's container.
	 * @param AssetData Asset to test.
	 * @param OutCollections Array of the collections that contain the asset.
	 *
	 * @return True if the container exists, false otherwise (see the output log for details on error).
	 */
	UFUNCTION(BlueprintCallable, Category="CollectionManager")
	UE_API bool GetCollectionsContainingAssetData(const FCollectionScriptingContainerSource Container, const FAssetData& AssetData, TArray<FCollectionScriptingRef>& OutCollections);

	/**
	 * Get the collections in the specified container that contain the given asset.
	 *
	 * @param Container Container to search for collections containing the given asset. "None" defaults to the base game's container.
	 * @param AssetPtr Asset to test.
	 * @param OutCollections Array of the collections that contain the asset.
	 *
	 * @return True if the container exists, false otherwise (see the output log for details on error).
	 */
	UFUNCTION(BlueprintCallable, Category="CollectionManager")
	UE_API bool GetCollectionsContainingAssetPtr(const FCollectionScriptingContainerSource Container, const UObject* AssetPtr, TArray<FCollectionScriptingRef>& OutCollections);

	/**
	 * Get the collection container for the base game.
	 *
	 * @return The collection container for the base game.
	 */
	UFUNCTION(BlueprintCallable, Category="CollectionManager")
	UE_API FCollectionScriptingContainerSource GetBaseGameCollectionContainer() const;

	/**
	 * Selects the collections in the most recently used Content Browser (summoning one if necessary)
	 * This is an asynchronous operation that can take a couple of frames to resolve the request
	 * 
	 * @param Collections The list of collections to sync to in the Content Browser
	 */
	UFUNCTION(BlueprintCallable, Category = "CollectionManager | Content Browser")
	UE_API void SyncBrowserToCollections(const TArray<FCollectionScriptingRef>& Collections);

private:
	bool SetCollectionColor(const FCollectionScriptingRef& Collection, const TOptional<FLinearColor>& Color);
};

#undef UE_API
