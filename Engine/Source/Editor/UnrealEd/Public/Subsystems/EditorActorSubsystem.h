// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorSubsystem.h"
#include "GameFramework/Actor.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "EditorActorSubsystem.generated.h"

class AActor;
class ABrush;
class UActorFactory;
class ULightComponent;

/** delegate type for triggering when new actors are dropped on to the viewport via drag and drop */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnEditNewActorsDropped, const TArray<UObject*>&, DroppedObjects, const TArray<AActor*>&, DroppedActors);
/** delegate type for triggering when new actors are placed on to the viewport. Triggers before NewActorsDropped if placement is caused by a drop action */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnEditNewActorsPlaced, UObject*, ObjToUse, const TArray<AActor*>&, PlacedActors);
/** delegate type for before edit cut actors is handled */
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnEditCutActorsBegin);
/** delegate type for after edit cut actors is handled */
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnEditCutActorsEnd);
/** delegate type for before edit copy actors is handled */
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnEditCopyActorsBegin);
/** delegate type for after edit copy actors is handled */
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnEditCopyActorsEnd); 
/** delegate type for before edit paste actors is handled */
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnEditPasteActorsBegin);
/** delegate type for after edit paste actors is handled */
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnEditPasteActorsEnd);
/** delegate type for before edit duplicate actors is handled */
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnDuplicateActorsBegin);
/** delegate type for after edit duplicate actors is handled */
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnDuplicateActorsEnd);
/** delegate type for before delete actors is handled */
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnDeleteActorsBegin);
/** delegate type for after delete actors is handled */
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnDeleteActorsEnd);
/** delegate type for when an actor changes its label. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnActorLabelChanged, AActor*, Actor);

/**
* UEditorActorUtilitiesSubsystem
* Subsystem for exposing actor related utilities to scripts,
*/
UCLASS(MinimalAPI)
class UEditorActorSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:

	UNREALED_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UNREALED_API virtual void Deinitialize() override;

	UPROPERTY(BlueprintAssignable, Category = "Editor Scripting | Level Utility")
	FOnEditNewActorsDropped OnNewActorsDropped;

	UPROPERTY(BlueprintAssignable, Category = "Editor Scripting | Level Utility")
	FOnEditNewActorsPlaced OnNewActorsPlaced;

	UPROPERTY(BlueprintAssignable, Category = "Editor Scripting | Level Utility")
	FOnEditCutActorsBegin OnEditCutActorsBegin;

	UPROPERTY(BlueprintAssignable, Category = "Editor Scripting | Level Utility")
	FOnEditCutActorsEnd OnEditCutActorsEnd;

	UPROPERTY(BlueprintAssignable, Category = "Editor Scripting | Level Utility")
	FOnEditCopyActorsBegin OnEditCopyActorsBegin;

	UPROPERTY(BlueprintAssignable, Category = "Editor Scripting | Level Utility")
	FOnEditCopyActorsEnd OnEditCopyActorsEnd;

	UPROPERTY(BlueprintAssignable, Category = "Editor Scripting | Level Utility")
	FOnEditPasteActorsBegin OnEditPasteActorsBegin;

	UPROPERTY(BlueprintAssignable, Category = "Editor Scripting | Level Utility")
	FOnEditPasteActorsEnd OnEditPasteActorsEnd;

	UPROPERTY(BlueprintAssignable, Category = "Editor Scripting | Level Utility")
	FOnEditCutActorsBegin OnDuplicateActorsBegin;

	UPROPERTY(BlueprintAssignable, Category = "Editor Scripting | Level Utility")
	FOnDuplicateActorsEnd OnDuplicateActorsEnd;

	UPROPERTY(BlueprintAssignable, Category = "Editor Scripting | Level Utility")
	FOnDeleteActorsBegin OnDeleteActorsBegin;

	UPROPERTY(BlueprintAssignable, Category = "Editor Scripting | Level Utility")
	FOnDeleteActorsEnd OnDeleteActorsEnd;

	UPROPERTY(BlueprintAssignable, Category = "Editor Scripting | Level Utility")
	FOnActorLabelChanged OnActorLabelChanged;

	struct FActorDuplicateParameters
	{
		ULevel* LevelOverride = nullptr;
		bool bTransact = true;
	};

	/**
	 * Duplicate an actor from the world editor.
	 * @param	ActorToDuplicate	Actor to duplicate.
	 * @param	ToWorld				World to place the duplicated actor in.
	 * @param	Offset				Translation to offset duplicated actor by.
	 * @return	The duplicated actor, or none if it didn't succeed
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility", meta = (AdvancedDisplay = 1))
	UNREALED_API AActor* DuplicateActor(AActor* ActorToDuplicate, UWorld* ToWorld = nullptr, FVector Offset = FVector::ZeroVector);
	UNREALED_API AActor* DuplicateActor(AActor* ActorToDuplicate, UWorld* ToWorld, FVector Offset, const FActorDuplicateParameters& DuplicateParams);

	/**
	 * Duplicate actors from the world editor.
	 * @param	ActorsToDuplicate	Actors to duplicate.
	 * @param	ToWorld				World to place the duplicated actors in.
	 * * @param	Offset				Translation to offset duplicated actors by.
	 * @return	The duplicated actors, or empty if it didn't succeed
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility", meta = (AdvancedDisplay = 1))
	UNREALED_API TArray<AActor*> DuplicateActors(const TArray<AActor*>& ActorsToDuplicate, UWorld* ToWorld = nullptr, FVector Offset = FVector::ZeroVector);
	UNREALED_API TArray<AActor*> DuplicateActors(const TArray<AActor*>& ActorsToDuplicate, UWorld* ToWorld, FVector Offset, const FActorDuplicateParameters& DuplicateParams);

	/**
	 * Duplicate all the selected actors in the given world
	 * @param	InWorld 	The world the actors are selected in.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility")
	UNREALED_API void DuplicateSelectedActors(UWorld* InWorld);

	/**
	 * Delete all the selected actors in the given world
	 * @param	InWorld 	The world the actors are selected in.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility")
	UNREALED_API void DeleteSelectedActors(UWorld* InWorld);

	/**
	 * Invert the selection in the given world
	 * @param	InWorld 	The world the selection is in.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility")
	UNREALED_API void InvertSelection(UWorld* InWorld);

	/**
	* Select all actors and BSP models in the given world, except those which are hidden
	*  @param	InWorld 	The world the actors are to be selected in.
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility")
	UNREALED_API void SelectAll(UWorld* InWorld);

	/**
	 * Select all children actors of the current selection.
	 *
	 * @param   bRecurseChildren	True to recurse through all descendants of the children
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility")
	UNREALED_API void SelectAllChildren(bool bRecurseChildren);

	/**
	 * Find all loaded Actors in the world editor. Exclude actor that are pending kill, in PIE, PreviewEditor, ...
	 * @return	List of found Actors
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility")
	UNREALED_API TArray<class AActor*> GetAllLevelActors();

	/**
	 * Find all loaded ActorComponent own by an actor in the world editor. Exclude actor that are pending kill, in PIE, PreviewEditor, ...
	 * @return	List of found ActorComponent
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility")
	UNREALED_API TArray<class UActorComponent*> GetAllLevelActorsComponents();

	/**
	 * Find all loaded Actors that are selected in the world editor. Exclude actor that are pending kill, in PIE, PreviewEditor, ...
	 * @param	ActorClass	Actor Class to find.
	 * @return	List of found Actors
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility")
	UNREALED_API TArray<class AActor*> GetSelectedLevelActors();

	/**
	 * Clear the current world editor selection and select the provided actors. Exclude actor that are pending kill, in PIE, PreviewEditor, ...
	 * @param	ActorsToSelect	Actor that should be selected in the world editor.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility")
	UNREALED_API void SetSelectedLevelActors(const TArray<class AActor*>& ActorsToSelect);

	// Remove all actors from the selection set
	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	UNREALED_API void ClearActorSelectionSet();

	// Selects nothing in the editor (another way to clear the selection)
	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	UNREALED_API void SelectNothing();

	// Set the selection state for the selected actor
	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	UNREALED_API void SetActorSelectionState(AActor* Actor, bool bShouldBeSelected);

	/**
	* Attempts to find the actor specified by PathToActor in the current editor world
	* @param	PathToActor	The path to the actor (e.g. PersistentLevel.PlayerStart)
	* @return	A reference to the actor, or none if it wasn't found
	*/
	UFUNCTION(BlueprintPure, Category = "Development|Editor")
	UNREALED_API AActor* GetActorReference(FString PathToActor);

	/**
	 * Create an actor and place it in the world editor. The Actor can be created from a Factory, Archetype, Blueprint, Class or an Asset.
	 * The actor will be created in the current level and will be selected.
	 * @param	ObjectToUse		Asset to attempt to use for an actor to place.
	 * @param	Location		Location of the new actor.
	 * @return	The created actor.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility")
	UNREALED_API AActor* SpawnActorFromObject(class UObject* ObjectToUse, FVector Location, FRotator Rotation = FRotator::ZeroRotator, bool bTransient = false);

	/**
	 * Create an actor and place it in the world editor. Can be created from a Blueprint or a Class.
	 * The actor will be created in the current level and will be selected.
	 * @param	ActorClass		Asset to attempt to use for an actor to place.
	 * @param	Location		Location of the new actor.
	 * @return	The created actor.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility", meta = (DeterminesOutputType = "ActorClass"))
	UNREALED_API AActor* SpawnActorFromClass(TSubclassOf<class AActor> ActorClass, FVector Location, FRotator Rotation = FRotator::ZeroRotator, bool bTransient = false);

	/**
	 * Destroy the actor from the world editor. Notify the Editor that the actor got destroyed.
	 * @param	ActorToDestroy	Actor to destroy.
	 * @return	True if the operation succeeds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility")
	UNREALED_API bool DestroyActor(AActor* ActorToDestroy);

	/**
	* Destroy the actors from the world editor. Notify the Editor that the actor got destroyed.
	* @param	ActorsToDestroy		Actors to destroy.
	* @return	True if the operation succeeds.
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility")
	UNREALED_API bool DestroyActors(const TArray<AActor*>& ActorsToDestroy);

	/**
	 * Replace in the level all Actors provided with a new actor of type ActorClass. Destroy all Actors provided.
	 * @param	Actors					List of Actors to replace.
	 * @param	ActorClass				Class/Blueprint of the new actor that will be spawn.
	 * @param	StaticMeshPackagePath	If the list contains Brushes and it is requested to change them to StaticMesh, StaticMeshPackagePath is the package path to where the StaticMesh will be created. ie. /Game/MyFolder/
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Dataprep", meta = (DeterminesOutputType = "ActorClass"))
	UNREALED_API TArray<class AActor*> ConvertActors(const TArray<class AActor*>& Actors, TSubclassOf<class AActor> ActorClass, const FString& StaticMeshPackagePath);

	/**
	 * Converts passed in light actors into new actors of another type.
	 * Note: This replaces the old actor with the new actor.
	 * Most properties of the old actor that can be copied are copied to the new actor during this process.
	 * Properties that can be copied are ones found in a common superclass between the actor to convert and the new
	 * class. Common light component properties between the two classes are also copied
	 *
	 * @param	InConvertToClass	The light class we are going to convert to.
	 */
	UNREALED_API static void ConvertLightActors(UClass* InConvertToClass);

	/**
	 * Converts passed in brushes into a single static mesh actor.
	 * Note: This replaces all the brushes with a single actor. This actor will not be attached to anything unless a single brush was converted.
	 *
	 * @param	InStaticMeshPackageName		The name to save the brushes to.
	 * @param	InBrushesToConvert			A list of brushes being converted.
	 *
	 * @return							Returns the newly created actor with the newly created static mesh.
	 */
	UNREALED_API static AActor* ConvertBrushesToStaticMesh(
		const FString& InStaticMeshPackageName, TArray<ABrush*>& InBrushesToConvert, const FVector& InPivotLocation
	);

	/**
	 * Converts passed in actors into new actors of the specified type.
	 * Note: This replaces the old actors with brand new actors while attempting to preserve as many properties as
	 * possible. Properties of the actors components are also attempted to be copied for any component names supplied in
	 * the third parameter. If a component name is specified, it is only copied if a component of the specified name
	 * exists in the source and destination actors, as well as in the class default object of the class of the source
	 * actor, and that all three of those components share a common base class. This approach is used instead of simply
	 * accepting component classes to copy because some actors could potentially have multiple of the same component
	 * type.
	 *
	 * @param	InActorsToConvert				Array of actors which should be converted to the new class type
	 * @param	InConvertToClass				Class to convert the provided actors to
	 * @param	InComponentsToConsider		Names of components to consider for property copying as well
	 * @param	bInUseSpecialCases			If true, looks for classes that can be handled by hardcoded conversions
	 * @param	InStaticMeshPackageName		The name to save the brushes to.
	 */
	UNREALED_API static void DoConvertActors(
		const TArray<AActor*>& InActorsToConvert,
		UClass* InConvertToClass,
		const TSet<FString>& InComponentsToConsider,
		bool bInUseSpecialCases,
		const FString& InStaticMeshPackageName
	);

	/**
	 * Sets up for a potentially deferred ConvertActors call, based on if any brushes are being converted to a static
	 * mesh. If one (or more) are being converted, the user will need to put in a package before the process continues.
	 *
	 * @param	InActorsToConvert			Array of actors which should be converted to the new class type
	 * @param	InConvertToClass			Class to convert the provided actors to
	 * @param	InComponentsToConsider	Names of components to consider for property copying as well
	 * @param	bInUseSpecialCases		If true, looks for classes that can be handled by hardcoded conversions
	 */
	UNREALED_API static void ConvertActors(
		const TArray<AActor*>& InActorsToConvert,
		UClass* InConvertToClass,
		const TSet<FString>& InComponentsToConsider,
		bool bInUseSpecialCases = false
	);

	/**
	 * Function to convert selected brushes into volumes of the provided class.
	 *
	 * @param	InVolumeClass	Class of volume that selected brushes should be converted into
	 */
	UNREALED_API static void ConvertSelectedBrushesToVolumes(UClass* InVolumeClass);

	/**
	 * Called to convert actors of one class type to another
	 *
	 * @param InFromClass The class converting from
	 * @param InToClass	The class converting to
	 */
	UNREALED_API static void ConvertActorsFromClass(const UClass* InFromClass, UClass* InToClass);

	/**
	 * Replaces the selected Actors with the same number of a different kind of Actor using the specified factory to spawn the new Actors
	 * note that only Location, Rotation, Drawscale, Drawscale3D, Tag, and Group are copied from the old Actors
	 *
	 * @param InFactory - the Factory to use to create Actors
	 */
	UNREALED_API static void ReplaceSelectedActors(
		UActorFactory* InFactory, const FAssetData& InAssetData, bool bInCopySourceProperties = true
	);

	/**
	 * Replaces specified Actors with the same number of a different kind of Actor using the specified factory to spawn the new Actors
	 * note that only Location, Rotation, Drawscale, Drawscale3D, Tag, and Group are copied from the old Actors
	 *
	 * @param InFactory - the Factory to use to create Actors
	 * @param InAssetData - the asset to feed the Factory
	 * @param InActorsToReplace - Actors to replace
	 * @param InOutNewActors - Actors that were created
	 */
	UNREALED_API static void ReplaceActors(
		UActorFactory* InFactory,
		const FAssetData& InAssetData,
		const TArray<AActor*>& InActorsToReplace,
		TArray<AActor*>* InOutNewActors = nullptr,
		bool bInCopySourceProperties = true
	);

	/**
	 * Sets the world transform of the given actor, if possible.
	 * @returns false if the world transform could not be set.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility")
	UNREALED_API bool SetActorTransform(AActor* InActor, const FTransform& InWorldTransform);

	/**
	 * Sets the world transform of the given component, if possible.
	 * @returns false if the world transform could not be set.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility")
	UNREALED_API bool SetComponentTransform(USceneComponent* InSceneComponent, const FTransform& InWorldTransform);

private:

	/** To fire before an Actor is Dropped */
	void BroadcastEditNewActorsDropped(const TArray<UObject*>& DroppedObjects, const TArray<AActor*>& DroppedActors);

	/** To fire before an Actor is Placed */
	void BroadcastEditNewActorsPlaced(UObject* ObjToUse, const TArray<AActor*>& PlacedActors);

	/** To fire before an Actor is Cut */
	void BroadcastEditCutActorsBegin();

	/** To fire after an Actor is Cut */
	void BroadcastEditCutActorsEnd();

	/** To fire before an Actor is Copied */
	void BroadcastEditCopyActorsBegin();

	/** To fire after an Actor is Copied */
	void BroadcastEditCopyActorsEnd();

	/** To fire before an Actor is Pasted */
	void BroadcastEditPasteActorsBegin();

	/** To fire after an Actor is Pasted */
	void BroadcastEditPasteActorsEnd();

	/** To fire before an Actor is duplicated */
	void BroadcastDuplicateActorsBegin();

	/** To fire after an Actor is duplicated */
	void BroadcastDuplicateActorsEnd();

	/** To fire before an Actor is Deleted */
	void BroadcastDeleteActorsBegin();

	/** To fire after an Actor is Deleted */
	void BroadcastDeleteActorsEnd();
	
	/** To fire after an Actor has changed label */
	void BroadcastActorLabelChanged(AActor* Actor);
};

