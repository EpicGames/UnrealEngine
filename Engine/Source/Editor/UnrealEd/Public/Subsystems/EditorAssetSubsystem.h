// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetRegistry/AssetData.h"
#include "EditorSubsystem.h"
#include "EditorAssetSubsystem.generated.h"

UENUM(BlueprintType, DisplayName = "Sort Order")
enum class EEditorAssetSortOrder : uint8
{
	Ascending,
	Descending
};

UENUM(BlueprintType, DisplayName = "Meta Data Sort Type")
enum class EEditorAssetMetaDataSortType : uint8
{
	String,
	Numeric,
	DateTime
};

/**
* UEditorAssetSubsystem
* Subsystem for exposing asset related utilities to scripts.
* Asset Paths can be represented in the following ways:
*		(Reference/Text Path)	StaticMesh'/Game/MyFolder/MyAsset.MyAsset'
*		(Full Name)				StaticMesh /Game/MyFolder/MyAsset.MyAsset
*		(Path Name)				/Game/MyFolder/MyAsset.MyAsset
*		(Package Name)			/Game/MyFolder/MyAsset
* Directory Paths can be represented in the following ways:
*		/Game/MyNewFolder/
*		/Game/MyNewFolder
*/
UCLASS(MinimalAPI)
class UEditorAssetSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()
	
public:

	UNREALED_API UEditorAssetSubsystem();
	UNREALED_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UNREALED_API virtual void Deinitialize() override;
	
	/**
	 * Load an asset. It will verify if the object is already loaded and only load it if it's necessary.
	 * @param	AssetPath		Asset Path of the asset to load
	 * @return	Found or loaded asset.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API UObject* LoadAsset(const FString& AssetPath);

	/**
	 * Load a Blueprint asset and return its generated class. It will verify if the object is already loaded and only load it if it's necessary.
	 * @param	AssetPath		Asset Path of the Blueprint asset.
	 * @return	Found or loaded class.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API UClass* LoadBlueprintClass(const FString& AssetPath);

	/**
	 * Return a valid AssetPath for a loaded asset.
	 * Similar to GetPathName(). The format will be: /Game/MyFolder/MyAsset.MyAsset
	 * @param	LoadedAsset		The loaded asset to get the path of.
	 * @return	If valid, the asset Path of the loaded asset.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API FString GetPathNameForLoadedAsset(UObject* LoadedAsset);

	/**
	 * Return the AssetData for the Asset that can then be used with AssetRegistryHelpers.
	 * @param	AssetPath	Asset Path to retrieve data from.
	 * @return	The AssetData found.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API FAssetData FindAssetData(const FString& AssetPath);

	/**
	 * Check if an asset exists in the Asset Registry.
	 * @param	AssetPath		Asset Path to check for existence.
	 * @return	True if the asset exists and is valid.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API bool DoesAssetExist(const FString& AssetPath);

	/**
	 * Check if assets exist in the Asset Registry.
	 * @param	AssetPaths		Asset Paths of the assets to check for existence.
	 * @return	True if all assets exist and are valid.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API bool DoAssetsExist(const TArray<FString>& AssetPaths);
	
	/**
	 * Find Package Referencers for an asset. Only Soft and Hard dependencies will be looked for.
	 * Soft are dependencies which don't need to be loaded for the object to be used.
	 * Hard are dependencies which are required for correct usage of the source asset and must be loaded at the same time.
	 * Other references may exist. The asset may be currently used in memory by another asset, by the editor or by code.
	 * Package dependencies are cached with the asset. False positives can happen until all the assets are loaded and re-saved.
	 * @param	AssetPath				Asset Path of the asset that we are looking for.
	 * @param	bLoadAssetsToConfirm	Whether the asset and any potential referencers will be loaded to confirm the dependencies.
	 * @return	The package paths of the referencers.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset", meta=(AdvancedDisplay=1))
	UNREALED_API TArray<FString> FindPackageReferencersForAsset(const FString& AssetPath, bool bLoadAssetsToConfirm = false);

	/**
	 * Consolidates assets by replacing all references/uses of the provided AssetsToConsolidate with references to AssetToConsolidateTo.
	 * This is useful when you want all references of assets to be replaced by a single asset.
	 * The function first attempts to directly replace all relevant references located within objects that are already loaded and in memory.
	 * Next, it deletes the AssetsToConsolidate, leaving behind object redirectors to AssetToConsolidateTo.
	 * @param	AssetToConsolidateTo	Asset to which all references of the AssetsToConsolidate will instead refer to after this operation completes.
	 * @param	AssetsToConsolidate		All references to these assets will be modified to reference AssetToConsolidateTo instead.
	 * @note	The AssetsToConsolidate are DELETED by this function.
	 * @note	Modified objects will be saved if the operation succeeds.
	 * @return	True if the operation succeeds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API bool ConsolidateAssets(UObject* AssetToConsolidateTo, const TArray<UObject*>& AssetsToConsolidate);
	
	/**
	 * Delete an asset that is already loaded.
	 * This is a Force Delete. It does not check if the asset has references in other Levels or by Actors.
	 * It will close all the asset editors and may clear the Transaction buffer (Undo History).
	 * Will try to mark the file as deleted.
	 * @param	AssetToDelete			Asset to delete.
	 * @return	True if the operation succeeds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API bool DeleteLoadedAsset(UObject* AssetToDelete);

	/**
	 * Delete assets that are already loaded.
	 * This is a Force Delete. It does not check if the assets have references in other Levels or by Actors.
	 * It will close all the asset editors and may clear the Transaction buffer (Undo History).
	 * Will try to mark the files as deleted.
	 * @param	AssetsToDelete			Loaded assets to delete.
	 * @return	True if the operation succeeds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API bool DeleteLoadedAssets(const TArray<UObject*>& AssetsToDelete);

	/**
	 * Delete the package an asset is in. All objects in the package will be deleted.
	 * This is a Force Delete. It does not check if the asset has references in other Levels or by Actors.
	 * It will close all the asset editors and may clear the Transaction buffer (Undo History).
	 * Will try to mark the file as deleted. The Asset will be loaded before being deleted.
	 * @param	AssetPathToDelete		Asset Path of the asset to delete.
	 * @return	True if the operation succeeds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API bool DeleteAsset(const FString& AssetPathToDelete);

	/**
	 * Delete the packages inside a directory. If the directory is then empty, delete the directory.
	 * This is a Force Delete. It does not check if the assets have references in other Levels or by Actors.
	 * It will close all the asset editors and may clear the Transaction buffer (Undo History).
	 * Will try to mark the file as deleted. Assets will be loaded before being deleted.
	 * The search is always recursive. It will try to delete the sub folders.
	 * @param	DirectoryPath		Directory that will be marked for deletion and deleted.
	 * @return	True if the operation succeeds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API bool DeleteDirectory(const FString& DirectoryPath);
	
	/**
	 * Duplicate an asset that is already loaded. Will try to checkout the file.
	 * @param	SourceAsset				Asset that we want to copy from.
	 * @param	DestinationAssetPath	Asset Path of the duplicated asset.
	 * @return	The duplicated object if the operation succeeds
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API UObject* DuplicateLoadedAsset(UObject* SourceAsset, const FString& DestinationAssetPath);

	/**
	 * Duplicate an asset. Will try to checkout the file. The Asset will be loaded before being duplicated.
	 * @param	SourceAssetPath			Asset Path of the asset that we want to copy from.
	 * @param	DestinationAssetPath	Asset Path of the duplicated asset.
	 * @return	The duplicated object if the operation succeeds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API UObject* DuplicateAsset(const FString& SourceAssetPath, const FString& DestinationAssetPath);

	/**
	 * Duplicate a directory and the assets in it.
	 * Will try to checkout the files. The Assets will be loaded before being duplicated.
	 * @param	SourceDirectoryPath			Directory of the assets that we want to duplicate from.
	 * @param	DestinationDirectoryPath	Directory of the duplicated asset.
	 * @return	The duplicated object if the operation succeeds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API bool DuplicateDirectory(const FString& SourceDirectoryPath, const FString& DestinationDirectoryPath);

	/**
	 * Rename an asset that is already loaded. Equivalent to a Move operation.
	 * Will try to checkout the file.
	 * @param	SourceAsset				Asset that we want to copy from.
	 * @param	DestinationAssetPath	Asset Path of the duplicated asset.
	 * @return	True if the operation succeeds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API bool RenameLoadedAsset(UObject* SourceAsset, const FString& DestinationAssetPath);

	/**
	 * Rename an asset. Equivalent to a Move operation.
	 * Will try to checkout the file. The Asset will be loaded before being renamed.
	 * @param	SourceAssetPath			Asset Path of the asset that we want to copy from.
	 * @param	DestinationAssetPath	Asset Path of the renamed asset.
	 * @return	True if the operation succeeds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API bool RenameAsset(const FString& SourceAssetPath, const FString& DestinationAssetPath);

	/**
	 * Rename a directory. Equivalent to a Move operation moving all contained assets.
	 * Will try to checkout the files. The Assets will be loaded before being renamed.
	 * @param	SourceDirectoryPath			Directory of the assets that we want to rename from.
	 * @param	DestinationDirectoryPath	Directory of the renamed asset.
	 * @return	True if the operation succeeds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API bool RenameDirectory(const FString& SourceDirectoryPath, const FString& DestinationDirectoryPath);
	
	/**
	 * Set the package dirty flag for an asset
	 * @param	Object			Object we want to set the package dirty state.
	 * @param	bDirtyState		The dirty state, true mean the asset package need to be save.
	 * @return	True if the operation succeeds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API bool SetDirtyFlag(UObject* Object, const bool bDirtyState);

	/**
	 * Checkout the asset corresponding to an object.
	 * @param	AssetToCheckout		Asset to checkout.
	 * @return	True if the operation succeeds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset | Revision Control", meta = (Keywords = "Source Control"))
	UNREALED_API bool CheckoutLoadedAsset(UObject* AssetToCheckout);

	/**
	 * Checkout the assets.
	 * @param	AssetsToCheckout		Assets to checkout.
	 * @return	True if the operation succeeds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset | Revision Control", meta = (Keywords = "Source Control"))
	UNREALED_API bool CheckoutLoadedAssets(const TArray<UObject*>& AssetsToCheckout);

	/**
	 * Checkout an asset.
	 * @param	AssetToCheckout		Asset Path of the asset that we want to checkout.
	 * @return	True if the operation succeeds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset | Revision Control", meta = (Keywords = "Source Control"))
	UNREALED_API bool CheckoutAsset(const FString& AssetToCheckout);

	/**
	 * Checkout all assets in a directory. It will load the assets if needed.
	 * All objects that are in the directory will be checked out. Assets will be loaded before being checked out.
	 * @param	DirectoryPath		Directory of the assets to be checked out.
	 * @param	bRecursive			If the AssetPath is a folder, the search will be recursive and will checkout the assets in the sub folders.
	 * @return	True if the operation succeeds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset | Revision Control", meta = (Keywords = "Source Control"))
	UNREALED_API bool CheckoutDirectory(const FString& DirectoryPath, bool bRecursive = true);
	
	/**
	 * Save the package the asset lives in. All objects that live in the package will be saved. Will try to checkout the file.
	 * @param	AssetToSave			Asset that we want to save.
	 * @param	bOnlyIfIsDirty		Only checkout asset that are dirty.
	 * @return	True if the operation succeeds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API bool SaveLoadedAsset(UObject* AssetToSave, bool bOnlyIfIsDirty = true);

	/**
	 * Save the packages the assets live in. All objects that live in the packages will be saved. Will try to checkout the files.
	 * @param	AssetsToSave		Assets that we want to save.
	 * @param	bOnlyIfIsDirty		Only checkout asset that are dirty.
	 * @return	True if the operation succeeds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API bool SaveLoadedAssets(const TArray<UObject*>& AssetsToSave, bool bOnlyIfIsDirty = true);

	/**
	 * Save the packages the assets live in. All objects that live in the package will be saved.
	 * Will try to checkout the file first. The Asset will be loaded before being saved.
	 * @param	AssetToSave		Asset Path of the asset that we want to save.
	 * @param	bOnlyIfIsDirty		Only checkout/save the asset if it's dirty.
	 * @return	True if the operation succeeds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API bool SaveAsset(const FString& AssetToSave, bool bOnlyIfIsDirty = true);

	/** Returns the length of the computed cooked package name and path */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API int32 GetAssetFilenameLengthForCooking(const FString& AssetPath);

	/** Returns the length of the computed cooked package name and path */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API int32 GetLoadedAssetFilenameLengthForCooking(const UObject* Asset);

	/**
	 * Save the packages the assets live in inside the directory. All objects that are in the directory will be saved.
	 * Will try to checkout the file first. Assets will be loaded before being saved.
	 * @param	DirectoryPath		Directory that will be checked out and saved.
	 * @param	bOnlyIfIsDirty		Only checkout asset that are dirty.
	 * @param	bRecursive			The search will be recursive and it will save the asset in the sub folders.
	 * @return	True if the operation succeeds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API bool SaveDirectory(const FString& DirectoryPath, bool bOnlyIfIsDirty = true, bool bRecursive = true);
	
	/**
	* Check if a directory exists.
	* @param	DirectoryPath		Long Path Name of the directory.
	* @return	True if it does exist and it is valid.
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API bool DoesDirectoryExist(const FString& DirectoryPath);

	/**
	 * Check if a directory contains any assets.
	 * @param	DirectoryPath		Long Path Name of the directory.
	 * @return	True if there is any assets.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API bool DoesDirectoryContainAssets(const FString& DirectoryPath, bool bRecursive = true);

	/**
	 * Create a directory on disk.
	 * @param	DirectoryPath		Long Path Name of the directory.
	 * @return	True if the operation succeeds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API bool MakeDirectory(const FString& DirectoryPath);
	
	/**
	 * Return the list of all the assets found in the DirectoryPath.
	 * @param	DirectoryPath		Directory path of the asset we want the list from.
	 * @param	bRecursive			The search will be recursive and will look in sub folders.
	 * @param	bIncludeFolder		The result will include folders name.
	 * @return	The list of assets found.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API TArray<FString> ListAssets(const FString& DirectoryPath, bool bRecursive = true, bool bIncludeFolder = false);

	/**
	 * Return the list of all the assets that have the pair of Tag/Value.
	 * @param TagName	The tag associated with the assets requested.
	 * @param TagValue	The value associated with the assets requested.
	 * @return	The list of assets found.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API TArray<FString> ListAssetsByTagValue(FName TagName, const FString& TagValue);

	/**
	 * Gets all TagValues (from Asset Registry) associated with an (unloaded) asset as strings value.
	 * @param	AssetPath		Asset Path we are trying to find.
	 * @return	The list of all TagNames & TagValues.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API TMap<FName, FString> GetTagValues(const FString& AssetPath);
	
	/**
     * Get all tags/values of a loaded asset's metadata.
     * @param	Object		The object from which to retrieve the metadata.
     * @return	The list of all Tags and Values.
     */
    UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset | Metadata")
    UNREALED_API TMap<FName, FString> GetMetadataTagValues(UObject* Object);
   
    /**
     * Get the value associated with the given tag of a loaded asset's metadata.
     * @param	Object		The object from which to retrieve the metadata.
     * @param	Tag			The tag to find in the metadata.
     * @return	The string value associated with the tag.
     */
    UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset | Metadata")
    UNREALED_API FString GetMetadataTag(UObject* Object, FName Tag);
   
    /**
     * Set the value associated with a given tag of a loaded asset's metadata.
     * @param	Object		The object from which to retrieve the metadata.
     * @param	Tag			The tag to set in the metadata.
     * @param	Value		The string value to associate with the tag.
     */
    UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset | Metadata")
    UNREALED_API void SetMetadataTag(UObject* Object, FName Tag, const FString& Value);
   
    /**
     * Remove the given tag from a loaded asset's metadata.
     * @param	Object		The object from which to retrieve the metadata.
     * @param	Tag			The tag to remove from the metadata.
     */
    UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset | Metadata")
    UNREALED_API void RemoveMetadataTag(UObject* Object, FName Tag);

	/**
	 * Delegate for extracting an asset from a file, for example from a drag and drop operation.
	 *
	 * @param Files				List of files.
	 * @param AssetDataArray	Add extracted assets here.
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnExtractAssetFromFile, const TArray<FString>& Files, TArray<FAssetData>& AssetDataArray);

	/**
	 * Dynamic version of FOnExtractAssetFromFile.
	 */
	DECLARE_DYNAMIC_DELEGATE_TwoParams(FOnExtractAssetFromFileDynamic, const TArray<FString>&, Files, TArray<FAssetData>&, AssetDataArray);

	/**
	 * Call this to add a callback to extract an asset from a file,
	 * for example from a drag and drop operation.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API void AddOnExtractAssetFromFile(FOnExtractAssetFromFileDynamic Delegate);

	/**
	 * Call this to remove a callback added with AddOnExtractAssetFromFile.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API void RemoveOnExtractAssetFromFile(FOnExtractAssetFromFileDynamic Delegate);
	
	/**
	 * Get the delegate for extracting an asset from a file,
	 * for example from a drag and drop operation.
	 * Add to this so you can do your own extraction.
	 *
	 * Broadcasting this will also call anything added with AddOnExtractAssetFromFile.
	 */
	FOnExtractAssetFromFile& GetOnExtractAssetFromFile() { return OnExtractAssetFromFile; }
	
	/**
	 * Gets all assets which have the given tags.
	 * 
	 * @params RequiredTags The tags the assets should have
	 * @params AllowedClasses The class types the contained assets should have
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API TArray<FAssetData> GetAllAssetsByMetaDataTags(const TSet<FName>& RequiredTags, const TSet<UClass*>& AllowedClasses);
	
	/**
	 * Sorts the assets based on their meta data's type.
	 * Supported types: FString, int, float, FDateTime.
	 * 
	 * @param Assets The assets to sort
	 * @param MetaDataTag The on which the sort is based
	 * @param MetaDataType The meta data type of MetaDataTag
	 * @param SortOrder Whether to sort ascending or descending
	 * @return Whether the data was sorted, e.g. false if not all assets have the MetaDataTag.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Asset")
	UNREALED_API bool SortByMetaData(UPARAM(Ref) TArray<FAssetData>& Assets, FName MetaDataTag, EEditorAssetMetaDataSortType MetaDataType, EEditorAssetSortOrder SortOrder);

private:

	/**
	 * Calls all the delegates in OnExtractAssetFromFileDynamicArray when OnExtractAssetFromFile
	 * is broadcast.
	 */
	UNREALED_API void CallOnExtractAssetFromFileDynamicArray(const TArray<FString>& Files,
		TArray<FAssetData>& InAssetDataArray);

	/** Delegate for extracting an asset from a file, for example from a drag and drop operation. */
	FOnExtractAssetFromFile OnExtractAssetFromFile;
	/** Array of dynamic delegates for extracting an asset from a file. */
	TArray<FOnExtractAssetFromFileDynamic> OnExtractAssetFromFileDynamicArray;

};
