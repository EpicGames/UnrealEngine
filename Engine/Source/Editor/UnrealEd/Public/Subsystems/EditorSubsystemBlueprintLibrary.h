// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SubclassOf.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "EditorSubsystem.h"

#include "EditorSubsystemBlueprintLibrary.generated.h"

UCLASS(MinimalAPI)
class UEditorSubsystemBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** Get a Local Player Subsystem from the Local Player associated with the provided context */
	UFUNCTION(BlueprintPure, Category = "Editor Subsystems", meta = (BlueprintInternalUseOnly = "true"))
	static UNREALED_API UEditorSubsystem* GetEditorSubsystem(TSubclassOf<UEditorSubsystem> Class);

	UFUNCTION()
	TArray<FName> GetPreviewPlatformOptions() const;
	
	/**
	 * Toggle the Preview Platform.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor")
	static UNREALED_API void TogglePreviewPlatform();

	/**
	 * Sets the Preview Platform
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor")
	static UNREALED_API void SetPreviewPlatform(UPARAM(meta = (GetOptions = "GetPreviewPlatformOptions")) FName PreviewShaderPlatformName);

	/**
	 * Gets the Preview Platform Name
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor")
	static UNREALED_API FName GetPreviewPlatformName();

	/**
	 * Disable the Preview Platform.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor")
	static UNREALED_API void DisablePreviewPlatform();
};