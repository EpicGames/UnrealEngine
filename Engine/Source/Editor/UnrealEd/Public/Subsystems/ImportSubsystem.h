// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"

#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Templates/SubclassOf.h"
#include "Containers/Queue.h"

#include "ImportSubsystem.generated.h"

class UFactory;

/**
 * Interface for tasks that need delayed execution
 */
class IImportSubsystemTask
{
public:
	virtual ~IImportSubsystemTask() = default;

	virtual void Run() = 0;
};

/**
 * UImportSubsystem
 * Subsystem for importing assets in the editor, 
 * Contains utility functions and callbacks for hooking into importing.
 */
UCLASS(MinimalAPI)
class UImportSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	UNREALED_API UImportSubsystem();

	UNREALED_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UNREALED_API virtual void Deinitialize() override;

	/* Import files next tick */
	UNREALED_API void ImportNextTick(const TArray<FString>& Files, const FString& DestinationPath);

	/** delegate type fired when new assets are being (re-)imported. Params: UFactory* InFactory, UClass* InClass, UObject* InParent, const FName& Name, const TCHAR* Type */
	DECLARE_MULTICAST_DELEGATE_FiveParams(FOnAssetPreImport, UFactory*, UClass*, UObject*, const FName&, const TCHAR*);
	/** delegate type fired when new assets have been (re-)imported. Note: InCreatedObject can be NULL if import failed. Params: UFactory* InFactory, UObject* InCreatedObject */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnAssetPostImport, UFactory*, UObject*);
	/** delegate type fired when new assets have been reimported. Note: InCreatedObject can be NULL if import failed. Params: UObject* InCreatedObject */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnAssetReimport, UObject*);
	/** delegate type fired when new LOD have been imported to an asset. */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnAssetPostLODImport, UObject*, int32);

	// Broadcast AssetPreImport, do not broadcast with OnAssetPostImport directly.
	UNREALED_API void BroadcastAssetPreImport(UFactory* InFactory, UClass* InClass, UObject* InParent, const FName& Name, const TCHAR* Type);
	// Broadcast AssetPostImport, do not broadcast with OnAssetPostImport directly.
	UNREALED_API void BroadcastAssetPostImport(UFactory* InFactory, UObject* InCreatedObject);
	// Broadcast AssetReimport, do not broadcast with OnAssetReimport directly.
	UNREALED_API void BroadcastAssetReimport(UObject* InCreatedObject);
	// Broadcast AssetPostLODImport, do not broadcast with OnAssetPostLODImport directly.
	UNREALED_API void BroadcastAssetPostLODImport(UObject* InObject, int32 inLODIndex);

	// Used to register and unregister ONLY use Broadcast functions to execute the delegate
	FOnAssetPreImport OnAssetPreImport;
	FOnAssetPostImport OnAssetPostImport;
	FOnAssetReimport OnAssetReimport;
	FOnAssetPostLODImport OnAssetPostLODImport;

private:

	void InterchangeBroadcastAssetPostImport(UObject* InCreatedObject);
	void InterchangeBroadcastAssetPostReimport(UObject* InCreatedObject);

	/* Run deferred logic waiting to be run next tick */
	void HandleNextTick();

	/** delegate type fired when new assets are being (re-)imported. Params: UFactory* InFactory, UClass* InClass, UObject* InParent, const FName& Name, const TCHAR* Type */
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_FiveParams(FOnAssetPreImport_Dyn, UFactory*, InFactory, UClass*, InClass, UObject*, InParent, const FName&, Name, const FString, Type);
	/** delegate type fired when new assets have been (re-)imported. Note: InCreatedObject can be NULL if import failed. Params: UFactory* InFactory, UObject* InCreatedObject */
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnAssetPostImport_Dyn, UFactory*, InFactory, UObject*, InCreatedObject);
	/** delegate type fired when new assets have been reimported. Note: InCreatedObject can be NULL if import failed. Params: UObject* InCreatedObject */
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnAssetReimport_Dyn, UObject*, InCreatedObject);
	/** delegate type fired when new LOD have been imported to an asset. */
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnAssetPostLODImport_Dyn, UObject*, InObject, int32, InLODIndex);

	UPROPERTY(BlueprintAssignable, DisplayName = "OnAssetPreImport", meta = (ScriptName = "OnAssetPreImport"))
	FOnAssetPreImport_Dyn OnAssetPreImport_BP;
	UPROPERTY(BlueprintAssignable, DisplayName = "OnAssetPostImport", meta = (ScriptName = "OnAssetPostImport"))
	FOnAssetPostImport_Dyn OnAssetPostImport_BP;
	UPROPERTY(BlueprintAssignable, DisplayName = "OnAssetReimport", meta = (ScriptName = "OnAssetReimport"))
	FOnAssetReimport_Dyn OnAssetReimport_BP;
	UPROPERTY(BlueprintAssignable, DisplayName = "OnAssetPostLODImport", meta = (ScriptName = "OnAssetPostLODImport"))
	FOnAssetPostLODImport_Dyn OnAssetPostLODImport_BP;

	/* Tasks waiting to be run next tick */
	TQueue<TSharedPtr<IImportSubsystemTask>> PendingTasks;
};
