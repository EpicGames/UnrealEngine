// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "Shader/ShaderTypes.h"
#include "MaterialShaderValueTypeSubsystem.generated.h"

class UMaterialShaderValueTypeObject;

/**
 * Editor subsystem that owns the flyweight UMaterialShaderValueTypeObject instances.
 * Lifetime tied to GEditor — flyweights are GC-traced via UPROPERTY and collected
 * naturally when the subsystem is destroyed during GEditor teardown.
 */
UCLASS(MinimalAPI)
class UMaterialShaderValueTypeSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	/** Get or create the flyweight for a given EValueType. */
	UMaterialShaderValueTypeObject* GetOrCreate(UE::Shader::EValueType InType);

private:
	UPROPERTY()
	TMap<uint8, TObjectPtr<UMaterialShaderValueTypeObject>> Flyweights;
};
