// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"

#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Templates/SubclassOf.h"
#include "Containers/Queue.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#include "PanelExtensionSubsystem.generated.h"

#define UE_API UNREALED_API

PRAGMA_DISABLE_DEPRECATION_WARNINGS

struct FPanelExtensionFactory
{
public:
	/** An identifier to allow removal later on. */
	FName Identifier;

	/** Weight used when sorting against other factories for this panel (higher weights sort first) */
	int32 SortWeight = 0;

	/**
	 * Delegate that generates the SExtensionPanel content widget.
	 * The FWeakObjectPtr param is an opaque context specific to the panel that is being extended
	 * which can be used to customize or populate the extension widget.
	 */
	DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<SWidget>, FCreateExtensionWidget, FWeakObjectPtr);
	FCreateExtensionWidget CreateExtensionWidget;

	UE_DEPRECATED(4.26, "Use FCreateExtensionWidget instead")
	DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<SWidget>, FGenericCreateWidget, const TArray<UObject*>&);

	UE_DEPRECATED(4.26, "Use CreateExtensionWidget instead")
	FGenericCreateWidget CreateWidget;
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS

template <>
struct TWidgetTypeTraits<class SExtensionPanel>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SExtensionPanel : public SCompoundWidget
{
public:
	UE_API ~SExtensionPanel();

	SLATE_BEGIN_ARGS(SExtensionPanel)
		: _ExtensionPanelID()
		, _DefaultWidget()
		, _ExtensionContext()
		, _WindowZoneOverride()
	{
	}

		/** The ID to identify this Extension point */
		SLATE_ATTRIBUTE(FName, ExtensionPanelID)
		/** The widget to use as content if FPanelExtensionFactory::CreateExtensionWidget returns nullptr */
		SLATE_ATTRIBUTE(TSharedPtr<SWidget>, DefaultWidget)
		/** Context used to customize or populate the extension widget (specific to each panel extension) */
		SLATE_ATTRIBUTE(FWeakObjectPtr, ExtensionContext)
		/** The window zone to return for this widget. Set to EWindowZone::TitleBar to make the window draggable on this widget. */
		SLATE_ATTRIBUTE(EWindowZone::Type, WindowZoneOverride)

	SLATE_END_ARGS()

	/** Constructs the widget */
	UE_API void Construct(const FArguments& InArgs);

	UObject* GetExtensionContext() const
	{
		return ExtensionContext.Get();
	}

	virtual EWindowZone::Type GetWindowZoneOverride() const override
	{
		return WindowZoneOverride.Get(EWindowZone::Unspecified);
	}

private:
	UE_API void RebuildWidget();

	FName ExtensionPanelID;
	TSharedPtr<SWidget> DefaultWidget;
	FWeakObjectPtr ExtensionContext;
	TAttribute<EWindowZone::Type> WindowZoneOverride;
};

/**
 * UPanelExtensionSubsystem
 * Subsystem for creating extensible panels in the Editor
 */
UCLASS(MinimalAPI)
class UPanelExtensionSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	UNREALED_API UPanelExtensionSubsystem();

	UNREALED_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UNREALED_API virtual void Deinitialize() override;

	UNREALED_API void RegisterPanelFactory(FName ExtensionPanelID, const FPanelExtensionFactory& InPanelExtensionFactory);
	UNREALED_API void UnregisterPanelFactory(FName Identifier, FName ExtensionPanelID = NAME_None);
	UNREALED_API bool IsPanelFactoryRegistered(FName Identifier, FName ExtensionPanelID = NAME_None) const;
	
	UNREALED_API void MarkDeprecatedExtension(FName ExtensionID, const FText& Message);

protected:
	friend class SExtensionPanel;
	UNREALED_API TSharedRef<SWidget> CreateWidget(FName ExtensionPanelID, FWeakObjectPtr ExtensionContext);

	DECLARE_MULTICAST_DELEGATE(FPanelFactoryRegistryChanged);
	UNREALED_API FPanelFactoryRegistryChanged& OnPanelFactoryRegistryChanged(FName ExtensionPanelID);

private:
	TMap<FName, TArray<FPanelExtensionFactory>> ExtensionPointMap;
	TMap<FName, FPanelFactoryRegistryChanged> PanelFactoryRegistryChangedCallbackMap;
	TMap<FName, FText> DeprecatedExtensions;
};

#undef UE_API
