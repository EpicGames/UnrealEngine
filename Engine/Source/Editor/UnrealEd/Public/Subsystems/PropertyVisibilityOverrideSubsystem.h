// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"

#include "PropertyVisibilityOverrideSubsystem.generated.h"

class FProperty;
class IPropertyHandle;
class UObject;

namespace UE::PropertyVisibility
{
	UNREALED_API bool ConsiderPropertyForOverriddenState(TNotNull<const FProperty*> Property);
}

/** Delegate for global property visibility checks (FProperty* only, called from multiple contexts) */
DECLARE_DELEGATE_RetVal_OneParam(bool, FShouldHidePropertyDelegate, const FProperty*);

/** Delegate for details-panel-specific property visibility checks (IPropertyHandle, called only from the details panel) */
DECLARE_DELEGATE_RetVal_OneParam(bool, FShouldHideDetailsPanelPropertyDelegate, const IPropertyHandle&);

UCLASS(MinimalAPI)
class UPropertyVisibilityOverrideSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	UNREALED_API static UPropertyVisibilityOverrideSubsystem* Get();

	virtual bool ShouldCreateSubsystem(UObject* Outer) const override { return true; }

	UNREALED_API void RegisterShouldHidePropertyDelegate(const FName& DelegateName, const FShouldHidePropertyDelegate& Delegate);
	UNREALED_API void UnregisterShouldHidePropertyDelegate(const FName& DelegateName);

	UNREALED_API virtual bool ShouldHideProperty(const FProperty* Property) const;

	UNREALED_API void RegisterShouldHideDetailsPanelPropertyDelegate(const FName& DelegateName, const FShouldHideDetailsPanelPropertyDelegate& Delegate);
	UNREALED_API void UnregisterShouldHideDetailsPanelPropertyDelegate(const FName& DelegateName);

	/**
	 * Details-panel-specific visibility check.
	 * Only call from the details panel where FScopedPushContext ensures valid TLS context.
	 * Returns true if any registered delegate says the property should be hidden.
	 */
	UNREALED_API bool ShouldHidePropertyForDetailsPanel(const IPropertyHandle& PropertyHandle) const;

private:
	TMap<FName, FShouldHidePropertyDelegate> ShouldHidePropertyDelegates;
	TMap<FName, FShouldHideDetailsPanelPropertyDelegate> DetailsPanelShouldHidePropertyDelegates;
};
