// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorSubsystem.h"

#include "UnrealEditorSubsystem.generated.h"

class FLevelEditorViewportClient;

/**
* UUnrealEditorSubsystem 
* Subsystem for exposing editor functionality to scripts
*/
UCLASS(MinimalAPI)
class UUnrealEditorSubsystem  : public UEditorSubsystem
{
	GENERATED_BODY()

public:

	/**
	 * Gets information about the camera position for the primary level editor viewport.  In non-editor builds, these will be zeroed
	 * In the UnrealEd module instead of Level Editor as it uses FLevelEditorViewportClient which is in this module
	 *
	 * @param	CameraLocation	(out) Current location of the level editing viewport camera, or zero if none found
	 * @param	CameraRotation	(out) Current rotation of the level editing viewport camera, or zero if none found
	 * @return	Whether or not we were able to get a camera for a level editing viewport
	 */
	UFUNCTION(BlueprintPure, Category = "Development|Editor")
	UNREALED_API bool GetLevelViewportCameraInfo(FVector& CameraLocation, FRotator& CameraRotation);

	/**
	* Sets information about the camera position for the primary level editor viewport.
	* In the UnrealEd module instead of Level Editor as it uses FLevelEditorViewportClient which is in this module
	*
	* @param	CameraLocation	Location the camera will be moved to.
	* @param	CameraRotation	Rotation the camera will be set to.
	*/
	UFUNCTION(BlueprintCallable, Category = "Development|Editor")
	UNREALED_API void SetLevelViewportCameraInfo(FVector CameraLocation, FRotator CameraRotation);

	/**
	 * Find the World in the world editor. It can then be used as WorldContext by other libraries like GameplayStatics.
	 * @return	The World used by the world editor.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility")
	UNREALED_API UWorld* GetEditorWorld();

	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Level Utility")
	UNREALED_API UWorld* GetGameWorld();

	/**
	 * Transforms the given 2D screen space coordinate into a 3D world-space point and direction using the primary level editor viewport.	 
	 * @param ScreenPosition	2D screen space to deproject. (in pixels)
	 * @param WorldPosition		(out) Corresponding 3D position in world space.
	 * @param WorldDirection	(out) World space direction vector away from the camera at the given 2d point.
	 * @return					True if the position was able to be projected. False otherwise.
	 */
	UFUNCTION(BlueprintPure, Category = "Development|Editor")
	UNREALED_API bool ScreenToWorld(const FVector2D& ScreenPosition, FVector& WorldPosition, FVector& WorldDirection) const;

	/**
	 * Transforms the given 3D world-space point into a its 2D screen space coordinate using the primary level editor viewport.	 
	 * @param WorldPosition		World position to project.
	 * @param ScreenPosition	(out) Corresponding 2D position in screen space (in pixels)	 
	 * @return					True if the position was able to be projected. False otherwise.
	 */
	UFUNCTION(BlueprintPure, Category = "Development|Editor")	
	UNREALED_API bool WorldToScreen(const FVector& WorldPosition, FVector2D& ScreenPosition) const;

	/**
	 * Returns the size of the primary level editor viewport.
	 * @param Size	(out)	The dimensions of the viewport (in pixels).
	 * @return				True if there is a primary level editor viewport.
	 */
	UFUNCTION(BlueprintPure, Category = "Development|Editor")
	UNREALED_API bool GetLevelViewportSize(FIntPoint& Size) const;


private:
	FLevelEditorViewportClient* GetViewportClient() const;
};
