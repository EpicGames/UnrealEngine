// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "AssetRegistry/AssetData.h"
#include "Tests/AutomationTestSettings.h"
#include "PlayInEditorDataTypes.h"
#include "EditorWorldUtils.h"

class UFactory;

#if WITH_AUTOMATION_TESTS
#include "Editor.h"

//////////////////////////////////////////////////////////////////////////
// FAutomationEditorCommonUtils
class FAutomationEditorCommonUtils
{
public:
	
	/**
	* Creates a new map for editing.  Also clears editor tools that could cause issues when changing maps
	*
	* @return - The UWorld for the new map
	*/
	static UNREALED_API UWorld* CreateNewMap();

	/**
	* Creates an editor world based on a specified template type or path.
	 *
	 * @param TemplateNameOrPath		The type of template to create (Basic, OpenWorld, ...).
	 * @param InInitializationValues	Configuration settings for the world initialization.
	 * @param InWorldType				The type of world to create (default: EWorldType::Editor).
	 * @return							A unique pointer to the created FScopedEditorWorld, or nullptr if the template is not found.
	 */
	static UNREALED_API TUniquePtr<FScopedEditorWorld> CreateScopedEditorWorld(const FString& TemplateNameOrPath, const UWorld::InitializationValues& InInitializationValues, 	EWorldType::Type InWorldType = EWorldType::Editor);

	/**
	* Imports an object using a given factory
	*
	* @param ImportFactory - The factory to use to import the object
	* @param ObjectName - The name of the object to create
	* @param PackagePath - The full path of the package file to create
	* @param ImportPath - The path to the object to import
	*/
	static UNREALED_API UObject* ImportAssetUsingFactory(UFactory* ImportFactory, const FString& ObjectName, const FString& PackageName, const FString& ImportPath);

	/**
	* Nulls out references to a given object
	*
	* @param InObject - Object to null references to
	*/
	static UNREALED_API void NullReferencesToObject(UObject* InObject);

	/**
	* gets a factory class based off an asset file extension
	*
	* @param AssetExtension - The file extension to use to find a supporting UFactory
	*/
	static UNREALED_API UClass* GetFactoryClassForType(const FString& AssetExtension);

	/**
	* Applies settings to an object by finding UProperties by name and calling ImportText
	*
	* @param InObject - The object to search for matching properties
	* @param PropertyChain - The list FProperty names recursively to search through
	* @param Value - The value to import on the found property
	*/
	static UNREALED_API void ApplyCustomFactorySetting(UObject* InObject, TArray<FString>& PropertyChain, const FString& Value);

	/**
	* Applies the custom factory settings
	*
	* @param InFactory - The factory to apply custom settings to
	* @param FactorySettings - An array of custom settings to apply to the factory
	*/
	static UNREALED_API void ApplyCustomFactorySettings(UFactory* InFactory, const TArray<FImportFactorySettingValues>& FactorySettings);

	/**
	* Writes a number to a text file.
	*
	* @param InTestName - is the folder that has the same name as the test. (For Example: "Performance").
	* @param InItemBeingTested - is the name for the thing that is being tested. (For Example: "MapName").
	* @param InFileName - is the name of the file with an extension
	* @param InEntry - is the double-precision number that is expected to be written to the file.
	* @param Delimiter - is the delimiter to be used. TEXT(",")
	*/
	static UNREALED_API void WriteToTextFile(const FString& InTestName, const FString& InTestItem, const FString& InFileName, const double& InEntry, const FString& Delimiter);

	/**
	* Returns the sum of the numbers available in an array of float.

	* @param InFloatArray - is the name of the array intended to be used.
	* @param bisAveragedInstead - will return the average of the available numbers instead of the sum.
	*/
	static UNREALED_API float TotalFromFloatArray(const TArray<float>& InFloatArray, bool bisAveragedInstead);

	/**
	* Returns the largest value from an array of float numbers.

	* @param InFloatArray - is the name of the array intended to be used.
	*/
	static UNREALED_API float LargestValueInFloatArray(const TArray<float>& InFloatArray);

	/**
	* Returns the contents of a text file as an array of FString.

	* @param InFileLocation - is the location of the file.
	* @param OutArray - The name of the array that will store the data.
	*/
	static UNREALED_API void CreateArrayFromFile(const FString& InFileLocation, TArray<FString>& OutArray);

	/**
	* Returns true if the archive/file can be written to otherwise false.

	* @param InFilePath - is the location of the file.
	* @param InArchiveName - is the name of the archive to be used.
	*/
	static UNREALED_API bool IsArchiveWriteable(const FString& InFilePath, const FArchive* InArchiveName);

	/**
	* Returns the first DeviceID 'Platform@Device'
	* Searches the automation preferences for the platform to use.

	* @param OutDeviceID - The variable that will hold the device ID.
	* @param InMapName - The map name to check against in the automation preferences.
	*/
	static UNREALED_API void GetLaunchOnDeviceID(FString& OutDeviceID, const FString& InMapName);

	/**
	* Returns the DeviceID 'Platform@Device'
	* Searches the automation preferences for the platform to use.

	* @param OutDeviceID - The variable that will hold the device ID.
	* @param InMapName - The map name to check against in the automation preferences.
	* @param InDeviceName - Device Name
	*/
	static UNREALED_API void GetLaunchOnDeviceID(FString& OutDeviceID, const FString& InMapName, const FString& InDeviceName);

	/**
	* Sets the first found ortho viewport camera to the desired location and rotation.

	* @param ViewLocation - Desired location for the viewport view.
	* @param ViewRotation - Desired rotation of the viewport view.
	*/
	static UNREALED_API bool SetOrthoViewportView(const FVector& ViewLocation, const FRotator& ViewRotation);

	/**
	* Set Play Session parameters to Active Viewport location
	* 
	* @param OutParams - FRequestPlaySessionParams object to set
	*/
	static UNREALED_API bool SetPlaySessionStartToActiveViewport(FRequestPlaySessionParams& OutParams);

	/**
	* Converts a package path to an asset path
	*
	* @param PackagePath - The package path to convert
	*/
	static UNREALED_API FString ConvertPackagePathToAssetPath(const FString& PackagePath);

	/**
	* Gets the asset data from a package path
	*
	* @param PackagePath - The package path used to look up the asset data
	*/
	static UNREALED_API FAssetData GetAssetDataFromPackagePath(const FString& PackagePath);

	/**
	* Loads the map specified by an automation test
	*
	* @param MapName - Map to load
	*/
	static UNREALED_API void LoadMap(const FString& MapName);

	/**
	* Run PIE
	*/
	static UNREALED_API void RunPIE(float PIEDuration = 3.f);
	
	/**
	* Generates a list of assets from the ENGINE and the GAME by a specific type.
	* This is to be used by the GetTest() function.
	*/
	static UNREALED_API void CollectTestsByClass(UClass * Class, TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands);
	
	/**
	* Generates a list of assets from the GAME by a specific type.
	* This is to be used by the GetTest() function.
	*/
	static UNREALED_API void CollectGameContentTestsByClass(UClass * Class, bool bRecursiveClass, TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands);
		
	/**
	* Generates a list of assets from the GAME by a specific type.
	* This is to be used by the GetTest() function.
	*/
	static UNREALED_API void CollectGameContentTests(TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands);
};


//////////////////////////////////////////////////////////////////////////
//Common latent commands used for automated editor testing.

/**
* Creates a latent command which the user can either undo or redo an action.
* True will trigger Undo, False will trigger a redo
*/
DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(UNREALED_API, FUndoRedoCommand, bool, bUndo);

/**
* Open editor for a particular asset
*/
DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(UNREALED_API, FOpenEditorForAssetCommand, FString, AssetName);

/**
* Close all asset editors
*/
DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND(UNREALED_API, FCloseAllAssetEditorsCommand);

/**
* Start PIE session
*/
DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(UNREALED_API, FStartPIECommand, bool, bSimulateInEditor);

/**
* End PlayMap session
*/
DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND(UNREALED_API, FEndPlayMapCommand);

/**
* Loads a map
*/
DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(UNREALED_API, FEditorLoadMap, FString, MapName);

/**
* Waits for shaders to finish compiling before moving on to the next thing.
*/
DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND(UNREALED_API, FWaitForShadersToFinishCompiling);

/**
* Latent command that changes the editor viewport to the first available bookmarked view.
*/
DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND(UNREALED_API, FChangeViewportToFirstAvailableBookmarkCommand);

/**
* Latent command that adds a static mesh to the worlds origin.
*/
DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND(UNREALED_API, FAddStaticMeshCommand);

/**
* Latent command that builds the lighting for the current level.
*/
DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND(UNREALED_API, FBuildLightingCommand);

/**
* Latent command that saves a copy of the level to a transient folder.
*/
DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(UNREALED_API, FSaveLevelCommand, FString, MapName);

/**
* Triggers a launch on using a specified device ID.
*/
DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(UNREALED_API, FLaunchOnCommand, FString, InLauncherDeviceID);

/**
* Wait for a cook by the book to finish.  Will time out after 3600 seconds (1 hr).
*/
DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND(UNREALED_API, FWaitToFinishCookByTheBookCommand);

/**
* Wait for a build and deploy to finish before moving on.
*/
DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND(UNREALED_API, FWaitToFinishBuildDeployCommand);

/**
* Latent command to delete a directory.
*/
DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(UNREALED_API, FDeleteDirCommand, FString, InFolderLocation);

/**
 * Waits for the editor world's navmesh to finish building before proceeding.
 * Ensures PIE duplicates a fully built navmesh rather than an empty one.
 * Timeout duration is configurable via UAutomationTestSettings::NavMeshBuildTimeoutSeconds.
 */
DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND(UNREALED_API, FWaitForEditorNavMeshBuildCommand);

/**
 * Starts a PIE session for an automation test and waits for it to be ready.
 * Defers PIE start so it can be preceded by latent wait commands (e.g. FWaitForEditorNavMeshBuildCommand).
 * Handles failure at all stages: if PIE fails to start, starts but ends before the world is ready,
 * or the world stalls during initialization, the test is marked as failed via AddError() and the
 * command terminates. All failure paths are guarded by PIEStartTimeoutSeconds.
 */
class UNREALED_API FStartPIEForAutomationCommand : public IAutomationLatentCommand
{
public:
	FStartPIEForAutomationCommand(FRequestPlaySessionParams InRequestParams);
	virtual ~FStartPIEForAutomationCommand();

	// Update returns false until its work is complete, then it returns true to indicate its work is complete.
	virtual bool Update() override;

private:
	void OnPostPIEStarted(const bool bInSimulateInEditor);
	void OnEndPIE(const bool bInSimulateInEditor);

	FRequestPlaySessionParams RequestParams;
	bool bPIEStarted;
	bool bPIEFailed;
	bool bSessionRequested;
};

#endif
