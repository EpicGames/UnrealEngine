// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
// Automation

#include "CoreMinimal.h"
#include "Framework/Commands/InputChord.h"
#include "AutomationScreenshotOptions.h"
#include "Misc/AutomationTest.h"

#if WITH_AUTOMATION_TESTS

class SWidget;
class UMaterial;
class UTexture;

//Materials

class FEditorPromotionTestUtilities
{

private:

	/**
	* Finds a visible widget by type.     SLOW!!!!!
	*
	* @param InParent - We search this widget and its children for a matching widget (recursive)
	* @param InWidgetType - The widget type we are searching for
	*/
	static TSharedPtr<SWidget> FindFirstWidgetByClass(TSharedRef<SWidget> InParent, const FName& InWidgetType);


public:

	/**
	* Gets the base path for this asset
	*/
	UNREALED_API static FString GetGamePath();


	/**
	* Creates a material from an existing texture
	*
	* @param InTexture - The texture to use as the diffuse for the new material
	*/
	UNREALED_API static UMaterial* CreateMaterialFromTexture(UTexture* InTexture);


	/**
	* Sets an editor keyboard shortcut
	*
	* @param CommandContext - The context of the command
	* @param Command - The command name to set
	* @param NewChord - The new input chord to assign
	*/
	UNREALED_API static bool SetEditorKeybinding(const FString& CommandContext, const FString& Command, const FInputChord& NewChord, const FInputChord& NewAlternateChord = FInputChord());

	
	/**
	* Gets an editor keyboard shortcut
	*
	* @param CommandContext - The context of the command
	* @param Command - The command name to get
	*/
	UNREALED_API static FInputChord GetEditorKeybinding(const FString& CommandContext, const FString& Command); //, FInputChord& CurrentChord);

	/**
	* Gets the current input chord or sets a new one if it doesn't exist
	*
	* @param Context - The context of the UI Command
	* @param Command - The name of the UI command
	*/
	UNREALED_API static FInputChord GetOrSetUICommand(const FString& Context, const FString& Command);


	/**
	* Sends a UI command to the active top level window after focusing on a widget of a given type
	*
	* @param InChord - The chord to send to the window
	* @param WidgetTypeToFocus - The widget type to find and focus on
	*/
	UNREALED_API static void SendCommandToCurrentEditor(const FInputChord& InChord, const FName& WidgetTypeToFocus);

	/**
	* Gets an object property value by name
	*
	* @param TargetObject - The object to modify
	* @param InVariableName - The name of the property
	*/
	UNREALED_API static FString GetPropertyByName(UObject* TargetObject, const FString& InVariableName);

	/**
	* Sets an object property value by name
	*
	* @param TargetObject - The object to modify
	* @param InVariableName - The name of the property
	*/
	UNREALED_API static void SetPropertyByName(UObject* TargetObject, const FString& InVariableName, const FString& NewValueString);

	/**
	* Starts a PIE session
	*/
	UNREALED_API static void StartPIE(bool bSimulateInEditor);

	/**
	* Ends a PIE session
	*/
	UNREALED_API static void EndPIE();

	/**
	* Takes an automation screenshot
	*
	* @param ScreenshotName - The sub name to use for the screenshot
	*/
	UNREALED_API static void TakeScreenshot(const FString& ScreenshotName, const FAutomationScreenshotOptions& Options, bool bUseTopWindow = false);
};

#endif // WITH_AUTOMATION_TESTS