// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "FbxAutomationCommon.generated.h"

class UFbxImportUI;

/** Import mesh type */
UENUM()
enum EFBXExpectedResultPreset : int
{
	/** Data should contain the number of error [int0]. */
	Error_Number,
	/** Data should contain the number of warning [int0]. */
	Warning_Number,
	/** Data should contain the number of static mesh created [int0]. */
	Created_Staticmesh_Number,
	/** Data should contain the number of skeletal mesh created [int0]. */
	Created_Skeletalmesh_Number,
	/** Data should contain the number of Material created [int0] under the target content folder. */
	Materials_Created_Number,
	/** Data should be the slot index [int0], and the expected original imported material slot name [string0]. */
	Material_Slot_Imported_Name,

	/** Data should be the total number of vertex for all LOD [int0]. */
	Vertex_Number,
	
	/** Data should be the expected number of LOD [int0]. */
	Lod_Number,
	/** Data should be the LOD index [int0] and total number of vertex for lod [int1]. */
	Vertex_Number_Lod,

	/** Data should contain the number of Material indexed by the mesh [int0]. */
	Mesh_Materials_Number,

	/** Data should be the LOD index [int0] and the expected number of section for a mesh [int1]. */
	Mesh_LOD_Section_Number,
	/** Data should be the LOD index [int0], section index [int1] and the expected number of vertex [int2]. */
	Mesh_LOD_Section_Vertex_Number,
	/** Data should be the LOD index [int0], section index [int1] and the expected number of triangle [int2]. */
	Mesh_LOD_Section_Triangle_Number,
	/** Data should be the LOD index [int0], section index [int1] and the expected name of material [string0]. */
	Mesh_LOD_Section_Material_Name,
	/** Data should be the LOD index [int0], section index [int1] and the expected material index of mesh materials [int2]. */
	Mesh_LOD_Section_Material_Index,
	/** Data should be the LOD index [int0], section index [int1] and the expected original imported material slot name [string0]. */
	Mesh_LOD_Section_Material_Imported_Name,
	
	/** Data should be the LOD index [int0], vertex index [int1] and the expected position of the vertex X[float0] Y[float1] Z[float2]. */
	Mesh_LOD_Vertex_Position,
	/** Data should be the LOD index [int0], vertex index [int1] and the expected normal of the vertex X[float0] Y[float1] Z[float2]. */
	Mesh_LOD_Vertex_Normal,

	/** Data should be the LOD index [int0] and the number of UV channel [int1] for the specified LOD. */
	LOD_UV_Channel_Number,

	/** Data should contain the number of bone created [int0]. */
	Bone_Number,
	/** Data should contain the bone index [int0] and a position xyz [float0 float1 float2] optionnaly you can pass a tolerance [float3]. */
	Bone_Position,
	
	/** Data should contain the Number of Frame [int0]. */
	Animation_Frame_Number,
	/** Data should contain the animation length [float0]. */
	Animation_Length,
	/** Data should contain the custom animation name [string0], key index [int0], and expected key value [float0]*/
	Animation_CustomCurve_KeyValue,
	/** Data should contain the custom animation name [string0], key index [int0], and expected arriving tangent value [float0]*/
	Animation_CustomCurve_KeyArriveTangent,
	/** Data should contain the custom animation name [string0], key index [int0], and expected leaving tangent value [float0]*/
	Animation_CustomCurve_KeyLeaveTangent,

	/** Data should contain the bone name [string0], the switch to inspect a alternate profile (0 = no, 1 = yes) [int0] and the expected number of vertex skin by the specified bone [int1]*/
	Skin_By_Bone_Vertex_Number,

	/** Data should contain the custom animation name [string0], key index [int0], and expected arriving tangent weight value [float0]*/
	Animation_CustomCurve_KeyArriveTangentWeight,
	/** Data should contain the custom animation name [string0], key index [int0], and expected leaving tangent weight value [float0]*/
	Animation_CustomCurve_KeyLeaveTangentWeight,
};

/** Import mesh type */
UENUM()
enum EFBXTestPlanActionType : int
{
	/*Normal import*/
	Import,
	/*Re-import the previous import, this is mandatory to make an import before*/
	Reimport,
	/*Add a new LOD*/
	AddLOD,
	/*Reimport an existing LOD*/
	ReimportLOD,
	/*The fbx will be imported, package will be save, object will be delete from memory then reload from the saved package. This mode force a delete of the asset after the test, no reimport is possible after*/
	ImportReload,
	/*Add alternate skinning to LOD 0, you must provide a "_alt" suffix file from the base import file*/
	AddAlternateSkinnig,
};

/**
* Container for detailing collision automated test data.
*/
USTRUCT()
struct FFbxTestPlanExpectedResult
{
	GENERATED_USTRUCT_BODY()

	/*Expected preset type*/
	UPROPERTY(EditAnywhere, Category = ExpectedResult)
	TEnumAsByte<enum EFBXExpectedResultPreset> ExpectedPresetsType;

	/*Expected preset data: integer */
	UPROPERTY(EditAnywhere, Category = ExpectedResult)
	TArray< int32 > ExpectedPresetsDataInteger;

	/*Expected preset data: double */
	UPROPERTY(EditAnywhere, Category = ExpectedResult)
	TArray< float > ExpectedPresetsDataFloat;

	/*Expected preset data: double */
	UPROPERTY(EditAnywhere, Category = ExpectedResult)
	TArray< double > ExpectedPresetsDataDouble;

	/*Expected preset data: string */
	UPROPERTY(EditAnywhere, Category = ExpectedResult)
	TArray< FString > ExpectedPresetsDataString;

	FFbxTestPlanExpectedResult()
		: ExpectedPresetsType(EFBXExpectedResultPreset::Error_Number)
	{
	}
};


/**
 * Container for detailing collision automated test data.
 */
UCLASS(HideCategories = Object, MinimalAPI)
class UFbxTestPlan : public UObject
{
	GENERATED_UCLASS_BODY()

	/* Name of the Test Plan*/
	UPROPERTY(EditAnywhere, Category = General)
	FString TestPlanName;

	/*Tell the system what we want to do*/
	UPROPERTY(EditAnywhere, Category = General)
	TEnumAsByte<enum EFBXTestPlanActionType> Action;

	/*The LOD index in case the user choose to add or reimport a LOD*/
	UPROPERTY(EditAnywhere, Category = General)
	int32 LodIndex;

	/*If true the test will delete all assets create in the import folder*/
	UPROPERTY(EditAnywhere, Category = General)
	bool bDeleteFolderAssets;

	/*Expected preset type*/
	UPROPERTY(EditAnywhere, Category = ExpectedResult)
	TArray< FFbxTestPlanExpectedResult > ExpectedResult;

	/* Options use for this test plan, Transient because we manually serialize the options. */
	UPROPERTY(EditAnywhere, Transient, Instanced, Category = Options)
	TObjectPtr<UFbxImportUI> ImportUI;
};

namespace FbxAutomationTestsAPI
{
	UNREALED_API void ReadFbxOptions(const FString &FileOptionAndResult, TArray<UFbxTestPlan*> &TestPlanArray);
	UNREALED_API void WriteFbxOptions(const FString &Filename, TArray<UFbxTestPlan*> &TestPlanArray);
}
