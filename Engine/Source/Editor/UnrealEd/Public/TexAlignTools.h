// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TexAlignTools.h: Tools for aligning textures on surfaces
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "TexAligner/TexAligner.h"

struct FBspSurf;

class FBspSurfIdx
{
public:
	FBspSurfIdx()
	{}
	FBspSurfIdx( FBspSurf* InSurf, int32 InIdx )
	{
		Surf = InSurf;
		Idx = InIdx;
	}

	FBspSurf* Surf;
	int32 Idx;
};

/**
 * A helper class to store the state of the various texture alignment tools.
 */
class FTexAlignTools
{
public:

	/** Constructor */
	FTexAlignTools();

	/** Destructor */
	~FTexAlignTools();

	/** A list of all available aligners. */
	TArray<UTexAligner*> Aligners;

	/**
	 * Creates the list of aligners.
	 */
	void Init();

	void Release();
	/**
	 * Returns the most appropriate texture aligner based on the type passed in.
	 */
	UNREALED_API UTexAligner* GetAligner( ETexAlign InTexAlign );

private:
	/** 
	 * Delegate handlers
	 **/
	void OnEditorFitTextureToSurface(UWorld* InWorld);

	bool bIsInit;

};

//This structure is using a static multicast delegate, so creating a static instance is dangerous because
//there is nothing to control the destruction order. If the multicast is destroy first we will have dangling pointer.
//The solution to this is to call release in the shutdown of the editor (see FUnrealEdMisc::OnExit) which happen before any static destructor.
extern UNREALED_API FTexAlignTools GTexAlignTools;
