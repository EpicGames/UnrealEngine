// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Framework/Text/SyntaxHighlighterTextLayoutMarshaller.h"
#include "Framework/Text/SyntaxTokenizer.h"
#include "Framework/Text/TextLayout.h"
#include "HAL/Platform.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"



/**
 * Syntax highlighting for hlsl text
 */
class FHLSLSyntaxHighlighterMarshaller : public FSyntaxHighlighterTextLayoutMarshaller
{
public:

	struct FSyntaxTextStyle
	{
		FSyntaxTextStyle(
			const FTextBlockStyle& InNormalTextStyle,
			const FTextBlockStyle& InOperatorTextStyle,
			const FTextBlockStyle& InKeywordTextStyle,
			const FTextBlockStyle& InStringTextStyle,
			const FTextBlockStyle& InNumberTextStyle,
			const FTextBlockStyle& InCommentTextStyle,
			const FTextBlockStyle& InPreProcessorKeywordTextStyle,
			const FTextBlockStyle& InErrorTextStyle
			) :
			NormalTextStyle(InNormalTextStyle),
			OperatorTextStyle(InOperatorTextStyle),
			KeywordTextStyle(InKeywordTextStyle),
			StringTextStyle(InStringTextStyle),
			NumberTextStyle(InNumberTextStyle),
			CommentTextStyle(InCommentTextStyle),
			PreProcessorKeywordTextStyle(InPreProcessorKeywordTextStyle),
			ErrorTextStyle(InErrorTextStyle)
		{
		}

		FTextBlockStyle NormalTextStyle;
		FTextBlockStyle OperatorTextStyle;
		FTextBlockStyle KeywordTextStyle;
		FTextBlockStyle StringTextStyle;
		FTextBlockStyle NumberTextStyle;
		FTextBlockStyle CommentTextStyle;
		FTextBlockStyle PreProcessorKeywordTextStyle;
		FTextBlockStyle ErrorTextStyle;
	};

	static UNREALED_API TSharedRef<FHLSLSyntaxHighlighterMarshaller> Create(const FSyntaxTextStyle& InSyntaxTextStyle);

protected:

	enum class EParseState : uint8
	{
		None,
		LookingForString,
		LookingForCharacter,
		LookingForSingleLineComment,
		LookingForMultiLineComment,
	};

	static UNREALED_API TSharedPtr<ISyntaxTokenizer> CreateTokenizer();
	
	UNREALED_API virtual void ParseTokens(const FString& SourceString, FTextLayout& TargetTextLayout, TArray<ISyntaxTokenizer::FTokenizedLine> TokenizedLines) override;

	UNREALED_API virtual FTextLayout::FNewLineData ProcessTokenizedLine(const ISyntaxTokenizer::FTokenizedLine& TokenizedLine, const int32& LineNumber, const FString& SourceString, EParseState& CurrentParseState);

	UNREALED_API FHLSLSyntaxHighlighterMarshaller(TSharedPtr<ISyntaxTokenizer> InTokenizer, const FSyntaxTextStyle& InSyntaxTextStyle);

	/** Styles used to display the text */
	FSyntaxTextStyle SyntaxTextStyle;
};
