// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Text/SyntaxTokenizer.h"

/**
 * Tokenize the text based on Hlsl tokens
 */
class FHlslSyntaxTokenizer : public ISyntaxTokenizer
{
public:
	/** 
	 * Create a new tokenizer
	 */
	static TSharedRef<FHlslSyntaxTokenizer> Create();

	virtual ~FHlslSyntaxTokenizer()= default;

	UNREALED_API virtual void Process(TArray<FTokenizedLine>& OutTokenizedLines, const FString& Input) override;

protected:
	UNREALED_API FHlslSyntaxTokenizer();

	void TokenizeLineRanges(const FString& Input, const TArray<FTextRange>& LineRanges, TArray<FTokenizedLine>& OutTokenizedLines);

	TArray<FString> Keywords;
	TArray<FString> Operators;
};
