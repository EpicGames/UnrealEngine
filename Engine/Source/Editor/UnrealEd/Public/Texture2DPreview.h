// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	Texture2DPreview.h: Definitions for previewing 2d textures.
==============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "BatchedElements.h"

/**
 * Batched element parameters for previewing 2d textures.
 */
class FBatchedElementTexture2DPreviewParameters : public FBatchedElementParameters
{
public:

	// MipLevel = -1 to auto-mip
	//	Layer is for VT, must be an integer
	// SliceIndex = -1 for all
	FBatchedElementTexture2DPreviewParameters(float InMipLevel, float InLayerIndex, float InSliceIndex, bool bInIsNormalMap, bool bInIsSingleChannel, bool bInIsSingleVTPhysicalSpace, bool bInIsVirtualTexture, bool bInIsTextureArray, bool bInUsePointSampling)
		: MipLevel(InMipLevel)
		, LayerIndex(InLayerIndex)
		, SliceIndex(InSliceIndex)
		, bIsNormalMap( bInIsNormalMap )
		, bIsSingleChannelFormat( bInIsSingleChannel )
		, bIsSingleVTPhysicalSpace(bInIsSingleVTPhysicalSpace)
		, bIsVirtualTexture(bInIsVirtualTexture)
		, bIsTextureArray(bInIsTextureArray)
		, bUsePointSampling(bInUsePointSampling)
	{
	}

	/** Binds vertex and pixel shaders for this element */
	UNREALED_API virtual void BindShaders(FRHICommandList& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit, ERHIFeatureLevel::Type InFeatureLevel, const FMatrix& InTransform, const float InGamma, const FMatrix& ColorWeights, const FTexture* Texture) override;

private:

	/** Parameters that need to be passed to the shader */
	float MipLevel;
	float LayerIndex;
	float SliceIndex;
	bool bIsNormalMap;
	bool bIsSingleChannelFormat;
	bool bIsSingleVTPhysicalSpace;

	/** Parameters that are used to select a shader permutation */
	bool bIsVirtualTexture;
	bool bIsTextureArray;

	/** Whether to use nearest-point sampling when rendering the texture */
	bool bUsePointSampling;
};
