// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UTexture;

namespace UE
{
namespace TextureAssetActions
{

UNREALED_API void TextureSource_Resize_WithDialog(const TArray<UTexture*> & InTextures);

UNREALED_API void TextureSource_ResizeToPowerOfTwo_WithDialog(const TArray<UTexture*> & InTextures);

UNREALED_API void TextureSource_ConvertTo8bit_WithDialog(const TArray<UTexture*> & InTextures);

UNREALED_API void TextureSource_JPEG_WithDialog(const TArray<UTexture*> & InTextures);

}
};
