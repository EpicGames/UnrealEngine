// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "TextureAssetActions.h"
#include "UObject/Object.h"
#include "TextureAssetActionsBP.generated.h"

/***

UTextureAssetActionsBlueprintFns provides BluePrint exposure of the TextureAssetActions functions.

TextureAssetActions provide utilities to modify the UTexture Source data which is stored in the uasset after import.

They do not modify the original source bitmap files on disk.  Those are not used by UE after import unless a reimport is done.

Modifying the UTexture Source data is in contrast to the UTexture Properties, which affect how the runtime GPU is build.

For example the UTexture property "MaxTextureSize" or "StretchToPowerOfTwo" may resize to limit GPU texture size, or stretch the GPU output to power of 2,
but they do not modify the Source bitmap in the uasset.  To modify the source uasset, use TextureAssetActions::ResizeTextureSource and ResizeTextureSourceToPowerOfTwo.

After these functions modify the Texture Source, the changed asset may need to be saved to retain those changes.

***/

UENUM(BlueprintType)
enum class ETextureAssetActionsBlueprintReturn : uint8
{
	Error = 0,
	Success = 1,
	AlreadyDone = 2
};

UCLASS(MinimalAPI, BlueprintType)
class UTextureAssetActionsBlueprintFns : public UObject
{
	GENERATED_BODY()

public:

	/**

	Resizes the texture Source to power of two, if it wasn't already.

	Only acts on textures larger than FilterThresholdValue.  Use FilterThresholdValue = 0 to disable filtering by size.

	Returns Success if the action was done, returns AlreadyDone if the action did not need to be done because texture already met the goal.
	Returns Error if the operation could not be applied because the texture is not a suitable type, or an unexpected error was encountered during processing.

	**/
	UFUNCTION(BlueprintCallable, Category = "TextureAssetActions")
    static UNREALED_API ETextureAssetActionsBlueprintReturn ResizeTextureSourceToPowerOfTwo(UTexture * Texture, int FilterThresholdValue );
	
	/**

	Resizes the texture Source to 8 bits per color channel, if it wasn't already.  HDR textures are converted to 16F (if they were 32F).

	If bNormalMapsKeep16bits is true, then normal maps whose source had >= 16 bits per color channel retain 16 bits per channel (rather than 8).
	
	Returns Success if the action was done, returns AlreadyDone if the action did not need to be done because texture already met the goal.
	Returns Error if the operation could not be applied because the texture is not a suitable type, or an unexpected error was encountered during processing.

	**/
	UFUNCTION(BlueprintCallable, Category = "TextureAssetActions")
    static UNREALED_API ETextureAssetActionsBlueprintReturn ConvertTo8bitTextureSource(UTexture * Texture,bool bNormalMapsKeep16bits );
	
	/**

	Resizes the texture Source to less or equal to ResizeToLessOrEqualToTargetSize

	Texture LODBias is automatically adjusted to try to keep output size the same when possible.

	Only acts on textures larger than ResizeToLessOrEqualToTargetSize.
	
	Returns Success if the action was done, returns AlreadyDone if the action did not need to be done because texture already met the goal.
	Returns Error if the operation could not be applied because the texture is not a suitable type, or an unexpected error was encountered during processing.

	**/
	UFUNCTION(BlueprintCallable, Category = "TextureAssetActions")
    static UNREALED_API ETextureAssetActionsBlueprintReturn ResizeTextureSource(UTexture * Texture,int ResizeToLessOrEqualToTargetSize );
	
	/**

	Compresses the texture source with JPEG.  This makes the uasset smaller on disk, but lower quality and slower to access.

	JPEG does not make the runtime game asset smaller on disk, only the source uasset.  JPEG can only be used on simple 2d textures.

	Only acts on texture sources that are 8 bit.  You may wish to use ConvertTo8bitTextureSource before this, optionally.

	Only acts on textures larger than FilterThresholdValue.  Use FilterThresholdValue = 0 to disable filtering by size.
	
	Returns Success if the action was done, returns AlreadyDone if the action did not need to be done because texture already met the goal.
	Returns Error if the operation could not be applied because the texture is not a suitable type, or an unexpected error was encountered during processing.

	**/
	UFUNCTION(BlueprintCallable, Category = "TextureAssetActions")
    static UNREALED_API ETextureAssetActionsBlueprintReturn CompressTextureSourceWithJPEG(UTexture * Texture, int FilterThresholdValue );
};

//================================================

// NOT exported , internal use only, not public APIs :
namespace UE
{
namespace TextureAssetActionsInternal
{

enum class ETextureAction
{
	Invalid = 0,
	Resize,
	ResizePow2,
	ConvertTo8bit,
	JPEG
};

// return true if Action should be applied to texture
//	ThresholdValue is a 1d dimension, textures smaller than Threshold are excluded
//	(set ThresholdValue == 0 for no size exclusion)
bool GetTextureFilterEnabled( UTexture * Texture, ETextureAction Action, int ThresholdValue, bool bNormalMapsKeep16bits, 
		bool * bOutExcludedBecauseAlreadySatisfied );

// Actions return true if something was done, false if not
//	can return false due to error or because no change was made

// Actions are always supported on simple texture types (no Layers, no UDIM, 2d and Cube)
//	more complex texture types are supported by some actions and not others YMMV

// you MUST check GetTextureFilterEnabled before calling these :
bool ResizeTextureSourceToPowerOfTwo(UTexture * Texture);
bool ConvertTo8bitTextureSource(UTexture * Texture,bool bNormalMapsKeep16bits);

// ResizeTextureSource resizes to <= TargetSize
//	if size was already <= TargetSize , no change is made and we return false
bool ResizeTextureSource(UTexture * Texture,int TargetSize);

// Source must be 8 bit for conversion to JPEG
//	you may optionally call ConvertTo8bitTextureSource before calling CompressTextureSourceWithJPEG
bool CompressTextureSourceWithJPEG(UTexture * Texture);

}
};

//================================================
