// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "ThumbnailRendering/ThumbnailRenderer.h"
#include "Misc/ObjectThumbnail.h"

class IPlugin;

struct FThumbnailExternalCacheSettings
{
	/** Recompress any lossless thumbnails */
	bool bRecompressLossless = false;

	/** Reduce size of any thumbnails to MaxImageSize */
	int32 MaxImageSize = INT32_MAX;
};

class FSaveThumbnailCacheTask
{
public:
	FObjectThumbnail ObjectThumbnail;
	FName Name;
	uint64 CompressedBytesHash = 0;

	void Compress(const FThumbnailExternalCacheSettings& InSettings);
};

struct FSaveThumbnailCacheDeduplicateKey
{
	FSaveThumbnailCacheDeduplicateKey() {}
	FSaveThumbnailCacheDeduplicateKey(uint64 InHash, int32 InNumBytes) : Hash(InHash), NumBytes(InNumBytes) {}

	uint64 Hash = 0;
	int32 NumBytes = 0;

	bool operator ==(const FSaveThumbnailCacheDeduplicateKey& Other) const
	{
		return Hash == Other.Hash && NumBytes == Other.NumBytes;
	}
};

inline uint32 GetTypeHash(const FSaveThumbnailCacheDeduplicateKey& InValue)
{
	return GetTypeHash(InValue.Hash);
}

struct FCombinedThumbnailCacheToSave
{
	TMap<FName, TSharedPtr<FSaveThumbnailCacheTask>> Tasks;
	FThumbnailExternalCacheSettings Settings;
	TMap<FSaveThumbnailCacheDeduplicateKey, TSharedPtr<FSaveThumbnailCacheTask>> DeduplicateMap;
	double AccumlatedLoadTime = 0.0;
};

class FThumbnailExternalCache
{
public:

	FThumbnailExternalCache();
	~FThumbnailExternalCache();
	
	/** Get thumbnail external cache */
	UNREALED_API static FThumbnailExternalCache& Get();

	/** Gets the name of the editor thumbnail cache file */
	UNREALED_API static const FString& GetCachedEditorThumbnailsFilename();

	/** Load thumbnails for the given object names from external cache */
	UNREALED_API bool LoadThumbnailsFromExternalCache(const TSet<FName>& InObjectFullNames, FThumbnailMap& InOutThumbnails);

	/** Save thumbnails for the given assets to an external file. Deterministic if assets were sorted. */
	UNREALED_API bool SaveExternalCache(const FString& InFilename, const TArrayView<FAssetData> InAssetDatas, const FThumbnailExternalCacheSettings& InSettings);

	/** Save thumbnails for the given assets to an external file. Deterministic if bSort=true. */
	UNREALED_API bool SaveExternalCache(const FString& InFilename, FCombinedThumbnailCacheToSave& InCache, const bool bSort);

	/** Sort asset data list if determinism needed */
	UNREALED_API static void SortAssetDatas(TArray<FAssetData>& InOutAssetDatas);

	enum class EThumbnailExternalCacheHeaderFlags : uint64
	{
		None = 0,
	};

	struct FThumbnailExternalCacheHeader
	{
		uint64 HeaderId = 0;
		uint64 Version = 0;
		uint64 Flags = 0;
		FString ImageFormatName;
		int64 ThumbnailTableOffset = 0;

		void Serialize(FArchive& Ar)
		{
			Ar << HeaderId;
			Ar << Version;
			Ar << Flags;
			Ar << ImageFormatName;
			Ar << ThumbnailTableOffset; // Offset must be serialized last
		}

		bool HasAnyFlags(EThumbnailExternalCacheHeaderFlags FlagsToCheck) const
		{
			return (Flags & (uint64)FlagsToCheck) != 0;
		}
	};

	struct FThumbnailEntry
	{
		int64 Offset = 0;
	};

	struct FThumbnailCacheFile
	{
		bool bUnableToOpenFile = false;
		FString Filename;
		FThumbnailExternalCacheHeader Header;
		TMap<FName, FThumbnailEntry> NameToEntry;
	};

	UNREALED_API void LoadCompressAndAppend(const TArrayView<FAssetData> InAssetDatas, FCombinedThumbnailCacheToSave& CombinedCache);

private:

	void SaveExternalCache(FArchive& Ar, const TArrayView<FAssetData> InAssetDatas, const FThumbnailExternalCacheSettings& InSettings);

	void Init();

	void Cleanup();

	void OnContentPathMounted(const FString& InAssetPath, const FString& InFilesystemPath);

	void OnContentPathDismounted(const FString& InAssetPath, const FString& InFilesystemPath);

	void LoadCacheFileIndexForPlugin(const TSharedPtr<IPlugin> InPlugin);

	bool LoadCacheFileIndex(const FString& Filename);

	bool LoadCacheFileIndex(FArchive& Ar, const TSharedPtr<FThumbnailCacheFile>& CacheFile);

private:

	TMap<FString, TSharedPtr<FThumbnailCacheFile>> CacheFiles;

	bool bHasInit = false;
	bool bIsSavingCache = false;
};
