// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "Containers/MRUArray.h"
#include "Editor.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "Engine/Blueprint.h"
#include "PreviewScene.h"
#include "TickableEditorObject.h"
#include "Animation/SkeletalMeshActor.h"
#include "ThumbnailHelpers.generated.h"

class FSceneViewFamily;
class USceneThumbnailInfo;

class FThumbnailPreviewScene : public FPreviewScene, public FTickableEditorObject
{
public:
	struct FConstructionValues
	{
		FConstructionValues()
			: bCreateSkySphere(true)
			, bDefaultLightingThumbnailScene(true)
			, LightRotation2(FRotator(-40.0, -144.678, 0.0))
			, LightBrightness2(5.0f)
			, LightRotation3(FRotator(299.235, 144.993, 0.0))
			, LightBrightness3(1.0f)
			, bSetSkyCubeMap(true)
			, bCreateFloorPlane(true)
		{
		}

		uint32 bCreateSkySphere : 1;
		uint32 bDefaultLightingThumbnailScene : 1;
		FRotator LightRotation2;
		float LightBrightness2;
		FRotator LightRotation3;
		float LightBrightness3;
		uint32 bSetSkyCubeMap : 1;
		uint32 bCreateFloorPlane : 1;

		FConstructionValues& SetCreateSkySphere(const bool bInValue) { bCreateSkySphere = bInValue; return *this; }
		FConstructionValues& SetDefaultLightingThumbnailScene(const bool bInValue) { bDefaultLightingThumbnailScene = bInValue; return *this; }
		FConstructionValues& SetLightRotation2(const FRotator& InValue) { LightRotation2 = InValue; return *this; }
		FConstructionValues& SetLightBrightness2(const float InValue) { LightBrightness2 = InValue; return *this; }
		FConstructionValues& SetLightRotation3(const FRotator& InValue) { LightRotation3 = InValue; return *this; }
		FConstructionValues& SetLightBrightness3(const float InValue) { LightBrightness3 = InValue; return *this; }
		FConstructionValues& SetSkyCubeMap(const bool bInValue) { bSetSkyCubeMap = bInValue; return *this; }
		FConstructionValues& SetCreateFloorPlane(const bool bInValue) { bCreateFloorPlane = bInValue; return *this; }
	};

	/** Constructor */
	UNREALED_API explicit FThumbnailPreviewScene(const FConstructionValues& InCVS = FConstructionValues());

	UE_DEPRECATED(5.0, "Use CreateView")
	UNREALED_API void GetView(FSceneViewFamily* ViewFamily, int32 X, int32 Y, uint32 SizeX, uint32 SizeY) const;

	/** Allocates then adds an FSceneView to the ViewFamily. */
	[[nodiscard]] UNREALED_API FSceneView* CreateView(FSceneViewFamily* ViewFamily, int32 X, int32 Y, uint32 SizeX, uint32 SizeY) const;

	/* Begin FTickableEditorObject */
	UNREALED_API virtual void Tick(float DeltaTime) override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
	UNREALED_API virtual TStatId GetStatId() const override;
	/* End FTickableEditorObject */

protected:
	/** Helper function to get the bounds offset to display an asset */
	UNREALED_API float GetBoundsZOffset(const FBoxSphereBounds& Bounds) const;

	/**
	  * Gets parameters to create a view matrix to be used by CreateView(). Implemented in children classes.
	  * @param InFOVDegrees  The FOV used to display the thumbnail. Often used to calculate the output parameters.
	  * @param OutOrigin	 The origin of the orbit view. Typically the center of the bounds of the target object.
	  * @param OutOrbitPitch The pitch of the orbit cam around the object.
	  * @param OutOrbitYaw	 The yaw of the orbit cam around the object.
	  * @param OutOrbitZoom  The camera distance from the object.
	  */
	virtual void GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const = 0;

	/** Returns FOV value to use for the view matrix */
	UNREALED_API virtual float GetFOV() const;
	virtual bool ShouldClampOrbitZoom() const { return true; }
};

class FParticleSystemThumbnailScene : public FThumbnailPreviewScene
{
public:
	/** Constructor/Destructor */
	UNREALED_API FParticleSystemThumbnailScene();
	UNREALED_API virtual ~FParticleSystemThumbnailScene();

	/** Sets the particle system to use in the next CreateView() */
	UNREALED_API void SetParticleSystem(class UParticleSystem* ParticleSystem);

protected:
	// FThumbnailPreviewScene implementation
	UNREALED_API virtual void GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const override;

protected:
	/** The particle system component used to display all particle system thumbnails */
	class UParticleSystemComponent* PartComponent;

	/** The FXSystem used to render all thumbnail particle systems */
	TSharedPtr<class FFXSystemInterface> ThumbnailFXSystem;
};

class FMaterialThumbnailScene : public FThumbnailPreviewScene
{
public:	
	/** Constructor */
	UNREALED_API FMaterialThumbnailScene();

	/** Sets the material to use in the next CreateView() */
	UNREALED_API void SetMaterialInterface(class UMaterialInterface* InMaterial);

	UNREALED_API bool ShouldSetSeparateTranslucency(class UMaterialInterface* InMaterial) const;

protected:
	// FThumbnailPreviewScene implementation
	UNREALED_API virtual void GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const override;

protected:
	/** The static mesh actor used to display all material thumbnails */
	class AStaticMeshActor* PreviewActor;
	/** Material being used for something that only makes sense to visualize as a plane (UI, particle sprites)*/
	bool bForcePlaneThumbnail;
};

class FSkeletalMeshThumbnailScene : public FThumbnailPreviewScene
{
public:
	/** Constructor */
	UNREALED_API FSkeletalMeshThumbnailScene();

	/** Sets the skeletal mesh to use in the next CreateView() */
	UNREALED_API void SetSkeletalMesh(class USkeletalMesh* InSkeletalMesh);

	/** Set whether to draw debug skeleton */
	UNREALED_API void SetDrawDebugSkeleton(bool bInDrawDebugSkeleton, const FLinearColor& InSkeletonColor);
	
	/** Returns the preview actor within the scene */
	class ASkeletalMeshActor* GetPreviewActor() { return PreviewActor; }

protected:
	// FThumbnailPreviewScene implementation
	UNREALED_API virtual void GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const override;

private:
	/** The skeletal mesh actor used to display all skeletal mesh thumbnails */
	class ASkeletalMeshActor* PreviewActor;

	/** The color used to draw the debug skeleton */
	FLinearColor DrawDebugColor = FLinearColor::White;
	
	/** Whether to draw debug skeleton */
	bool bDrawDebugSkeleton = false;
};

class FStaticMeshThumbnailScene : public FThumbnailPreviewScene
{
public:
	/** Constructor */
	UNREALED_API FStaticMeshThumbnailScene();

	/** Sets the static mesh to use in the next CreateView() */
	UNREALED_API void SetStaticMesh(class UStaticMesh* StaticMesh);

	/** Sets override materials for the static mesh  */
	UNREALED_API void SetOverrideMaterials(const TArray<class UMaterialInterface*>& OverrideMaterials);

protected:
	// FThumbnailPreviewScene implementation
	UNREALED_API virtual void GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const override;

private:
	/** The static mesh actor used to display all static mesh thumbnails */
	class AStaticMeshActor* PreviewActor;
};

UCLASS(ClassGroup = ISkeletalMeshes, ComponentWrapperClass, ConversionRoot, meta = (ChildCanTick))
class AAnimationThumbnailSkeletalMeshActor : public ASkeletalMeshActor
{
	GENERATED_UCLASS_BODY()
};

class FAnimationSequenceThumbnailScene : public FThumbnailPreviewScene
{
public:
	/** Constructor */
	UNREALED_API FAnimationSequenceThumbnailScene();

	/** Sets the animation to use in the next CreateView() */
	UNREALED_API bool SetAnimation(class UAnimSequenceBase* InAnimation);

protected:
	// FThumbnailPreviewScene implementation
	UNREALED_API virtual void GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const override;
	
	//Clean up the children of this component
	UNREALED_API void CleanupComponentChildren(USceneComponent* Component);

private:
	/** The skeletal mesh actor used to display all animation thumbnails */
	class AAnimationThumbnailSkeletalMeshActor* PreviewActor;

	/** Animation we are generating the thumbnail for */
	class UAnimSequenceBase* PreviewAnimation;
};

class FBlendSpaceThumbnailScene : public FThumbnailPreviewScene
{
public:
	/** Constructor */
	UNREALED_API FBlendSpaceThumbnailScene();

	/** Sets the animation to use in the next CreateView() */
	UNREALED_API bool SetBlendSpace(class UBlendSpace* InBlendSpace);

protected:
	// FThumbnailPreviewScene implementation
	UNREALED_API virtual void GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const override;

	//Clean up the children of this component
	UNREALED_API void CleanupComponentChildren(USceneComponent* Component);

private:
	/** The skeletal mesh actor used to display all animation thumbnails */
	class AAnimationThumbnailSkeletalMeshActor* PreviewActor;

	/** Animation we are generating the thumbnail for */
	class UBlendSpace* PreviewAnimation;
};

class FAnimBlueprintThumbnailScene : public FThumbnailPreviewScene
{
public:
	/** Constructor */
	UNREALED_API FAnimBlueprintThumbnailScene();

	/** Sets the animation blueprint to use in the next CreateView() */
	UNREALED_API bool SetAnimBlueprint(class UAnimBlueprint* InBlueprint);

protected:
	// FThumbnailPreviewScene implementation
	UNREALED_API virtual void GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const override;

	//Clean up the children of this component
	UNREALED_API void CleanupComponentChildren(USceneComponent* Component);

private:
	/** The skeletal mesh actor used to display all animation thumbnails */
	class ASkeletalMeshActor* PreviewActor;

	/** Animation Blueprint we are generating the thumbnail for */
	class UAnimBlueprint* PreviewBlueprint;
};

class FPhysicsAssetThumbnailScene : public FThumbnailPreviewScene
{
public:
	/** Constructor */
	UNREALED_API FPhysicsAssetThumbnailScene();

	/** Sets the skeletal mesh to use in the next CreateView() */
	UNREALED_API void SetPhysicsAsset(class UPhysicsAsset* InPhysicsAsset);

protected:
	// FThumbnailPreviewScene implementation
	UNREALED_API virtual void GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const override;

private:
	/** The skeletal mesh actor used to display all physics asset thumbnails */
	class ASkeletalMeshActor* PreviewActor;
};

class UActorComponent;

class FClassActorThumbnailScene : public FThumbnailPreviewScene
{
public:

	UNREALED_API FClassActorThumbnailScene();

	/** Returns true if this component can be visualized */
	static UNREALED_API bool IsValidComponentForVisualization(const UActorComponent* Component);

protected:
	// FThumbnailPreviewScene implementation
	UNREALED_API virtual void GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const override;

	/** Sets the object (class or blueprint) used in the next CreateView() */
	UNREALED_API void SpawnPreviewActor(class UClass* Obj);

	/** Get the scene thumbnail info to use for the object currently being rendered */
	virtual USceneThumbnailInfo* GetSceneThumbnailInfo(const float TargetDistance) const = 0;

	UNREALED_API FBoxSphereBounds GetPreviewActorBounds() const;

private:

	/** Clears out any stale actors in this scene if PreviewActor enters a stale state */
	UNREALED_API void ClearStaleActors();

	int32 NumStartingActors;
	TWeakObjectPtr<class AActor> PreviewActor;
};

class FBlueprintThumbnailScene : public FClassActorThumbnailScene
{
public:
	/** Constructor/Destructor */
	UNREALED_API FBlueprintThumbnailScene();

	/** Sets the static mesh to use in the next CreateView() */
	UNREALED_API void SetBlueprint(class UBlueprint* Blueprint);

	/** Refreshes components for the specified blueprint */
	UNREALED_API void BlueprintChanged(class UBlueprint* Blueprint);

	TWeakObjectPtr<class UBlueprint> GetCurrentBlueprint() const { return CurrentBlueprint; }
protected:

	/** Get the scene thumbnail info to use for the object currently being rendered */
	UNREALED_API virtual USceneThumbnailInfo* GetSceneThumbnailInfo(const float TargetDistance) const override;

private:
	/** The blueprint that is currently being rendered. NULL when not rendering. */
	TWeakObjectPtr<class UBlueprint> CurrentBlueprint;
};

class FClassThumbnailScene : public FClassActorThumbnailScene
{
public:
	/** Constructor/Destructor */
	UNREALED_API FClassThumbnailScene();

	/** Sets the class use in the next CreateView() */
	UNREALED_API void SetClass(class UClass* Class);

	UClass* GetCurrentClass() const { return CurrentClass; }

protected:
	/** Get the scene thumbnail info to use for the object currently being rendered */
	UNREALED_API virtual USceneThumbnailInfo* GetSceneThumbnailInfo(const float TargetDistance) const override;

private:
	/** The class that is currently being rendered. NULL when not rendering. */
	UClass* CurrentClass;
};

/** Handles instancing thumbnail scenes for Class and Blueprint types (use the class or generated class as the key). */
template <typename ThumbnailSceneType, int32 MaxNumScenes>
class TClassInstanceThumbnailScene
{
public:
	/** Constructor */
	TClassInstanceThumbnailScene()
	{
		InstancedThumbnailScenes.Reserve(MaxNumScenes);
		MRUClassNames.Reserve(MaxNumScenes);
	}

	/** Find an existing thumbnail scene instance for this class type. */
	TSharedPtr<ThumbnailSceneType> FindThumbnailScene(const UClass* InClass) const
	{
		check(InClass);
		const FName ClassName = InClass->GetFName();

		TSharedPtr<ThumbnailSceneType> Scene = InstancedThumbnailScenes.FindRef(ClassName);
		if (Scene.IsValid())
		{
			MRUClassNames.AddUnique(ClassName);
		}

		return Scene;
	}

	/** Find or create a thumbnail scene instance for this class type. */
	TSharedRef<ThumbnailSceneType> EnsureThumbnailScene(const UClass* InClass)
	{
		check(InClass);
		const FName ClassName = InClass->GetFName();

		TSharedPtr<ThumbnailSceneType> ExistingThumbnailScene = InstancedThumbnailScenes.FindRef(ClassName);
		if (!ExistingThumbnailScene.IsValid())
		{
			if (InstancedThumbnailScenes.Num() >= MaxNumScenes)
			{
				FName ClassNameToRemove = MRUClassNames.Pop();
				TSharedPtr<ThumbnailSceneType> SceneToDelete;
				InstancedThumbnailScenes.RemoveAndCopyValue(ClassNameToRemove, SceneToDelete);
				ScenesToDelete.Add(MoveTemp(SceneToDelete));

				// Now that we have scenes to delete, schedule them to be destroyed in batches periodically until we have none left.
				// Destroying many scenes at once can result in large stalls when iterating through content so we attempt to 
				// smooth that out here by deferring destruction, doing destruction in small, timed, batches and destroying the 
				// least recently used first to avoid deleting something we are likely to immediately recreate
				const int32 MinBatchSize = 8;
				if (!SceneDeleteTimerHandle.IsValid() && ScenesToDelete.Num() >= MinBatchSize)
				{
					checkf(IsInGameThread(), TEXT("Must be running on the game thread."));
					GEditor->GetTimerManager()->SetTimer(SceneDeleteTimerHandle, FTimerDelegate::CreateLambda([this]() mutable
						{
							checkf(IsInGameThread(), TEXT("Must be running on the game thread."));
							const double TimeoutSeconds = 0.25;

							int32 Iteration = 0;
							double StartTime = FPlatformTime::Seconds();
							while (!ScenesToDelete.IsEmpty())
							{
								if (++Iteration > MinBatchSize)
								{
									if ((FPlatformTime::Seconds() - StartTime) > TimeoutSeconds)
									{
										break;
									}
									Iteration = 1;
								}

								// The destruction of these scenes can be quite expensive if the world has many objects
								ScenesToDelete.Pop(EAllowShrinking::No);
							}

							CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

							if (ScenesToDelete.IsEmpty())
							{
								ScenesToDelete.Shrink();
								FTimerHandle LocalHandle = SceneDeleteTimerHandle;
								SceneDeleteTimerHandle.Invalidate();
								GEditor->GetTimerManager()->ClearTimer(LocalHandle); // TimerManager owns this lambda and clearing the timer will destroy it
							}
						}), 2.0f/*InRate*/, true /*bInLoop*/, 1.0f/*InFirstDelay*/);
				}
			}

			ExistingThumbnailScene = MakeShareable(new ThumbnailSceneType());
			InstancedThumbnailScenes.Add(ClassName, ExistingThumbnailScene);
			MRUClassNames.Add(ClassName);
		}

		return ExistingThumbnailScene.ToSharedRef();
	}

	/** Removes the thumbnail scene instance for the specified class */
	void RemoveThumbnailScene(const UClass* InClass)
	{
		check(InClass);
		const FName ClassName = InClass->GetFName();
		InstancedThumbnailScenes.Remove(ClassName);
		MRUClassNames.RemoveSingle(ClassName);
	}

	/** Clears all thumbnail scenes */
	void Clear()
	{
		InstancedThumbnailScenes.Reset();
		MRUClassNames.Reset();
	}

private:
	/**
	 * Mapping between the class type and its thumbnail scene.
	 * @note This uses the class name rather than the class pointer to avoid leaving behind stale class instances as Blueprints are re-compiled.
	 */
	TMap<FName, TSharedPtr<ThumbnailSceneType>> InstancedThumbnailScenes;

	/** Tracks the most recently used class names (most recent in the front) so we may prefer deleting the least recently used thumbnail scenes	*/
	mutable TMRUArray<FName, TFixedAllocator<MaxNumScenes>> MRUClassNames;
	/** Thumbnail scenes enqueued for deletion. Must only be modified by the game thread. */
	TArray<TSharedPtr<ThumbnailSceneType>> ScenesToDelete;
	/** Handle to time to periodically delete batches of enqueued scenes for deletion. */
	FTimerHandle SceneDeleteTimerHandle;
};


/** Handles instancing thumbnail scenes for Object based Asset types (use the path as the key). */
template <typename ThumbnailSceneType, int32 MaxNumScenes>
class TObjectInstanceThumbnailScene
{
public:
	/** Constructor */
	TObjectInstanceThumbnailScene()
	{
		InstancedThumbnailScenes.Reserve(MaxNumScenes);
	}

	/** Find an existing thumbnail scene instance for this class type. */
	TSharedPtr<ThumbnailSceneType> FindThumbnailScene(const UObject* InObject) const
	{
		check(InObject);
		const FString ObjectPath = InObject->GetPathName();

		return InstancedThumbnailScenes.FindRef(ObjectPath);
	}

	/** Find or create a thumbnail scene instance for this class type. */
	TSharedRef<ThumbnailSceneType> EnsureThumbnailScene(const UObject* InObject)
	{
		check(InObject);
		const FString ObjectPath = InObject->GetPathName();

		TSharedPtr<ThumbnailSceneType> ExistingThumbnailScene = InstancedThumbnailScenes.FindRef(ObjectPath);
		if (!ExistingThumbnailScene.IsValid())
		{
			if (InstancedThumbnailScenes.Num() >= MaxNumScenes)
			{
				InstancedThumbnailScenes.Reset();
				// Will hitch but is better than a crash
				CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
			}

			ExistingThumbnailScene = MakeShareable(new ThumbnailSceneType());
			InstancedThumbnailScenes.Add(ObjectPath, ExistingThumbnailScene);
		}

		return ExistingThumbnailScene.ToSharedRef();
	}

	/** Removes the thumbnail scene instance for the specified class */
	void RemoveThumbnailScene(const UObject* InObject)
	{
		check(InObject);
		InstancedThumbnailScenes.Remove(InObject->GetPathName());
	}

	/** Clears all thumbnail scenes */
	void Clear()
	{
		InstancedThumbnailScenes.Reset();
	}

private:
	/**
	 * Mapping between the class type and its thumbnail scene.
	 * @note This uses the class name rather than the class pointer to avoid leaving behind stale class instances as Blueprints are re-compiled.
	 */
	TMap<FString, TSharedPtr<ThumbnailSceneType>> InstancedThumbnailScenes;
};

