// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreGlobals.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Misc/AssertionMacros.h"
#include "Tickable.h"


/**
 * This class provides common registration for gamethread editor only tickable objects. It is an
 * abstract base class requiring you to implement the GetStatId, IsTickable, and Tick methods.
 * If you need a class that can tick in both the Editor and at Runtime then use FTickableGameObject
 * instead, overriding the IsTickableInEditor() function instead.
 */
class FTickableEditorObject : public FTickableObjectBase
{
public:
	UE_NONCOPYABLE(FTickableEditorObject);

	/** Calls Tick on every tickable editor object with tick enabled */
	static void TickObjects(const float DeltaSeconds)
	{
		FTickableStatics& Statics = GetStatics();

		SimpleTickObjects(Statics, [DeltaSeconds](FTickableObjectBase* TickableObject)
		{
			ObjectBeingTicked = TickableObject;
			TickableObject->Tick(DeltaSeconds);
			ObjectBeingTicked = nullptr;
		});
	}

	/** Registers this instance with the static array of tickable objects. */
	FTickableEditorObject()
	{
		ensure(IsInGameThread() || IsInAsyncLoadingThread());

		FTickableStatics& Statics = GetStatics();
		Statics.QueueTickableObjectForAdd(this);
	}

	/** Removes this instance from the static array of tickable objects. */
	virtual ~FTickableEditorObject()
	{
		ensureMsgf(ObjectBeingTicked != this, TEXT("Detected possible memory stomp. We are in the Tickable objects Tick function but hit its deconstructor, the 'this' pointer for the Object will now be invalid"));
		ensure(IsInGameThread() || IsInAsyncLoadingThread());
		
		FTickableStatics& Statics = GetStatics();
		Statics.SetTickTypeForTickableObject(this, ETickableTickType::Never);
	}

private:
	/** Set if we are in the Tick function for an editor tickable object */
	static UNREALED_API FTickableObjectBase* ObjectBeingTicked;

	/** Returns the tracking struct for this type */
	static UNREALED_API FTickableStatics& GetStatics();
};

/**
 * The same as FTickableEditorObject, but for systems that need to be ticked periodically during
 * cooking.
 * If a system needs to be cooked both during cook commandlet and in editor without the cook commandlet,
 * it should dual-inherit from both FTickableCookObject and FTickableEditorObject.
 */
class FTickableCookObject : public FTickableObjectBase
{
public:
	UE_NONCOPYABLE(FTickableCookObject);

	/** Calls TickCook on every enabled tickable object */
	static void TickObjects(const float DeltaSeconds, bool bCookComplete)
	{
		FTickableStatics& Statics = GetStatics();

		SimpleTickObjects(Statics, [DeltaSeconds, bCookComplete](FTickableObjectBase* TickableObject)
		{
			FTickableCookObject* CookTickableObject = static_cast<FTickableCookObject*>(TickableObject);
			ObjectBeingTicked = TickableObject;
			CookTickableObject->TickCook(DeltaSeconds, bCookComplete);
			ObjectBeingTicked = nullptr;
		});
	}

	/** Registers this instance with the static array of tickable objects. */
	FTickableCookObject()
	{
		ensure(IsInGameThread() || IsInAsyncLoadingThread());

		FTickableStatics& Statics = GetStatics();
		Statics.QueueTickableObjectForAdd(this);
	}

	/** Removes this instance from the static array of tickable objects. */
	virtual ~FTickableCookObject()
	{
		ensureMsgf(ObjectBeingTicked != this, TEXT("Detected possible memory stomp. We are in the Tickable objects Tick function but hit its deconstructor, the 'this' pointer for the Object will now be invalid"));
		ensure(IsInGameThread() || IsInAsyncLoadingThread());

		FTickableStatics& Statics = GetStatics();
		Statics.SetTickTypeForTickableObject(this, ETickableTickType::Never);
	}

	/** Cook tick virtual, must be implemented in subclass */
	virtual void TickCook(float DeltaTime, bool bCookCompete) = 0;

private:

	/** Set if we are in the Tick function for an editor tickable object */
	static UNREALED_API FTickableObjectBase* ObjectBeingTicked;

	/** Returns the tracking struct for this type */
	static UNREALED_API FTickableStatics& GetStatics();
};
