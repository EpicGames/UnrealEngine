// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "UObject/GCObject.h"
#include "Framework/Docking/TabManager.h"
#include "Toolkits/IToolkit.h"
#include "AssetDefinition.h"

#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/Docking/LayoutService.h"
#include "Toolkits/IToolkitHost.h"
#include "Toolkits/BaseToolkit.h"
#include "UnrealEdMisc.h"
#include "Subsystems/AssetEditorSubsystem.h"

class FAssetEditorModeManager;
class FEditorModeTools;
class FMenuBuilder;
struct FToolMenuContext;
struct FToolMenuSection;
class SBorder;
class SStandaloneAssetEditorToolkitHost;
class FWorkspaceItem;
class UToolMenu;
class UViewportInteractionsBehaviorSource;

namespace UE::Editor::Toolbars
{
	enum class ECreateStatusBarOptions : uint8;
}

DECLARE_DELEGATE_RetVal( bool, FRequestAssetEditorClose );
DECLARE_DELEGATE_RetVal( void, FAssetEditorClosing);


/**
 * The location of the asset editor toolkit tab
 * Note: These values are serialized into an ini file as an int32
 */
enum class EAssetEditorToolkitTabLocation : int32
{
	/** The tab is within the "DockedToolkit" tab stack */
	Docked,
	/** The tab is within the "StandaloneToolkit" tab stack */
	Standalone,
};

DECLARE_DELEGATE_RetVal_TwoParams(TSharedRef<FExtender>, FAssetEditorExtender, const TSharedRef<FUICommandList>, const TArray<UObject*>);

/**
* Extensibility managers simply keep a series of FExtenders for a single menu/toolbar/anything
* It is here to keep a standardized approach to editor extensibility among modules
*/
class FExtensibilityManager
{
public:
	virtual ~FExtensibilityManager() = default;

	/** Functions for outsiders to add or remove their extenders */
	virtual void AddExtender(TSharedPtr<FExtender> Extender) { Extenders.AddUnique(Extender); }
	virtual void RemoveExtender(TSharedPtr<FExtender> Extender) { Extenders.Remove(Extender); }

	/** Gets all extender delegates for this manager */
	virtual TArray<FAssetEditorExtender>& GetExtenderDelegates() { return ExtenderDelegates; }

	/** Gets all extenders, consolidated, for use by the editor to be extended */
	UNREALED_API virtual TSharedPtr<FExtender> GetAllExtenders();
	/** Gets all extenders and asset editor extenders from delegates consolidated */
	UNREALED_API virtual TSharedPtr<FExtender> GetAllExtenders(const TSharedRef<FUICommandList>& CommandList, const TArray<UObject*>& ContextSensitiveObjects);

private:
	/** A list of extenders the editor will use */
	TArray<TSharedPtr<FExtender>> Extenders;
	/** A list of extender delegates the editor will use */
	TArray<FAssetEditorExtender> ExtenderDelegates;
};

/** Indicates that a class has a default menu that is extensible */
class IHasMenuExtensibility
{
public:
	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() = 0;
};

/** Indicates that a class has a default toolbar that is extensible */
class IHasToolBarExtensibility
{
public:
	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() = 0;
};

/**
 * Allows an asset editor to specify which custom menus/toolbar items are visible in read-only mode, with
 * the default behavior being that all entries specific to an asset editor are hidden
 */
struct FReadOnlyAssetEditorCustomization
{
	// Permission list for the main menu
	FNamePermissionList MainMenuPermissionList;

	// Permission list for the toolbar
	FNamePermissionList ToolbarPermissionList;

	// Permission list for the various submenus on the main menu (e.g File, Help..)
	TMap<FName, FNamePermissionList> MainMenuSubmenuPermissionLists;
};

/**
 * Base class for toolkits that are used for asset editing (abstract)
 */
class FAssetEditorToolkit
	: public IAssetEditorInstance
	, public FBaseToolkit
	, public TSharedFromThis<FAssetEditorToolkit>
{

public:


	/** Default constructor */
	UNREALED_API FAssetEditorToolkit();

	/**
	 * Initializes this asset editor.  Called immediately after construction.
	 * Override PostInitAssetEditor if you need to do additional initialization
	 *
	 * @param	Mode					Asset editing mode for this editor (standalone or world-centric)
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	AppIdentifier			When Mode is Standalone, this is the app identifier of the app that should host this toolkit
	 * @param	StandaloneDefaultLayout	The default layout for a standalone asset editor
	 * @param	bCreateDefaultToolbar	The default toolbar, which can be extended
	 * @param	bCreateDefaultStandaloneMenu	True if in standalone mode, the asset editor should automatically generate a default "asset" menu, or false if you're going to do this yourself in your derived asset editor's implementation
	 * @param	ObjectToEdit			The object to edit
	 * @param	bInIsToolbarFocusable	Whether the buttons on the default toolbar can receive keyboard focus
	 * @param	bUseSmallToolbarIcons	Whether the buttons on the default toolbar use the small icons
	 * @param	InOpenMethod			Override whether the Asset Editor is being opened in read only or edit mode (otherwise automatically set by the asset editor subsytem for any asset editors opened through it)
	 */
	UNREALED_API void InitAssetEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, const FName AppIdentifier, const TSharedRef<FTabManager::FLayout>& StandaloneDefaultLayout, const bool bCreateDefaultStandaloneMenu, const bool bCreateDefaultToolbar, const TArray<UObject*>& ObjectsToEdit, const bool bInIsToolbarFocusable = false, const bool bInUseSmallToolbarIcons = false, const TOptional<EAssetOpenMethod>& InOpenMethod = TOptional<EAssetOpenMethod>());
	UNREALED_API void InitAssetEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, const FName AppIdentifier, const TSharedRef<FTabManager::FLayout>& StandaloneDefaultLayout, const bool bCreateDefaultStandaloneMenu, const bool bCreateDefaultToolbar, UObject* ObjectToEdit, const bool bInIsToolbarFocusable = false, const bool bInUseSmallToolbarIcons = false, const TOptional<EAssetOpenMethod>& InOpenMethod = TOptional<EAssetOpenMethod>());

	FAssetEditorToolkit(const FAssetEditorToolkit&) = delete;
	FAssetEditorToolkit& operator=(const FAssetEditorToolkit&) = delete;
	
	/** Virtual destructor, so that we can clean up our app when destroyed */
	UNREALED_API virtual ~FAssetEditorToolkit();

	/** IToolkit interface */
	UNREALED_API virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
	UNREALED_API virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
	UNREALED_API virtual bool IsAssetEditor() const override;
	UNREALED_API virtual const TArray< UObject* >* GetObjectsCurrentlyBeingEdited() const override;
	virtual FName GetToolkitFName() const override = 0;				// Must implement in derived class!
	virtual FText GetBaseToolkitName() const override = 0;			// Must implement in derived class!
	UNREALED_API virtual FText GetToolkitName() const override;
	UNREALED_API virtual FText GetTabSuffix() const override;
	UNREALED_API virtual FText GetToolkitToolTipText() const override;
	virtual FString GetWorldCentricTabPrefix() const override = 0;	// Must implement in derived class!
	UNREALED_API virtual FEditorModeTools& GetEditorModeManager() const final;

	/** IAssetEditorInstance interface */
	UNREALED_API virtual FName GetEditorName() const override;
	UNREALED_API virtual void FocusWindow(UObject* ObjectToFocusOn = nullptr) override;
	UE_DEPRECATED(5.3, "Use CloseWindow that takes in an EAssetEditorCloseReason instead")
	UNREALED_API virtual bool CloseWindow() override;
	UNREALED_API virtual bool CloseWindow(EAssetEditorCloseReason InCloseReason) override;
	virtual bool IsPrimaryEditor() const override { return true; };
	UNREALED_API virtual void InvokeTab(const FTabId& TabId) override;
	UNREALED_API virtual FName GetEditingAssetTypeName() const override;
	UNREALED_API virtual TSharedPtr<FTabManager> GetAssociatedTabManager() override;
	UNREALED_API virtual double GetLastActivationTime() override;
	UNREALED_API virtual void RemoveEditingAsset(UObject* Asset) override;
	virtual EAssetOpenMethod GetOpenMethod() const override { return OpenMethod; }

	/**
	 * Fills in the supplied menu with commands for working with this asset file
	 *
	 * @param	MenuBuilder		The menu to add commands to
	 */
	UNREALED_API void FillDefaultFileMenuOpenCommands(FToolMenuSection& InSection);

	/**
	 * Fills in the supplied menu with commands for working with this asset file
	 *
	 * @param	MenuBuilder		The menu to add commands to
	 */
	UNREALED_API void FillDefaultFileMenuCommands(FToolMenuSection& InSection);

	/**
	 * Fills in the supplied menu with commands for modifying this asset that are generally common to most asset editors
	 *
	 * @param	MenuBuilder		The menu to add commands to
	 */
	UNREALED_API void FillDefaultAssetMenuCommands(FToolMenuSection& InSection);

	/**
	 * Fills in the supplied menu with commands for the help menu
	 *
	 * @param	MenuBuilder		The menu to add commands to
	 */
	UNREALED_API void FillDefaultHelpMenuCommands(FToolMenuSection& InSection);

	/** @return	For standalone asset editing tool-kits, returns the toolkit host that was last hosting this asset editor before it was switched to standalone mode (if it's still valid.)  Returns null if these conditions aren't met. */
	UNREALED_API TSharedPtr< IToolkitHost > GetPreviousWorldCentricToolkitHost();

	/**
	 * Static: Used internally to set the world-centric toolkit host for a newly-created standalone asset editing toolkit
	 *
	 * @param ToolkitHost	The tool-kit to use if/when this toolkit is switched back to world-centric mode
	 */
	static UNREALED_API void SetPreviousWorldCentricToolkitHostForNewAssetEditor(TSharedRef< IToolkitHost > ToolkitHost);

	/**
	 * Registers a drawer for the asset editor status bar
	 */
	UNREALED_API void RegisterDrawer(struct FWidgetDrawerConfig&& Drawer, int32 SlotIndex = INDEX_NONE);

	/** Applies the passed in layout (or the saved user-modified version if available).  Must be called after InitAssetEditor. */
	UNREALED_API void RestoreFromLayout(const TSharedRef<FTabManager::FLayout>& NewLayout);

	/** @return Returns this asset editor's tab manager object.  May be nullptr for non-standalone toolkits */
	TSharedPtr<FTabManager> GetTabManager()
	{
		return TabManager;
	}

	/** Registers default tool bar */
	static UNREALED_API void RegisterDefaultToolBar();

	/** Makes a default asset editing toolbar */
	UNREALED_API void GenerateToolbar();

	/** Regenerates the menubar and toolbar widgets */
	UNREALED_API void RegenerateMenusAndToolbars();

	/** Get name used by tool menu */
	UNREALED_API virtual FName GetToolMenuToolbarName(FName& OutParentName) const;
	UNREALED_API virtual void InitToolMenuContext(FToolMenuContext& MenuContext);
	/** Method that can be overriden by derived classes to provide a custom MenuBar to the Standalone Toolkit Host. */
	virtual TSharedPtr<SWidget> CreateMenuBar(const TSharedPtr<FTabManager>& InTabManager, const FName InMenuName, FToolMenuContext& InToolMenuContext) { return nullptr; }
	UNREALED_API FName GetToolMenuToolbarName() const;
	UNREALED_API FName GetToolMenuAppName() const;
	UNREALED_API FName GetToolMenuName() const;

	/** Called at the end of RegenerateMenusAndToolbars() */
	virtual void PostRegenerateMenusAndToolbars() { }

	// Called when another toolkit (such as a ed mode toolkit) is being hosted in this asset editor toolkit
	virtual void OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit) {}

	// Called when another toolkit (such as a ed mode toolkit) is no longer being hosted in this asset editor toolkit
	virtual void OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit) {}

	/* Called when a toolkit requests an overlay widget to be added to the viewport, optionally with a specific Z-Order. Not relevant in the absence
	 * of a viewport.
	 */
	virtual void AddViewportOverlayWidget(TSharedRef<SWidget> InViewportOverlayWidget, int32 ZOrder = INDEX_NONE) {}

	/** Called when a toolkit requests the overlay widget to be removed. */
	virtual void RemoveViewportOverlayWidget(TSharedRef<SWidget> InViewportOverlayWidget) {}
	
	/** Adds or removes extenders to the default menu or the toolbar menu this asset editor */
	UNREALED_API void AddMenuExtender(TSharedPtr<FExtender> Extender);
	UNREALED_API void RemoveMenuExtender(TSharedPtr<FExtender> Extender);
	UNREALED_API void AddToolbarExtender(TSharedPtr<FExtender> Extender);
	UNREALED_API void RemoveToolbarExtender(TSharedPtr<FExtender> Extender);

	/** Returns the default extensibility managers, these are applied for all asset types */
	static UNREALED_API TSharedPtr<FExtensibilityManager> GetSharedMenuExtensibilityManager();
	static UNREALED_API TSharedPtr<FExtensibilityManager> GetSharedToolBarExtensibilityManager();
	
	/**
	 * Allows the caller to set a menu overlay, displayed to the far right of the editor's menu bar
	 *
	 * @param Widget The widget to use as the overlay
	 */
	UNREALED_API void SetMenuOverlay(TSharedRef<SWidget> Widget);

	/** Adds or removes widgets from the default toolbar in this asset editor */
	UNREALED_API void AddToolbarWidget(TSharedRef<SWidget> Widget);
	UNREALED_API void RemoveAllToolbarWidgets();

	/** True if this actually is editing an asset */
	UNREALED_API bool IsActuallyAnAsset() const;

	/** Called by StandaloneAssetEditorToolkitHost to determine what sections of the StatusBar should be created. */
	UNREALED_API virtual UE::Editor::Toolbars::ECreateStatusBarOptions GetStatusBarCreationOptions() const;
	
	/**
	 * Gets the text to display in a toolkit titlebar for an object
	 * @param	InObject	The object we want a description of
	 * @return a formatted description of the object state (e.g. "MyObject*")
	 */
	static UNREALED_API FText GetLabelForObject(const UObject* InObject);

	/**
	 * Gets the text to display in a toolkit tooltip for an object
	 * @param	InObject	The object we want a description of
	 * @return a formatted description of the object
	 */
	static UNREALED_API FText GetToolTipTextForObject(const UObject* InObject);

	/** Get the asset editor mode manager we are using */
	UE_DEPRECATED(4.26, "Use GetEditorModeManager instead.")
	UNREALED_API class FAssetEditorModeManager* GetAssetEditorModeManager() const;

	/** Set the asset editor mode manager we are using */
	UE_DEPRECATED(4.26, "Override CreateEditorModeManager on the toolkit class instead.")
	UNREALED_API void SetAssetEditorModeManager(class FAssetEditorModeManager* InModeManager);

	virtual void AddGraphEditorPinActionsToContextMenu(FToolMenuSection& InSection) const {};

	/** React to a Drag&Drop events from the viewport */
	virtual void OnViewportDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) {}
	virtual void OnViewportDragLeave(const FDragDropEvent& DragDropEvent) {}
	UNREALED_API virtual FReply OnViewportDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent);

protected:
	friend class UAssetEditorToolkitMenuContext;

	/**
	 * Allows to extend dock tab context menu for an asset editor
	 */
	UNREALED_API virtual void OnExtendTabContextMenu(FMenuBuilder& InMenuBuilder);

	/**
	 * Are there any assets to browse to
	 */
	bool OnCanBrowseAssetInContentBrowser() const;

	/**
	 * Browse to assets in content browser
	 */
	void OnBrowseAssetInContentBrowser();

	/**
	 * Perform any initialization that should happen after the basic toolkit needs are created for the asset editor.
	 * For example, if you would like to activate a mode that is not the default from a mode manager, do that here.
	 */
	virtual void PostInitAssetEditor() {}

	/**	Returns true if this toolkit has any objects being edited */
	bool HasEditingObject() const { return !EditingObjects.IsEmpty(); }

	/**	Returns the single object currently being edited. Asserts if currently editing no object or multiple objects */
	UNREALED_API UObject* GetEditingObject() const;

	/**	Returns an array of all the objects currently being edited. Asserts if editing no objects */
	UNREALED_API const TArray< UObject* >& GetEditingObjects() const;
	UNREALED_API TArray<TObjectPtr<UObject>>& GetEditingObjectPtrs();

	/** Generate the toolbar for common asset actions like Save*/
	virtual UNREALED_API UToolMenu* GenerateCommonActionsToolbar(FToolMenuContext& MenuContext);

	/** Generate the toolbar for read only mode specific content */
	UToolMenu* GenerateReadOnlyToolbar(FToolMenuContext& MenuContext);

	/** Get the collection of edited objects that can be saved. */
	UNREALED_API virtual void GetSaveableObjects(TArray<UObject*>& OutObjects) const;

	/** Adds an item to the Editing Objects list */
	UNREALED_API virtual void AddEditingObject(UObject* Object);

	/** Removes an item from the Editing Objects list */
	UNREALED_API virtual void RemoveEditingObject(UObject* Object);

	/** Called to test if "Save" should be enabled for this asset */
	virtual bool CanSaveAsset() const { return true; }

	/** Internal function to check if the asset can be saved that calls CanSaveAsset */
	bool CanSaveAsset_Internal() const;

	/** Called to test if "Save" should be visible for this asset */
	UNREALED_API virtual bool IsSaveAssetVisible() const;

	/** Called when "Save" is clicked for this asset */
	UNREALED_API virtual void SaveAsset_Execute();

	/** Called from SaveAsset_Execute when assets have been saved */
	virtual void OnAssetsSaved(const TArray<UObject*>& SavedObjects) {}

	/** Called to test if "Save As" should be enabled for this asset */
	virtual bool CanSaveAssetAs() const { return true; }

	/** Internal function to check if the asset can be saved that calls CanSaveAssetAs */
	bool CanSaveAssetAs_Internal() const;

	/** Called to test if "Save As" should be visible for this asset */
	UNREALED_API virtual bool IsSaveAssetAsVisible() const;

	/** Called when "Save As" is clicked for this asset */
	UNREALED_API virtual void SaveAssetAs_Execute();

	/** Whether to open a new asset editor for the newly-saved Asset. Called from SaveAssetAs_Execute for all newly-saved Assets sequentially. */
	virtual bool ShouldReopenEditorForSavedAsset(const UObject* Asset) const { return true; }

	/** Called from SaveAssetAs_Execute when assets have been saved */
	virtual void OnAssetsSavedAs(const TArray<UObject*>& SavedObjects) {}

	/** Called to test if "Find in Content Browser" should be enabled for this asset */
	virtual bool CanFindInContentBrowser() const { return true; }

	/** Called to get the desired visibility when the asset is compiling */
	UNREALED_API virtual EVisibility GetVisibilityWhileAssetCompiling() const;
	
	/** Called to test if "Find in Content Browser" should be visible for this asset */
	virtual bool IsFindInContentBrowserButtonVisible() const { return true; }

	/** Called when "Find in Content Browser" is clicked for this asset */
	UNREALED_API virtual void FindInContentBrowser_Execute();

	/** Called when "Browse Documentation" is clicked for this asset */
	UNREALED_API virtual void BrowseDocumentation_Execute() const;

	/** @return the documentation location for this editor */
	UNREALED_API virtual FString GetDocumentationLink() const;

	/** Called to check to see if there's an asset capable of being reimported */
	UNREALED_API virtual bool CanReimport() const;
	/** Internal function to check if the asset can be reimported that calls CanReimport*/
	bool CanReimport_Internal() const;
	UNREALED_API virtual bool CanReimport(UObject* EditingObject) const;
	
	/** Called when "Reimport" is clicked for this asset */
	UNREALED_API virtual void Reimport_Execute();
	UNREALED_API virtual void Reimport_Execute(UObject* EditingObject);

	/** Called to determine if the user should be prompted for a new file if one is missing during an asset reload */
	UNREALED_API virtual bool ShouldPromptForNewFilesOnReload(const UObject& object) const;

	/** Called when this toolkit is requested to close. Returns false if closing should be prevented. */
	UE_DEPRECATED(5.3, "Use OnRequestClose that takes in an EAssetEditorCloseReason instead")
	virtual bool OnRequestClose() { return true; }

	/** Called when this toolkit is requested to close. Returns false if closing should be prevented. */
	UNREALED_API virtual bool OnRequestClose(EAssetEditorCloseReason InCloseReason);

	/** Called when this toolkit is being closed */
	virtual void OnClose() {}
	/**
	  * Static: Called when "Switch to Standalone Editor" is clicked for the asset editor
	  *
	  * @param ThisToolkitWeakRef	The toolkit that we want to restart in standalone mode
	  */
	static UNREALED_API void SwitchToStandaloneEditor_Execute(TWeakPtr< FAssetEditorToolkit > ThisToolkitWeakRef);

	/**
	  * Static: Called when "Switch to World-Centric Editor" is clicked for the asset editor
	  *
	  * @param	ThisToolkitWeakRef			The toolkit that we want to restart in world-centric mode
	  */
	static UNREALED_API void SwitchToWorldCentricEditor_Execute(TWeakPtr< FAssetEditorToolkit > ThisToolkitWeakRef);

	/** @return a pointer to the brush to use for the tab icon */
	UNREALED_API virtual const FSlateBrush* GetDefaultTabIcon() const;

	/** @return the color to use for the tab color */
	UNREALED_API virtual FLinearColor GetDefaultTabColor() const;

	UNREALED_API virtual void CreateEditorModeManager() override;

	/** Specify the permission lists to use in read only mode */
	virtual void SetupReadOnlyMenuProfiles(FReadOnlyAssetEditorCustomization& OutReadOnlyCustomization) { }

	/** Get the name of the profile registered with UToolMenus for read only menu customizations */
	virtual FName GetReadOnlyMenuProfileName() { return ReadOnlyMenuProfileName; }

	/** Can be overriden to prevent registering the default commands.  */
	UNREALED_API virtual void MapToolkitCommands();

	/** 
	 * Applies the passed in layout.
	 * @param NewLayout The layout to restore.
	 * @param bLoadUserLayout Whether to attempt loading the saved user-modified version.
	 * @note Must be called after InitAssetEditor. 
	 */
	UNREALED_API void RestoreFromLayout(const TSharedRef<FTabManager::FLayout>& NewLayout, bool bLoadUserLayout);

	/** 
	 * Enables standalone toolkit implementations to override the tab layout identifier so it differs from "StandaloneToolkit" or one of its variations. 
	 * If it's not overridden or NAME_None is returned, the default id will be used.
	 */
	virtual FName GetOverrideTabId() const { return NAME_None; }
	
	UE_EXPERIMENTAL(5.8, "Viewport Interactions are Experimental")
	UNREALED_API virtual void OnBuildViewportInteractions(UViewportInteractionsBehaviorSource* Source);
	
	UE_EXPERIMENTAL(5.8, "Viewport Interactions are Experimental")
	virtual void PostBuildViewportInteractions(const UViewportInteractionsBehaviorSource* Source) {};

private:
	// Callback for persisting the Asset Editor's layout.
	void HandleTabManagerPersistLayout( const TSharedRef<FTabManager::FLayout>& LayoutToSave )
	{
		FLayoutSaveRestore::SaveToConfig(GEditorLayoutIni, LayoutToSave);
	}

	/** Initialize the base level customizations for read only mode and register the profile */
	void InitializeReadOnlyMenuProfiles();

private:
	/**
	 * Report the references of the EditingObjects to the GC.  The level of indirection necessary 
	 * so that we don't break compatibility with all the asset editors out there that individually 
	 * implement FGCObject.
	 */
	class FGCEditingObjects : public FGCObject
	{
	public:
		FGCEditingObjects(FAssetEditorToolkit& InOwnerToolkit) : OwnerToolkit(InOwnerToolkit) {}

		/** FGCObject interface */
		UNREALED_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		UNREALED_API virtual FString GetReferencerName() const override;

	private:
		FAssetEditorToolkit& OwnerToolkit;
	} GCEditingObjects;

protected:

	/** For standalone asset editing tool-kits that were switched from world-centric mode on the fly, this stores
	    the toolkit host (level editor) that hosted this toolkit last.  This is used to allow the user to switch the
		toolkit back to world-centric mode. */
	TWeakPtr< IToolkitHost > PreviousWorldCentricToolkitHost;

	/** Controls our internal layout */
	TSharedPtr<FTabManager> TabManager;

	/** Whether only dirty assets should be prompted about on save - otherwise all edited assets will be prompted to the user for save/check-out */
	bool bCheckDirtyOnAssetSave;

	/** The editor mode manager */
	TSharedPtr<FEditorModeTools> EditorModeManager;
	UE_DEPRECATED(4.26, "Use EditorModeManager instead.")
	FAssetEditorModeManager* AssetEditorModeManager;

	/** Array of layout extenders */
	TArray<TSharedPtr<FLayoutExtender>> LayoutExtenders;

	/** The base category that tabs are registered to, allows for child classes to register to the same point. */
	TSharedPtr<FWorkspaceItem> AssetEditorTabsCategory;

	/** What ini file should be used to persist and restore the user's layout. Defaults to GEditorLayoutIni. */
	FString LayoutIni = GEditorLayoutIni;

private:
	static UNREALED_API const FName DefaultAssetEditorToolBarName;
	/** The toolkit standalone host; may be nullptr */
	TWeakPtr<SStandaloneAssetEditorToolkitHost> StandaloneHost;

	/** Static: World centric toolkit host to use for the next created asset editing toolkit */
	static UNREALED_API TWeakPtr< IToolkitHost > PreviousWorldCentricToolkitHostForNewAssetEditor;

	/** The extensibility managers shared by all asset types */
	static UNREALED_API TSharedPtr<FExtensibilityManager> SharedMenuExtensibilityManager;
	static UNREALED_API TSharedPtr<FExtensibilityManager> SharedToolBarExtensibilityManager;

	/** The name of the profile registered with UToolMenus for read only menu customizations */
	static UNREALED_API const FName ReadOnlyMenuProfileName;

	/** The object we're currently editing */
	// @todo toolkit minor: Currently we don't need to serialize this object reference because the AssetEditorSubsystem is kept in sync (and will always serialize it.)
	TArray<TObjectPtr<UObject>> EditingObjects;
	
	/** Asset Editor Default Toolbar */
	TSharedPtr<SWidget> Toolbar;

	/** The widget that will house the default Toolbar widget */
	TSharedPtr<SBorder> ToolbarWidgetContent;

	/** The menu extenders to populate the main toolbar with */
	TArray<TSharedPtr<FExtender>> ToolbarExtenders;

	/** Additional widgets to be added to the toolbar */
	TArray<TSharedRef<SWidget>> ToolbarWidgets;

	/** Whether the buttons on the default toolbar can receive keyboard focus */
	bool bIsToolbarFocusable;

	/** Whether the buttons on the default toolbar use small icons */
	bool bIsToolbarUsingSmallIcons;

	/** Whether the asset editor was opened in edit mode or read only mode */
	EAssetOpenMethod OpenMethod;

	/** Determines menu/toolbar customization in read only mode */
	FReadOnlyAssetEditorCustomization ReadOnlyCustomization;
};
