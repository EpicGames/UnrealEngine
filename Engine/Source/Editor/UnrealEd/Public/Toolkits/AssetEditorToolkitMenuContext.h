// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/GCObject.h"

#include "AssetEditorToolkitMenuContext.generated.h"

class FAssetEditorToolkit;

UCLASS(MinimalAPI)
class UAssetEditorToolkitMenuContext : public UObject
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, Category = "Tool Menus")
	UNREALED_API TArray<UObject*> GetEditingObjects() const;

	TWeakPtr<FAssetEditorToolkit> Toolkit;
};
