// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ToolkitBuilder.h"
#include "Framework/Docking/WorkspaceItem.h"
#include "Framework/Docking/TabManager.h"
#include "Toolkits/IToolkit.h"
#include "Framework/Commands/UICommandList.h"
#include "Toolkits/IToolkitHost.h"
#include "Tools/Modes.h"


class UInteractiveTool;
class UInteractiveToolManager;
class UEdMode;
class IDetailsView;
struct FDetailsViewArgs;
class FAssetEditorModeUILayer;
class SBorder;
class UToolMenu;

/**
 * Base class for all toolkits (abstract).
 */
class FBaseToolkit
	: public IToolkit
{
public:

	/** FBaseToolkit constructor */
	UNREALED_API FBaseToolkit();

	/** Virtual destructor */
	UNREALED_API virtual ~FBaseToolkit();

public:

	// IToolkit interface

	UNREALED_API virtual FName GetToolkitContextFName() const override;
	virtual FText GetTabSuffix() const override { return FText::GetEmpty(); }
	UNREALED_API virtual bool ProcessCommandBindings(const FKeyEvent& InKeyEvent) const override;
	UNREALED_API virtual bool IsHosted() const override;
	UNREALED_API virtual const TSharedRef<IToolkitHost> GetToolkitHost() const override;
	UNREALED_API virtual void BringToolkitToFront() override;
	UNREALED_API virtual TSharedPtr<SWidget> GetInlineContent() const override;
	UNREALED_API virtual bool IsBlueprintEditor() const override;
	virtual TSharedRef<FWorkspaceItem> GetWorkspaceMenuCategory() const override { return WorkspaceMenuCategory.ToSharedRef(); }

	virtual FEditorModeTools& GetEditorModeManager() const = 0;

public:

	/** @return	Returns true if this is a world-centric asset editor.  That is, the user is editing the asset inline in a Level Editor app. */
	UNREALED_API bool IsWorldCentricAssetEditor() const;	

	/** @returns Returns our toolkit command list */
	const TSharedRef<FUICommandList> GetToolkitCommands() const
	{
		return ToolkitCommands;
	}

protected:

	/** @return Returns the prefix string to use for tabs created for this toolkit.  In world-centric mode, tabs get a
	    name prefix to make them distinguishable from other tabs */
	UNREALED_API FString GetTabPrefix() const;

	/** @return Returns the color to use for tabs created for this toolkit.  In world-centric mode, tabs may be colored to
	    make them more easy to distinguish compared to other tabs. */
	UNREALED_API FLinearColor GetTabColorScale() const;

	// Creates the Editor mode manager for your class. Default is to create none, for legacy reasons.
	UNREALED_API virtual void CreateEditorModeManager();

protected:

	/** Asset editing mode, set at creation-time and never changes */
	EToolkitMode::Type ToolkitMode;

	/** List of UI commands for this toolkit.  This should be filled in by the derived class! */
	TSharedRef<FUICommandList> ToolkitCommands;

	/** The host application for this editor.  If editing in world-centric mode, this is the level level editor that we're editing the asset within 
	    Use GetToolkitHost() method to access this member. */
	TWeakPtr<IToolkitHost> ToolkitHost;

	/** The workspace menu category of this toolkit */
	TSharedPtr<FWorkspaceItem> WorkspaceMenuCategory;
};

/**
 * This FModeToolkit just creates a basic UI panel that allows various InteractiveTools to
 * be initialized, and a DetailsView used to show properties of the active Tool.
 */
class FModeToolkit
	: public FBaseToolkit
	, public TSharedFromThis<FModeToolkit>
{
public:

	/** Initializes the mode toolkit */
	UNREALED_API virtual void Init(const TSharedPtr<IToolkitHost>& InitToolkitHost);
	UNREALED_API virtual void Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode);
	UNREALED_API ~FModeToolkit();

public:
	UNREALED_API virtual void SetModeUILayer(const TSharedPtr<FAssetEditorModeUILayer> InLayer) override;

	// FBaseToolkit overrides

	UNREALED_API virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& TabManager) final;
	UNREALED_API virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& TabManager) final;

	// IToolkit interface
	UNREALED_API virtual FName GetToolkitFName() const override;
	UNREALED_API virtual FText GetBaseToolkitName() const override;
	virtual FText GetToolkitName() const override { return GetBaseToolkitName(); }
	virtual FText GetToolkitToolTipText() const override { return GetBaseToolkitName(); }
	UNREALED_API virtual FString GetWorldCentricTabPrefix() const override;
	UNREALED_API virtual bool IsAssetEditor() const override;
	UNREALED_API virtual const TArray<UObject*>* GetObjectsCurrentlyBeingEdited() const override;
	UNREALED_API virtual FLinearColor GetWorldCentricTabColorScale() const override;
	UNREALED_API virtual FEditorModeTools& GetEditorModeManager() const final;

	UNREALED_API virtual TWeakObjectPtr<UEdMode> GetScriptableEditorMode() const final;
	UNREALED_API virtual TSharedPtr<SWidget> GetInlineContent() const;
	UNREALED_API virtual FEdMode* GetEditorMode() const;
	UNREALED_API virtual	FText GetEditorModeDisplayName() const;
	UNREALED_API virtual FSlateIcon GetEditorModeIcon() const;
	/** Returns the number of Mode specific tabs in the mode toolbar **/
	virtual void GetToolPaletteNames(TArray<FName>& PaletteNames) const {}

	/**
	 * @param PaletteIndex      The index of the ToolPalette to build
	 * @returns the name of Tool Palette
	 **/
	virtual FText GetToolPaletteDisplayName(FName Palette) const { return FText(); }

	/* Exclusive Tool Palettes only allow users to use tools from one palette at a time */
	virtual bool HasExclusiveToolPalettes() const { return true; }

	/* Integrated Tool Palettes show up in the same panel as their details */
	virtual bool HasIntegratedToolPalettes() const { return true; }

	/**
	 * @param PaletteIndex      The index of the ToolPalette to build
	 * @param ToolbarBuilder    The builder to use for given PaletteIndex
	**/
	UNREALED_API virtual void BuildToolPalette(FName Palette, class FToolBarBuilder& ToolbarBuilder);

	virtual FText GetActiveToolDisplayName() const { return FText(); }
	virtual FText GetActiveToolMessage() const { return FText(); }

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPaletteChanged, FName);
	FOnPaletteChanged& OnPaletteChanged() { return OnPaletteChangedDelegate; }

	virtual void OnToolPaletteChanged(FName PaletteName) {};
	UNREALED_API void SetCurrentPalette(FName InName);
	UNREALED_API FName GetCurrentPalette() const;
	UNREALED_API void SetModeSettingsObject(UObject* InSettingsObject);
	UNREALED_API virtual void InvokeUI();

	/**
	* Override this function to perform any UI cleanup before the ModeUILayer is no longer valid.
	* 
	* Note: If you had an existing delegate binding to ModeUILayer.Pin()->ToolkitHostShutdownUI() you
	* will need to remove that in order for this method to be invoked.  Overriding this method is preferred.
	*/
	virtual void ShutdownUI() {}

	/**
	 * Override this function to extend the secondary mode toolbar (that appears below the main toolbar) for your mode by
	 * filling in the UToolMenu that is passed in
	 */
	virtual void ExtendSecondaryModeToolbar(UToolMenu *InModeToolbarMenu) {}

	/**
	 * Force the Mode Toolbar/Palette to be repopulated with the current ToolPaletteNames
	 */
	UNREALED_API virtual void RebuildModeToolPalette();

protected:
	UNREALED_API virtual void OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool);
	UNREALED_API virtual void OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool);

	virtual void CustomizeModeDetailsViewArgs(FDetailsViewArgs& ArgsInOut) {}
	virtual void CustomizeDetailsViewArgs(FDetailsViewArgs& ArgsInOut) {}

	UNREALED_API virtual void RequestModeUITabs();

	UNREALED_API void OnModeIDChanged(const FEditorModeID& InID, bool bIsEntering);
	UNREALED_API const FEditorModeInfo* GetEditorModeInfo() const;

	/**
	 * Whether or not the mode toolbar should be shown.  If any active modes generated a toolbar this method will return true
	 */
	UNREALED_API virtual bool ShouldShowModeToolbar() const;
	UNREALED_API TSharedRef<SDockTab> CreatePrimaryModePanel(const FSpawnTabArgs& Args);
	UNREALED_API void UpdatePrimaryModePanel();
	UNREALED_API EVisibility GetInlineContentHolderVisibility() const;
	UNREALED_API EVisibility GetNoToolSelectedTextVisibility() const;

	/**
	 * Creates the mode toolbar tab if needed
	 */
	UNREALED_API TSharedRef<SDockTab> MakeModeToolbarTab(const FSpawnTabArgs& Args);

	/**
	 * Creates the entire tool palette widget, override to specify toolbar style
	 */
	UNREALED_API virtual TSharedRef<SWidget> CreatePaletteWidget(TSharedPtr<FUICommandList> InCommandList, FName InToolbarCustomizationName, FName InPaletteName);

	UNREALED_API void SpawnOrUpdateModeToolbar();
	UNREALED_API virtual void RebuildModeToolBar();


protected:
	TSharedPtr<SWidget> ToolkitWidget;
	TSharedPtr<IDetailsView> ModeDetailsView;
	TSharedPtr<IDetailsView> DetailsView;

	TWeakObjectPtr<UEdMode> OwningEditorMode;

	FName CurrentPaletteName;
	FOnPaletteChanged OnPaletteChangedDelegate;

	/** Inline content area for editor modes */
	TSharedPtr<SBorder> InlineContentHolder;

	/** The container holding the mode toolbar */
	TSharedPtr<SBorder> ModeToolBarContainer;

	/** The active tool header area **/
	TSharedPtr<SBorder> ModeToolHeader;
	TWeakPtr<SDockTab> PrimaryTab;
	/** The dock tab for any modes that generate a toolbar */
	TWeakPtr<SDockTab> ModeToolbarTab;
	FMinorTabConfig PrimaryTabInfo;
	FMinorTabConfig ToolbarInfo;


	/** The actual toolbar rows will be placed in this vertical box */
	TWeakPtr<SVerticalBox> ModeToolbarBox;

	/** The modes palette toolbar **/
	TWeakPtr<SWidgetSwitcher> ModeToolbarPaletteSwitcher;

	TWeakPtr<FAssetEditorModeUILayer> ModeUILayer;

	TArray<TSharedPtr<SDockTab>> CreatedTabs;

	struct FEdModeToolbarRow
	{
		FEdModeToolbarRow(FName InModeID, FName InPaletteName, FText InDisplayName, TSharedRef<SWidget> InToolbarWidget)
			: ModeID(InModeID)
			, PaletteName(InPaletteName)
			, DisplayName(InDisplayName)
			, ToolbarWidget(InToolbarWidget)
		{}
		FName ModeID;
		FName PaletteName;
		FText DisplayName;
		TSharedPtr<SWidget> ToolbarWidget;
	};

	/** All toolbar rows generated by active modes.  There will be one row per active mode that generates a toolbar */
	TArray<FEdModeToolbarRow> ActiveToolBarRows;

	/** a feature flag that can be set to true in order to make and use a ToolkitBuilder,
	 *  if one is defined for a given mode. */ 
	bool bUsesToolkitBuilder = false;
	
	/** The FToolkitBuilder which will build the UI for this mode, if defined */
	TSharedPtr<FToolkitBuilder> ToolkitBuilder;

	/** returns whether or not this FModeToolkit has a ToolkitBuilder defined */
	UNREALED_API bool HasToolkitBuilder() const;

	/** The sections of the toolkit, defined for the ToolkitBuilder (if present) */
	TSharedPtr<FToolkitSections> ToolkitSections;
};
