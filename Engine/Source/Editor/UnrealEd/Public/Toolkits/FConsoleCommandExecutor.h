// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"
#include "GlobalEditorCommonCommands.h"


/**
* Executor for Unreal console commands
*/
class FConsoleCommandExecutor : public IConsoleCommandExecutor
{
public:
	static UNREALED_API FName StaticName();
	UNREALED_API virtual FName GetName() const override;
	UNREALED_API virtual FText GetDisplayName() const override;
	UNREALED_API virtual FText GetDescription() const override;
	UNREALED_API virtual FText GetHintText() const override;
	UNREALED_API virtual void GetSuggestedCompletions(const TCHAR* Input, TArray<FConsoleSuggestion>& Out) override;
	UNREALED_API virtual void GetExecHistory(TArray<FString>& Out) override;
	UNREALED_API virtual bool Exec(const TCHAR* Input) override;
	UNREALED_API virtual bool AllowHotKeyClose() const override;
	UNREALED_API virtual bool AllowMultiLine() const override;

	virtual FInputChord GetHotKey() const override
	{
		return FGlobalEditorCommonCommands::Get().OpenConsoleCommandBox->GetActiveChord(EMultipleKeyBindingIndex::Primary).Get();
	}

	virtual FInputChord GetIterateExecutorHotKey() const override
	{
		return FGlobalEditorCommonCommands::Get().FGlobalEditorCommonCommands::Get().SelectNextConsoleExecutor->GetActiveChord(EMultipleKeyBindingIndex::Primary).Get();
	}
private:
	UNREALED_API bool ExecInternal(const TCHAR*) const;
};
