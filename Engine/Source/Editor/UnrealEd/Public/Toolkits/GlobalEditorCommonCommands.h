// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Docking/TabManager.h"

#define UE_API UNREALED_API

class IMenu;
class SWindow;

UE_DECLARE_TCOMMANDS(class FGlobalEditorCommonCommands, UE_API)

// Global editor common commands
// Note: There is no real global command concept, so these must still be registered in each editor
class FGlobalEditorCommonCommands : public TCommands< FGlobalEditorCommonCommands >
{
public:
	UE_API FGlobalEditorCommonCommands();
	UE_API ~FGlobalEditorCommonCommands();

	UE_API virtual void RegisterCommands() override;

	UE_API static void MapActions(TSharedRef<FUICommandList>& ToolkitCommands);

protected:
	static UE_API void OnPressedCtrlTab(TSharedPtr<FUICommandInfo> TriggeringCommand);
	static UE_API void OnSummonedAssetPicker();
	static UE_API void OnSummonedConsoleCommandBox();
	static UE_API void OnOpenContentBrowserDrawer();
	static UE_API void OnOpenOutputLogDrawer();
	static UE_API void OnOpenFindInAllBlueprints();
	
	static UE_API TSharedRef<SDockTab> SpawnAssetPicker(const FSpawnTabArgs& InArgs);

	static UE_API TSharedPtr<IMenu> OpenPopupMenu(TSharedRef<SWidget> WindowContents, const FVector2D& PopupDesiredSize);
	
	static void OpenSettings(FName InContainerName, FName InCategoryName, FName InSectionName);
	
public:
	TSharedPtr<FUICommandInfo> FindInContentBrowser;
	TSharedPtr<FUICommandInfo> SummonControlTabNavigation;
	TSharedPtr<FUICommandInfo> SummonControlTabNavigationAlternate;
	TSharedPtr<FUICommandInfo> SummonControlTabNavigationBackwards;
	TSharedPtr<FUICommandInfo> SummonControlTabNavigationBackwardsAlternate;
	TSharedPtr<FUICommandInfo> SummonOpenAssetDialog;
	TSharedPtr<FUICommandInfo> SummonOpenAssetDialogAlternate;
	TSharedPtr<FUICommandInfo> OpenDocumentation;
	TSharedPtr<FUICommandInfo> OpenConsoleCommandBox;
	TSharedPtr<FUICommandInfo> SelectNextConsoleExecutor;
	TSharedPtr<FUICommandInfo> OpenOutputLogDrawer;
	TSharedPtr<FUICommandInfo> OpenContentBrowserDrawer;
	TSharedPtr<FUICommandInfo> OpenFindInAllBlueprints;

	/** Level file commands */
	TSharedPtr<FUICommandInfo> OpenLevel;
	
	/** Edit/Configuration commands */
	TSharedPtr<FUICommandInfo> OpenEditorPreferences;
	TSharedPtr<FUICommandInfo> OpenKeyboardShortcuts;
	TSharedPtr<FUICommandInfo> OpenProjectSettings;
};

#undef UE_API
