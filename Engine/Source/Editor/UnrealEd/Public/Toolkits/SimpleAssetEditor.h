// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/IToolkitHost.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "PropertyEditorDelegates.h"

class IDetailsView;

class FSimpleAssetEditor : public FAssetEditorToolkit
{
public:
	/** Delegate that, given an array of assets, returns an array of objects to use in the details view of an FSimpleAssetEditor */
	DECLARE_DELEGATE_RetVal_OneParam(TArray<UObject*>, FGetDetailsViewObjects, const TArray<UObject*>&);

	UNREALED_API virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	UNREALED_API virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;


	/**
	 * Edits the specified asset object
	 *
	 * @param	Mode					Asset editing mode for this editor (standalone or world-centric)
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	ObjectsToEdit			The object to edit
	 * @param	GetDetailsViewObjects	If bound, a delegate to get the array of objects to use in the details view; uses ObjectsToEdit if not bound
	 */
	UNREALED_API void InitEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, const TArray<UObject*>& ObjectsToEdit, FGetDetailsViewObjects GetDetailsViewObjects );

	/** Destructor */
	UNREALED_API virtual ~FSimpleAssetEditor();

	/** IToolkit interface */
	UNREALED_API virtual FName GetToolkitFName() const override;
	UNREALED_API virtual FText GetBaseToolkitName() const override;
	UNREALED_API virtual FText GetToolkitName() const override;
	UNREALED_API virtual FText GetToolkitToolTipText() const override;
	UNREALED_API virtual FString GetWorldCentricTabPrefix() const override;
	UNREALED_API virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual bool IsPrimaryEditor() const override { return true; }
	virtual bool IsSimpleAssetEditor() const override { return true; }
	UNREALED_API virtual FName GetEditingAssetTypeName() const override;

	/** FAssetEditorToolkit interface */
	UNREALED_API virtual void PostRegenerateMenusAndToolbars() override;

	/** Used to show or hide certain properties */
	UNREALED_API void SetPropertyVisibilityDelegate(FIsPropertyVisible InVisibilityDelegate);
	/** Can be used to disable the details view making it read-only */
	UNREALED_API void SetPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled InPropertyEditingDelegate);

protected:
	/** Handler for "Find parent class in CB" button */
	FReply OnFindParentClassInContentBrowserClicked(TObjectPtr<UObject> SyncToClass) const;

	/** Handler for "Edit parent class" button */
	FReply OnEditParentClassClicked(TObjectPtr<UObject> EditClass) const;

	/** Check if the details panel has editing enabled, checks both the asset editor open method to see if we are opened in edit mode and
	 *  the delegate a user can supply using SetPropertyEditingEnabledDelegate
	 */
	bool IsPropertyEditingEnabled() const;

private:
	/** Create the properties tab and its content */
	UNREALED_API TSharedRef<SDockTab> SpawnPropertiesTab( const FSpawnTabArgs& Args );

	/** Handles when an asset is imported */
	UNREALED_API void HandleAssetPostImport(class UFactory* InFactory, UObject* InObject);

	/** Called when objects need to be swapped out for new versions, like after a blueprint recompile. */
	UNREALED_API void OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap);

	/** Details view */
	TSharedPtr< class IDetailsView > DetailsView;

	/** App Identifier. Technically, all simple editors are the same app, despite editing a variety of assets. */
	static UNREALED_API const FName SimpleEditorAppIdentifier;

	/**	The tab ids for all the tabs used */
	static UNREALED_API const FName PropertiesTabId;

	/** The objects open within this editor */
	TArray<UObject*> EditingObjects;

	/** User provided delegate to check if property editing is enabled in the details panel */
	FIsPropertyEditingEnabled IsPropertyEditingEnabledDelegate;

public:
	/** The name given to all instances of this type of editor */
	static UNREALED_API const FName ToolkitFName;

	static UNREALED_API TSharedRef<FSimpleAssetEditor> CreateEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UObject* ObjectToEdit, FGetDetailsViewObjects GetDetailsViewObjects = FGetDetailsViewObjects() );

	static UNREALED_API TSharedRef<FSimpleAssetEditor> CreateEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, const TArray<UObject*>& ObjectsToEdit, FGetDetailsViewObjects GetDetailsViewObjects = FGetDetailsViewObjects() );
};
