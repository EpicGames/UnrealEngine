// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Templates/SharedPointer.h"
#include "Framework/Docking/TabManager.h"
#include "AssetEditorViewportLayout.h"

class SDockTab;
class FSpawnTabArgs;
class SEditorViewport;
class FLayoutExtender;
class FPreviewScene;
class FEditorViewportClient;
class UAssetEditor;

class FBaseAssetToolkit : public FAssetEditorToolkit
{
public:
	UNREALED_API FBaseAssetToolkit(UAssetEditor* InOwningAssetEditor);

	UNREALED_API ~FBaseAssetToolkit();
	virtual FName GetToolkitFName() const override
	{
		return NAME_None;
	}

	virtual FText GetBaseToolkitName() const override
	{
		return FText::GetEmpty();
	}
	virtual FString GetWorldCentricTabPrefix() const override
	{
		return TEXT("");
	}
	virtual FLinearColor GetWorldCentricTabColorScale() const override
	{
		return FLinearColor::White;
	}
	UNREALED_API virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;
	UNREALED_API virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;

	UNREALED_API virtual const TSharedRef<FTabManager::FLayout> GetDefaultLayout() const;
	UNREALED_API virtual void CreateWidgets();
	UNREALED_API virtual void SetEditingObject(class UObject* InObject);
	UNREALED_API virtual void CreateEditorModeManager() override;

public:
	static UNREALED_API const FName ViewportTabID;
	static UNREALED_API const FName DetailsTabID;

protected:
	UNREALED_API virtual void RegisterToolbar();
	UNREALED_API virtual AssetEditorViewportFactoryFunction GetViewportDelegate();
	UNREALED_API virtual TSharedPtr<FEditorViewportClient> CreateEditorViewportClient() const;
	UNREALED_API virtual TSharedRef<SDockTab> SpawnTab_Viewport(const FSpawnTabArgs& Args);
	UNREALED_API virtual TSharedRef<SDockTab> SpawnTab_Details(const FSpawnTabArgs& Args);
	UNREALED_API virtual TSharedRef<IDetailsView> CreateDetailsView();

protected:
	/** Property View */
	TSharedPtr<class IDetailsView> DetailsView;
	// Tracking the active viewports in this editor.
	TSharedPtr<class FEditorViewportTabContent> ViewportTabContent;
	/** Storage for our viewport creation function that will be passed to the viewport layout system*/
	AssetEditorViewportFactoryFunction ViewportDelegate;
	TSharedPtr<FEditorViewportClient> ViewportClient;
	/** Extender for adding to the default layout for this asset editor */
	TSharedPtr<FLayoutExtender> LayoutExtender;
	TSharedPtr<FTabManager::FLayout> StandaloneDefaultLayout;
	FString LayoutAppendix;
	UAssetEditor* OwningAssetEditor;
};
