// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/LegacyEdModeWidgetHelpers.h"

#include "DefaultEdMode.generated.h"

UCLASS(Transient, MinimalAPI)
class UEdModeDefault : public UBaseLegacyWidgetEdMode
{
	GENERATED_BODY()

public:
	UNREALED_API UEdModeDefault();

	UNREALED_API virtual bool UsesPropertyWidgets() const override;
	UNREALED_API virtual bool UsesToolkits() const override;

protected:
	UNREALED_API virtual TSharedRef<FLegacyEdModeWidgetHelper> CreateWidgetHelper() override;
};

namespace FAssetEdModes
{
	UNREALED_API extern const FEditorModeID EM_AssetDefault;
}

UCLASS(Transient, MinimalAPI)
class UAssetEdModeDefault : public UEdModeDefault
{
	GENERATED_BODY()

public:
	UNREALED_API UAssetEdModeDefault();

protected:
	UNREALED_API virtual TSharedRef<FLegacyEdModeWidgetHelper> CreateWidgetHelper() override;
};