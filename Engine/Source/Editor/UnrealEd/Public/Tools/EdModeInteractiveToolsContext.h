// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Editor.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/World.h"
#include "HAL/Platform.h"
#include "InputCoreTypes.h"
#include "InputState.h"
#include "InteractiveTool.h"
#include "InteractiveToolManager.h"
#include "InteractiveToolsContext.h"
#include "Math/Ray.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "EdModeInteractiveToolsContext.generated.h"

#define UE_API UNREALED_API

class FCanvas;
class FEdMode;
class FEditorModeTools;
class FEditorViewportClient;
class FPrimitiveDrawInterface;
class FSceneView;
class FViewport;
class FViewportClient;
class ILevelEditor;
class IToolsContextQueriesAPI;
class IToolsContextRenderAPI;
class IToolsContextTransactionsAPI;
class UEdModeInteractiveToolsContext;
class UGizmoViewContext;
class UInputRouter;
class UInteractiveToolBuilder;
class UMaterialInterface;
class UObject;
class USelection;
class UTypedElementSelectionSet;
class UViewportInteractionsBehaviorSource;
struct FToolBuilderState;

/**
 * UEditorInteractiveToolsContext is an extension/adapter of an InteractiveToolsContext designed 
 * for use in the UE Editor. Currently this implementation assumes that it is created by a
 * Mode Manager (FEditorModeTools), and that the Mode Manager will call various API functions
 * like Render() and Tick() when necessary. 
 * 
 * 
 * allows it to be easily embedded inside an FEdMode. A set of functions are provided which can be
 * called from the FEdMode functions of the same name. These will handle the data type
 * conversions and forwarding calls necessary to operate the ToolsContext
 */
UCLASS(MinimalAPI, Transient)
class UEditorInteractiveToolsContext : public UInteractiveToolsContext
{
	GENERATED_BODY()

public:
	UE_API UEditorInteractiveToolsContext();

	/**
	 * Initialize a new ToolsContext for an EdMode owned by the given InEditorModeManager
	 * @param 
	 */
	UE_API void InitializeContextWithEditorModeManager(FEditorModeTools* InEditorModeManager, UInputRouter* UseInputRouter = nullptr);

	/** Shutdown ToolsContext and clean up any connections/etc */
	UE_API virtual void ShutdownContext();

	// default behavior is to accept active tool
	UE_API virtual void TerminateActiveToolsOnPIEStart();

	// default behavior is to accept active tool
	UE_API virtual void TerminateActiveToolsOnSaveWorld();

	// default behavior is to cancel active tool
	UE_API virtual void TerminateActiveToolsOnWorldTearDown();

	// default behavior is to cancel active tool
	UE_API virtual void TerminateActiveToolsOnLevelChange();

	FEditorModeTools* GetParentEditorModeManager() const { return EditorModeManager; }

	IToolsContextQueriesAPI* GetQueriesAPI() const { return QueriesAPI; }
	IToolsContextTransactionsAPI* GetTransactionAPI() const { return TransactionAPI; }

	/** Call this to notify the Editor that the viewports this ToolsContext is related to may need a repaint, ie during interactive tool usage */
	UE_API virtual void PostInvalidation();

	// UObject Interface
	UE_API virtual UWorld* GetWorld() const override;

	// call functions of the same name on the ToolManager and GizmoManager
	UE_API virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime);
	UE_API virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI);
	UE_API virtual void DrawHUD(FViewportClient* ViewportClient,FViewport* Viewport,const FSceneView* View, FCanvas* Canvas);

	// Convert any pending PostInvalidation timestamp bumps into ViewportClient->Invalidate() calls
	UE_API virtual void PropagateInvalidations(FEditorViewportClient* ViewportClient);

	// These delegates can be used to hook into the Render() / DrawHUD() / Tick() calls above. In particular, non-legacy UEdMode's
	// don't normally receive Render() and DrawHUD() calls from the mode manager, but can attach to these.
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnRender, IToolsContextRenderAPI* RenderAPI);
	FOnRender OnRender;
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnDrawHUD, FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI);
	FOnDrawHUD OnDrawHUD;
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnTick, float DeltaTime);
	FOnTick OnTick;

	/** @return true if selected actors/components can be deleted */
	UE_API virtual bool ProcessEditDelete();

	//
	// Utility functions useful for hooking up to UICommand/etc
	//

	UE_API virtual bool CanStartTool(const FString ToolTypeIdentifier) const;
	UE_API virtual bool HasActiveTool() const;
	UE_API virtual FString GetActiveToolName() const;
	UE_API virtual bool ActiveToolHasAccept() const;
	UE_API virtual bool CanAcceptActiveTool() const;
	UE_API virtual bool CanCancelActiveTool() const;
	UE_API virtual bool CanCompleteActiveTool() const;
	UE_API virtual void StartTool(const FString ToolTypeIdentifier);
	UE_API virtual void EndTool(EToolShutdownType ShutdownType);
	UE_API void Activate();
	UE_API void Deactivate();


	/** @return Ray into 3D scene at last mouse event */
	virtual FRay GetLastWorldRay() const
	{
		check(false);
		return FRay();
	}

	//
	// Configuration functions
	//

	/*
	 * Configure whether ::Render() should early-out for HitProxy rendering passes.
	 * If the Mode does not use HitProxy, and the Tools/Gizmos have expensive Render() calls, this can help with interactive performance.
	 */
	UE_API void SetEnableRenderingDuringHitProxyPass(bool bEnabled);

	/** @return true if HitProxy rendering will be allowed in ::Render() */
	bool GetEnableRenderingDuringHitProxyPass() const { return bEnableRenderingDuringHitProxyPass; }


	/**
	 * Configure whether Transform Gizmos created by the ITF (eg CombinedTransformGizmo) should prefer to show in 'Combined' mode.
	 * If this is disabled, the Gizmo should respect the active Editor Gizmo setting (eg in the Level Viewport)
	 */
	UE_API void SetForceCombinedGizmoMode(bool bEnabled);

	/** @return true if Force Combined Gizmo mode is Enabled */
	bool GetForceCombinedGizmoModeEnabled() const { return bForceCombinedGizmoMode; }


	/**
	 * Configure whether Transform Gizmos created by the ITF (eg CombinedTransformGizmo) should, when in World coordinate system,
	 * snap to an Absolute world-aligned grid, or snap Relative to the initial position of any particular gizmo transform.
	 * Relative is the default and is also the behavior of the standard UE Gizmo.
	 */
	UE_API void SetAbsoluteWorldSnappingEnabled(bool bEnabled);

	/** @return true if Absolute World Snapping mode is Enabled */
	bool GetAbsoluteWorldSnappingEnabled() const { return bEnableAbsoluteWorldSnapping; }

	/*
	 * Configure whether tools should shutdown when entering PIE.
	 * By default, the context will shut down any active tools on PIE start.
	 * Setting this to true will prevent this, but the system makes no promises about whether tools work properly across PIE start/run/shutdown.
	 * Therefore, do not use this if you do not know for certain that it is safe to persist tools across PIE start in your situation.
	 */
	UE_API void SetDeactivateToolsOnPIEStart(bool bDeactivateTools);

	/** @return true if tools should be deactivated when PIE is started */
	bool GetDeactivateToolsOnPIEStart() const { return bDeactivateOnPIEStart; }

	/*
	 * Configure whether tools should shutdown when the world is saving.
	 * By default, the context will shut down any active tools on save.
	 * Setting this to true will prevent this, but the system makes no promises about whether tools work properly across a save.
	 * Therefore, do not use this if you do not know for certain that it is safe to persist tools across a save in your situation.
	 */
	UE_API void SetDeactivateToolsOnSaveWorld(bool bDeactivateTools);

	/** @return true if tools should be deactivated when save is started */
	bool GetDeactivateToolsOnSaveWorld() const { return bDeactivateOnSaveWorld; }

	/*
	* Configure whether tools should shutdown when the level gets changed.
	* By default, the context will shut down any active tools on level change.
	* Setting this to false will prevent this, but the system makes no promises about whether tools work properly across a level switch.
	* Therefore, do not use this if you do not know for certain that it is safe to persist tools across a level change in your situation.
	*/
	UE_API void SetDeactivateToolsOnLevelChange(bool bDeactivateTools);

	/** @return true if tools should be deactivated when level change is started */
	bool GetDeactivateToolsOnLevelChange() const { return bDeactivateOnLevelChange; }

protected:

	// we hide these 
	UE_API virtual void Initialize(IToolsContextQueriesAPI* QueriesAPI, IToolsContextTransactionsAPI* TransactionsAPI) override;
	UE_API virtual void Shutdown() override;

	UE_API virtual void DeactivateActiveTool(EToolSide WhichSide, EToolShutdownType ShutdownType);
	UE_API virtual void DeactivateAllActiveTools(EToolShutdownType ShutdownType);

public:
	UPROPERTY()
	TObjectPtr<UMaterialInterface> StandardVertexColorMaterial;

protected:
	// EdMode implementation of InteractiveToolFramework APIs - see ToolContextInterfaces.h
	IToolsContextQueriesAPI* QueriesAPI;
	IToolsContextTransactionsAPI* TransactionAPI;

	// Tools need to be able to Invalidate the view, in case it is not Realtime.
	// Currently we do this very aggressively, and also force Realtime to be on, but in general we should be able to rely on Invalidation.
	// However there are multiple Views and we do not want to Invalidate immediately, so we store a timestamp for each
	// ViewportClient, and invalidate it when we see it if it's timestamp is out-of-date.
	// (In theory this map will continually grow as new Viewports are created...)
	TMap<FViewportClient*, int32> InvalidationMap;
	// current invalidation timestamp, incremented by invalidation calls
	int32 InvalidationTimestamp = 0;

	// An object in which we save the current scene view information that gizmos can use on the game thread
	// to figure out how big the gizmo is for hit testing. Lives in the context store, but we keep a pointer here
	// to avoid having to look for it.
	UGizmoViewContext* GizmoViewContext = nullptr;

	// Utility function to convert viewport x/y from mouse events (and others?) into scene ray.
	// Copy-pasted from other Editor code, seems kind of expensive?
	static UE_API FRay GetRayFromMousePos(FEditorViewportClient* ViewportClient, FViewport* Viewport, int MouseX, int MouseY);

	// Utility function to convert viewport x/y from mouse events (and others?) into scene ray.
	// Copy-pasted from other Editor code, seems kind of expensive?
	static UE_API FRay GetRayFromMousePos(FEditorViewportClient* ViewportClient, FViewport* Viewport, int MouseX, int MouseY, const bool bInClampMouseToBounds);

	// editor UI state that we set before starting tool and when exiting tool
	// Currently disabling anti-aliasing during active Tools because it causes PDI flickering
	UE_API void SetEditorStateForTool();
	UE_API void RestoreEditorState();

	UE_API void OnToolEnded(UInteractiveToolManager* InToolManager, UInteractiveTool* InEndedTool);
	UE_API void OnToolPostBuild(UInteractiveToolManager* InToolManager, EToolSide InSide, UInteractiveTool* InBuiltTool, UInteractiveToolBuilder* InToolBuilder, const FToolBuilderState& ToolState);

	/** Returns a ViewInteractionState based on the input ViewportClient, and context's ModeManager. */
	UE_API EViewInteractionState GetViewInteractionStateFromViewportClient(const FEditorViewportClient* InViewportClient) const;

	TOptional<FString> PendingToolToStart = {};
	TOptional<EToolShutdownType> PendingToolShutdownType = {};

private:
	FEditorModeTools* EditorModeManager = nullptr;

	// currently defaulting to enabled as FEdModes generally assume this, and in most cases hitproxy pass is not expensive.
	bool bEnableRenderingDuringHitProxyPass = true;

	bool bForceCombinedGizmoMode = false;
	bool bEnableAbsoluteWorldSnapping = false;

	bool bDeactivateOnPIEStart = true;

	bool bDeactivateOnSaveWorld = true;
	bool bIsActive = false;

	bool bDeactivateOnLevelChange = true;
};

DECLARE_MULTICAST_DELEGATE_OneParam(FGetUnsupportedInteractions, TArray<FName>& /* Interactions */)

/**
 * UModeManagerInteractiveToolsContext extends UEditorInteractiveToolsContext with various functions for handling 
 * device (mouse) input. These functions are currently called by the EdMode Manager (FEditorModeTools).
 */
UCLASS(MinimalAPI, Transient)
class UModeManagerInteractiveToolsContext : public UEditorInteractiveToolsContext
{
	GENERATED_BODY()

	//
	// UEditorInteractiveToolsContext API implementations that also forward calls to any child EdMode ToolsContexts
	//
public:
	UE_API virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;
	UE_API virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	UE_API virtual void DrawHUD(FViewportClient* ViewportClient,FViewport* Viewport,const FSceneView* View, FCanvas* Canvas) override;
	UE_API virtual void PropagateInvalidations(FEditorViewportClient* ViewportClient) override;

	UE_API virtual bool ProcessEditDelete() override;

	/**
	 * Called when a new interaction is about to be added. Argument is a TArray<FName>& list of interactions which should not be currently
	 * supported. For example, an Ed Mode could prevent a specific interaction from being added by registering to this delegate, and adding the interaction name to the array.
	 */
	FGetUnsupportedInteractions& OnGetUnsupportedInteractions()
	{
		return GetUnsupportedInteractionsDelegate;
	}

	//
	// Input handling, these functions forward ViewportClient events to the UInputRouter
	//
public:
	UE_API bool InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event);
	/**
	 * This updates internal state like InputKey, but doesn't route the results to the input router. 
	 * Use this if the input is captured by some higher system, to avoid this class from having an
	 * incorrect view of e.g. the mouse state because it did not receive a mouse release event.
	 */
	UE_API void UpdateStateWithoutRoutingInputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event);

	UE_API bool MouseEnter(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y);
	UE_API bool MouseLeave(FEditorViewportClient* ViewportClient, FViewport* Viewport);
	UE_API bool MouseMove(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y);
	UE_API bool LostFocus(FEditorViewportClient* InViewportClient, FViewport* Viewport) const;

	UE_API bool StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport);
	UE_API bool CapturedMouseMove(FEditorViewportClient* InViewportClient, FViewport* InViewport, int32 InMouseX, int32 InMouseY);
	UE_API bool EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport);

	/** @return True if the context has overriden the cursor style, false if not. */
	UE_API bool GetCursor(EMouseCursor::Type& OutCursor) const;

	/** @return The Slate widget to use for the cursor if any. Only called if GetCursor returns Custom. */
	UE_API TSharedPtr<SWidget> GetCursorWidget() const;
	
	UE_EXPERIMENTAL(5.8, "The extension of the visibility override interface into the editor context is experimental")
	/** @return true when the out values have been set. */
	UE_API bool GetCursorVisibilityOverride(bool& bOutHardwareCursorVisible, bool& bOutSoftwareCursorVisible) const;

	/** @return Ray into 3D scene at last mouse event */
	UE_API virtual FRay GetLastWorldRay() const override;

	/** @return currently Hovered viewport client */
	UE_API FEditorViewportClient* GetHoveredViewportClient() const;

	/** @return currently Focused viewport client */
	UE_API FEditorViewportClient* GetFocusedViewportClient() const;

protected:
	UE_API virtual void DeactivateAllActiveTools(EToolShutdownType ShutdownType) override;
	UE_API virtual void Initialize(IToolsContextQueriesAPI* QueriesAPIIn, IToolsContextTransactionsAPI* TransactionsAPIIn) override;
	UE_API virtual void Shutdown() override;
		
	FDeviceButtonState OnGetButtonStateFallback(const FKey& Key) const;
	
	/** Input event instance used to keep track of various button states, etc, that we cannot directly query on-demand */
	FInputDeviceState CurrentMouseState;
	// called when PIE is about to start, shuts down active tools
	FDelegateHandle BeginPIEDelegateHandle;
	// called before a Save starts. This currently shuts down active tools.
	FDelegateHandle PreSaveWorldDelegateHandle;
	// called when a map is changed
	FDelegateHandle WorldTearDownDelegateHandle;
	// called when viewport clients change
	FDelegateHandle ViewportClientListChangedHandle;

private:
	void UpdateMousePosition(FEditorViewportClient* InViewportClient, FViewport* InViewport, int32 X, int32 Y);
	void OnApplicationFocusChanged(bool bInIsFocused);
	
	bool bIsTrackingMouse;

	FGetUnsupportedInteractions GetUnsupportedInteractionsDelegate;



protected:
	UPROPERTY()
	TArray<TObjectPtr<UEdModeInteractiveToolsContext>> EdModeToolsContexts;

public:
	/** 
	* Create and initialize a new EdMode-level ToolsContext derived from the ModeManager ToolsContext.
	* The EdMode ToolsContext does not have it's own InputRouter, it shares the InputRouter with the ModeManager ToolsContext.
	* The ModeManager ToolsContext keeps track of these derived ToolsContext's and automatically Tick()'s them/etc.
	* When the child ToolsContext is shut down, OnChildEdModeToolsContextShutdown() must be called to clean up
	* @return new ToolsContext
	*/
	UE_API UEdModeInteractiveToolsContext* CreateNewChildEdModeToolsContext();

	/**
	 * Call to add a child EdMode ToolsContext created using the above function
	 * @return true if child was added
	 */
	UE_API bool OnChildEdModeActivated(UEdModeInteractiveToolsContext* ChildToolsContext);

	/**
	 * Call to release a child EdMode ToolsContext created using the above function
	 * @return true if child was found and removed
	 */
	UE_API bool OnChildEdModeDeactivated(UEdModeInteractiveToolsContext* ChildToolsContext);

	
	/**
	 * Mark Editor Modes Tools for Drag Tools Support
	 */
	UE_DEPRECATED(5.8, "Drag tools have been merged with VIewport Interactions")
	void SetDragToolsEnabled(bool bInEnabled) { SetViewportInteractionsEnabled(bInEnabled); }

	/**
	 * Enable ITF Input tools support (e.g. Viewport Interactions as camera movement and selection)
	 */
	UE_API void SetViewportInteractionsEnabled(bool bInEnabled);

	/**
	 * Are Drag Tools supported? e.g. Marquee Select
	 * Enable/Disable support by using SetDragToolsEnabled(...)
	 */
	UE_DEPRECATED(5.8, "Drag tools have been merged with VIewport Interactions")
	bool UsesDragTools() const { return UsesViewportInteractions(); } 

	/**
	 * Are Viewport Interactions supported? e.g. camera movements, commands, clicking, etc
	 */
	UE_API bool UsesViewportInteractions() const;

	/**
	 * Return the current Input Tools Behavior Source - nullptr if not enabled
	 */
	UE_API UViewportInteractionsBehaviorSource* GetViewportInteractionsBehaviorSource();
	
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "Use OnBuildViewportInteractions instead.")
	DECLARE_MULTICAST_DELEGATE(FViewportInteractionsRegisterStateChanged)
	UE_DEPRECATED(5.8, "Use OnBuildViewportInteractions instead.")
	FViewportInteractionsRegisterStateChanged& OnRegisterViewportInteractions()
	{
		return OnRegisterViewportInteractionsDelegate;
	}
	UE_DEPRECATED(5.8, "Use OnBuildViewportInteractions instead.")
	FViewportInteractionsRegisterStateChanged OnRegisterViewportInteractionsDelegate;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS	
	UE_EXPERIMENTAL(5.8, "Viewport interactions are experimental")
	DECLARE_MULTICAST_DELEGATE_OneParam(FViewportInteractionsSourceMessage, UViewportInteractionsBehaviorSource*)
	
	UE_EXPERIMENTAL(5.8, "Viewport interactions are experimental")
	DECLARE_MULTICAST_DELEGATE_OneParam(FViewportInteractionsSourceConstMessage, const UViewportInteractionsBehaviorSource*)
	
	UE_EXPERIMENTAL(5.8, "Viewport interactions are experimental")
	/**
	 * Called when viewport interactions are built. Add any custom interactions to the behavior source that is passed in via the event.
	 * Interactions are automatically removed when editor modes change.
	 */
	FViewportInteractionsSourceMessage& OnBuildViewportInteractions() { return OnBuildViewportInteractionsEvent; }


	UE_EXPERIMENTAL(5.8, "Viewport interactions are experimental")
	/**
	 * Called after viewport interactions are built. Can be used to apply any post-processing of behaviors after all behaviors are present.
	 */
	FViewportInteractionsSourceConstMessage& PostBuildViewportInteractions() { return PostBuildViewportInteractionsEvent; }
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

protected:
	/**
	 * Activating Drag Tools if current Mode Manager requires them
	 */
	UE_DEPRECATED(5.8, "Drag tools have been merged with viewport interactions.")
	void RegisterDragTools() {};

	/**
	 * Activating Viewport Interactions if current Mode Manager requires them
	 */
	void RegisterViewportInteractions();

	/**
	 * Deactivating Drag Tools
	 */
	UE_DEPRECATED(5.8, "Drag tools have been merged with viewport interactions.")
	void UnregisterDragTools() {};

	/**
	 * Deactivating Viewport Interactions
	 */
	void UnregisterViewportInteractions();

	/**
	 * Responding to Mode Changes in order to deactivate/activate Drag Tools accordingly
	 */
	UE_API void OnEditorModeChanged(const FEditorModeID& InModeID, bool bInIsEnteringMode);

	//~Begin ITF Alt processing Bypass

	/**
	 * These functions are declared protected to avoid having to deal with deprecation later on.
	 * The Alt modifier bypass will be removed once a complete switch to ITF happens
	 */

	/** Enable/Disable Alt modifier bypass */
	UE_API void SetBypassAltModifier(bool bInBypassAltModifier);

	/** Is this Interactive Tools Context skipping Alt processing? */
	bool BypassAltModifier() const { return bBypassAltModifier; }

	/** Should we skip processing Alt modifier inputs? See UModeManagerInteractiveToolsContext::InputKey */
	bool bBypassAltModifier = true;

	//~End ITF Alt processing Bypass

	UPROPERTY(Transient)
	TObjectPtr<UViewportInteractionsBehaviorSource> ViewportInteractionsBehaviorSource;
	
	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	UE_EXPERIMENTAL(5.8, "Viewport interactions are experimental")
	FViewportInteractionsSourceMessage OnBuildViewportInteractionsEvent;
	
	UE_EXPERIMENTAL(5.8, "Viewport interactions are experimental")
	FViewportInteractionsSourceConstMessage PostBuildViewportInteractionsEvent;
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

	/** Whether viewport interactions can ever be supported */
	bool bUsesViewportInteractions = false;
	/** Whether the currently active editor modes currently support viewport interactions */
	bool bModesSupportViewportInteractions = false;
	/** A flag that allows de/composing viewport interactions once per set of mode changes */
	bool bViewportInteractionsDirty = false;
};


/**
 * UEdModeInteractiveToolsContext is an UEditorInteractiveToolsContext intended for use/lifetime in the context of a UEdMode.
 * This ITC subclass is dependent on a UModeManagerInteractiveToolsContext to provide an InputRouter.
 */
UCLASS(MinimalAPI, Transient)
class UEdModeInteractiveToolsContext : public UEditorInteractiveToolsContext
{
	friend class UModeManagerInteractiveToolsContext;

	GENERATED_BODY()
public:
	/**
	 * Initialize a new EdModeToolsContext that is derived from a ModeManagerToolsContext.
	 * This new ToolsContext will not have it's own InputRouter, it will share the InputRouter with the ModeManagerToolsContext
	 */
	UE_API void InitializeContextFromModeManagerContext(UModeManagerInteractiveToolsContext* ModeManagerToolsContext);

	/** @return Ray into 3D scene at last mouse event */
	UE_API virtual FRay GetLastWorldRay() const override;

protected:
	UPROPERTY()
	TObjectPtr<UModeManagerInteractiveToolsContext> ParentModeManagerToolsContext = nullptr;
};

#undef UE_API
