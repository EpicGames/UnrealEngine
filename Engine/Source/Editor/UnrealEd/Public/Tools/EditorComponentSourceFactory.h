// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ComponentSourceInterfaces.h"
#include "TargetInterfaces/MeshTargetInterfaceTypes.h"
#include "Components/PrimitiveComponent.h"
#include "Components/StaticMeshComponent.h"



class FStaticMeshComponentTargetFactory : public FComponentTargetFactory
{
public:
	// new FStaticMeshComponentTargets returned by Build() will be requested for this LOD
	EMeshLODIdentifier CurrentEditingLOD = EMeshLODIdentifier::MaxQuality;

	UNREALED_API bool CanBuild( UActorComponent* Candidate ) override;
	UNREALED_API TUniquePtr<FPrimitiveComponentTarget> Build( UPrimitiveComponent* PrimitiveComponent ) override;
};



class FStaticMeshComponentTarget : public FPrimitiveComponentTarget
{
public:

	UNREALED_API FStaticMeshComponentTarget(UPrimitiveComponent* Component, EMeshLODIdentifier EditingLOD = EMeshLODIdentifier::LOD0);

	UNREALED_API virtual bool IsValid() const override;

	UNREALED_API virtual void GetMaterialSet(FComponentMaterialSet& MaterialSetOut, bool bAssetMaterials) const override;

	UNREALED_API FMeshDescription* GetMesh() override;

	UNREALED_API void CommitMesh( const FCommitter& ) override;

	UNREALED_API virtual void CommitMaterialSetUpdate(const FComponentMaterialSet& MaterialSet, bool bApplyToAsset) override;

	UNREALED_API virtual bool HasSameSourceData(const FPrimitiveComponentTarget& OtherTarget) const override;


protected:
	// LOD to edit, default is to edit LOD0
	EMeshLODIdentifier EditingLOD = EMeshLODIdentifier::LOD0;
};




