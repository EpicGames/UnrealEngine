// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandList.h"
#include "InteractiveTool.h"
#include "StandardToolModeCommands.h"
#include "Styling/ISlateStyle.h"

namespace UE
{
	/**
	 * Interface for functions that get called on command objects on tool start/end. Used to allow the caller to
	 *  not know the class at compile time, so that, for instance, tools injected via a plugin can still have
	 *  hotkey support.
	 * The command object's RegisterCommands() function still needs to be called in the module's startup method,
	 *  but then this interface can be provided to the host to call when the tool starts/ends.
	 */
	class IInteractiveToolCommandsInterface
	{
	public:
		/**
		 * Bind any of the registered UICommands to the given Tool, if they are compatible.
		 */
		virtual void BindCommandsForCurrentTool(TSharedPtr<FUICommandList> UICommandList, UInteractiveTool* Tool) const = 0;
		/**
		 * Unbind all of the currently-registered commands
		 */
		virtual void UnbindActiveCommands(TSharedPtr<FUICommandList> UICommandList) const = 0;
	};
}

/**
 * TInteractiveToolCommands is a base class that handles connecting up Tool Actions 
 * (ie the FInteractiveToolAction provided by a UInteractiveTool) to the UnrealEditor
 * Command system, which allows for remappable hotkeys, etc
 * 
 * Usage is as follows:
 *    - in your EdMode Module, create a subclass-instance of this, say FMyToolCommands and call FMyToolCommands::Register() in your ::StartupModule() function
 *        - subclass must implement ::GetToolDefaultObjectList(), here you just add GetMutableDefault<MyToolX> to the input list for all your Tools
 *    - add a member TSharedPtr<FUICommandList> UICommandList; to your EdMode impl
 *    - when you start a new Tool, call FMyToolCommands::Get().BindCommandsForCurrentTool(UICommandList, NewTool)
 *    - when you end a Tool, call FMyToolCommands::Get().UnbindActiveCommands()
 *    - in your EdModeImpl::InputKey override, call UICommandList->ProcessCommandBindings()
 */
template<typename CommandContextType>
class TInteractiveToolCommands : public TCommands<CommandContextType>, 
	public UE::IInteractiveToolCommandsInterface
{

public:
	/**
	 * Initialize commands. Collects possible Actions from the available UInteractiveTools
	 * (provided by GetToolDefaultObjectList) and then registers a FUICommandInfo for each.
	 * This needs to be called in ::StartupModule for your EdMode Module
	 */
	virtual void RegisterCommands() override;

	/**
	 * Bind any of the registered UICommands to the given Tool, if they are compatible.
	 */
	virtual void BindCommandsForCurrentTool(TSharedPtr<FUICommandList> UICommandList, UInteractiveTool* Tool) const override;

	/**
	 * Unbind all of the currently-registered commands
	 */
	virtual void UnbindActiveCommands(TSharedPtr<FUICommandList> UICommandList) const override;


	//
	// Interface that subclasses need to implement
	// 


	/**
	 * RegisterCommands() needs actual UInteractiveTool instances for all the Tools that want to
	 * provide Actions which will be connected to hotkeys. We can do this based on the CDO's for
	 * each tool, returned by GetMutableDefault<UYourInteractiveToolType>(). The Tool CDOs are
	 * not owned by a ToolManager and we will only call .GetActionSet() on them.
	 */
	virtual void GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs) = 0;


protected:

	/** Forwarding constructor */
	TInteractiveToolCommands(const FName InContextName, const FText& InContextDesc, const FName InContextParent, const FName InStyleSetName)
		: TCommands<CommandContextType>(InContextName, InContextDesc, InContextParent, InStyleSetName)
	{
	}

	/**
	 * Query FStandardToolModeCommands to find an existing command/hotkey for this standard tool action
	 */
	virtual TSharedPtr<FUICommandInfo> FindStandardCommand(EStandardToolActions ToolActionID);

	/**
	 * List of pairs of known Tool Actions and their associated UICommands.
	 * This list is used at bind-time to connect up the already-registered UICommands to the active Tool
	 */
	struct FToolActionCommand
	{
		FInteractiveToolAction ToolAction;
		TSharedPtr<FUICommandInfo> UICommand;
	};
	TArray<FToolActionCommand> ActionCommands;


	//
	// Support for sharing common commands between Tools where the commands
	// are not shared across Tool Modes (and hence not in FStandardToolModeCommands)
	// 

	/** List FToolActionCommands for standard Tool Actions */
	TMap<EStandardToolActions, FToolActionCommand> SharedActionCommands;

	/**
	 * Find or Create a UICommand for a standard Tool Action, that will be shared across Tools
	 */
	bool FindOrCreateSharedCommand(EStandardToolActions ActionID, FToolActionCommand& FoundOut);

	/**
	 * Create a suitable FInteractiveToolAction for the standard Tool Action, ie with suitable
	 * command name and description strings and hotkeys. This info will be used to create the shared UICommand.
	 */
	virtual bool IntializeStandardToolAction(EStandardToolActions ActionID, FInteractiveToolAction& ToolActionOut);

	/**
	 * Utility function that registeres a Tool Aciton as a UICommand
	 */
	void RegisterUIToolCommand(const FInteractiveToolAction& ToolAction, TSharedPtr<FUICommandInfo>& UICommandInfo);
};






template<typename CommandContextType>
void TInteractiveToolCommands<CommandContextType>::RegisterCommands()
{
	// make sure standard commands are registered
	FStandardToolModeCommands::Register();

	// get list of all tools used by command set
	TArray<UInteractiveTool*> AllToolsCDOs;
	GetToolDefaultObjectList(AllToolsCDOs);

	// collect all the actions in these tools
	TArray<FInteractiveToolAction> ToolActions;
	for (UInteractiveTool* Tool : AllToolsCDOs)
	{
		Tool->GetActionSet()->CollectActions(ToolActions);
	}

	// register all the commands in all the actions
	int NumActions = ToolActions.Num();
	ActionCommands.SetNum(NumActions);
	bool bEverRegistered = false;
	for (int k = 0; k < NumActions; ++k)
	{
		const FInteractiveToolAction& ToolAction = ToolActions[k];
		ActionCommands[k].ToolAction = ToolAction;

		bool bRegistered = false;
		if ((int32)ToolAction.ActionID < (int32)EStandardToolActions::BaseClientDefinedActionID)
		{
			TSharedPtr<FUICommandInfo> FoundStandard = FindStandardCommand((EStandardToolActions)ToolAction.ActionID);
			if (FoundStandard.IsValid())
			{
				ActionCommands[k].UICommand = FoundStandard;
				bRegistered = true;
			}
			else
			{
				FToolActionCommand StandardActionCommand;
				bool bFoundRemap = FindOrCreateSharedCommand((EStandardToolActions)ToolAction.ActionID, StandardActionCommand);
				if (bFoundRemap)
				{
					ActionCommands[k].UICommand = StandardActionCommand.UICommand;
					bRegistered = true;
				}
			}
		}

		if (bRegistered == false)
		{
			RegisterUIToolCommand(ActionCommands[k].ToolAction, ActionCommands[k].UICommand);
			bEverRegistered = true;
		}
	}

	// If you hit this ensure, then your tool must not have registered any actions that were not of the "standard" variety
	//  (e.g. with ID like EStandardToolModeCommands::IncreaseBrushSize, etc). This is currently not supported, and your
	//  tool will likely crash when started. Either don't give the tool a command objects (i.e., not use hotkeys), or add
	//  a hotkey of your own.
	// TODO: Support all-standard-hotkey objects, jira UE-221911
	ensureMsgf(bEverRegistered || ToolActions.Num() == 0, TEXT("Command object had actions, but never registered a FUICommandInfo "
		"object of its own, so its singleton pointer may not be kept alive, causing a crash on a subsequent Get() call. Currently, "
		"a command object must register at least one \"non-standard\" action through the tool to be safely used."));
	// We could remove the ToolActions.Num() == 0 exception above, but it is convenient to set up empty hotkey objects
	//  for tools to populate later in development. On the other hand, registering hotkeys and still hitting a crash is
	//  more perplexing and harder to catch, hence this ensure.
}


template<typename CommandContextType>
void TInteractiveToolCommands<CommandContextType>::BindCommandsForCurrentTool(TSharedPtr<FUICommandList> UICommandList, UInteractiveTool* Tool) const
{
	UClass* ToolClassType = Tool->GetClass();

	int NumActionCommands = ActionCommands.Num();
	for (int32 k = 0; k < NumActionCommands; ++k)
	{
		const FToolActionCommand& ActionCommand = ActionCommands[k];
		if (ActionCommand.ToolAction.ClassType == ToolClassType)
		{
			UICommandList->MapAction(
				ActionCommand.UICommand,
				FExecuteAction::CreateUObject(Tool, &UInteractiveTool::ExecuteAction, ActionCommand.ToolAction.ActionID),
				(ActionCommand.ToolAction.KeyRepeatMode == EToolActionKeyRepeatMode::Enabled)
					? EUIActionRepeatMode::RepeatEnabled
					: EUIActionRepeatMode::RepeatDisabled);
			//FCanExecuteAction::CreateRaw(this, &FEdModeFoliage::CurrentToolUsesBrush));
		}
	}
}


template<typename CommandContextType>
void TInteractiveToolCommands<CommandContextType>::UnbindActiveCommands(TSharedPtr<FUICommandList> UICommandList) const
{
	// @todo would be more efficient if we kept track of which commands were mapped.
	// However currently this function must be const because TCommands::Get() returns a const
	for (const FToolActionCommand& ActionCommand : ActionCommands)
	{
		if (UICommandList->IsActionMapped(ActionCommand.UICommand))
		{
			UICommandList->UnmapAction(ActionCommand.UICommand);
		}
	}
}






//
// Support for re-using known commands from FStandardToolModeCommands
//

template<typename CommandContextType>
TSharedPtr<FUICommandInfo> TInteractiveToolCommands<CommandContextType>::FindStandardCommand(EStandardToolActions ToolActionID)
{
	const FStandardToolModeCommands& StdCommands = FStandardToolModeCommands::Get();
	switch (ToolActionID)
	{
	case EStandardToolActions::IncreaseBrushSize:
		return StdCommands.FindStandardCommand(EStandardToolModeCommands::IncreaseBrushSize);
	case EStandardToolActions::DecreaseBrushSize:
		return StdCommands.FindStandardCommand(EStandardToolModeCommands::DecreaseBrushSize);
	case EStandardToolActions::ToggleWireframe:
		return StdCommands.FindStandardCommand(EStandardToolModeCommands::ToggleWireframe);
	default:
		return nullptr;
	}
}






//
// Support for sharing UI commands between multiple Tools, where the
// shared command is not at the shared-between-modes level
// 

template<typename CommandContextType>
bool TInteractiveToolCommands<CommandContextType>::FindOrCreateSharedCommand(EStandardToolActions ActionID, FToolActionCommand& FoundOut)
{
	if (SharedActionCommands.Contains(ActionID))
	{
		FoundOut = SharedActionCommands[ActionID];
		return true;
	}

	FInteractiveToolAction NewAction;
	bool bIsKnownAction = IntializeStandardToolAction(ActionID, NewAction);

	if (bIsKnownAction)
	{
		check(NewAction.ActionID == (int32)ActionID);
		FToolActionCommand NewActionCommand;
		NewActionCommand.ToolAction = NewAction;
		RegisterUIToolCommand(NewActionCommand.ToolAction, NewActionCommand.UICommand);
		FoundOut = SharedActionCommands.Add(ActionID, NewActionCommand);
		return true;
	}
	return false;
}


template<typename CommandContextType>
bool TInteractiveToolCommands<CommandContextType>::IntializeStandardToolAction(EStandardToolActions ActionID, FInteractiveToolAction& ToolActionOut)
{
	// base class does not have any standard Tool actions. Subclasses can override
	// this function, populating ToolActionOut and returning true, if the ActionID should be shared between multiple Tools
	
	return false;
}




template<typename CommandContextType>
void TInteractiveToolCommands<CommandContextType>::RegisterUIToolCommand(const FInteractiveToolAction& ToolAction, TSharedPtr<FUICommandInfo>& UICommandInfo)
{
	const FString DotString = FString(TEXT(".")) + ToolAction.ActionName;

	FUICommandInfo::MakeCommandInfo(
		this->AsShared(),
		UICommandInfo,
		*ToolAction.ActionName,
		ToolAction.ShortName,
		ToolAction.Description,
		FSlateIcon(this->GetStyleSetName(), ISlateStyle::Join(this->GetContextName(), TCHAR_TO_ANSI(*DotString))),
		EUserInterfaceActionType::Button,
		FInputChord(ToolAction.DefaultModifiers, ToolAction.DefaultKey)
		);
}




