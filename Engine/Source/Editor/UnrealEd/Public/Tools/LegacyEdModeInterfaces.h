// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnrealWidgetFwd.h"
#include "Engine/EngineBaseTypes.h"
#include "InputCoreTypes.h"
#include "Math/Axis.h"
#include "Math/Vector.h"
#include "UObject/Interface.h"
#include "Editor.h"

#include "LegacyEdModeInterfaces.generated.h"

class FEditorViewportClient;
struct FConvexVolume;
enum EModeTools : int8;
class FModeTool;
class FSceneView;
class FPrimitiveDrawInterface;
class FViewport;
class HHitProxy;
struct FViewportClick;
class FCanvas;
class UTexture2D;

UINTERFACE(NotBlueprintable, MinimalAPI)
class ULegacyEdModeSelectInterface : public UInterface
{
	GENERATED_BODY()
};

class ILegacyEdModeSelectInterface
{
	GENERATED_BODY()
public:
	/**
	 * Lets each mode/tool handle box selection in its own way.
	 *
	 * @param	InBox	The selection box to use, in worldspace coordinates.
	 * @return		true if something was selected/deselected, false otherwise.
	 */
	virtual bool BoxSelect(FBox& InBox, bool InSelect = true) = 0;

	/**
	 * Lets each mode/tool handle frustum selection in its own way.
	 *
	 * @param	InFrustum	The selection box to use, in worldspace coordinates.
	 * @return	true if something was selected/deselected, false otherwise.
	 */
	virtual bool FrustumSelect(const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, bool InSelect = true) = 0;
};

UINTERFACE(NotBlueprintable, MinimalAPI)
class ULegacyEdModeWidgetInterface : public UInterface
{
	GENERATED_BODY()
};

class ILegacyEdModeWidgetInterface
{
	GENERATED_BODY()

public:
	/** If the EdMode is handling InputDelta (i.e., returning true from it), this allows a mode to indicated whether or not the Widget should also move. */
	virtual bool AllowWidgetMove() = 0;

	/** Check to see if the current widget mode can be cycled */
	virtual bool CanCycleWidgetMode() const = 0;

	virtual bool ShowModeWidgets() const = 0;
	/**
	 * Allows each mode to customize the axis pieces of the widget they want drawn.
	 *
	 * @param	InwidgetMode	The current widget mode
	 *
	 * @return					A bitfield comprised of AXIS_* values
	 */
	virtual EAxisList::Type GetWidgetAxisToDraw(UE::Widget::EWidgetMode InWidgetMode) const = 0;

	/**
	 * Allows each mode/tool to determine a good location for the widget to be drawn at.
	 */
	virtual FVector GetWidgetLocation() const = 0;

	/**
	 * Lets the mode determine if it wants to draw the widget or not.
	 */
	virtual bool ShouldDrawWidget() const = 0;

	/**
	 * Lets each tool determine if it wants to use the editor widget or not.  If the tool doesn't want to use it,
	 * it will be fed raw mouse delta information (not snapped or altered in any way).
	 */
	virtual bool UsesTransformWidget() const = 0;

	/**
	 * Lets each mode selectively exclude certain widget types.
	 */
	virtual bool UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const = 0;

	virtual FVector GetWidgetNormalFromCurrentAxis(void* InData) = 0;

	/** @name Current widget axis. */
	//@{
	virtual void SetCurrentWidgetAxis(EAxisList::Type InAxis) = 0;
	virtual EAxisList::Type GetCurrentWidgetAxis() const = 0;
	//@}

	/**
	 * Lets each mode selectively enable widgets for editing properties tagged with 'Show 3D Widget' metadata.
	 */
	virtual bool UsesPropertyWidgets() const = 0;

	virtual bool GetCustomDrawingCoordinateSystem(FMatrix& InMatrix, void* InData) = 0;
	virtual bool GetCustomInputCoordinateSystem(FMatrix& InMatrix, void* InData) = 0;

	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) {}
	virtual void DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas) {}
	virtual UTexture2D* GetVertexTexture() { return GEditor->DefaultBSPVertexTexture; }
};

UINTERFACE(NotBlueprintable, MinimalAPI)
class ULegacyEdModeToolInterface : public UInterface
{
	GENERATED_BODY()
};

class ILegacyEdModeToolInterface
{
	GENERATED_BODY()

public:
	// Tools
	virtual void SetCurrentTool(EModeTools InID) = 0;
	virtual void SetCurrentTool(FModeTool* InModeTool) = 0;
	virtual FModeTool* FindTool(EModeTools InID) = 0;

	virtual const TArray<FModeTool*>& GetTools() const = 0;

	/** Returns the current tool. */
	virtual FModeTool* GetCurrentTool() = 0;
	virtual const FModeTool* GetCurrentTool() const = 0;
};

UINTERFACE(NotBlueprintable, MinimalAPI)
class ULegacyEdModeDrawHelperInterface : public UInterface
{
	GENERATED_BODY()
};

class ILegacyEdModeDrawHelperInterface
{
	GENERATED_BODY()

public:
	virtual void Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI) = 0;
};

UINTERFACE(NotBlueprintable, MinimalAPI)
class ULegacyEdModeViewportInterface : public UInterface
{
	GENERATED_BODY()
};

class ILegacyEdModeViewportInterface
{
	GENERATED_BODY()

public:
	virtual bool MouseEnter(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y) { return false; }

	virtual bool MouseLeave(FEditorViewportClient* ViewportClient, FViewport* Viewport) { return false; }

	virtual bool MouseMove(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y) { return false; }

	virtual bool ReceivedFocus(FEditorViewportClient* ViewportClient, FViewport* Viewport) { return false; }

	virtual bool LostFocus(FEditorViewportClient* ViewportClient, FViewport* Viewport) { return false; }

	/**
	 * Called when the mouse is moved while a window input capture is in effect
	 *
	 * @param	InViewportClient	Level editor viewport client that captured the mouse input
	 * @param	InViewport			Viewport that captured the mouse input
	 * @param	InMouseX			New mouse cursor X coordinate
	 * @param	InMouseY			New mouse cursor Y coordinate
	 *
	 * @return	true if input was handled
	 */
	virtual bool CapturedMouseMove(FEditorViewportClient* InViewportClient, FViewport* InViewport, int32 InMouseX, int32 InMouseY) { return false; }

	/** Process all captured mouse moves that occurred during the current frame */
	virtual bool ProcessCapturedMouseMoves(FEditorViewportClient* InViewportClient, FViewport* InViewport, const TArrayView<FIntPoint>& CapturedMouseMoves) { return false; }

	virtual bool InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event) { return false; }
	virtual bool InputAxis(FEditorViewportClient* InViewportClient, FViewport* Viewport, int32 ControllerId, FKey Key, float Delta, float DeltaTime) { return false; }
	virtual bool InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale) { return false; }
	virtual bool StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) { return false; }
	virtual bool EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) { return false; }

	/** Called before mouse movement is converted to drag/rot */
	virtual bool PreConvertMouseMovement(FEditorViewportClient* InViewportClient) { return false; }

	/** Called after mouse movement is converted to drag/rot */
	virtual bool PostConvertMouseMovement(FEditorViewportClient* InViewportClient) { return false; }

	virtual bool HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click) { return false; }

	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) {}

	/** @return True if this mode allows the viewport to use a drag tool */
	virtual bool AllowsViewportDragTool() const { return false; }

	/** If the Edmode is handling its own mouse deltas, it can disable the MouseDeltaTacker */
	virtual bool DisallowMouseDeltaTracking() const { return false; }
	
	/**
	 * @return True if this mode requires the non-ITF gizmo and viewport input system.
	 */
	virtual bool RequiresLegacyViewportInteractions() const { return true; }
};
