// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "Math/Vector.h"
#include "Tools/LegacyEdModeInterfaces.h"
#include "Tools/UEdMode.h"

#include "LegacyEdModeWidgetHelpers.generated.h"

class FCanvas;
class FEditorModeTools;
class FEditorViewportClient;
class FModeTool;
class FPrimitiveDrawInterface;
class FSceneView;
class FViewport;
struct FConvexVolume;

enum EModeTools : int8;
class HHitProxy;
struct FViewportClick;
class UObject;

class FLegacyEdModeWidgetHelper
{
	friend class FEditorModeRegistry;
	friend class UBaseLegacyWidgetEdMode;

public:
	UNREALED_API FLegacyEdModeWidgetHelper();
	virtual ~FLegacyEdModeWidgetHelper() = default;

	UNREALED_API virtual bool AllowWidgetMove();
	UNREALED_API virtual bool CanCycleWidgetMode() const;
	UNREALED_API virtual bool ShowModeWidgets() const;
	UNREALED_API virtual EAxisList::Type GetWidgetAxisToDraw(UE::Widget::EWidgetMode InWidgetMode) const;
	UNREALED_API virtual FVector GetWidgetLocation() const;
	UNREALED_API virtual bool ShouldDrawWidget() const;
	UNREALED_API virtual bool UsesTransformWidget() const;
	UNREALED_API virtual bool UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const;
	UNREALED_API virtual FVector GetWidgetNormalFromCurrentAxis(void* InData);
	UNREALED_API virtual void SetCurrentWidgetAxis(EAxisList::Type InAxis) final;
	UNREALED_API virtual EAxisList::Type GetCurrentWidgetAxis() const final;
	UNREALED_API virtual bool UsesPropertyWidgets() const;
	UNREALED_API virtual bool GetCustomDrawingCoordinateSystem(FMatrix& InMatrix, void* InData);
	UNREALED_API virtual bool GetCustomInputCoordinateSystem(FMatrix& InMatrix, void* InData);
	
	UNREALED_API virtual void ActorSelectionChangeNotify();
	UNREALED_API virtual bool AllowsViewportDragTool() const;
	UNREALED_API virtual bool RequiresLegacyViewportInteractions() const;

	// Property Widgets
	/** Structure that holds info about our optional property widget */
	struct FPropertyWidgetInfo
	{
		FString PropertyName;
		int32 PropertyIndex;
		FName PropertyValidationName;
		FString DisplayName;
		bool bIsTransform;
		bool bLocalSpace;

		FPropertyWidgetInfo()
			: PropertyIndex(INDEX_NONE)
			, bIsTransform(false)
			, bLocalSpace(true)
		{
		}

		/**
		 * Returns the appropriate color, and transform of the widget in local space
		 */
		void GetTransformAndColor(UObject* BestSelectedItem, bool bIsSelected, FTransform& OutLocalTransform, FString& OutValidationMessage, FColor& OutDrawColor) const;
	};

	UNREALED_API virtual bool InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale);
	UNREALED_API virtual bool HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click);
	UNREALED_API virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI);
	UNREALED_API virtual void DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas);

	// @TODO: Find a better home for these?
	/** Value of UPROPERTY can be edited with a widget in the editor (translation, rotation) */
	static UNREALED_API const FName MD_MakeEditWidget;
	/** Specifies a function used for validation of the current value of a property.  The function returns a string that is empty if the value is valid, or contains an error description if the value is invalid */
	static UNREALED_API const FName MD_ValidateWidgetUsing;
	/** If the value of a UPROPERTY's edit widget's transform or translation is stored on the property in local space (defaults true) */
	static UNREALED_API const FName MD_WidgetLocalSpace;
	/** Returns true if this structure can support creating a widget in the editor */
	static UNREALED_API bool CanCreateWidgetForStructure(const UStruct* InPropStruct);
	/** Returns true if this property can support creating a widget in the editor */
	static UNREALED_API bool CanCreateWidgetForProperty(FProperty* InProp);
	/** See if we should create a widget for the supplied property when selecting an actor instance */
	static UNREALED_API bool ShouldCreateWidgetForProperty(FProperty* InProp);

protected:
	/**
	 * Returns the first selected Actor, or NULL if there is no selection.
	 */
	UNREALED_API AActor* GetFirstSelectedActorInstance() const;

	/**
	 * Gets an array of property widget info structures for the given struct/class type for the given container.
	 *
	 * @param InStruct The type of structure/class to access widget info structures for.
	 * @param InContainer The container of the given type.
	 * @param OutInfos An array of widget info structures (output).
	 */
	UNREALED_API void GetPropertyWidgetInfos(const UStruct* InStruct, const void* InContainer, TArray<FPropertyWidgetInfo>& OutInfos) const;

	/** Finds the best item to display widgets for (preferring selected components over selected actors) */
	UNREALED_API virtual UObject* GetItemToTryDisplayingWidgetsFor(FTransform& OutWidgetToWorld) const;
	
	/** Name of the property currently being edited */
	FString EditedPropertyName;

	/** If the property being edited is an array property, this is the index of the element we're currently dealing with */
	int32 EditedPropertyIndex;

	/** Indicates the property being edited is a transform */
	bool bEditedPropertyIsTransform;

	/** World or local space */
	bool bLocalSpace;

	/** The current axis that is being dragged on the widget. */
	EAxisList::Type CurrentWidgetAxis;

	/** Pointer back to the mode tools that we are registered with */
	FEditorModeTools* Owner;

	TScriptInterface<ILegacyEdModeWidgetInterface> ParentModeInterface;
};

// This class is to aid transitioning from native FEdModes to UEdModes, in the case that the FEdMode used property widgets and/or transform widgets
// To use this class:
//   1. Subclass FLegacyEdModeWidgetHelper, and override the methods needed for your native FEdMode implementation
//   2. Transition your native FEdMode to a UObject, by inheriting from this class
//   3. Override CreateWidgetHelper function to return a SharedRef to the class you created in step 1.
UCLASS(Abstract, MinimalAPI)
class UBaseLegacyWidgetEdMode : public UEdMode, public ILegacyEdModeWidgetInterface, public ILegacyEdModeViewportInterface
{
	GENERATED_BODY()

public:
	// UEdMode overrides
	// If you need to do any initialization in your mode, be sure to still call through to this function.
	// It creates the WidgetHelper and hooks up the mode manager pointer for you.
	UNREALED_API virtual void Initialize() override;
	UNREALED_API virtual bool InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale) override;
	UNREALED_API virtual bool HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click) override;
	UNREALED_API virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	UNREALED_API virtual void DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas) override;

	// ILegacyEdModeWidgetInterface overrides are just a pass through to the FLegacyEdModeWidgetHelper
	UNREALED_API virtual bool AllowWidgetMove() override;
	UNREALED_API virtual bool CanCycleWidgetMode() const override;
	UNREALED_API virtual bool ShowModeWidgets() const override;
	UNREALED_API virtual EAxisList::Type GetWidgetAxisToDraw(UE::Widget::EWidgetMode InWidgetMode) const override;
	UNREALED_API virtual FVector GetWidgetLocation() const override;
	UNREALED_API virtual bool ShouldDrawWidget() const override;
	UNREALED_API virtual bool UsesTransformWidget() const override;
	UNREALED_API virtual bool UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const override;
	UNREALED_API virtual FVector GetWidgetNormalFromCurrentAxis(void* InData) override;
	UNREALED_API virtual void SetCurrentWidgetAxis(EAxisList::Type InAxis) override;
	UNREALED_API virtual EAxisList::Type GetCurrentWidgetAxis() const override;
	UNREALED_API virtual bool UsesPropertyWidgets() const override;
	UNREALED_API virtual bool GetCustomDrawingCoordinateSystem(FMatrix& InMatrix, void* InData) override;
	UNREALED_API virtual bool GetCustomInputCoordinateSystem(FMatrix& InMatrix, void* InData) override;
	UNREALED_API virtual void ActorSelectionChangeNotify() override;
	UNREALED_API virtual bool AllowsViewportDragTool() const override;
	UNREALED_API virtual bool RequiresLegacyViewportInteractions() const override;

protected:
	UNREALED_API virtual TSharedRef<FLegacyEdModeWidgetHelper> CreateWidgetHelper();

	TSharedPtr<FLegacyEdModeWidgetHelper> WidgetHelper;
};
