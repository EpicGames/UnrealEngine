// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Templates/SharedPointer.h"

class FUICommandInfo;


/**
 * Integer identifiers for "Standard" Tool commands, that many EditorModes / AssetEditors / etc may share.
 * This allows a single hotkey binding to be used across many contexts.
 */
enum class EStandardToolModeCommands
{
	IncreaseBrushSize = 100,
	DecreaseBrushSize = 101,
	ToggleWireframe = 102
};

UE_DECLARE_TCOMMANDS(class FStandardToolModeCommands, UNREALED_API)

/**
 * FStandardToolModeCommands provides standard commands for Tools. This allows
 * for "sharing" common hotkey bindings between multiple EditorModes
 * 
 * The set of standard commands is defined by EStandardToolModeCommands.
 * You must call FStandardToolModeCommands::Register() on module startup to register these commands
 * (note that they may already be registered by another module, but you should call this to be safe).
 * 
 * Then, when you want to Bind a standard command in your FUICommandList,
 * call FStandardToolModeCommands::Get().FindStandardCommand() to get the registered UICommandInfo.
 * 
 */
class FStandardToolModeCommands : public TCommands<FStandardToolModeCommands>
{
public:
	UNREALED_API FStandardToolModeCommands();

	/**
	 * Registers the set of standard commands. Call on module startup.
	 */
	UNREALED_API virtual void RegisterCommands() override;

	/**
	 * Look up the UICommandInfo for a standard command
	 */
	UNREALED_API TSharedPtr<FUICommandInfo> FindStandardCommand(EStandardToolModeCommands Command) const;

protected:
	TMap<EStandardToolModeCommands, TSharedPtr<FUICommandInfo>> Commands;

};


