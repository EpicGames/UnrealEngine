// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "WidgetModeManager.h"
#include "Input/Events.h"
#include "Templates/SharedPointer.h"

/**
 * Mixin that handles complexity of tool support for you. Calling base widget methods only when a tool does not handle them if appropriate.
 * 
 * Note: Widget must have overriden SWidget::OnPaint to be protected at a minimum to use. Also currently does not support touch or gamepad navigation. 
 *  
 * @TODO: Does not currently allow for ArrangeChildren support this might be useful for rendering widgets on top of the tool supporting widget?
 * A general overlay that accepts addtional widgets to render on top of the tool supporting widget might be interesting. For instance, an editable
 * preview graph / editor of the selected widget might work this way (Such as for graph nodes). 
 * 
 */
template <typename ToolCompatibleWidget>
class TToolCompatibleMixin : public ToolCompatibleWidget
{
	static_assert(std::is_base_of<SWidget, ToolCompatibleWidget>::value, "TToolCompatibleMixin only works with SWidgets.");
public:

	/** Forwarding constructor to ensure mixin members always initialized (Currently none) */
	template<typename ... Args>
	TToolCompatibleMixin(Args&&... args)
		: ToolCompatibleWidget(Forward<Args>(args)...)
	{ }

	// ~Begin SWidget Interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override;
	virtual void OnFocusLost(const FFocusEvent& InFocusEvent) override;
	virtual FReply OnKeyChar(const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent) override;
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual FReply OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override;
	virtual TOptional<EMouseCursor::Type> GetCursor() const override;
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	// ~End SWidget Interface

	/** Use this if your widget is created after your editor's mode manager */
	void SetParentToolkit(TSharedPtr<IToolkit> InParentToolkit);

	/** Use this if your widget is created after your editor's mode manager */
	TSharedPtr<IToolkit> GetParentToolkit();

	/** Helper method for accessing our toolkit's widget mode tool manager */
	FWidgetModeManager* GetWidgetModeManger() const;

protected:
	/** Toolkit containing this tool compatible widget. */
	TWeakPtr<IToolkit> ParentToolkit;

};

#include "Tools/ToolCompatible.inl" // IWYU pragma: export
