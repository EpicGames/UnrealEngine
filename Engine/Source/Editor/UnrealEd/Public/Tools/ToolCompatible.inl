// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

template <typename ToolCompatibleWidget> class TToolCompatibleMixin;

template <typename ToolCompatibleWidget>
void TToolCompatibleMixin<ToolCompatibleWidget>::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	ToolCompatibleWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	if (FWidgetModeManager* WidgetModeManager = GetWidgetModeManger())
	{
		WidgetModeManager->Tick(nullptr, InDeltaTime);
	}
}

template <typename ToolCompatibleWidget>
FReply TToolCompatibleMixin<ToolCompatibleWidget>::OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent)
{
	FReply Reply = ToolCompatibleWidget::OnFocusReceived(MyGeometry, InFocusEvent);
	if (FWidgetModeManager* WidgetModeManager = GetWidgetModeManger())
	{
		WidgetModeManager->ReceivedFocus(nullptr, nullptr);
	}
	return Reply;
}

template <typename ToolCompatibleWidget>
void TToolCompatibleMixin<ToolCompatibleWidget>::OnFocusLost(const FFocusEvent& InFocusEvent)
{
	ToolCompatibleWidget::OnFocusLost(InFocusEvent);
	if (FWidgetModeManager* WidgetModeManager = GetWidgetModeManger())
	{
		WidgetModeManager->LostFocus(nullptr, nullptr);
	}
}

template <typename ToolCompatibleWidget>
FReply TToolCompatibleMixin<ToolCompatibleWidget>::OnKeyChar(const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent)
{
	if (FWidgetModeManager* WidgetModeManager = GetWidgetModeManger())
	{
		if (WidgetModeManager->OnKeyChar(MyGeometry, InCharacterEvent))
		{
			return FReply::Handled();
		}
	}
	return ToolCompatibleWidget::OnKeyChar(MyGeometry, InCharacterEvent);
}

template <typename ToolCompatibleWidget>
FReply TToolCompatibleMixin<ToolCompatibleWidget>::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (FWidgetModeManager* WidgetModeManager = GetWidgetModeManger())
	{
		if (WidgetModeManager->OnKeyDown(MyGeometry, InKeyEvent))
		{
			return FReply::Handled();
		}
	}
	return ToolCompatibleWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

template <typename ToolCompatibleWidget>
FReply TToolCompatibleMixin<ToolCompatibleWidget>::OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (FWidgetModeManager* WidgetModeManager = GetWidgetModeManger())
	{
		if (WidgetModeManager->OnKeyUp(MyGeometry, InKeyEvent))
		{
			return FReply::Handled();
		}
	}
	return ToolCompatibleWidget::OnKeyUp(MyGeometry, InKeyEvent);
}

template <typename ToolCompatibleWidget>
FReply TToolCompatibleMixin<ToolCompatibleWidget>::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (FWidgetModeManager* WidgetModeManager = GetWidgetModeManger())
	{
		if (WidgetModeManager->OnMouseButtonDown(MyGeometry, MouseEvent))
		{
			return FReply::Handled();
		}
	}
	return ToolCompatibleWidget::OnMouseButtonDown(MyGeometry, MouseEvent);
}

template <typename ToolCompatibleWidget>
FReply TToolCompatibleMixin<ToolCompatibleWidget>::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (FWidgetModeManager* WidgetModeManager = GetWidgetModeManger())
	{
		if (WidgetModeManager->OnMouseButtonUp(MyGeometry, MouseEvent))
		{
			return FReply::Handled();
		}
	}
	return ToolCompatibleWidget::OnMouseButtonUp(MyGeometry, MouseEvent);
}

template <typename ToolCompatibleWidget>
FReply TToolCompatibleMixin<ToolCompatibleWidget>::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = ToolCompatibleWidget::OnMouseMove(MyGeometry, MouseEvent);
	if (FWidgetModeManager* WidgetModeManager = GetWidgetModeManger())
	{
		WidgetModeManager->OnMouseMove(MyGeometry, MouseEvent);
	}
	return Reply;
}

template <typename ToolCompatibleWidget>
void TToolCompatibleMixin<ToolCompatibleWidget>::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	ToolCompatibleWidget::OnMouseEnter(MyGeometry, MouseEvent);
	if (FWidgetModeManager* WidgetModeManager = GetWidgetModeManger())
	{
		WidgetModeManager->MouseEnter(nullptr, nullptr, FMath::TruncToInt32(MouseEvent.GetScreenSpacePosition().X), FMath::TruncToInt32(MouseEvent.GetScreenSpacePosition().Y));
	}
}

template <typename ToolCompatibleWidget>
void TToolCompatibleMixin<ToolCompatibleWidget>::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	ToolCompatibleWidget::OnMouseLeave(MouseEvent);
	if (FWidgetModeManager* WidgetModeManager = GetWidgetModeManger())
	{
		WidgetModeManager->MouseLeave(nullptr, nullptr);
	}
}

template <typename ToolCompatibleWidget>
FReply TToolCompatibleMixin<ToolCompatibleWidget>::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (FWidgetModeManager* WidgetModeManager = GetWidgetModeManger())
	{
		if (WidgetModeManager->OnMouseWheel(MyGeometry, MouseEvent))
		{
			return FReply::Handled();
		}
	}
	return ToolCompatibleWidget::OnMouseWheel(MyGeometry, MouseEvent);
}

template <typename ToolCompatibleWidget>
FReply TToolCompatibleMixin<ToolCompatibleWidget>::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (FWidgetModeManager* WidgetModeManager = GetWidgetModeManger())
	{
		if (WidgetModeManager->OnMouseButtonDoubleClick(InMyGeometry, InMouseEvent))
		{
			return FReply::Handled();
		}
	}
	return ToolCompatibleWidget::OnMouseButtonDoubleClick(InMyGeometry, InMouseEvent);
}

template <typename ToolCompatibleWidget>
FReply TToolCompatibleMixin<ToolCompatibleWidget>::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (FWidgetModeManager* WidgetModeManager = GetWidgetModeManger())
	{
		if (WidgetModeManager->OnDragDetected(MyGeometry, MouseEvent))
		{
			return FReply::Handled();
		}
	}
	return ToolCompatibleWidget::OnDragDetected(MyGeometry, MouseEvent);
}

template <typename ToolCompatibleWidget>
void TToolCompatibleMixin<ToolCompatibleWidget>::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	ToolCompatibleWidget::OnMouseCaptureLost(CaptureLostEvent);
	if (FWidgetModeManager* WidgetModeManager = GetWidgetModeManger())
	{
		WidgetModeManager->OnMouseCaptureLost(CaptureLostEvent);
	}
}

template <typename ToolCompatibleWidget>
TOptional<EMouseCursor::Type> TToolCompatibleMixin<ToolCompatibleWidget>::GetCursor() const
{
	if (FWidgetModeManager* WidgetModeManager = GetWidgetModeManger())
	{
		EMouseCursor::Type EditorModeCursor = EMouseCursor::Default;
		if (WidgetModeManager->GetCursor(EditorModeCursor))
		{
			return EditorModeCursor;
		}
	}

	return ToolCompatibleWidget::GetCursor();
}

template <typename ToolCompatibleWidget>
int32 TToolCompatibleMixin<ToolCompatibleWidget>::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	int32 OutLayerId = ToolCompatibleWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	if (FWidgetModeManager* WidgetModeManager = GetWidgetModeManger())
	{
		return WidgetModeManager->OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, OutLayerId, InWidgetStyle, bParentEnabled);
	}

	return OutLayerId;
}

template <typename ToolCompatibleWidget>
void TToolCompatibleMixin<ToolCompatibleWidget>::SetParentToolkit(TSharedPtr<IToolkit> InParentToolkit)
{
	ParentToolkit = InParentToolkit; 
}

template <typename ToolCompatibleWidget>
TSharedPtr<IToolkit> TToolCompatibleMixin<ToolCompatibleWidget>::GetParentToolkit()
{
	return ParentToolkit.Pin();
}

template <typename ToolCompatibleWidget>
FWidgetModeManager* TToolCompatibleMixin<ToolCompatibleWidget>::GetWidgetModeManger() const
{
	if (TSharedPtr<IToolkit> ParentToolkitPtr = ParentToolkit.Pin())
	{
		return static_cast<FWidgetModeManager*>(&ParentToolkitPtr->GetEditorModeManager());
	}

	return nullptr;
}