// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/AssetEditorToolkit.h"
#include "Templates/SharedPointer.h"
#include "UAssetEditor.generated.h"

class FBaseAssetToolkit;
class SDockTab;
class FSpawnTabArgs;
class SEditorViewport;
class FLayoutExtender;
class FTabManager;
class UObject;
struct FTabId;

/**
 * Base class for all asset editors.
 */
UCLASS(Abstract, MinimalAPI)
class UAssetEditor : public UObject, public IAssetEditorInstance
{
	GENERATED_BODY()

public:
	UNREALED_API UAssetEditor();

	UNREALED_API virtual FName GetEditorName() const override;
	UNREALED_API virtual void FocusWindow(UObject* ObjectToFocusOn = nullptr) override;
	virtual bool IsPrimaryEditor() const override
	{
		return true;
	}
	virtual void InvokeTab(const FTabId& TabId) override {}
	UNREALED_API virtual TSharedPtr<FTabManager> GetAssociatedTabManager() override;
	virtual double GetLastActivationTime() override
	{
		return 0.0;
	}
	virtual void RemoveEditingAsset(UObject* Asset) override {}

	UNREALED_API void Initialize();
	UNREALED_API virtual void GetObjectsToEdit(TArray<UObject*>& InObjectsToEdit);
	UNREALED_API virtual TSharedPtr<FBaseAssetToolkit> CreateToolkit();
	UNREALED_API void OnToolkitClosed();

protected:
	FBaseAssetToolkit* ToolkitInstance;
};
