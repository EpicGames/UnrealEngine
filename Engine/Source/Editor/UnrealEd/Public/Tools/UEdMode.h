// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/Modes.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "GenericPlatform/ICursor.h"

#include "UEdMode.generated.h"

class FEditorModeTools;
class FModeToolkit;
class UEditorInteractiveToolsContext;
class UModeManagerInteractiveToolsContext;
class UEdModeInteractiveToolsContext;
class UInteractiveToolManager;
class UInteractiveTool;

class UInteractiveToolBuilder;
class FUICommandInfo;
class FUICommandList;
class FEdMode;

/** Outcomes when determining whether it's possible to perform an action on the edit modes*/
namespace EEditAction
{
	enum Type
	{
		/** Can't process this action */
		Skip = 0,
		/** Can process this action */
		Process,
		/** Stop evaluating other modes (early out) */
		Halt,
	};
};

/** Generic Asset operations that can be disallowed by edit modes */
enum class EAssetOperation
{
	/** Can Asset be deleted */
	Delete,
	/** Can Asset be duplicated / saved as */
	Duplicate,
	/** Can Asset be saved */
	Save,
	/** Can Asset be renamed */
	Rename
};

/** 
 * EToolsContextScope is used to determine the visibility/lifetime of Tools for a ToolsContext.
 * For example Tools at the EdMode scope level will only be available when that Mode is active,
 * will be unregistered when that mode Exits, and so on. 
 */
enum class EToolsContextScope
{
	/** Editor-wide Tools Scope */
	Editor,
	/** Mode-Specific Tools Scope */
	EdMode,
	/** Default mode configured in UEdMode */
	Default
};

/**
 * Base class for all editor modes.
 * 
 * Modes can be switched. Different modes may have different widgets / editing functionality / layouts.
 * Modes can also be compatible, so multiple modes can concurrently affect a single asset editor. 
 * Modes defined in separate modules, say from a plugin, can modify existing asset editors in a different module.
 */
UCLASS(Abstract, MinimalAPI)
class UEdMode : public UObject
{
	GENERATED_BODY()

public:
	/** Friends so it can access mode's internals on construction */
	friend class UAssetEditorSubsystem;

	UNREALED_API UEdMode();

	UNREALED_API virtual void Initialize();

	// Added for handling EDIT Command...
	virtual EEditAction::Type GetActionEditDuplicate() { return EEditAction::Skip; }
	virtual EEditAction::Type GetActionEditDelete() { return EEditAction::Skip; }
	virtual EEditAction::Type GetActionEditCut() { return EEditAction::Skip; }
	virtual EEditAction::Type GetActionEditCopy() { return EEditAction::Skip; }
	virtual EEditAction::Type GetActionEditPaste() { return EEditAction::Skip; }
	virtual EEditAction::Type GetActionDragDuplicate() { return GetActionEditDuplicate(); }
	virtual bool ProcessEditDuplicate() { return false; }
	UNREALED_API virtual bool ProcessEditDelete();
	virtual bool ProcessEditCut() { return false; }
	virtual bool ProcessEditCopy() { return false; }
	virtual bool ProcessEditPaste() { return false; }

	virtual bool IsCompatibleWith(FEditorModeID OtherModeID) const { return false; }
	virtual void ActorMoveNotify() {}
	virtual void ActorsDuplicatedNotify(TArray<AActor*>& PreDuplicateSelection, TArray<AActor*>& PostDuplicateSelection, bool bOffsetLocations) {}
	virtual void ActorSelectionChangeNotify() {};
	virtual void ElementSelectionChangeNotify() {};
	virtual void ActorPropChangeNotify() {}
	virtual void MapChangeNotify() {}

	/**
	 * Lets each mode/tool specify a pivot point around which the camera should orbit
	 * @param	OutPivot	The custom pivot point returned by the mode/tool
	 * @return	true if a custom pivot point was specified, false otherwise.
	 */
	virtual bool GetPivotForOrbit(FVector& OutPivot) const { return false; }

	/**
	 * Get a cursor to override the default with, if any.
	 * @return true if the cursor was overridden.
	 */
	virtual bool GetCursor(EMouseCursor::Type& OutCursor) const { return false; }

	/** Get override cursor visibility settings */
	virtual bool GetOverrideCursorVisibility(bool& bWantsOverride, bool& bHardwareCursorVisible, bool bSoftwareCursorVisible) const { return false; }

	virtual bool ShouldDrawBrushWireframe(AActor* InActor) const { return true; }

	/** If Rotation Snap should be enabled for this mode*/
	UNREALED_API virtual bool IsSnapRotationEnabled();

	/** If this mode should override the snap rotation
	* @param	Rotation		The Rotation Override
	*
	* @return					True if you have overridden the value
	*/
	virtual bool SnapRotatorToGridOverride(FRotator& Rotation) { return false; };

	virtual void UpdateInternalData() {}

	UNREALED_API virtual void Enter();

	/**
	 * Registers and maps the provided UI command to actions that start / stop a given tool.
	 * Later on this UI command can be referenced to add the tool to a toolbar.
	 * 
	 * @param	UICommand		Command to map tool start / stop actions to
	 * @param	ToolIdentifier	Unique string identifier for the tool, used to check if tool is active
	 * @param	Builder			Builder for tool to be used by actions
	 * @param	ToolScope		Scope to determine lifetime of tool (Editor, Mode, etc)
	 */
	UNREALED_API virtual void RegisterTool(TSharedPtr<FUICommandInfo> UICommand, FString ToolIdentifier, UInteractiveToolBuilder* Builder, EToolsContextScope ToolScope = EToolsContextScope::Default);

	/** 
	 * Subclasses can override this to add additional checks on whether a tool should be allowed to start.
	 * By default the check disallows starting tools during play/simulate in editor.
	 */
	UNREALED_API virtual bool ShouldToolStartBeAllowed(const FString& ToolIdentifier) const;
	UNREALED_API virtual void Exit();

	virtual void PostUndo() {}

	virtual void ModeTick(float DeltaTime) {}

	/**
	 * Check to see if this UEdMode wants to disallow AutoSave
	 * @return true if AutoSave can be applied right now
	 */
	virtual bool CanAutoSave() const { return true; }

	/**
	 * Check to see if this UEdMode wants to disallow operation on current asset
	 * @return true if operation is supported right now
	 */
	virtual bool IsOperationSupportedForCurrentAsset(EAssetOperation InOperation) const { return true; }

	UNREALED_API virtual void SelectNone();
	virtual void SelectionChanged() {}

	/**
	 * Allows an editor mode to override the bounding box used to focus the viewport on an editor selection.
	 * Note: Not used when HasCustomViewportFocus() returns true.
	 *
	 * @param Actor			The selected actor that is being considered for focus
	 * @param PrimitiveComponent	The component in the actor being considered for focus
	 * @param InOutBox		The box that should be computed for the actor and component
	 * @return bool			true if the mode overrides the box and populated InOutBox, false if it did not populate InOutBox
	 */
	virtual bool ComputeBoundingBoxForViewportFocus(AActor* Actor, UPrimitiveComponent* PrimitiveComponent, FBox& InOutBox) const { return false; }

	// Allows an editor mode to override the viewport focus logic, regardless of editor selection.
	// @return true if the mode wants to control viewport focus
	UNREALED_API virtual bool HasCustomViewportFocus() const;

	// If HasCustomViewportFocus() returns true, this is called to apply the desired viewport focus
	UNREALED_API virtual FBox ComputeCustomViewportFocus() const;

private:
	// @return the active focus box from the active tool's ITF focus api, if applicable, or an invalid box otherwise
	FBox GetFocusBoxFromActiveToolFocusAPI() const;
public:

	/** Handling SelectActor */
	virtual bool Select(AActor* InActor, bool bInSelected) { return 0; }

	/** Check to see if an actor can be selected in this mode - no side effects */
	virtual bool IsSelectionAllowed(AActor* InActor, bool bInSelection) const { return true; }

	/** Check to see if an actor selection is exclusively disallowed by a mode -- no side effects */
	virtual bool IsSelectionDisallowed(AActor* InActor, bool bInSelection) const { return false; }

	/** Check to see if an actor edit is exclusively disallowed by a mode */
	virtual bool IsEditingDisallowed(AActor* InActor) const { return false; }

	/** Returns the editor mode identifier. */
	FEditorModeID GetID() const { return Info.ID; }

	/** Returns the editor mode information. */
	const FEditorModeInfo& GetModeInfo() const { return Info; }

	/** True if this mode uses a toolkit mode (eventually they all should) */
	UNREALED_API virtual bool UsesToolkits() const;

	/** Gets the toolkit created by this mode */
	TWeakPtr<FModeToolkit> GetToolkit() { return Toolkit; }

	/** Returns the world this toolkit is editing */
	UNREALED_API UWorld* GetWorld() const;

	/** Returns the owning mode manager for this mode */
	UNREALED_API FEditorModeTools* GetModeManager() const;

	/**
	 * For use by the EditorModeTools class to get the legacy FEdMode type from a legacy FEdMode wrapper
	 * You should not need to override this function in your UEdMode implementation.
	*/
	virtual FEdMode* AsLegacyMode() { return nullptr; }
	
	virtual bool OnRequestClose() 
	{
		return true;
	}

protected:

	/** Information pertaining to this mode. Should be assigned in the constructor. */
	FEditorModeInfo Info;

	/** Editor Mode Toolkit that is associated with this toolkit mode */
	TSharedPtr<FModeToolkit> Toolkit;

	/** Pointer back to the mode tools that we are registered with */
	FEditorModeTools* Owner;

protected:
	/**
	 * Returns the first selected Actor, or NULL if there is no selection.
	 */
	UNREALED_API AActor* GetFirstSelectedActorInstance() const;

	bool bHaveSavedEditorState;
	bool bSavedAntiAliasingState;

public:

	/**
	 * Default Scope for InteractiveToolsContext API functions, eg RegisterTool(), GetToolManager(), GetInteractiveToolsContext().
	 * See EToolsContextScope for details. Defaults to Ed Mode scope. 
	 */
	EToolsContextScope GetDefaultToolScope() const { return EToolsContextScope::EdMode; }

	/**
	 * @return active ToolManager for the desired (or default) ToolsContext Scope
	 */
	UNREALED_API UInteractiveToolManager* GetToolManager(EToolsContextScope ToolScope = EToolsContextScope::Default) const;

	/**
	 * @return active ToolsContext for the desired (or default) ToolsContext Scope
	 */
	UNREALED_API UEditorInteractiveToolsContext* GetInteractiveToolsContext(EToolsContextScope ToolScope = EToolsContextScope::Default) const;


	virtual TMap<FName, TArray<TSharedPtr<FUICommandInfo>>> GetModeCommands() const
	{
		return TMap<FName, TArray<TSharedPtr<FUICommandInfo>>>();
	};

protected:
	UNREALED_API virtual void CreateToolkit();
	virtual void OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool) {}
	virtual void OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool) {}
	virtual void ActivateDefaultTool() {}
	UNREALED_API virtual void BindCommands();
	UNREALED_API void OnModeActivated(const FEditorModeID& InID, bool bIsActive);

private:

	/** Reference to the ModeManager-level ToolsContext shared across all EdModes */
	TWeakObjectPtr<UModeManagerInteractiveToolsContext> EditorToolsContext;

	/** The ToolsContext for this Mode, created as a child of the EditorToolsContext (shares InputRouter) */
	UPROPERTY()
	TObjectPtr<UEdModeInteractiveToolsContext> ModeToolsContext;

	/** List of Tools this Mode has registered in the EditorToolsContext, to be unregistered on Mode shutdown */
	TArray<TPair<TSharedPtr<FUICommandInfo>, FString>> RegisteredEditorTools;

protected:

	/** Command list lives here so that the key bindings on the commands can be processed in the viewport. */
	TSharedPtr<FUICommandList> ToolCommandList;

	UPROPERTY()
	TSoftClassPtr<UObject> SettingsClass;

	UPROPERTY(Transient)
	TObjectPtr<UObject> SettingsObject;

protected:

	UNREALED_API void CreateInteractiveToolsContexts();
	UNREALED_API void DestroyInteractiveToolsContexts();
};
