// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/NotNull.h"
#include "Templates/Function.h"
#include "UObject/Object.h"

class FProperty;

namespace UE::OverrideOperations
{

/**
 * Resets the properties of the object to its (archetype) defaults. Does not reset subobjects.
 * Added and removed subobjects will be removed and recreated respectively.
 * Supports transactions, but does not start a transaction scope on its own.
 * @param Object The object that should be reset.
 * @param PropertyPredicate [optional] Predicate to only reset specific properties. Resets all properties if empty.
 */
UNREALED_API bool ResetObject(TNotNull<UObject*> Object, const TFunction<bool(TNotNull<const FProperty*>)>& PropertyPredicate = {});
/**
 * Resets the properties the whole object tree to their (archetype) defaults. This includes all subobjects.
 * Added and removed subobjects will be removed and recreated respectively.
 * Supports transactions, but does not start a transaction scope on its own.
 * @param Object The (root) object that should be reset.
 * @param ObjectPredicate [optional] Predicate to only reset specific objects. Resets all objects if empty.
 * @param PropertyPredicate [optional] Predicate to only reset specific properties. Resets all properties if empty.
 */
UNREALED_API bool ResetObjectTree(TNotNull<UObject*> Object, const TFunction<bool(TNotNull<const UObject*>)>& ObjectPredicate = {},
	const TFunction<bool(TNotNull<const UObject*>, TNotNull<const FProperty*>)>& PropertyPredicate = {});

}
