// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()

#include "Engine.h"
#include "EngineDefines.h"

#include "Misc/Timespan.h"
#include "SlateBasics.h"

#include "EditorComponents.h"
#include "EditorReimportHandler.h"
#include "TexAlignTools.h"

#include "TickableEditorObject.h"

// @todo Classes.h
#include "ActorFactories/ActorFactory.h"
#include "ActorFactories/ActorFactoryAmbientSound.h"
#include "ActorFactories/ActorFactoryBlueprint.h"
#include "ActorFactories/ActorFactoryBoxReflectionCapture.h"
#include "ActorFactories/ActorFactoryBoxVolume.h"
#include "ActorFactories/ActorFactoryCameraActor.h"
#include "ActorFactories/ActorFactoryCharacter.h"
#include "ActorFactories/ActorFactoryClass.h"
#include "ActorFactories/ActorFactoryCylinderVolume.h"
#include "ActorFactories/ActorFactoryDeferredDecal.h"
#include "ActorFactories/ActorFactoryDirectionalLight.h"
#include "ActorFactories/ActorFactoryEmitter.h"
#include "ActorFactories/ActorFactoryEmptyActor.h"
#include "ActorFactories/ActorFactoryPawn.h"
#include "ActorFactories/ActorFactoryExponentialHeightFog.h"
#include "ActorFactories/ActorFactoryNote.h"
#include "ActorFactories/ActorFactoryPhysicsAsset.h"
#include "ActorFactories/ActorFactoryPlaneReflectionCapture.h"
#include "ActorFactories/ActorFactoryPlayerStart.h"
#include "ActorFactories/ActorFactoryPointLight.h"
#include "ActorFactories/ActorFactorySkeletalMesh.h"
#include "ActorFactories/ActorFactoryAnimationAsset.h"
#include "ActorFactories/ActorFactorySkyLight.h"
#include "ActorFactories/ActorFactorySphereReflectionCapture.h"
#include "ActorFactories/ActorFactorySphereVolume.h"
#include "ActorFactories/ActorFactorySpotLight.h"
#include "ActorFactories/ActorFactoryStaticMesh.h"
#include "ActorFactories/ActorFactoryBasicShape.h"
#include "ActorFactories/ActorFactoryInteractiveFoliage.h"
#include "ActorFactories/ActorFactoryTargetPoint.h"
#include "ActorFactories/ActorFactoryTextRender.h"
#include "ActorFactories/ActorFactoryTriggerBox.h"
#include "ActorFactories/ActorFactoryTriggerCapsule.h"
#include "ActorFactories/ActorFactoryTriggerSphere.h"
#include "ActorFactories/ActorFactoryVectorFieldVolume.h"
#include "Exporters/AnimSequenceExporterFBX.h"
#include "Preferences/CascadeOptions.h"
#include "Settings/ClassViewerSettings.h"
#include "Commandlets/CompressAnimationsCommandlet.h"
#include "Settings/ContentBrowserSettings.h"
#include "Commandlets/CookCommandlet.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "Preferences/CurveEdOptions.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Editor/UnrealEdTypes.h"
#include "MaterialEditor/DEditorParameterValue.h"
#include "MaterialEditor/DEditorFontParameterValue.h"
#include "MaterialEditor/DEditorMaterialLayersParameterValue.h"
#include "MaterialEditor/DEditorScalarParameterValue.h"
#include "MaterialEditor/DEditorStaticComponentMaskParameterValue.h"
#include "MaterialEditor/DEditorStaticSwitchParameterValue.h"
#include "MaterialEditor/DEditorTextureParameterValue.h"
#include "MaterialEditor/DEditorVectorParameterValue.h"
#include "Commandlets/DerivedDataCacheCommandlet.h"
#include "Commandlets/DiffAssetsCommandlet.h"
#include "Commandlets/DiffFilesCommandlet.h"
#include "Commandlets/DumpBlueprintsInfoCommandlet.h"
#include "Commandlets/DumpHiddenCategoriesCommandlet.h"
#include "GraphEditAction.h"
#include "MaterialGraph/MaterialGraph.h"
#include "EdGraphNode_Comment.h"
#include "MaterialGraph/MaterialGraphNode_Comment.h"
#include "Animation/EditorAnimBaseObj.h"
#include "Animation/EditorAnimCompositeSegment.h"
#include "Animation/EditorAnimSegment.h"
#include "DisplayDebugHelpers.h"
#include "Animation/AnimInstance.h"
#include "Animation/EditorCompositeSection.h"
#include "Animation/EditorNotifyObject.h"
#include "Builders/EditorBrushBuilder.h"
#include "Builders/ConeBuilder.h"
#include "Builders/CubeBuilder.h"
#include "Builders/CurvedStairBuilder.h"
#include "Builders/CylinderBuilder.h"
#include "Builders/LinearStairBuilder.h"
#include "Builders/SheetBuilder.h"
#include "Builders/SpiralStairBuilder.h"
#include "Builders/TetrahedronBuilder.h"
#include "Builders/VolumetricBuilder.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Editor/Transactor.h"
#include "Settings/LevelEditorPlaySettings.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Editor/EditorEngine.h"
#include "IPackageAutoSaver.h"
#include "ISourceControlModule.h"
#include "ComponentVisualizer.h"
#include "ComponentVisualizerManager.h"
#include "Editor/UnrealEdEngine.h"
#include "Settings/EditorExperimentalSettings.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "Settings/EditorMiscSettings.h"
#include "Animation/EditorParentPlayerListObj.h"
#include "Animation/EditorSkeletonNotifyObj.h"
#include "Exporters/ExportTextContainer.h"
#include "Factories/Factory.h"
#include "Factories/AnimBlueprintFactory.h"
#include "Factories/AnimCompositeFactory.h"
#include "Factories/AnimMontageFactory.h"
#include "Factories/AnimSequenceFactory.h"
#include "Factories/PoseAssetFactory.h"
#include "Factories/BlendSpaceFactory1D.h"
#include "Factories/AimOffsetBlendSpaceFactory1D.h"
#include "Factories/BlendSpaceFactoryNew.h"
#include "Factories/AimOffsetBlendSpaceFactoryNew.h"
#include "Factories/BlueprintFactory.h"
#include "Factories/BlueprintFunctionLibraryFactory.h"
#include "Factories/BlueprintMacroFactory.h"
#include "Factories/BlueprintInterfaceFactory.h"
#include "Factories/EnumFactory.h"
#include "Factories/FbxFactory.h"
#include "Factories/FbxSceneImportFactory.h"
#include "Factories/ReimportFbxAnimSequenceFactory.h"
#include "Factories/ReimportFbxSkeletalMeshFactory.h"
#include "Factories/ReimportFbxStaticMeshFactory.h"
#include "Factories/ReimportFbxSceneFactory.h"
#include "Factories/FontFactory.h"
#include "Factories/FontFileImportFactory.h"
#include "Factories/ForceFeedbackEffectFactory.h"
#include "Factories/HapticFeedbackEffectCurveFactory.h"
#include "Factories/HapticFeedbackEffectBufferFactory.h"
#include "Factories/HapticFeedbackEffectSoundWaveFactory.h"
#include "Factories/LevelFactory.h"
#include "Factories/MaterialFactoryNew.h"
#include "Factories/MaterialFunctionFactoryNew.h"
#include "Factories/MaterialInstanceConstantFactoryNew.h"
#include "Factories/MaterialParameterCollectionFactoryNew.h"
#include "Factories/ModelFactory.h"
#include "Factories/ObjectLibraryFactory.h"
#include "Factories/PackageFactory.h"
#include "Factories/PackFactory.h"
#include "Factories/ParticleSystemFactoryNew.h"
#include "Factories/PhysicalMaterialFactoryNew.h"
#include "Factories/PolysFactory.h"
#include "Factories/SkeletonFactory.h"
#include "Factories/SlateBrushAssetFactory.h"
#include "Factories/SlateWidgetStyleAssetFactory.h"
#include "Factories/StructureFactory.h"
#include "Factories/SubsurfaceProfileFactory.h"
#include "Factories/Texture2dFactoryNew.h"
#include "Factories/TextureFactory.h"
#include "Factories/ReimportTextureFactory.h"
#include "Factories/TrueTypeFontFactory.h"
#include "Factories/TextureRenderTargetCubeFactoryNew.h"
#include "Factories/TextureRenderTargetFactoryNew.h"
#include "Factories/ToonProfileFactory.h"
#include "Factories/TouchInterfaceFactory.h"
#include "Factories/VectorFieldStaticFactory.h"
#include "Factories/ReimportVectorFieldStaticFactory.h"
#include "Factories/WorldFactory.h"
#include "Factories/FbxAssetImportData.h"
#include "Factories/FbxAnimSequenceImportData.h"
#include "Factories/FbxMeshImportData.h"
#include "Factories/FbxSkeletalMeshImportData.h"
#include "Factories/FbxStaticMeshImportData.h"
#include "Factories/FbxTextureImportData.h"
#include "Factories/FbxImportUI.h"
#include "Factories/FbxSceneImportData.h"
#include "Factories/FbxSceneImportOptions.h"
#include "Factories/FbxSceneImportOptionsSkeletalMesh.h"
#include "Factories/FbxSceneImportOptionsStaticMesh.h"
#include "Commandlets/FileServerCommandlet.h"
#include "Commandlets/GatherTextCommandletBase.h"
#include "Commandlets/GatherTextCommandlet.h"
#include "Commandlets/GatherTextFromAssetsCommandlet.h"
#include "Commandlets/GatherTextFromMetadataCommandlet.h"
#include "Commandlets/GatherTextFromSourceCommandlet.h"
#include "Commandlets/GenerateGatherArchiveCommandlet.h"
#include "Commandlets/GenerateGatherManifestCommandlet.h"
#include "Commandlets/GenerateTextLocalizationReportCommandlet.h"
#include "Commandlets/GenerateTextLocalizationResourceCommandlet.h"
#include "Commandlets/InternationalizationConditioningCommandlet.h"
#include "Commandlets/InternationalizationExportCommandlet.h"
#include "Commandlets/GenerateBlueprintAPICommandlet.h"
#include "Commandlets/GenerateDistillFileSetsCommandlet.h"
#include "Editor/GroupActor.h"
#include "Settings/LevelEditorMiscSettings.h"
#include "Exporters/LevelExporterFBX.h"
#include "Exporters/LevelExporterLOD.h"
#include "Exporters/LevelExporterOBJ.h"
#include "Exporters/LevelExporterSTL.h"
#include "Exporters/LevelExporterT3D.h"
#include "Preferences/LightmassOptionsObject.h"
#include "Commandlets/ListMaterialsUsedWithMeshEmittersCommandlet.h"
#include "Commandlets/ListStaticMeshesImportedFromSpeedTreesCommandlet.h"
#include "Commandlets/LoadPackageCommandlet.h"
#include "MaterialEditor/MaterialEditorInstanceConstant.h"
#include "MaterialEditor/MaterialEditorMeshComponent.h"
#include "Preferences/MaterialEditorOptions.h"
#include "MaterialGraph/MaterialGraphNode_Base.h"
#include "MaterialGraph/MaterialGraphNode.h"
#include "MaterialGraph/MaterialGraphNode_Root.h"
#include "MaterialGraph/MaterialGraphNode_Composite.h"
#include "MaterialGraph/MaterialGraphSchema.h"
#include "Exporters/ModelExporterT3D.h"
#include "Exporters/ObjectExporterT3D.h"
#include "Commandlets/ParticleSystemAuditCommandlet.h"
#include "Preferences/PersonaOptions.h"
#include "Preferences/PhysicsAssetEditorOptions.h"
#include "Commandlets/PkgInfoCommandlet.h"
#include "Exporters/PolysExporterOBJ.h"
#include "Exporters/PolysExporterT3D.h"
#include "MaterialEditor/PreviewMaterial.h"
#include "Settings/ProjectPackagingSettings.h"
#include "Commandlets/ReplaceActorCommandlet.h"
#include "Commandlets/ResavePackagesCommandlet.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "ThumbnailRendering/SceneThumbnailInfoWithPrimitive.h"
#include "ThumbnailRendering/WorldThumbnailInfo.h"
#include "Exporters/SequenceExporterT3D.h"
#include "Exporters/SkeletalMeshExporterFBX.h"
#include "Exporters/SoundExporterOGG.h"
#include "Exporters/SoundExporterWAV.h"
#include "Exporters/SoundSurroundExporterWAV.h"
#include "Exporters/StaticMeshExporterFBX.h"
#include "Exporters/StaticMeshExporterOBJ.h"
#include "Editor/TemplateMapMetadata.h"
#include "TexAligner/TexAligner.h"
#include "TexAligner/TexAlignerBox.h"
#include "TexAligner/TexAlignerDefault.h"
#include "TexAligner/TexAlignerFit.h"
#include "TexAligner/TexAlignerPlanar.h"
#include "Exporters/TextBufferExporterTXT.h"
#include "Exporters/TextureCubeExporterHDR.h"
#include "Exporters/TextureExporterBMP.h"
#include "Exporters/TextureExporterHDR.h"
#include "Exporters/RenderTargetExporterHDR.h"
#include "Exporters/TextureExporterTGA.h"
#include "ThumbnailRendering/ThumbnailRenderer.h"
#include "ThumbnailRendering/DefaultSizedThumbnailRenderer.h"
#include "ThumbnailHelpers.h"
#include "ThumbnailRendering/AnimBlueprintThumbnailRenderer.h"
#include "ThumbnailRendering/AnimSequenceThumbnailRenderer.h"
#include "ThumbnailRendering/BlendSpaceThumbnailRenderer.h"
#include "ThumbnailRendering/BlueprintThumbnailRenderer.h"
#include "ThumbnailRendering/ClassThumbnailRenderer.h"
#include "ThumbnailRendering/LevelThumbnailRenderer.h"
#include "ThumbnailRendering/MaterialFunctionThumbnailRenderer.h"
#include "ThumbnailRendering/MaterialInstanceThumbnailRenderer.h"
#include "ThumbnailRendering/SkeletalMeshThumbnailRenderer.h"
#include "ThumbnailRendering/SlateBrushThumbnailRenderer.h"
#include "ThumbnailRendering/StaticMeshThumbnailRenderer.h"
#include "ThumbnailRendering/WorldThumbnailRenderer.h"
#include "ThumbnailRendering/SoundWaveThumbnailRenderer.h"
#include "ThumbnailRendering/TextureThumbnailRenderer.h"
#include "ThumbnailRendering/FontThumbnailRenderer.h"
#include "ThumbnailRendering/ParticleSystemThumbnailRenderer.h"
#include "ThumbnailRendering/SubsurfaceProfileRenderer.h"
#include "ThumbnailRendering/SpecularProfileRenderer.h"
#include "ThumbnailRendering/NeuralProfileRenderer.h"
#include "ThumbnailRendering/ToonProfileRenderer.h"
#include "Factories/TextureCubeThumbnailRenderer.h"
#include "Factories/Texture2dArrayThumbnailRenderer.h"
#include "Editor/TransBuffer.h"
#include "Preferences/UnrealEdKeyBindings.h"
#include "Preferences/UnrealEdOptions.h"
#include "Commandlets/UpdateGameProjectCommandlet.h"
#include "UserDefinedStructure/UserDefinedStructEditorData.h"
#include "Commandlets/WrangleContentCommandlet.h"
//////////////////////	@todo Classes.h

#include "Kismet2/ComponentEditorUtils.h"
#include "Commandlets/EditorCommandlets.h"
#include "EditorUndoClient.h"
#include "EditorModeTools.h"
#include "UnrealWidgetFwd.h"
#include "Editor.h"

#include "EditorViewportClient.h"
#include "LevelEditorViewport.h"

#include "EditorModeRegistry.h"
#include "EdMode.h"
#include "EditorModeManager.h"
#include "EditorModes.h"

#include "MRUList.h"

#include "UnrealEdMisc.h"
#include "EditorDirectories.h"
#include "Utils.h"
#include "FileHelpers.h"
#include "PhysicsManipulationMode.h"
#include "PhysicsAssetUtils.h"

#include "ParticleDefinitions.h"

#include "Dialogs/Dialogs.h"
#include "Viewports.h"

#include "UnrealEdGlobals.h"

#include "UnrealEdMessages.h"

#include "EditorAnalytics.h"
