// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

extern UNREALED_API class UUnrealEdEngine* GUnrealEd;

UNREALED_API int32 EditorInit( class IEngineLoop& EngineLoop );

/**
 * Similar to EditorInit(), but the IMainFrameModule is recreated rather than just created.
 * This function is useful e.g., to load a new layout without having to restart the whole editor.
 */
UNREALED_API int32 EditorReinit();

UNREALED_API void EditorExit();
