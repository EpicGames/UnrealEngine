// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "Widgets/SWindow.h"
#include "Editor.h"

class FPerformanceAnalyticsStats;
class FTickableEditorObject;
class FUICommandInfo;
class FConsoleCommandExecutor;
class FEditorAxisDisplayInfo;

enum class EMapChangeType : uint8
{
	/** Map has just been loaded*/
	LoadMap,

	/** Map is about to be saved*/
	SaveMap,

	/** A new map is loaded*/
	NewMap,

	/** The world is about to be torn down */
	TearDownWorld,
};

/** The public interface for the unreal editor misc singleton. */
class FUnrealEdMisc
{
public:

	/** Singleton accessor */
	static UNREALED_API FUnrealEdMisc& Get();

	/** Destructor */
	UNREALED_API virtual ~FUnrealEdMisc();

	/** Initalizes various systems */
	UNREALED_API virtual void OnInit();

	/* Check if this we are editing a template project, and if so mount any shared resource paths it uses */
	UNREALED_API void MountTemplateSharedPaths();

	/* Cleans up various systems */
	UNREALED_API virtual void OnExit();

	/** Performs any required cleanup in the case of a fatal error. */
	UNREALED_API virtual void ShutdownAfterError();

	/**
	 *	Whether or not the map build in progressed was cancelled by the user. 
	 *
	 *	@return	bool		the current state of the flag
	 */
	bool GetMapBuildCancelled() const
	{
		return bCancelBuild;
	}

	/**
	 * Sets the flag that states whether or not the map build was cancelled.
	 *
	 * @param InCancelled	New state for the cancelled flag.
	 */
	void SetMapBuildCancelled( const bool InCancelled )
	{
		bCancelBuild = InCancelled;
	}

	/**
	 * Get the project name we will use to reload the editor when switching projects
	 *
	 * @return	FString		Name of the project the editor will switch to
	 */
	const FString& GetPendingProjectName() const
	{
		return PendingProjectName;
	}

	/**
	 * Set the project name we will use to reload the editor when switching projects
	 *
	 * @param ProjectName		Name of the project to switch to
	 */
	void SetPendingProjectName( const FString& ProjectName )
	{
		PendingProjectName = ProjectName;
	}

	/** Clear the project name we will use to reload the editor when switching projects */
	void ClearPendingProjectName( )
	{
		PendingProjectName.Empty();
		PendingCommandLine.Reset();
	}

	/**
	 * Sets whether saving the layout on close is allowed.
	 *
	 * @param bIsEnabled	true if saving on close is allowed.
	 */
	UE_DEPRECATED(5.6, "Please use FGlobalTabmanager::Get()->SetCanSavePersistentLayouts instead.")
	UNREALED_API void AllowSavingLayoutOnClose(bool bIsEnabled);

	/** Returns true if saving layout on close is allowed. */
	UE_DEPRECATED(5.6, "Please use FGlobalTabmanager::Get()->CanSavePersistentLayouts instead.")
	UNREALED_API bool IsSavingLayoutOnClosedAllowed();

	/**
	 * Sets the config file to use for restoring Config files.
	 *
	 * @param InBackupFile		The filename of the backup file to restore
	 * @param InConfigFile		The config filename to overwrite
	 */
	void SetConfigRestoreFilename(FString InBackupFile, FString InConfigFile)
	{
		RestoreConfigFiles.FindOrAdd(InConfigFile) = InBackupFile;
	}

	/** Clears the config restore name for restoring the specified config file */
	void ClearConfigRestoreFilename(FString Destination)
	{
		RestoreConfigFiles.Remove(Destination);
	}

	/** Clears all the restore names for restoring config files */
	void ClearConfigRestoreFilenames()
	{
		RestoreConfigFiles.Empty();
	}

	/** Retrieves the map of config->backup config filenames to be used for restoring config files */
	const TMap<FString, FString>& GetConfigRestoreFilenames()
	{
		return RestoreConfigFiles;
	}

	/**
	 * Sets whether the preferences file should be deleted.
	 *
	 * @param bIsEnabled	true if the preferences should be deleted.
	 */
	void ForceDeletePreferences(bool bIsEnabled)
	{
		bDeletePreferences = bIsEnabled;
	}

	/** Returns true if preferences should be deleted. */
	bool IsDeletePreferences()
	{
		return bDeletePreferences;
	}

	/** Opens the specified project file or game. Restarts the editor */
	UNREALED_API void SwitchProject(const FString& GameOrProjectFileName, bool bWarn = true, const TOptional<FString>& NewCommandLine = TOptional<FString>());

	/** Restarts the editor, reopening the current project, if any */
	UNREALED_API void RestartEditor(bool bWarn = true, const TOptional<FString>& NewCommandLine = TOptional<FString>());

	/** Ticks the performance analytics used by the analytics heartbeat */
	UNREALED_API void TickPerformanceAnalytics();

	/** Triggers asset analytics if it hasn't been run yet */
	UNREALED_API void TickAssetAnalytics();

	/**
	 * Fetches a URL from the config and optionally switches it to rocket if required
	 *
	 * @param InKey			The key to lookup in the config file
	 * @param OutURL		The URL string listed for the key in the config
	 * @param bCheckRocket	if true, will attempt to change the URL from udn to rocket if possible
	 *
	 * @returns true if successful
	 */
	UNREALED_API bool GetURL( const TCHAR* InKey, FString& OutURL, const bool bCheckRocket = false ) const;

	UNREALED_API void ReplaceDocumentationURLWildcards(FString& Url, const FCultureRef& Culture, const FString& PageId = FString());

	/** Returns the editor executable to use to execute commandlets */
	UNREALED_API FString GetExecutableForCommandlets() const;

	/** 
	 * Opens the Unreal Engine Launcher marketplace page
	 *
	 * @param CustomLocation	Optional custom location within the marketplace to navigate to.  If not specified the launcher will open to the root marketplace page
	 */
	UNREALED_API void OpenMarketplace(const FString& CustomLocation = TEXT(""));

	/** Constructor, private - use Get() function */
	UNREALED_API FUnrealEdMisc();

	/** Displays a property dialog based upon what is currently selected. If any actors are selected, the actor property dialog is displayed. */
	UNREALED_API void CB_SelectedProps();
	UNREALED_API void CB_DisplayLoadErrors();
	UNREALED_API void CB_RefreshEditor();

	/** Tells the editor that something has been done to change the map.  Can be anything from loading a whole new map to changing the BSP. */
	UNREALED_API void CB_MapChange( uint32 InFlags );
	UNREALED_API void CB_RedrawAllViewports();
	UNREALED_API void CB_EditorModeWindowClosed(const TSharedRef<SWindow>&);
	UNREALED_API void CB_LevelActorsAdded(class AActor* InActor);

	/** Called right before unit testing is about to begin */
	UNREALED_API void CB_PreAutomationTesting();

	/** Called right after unit testing concludes */
	UNREALED_API void CB_PostAutomationTesting();

	UNREALED_API void OnEditorChangeMode(FEditorModeID NewEditorMode);
	UNREALED_API void OnEditorPreModal();
	UNREALED_API void OnEditorPostModal();
	UNREALED_API void OnEnsureFailed(const ANSICHAR* Expr, const ANSICHAR* File, int32 Line, const TCHAR* Msg, const TCHAR* CombinedErrorMsg);

	/** BP nativization settings */
	UE_DEPRECATED(5.0, "Blueprint Nativization has been removed as a supported feature. This delegate method will eventually be removed.")
	void OnNativizeBlueprintsSettingChanged(const FString& PackageName, bool bSelect) {}

	/** Called from tab manager when the tab changes */
	UNREALED_API void OnActiveTabChanged(TSharedPtr<SDockTab> PreviouslyActive, TSharedPtr<SDockTab> NewlyActivated);
	UNREALED_API void OnTabForegrounded(TSharedPtr<SDockTab> ForegroundTab, TSharedPtr<SDockTab> BackgroundTab);
	UNREALED_API void OnUserActivityTabChanged(TSharedPtr<SDockTab> InTab);

	/** Delegate that gets called by modules that can't directly access Engine */
	UNREALED_API void OnDeferCommand( const FString& DeferredCommand );

	/** Start the performance survey that attempts to monitor editor performance in the default or simple startup maps */
	UNREALED_API void BeginPerformanceSurvey();

	/** End the performance survey that attempts to monitor editor performance in the default or simple startup maps */
	UNREALED_API void CancelPerformanceSurvey();

	/**
	 * Called when a map is changed (loaded,saved,new map, etc)
	 */
	UNREALED_API void OnMapChanged( UWorld* World, EMapChangeType MapChangeType );

	/** Called when the input manager records a user-defined chord */
	UNREALED_API void OnUserDefinedChordChanged(const FUICommandInfo& CommandInfo);

	/** Delegate for (default) message log UObject token activation - selects the object that the token refers to (if any) */
	UNREALED_API void OnMessageTokenActivated(const TSharedRef<class IMessageToken>& Token);

	/** Delegate for (default) display name of UObject tokens. Can display the name of the actor if an object is/is part of one */
	UNREALED_API FText OnGetDisplayName(const UObject* InObject, const bool bFullPath);

	/** Delegate for (default) message log message selection - selects the objects that the tokens refer to (if any) */
	UNREALED_API void OnMessageSelectionChanged(TArray< TSharedRef<class FTokenizedMessage> >& Selection);

	/** Delegate used to go to assets in the content browser */
	UNREALED_API void OnGotoAsset(const FString& InAssetPath) const;

	/** Delegate used on message log AActor token activation */
	UNREALED_API void OnActorTokenActivated(const TSharedRef<class IMessageToken>& Token);

	/** Delegate used to get a display name for a message log FAssetData token  */
	UNREALED_API FText OnGetAssetDataDisplayName(const FAssetData& InObject, const bool bFullPath);

	/** Delegate used on message log FAssetData token activation */
	UNREALED_API void OnAssetDataTokenActivated(const TSharedRef<class IMessageToken>& Token);

	/** Delegate used to update the map of asset update counts */
	UNREALED_API void OnObjectSaved(UObject* SavedObject, FObjectPreSaveContext SaveContext);

	/** Logs an update to an asset */
	UNREALED_API void LogAssetUpdate(UObject* UpdatedAsset, FObjectPreSaveContext SaveContext);

	/** Initialize engine analytics */
	UNREALED_API void InitEngineAnalytics();

	/** Called when the heartbeat event should be sent to engine analytics */
	UNREALED_API void EditorAnalyticsHeartbeat();

	/** Handles "Enable World Composition" option in WorldSettings */
	UNREALED_API bool EnableWorldComposition(UWorld* InWorld, bool bEnable);

	/** Get the path to the executable that runs the editor */
	UNREALED_API static FString GetProjectEditorBinaryPath();

	/** Finds a map using only the map name, no extension, no path, also caches it for faster lookup next time. */
	UNREALED_API FString FindMapFileFromPartialName(const FString& PartialMapName);

	/** Launch an editor instance, passing ProjectName and the command line args from this running instance */
	UNREALED_API bool SpawnEditorInstance(const FString& ProjectName);

private:
	/** Delegates to support multi-select of message tokens.  Selections from calls to OnMessageTokenActivated or OnActorTokenActivated are batched and selected as a group. */
	UNREALED_API void OnMultiSelectActorFromMessageTokenBegin();
	UNREALED_API void OnMultiSelectActorFromMessageTokenEnd();

	UNREALED_API void SelectActorFromMessageToken(AActor* InActor);

	UNREALED_API void PreSaveWorld(class UWorld* World, FObjectPreSaveContext ObjectSaveContext);

	/** Stores whether or not the current map build was cancelled. */
	bool bCancelBuild;

	/** Has the system has been initialized? */
	bool bInitialized;

	/** The name of a pending project.  When the editor shuts down it will switch to this project if not empty */ 
	FString PendingProjectName;

	/** Optional alt command-line to use when the editor shuts down and switches projects (if unset, the restart will use the same command-line as the current process). */
	TOptional<FString> PendingCommandLine;

	/** Map of config->backup filenames to restore on shutdown of the editor */
	TMap<FString, FString> RestoreConfigFiles;

	/** true if the preferences config file should be deleted. */
	bool bDeletePreferences;

	/** true if editor performance is being monitored */
	bool bIsSurveyingPerformance;

	/** true if an asset analytics pass is pending */
	bool bIsAssetAnalyticsPending;

	/** The time that the last performance survey frame rate sample happened */
	FDateTime LastFrameRateTime;

	/** An array of frame rate samples used by the performance survey */
	TArray<float> FrameRateSamples;

	/** Statistical information needed by the analytics to report editor performance */
	TUniquePtr<FPerformanceAnalyticsStats> PerformanceAnalyticsStats;

	/** handler to notify about navigation building process */
	TSharedPtr<FTickableEditorObject> NavigationBuildingNotificationHandler;

	/** Package names and the number of times they have been updated */
	TMap<FName, uint32> NumUpdatesByAssetName;

	/** Pointer to the classic "Cmd" executor */
	TUniquePtr<FConsoleCommandExecutor> CmdExec;

	/** Pointer to the EditorAxisDisplayInfo implementation. */
	TUniquePtr<FEditorAxisDisplayInfo> EditorAxisDisplayInfo;

	/** Handle to the registered OnUserDefinedChordChanged delegate. */
	FDelegateHandle OnUserDefinedChordChangedDelegateHandle;

	/** Handle to the registered OnMapChanged delegate. */
	FDelegateHandle OnMapChangedDelegateHandle;
	
	/** Handle to the registered OnActiveTabChanged delegate. */
	FDelegateHandle OnActiveTabChangedDelegateHandle;

	/** Handle to the registered OnTabForegrounded delegate. */
	FDelegateHandle OnTabForegroundedDelegateHandle;

	FTimerHandle EditorAnalyticsHeartbeatTimerHandle;	

	bool bActorMultiSelect = false;
	TArray<AActor*> ActorsSelected;
	TArray<FGuid> UnloadedActorsSelected;
};
