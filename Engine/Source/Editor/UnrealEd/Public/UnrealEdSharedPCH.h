// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EngineSharedPCH.h"

// From Messaging:
#include "IMessageContext.h"

// From AssetRegistry:
#include "AssetRegistry/ARFilter.h"

// From BlueprintGraph:
#include "BlueprintNodeSignature.h"
#include "K2Node.h"
#include "EdGraphSchema_K2.h"
#include "K2Node_EditablePinBase.h"

// From GameplayTasks:
#include "GameplayTaskOwnerInterface.h"
#include "GameplayTaskTypes.h"
#include "GameplayTask.h"

// From UnrealEd:
#include "AssetThumbnail.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Editor/UnrealEdTypes.h"
#include "EditorComponents.h"
#include "EditorUndoClient.h"
#include "EditorViewportClient.h"
#include "Factories/Factory.h"
#include "GraphEditor.h"
#include "ScopedTransaction.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "Settings/LevelEditorPlaySettings.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "TickableEditorObject.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Toolkits/BaseToolkit.h"
#include "Toolkits/IToolkit.h"
#include "Toolkits/IToolkitHost.h"
#include "UnrealWidgetFwd.h"
#include "Viewports.h"

// From ToolMenus
#include "ToolMenus.h"