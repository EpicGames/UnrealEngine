// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HitProxies.h"
#include "UObject/GCObject.h"
#include "UnrealWidgetFwd.h"

class FCanvas;
class FEditorModeTools;
class FEditorViewportClient;
class FPrimitiveDrawInterface;
class FSceneView;
class UMaterialInstanceDynamic;
class UMaterialInterface;
class FMaterialRenderProxy;

class FWidget : public FGCObject
{
public:
	constexpr static float AXIS_LENGTH                         = 35.0f;
	constexpr static float TRANSLATE_ROTATE_AXIS_CIRCLE_RADIUS = 20.0f;
	constexpr static float TWOD_AXIS_CIRCLE_RADIUS             = 10.0f;
	constexpr static float INNER_AXIS_CIRCLE_RADIUS            = 48.0f;
	constexpr static float OUTER_AXIS_CIRCLE_RADIUS            = 56.0f;
	constexpr static float ROTATION_TEXT_RADIUS                = 75.0f;
	constexpr static int32 AXIS_CIRCLE_SIDES                   = 24;
	constexpr static float AXIS_LENGTH_SCALE_OFFSET            = 5.0f;

	FWidget();

	/**
	 * Sets editor mode tools to use in this widget
	 */
	UNREALED_API void SetUsesEditorModeTools(FEditorModeTools* InEditorModeTools);

	/**
	 * Renders any widget specific HUD text
	 * @param Canvas - Canvas to use for 2d rendering
	 */
	void DrawHUD(FCanvas* Canvas);

	// @param ViewportClient must not be 0
	void Render(const FSceneView* View, FPrimitiveDrawInterface* PDI, FEditorViewportClient* ViewportClient);

public:
	/**
	 * Converts mouse movement on the screen to widget axis movement/rotation.
	 */
	void ConvertMouseMovementToAxisMovement(FSceneView* InView, FEditorViewportClient* InViewportClient,
	                                        bool bInUsedDragModifier, FVector& InDiff, FVector& OutDrag,
	                                        FRotator& OutRotation, FVector& OutScale);

	/**
	 * Absolute Translation conversion from mouse movement on the screen to widget axis movement/rotation.
	 */
	void AbsoluteTranslationConvertMouseMovementToAxisMovement(FSceneView* InView,
	                                                           FEditorViewportClient* InViewportClient,
	                                                           const FVector& InLocation,
	                                                           const FVector2D& InMousePosition, FVector& OutDrag,
	                                                           FRotator& OutRotation, FVector& OutScale);

	/** 
	 * Grab the initial offset again first time input is captured
	 */
	void ResetInitialTranslationOffset(void)
	{
		bAbsoluteTranslationInitialOffsetCached = false;
	}

	/** Only some modes support Absolute Translation Movement.  Check current mode */
	static bool AllowsAbsoluteTranslationMovement(UE::Widget::EWidgetMode WidgetMode);

	/** Only some modes support Absolute Rotation Movement.  Check current mode */
	static bool AllowsAbsoluteRotationMovement(UE::Widget::EWidgetMode WidgetMode, EAxisList::Type InAxisType);
	/**
	 * Sets the default visibility of the widget, if it is not overridden by an active editor mode tool.
	 *
	 * @param	bInVisibility	true for visible
	 */
	void SetDefaultVisibility(bool bInDefaultVisibility)
	{
		bDefaultVisibility = bInDefaultVisibility;
	}

	/**
	 * Sets the axis currently being moused over.  Typically called by FMouseDeltaTracker or FLevelEditorViewportClient.
	 *
	 * @param	InCurrentAxis	The new axis value.
	 */
	void SetCurrentAxis(EAxisList::Type InCurrentAxis)
	{
		CurrentAxis = InCurrentAxis;
	}

	/**
	 * @return	The axis currently being moused over.
	 */
	EAxisList::Type GetCurrentAxis() const
	{
		return CurrentAxis;
	}

	/** 
	 * @return	The widget origin in viewport space.
	 */
	FVector2D GetOrigin() const
	{
		return Origin;
	}

	/**
	 * @return	The mouse drag start position in viewport space.
	 */
	void SetDragStartPosition(const FVector2D& Position)
	{
		DragStartPos = Position;
		LastDragPos  = DragStartPos;
	}

	/**
	 * Returns whether we are actively dragging
	 */
	bool IsDragging(void) const
	{
		return bDragging;
	}

	/**
	 * Sets if we are currently engaging the widget in dragging
	 */
	void SetDragging(const bool InDragging)
	{
		bDragging = InDragging;
	}

	/**
	 * Sets if we are currently engaging the widget in dragging
	 */
	void SetSnapEnabled(const bool InSnapEnabled)
	{
		bSnapEnabled = InSnapEnabled;
	}

	/** 
	 * FGCObject interface: Serializes the widget reference so they dont get garbage collected.
	 *
	 * @param Ar	FArchive to serialize with
	 */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return "FWidget";
	}

	/**
	 * Gets the axis to draw based on the current widget mode
	 */
	EAxisList::Type GetAxisToDraw(UE::Widget::EWidgetMode WidgetMode) const;

	/** @return true if the widget is disabled */
	bool IsWidgetDisabled() const;

	/** Updates the delta rotation on the widget */
	UNREALED_API void UpdateDeltaRotation();

	/** Resets the total delta rotation back to zero */
	void ResetDeltaRotation()
	{
		TotalDeltaRotation = 0;
	}

	/** @return the rotation speed of the widget */
	static float GetRotationSpeed()
	{
		return (2.f * (float)PI) / 360.f;
	}

private:
	void ConvertMouseToAxis_Translate(FVector2D DragDir, FVector& InOutDelta, FVector& OutDrag) const;
	void ConvertMouseToAxis_Rotate(FVector2D TangentDir, FVector2D DragDir, FSceneView* InView,
	                               FEditorViewportClient* InViewportClient, FVector& InOutDelta, FRotator& OutRotation);
	void ConvertMouseToAxis_Scale(FVector2D DragDir, FVector& InOutDelta, FVector& OutScale);
	void ConvertMouseToAxis_TranslateRotateZ(FVector2D TangentDir, FVector2D DragDir, FVector& InOutDelta,
	                                         FVector& OutDrag, FRotator& OutRotation);
	void ConvertMouseToAxis_WM_2D(FVector2D TangentDir, FVector2D DragDir, FVector& InOutDelta, FVector& OutDrag,
	                              FRotator& OutRotation);

	struct FAbsoluteMovementParams
	{
		/** The normal of the plane to project onto */
		FVector PlaneNormal;
		/** A vector that represent any displacement we want to mute (remove an axis if we're doing axis movement)*/
		FVector NormalToRemove;
		/** The current position of the widget */
		FVector Position;

		//Coordinate System Axes
		FVector XAxis;
		FVector YAxis;
		FVector ZAxis;

		//true if camera movement is locked to the object
		bool bMovementLockedToCamera;

		//Direction in world space to the current mouse location
		FVector PixelDir;
		//Direction in world space of the middle of the camera
		FVector CameraDir;
		FVector EyePos;

		//whether to snap the requested positionto the grid
		bool bPositionSnapping;
	};

	void AbsoluteConvertMouseToAxis_Translate(FSceneView* InView, const FMatrix& InputCoordSystem,
	                                          FAbsoluteMovementParams& InOutParams, FVector& OutDrag);
	void AbsoluteConvertMouseToAxis_WM_2D(const FMatrix& InputCoordSystem, FAbsoluteMovementParams& InOutParams,
	                                      FVector& OutDrag, FRotator& OutRotation);
	void AbsoluteConvertMouseToAxis_TranslateRotateZ(const FMatrix& InputCoordSystem,
	                                                 FAbsoluteMovementParams& InOutParams, FVector& OutDrag,
	                                                 FRotator& OutRotation);

	/**
	 * Render helper functions
	 */
	void RenderGrid(const FSceneView* View, FPrimitiveDrawInterface* PDI, FEditorViewportClient* ViewportClient);
	void Render_Axis(const FSceneView* View, FPrimitiveDrawInterface* PDI, EAxisList::Type InAxis, FMatrix& InMatrix,
	                 UMaterialInterface* InMaterial, const FLinearColor& InColor, FVector2D& OutAxisDir,
	                 const FVector& InScale, bool bDrawWidget, bool bCubeHead = false, float AxisLengthOffset = 0);
	void Render_Cube(FPrimitiveDrawInterface* PDI, const FMatrix& InMatrix, const UMaterialInterface* InMaterial,
	                 const FVector& InScale);
	UNREALED_API void Render_Translate(const FSceneView* View, FPrimitiveDrawInterface* PDI,
	                                   FEditorViewportClient* ViewportClient, const FVector& InLocation,
	                                   bool bDrawWidget);
	UNREALED_API void Render_Rotate(const FSceneView* View, FPrimitiveDrawInterface* PDI,
	                                FEditorViewportClient* ViewportClient, const FVector& InLocation, bool bDrawWidget);
	void Render_Scale(const FSceneView* View, FPrimitiveDrawInterface* PDI, FEditorViewportClient* ViewportClient,
	                  const FVector& InLocation, bool bDrawWidget);
	void Render_TranslateRotateZ(const FSceneView* View, FPrimitiveDrawInterface* PDI,
	                             FEditorViewportClient* ViewportClient, const FVector& InLocation, bool bDrawWidget);
	void Render_2D(const FSceneView* View, FPrimitiveDrawInterface* PDI, FEditorViewportClient* ViewportClient,
	               const FVector& InLocation, bool bDrawWidget);


	struct FThickArcParams
	{
		FThickArcParams(FPrimitiveDrawInterface* InPDI, const FVector& InPosition, UMaterialInterface* InMaterial,
		                const float InInnerRadius, const float InOuterRadius)
		    : Position(InPosition)
		    , PDI(InPDI)
		    , Material(InMaterial)
		    , InnerRadius(InInnerRadius)
		    , OuterRadius(InOuterRadius)
		{}

		/** The current position of the widget */
		FVector Position;

		//interface for Drawing
		FPrimitiveDrawInterface* PDI;

		//Material to use to render
		UMaterialInterface* Material;

		//Radii
		float InnerRadius;
		float OuterRadius;
	};

	/**
	 * Returns the Delta from the current position that the absolute movement system wants the object to be at
	 * @param InParams - Structure containing all the information needed for absolute movement
	 * @return - The requested delta from the current position
	 */
	FVector GetAbsoluteTranslationDelta(const FAbsoluteMovementParams& InParams);
	/**
	 * Returns the offset from the initial selection point
	 */
	FVector GetAbsoluteTranslationInitialOffset(const FVector& InNewPosition, const FVector& InCurrentPosition);

	/**
	 * Returns true if we're in Local Space editing mode or editing BSP (which uses the World axes anyway
	 */
	bool IsRotationLocalSpace() const;

	/**
	 * Returns how far we have just rotated
	 */
	float GetDeltaRotation() const;

	/**
	 * If actively dragging, draws a ring representing the potential rotation of the selected objects, snap ticks, and "delta" markers
	 * If not actively dragging, draws a quarter ring representing the closest quadrant to the camera
	 * @param View - Information about the scene/camera/etc
	 * @param PDI - Drawing interface
	 * @param InAxis - Enumeration of axis to rotate about
	 * @param InLocation - The Origin of the widget
	 * @param Axis0 - The Axis that describes a 0 degree rotation
	 * @param Axis1 - The Axis that describes a 90 degree rotation
	 * @param InDirectionToWidget - Direction from camera to the widget
	 * @param InColor - The color associated with the axis of rotation
	 * @param InScale - Multiplier to maintain a constant screen size for rendering the widget
	 * @param OutAxisDir - Viewport-space direction of rotation arc chord is placed here
	 */
	void DrawRotationArc(const FSceneView* View, FPrimitiveDrawInterface* PDI, EAxisList::Type InAxis,
	                     const FVector& InLocation, const FVector& Axis0, const FVector& Axis1,
	                     const FVector& InDirectionToWidget, const FColor& InColor, const float InScale,
	                     FVector2D& OutAxisEnd);

	/**
	 * If actively dragging, draws a ring representing the potential rotation of the selected objects, snap ticks, and "delta" markers
	 * If not actively dragging, draws a quarter ring representing the closest quadrant to the camera
	 * @param View - Information about the scene/camera/etc
	 * @param PDI - Drawing interface
	 * @param InAxis - Enumeration of axis to rotate about
	 * @param InLocation - The Origin of the widget
	 * @param Axis0 - The Axis that describes a 0 degree rotation
	 * @param Axis1 - The Axis that describes a 90 degree rotation
	 * @param InStartAngle - The starting angle about (Axis0^Axis1) to render the arc
	 * @param InEndAngle - The ending angle about (Axis0^Axis1) to render the arc
	 * @param InColor - The color associated with the axis of rotation
	 * @param InScale - Multiplier to maintain a constant screen size for rendering the widget
	 */
	void DrawPartialRotationArc(const FSceneView* View, FPrimitiveDrawInterface* PDI, EAxisList::Type InAxis,
	                            const FVector& InLocation, const FVector& Axis0, const FVector& Axis1,
	                            const float InStartAngle, const float InEndAngle, const FColor& InColor,
	                            const float InScale, const FVector& InDirectionToWidget);

	/**
	 * Renders a portion of an arc for the rotation widget
	 * @param InParams - Material, Radii, etc
	 * @param InStartAxis - Start of the arc
	 * @param InEndAxis - End of the arc
	 * @param InColor - Color to use for the arc
	 */
	void DrawThickArc(const FThickArcParams& InParams, const FVector& Axis0, const FVector& Axis1,
	                  const float InStartAngle, const float InEndAngle, const FColor& InColor,
	                  const FVector& InDirectionToWidget, bool bIsOrtho);

	/**
	 * Draws protractor like ticks where the rotation widget would snap too.
	 * Also, used to draw the wider axis tick marks
	 * @param PDI - Drawing interface
	 * @param InLocation - The Origin of the widget
	 * @param Axis0 - The Axis that describes a 0 degree rotation
	 * @param Axis1 - The Axis that describes a 90 degree rotation
	 * @param InAngle - The Angle to rotate about the axis of rotation, the vector (Axis0 ^ Axis1)
	 * @param InColor - The color to use for line/poly drawing
	 * @param InScale - Multiplier to maintain a constant screen size for rendering the widget
	 * @param InWidthPercent - The percent of the distance between the outer ring and inner ring to use for tangential thickness
	 * @param InPercentSize - The percent of the distance between the outer ring and inner ring to use for radial distance
	 */
	void DrawSnapMarker(FPrimitiveDrawInterface* PDI, const FVector& InLocation, const FVector& Axis0,
	                    const FVector& Axis1, const FColor& InColor, const float InScale,
	                    const float InWidthPercent = 0.0f, const float InPercentSize = 1.0f);

	/**
	 * Draw Start/Stop Marker to show delta rotations along the arc of rotation
	 * @param PDI - Drawing interface
	 * @param InLocation - The Origin of the widget
	 * @param Axis0 - The Axis that describes a 0 degree rotation
	 * @param Axis1 - The Axis that describes a 90 degree rotation
	 * @param InAngle - The Angle to rotate about the axis of rotation, the vector (Axis0 ^ Axis1)
	 * @param InColor - The color to use for line/poly drawing
	 * @param InScale - Multiplier to maintain a constant screen size for rendering the widget
	 */
	void DrawStartStopMarker(FPrimitiveDrawInterface* PDI, const FVector& InLocation, const FVector& Axis0,
	                         const FVector& Axis1, const float InAngle, const FColor& InColor, const float InScale);

	/**
	 * Caches off HUD text to display after 3d rendering is complete
	 * @param View - Information about the scene/camera/etc
	 * @param PDI - Drawing interface
	 * @param InLocation - The Origin of the widget
	 * @param Axis0 - The Axis that describes a 0 degree rotation
	 * @param Axis1 - The Axis that describes a 90 degree rotation
	 * @param AngleOfAngle - angle we've rotated so far (in degrees)
	 * @param InScale - Multiplier to maintain a constant screen size for rendering the widget
	 */
	void CacheRotationHUDText(const FSceneView* View, FPrimitiveDrawInterface* PDI, const FVector& InLocation,
	                          const FVector& Axis0, const FVector& Axis1, const float AngleOfChange,
	                          const float InScale);

	/**
	 * Gets the axis to use when converting mouse movement, accounting for Ortho views.
	 *
	 * @param InDiff Difference vector to determine dominant axis.
	 * @param ViewportType Type of viewport for ortho checks.
	 *
	 * @return Index of the dominant axis.
	 */
	uint32 GetDominantAxisIndex(const FVector& InDiff, FEditorViewportClient* ViewportClient) const;


	void DrawColoredSphere(FPrimitiveDrawInterface* PDI, const FVector& Center, const FRotator& Orientation,
	                       FColor Color, const FVector& Radii, int32 NumSides, int32 NumRings,
	                       const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority,
	                       bool bDisableBackfaceCulling);

	/** The axis currently being moused over */
	EAxisList::Type CurrentAxis;

	/** Viewport space origin location of the widget */
	FVector2D Origin;
	/** Viewport space direction vectors of the axes on the widget */
	FVector2D XAxisDir, YAxisDir, ZAxisDir;
	/** Drag start position in viewport space */
	FVector2D DragStartPos;
	/** Last mouse position in viewport space */
	FVector2D LastDragPos;
	enum
	{
		AXIS_ARROW_SEGMENTS = 16
	};

	/** Materials and colors to be used when drawing the items for each axis */
	TObjectPtr<UMaterialInterface> TransparentPlaneMaterialXY;
	TObjectPtr<UMaterialInterface> GridMaterial;

	TObjectPtr<UMaterialInstanceDynamic> AxisMaterialX;
	TObjectPtr<UMaterialInstanceDynamic> AxisMaterialY;
	TObjectPtr<UMaterialInstanceDynamic> AxisMaterialZ;
	TObjectPtr<UMaterialInstanceDynamic> CurrentAxisMaterial;
	TObjectPtr<UMaterialInstanceDynamic> OpaquePlaneMaterialXY;

	FLinearColor AxisColorX, AxisColorY, AxisColorZ;
	FLinearColor ScreenAxisColor;
	FColor PlaneColorXY, ScreenSpaceColor, CurrentColor;
	FColor ArcBallColor;

	/** Any mode tools being used */
	FEditorModeTools* EditorModeTools;

	/**
	 * An extra matrix to apply to the widget before drawing it (allows for local/custom coordinate systems).
	 */
	FMatrix CustomCoordSystem;

	/** The space of the custom coord system */
	ECoordSystem CustomCoordSystemSpace;

	//location in the viewport to render the hud string
	FVector2D HUDInfoPos;
	//string to be displayed on top of the viewport
	FString HUDString;

	/** Whether Absolute Translation cache position has been captured */
	bool bAbsoluteTranslationInitialOffsetCached;
	/** The initial offset where the widget was first clicked */
	FVector InitialTranslationOffset;
	/** The initial position of the widget before it was clicked */
	FVector InitialTranslationPosition;
	/** Whether or not the widget is actively dragging */
	bool bDragging;
	/** Whether or not snapping is enabled for all actors */
	bool bSnapEnabled;
	/** Default visibility for the widget if an Editor Mode Tool doesn't override it */
	bool bDefaultVisibility;
	/** Whether we are drawing the full ring in rotation mode (ortho viewports only) */
	bool bIsOrthoDrawingFullRing;

	/** Total delta rotation applied since the widget was dragged */
	float TotalDeltaRotation;

	/** Current delta rotation applied to the rotation widget */
	float CurrentDeltaRotation;
};

/**
 * Widget hit proxy.
 */
struct HWidgetAxis : public HHitProxy
{
	DECLARE_HIT_PROXY(UNREALED_API);

	EAxisList::Type Axis;
	uint32 bDisabled : 1;

	UNREALED_API HWidgetAxis(EAxisList::Type InAxis, bool InbDisabled = false, EHitProxyPriority InHitProxy = HPP_UI);

	virtual EMouseCursor::Type GetMouseCursor() override;

	/**
	 * Method that specifies whether the hit proxy *always* allows translucent primitives to be associated with it or not,
	 * regardless of any other engine/editor setting. For example, if translucent selection was disabled, any hit proxies
	 *  returning true would still allow translucent selection. In this specific case, true is always returned because geometry
	 * mode hit proxies always need to be selectable or geometry mode will not function correctly.
	 *
	 * @return	true if translucent primitives are always allowed with this hit proxy; false otherwise
	 */
	virtual bool AlwaysAllowsTranslucentPrimitives() const override;
};
