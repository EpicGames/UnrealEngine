// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HitProxies.h"
#include "Factories/Factory.h"

class FPrimitiveDrawInterface;
class FSceneView;
struct FViewportClick;

DECLARE_LOG_CATEGORY_EXTERN(LogUtils, Log, All);

enum EWidgetMovementMode
{
	WMM_Rotate,
	WMM_Translate,
	WMM_Scale,
	WMM_MAX
};

struct HWidgetUtilProxy : public HHitProxy
{
	DECLARE_HIT_PROXY( UNREALED_API );

	int32						Info1;
	int32						Info2;
	EAxisList::Type				Axis;
	FMatrix						WidgetMatrix;
	EWidgetMovementMode			Mode;

	HWidgetUtilProxy(int32 InInfo1, int32 InInfo2, EAxisList::Type InAxis, const FMatrix& InWidgetMatrix, EWidgetMovementMode bInMode):
	HHitProxy(HPP_UI),
		Info1(InInfo1),
		Info2(InInfo2),
		Axis(InAxis),
		WidgetMatrix(InWidgetMatrix),
		Mode(bInMode)
	{}

	virtual void CalcVectors(FSceneView* SceneView, const FViewportClick& Click, FVector& LocalManDir, FVector& WorldManDir, float& DragDirX, float& DragDirY);
};

class FUnrealEdUtils
{
public:
	UNREALED_API static void DrawWidget(const FSceneView* View,FPrimitiveDrawInterface* PDI, const FMatrix& WidgetMatrix, int32 InInfo1, int32 InInfo2, EAxisList::Type HighlightAxis, EWidgetMovementMode bInMode);
	UNREALED_API static void DrawWidget(const FSceneView* View,FPrimitiveDrawInterface* PDI, const FMatrix& WidgetMatrix, int32 InInfo1, int32 InInfo2, EAxisList::Type HighlightAxis, EWidgetMovementMode bInMode, bool bHitTesting);
};

/**
 * Import an object using a UFactory.
 */
template< class T > T* ImportObject( UObject* Outer, FName Name, EObjectFlags Flags, const TCHAR* Filename=TEXT(""), UObject* Context=NULL, UFactory* Factory=NULL, const TCHAR* Parms=NULL, FFeedbackContext* Warn=GWarn )
{
	return (T*)UFactory::StaticImportObject( T::StaticClass(), Outer, Name, Flags, Filename, Context, Factory, Parms, Warn );
}

