// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CameraController.h"

/**
 * Helper class used by FEditorViewportClient to allow external inputs to move the camera.
 * Used by ITF Viewport interactions.
 * When deltas from this struct are used, they should be reset since they are used and updated on tick.
 * Please use ConsumeTransformDelta(...) and ConsumeImpulseData(...) to do that.
 */
struct FViewportClientNavigationHelper
{
public:
	FViewportClientNavigationHelper();

	/**
	 * A location delta which can be used to move a Viewport camera
	 */
	FVector LocationDelta;

	/**
	 * A rotation delta which can be used to rotate a Viewport camera
	 */
	FRotator RotationDelta;

	/**
	 * A delta which is used by Editor Viewport Client to compute orbit movement values.
	 * In perspective view, Z indicates a Dolly action.
	 * In orthographic view, Z indicates an orthographic zoom
	 */
	FVector OrbitDelta;

	/**
	 * Impulse data used to control rotation, translation and FOV of a camera
	 * See FEditorViewportClient::UpdateCameraMovement
	 */
	FCameraControllerUserImpulseData ImpulseDataDelta;

	/**
	 * Check whether non-zero location, rotation, or orbit deltas are available.
	 * Useful to skip certain logic in FEditorViewportClient::UpdateMouseDelta() or for other purposes
	 */
	bool HasTransformDelta() const;

	/**
	 * Retrieve current location and rotation deltas, and zero them out
	 * @param OutLocationDelta location delta will be copied to OutLocationDelta
	 * @param OutRotationDelta rotation delta will be copied to OutRotationDelta
	 */
	void ConsumeTransformDelta(FVector& OutLocationDelta, FRotator& OutRotationDelta, FVector& OutOrbitDelta);

	/**
	 * Retrieve current impulse deltas and zero them out
	 * @param OutImpulseData current impulse data will be copied to OutImpulseData before being reset
	 */
	void ConsumeImpulseData(FCameraControllerUserImpulseData& OutImpulseData);

	/**
	 * Retrieve current delta used to generate orbit movement
	 * @param OutOrbitDelta
	 */
	void ConsumeOrbitDelta(FVector& OutOrbitDelta);

	/**
	 * Sets Location and Rotation deltas to zero
	 */
	void ResetTransformDelta();

	/**
	 * Sets all impulse data to zero
	 */
	void ResetImpulseData();

	/**
	 * This flag is true when the user is controlling a viewport camera using the mouse
	 */
	bool bIsMouseLooking;

	UE_DEPRECATED(5.8, "FEditorViewportClient::TranslateViewportCamera can be used to translate the camera when needed.")
	bool bCameraFollowsGizmo;
};
