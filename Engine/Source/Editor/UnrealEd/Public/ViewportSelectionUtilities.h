// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HitProxies.h"
#include "UObject/ObjectPtr.h"

class AActor;
class ABrush;
class FEditorViewportClient;
class FLevelEditorViewportClient;
class HHitProxy;
class UModel;
class USceneComponent;
struct FTypedElementHandle;
struct FViewportClick;
struct HActor;
struct HGeomEdgeProxy;
struct HGeomPolyProxy;
struct HGeomVertexProxy;

/**
 * A hit proxy class for sockets in the main editor viewports.
 */
struct HLevelSocketProxy : public HHitProxy
{
	DECLARE_HIT_PROXY(UNREALED_API);

	TObjectPtr<AActor> Actor;
	TObjectPtr<USceneComponent> SceneComponent;
	FName SocketName;

	HLevelSocketProxy(AActor* InActor, USceneComponent* InSceneComponent, FName InSocketName)
		:	HHitProxy(HPP_UI)
		,   Actor( InActor )
		,   SceneComponent( InSceneComponent )
		,	SocketName( InSocketName )
	{}
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override
	{
		Collector.AddReferencedObject( Actor );
		Collector.AddReferencedObject( SceneComponent );
	}
};


namespace UE::Editor::ViewportSelectionUtilities
{
bool UNREALED_API ClickElement(FEditorViewportClient* InViewportClient, const FTypedElementHandle& InHitElement, const FViewportClick& InClick);

// Possibly not needed anymore
bool UNREALED_API ClickActor(FEditorViewportClient* InViewportClient, AActor* InActor, const FViewportClick& InClick, bool bInAllowSelectionChange);

// Possibly not needed anymore
bool UNREALED_API ClickComponent(FEditorViewportClient* InViewportClient, HActor* InActorHitProxy, const FViewportClick& InClick);

void UNREALED_API ClickBrushVertex(FEditorViewportClient* InViewportClient, ABrush* InBrush, FVector* InVertex, const FViewportClick& InClick);

void UNREALED_API ClickStaticMeshVertex(	FEditorViewportClient* InViewportClient, AActor* InActor, FVector& InVertex, const FViewportClick& InClick);

void UNREALED_API ClickSurface(FEditorViewportClient* InViewportClient, UModel* InModel, int32 InSurf, const FViewportClick& InClick);

void UNREALED_API ClickBackdrop(FEditorViewportClient* InViewportClient, const FViewportClick& InClick);

void UNREALED_API ClickLevelSocket(FEditorViewportClient* InViewportClient, HHitProxy* InHitProxy, const FViewportClick& InClick);
} // namespace UE::Editor::ViewportUtilities


// This namespace used to live in now deprecated "LevelViewportClickHandlers.h"
namespace LevelViewportClickHandlers
{
	UE_DEPRECATED(5.7, "Level Editor Viewport Menu is now summoned directly by Level Editor.")
	bool ClickViewport(FLevelEditorViewportClient* ViewportClient, const FViewportClick& Click);

	UE_DEPRECATED(5.7, "Use UE::Editor::ViewportUtilities::ClickElement taking FEditorViewportClient* as argument.")
	bool ClickElement(FLevelEditorViewportClient* ViewportClient, const FTypedElementHandle& HitElement, const FViewportClick& Click);

	UE_DEPRECATED(5.7, "Use UE::Editor::ViewportUtilities::ClickActor taking FEditorViewportClient* as argument.")
	bool UNREALED_API ClickActor(FLevelEditorViewportClient* ViewportClient,AActor* Actor,const FViewportClick& Click,bool bAllowSelectionChange);

	UE_DEPRECATED(5.7, "Use UE::Editor::ViewportUtilities::ClickComponent taking FEditorViewportClient* as argument.")
	bool ClickComponent(FLevelEditorViewportClient* ViewportClient, HActor* ActorHitProxy, const FViewportClick& Click);

	UE_DEPRECATED(5.7, "Use UE::Editor::ViewportUtilities::ClickBrushVertex taking FEditorViewportClient* as argument.")
	void ClickBrushVertex(FLevelEditorViewportClient* ViewportClient,ABrush* InBrush,FVector* InVertex,const FViewportClick& Click);

	UE_DEPRECATED(5.7, "Use UE::Editor::ViewportUtilities::ClickStaticMeshVertex taking FEditorViewportClient* as argument.")
	void ClickStaticMeshVertex(FLevelEditorViewportClient* ViewportClient,AActor* InActor,FVector& InVertex,const FViewportClick& Click);

	UE_DEPRECATED(5.7, "Use UE::Editor::ViewportUtilities::ClickSurface taking FEditorViewportClient* as argument.")
	void ClickSurface(FLevelEditorViewportClient* ViewportClient, UModel* Model, int32 iSurf, const FViewportClick& Click);

	UE_DEPRECATED(5.7, "Use UE::Editor::ViewportUtilities::ClickBackdrop taking FEditorViewportClient* as argument.")
	void ClickBackdrop(FLevelEditorViewportClient* ViewportClient,const FViewportClick& Click);

	UE_DEPRECATED(5.7, "Use UE::Editor::ViewportUtilities::ClickLevelSocket taking FEditorViewportClient* as argument.")
	void ClickLevelSocket(FLevelEditorViewportClient* ViewportClient, HHitProxy* HitProxy, const FViewportClick& Click);
}
