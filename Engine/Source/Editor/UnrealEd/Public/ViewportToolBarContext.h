// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ViewportToolBarContext.generated.h"

class SViewportToolBar;
class SLevelViewport;

UCLASS(MinimalAPI)
class UViewportToolBarContext : public UObject
{
	GENERATED_BODY()
public:
	TWeakPtr<SViewportToolBar> ViewportToolBar;
	TWeakPtr<SLevelViewport> Viewport;
};
