// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/DelegateCombinations.h"
#include "Engine/EngineBaseTypes.h"
#include "Templates/SharedPointerFwd.h"
#include "ToolMenuDelegates.h"
#include "ToolMenuEntry.h"
#include "UnrealEdViewportToolbarContext.h"
#include "UnrealWidgetFwd.h"

#define UE_API UNREALED_API

class FEditorViewportClient;
class FTabManager;
class FText;
class IPreviewProfileController;
class IPreviewLODController;
class SEditorViewport;
class UToolMenu;
enum ELevelViewportType : int;
enum ERotationGridMode : int;
struct FNewToolMenuChoice;
struct FTabId;
struct FToolMenuEntry;
class UGameViewportClient;

namespace UE::UnrealEd
{

/** Lists View Mode Menu Sections which can be shown/hidden based on specific menu requirements */
enum EHidableViewModeMenuSections : uint8
{
	Exposure = 0,
	GPUSkinCache = 1
};

/** The value of this function is controlled by the CVAR "ToolMenusViewportToolbars". */
UE_DEPRECATED(5.7, "The legacy viewport toolbars are no longer supported and are no longer ever visible.")
UNREALED_API bool ShowOldViewportToolbars();

/** The value of this function is controlled by the CVAR "ToolMenusViewportToolbars". */
UE_DEPRECATED(5.7, "The legacy viewport toolbars are no longer supported. The new toolbars are always visible.")
UNREALED_API bool ShowNewViewportToolbars();

UNREALED_API FSlateIcon GetIconFromCoordSystem(ECoordSystem InCoordSystem);

UE_DEPRECATED(5.6, "Use CreateTransformsSubmenu() instead.")
UNREALED_API FToolMenuEntry CreateViewportToolbarTransformsSection();
UNREALED_API FToolMenuEntry CreateTransformsSubmenu();

UE_DEPRECATED(5.6, "Viewport toolbar no longer has a Select Menu.")
UNREALED_API FToolMenuEntry CreateViewportToolbarSelectSection();

UE_DEPRECATED(5.6, "Use CreateSnappingSubmenu() instead.")
UNREALED_API FToolMenuEntry CreateViewportToolbarSnappingSubmenu();
UNREALED_API FToolMenuEntry CreateSnappingSubmenu();

UNREALED_API FText GetViewModesSubmenuLabel(TWeakPtr<SEditorViewport> InViewport);
UNREALED_API FText GetViewModesSubmenuLabel(const UGameViewportClient* InViewportClient);

UNREALED_API FSlateIcon GetViewModesSubmenuIcon(const TWeakPtr<SEditorViewport>& InViewport);
UNREALED_API FSlateIcon GetViewModesSubmenuIcon(const UGameViewportClient* InViewportClient);

/**
 * Populate a given UToolMenu with entries for a View Modes viewport toolbar submenu.
 *
 * @param InMenu The menu to populate with entries.
 */
UNREALED_API void PopulateViewModesMenu(UToolMenu* InMenu);

/** Create a Viewport Toolbar Context with common values (many Asset Editors have the same settings) */
UNREALED_API UUnrealEdViewportToolbarContext* CreateViewportToolbarDefaultContext(const TWeakPtr<SEditorViewport>& InViewport
);

UE_DEPRECATED(5.6, "Use CreateViewModesSubmenu() instead.")
UNREALED_API FToolMenuEntry CreateViewportToolbarViewModesSubmenu();
UNREALED_API FToolMenuEntry CreateViewModesSubmenu();

DECLARE_DELEGATE_TwoParams(FRotationGridCheckboxListExecuteActionDelegate, int, ERotationGridMode);
DECLARE_DELEGATE_RetVal_TwoParams(bool, FRotationGridCheckboxListIsCheckedDelegate, int, ERotationGridMode);

DECLARE_DELEGATE_OneParam(FLocationGridCheckboxListExecuteActionDelegate, int);
DECLARE_DELEGATE_RetVal_OneParam(bool, FLocationGridCheckboxListIsCheckedDelegate, int);

DECLARE_DELEGATE_RetVal(TArray<float>, FLocationGridValuesArrayDelegate);

DECLARE_DELEGATE_OneParam(FScaleGridCheckboxListExecuteActionDelegate, int);
DECLARE_DELEGATE_RetVal_OneParam(bool, FScaleGridCheckboxListIsCheckedDelegate, int);

DECLARE_DELEGATE_OneParam(FNumericEntryExecuteActionDelegate, float);
DECLARE_DELEGATE_OneParam(FNumericEntryExecuteActionDelegateInt32, int32);

DECLARE_DELEGATE_TwoParams(FOnViewportClientCamSpeedChanged, const TSharedRef<SEditorViewport>&, int32);
DECLARE_DELEGATE_TwoParams(FOnViewportClientCamSpeedScalarChanged, const TSharedRef<SEditorViewport>&, float);

// This will be deprecated in the future. Prefer using CreateSnappingSubmenu() to get snapping tools.
UNREALED_API TSharedRef<SWidget> BuildRotationGridCheckBoxList(
	FName InExtentionHook,
	const FText& InHeading,
	const TArray<float>& InGridSizes,
	ERotationGridMode InGridMode,
	const FRotationGridCheckboxListExecuteActionDelegate& InExecuteAction,
	const FRotationGridCheckboxListIsCheckedDelegate& InIsActionChecked,
	const TSharedPtr<FUICommandList>& InCommandList = {}
);

UNREALED_API FText GetRotationGridLabel();
// This will be deprecated in the future. Prefer using CreateSnappingSubmenu() to get snapping tools.
UNREALED_API TSharedRef<SWidget> CreateRotationGridSnapMenu(
	const FRotationGridCheckboxListExecuteActionDelegate& InExecuteDelegate,
	const FRotationGridCheckboxListIsCheckedDelegate& InIsCheckedDelegate,
	const TAttribute<bool>& InIsEnabledDelegate = TAttribute<bool>(true),
	const TSharedPtr<FUICommandList>& InCommandList = {}
);

UNREALED_API FText GetLocationGridLabel();

UE_DEPRECATED(5.6, "Use CreateLocationGridSnapMenu() using FLocationGridSnapMenuOptions as argument instead.")
UNREALED_API TSharedRef<SWidget> CreateLocationGridSnapMenu(
	const FLocationGridCheckboxListExecuteActionDelegate& InExecuteDelegate,
	const FLocationGridCheckboxListIsCheckedDelegate& InIsCheckedDelegate,
	const TArray<float>& InGridSizes,
	const TAttribute<bool>& InIsEnabledDelegate = TAttribute<bool>(true),
	const TSharedPtr<FUICommandList>& InCommandList = {}
);

struct FLocationGridSnapMenuOptions
{
	FName MenuName;
	FLocationGridCheckboxListExecuteActionDelegate ExecuteDelegate;
	FLocationGridCheckboxListIsCheckedDelegate IsCheckedDelegate;
	FLocationGridValuesArrayDelegate GridValuesArrayDelegate;
	TAttribute<bool> IsEnabledDelegate = TAttribute<bool>(true);
	TSharedPtr<FUICommandList> CommandList;
};

// This will be deprecated in the future. Prefer using CreateSnappingSubmenu() to get snapping tools.
UNREALED_API TSharedRef<SWidget> CreateLocationGridSnapMenu(const FLocationGridSnapMenuOptions& InMenuOptions = {});

UNREALED_API FText GetScaleGridLabel();
// This will be deprecated in the future. Prefer using CreateSnappingSubmenu() to get snapping tools.
UNREALED_API TSharedRef<SWidget> CreateScaleGridSnapMenu(
	const FScaleGridCheckboxListExecuteActionDelegate& InExecuteDelegate,
	const FScaleGridCheckboxListIsCheckedDelegate& InIsCheckedDelegate,
	const TArray<float>& InGridSizes,
	const TAttribute<bool>& InIsEnabledDelegate = TAttribute<bool>(true),
	const TSharedPtr<FUICommandList>& InCommandList = {},
	const TAttribute<bool>& ShowPreserveNonUniformScaleOption = TAttribute<bool>(false),
	const FUIAction& PreserveNonUniformScaleUIAction = FUIAction()
);

UE_DEPRECATED(5.6, "Use the native FToolMenuEntry::InitSubmenu() functions.")
UNREALED_API FToolMenuEntry CreateCheckboxSubmenu(
	const FName InName,
	const TAttribute<FText>& InLabel,
	const TAttribute<FText>& InToolTip,
	const FToolMenuExecuteAction& InCheckboxExecuteAction,
	const FToolMenuCanExecuteAction& InCheckboxCanExecuteAction,
	const FToolMenuGetActionCheckState& InCheckboxActionCheckState,
	const FNewToolMenuChoice& InMakeMenu
);

UNREALED_API FToolMenuEntry CreateNumericEntry(
	const FName InName,
	const FText& InLabel,
	const FText& InTooltip,
	const FCanExecuteAction& InCanExecuteAction,
	const FNumericEntryExecuteActionDelegate& InOnValueChanged,
	const TAttribute<float>& InGetValue,
	float InMinValue = 0.0f,
	float InMaxValue = 1.0f,
	int32 InMaxFractionalDigits = 2
);

UNREALED_API FToolMenuEntry CreateNumericEntry(
	const FName InName,
	const FText& InLabel,
	const FText& InTooltip,
	const FCanExecuteAction& InCanExecuteAction,
	const FNumericEntryExecuteActionDelegateInt32& InOnValueChanged,
	const TAttribute<int32>& InGetValue,
	int32 InMinValue = 0,
	int32 InMaxValue = 1
);

UNREALED_API TSharedRef<SWidget> CreateCameraSpeedWidget(const TWeakPtr<SEditorViewport>& WeakViewport);
UNREALED_API TSharedRef<SWidget> CreateCameraSpeedWidget(const TSharedPtr<FEditorViewportClient>& ViewportClient);
UNREALED_API FText GetCameraSpeedLabel(const TWeakPtr<SEditorViewport>& WeakViewport);
UNREALED_API FText GetCameraSpeedLabel(const TSharedPtr<FEditorViewportClient>& ViewportClient);

UNREALED_API FText GetCameraSubmenuLabelFromViewportType(const ELevelViewportType ViewportType);
UNREALED_API FName GetCameraSubmenuIconFNameFromViewportType(const ELevelViewportType ViewportType);
UE_DEPRECATED(5.6, "Use CreateCameraSubmenu() instead.")
UNREALED_API FToolMenuEntry CreateViewportToolbarCameraSubmenu();

struct FViewportCameraMenuOptions
{
	UE_API FViewportCameraMenuOptions();

	bool bShowCameraMovement: 1;
	bool bShowFieldOfView: 1;
	bool bShowNearAndFarPlanes: 1;
	
	UE_API FViewportCameraMenuOptions& ShowAll();
	UE_API FViewportCameraMenuOptions& ShowCameraMovement();
	UE_API FViewportCameraMenuOptions& ShowLensControls();
};

UNREALED_API FToolMenuEntry CreateCameraSubmenu(const FViewportCameraMenuOptions& InOptions = {});

UE_DEPRECATED(5.6, "Use CreateAssetViewerProfileSubmenu() instead.")
UNREALED_API FToolMenuEntry CreateViewportToolbarAssetViewerProfileSubmenu();
UNREALED_API FToolMenuEntry CreateAssetViewerProfileSubmenu();
UNREALED_API void ExtendPreviewSceneSettingsWithTabEntry(FName InAssetViewerProfileSubmenuName);

/**
 * Creates a submenu that displays and manipulates the current LOD of the provided LODController.
 * This should be invoked from within a dynamic menu or section to provide the correct controller.
 */
UNREALED_API FToolMenuEntry CreatePreviewLODSelectionSubmenu(TWeakPtr<IPreviewLODController> LODController);
/**
 * Fills the given UToolMenu with the appropriate menu items to reflect & manipulate the provided LODController
 */
UNREALED_API void FillPreviewLODSelectionSubmenu(UToolMenu* InToolMenu, TWeakPtr<IPreviewLODController> LODController);

UNREALED_API void AddExposureSection(UToolMenu* InMenu, const TSharedPtr<SEditorViewport>& EditorViewport);
UNREALED_API void PopulateCameraMenu(UToolMenu* InMenu, const FViewportCameraMenuOptions& InOptions = {});

/**
 * Adds Field of View and Far/Near View Plane entries to the specified Camera Submenu
 */
UE_DEPRECATED(5.6, "Use the options parameters of CreateCameraSubmenu() or PopulateCameraMenu() instead.")
UNREALED_API void ExtendCameraSubmenu(FName InCameraOptionsSubmenuName, bool bInShowViewPlaneEntries = true);

UNREALED_API void GenerateViewportTypeMenu(UToolMenu* InMenu);

UNREALED_API bool ShouldShowViewportRealtimeWarning(const FEditorViewportClient& ViewportClient);

UNREALED_API FToolMenuEntry CreatePerformanceAndScalabilitySubmenu();

UE_DEPRECATED_FORGAME(5.6, "Do not use. Will be made private.")
UNREALED_API bool IsScalabilityWarningVisible();

UE_DEPRECATED_FORGAME(5.6, "Do not use. Will be made private.")
UNREALED_API FText GetScalabilityWarningLabel();

UE_DEPRECATED_FORGAME(5.6, "Do not use. Will be made private.")
UNREALED_API FText GetScalabilityWarningTooltip();

/**
 * Creates a Show submenu with custom content
 */
UNREALED_API FToolMenuEntry CreateShowSubmenu(const FNewToolMenuChoice& InSubmenuChoice);

/**
 * Creates a Show submenu with commonly used show flags
 */
UNREALED_API FToolMenuEntry CreateDefaultShowSubmenu();

/**
 * Adds common flags sections to the specified menu
 */
UNREALED_API void AddDefaultShowFlags(UToolMenu* InMenu);

UNREALED_API FToolMenuEntry CreateToggleRealtimeEntry();
UNREALED_API FToolMenuEntry CreateRemoveRealtimeOverrideEntry(TWeakPtr<SEditorViewport> WeakViewport);

UNREALED_API FOnViewportClientCamSpeedChanged& OnViewportClientCamSpeedChanged();

UE_DEPRECATED(5.7, "Use OnViewportClientCamSpeedChanged() instead")
UNREALED_API FOnViewportClientCamSpeedScalarChanged& OnViewportClientCamSpeedScalarChanged();

UNREALED_API FToolMenuEntry CreateCameraSpeedMenu();

UNREALED_API void ConstructScreenPercentageMenu(UToolMenu* InMenu);
UNREALED_API FToolMenuEntry CreateScreenPercentageSubmenu();

UNREALED_API FToolMenuEntry CreateScalabilitySubmenu();
UNREALED_API FToolMenuEntry CreateResetScalabilitySubmenu();

UNREALED_API FText GetCameraSpeedTooltip();

struct FOrthographicClippingPlanesSubmenuOptions
{
	bool bDisplayAsToggle = true;
};

UNREALED_API FToolMenuEntry CreateOrthographicClippingPlanesSubmenu(const FOrthographicClippingPlanesSubmenuOptions& InOptions = {});

/**
 * Returns Visible if the provided Viewport Client is in perspective view, Collapsed if not
 */
UNREALED_API TAttribute<EVisibility> GetPerspectiveOnlyVisibility(const TSharedPtr<FEditorViewportClient>& InViewportClient);
/**
 * Creates an attribute that provides whether the viewport client is in perspective view.
 */
UNREALED_API TAttribute<bool> GetIsPerspectiveAttribute(const TSharedPtr<FEditorViewportClient>& InViewportClient);
/**
 * Creates an attribute that provides whether the viewport client is in an orthographic view.
 */
UNREALED_API TAttribute<bool> GetIsOrthographicAttribute(const TSharedPtr<FEditorViewportClient>& InViewportClient);

// Camera Menu Widgets
TSharedRef<SWidget> CreateCameraMenuWidget(const TSharedRef<SEditorViewport>& InViewport, bool bInShowExposureSettings = false);
TSharedRef<SWidget> CreateFOVMenuWidget(const TSharedRef<SEditorViewport>& InViewport);
TSharedRef<SWidget> CreateNearViewPlaneMenuWidget(const TSharedRef<SEditorViewport>& InViewport);
TSharedRef<SWidget> CreateFarViewPlaneMenuWidget(const TSharedRef<SEditorViewport>& InViewport);

// Screen Percentage Submenu Widgets
TSharedRef<SWidget> CreateCurrentPercentageWidget(FEditorViewportClient& InViewportClient);
TSharedRef<SWidget> CreateResolutionsWidget(FEditorViewportClient& InViewportClient);
TSharedRef<SWidget> CreateActiveViewportWidget(FEditorViewportClient& InViewportClient);
TSharedRef<SWidget> CreateSetFromWidget(FEditorViewportClient& InViewportClient);
TSharedRef<SWidget> CreateCurrentScreenPercentageSettingWidget(FEditorViewportClient& InViewportClient);
TSharedRef<SWidget> CreateCurrentScreenPercentageWidget(FEditorViewportClient& InViewportClient);

} // namespace UE::UnrealEd

#undef UE_API
