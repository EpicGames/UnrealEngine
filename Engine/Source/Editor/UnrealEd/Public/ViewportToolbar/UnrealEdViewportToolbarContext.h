// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineBaseTypes.h"
#include "Framework/Docking/TabManager.h"
#include "ShowFlags.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"

#include "UnrealEdViewportToolbarContext.generated.h"

#define UE_API UNREALED_API

class FAssetEditorToolkit;
class IPreviewProfileController;
enum ERotationGridMode : int;

namespace UE::UnrealEd
{
	DECLARE_DELEGATE_RetVal_OneParam(bool, IsViewModeSupportedDelegate, EViewModeIndex);

	enum EHidableViewModeMenuSections : uint8;
	DECLARE_DELEGATE_RetVal_OneParam(bool, DoesViewModeMenuShowSectionDelegate, EHidableViewModeMenuSections);
}

class SEditorViewport;
class FEditorViewportClient;
struct FToolMenuContext;

UCLASS(MinimalAPI)
class UUnrealEdViewportToolbarContext : public UObject
{
	GENERATED_BODY()

public:
	/** Actual viewport for context */
	TWeakPtr<SEditorViewport> Viewport;

	/** View mode filters for context menu */
	UE::UnrealEd::IsViewModeSupportedDelegate IsViewModeSupported;
	UE::UnrealEd::DoesViewModeMenuShowSectionDelegate DoesViewModeMenuShowSection;

	/**
	 * Whether the current editor should show menu entries to select between coordinate systems (e.g. Local vs World)
	 */
	bool bShowCoordinateSystemControls = true;

	/**
	 * Add flags to this array to remove them from the Show Menu
	 */
	TArray<FEngineShowFlags::EShowFlag> ExcludedShowMenuFlags;

	/**
	 * Add flags to this array to remove them from the Show Menu (Group)
	 */
	TArray<EShowFlagGroup> ExcludedShowMenuGroupFlags;

	/**
	 * Can be used to retrieve data e.g. TabManager
	 */
	TWeakPtr<FAssetEditorToolkit> AssetEditorToolkit;

	/**
	 * Identifier for the Preview Settings Tab
	 */
	FTabId PreviewSettingsTabId;
	
	static UE_API TSharedPtr<SEditorViewport> GetEditorViewport(const FToolMenuContext& Context);
	static UE_API TSharedPtr<FEditorViewportClient> GetEditorViewportClient(const FToolMenuContext& Context);
	
	UE_API virtual TSharedPtr<IPreviewProfileController> GetPreviewProfileController() const;
	
	bool bShowSurfaceSnap = true;
	
	UE_API virtual void RefreshViewport();
	
	UE_API virtual FText GetGridSnapLabel() const;
	UE_API virtual TArray<float> GetGridSnapSizes() const;
	UE_API virtual bool IsGridSnapSizeActive(int32 GridSizeIndex) const;
	UE_API virtual void SetGridSnapSize(int32 GridSizeIndex);
	
	UE_API virtual FText GetRotationSnapLabel() const;
	UE_API virtual bool IsRotationSnapActive(int32 RotationIndex, ERotationGridMode RotationMode) const;
	UE_API virtual void SetRotationSnapSize(int32 RotationIndex, ERotationGridMode RotationMode);
	
	UE_API virtual FText GetScaleSnapLabel() const;
	UE_API virtual TArray<float> GetScaleSnapSizes() const;
	UE_API virtual bool IsScaleSnapActive(int32 ScaleIndex) const;
	UE_API virtual void SetScaleSnapSize(int32 ScaleIndex);
};

#undef UE_API
