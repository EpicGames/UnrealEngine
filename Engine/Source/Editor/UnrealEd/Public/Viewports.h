// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Viewports.h: The viewport windows used by the editor
=============================================================================*/

#pragma once

#include "CoreMinimal.h"

namespace EditorViewportDefs
{
	/** Default camera position for level editor perspective viewports */
	inline const FVector DefaultPerspectiveViewLocation( 0.0f, 1024.0f, 512.0f );

	/** Default camera orientation for level editor perspective viewports */
	inline const FRotator DefaultPerspectiveViewRotation( -15.0f, -90.0f, 0 );

	/** Default camera field of view angle for level editor perspective viewports */
	inline const float DefaultPerspectiveFOVAngle( 90.0f );
}
