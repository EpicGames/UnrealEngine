// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	VolumeTexturePreview.h: Definitions for previewing 2d textures.
==============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "BatchedElements.h"

/**
 * Batched element parameters for previewing 3d textures.
 */
class FBatchedElementVolumeTexturePreviewParameters : public FBatchedElementParameters
{
public:
	FBatchedElementVolumeTexturePreviewParameters(bool InViewModeAsDepthSlices, float InMipLevel, float InOpacity, bool InShowSlices, const FRotator& InTraceOrientation, bool bInUsePointSampling)
		: bViewModeAsDepthSlices(InViewModeAsDepthSlices)
		, MipLevel(InMipLevel)
		, Opacity(InOpacity)
		, bShowSlices(InShowSlices)
		, TraceOrientation(InTraceOrientation)
		, bUsePointSampling(bInUsePointSampling)
	{
	}

	/** Binds vertex and pixel shaders for this element */
	UNREALED_API virtual void BindShaders(FRHICommandList& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit, ERHIFeatureLevel::Type InFeatureLevel, const FMatrix& InTransform, const float InGamma, const FMatrix& ColorWeights, const FTexture* Texture) override;

private:

	/** Whether to render depth slices or trace into the volume */
	bool bViewModeAsDepthSlices;
	
	/** The mip level to visualize */
	float MipLevel;

	float Opacity;

	/** Whether to show each depth slice of the volume */
	 bool bShowSlices;

	/** The orientation when tracing */
	FRotator TraceOrientation;

	/** Whether to use nearest-point sampling when rendering the volume */
	bool bUsePointSampling;
};
