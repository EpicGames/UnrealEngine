// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/IToolkit.h"
#include "EditorModeManager.h"

/**
 * Tool manager that focuses on tools which the behavior of a certain widget (Ex: Design / node panels)
 * We only provide an interface where the tool tool needs extra-arg info (Ex: paint geometry) 
 * to perform it's specialized behavior.
 * 
 * In all other cases (Tick, FocusLost, etc), the default editor mode tool behavior is used and sufficient.
 */
class FWidgetModeManager : public FEditorModeTools
{
public:
	UNREALED_API FWidgetModeManager();

	// ~Begin SWidget "Proxy" Interface
	UNREALED_API virtual bool OnKeyChar(const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent);
	UNREALED_API virtual bool OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);
	UNREALED_API virtual bool OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);
	UNREALED_API virtual bool OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	UNREALED_API virtual bool OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	UNREALED_API virtual bool OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	UNREALED_API virtual bool OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	UNREALED_API virtual bool OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent);
	UNREALED_API virtual bool OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	UNREALED_API virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent);
	UNREALED_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled);
	// ~End SWidget "Proxy" Interface

	/** Not done in constructor as managed widget typically will not exist during mode construction for editors */
	UNREALED_API virtual void SetManagedWidget(TSharedPtr<SWidget> InManagedWidget);

	/** Not done in constructor as managed widget typically will not exist during mode construction for editors */
	UNREALED_API TSharedPtr<SWidget> GetManagedWidget() const;
	
	/** Toolkit owning the widget of interest */
	TWeakPtr<IToolkit> OwningToolkit;

protected:
	/** Managed widget is forwarded to tool builders / tools to interact with the widget they are operating on */
	TWeakPtr<SWidget> ManagedWidget;

	class UWidgetToolsContext* CachedWidgetToolContext;
};
