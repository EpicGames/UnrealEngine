// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Docking/WorkspaceItem.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Docking/LayoutExtender.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"

DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<class FWorkflowTabFactory>, FCreateWorkflowTabFactory, TSharedPtr<class FAssetEditorToolkit>)

class FExtender;
class FWorkflowCentricApplication;


/////////////////////////////////////////////////////
// FApplicationMode

class FApplicationMode : public TSharedFromThis<FApplicationMode>
{
private:
	friend class FWorkflowCentricApplication;

	// A shared weak reference to the host.  If a mode extender creates a new mode that references its given mode,
	// the host will use ShareHost to ensure that the new mode and the given mode share the same reference to the host.
	TSharedRef<TWeakPtr<FWorkflowCentricApplication>> HostWeakPtr{ MakeShared<TWeakPtr<FWorkflowCentricApplication>>() };

protected:
	/** The tab factories we support */
	FWorkflowAllowedTabSet ApplicationModeTabFactories;

	// The layout to use in this mode
	TSharedPtr<FTabManager::FLayout> TabLayout;

	// The list of tabs that can be spawned in this game mode.
	// Note: This list should be treated as readonly outside of the constructor.
	//FWorkflowAllowedTabSet AllowableTabs;

	// The internal name of this mode
	FName ModeName;

	//@TODO: For test suite use only
	FString UserLayoutString;
	
	/** The toolbar extension for this mode */
	TSharedPtr<FExtender> ToolbarExtender;

	/** The workspace menu category for this mode */
	TSharedPtr<FWorkspaceItem> WorkspaceMenuCategory;

	/** What ini file should be used to persist and restore the user's layout. Defaults to GEditorLayoutIni. */
	FString LayoutIni = GEditorLayoutIni;

	// ShareHost allows the host to set the host reference for potentially related modes.
	UNREALED_API void ShareHost(const FApplicationMode* Mode);

	void SetHost(TWeakPtr<FWorkflowCentricApplication> Host)
	{
		*HostWeakPtr = Host;
	}

public:
	UNREALED_API FApplicationMode(FName InModeName);

	UNREALED_API FApplicationMode(FName InModeName, FText(*GetLocalizedMode)(const FName));

	virtual ~FApplicationMode() = default;

	TSharedPtr<FWorkflowCentricApplication> GetHost() const
	{
		return HostWeakPtr->Pin();
	}

	UNREALED_API void DeactivateMode();
	UNREALED_API TSharedRef<FTabManager::FLayout> ActivateMode();

	UNREALED_API void AddTabFactory(FCreateWorkflowTabFactory FactoryCreator);
	UNREALED_API void RemoveTabFactory(FName TabFactoryID);

	// Register this mode's tab factories with its host's tab manager.
	UNREALED_API void RegisterTabFactoriesWithHost();

	// Interface to register this mode's tab factories with an arbitrary tab manager.
	UNREALED_API void RegisterTabFactoriesWithManager(const TSharedPtr<FTabManager>& InTabManager);

protected:
	// Register this mode's tab factories with an arbitrary app and tab manager.
	// Derived classes should call this if they override RegisterTabFactories,
	// but want the base class functionality.
	UNREALED_API void RegisterTabFactoriesWithAppAndManager(FWorkflowCentricApplication* InApp, const TSharedRef<FTabManager>& InTabManager);

private:
	// Implementation of registering this mode's tab factories with an arbitrary tab manager.
	// This is private to prevent derived classes from calling it.  It didn't used to do anything.
	// Now it calls RegisterTabFactoriesWithAppAndManager with the host and the provided tab manager.
	// If derived classes want this behaviour, they can call RegisterTabFactoriesWithAppAndManager directly.
	UNREALED_API virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager);

public:
	virtual void PreDeactivateMode() {}
	virtual void PostActivateMode() {}

	FName GetModeName() const
	{
		return ModeName;
	}

	TSharedPtr<FExtender> GetToolbarExtender() { return ToolbarExtender; }

	/** @return The the workspace category for this asset editor */
	TSharedRef<FWorkspaceItem> GetWorkspaceMenuCategory() const { return WorkspaceMenuCategory.ToSharedRef(); }

	/** Extender for adding to the default layout for this mode */
	TSharedPtr<FLayoutExtender> LayoutExtender;
};
