// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Types/SlateStructs.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/SlateDelegates.h"
#include "IContentBrowserSingleton.h"

class SBorder;
class SComboButton;

// This widget is a content reference, which can optionally show buttons for interacting with the reference
class SContentReference : public SCompoundWidget
{
public:

	/** Allows for informing user when someone tries to set a new reference*/
	DECLARE_DELEGATE_OneParam(FOnSetReference, UObject*)

	SLATE_BEGIN_ARGS(SContentReference)
		: _Style(TEXT("ContentReference"))
		, _AssetReference((UObject*)NULL)
		, _ShowFindInBrowserButton(true)
		, _ShowToolsButton(false)
		, _AllowSelectingNewAsset(false)
		, _AllowClearingReference(false)
		, _AllowedClass((UClass*)NULL)
		, _WidthOverride(FOptionalSize())
		, _AssetPickerSizeOverride(FVector2D(384, 768))
		, _InitialAssetViewType(EAssetViewType::Tile)
	{}
		// The style of the content reference widget (optional)
		SLATE_ARGUMENT(FName, Style)

		// The asset being displayed by this widget
		SLATE_ATTRIBUTE(UObject*, AssetReference)

		// Should the find in content browser button be shown?
		SLATE_ATTRIBUTE(bool, ShowFindInBrowserButton)

		// Should the tools button be shown?
		SLATE_ATTRIBUTE(bool, ShowToolsButton)

		// Can the user pick a new asset to reference?
		SLATE_ATTRIBUTE(bool, AllowSelectingNewAsset)

		// Can the user clear the existing asset reference?
		SLATE_ATTRIBUTE(bool, AllowClearingReference)

		// If specified, the asset references are restricted to this class
		SLATE_ATTRIBUTE(UClass*, AllowedClass)

		// The event to call when the tools button is clicked
		SLATE_EVENT(FOnClicked, OnClickedTools)

		// If bound, this delegate will be executed for each asset in the asset picker to see if it should be filtered
		SLATE_EVENT(FOnShouldFilterAsset, OnShouldFilterAsset)

		//when user changes the referenced object
		SLATE_EVENT(FOnSetReference, OnSetReference)

		/** When specified, the path box will request this fixed size. */
		SLATE_ATTRIBUTE(FOptionalSize, WidthOverride)

		// when you'd like to modify size of the asset picker window
		SLATE_ATTRIBUTE(FVector2D, AssetPickerSizeOverride)

		// initial view type
		SLATE_ARGUMENT(EAssetViewType::Type, InitialAssetViewType)
		// The event to call when the user selects a new asset or clears the existing reference
		//@TODO: Support that stuff!
	SLATE_END_ARGS()

public:
	// Construct a SContentReference
	UNREALED_API void Construct(const FArguments& InArgs);

	// Causes the asset picker to open, allowing the user to select a new asset
	UNREALED_API void OpenAssetPickerMenu();

protected:
	UNREALED_API EVisibility GetUseButtonVisibility() const;
	UNREALED_API EVisibility GetPickButtonVisibility() const;
	UNREALED_API EVisibility GetFindButtonVisibility() const;
	UNREALED_API EVisibility GetClearButtonVisibility() const;
	UNREALED_API EVisibility GetToolsButtonVisibility() const;

	UNREALED_API FReply OnClickUseButton();
	UNREALED_API FReply OnClickFindButton();
	UNREALED_API FReply OnClickClearButton();

	UNREALED_API TSharedRef<SWidget> MakeAssetPickerMenu();
	UNREALED_API void OnAssetSelectedFromPicker(const FAssetData& AssetData);

	UNREALED_API FText GetAssetShortName() const;
	UNREALED_API FText GetAssetFullName() const;

	UNREALED_API FReply OnDoubleClickedOnAssetName(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent);
	UNREALED_API void OpenAssetToEdit();

	UNREALED_API const FSlateBrush* GetBorderImage() const;

	static UNREALED_API void FindObjectInContentBrowser(UObject* Object);
protected:
	// Attributes
	TAttribute<UObject*> AssetReference;
	TAttribute<bool> ShowFindInBrowserButton;
	TAttribute<bool> ShowToolsButton;
	TAttribute<bool> AllowSelectingNewAsset;
	TAttribute<bool> AllowClearingReference;
	TAttribute<UClass*> AllowedClass;

	// Asset picker configuration
	TAttribute<FVector2D>	AssetPickerSizeOverride;
	EAssetViewType::Type	InitialAssetViewType;

	// Delegates
	FOnShouldFilterAsset OnShouldFilterAsset;
	FOnSetReference OnSetReference;

	// Widgets
	TSharedPtr<SBorder> AssetReferenceNameBorderWidget;
	TSharedPtr<SComboButton> PickerComboButton;

	// Resources
	const FSlateBrush* BorderImageNormal;
	const FSlateBrush* BorderImageHovered;
};
