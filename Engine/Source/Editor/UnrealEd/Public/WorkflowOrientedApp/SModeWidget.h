// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/Platform.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

struct FSlateBrush;

// Event sent when a mode change is requested
DECLARE_DELEGATE_OneParam( FOnModeChangeRequested, FName );

enum class ECheckBoxState : uint8;

// This is the base class for a mode widget in a workflow oriented editor
class SModeWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SModeWidget)
		: _CanBeSelected(true)
	{}
		// The currently active mode, used to determine which mode is highlighted
		SLATE_ATTRIBUTE( FName, OnGetActiveMode )

		// The image for the icon
		SLATE_ATTRIBUTE( const FSlateBrush*, IconImage )

		// The delegate that will be called when this widget wants to change the active mode
		SLATE_EVENT( FOnModeChangeRequested, OnSetActiveMode )

		// Can this mode ever be selected?
		SLATE_ATTRIBUTE( bool, CanBeSelected )

		// Slot for the always displayed contents
		SLATE_NAMED_SLOT( FArguments, ShortContents )

		SLATE_ATTRIBUTE( const FSlateBrush*, DirtyMarkerBrush)
		
	SLATE_END_ARGS()

public:
	// Construct a SModeWidget
	UNREALED_API void Construct(const FArguments& InArgs, const FText& InText, const FName InMode);

private:
	bool IsActiveMode() const;
	ECheckBoxState GetModeCheckState() const;
	void OnModeTabClicked(ECheckBoxState CheckBoxState);

	FSlateFontInfo GetDesiredTitleFont() const;
private:
	// The active mode of this group
	TAttribute<FName> OnGetActiveMode;

	// The delegate to call when this mode is selected
	FOnModeChangeRequested OnSetActiveMode;

	// Can this mode be selected?
	TAttribute<bool> CanBeSelected;

	// The text representation of this mode
	FText ModeText;

	// The mode this widget is representing
	FName ThisMode;

	// Border images
	const FSlateBrush* ActiveModeBorderImage;
	const FSlateBrush* InactiveModeBorderImage;
	const FSlateBrush* HoverBorderImage;
};
