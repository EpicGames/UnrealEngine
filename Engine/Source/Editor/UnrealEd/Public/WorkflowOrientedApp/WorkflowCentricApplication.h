// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/AssetEditorToolkit.h"

class FApplicationMode;
class FWorkflowAllowedTabSet;
class FWorkflowTabFactory;
class UToolMenu;

// Delegate for mutating a mode
DECLARE_DELEGATE_RetVal_TwoParams(TSharedRef<FApplicationMode>, FWorkflowApplicationModeExtender, const FName /*ModeName*/, TSharedRef<FApplicationMode> /*InMode*/);

/////////////////////////////////////////////////////
// FWorkflowCentricApplication

class FWorkflowCentricApplication : public FAssetEditorToolkit
{
public:
	// IToolkit interface
	UNREALED_API virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	UNREALED_API virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	// End of IToolkit interface
	
	// FAssetEditorToolkit interface
	UNREALED_API virtual bool OnRequestClose(EAssetEditorCloseReason InCloseReason) override;
	UNREALED_API virtual void OnClose() override;
	UNREALED_API virtual FName GetToolMenuToolbarName(FName& OutParentName) const override;
	// End of FAssetEditorToolkit interface

	// Returns the name of the toolbar for the given mode along with the name of the toolbar's parent.
	UNREALED_API FName GetToolMenuToolbarNameForMode(const FName InModeName, FName& OutParentName) const;

	// Returns the current mode of this application
	UNREALED_API FName GetCurrentMode() const;
	TSharedPtr<FApplicationMode> GetCurrentModePtr() const { return CurrentAppModePtr; }
	bool IsModeCurrent(FName ModeToCheck) const {return GetCurrentMode() == ModeToCheck;}

	// Attempt to set the current mode.  If this mode is illegal or unknown, the mode will remain unchanged.
	UNREALED_API virtual void SetCurrentMode(FName NewMode);

	UNREALED_API void PushTabFactories(FWorkflowAllowedTabSet& FactorySetToPush) const;

	// Gets the mode extender list for all workflow applications (append to customize a specific mode)
	static TArray<FWorkflowApplicationModeExtender>& GetModeExtenderList() { return ModeExtenderList; }

	/**
	* Registers toolbar for a mode if not already registered
	*
	* @param InModeName - Name of the mode to register toolbar for
	* @return nullptr if toolbar already registered.
	*/
	UNREALED_API virtual UToolMenu* RegisterModeToolbarIfUnregistered(const FName InModeName);
protected:

	UNREALED_API virtual void AddApplicationMode(FName ModeName, TSharedRef<FApplicationMode> Mode);

	// Remove an application mode.
	// @return true if the mode was successfully removed.
	// @note Has no effect if you call this function with the current mode.
	UNREALED_API virtual bool RemoveApplicationMode(FName ModeName);

	// Remove all application modes that were added through AddApplicationMode.
	UNREALED_API virtual void RemoveAllApplicationModes();

	// Get the list of registered application modes. Use AddApplicationMode to add modes.
	UNREALED_API virtual const TMap< FName, TSharedPtr<FApplicationMode> >& GetApplicationModeList() const;
protected:
	TSharedPtr<FApplicationMode> CurrentAppModePtr;

private:
	// List of modes; do not access directly, use AddApplicationMode and SetCurrentMode
	TMap< FName, TSharedPtr<FApplicationMode> > ApplicationModeList;

	static UNREALED_API TArray<FWorkflowApplicationModeExtender> ModeExtenderList;
};


