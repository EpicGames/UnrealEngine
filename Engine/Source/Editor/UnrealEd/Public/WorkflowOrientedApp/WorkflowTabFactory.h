// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Widgets/SWidget.h"
#include "Textures/SlateIcon.h"
#include "Toolkits/AssetEditorToolkit.h"

class FApplicationMode;
class FMenuBuilder;
class FTabInfo;
class SToolTip;

/////////////////////////////////////////////////////
// FTabPayload

struct FTabPayload
{
public:
	// The type of this payload
	const FName PayloadType;
public:
	virtual ~FTabPayload() = default;

	// Is this payload equal to the other one?
	virtual bool IsEqual(const TSharedRef<FTabPayload>& OtherPayload) const
	{
		return this->PayloadType == OtherPayload->PayloadType;
	}

	virtual bool IsValid() const
	{
		return true;
	}

protected:
	// Buried constructor.  This is a base class meant to be inherited from
	FTabPayload(FName InPayloadType)
		: PayloadType(InPayloadType)
	{
	}
};

/////////////////////////////////////////////////////
// FWorkflowTabSpawnInfo

class FWorkflowTabSpawnInfo
{
public:
	// Payload to spawn the tab with
	TSharedPtr<FTabPayload> Payload;

	// The TabInfo being used to spawn this
	TSharedPtr<class FTabInfo> TabInfo;

	/** The parent tab manager */
	TSharedPtr<FTabManager> TabManager;

public:
	FWorkflowTabSpawnInfo()
		: TabInfo(NULL)
	{
	}
};

/////////////////////////////////////////////////////
// FWorkflowTabFactory

class FWorkflowTabFactory : public TSharedFromThis<FWorkflowTabFactory>
{
protected:
	FName TabIdentifier;
	ETabRole TabRole;

	FText TabLabel;
	FSlateIcon TabIcon;

	float InsideTabPadding;

	bool bIsSingleton;
	bool bShouldAutosize;

	FText ViewMenuDescription;
	FText ViewMenuTooltip;

	TSharedPtr<FWorkspaceItem> Group;

	TWeakPtr<class FAssetEditorToolkit> HostingApp;

public:
	virtual ~FWorkflowTabFactory() = default;

	// Creates a toolkit tab given the spawning information
	UNREALED_API virtual TSharedRef<SDockTab> SpawnTab(const FWorkflowTabSpawnInfo& Info) const;
	// Creates a toolkit tab without any content, given the spawning information
	UNREALED_API virtual TSharedRef<SDockTab> SpawnBlankTab() const;

	/**
	 * Updates a given tab with new content
	 *
	 * @param InDockTab		The tab to update
	 * @param InInfo		The spawning info, used to generate required tab information
	 * @param InContent		The content to update the tab with
	 */
	UNREALED_API virtual void UpdateTab(TSharedPtr<SDockTab> InDockTab, const FWorkflowTabSpawnInfo& Info, TSharedPtr< SWidget > InContent) const;

	// Creates a view menu entry for this factory
	UNREALED_API virtual void CreateViewMenuEntry(FMenuBuilder& MenuBuilder, const FUIAction& Action) const;

	/** Register a tab spawner for this factory */
	UNREALED_API virtual FTabSpawnerEntry& RegisterTabSpawner(TSharedRef<FTabManager> TabManager, const FApplicationMode* CurrentApplicationMode) const;

public:
	// Constructor
	UNREALED_API FWorkflowTabFactory(FName InIdentifier, TSharedPtr<class FAssetEditorToolkit> InHostingApp);

	// Get the identifier of this factory
	FName GetIdentifier() const { return TabIdentifier; }

	// Are tabs produced by this factory limited to only having one open at a time?
	bool IsSingleton() const { return bIsSingleton; }

	// Creates the label for the tab
	UNREALED_API virtual TAttribute<FText> ConstructTabName(const FWorkflowTabSpawnInfo& Info) const;

	// Creates the label suffix for the tab
	UNREALED_API virtual TAttribute<FText> ConstructTabLabelSuffix(const FWorkflowTabSpawnInfo& Info) const;

	/** Returns TRUE if the payload is valid */
	virtual bool IsPayloadValid(TSharedRef<FTabPayload> Payload) const { return false; }

protected:
	// Call this to enable the default padding surrounding the tab contents
	UNREALED_API void EnableTabPadding();

	/** Callback function for spawning the tab */
	UNREALED_API virtual TSharedRef<SDockTab> OnSpawnTab(const FSpawnTabArgs& SpawnArgs, TWeakPtr<FTabManager> WeakTabManager) const;

	/** Callback function for checking to see if a tab can/should be spawned */
	UNREALED_API virtual bool CanSpawnTab(const FSpawnTabArgs& SpawnArgs, TWeakPtr<FTabManager> WeakTabManager) const;

public:
	// Creates the widget that will be the body of the new tab
	// Users will most commonly override just this.
	UNREALED_API virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const;

	// Virtual to create an icon for the tab
	UNREALED_API virtual const FSlateBrush* GetTabIcon(const FWorkflowTabSpawnInfo& Info) const;
	
	// Virtual to create an icon for the menu entry
	UNREALED_API virtual const FSlateIcon& GetTabSpawnerIcon(const FWorkflowTabSpawnInfo& Info) const;

	// Create a tooltip widget for the tab
	UNREALED_API virtual TSharedPtr<SToolTip> CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const;

	// Virtual to get optional extra tooltip text for tab
	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const
	{
		return TabLabel;
	}

private:
	friend class FDocumentTracker;
};

/////////////////////////////////////////////////////
// FDocumentTabFactory

class FDocumentTabFactory : public FWorkflowTabFactory
{
public:
	// Saves the state for this tab
	//@TODO: DOCMANAGEMENT: This is very special-cased and isn't a general purpose solution
	virtual void SaveState(TSharedPtr<SDockTab> Tab, TSharedPtr<FTabPayload> Payload) const {}

	// Called when a tab created from this factory is activated
	virtual void OnTabActivated(TSharedPtr<SDockTab> Tab) const {}

	// Called when a tab created from this factory is brought to the foreground
	virtual void OnTabForegrounded(TSharedPtr<SDockTab> Tab) const {}

	// Called when a tab created from this factory is sent to the background
	virtual void OnTabBackgrounded(TSharedPtr<SDockTab> Tab) const {}

	// Called when a tab created from this factory should be refreshed
	virtual void OnTabRefreshed(TSharedPtr<SDockTab> Tab) const {}

	// Called when a tab is being closed
	//virtual void OnTabClosed(TSharedRef<SDockTab> Tab, TSharedPtr<FTabPayload> Payload) const {}

	// Does this tab support the specified payload
	virtual bool IsPayloadSupported(TSharedRef<FTabPayload> Payload) const { return false; }

	/** Creates a tab history node from the passed in payload */
	UNREALED_API virtual TSharedRef<struct FGenericTabHistory> CreateTabHistoryNode(TSharedPtr<FTabPayload> Payload);

protected:
	UNREALED_API FDocumentTabFactory(FName InIdentifier, TSharedPtr<class FAssetEditorToolkit> InHostingApp);
};
