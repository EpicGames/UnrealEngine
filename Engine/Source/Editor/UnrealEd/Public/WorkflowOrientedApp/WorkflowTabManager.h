// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Framework/Docking/TabManager.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"

class SMenuAnchor;
struct FGenericTabHistory;

/////////////////////////////////////////////////////

namespace EDocumentOpenBehavior
{
	enum Type
	{
		OnlyReturnExisting,
		AutomaticallyOpen
	};
}

/////////////////////////////////////////////////////
// FWorkflowAllowedTabSet

class FWorkflowAllowedTabSet
{
public:
	// Searches this set for a factory with the specified ID, or returns NULL.
	UNREALED_API TSharedPtr<class FWorkflowTabFactory> GetFactory(FName FactoryID) const;

	// Registers a factory with this set - must be unique.
	UNREALED_API void RegisterFactory(TSharedPtr<class FWorkflowTabFactory> Factory);

	// Unregisters a factory with this specified ID.
	UNREALED_API void UnregisterFactory(FName FactoryID);

	// Merges in a set of factories into this set - duplicates are OK.
	UNREALED_API void MergeInSet(const FWorkflowAllowedTabSet& OtherSet);

	// Clears the set
	UNREALED_API void Clear();

	// Allows iteration over Factories
	UNREALED_API TMap< FName, TSharedPtr<class FWorkflowTabFactory> >::TIterator CreateIterator();
	UNREALED_API TMap< FName, TSharedPtr<class FWorkflowTabFactory> >::TConstIterator CreateConstIterator() const;

private:
	// Map of available factories in this mode
	TMap< FName, TSharedPtr<class FWorkflowTabFactory> > Factories;

	friend class FDocumentTracker;
};

/////////////////////////////////////////////////////
// FTabInfo

class FTabInfo : public TSharedFromThis<FTabInfo>
{
public:
	UNREALED_API FTabInfo(const TSharedRef<SDockTab>& InTab, const TSharedPtr<FDocumentTabFactory>& InSpawner, const TSharedPtr<class FDocumentTracker>& InTracker);

	/** Returns TRUE if the payload used by this TabInfo is the same as passed in */
	UNREALED_API bool PayloadMatches(const TSharedPtr<FTabPayload> TestPayload) const;

	static UNREALED_API bool PayloadMatches(TSharedPtr<FTabPayload> A, TSharedPtr<FTabPayload> B);

	/**
	 * Adds history data to immediately after the current location in the history list, wiping out any history after in the process
	 *
	 * @param InHistoryNode		The history node to add
	 * @param bInSaveHistory	TRUE if history should be saved
	 */
	UNREALED_API void AddTabHistory(TSharedPtr< struct FGenericTabHistory > InHistoryNode, bool bInSaveHistory = true);

	/** Retrieves the history currently visible in this tab */
	UNREALED_API TSharedPtr<struct FGenericTabHistory> GetCurrentHistory() const;

	/** Sets the current history and evoke it, this can either be an old or new history node */
	UNREALED_API void SetCurrentHistory(TSharedPtr<struct FGenericTabHistory> NewHistory, bool bInSaveHistory = true, bool bShouldRestore = true);

	/** Single step forward in history */
	UNREALED_API FReply OnGoForwardInHistory();

	/** Single step back in history */
	UNREALED_API FReply OnGoBackInHistory();

	/** Jumps backwards to the nearest valid history. If no history backwards is valid, jumps forward. In the case that there is no valid history, the tab is closed */
	UNREALED_API void JumpToNearestValidHistoryData();

	/** Retrieves the factory from the current history data, which is being displayed by the tab */
	UNREALED_API TWeakPtr<FDocumentTabFactory> GetFactory() const;

	/** Retrieves the payload from the current history data, which is being displayed by the tab */
	UNREALED_API TSharedPtr<FTabPayload> GetPayload() const;

	/** Retrieves the tab used by this TabInfo */
	TWeakPtr<class SDockTab> GetTab() const
	{
		return Tab;
	}

	/** Jumps immediately to an index in the history if valid */
	UNREALED_API void GoToHistoryIndex(int32 InHistoryIdx);

	/** Returns TRUE if stepping backward in history is allowed */
	UNREALED_API bool CanStepBackwardInHistory() const;

	/** Returns TRUE if stepping forward in history is allowed */
	UNREALED_API bool CanStepForwardInHistory() const;

	/** Creates the history navigation widget used for navigating back and forward in history */
	UNREALED_API TSharedRef< SWidget > CreateHistoryNavigationWidget();

private:
	/**
	 * Mouse down callback to display a history menu
	 *
	 * @param InMenuAnchor		This is the anchor the menu will use for positioning
	 */
	UNREALED_API FReply OnMouseDownHistory( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, TWeakPtr< SMenuAnchor > InMenuAnchor );

	/** 
	 * Callback to create the history menu.
	 *
	 * @param bInBackHistory		TRUE if the back history is requested, FALSE if the forward history is
	 *
	 * @return						The menu widget displaying all available history
	 */
	UNREALED_API TSharedRef<SWidget> CreateHistoryMenu(bool bInBackHistory) const;

private:	
	/** Cached history navigation widget */
	TSharedPtr<SWidget> HistoryNavigationWidget;
	/** Tab this info represents */
	TWeakPtr<class SDockTab> Tab;
	/** Current tab history element in this tab, may not necessarily be in the global history */
	TSharedPtr<struct FGenericTabHistory> CurrentHistory;
	/** Pointer to document tracker that spawned us */
	TWeakPtr<class FDocumentTracker> WeakTracker;
};

/////////////////////////////////////////////////////
// FGenericTabHistory

struct FGenericTabHistory
{
public:
	/**
	 * @param InFactory		The factory used to regenerate the content
	 * @param InPayload		The payload object used to regenerate the content
	 */
	FGenericTabHistory(TSharedPtr<class FDocumentTabFactory> InFactory, TSharedPtr<FTabPayload> InPayload)
		: FactoryPtr(InFactory)
		, Payload(InPayload)
	{

	}

	virtual ~FGenericTabHistory() = default;

	/**
	 * Evokes the history data this struct represents, reinstating the payload and other data it had at the time the history was saved
	 *
	 * @param InTab				The tab to update with the history data
	 */
	virtual void EvokeHistory(TSharedPtr<FTabInfo> InTabInfo, bool bPrevTabMatches)
	{
		FWorkflowTabSpawnInfo SpawnInfo;
		SpawnInfo.Payload = Payload;
		SpawnInfo.TabInfo = InTabInfo;

		if(!bPrevTabMatches)
		{	
			FactoryPtr.Pin()->UpdateTab(InTabInfo->GetTab().Pin(), SpawnInfo, FactoryPtr.Pin()->CreateTabBody(SpawnInfo));
		}
	}

	/** Saves any important information from the payload into the history node */
	virtual void SaveHistory() {}

	/** Retrieves any important information from the history node and sets it in the payload */
	virtual void RestoreHistory() {}

	/** Returns the Payload this history node represents */
	TSharedPtr<FTabPayload> GetPayload() const
	{
		return Payload;
	}

	/** Returns the Factory this history node represents */
	TWeakPtr<FDocumentTabFactory> GetFactory() const
	{
		return FactoryPtr;
	}

	/**
	 * Checks if the payload for this history node is currently valid
	 *
	 * @return		TRUE if the history is valid
	 */
	bool IsHistoryValid() const
	{
		return FactoryPtr.Pin()->IsPayloadValid(Payload.ToSharedRef());
	}

	/** Returns the title used to display this history node */
	TAttribute<FText> GetHistoryTitle() const
	{
		FWorkflowTabSpawnInfo SpawnInfo;
		SpawnInfo.Payload = Payload;

		return FactoryPtr.Pin()->ConstructTabName(SpawnInfo);
	}

	/** Returns the tab this is bound to */
	TSharedPtr<FTabInfo> GetBoundTab() const
	{
		return BoundTab.Pin();
	}

	/** Binds this to a specific tab */
	void BindToTab(const TSharedPtr<FTabInfo>& InTab)
	{
		BoundTab = InTab;
	}

protected:
	/** The factory used to generate widgets for the payload */
	TWeakPtr<class FDocumentTabFactory> FactoryPtr;

	/** The payload this history node represents */
	TSharedPtr<FTabPayload> Payload;

	/** Specific tab this is bound to, may be null */
	TWeakPtr<FTabInfo> BoundTab;
};

/////////////////////////////////////////////////////
// FDocumentTracker

DECLARE_DELEGATE_OneParam(FOnTabActivated, TSharedPtr<SDockTab>);

class FDocumentTracker : public TSharedFromThis<FDocumentTracker>
{
public:
	UNREALED_API explicit FDocumentTracker(FName InDefaultDocumentId = "Document");
	UNREALED_API ~FDocumentTracker();

	UNREALED_API void ClearDocumentFactories();
	UNREALED_API void RegisterDocumentFactory(TSharedPtr<class FDocumentTabFactory> Factory);

	// Creates a new tab manager
	UNREALED_API void Initialize(TSharedPtr<FAssetEditorToolkit> InHostingApp);

	UNREALED_API void SetTabManager(const TSharedRef<FTabManager>& InTabManager);

	/** Are we opening a new document, or restoring a previously opened document */
	enum EOpenDocumentCause
	{
		// Opens a new document, will open in existing tab if one is found
		OpenNewDocument,
		// Always open in a new tab
		ForceOpenNewDocument,
		// Restoring tab state from a previous session
		RestorePreviousDocument,
		// Navigating within an open tab
		NavigatingCurrentDocument,
		// Navigate without saving current document position
		QuickNavigateCurrentDocument,
		// Key/Mouse binding for back, turns into NavigatingHistory
		NavigateBackwards,
		// Key/Mouse binding for forward
		NavigateForwards,
		// Going forward or back in the history
		NavigatingHistory,
		// Same as OpenNewDocument, but does not create a "Document" tab
		SpawnManagedDocument,

		// Deprecated, treated the same as OpenNewDocument
		CreateHistoryEvent,
	};

	/**
	 * Handles opening a document and handling tab navigation or forcing a new tab to be opened
	 *
	 * @param InPayload					Payload to use for the tab
	 * @param InOpenCause				The cause of the document being opened
	 *
	 * @return							The tab modified
	 */
	UNREALED_API TSharedPtr<SDockTab> OpenDocument(TSharedPtr<FTabPayload> InPayload, EOpenDocumentCause InOpenCause);

	// Closes any tabs of the specified type that match the payload (or all of them if the payload is NULL)
	UNREALED_API void CloseTab(TSharedPtr<FTabPayload> Payload);

	/** Cleans any invalid tabs, either moving them to a valid place in history or closing the tab altogether */
	UNREALED_API void CleanInvalidTabs();

	// Calls OnTabRefreshed for each open tab (on the factories that created them)
	UNREALED_API void RefreshAllTabs() const;

	// Finds a factory that can deal with the supplied payload
	UNREALED_API TSharedPtr<FDocumentTabFactory> FindSupportingFactory(TSharedRef<FTabPayload> Payload) const;

	// Finds all tabs that match the specified identifier and optional payload, placing them in the specified array
	UNREALED_API void FindMatchingTabs(TSharedPtr<FTabPayload> Payload, /*inout*/ TArray< TSharedPtr<SDockTab> >& Results);

	// Finds all tabs that match the specified identifier, placing them in the specified array
	UNREALED_API void FindAllTabsForFactory(const TWeakPtr<FDocumentTabFactory>& Factory, /*inout*/ TArray< TSharedPtr<SDockTab> >& Results);

	// Saves the state of all tabs
	UNREALED_API void SaveAllState();

	// Replaces the open payload in the specified tab with a new one; recreating the contents
	UNREALED_API void ReplacePayloadInTab(TSharedPtr<SDockTab> TargetTab, TSharedPtr<FTabPayload> NewPayload);

	/** Get all the live documents tracked by this tracker */
	UNREALED_API TArray< TSharedPtr<SDockTab> > GetAllDocumentTabs() const;

	/** Returns the current active tab */
	UNREALED_API TSharedPtr<SDockTab> GetActiveTab() const;
private:

	// List of open tabs; used to bring one to front and focus it
	typedef TArray< TSharedPtr< FTabInfo > > FTabList;
private:
	// List of currently spawned tool panels
	FTabList SpawnedTabs;
	
	// List of allowable tab spawning mechanisms
	TMap< FName, TSharedPtr<class FDocumentTabFactory> > PotentialTabFactories;

	// The hosting application we're associated with
	TWeakPtr<FAssetEditorToolkit> HostingAppPtr;

	// The Slate tab manager for the application we are managing documents for
	TSharedPtr<FTabManager> TabManager;

	// The last active tab info to be selected, used for opening new documents in the most recently used tab
	TWeakPtr<FTabInfo> WeakLastEditedTabInfo;

	/** Handle to the registered OnActiveTabChanged delegate */
	FDelegateHandle OnActiveTabChangedDelegateHandle;

	/** Handle to the registered OnTabForegrounded delegate */
	FDelegateHandle TabForegroundedDelegateHandle;

	/** List of history data nodes */
	TArray< TSharedPtr<FGenericTabHistory> > History;

	/** Current history index */
	int32 CurrentHistoryIndex;

	/**  Document ID to use when inserting documents into the tab manager. If this is NAME_None, the tab factory's TabIdentifier will be used. */
	FName DefaultDocumentId;

private:
	// Clean the spawned list 
	UNREALED_API FTabList& GetSpawnedList();

	// Cleans the LastEditedTabInfo and returns pinned
	UNREALED_API TSharedPtr<FTabInfo> GetLastEditedTabInfo();

	// Called by the global active tab changed callback; dispatches to individually registered callbacks
	UNREALED_API void OnActiveTabChanged(TSharedPtr<SDockTab> PreviouslyActive, TSharedPtr<SDockTab> NewlyActivated);

	// Called by the global tab manager foregrounds a tab; dispatches to individually registered callbacks
	UNREALED_API void OnTabForegrounded(TSharedPtr<SDockTab> PreviouslyActive, TSharedPtr<SDockTab> NewlyActivated);

	/** Callback for navigating tabs with buttons */
	UNREALED_API FReply OnNavigateTab(FDocumentTracker::EOpenDocumentCause InCause);

	/** Helper function to find the tab in the foreground */
	UNREALED_API TWeakPtr< FTabInfo > FindTabInForeground();

	/** Navigates to specific index in tab history, returns success if it was found */
	UNREALED_API bool NavigateToTabHistory(int32 InHistoryIdx);

	/** Returns the currently active tab history, may be null */
	UNREALED_API TSharedPtr<FGenericTabHistory> GetCurrentTabHistory();

	/**
	 * Navigates current tab
	 *
	 * @param InPayload					Payload to use for the tab
	 * @param InNavigateCause			The cause of the navigation
	 *
	 * @return							The tab navigated and modified
	 */
	UNREALED_API TSharedPtr<SDockTab> NavigateCurrentTab(TSharedPtr<FTabPayload> InPayload, EOpenDocumentCause InNavigateCause);

	/**
	 * Opens a new tab
	 *
	 * @param InTabHistory				New or existing tab history to open tab to
	 * @param InOpenCause				The cause of the tab being opened
	 *
	 * @return							The tab opened and modified
	 */
	UNREALED_API TSharedPtr<SDockTab> OpenNewTab(TSharedPtr<FGenericTabHistory> InTabHistory, EOpenDocumentCause InOpenCause);

	friend class FTabInfo;
};

