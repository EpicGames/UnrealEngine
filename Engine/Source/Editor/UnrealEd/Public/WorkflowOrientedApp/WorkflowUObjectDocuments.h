// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "Misc/Attribute.h"
#include "Templates/Casts.h"
#include "Widgets/SWidget.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"


/////////////////////////////////////////////////////
// FTabPayload_UObject

struct FTabPayload_UObject : public FTabPayload
{
public:
	// Create a new payload wrapper for a UObject
	static TSharedRef<FTabPayload_UObject> Make(const UObject* DocumentID)
	{
		return MakeShareable(new FTabPayload_UObject(const_cast<UObject*>(DocumentID)));
	}

	// Helper method to get the payload object as a specific type
	// (checks both that the payload is a FTabPayload_UObject, and that the type is correct)
	template <typename CastType>
	static CastType* CastChecked(TSharedPtr<FTabPayload> Payload)
	{
		check((Payload.IsValid()) && (Payload->PayloadType == NAME_Object));

		UObject* UntypedObject = StaticCastSharedPtr<FTabPayload_UObject>(Payload)->DocumentID.Get(true);
		return ::CastChecked<CastType>(UntypedObject);
	}

	// Determine if another payload is the same as this one
	virtual bool IsEqual(const TSharedRef<FTabPayload>& OtherPayload) const override
	{
		if (OtherPayload->PayloadType == PayloadType)
		{
			return this->DocumentID.HasSameIndexAndSerialNumber(StaticCastSharedRef<FTabPayload_UObject>(OtherPayload)->DocumentID);
		}

		return false;
	}

	virtual bool IsValid() const override
	{
		return DocumentID.IsValid();
	}

	virtual ~FTabPayload_UObject() = default;

private:
	// Buried constructor. Use Make instead!
	FTabPayload_UObject(UObject* InDocumentID)
		: FTabPayload(NAME_Object)
		, DocumentID(InDocumentID)
	{
	}

private:
	// The object that is the real payload
	TWeakObjectPtr<UObject> DocumentID;
};

/////////////////////////////////////////////////////
// FDocumentTabFactoryForObjects

template<typename BaseClass>
struct FDocumentTabFactoryForObjects : public FDocumentTabFactory
{
public:
	// Does this factory support this type of objects?
	virtual bool SupportsObjectType(UObject* DocumentID) const
	{
		return DocumentID->IsA(BaseClass::StaticClass());
	}
public:

	// FWorkflowTabFactory public interface
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override
	{
		BaseClass* TypedDocumentID = FTabPayload_UObject::CastChecked<BaseClass>(Info.Payload);
		return CreateTabBodyForObject(Info, TypedDocumentID);
	}

	virtual const FSlateBrush* GetTabIcon(const FWorkflowTabSpawnInfo& Info) const override
	{
		BaseClass* TypedDocumentID = FTabPayload_UObject::CastChecked<BaseClass>(Info.Payload);
		return GetTabIconForObject(Info, TypedDocumentID);
	}

	virtual bool IsPayloadSupported(TSharedRef<FTabPayload> Payload) const override
	{
		if (Payload->PayloadType == NAME_Object && Payload->IsValid())
		{
			UObject* DocumentID = FTabPayload_UObject::CastChecked<UObject>(Payload);
			return SupportsObjectType(DocumentID);
		}
		return false;
	}

	virtual bool IsPayloadValid(TSharedRef<FTabPayload> Payload) const override
	{
		if (Payload->PayloadType == NAME_Object)
		{
			return Payload->IsValid();
		}
		return false;
	}
	// End of FWorkflowTabFactory public interface

protected:
	// Protected constructor
	FDocumentTabFactoryForObjects(FName InIdentifier, TSharedPtr<class FAssetEditorToolkit> InHostingApp)
		: FDocumentTabFactory(InIdentifier, InHostingApp)
	{
	}

	// Creates the label for the tab
	virtual TAttribute<FText> ConstructTabName(const FWorkflowTabSpawnInfo& Info) const override
	{
		check(Info.Payload.IsValid());
		BaseClass* TypedDocumentID = FTabPayload_UObject::CastChecked<BaseClass>(Info.Payload);
		return ConstructTabNameForObject(TypedDocumentID);
		//@TODO: DOCMANAGEMENT: Need to incorporate Info.TabPrefix into these!
	}

	// Methods to override in subclasses that make life easier for UObject-based documents
	virtual TAttribute<FText> ConstructTabNameForObject(BaseClass* DocumentID) const=0;
	virtual TSharedRef<SWidget> CreateTabBodyForObject(const FWorkflowTabSpawnInfo& Info, BaseClass* DocumentID) const=0;
	virtual const FSlateBrush* GetTabIconForObject(const FWorkflowTabSpawnInfo& Info, BaseClass* DocumentID) const=0;
};










