// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "Misc/Guid.h"
#include "WorldFoldersImplementation.h"
#include "WorldPersistentFolders.h"
#include "WorldTransientFolders.h"
#include "WorldFolders.generated.h"

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
}

DECLARE_LOG_CATEGORY_EXTERN(LogWorldFolders, Log, All)

USTRUCT()
struct FActorFolderProps
{
	GENERATED_USTRUCT_BODY()

	FActorFolderProps() : bIsExpanded(true) {}

	FORCEINLINE friend FArchive& operator<<(FArchive& Ar, FActorFolderProps& Folder)
	{
		return Ar << Folder.bIsExpanded;
	}

	bool bIsExpanded;
};

USTRUCT()
struct FActorPlacementFolder
{
	GENERATED_BODY()

	UE_INTERNAL
	UPROPERTY()
	FName Path;

	UE_INTERNAL
	UPROPERTY()
	TWeakObjectPtr<UObject> RootObjectPtr;
	
	UE_INTERNAL
	UPROPERTY()
	FGuid ActorFolderGuid;

	FActorPlacementFolder& operator= (const FFolder& InOtherFolder);

	UE_INTERNAL void Reset();
	UE_INTERNAL FFolder GetFolder() const;
};

/**
 * Per-World Actor Folders UObject (used to support undo/redo reliably)
 * This class is reserved for low-level Engine use, therefore relevant APIs are marked UE_INTERNAL.
 * The public, non-engine API for Folder operations is through the FActorFolders struct.
 */
UCLASS(MinimalAPI)
class UWorldFolders : public UObject
{
public:
	GENERATED_BODY()

	UE_INTERNAL UNREALED_API void Initialize(UWorld* InWorld);

	UE_INTERNAL UNREALED_API void RebuildList();
	UE_INTERNAL UNREALED_API bool AddFolder(const FFolder& InFolder);
	UE_INTERNAL UNREALED_API bool RemoveFolder(const FFolder& InFolder, bool bShouldDeleteFolder = false);
	UE_INTERNAL UNREALED_API bool RenameFolder(const FFolder& InOldFolder, const FFolder& InNewFolder);
	UE_INTERNAL UNREALED_API bool IsFolderExpanded(const FFolder& InFolder) const;
	UE_INTERNAL UNREALED_API bool SetIsFolderExpanded(const FFolder& InFolder, bool bIsExpanded);
	UE_INTERNAL UNREALED_API FFolder GetActorEditorContextFolder(bool bMustMatchCurrentLevel) const;
	UE_INTERNAL UNREALED_API bool SetActorEditorContextFolder(const FFolder& InFolder);
	UE_INTERNAL UNREALED_API void PushActorEditorContext(bool bDuplicateContext = false);
	UE_INTERNAL UNREALED_API void PopActorEditorContext();
	UE_INTERNAL UNREALED_API bool ContainsFolder(const FFolder& InFolder) const;
	UE_INTERNAL UNREALED_API void ForEachFolder(TFunctionRef<bool(const FFolder&)> Operation);
	UE_INTERNAL UNREALED_API void ForEachFolderWithRootObject(const FFolder::FRootObject& InFolderRootObject, TFunctionRef<bool(const FFolder&)> Operation);
	UE_INTERNAL UNREALED_API void SaveState();
	UE_INTERNAL UNREALED_API UWorld* GetWorld() const;
	
	// Internal function called when this object is removed from the FActorFolders bookkeeping (when the owning world is destroyed)
	UE_INTERNAL UNREALED_API void OnRemoved();

	//~ Begin UObject
	UE_INTERNAL UNREALED_API virtual void Serialize(FArchive& Ar) override;
	UE_INTERNAL UNREALED_API virtual void PreEditUndo() override;
	UE_INTERNAL UNREALED_API virtual void PostEditUndo() override;
	//~ End UObject

	//~ Begin Deprecated
	UE_INTERNAL UNREALED_API FActorFolderProps* GetFolderProperties(const FFolder& InFolder);
	//~ End Deprecated

private:

	UNREALED_API void BroadcastOnActorFolderCreated(const FFolder& InFolder);
	UNREALED_API void BroadcastOnActorFolderDeleted(const FFolder& InFolder);
	UNREALED_API void BroadcastOnActorFolderMoved(const FFolder& InSrcFolder, const FFolder& InDstFolder);

	UNREALED_API FWorldFoldersImplementation& GetImpl(const FFolder& InFolder) const;
	UNREALED_API bool IsUsingPersistentFolders(const FFolder& InFolder) const;

	UNREALED_API FString GetWorldStateFilename() const;
	UNREALED_API void LoadState();

	void RebuildListInternal();
	
	// Internal function for adding a folder to ensure all code paths go through a common place
	void AddFolder_Internal(const FFolder& InFolder, const FActorFolderProps& FolderProperties = FActorFolderProps());
	// Internal function for removing a folder to ensure all code paths go through a common place
	void RemoveFolder_Internal(const FFolder& InFolder);
	// Internal function for re-adding a folder (remove and add in one shot)  to ensure all code paths go through a common place
	void ReAddFolder_Internal(const FFolder& InOldFolder, const FFolder& InNewFolder, const FActorFolderProps& InNewFolderProperties = FActorFolderProps());

	TUniquePtr<FWorldPersistentFolders> PersistentFolders;
	TUniquePtr<FWorldTransientFolders> TransientFolders;

	TWeakObjectPtr<UWorld> World;
	TMap<FFolder, FActorFolderProps> FoldersProperties;
	TMap<FFolder, FActorFolderProps> LoadedStateFoldersProperties;
	
	// Cached list of folders before an undo/redo operation
	TSet<FFolder> PreUndoFolders;

	UPROPERTY()
	FActorPlacementFolder CurrentFolder;

	TArray<FActorPlacementFolder> CurrentFolderStack;

	bool bQueuedRebuild;
	
	// Cache the editor data storage ptr to avoid having to grab it for every operation
	UE::Editor::DataStorage::ICoreProvider* DataStorage = nullptr;
	/**	Delegate run when TEDS is first initialized */
	FDelegateHandle OnDataStorageInitialized;
	/**	Folders queued for TEDS registration because they were added before DataStorage became available. Emptied when DataStorage is initialized. */
	TSet<FFolder> PendingTEDSRegistrations;

	friend class FWorldFoldersImplementation;
	friend class FWorldPersistentFolders;
	friend class FWorldTransientFolders;
};
