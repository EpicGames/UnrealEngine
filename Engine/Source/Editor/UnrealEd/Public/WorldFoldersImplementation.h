// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Folder.h"

class UWorld;
class UWorldFolders;
struct FFolder;

/**
 * This class is reserved for low-level Engine use, therefore relevant APIs are marked UE_INTERNAL.
 * The public, non-engine API for Folder operations is through the FActorFolders struct.
 */
class FWorldFoldersImplementation
{
public:
	FWorldFoldersImplementation(UWorldFolders& Owner);
	virtual ~FWorldFoldersImplementation() = default;
	UE_INTERNAL virtual bool AddFolder(const FFolder& InFolder) { return true; }
	UE_INTERNAL virtual bool RemoveFolder(const FFolder& InFolder, bool bShouldDeleteFolder) { return true; }
	UE_INTERNAL virtual bool RenameFolder(const FFolder& InOldFolder, const FFolder& InNewFolder) { return true; }
	UE_INTERNAL virtual bool ContainsFolder(const FFolder& InFolder) const;
	UE_INTERNAL UWorld* GetWorld() const;

protected:
	UWorldFolders& Owner;
};
