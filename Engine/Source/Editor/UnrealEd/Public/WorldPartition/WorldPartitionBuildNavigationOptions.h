// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldPartitionBuildNavigationOptions.generated.h"

UCLASS()
class UWorldPartitionBuildNavigationOptions : public UObject
{
	GENERATED_BODY()

public:
	/** Use verbose logging. */
	UPROPERTY(EditAnywhere, Category = "Build Navigation")
	bool bVerbose = false;

	/** When enabled, delete all NavigationDataChunkActors instead of generating them. */
	UPROPERTY(EditAnywhere, Category = "Build Navigation")
	bool bCleanPackages = false;
};
