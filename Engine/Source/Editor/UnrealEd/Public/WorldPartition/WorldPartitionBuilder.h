// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PackageSourceControlHelper.h"
#include "Engine/World.h"
#include "WorldPartitionBuilder.generated.h"

typedef FInt64Vector3 FWorldBuilderCellCoord;

/**
 * Structure containing information about a World Partition Builder cell
 */
struct FCellInfo
{
	FCellInfo();

	/**
	 * Location of the cell, expressed inside World Partition Builder space
	 * (floor(Coordinate) / IterativeCellSize)
	 */
	FWorldBuilderCellCoord Location;

	/** Bounds of the cell */
	FBox Bounds;

	/** Whole space */
	FBox EditorBounds;

	/** The size of a cell used by the World Partition Builder */
	int32 IterativeCellSize;

	static UNREALED_API FWorldBuilderCellCoord GetCellCoord(const FVector& InPos, const int32 InCellSize);
	static UNREALED_API FWorldBuilderCellCoord GetCellCount(const FBox& InBounds, const int32 InCellSize);
};

UCLASS(Abstract, Config=Engine, MinimalAPI)
class UWorldPartitionBuilder : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	enum ELoadingMode
	{
		Custom,
		EntireWorld,
		IterativeCells,
		IterativeCells2D,
	};

	UNREALED_API bool RunBuilder(UWorld* World);

	virtual bool RequiresCommandletRendering() const PURE_VIRTUAL(UWorldPartitionBuilder::RequiresCommandletRendering, return false;);
	virtual ELoadingMode GetLoadingMode() const PURE_VIRTUAL(UWorldPartitionBuilder::GetLoadingMode, return ELoadingMode::Custom;);
	
	UNREALED_API bool Run(UWorld* World, FPackageSourceControlHelper& PackageHelper);

	virtual bool PreWorldInitialization(UWorld* World, FPackageSourceControlHelper& PackageHelper) { return true; }

	static UNREALED_API bool SavePackages(const TArray<UPackage*>& Packages, FPackageSourceControlHelper& PackageHelper, bool bErrorsAsWarnings = false);
	static UNREALED_API bool DeletePackages(const TArray<UPackage*>& Packages, FPackageSourceControlHelper& PackageHelper, bool bErrorsAsWarnings = false);
	static UNREALED_API bool DeletePackages(const TArray<FString>& PackageNames, FPackageSourceControlHelper& PackageHelper, bool bErrorsAsWarnings = false);

	DECLARE_DELEGATE_RetVal_TwoParams(bool, FModifiedFilesHandler, const TArray<FString>&, const FString&); /*bool ModifiedFileHander(ModifiedFiles, ChangeDescription) */
	UNREALED_API void SetModifiedFilesHandler(const FModifiedFilesHandler& ModifiedFilesHandler);

	/**
	 * Overridable method for derived classes to add additional world to be process by world builder
	 * This is called after `RunInternal` is executed.
	 */
	virtual bool ShouldProcessAdditionalWorlds(UWorld* InWorld, TArray<FString>& OutPackageNames) const { return false; }

protected:
	/**
	 * Overridable method for derived classes to perform operations when world builder process starts.
	 * This is called before loading data (e.g. data layers, editor cells) and before calling `RunInternal`.
	 */
	virtual bool PreRun(UWorld* World, FPackageSourceControlHelper& PackageHelper) { return true; }

	virtual bool RunInternal(UWorld* World, const FCellInfo& InCellInfo, FPackageSourceControlHelper& PackageHelper) PURE_VIRTUAL(UWorldPartition::RunInternal, return false;);

	/**
	 * Overridable method for derived classes to perform operations when world builder process completes.
	 * This is called after loading all data (e.g. data layers, editor cells) and after calling `RunInternal` for all editor cells.
	 */
	virtual bool PostRun(UWorld* World, FPackageSourceControlHelper& PackageHelper, const bool bInRunSuccess) { return bInRunSuccess; }

	/**
	 * Overridable method for derived classes to perform operations when world builder has unloaded the world.
	 */
	virtual bool PostWorldTeardown(FPackageSourceControlHelper& PackageHelper) { return true; }

	/**
	 * When using EloadingMode::IterativeCells/IterativeCells2D return true to skip processing of cell.
	 */
	virtual bool ShouldSkipCell(const FWorldBuilderCellCoord& CellCoord) const { return false; }

	/** 
	 * Some builders may have the ability to process non partitioned worlds.
	 */
	virtual bool CanProcessNonPartitionedWorlds() const { return false; }

	/**
	 * Some builders may decide to skip processing some worlds before initializing them.
	 */
	virtual bool ShouldProcessWorld(UWorld* World) const { return true; }

	UNREALED_API bool OnFilesModified(const TArray<FString>& InModifiedFiles, const FString& InChangelistDescription) const;
	UNREALED_API bool OnPackagesModified(const TArray<UPackage*>& InModifiedPackages, const FString& InChangelistDescription) const;

	/**
	 * Test if the builder was provided the given parameter.
	 * @param	Param	Parameter to look for.
	 * @return true if Parameter was provided, false otherwise.
	 */
	bool HasParam(const FString& Param) const
	{
		return FParse::Param(*Args, *Param);
	}

	/**
	 * Retrieve the given parameter's value.
	 * @param	Param	Parameter to look for.
	 * @param	Value	[out] Will contain the value if parameter is found, otherwise will be left unchanged.
	 * @return true if Parameter was provided, false otherwise.
	 */
	template <typename T>
	bool GetParamValue(const FString& Param, T& Value) const
	{
		return FParse::Value(*Args, *Param, Value);
	}

	/**
	 * Retrieve the arguments provided to the builder.
	 * @return the arguments provided to the builder.
	 */
	const FString& GetBuilderArgs() const
	{
		return Args;
	}

	UE_DEPRECATED(5.3, "Use OnFilesModified() instead")
	UNREALED_API bool AutoSubmitFiles(const TArray<FString>& InModifiedFiles, const FString& InChangelistDescription) const;
	UE_DEPRECATED(5.3, "Use OnPackagesModified() instead")
	UNREALED_API bool AutoSubmitPackages(const TArray<UPackage*>& InModifiedPackages, const FString& InChangelistDescription) const;

	UNREALED_API virtual UWorld::InitializationValues GetWorldInitializationValues() const;

	UE_DEPRECATED(5.3, "You must override the version that takes a World parameter")
	virtual bool PreWorldInitialization(FPackageSourceControlHelper& PackageHelper) final { return true; }

	bool IsForceIncludedDataLayer(const UDataLayerInstance* InDataLayerInstance) const;
	bool IsForceExcludedDataLayer(const UDataLayerInstance* InDataLayerInstance) const;

	int32 IterativeCellSize = 102400;
	int32 IterativeCellOverlapSize = 0;
	FBox  IterativeWorldBounds;

	UE_DEPRECATED(5.6, "Data layers to include must now be specified using IncludedDataLayers")
	TSet<FName> DataLayerShortNames;
	UE_DEPRECATED(5.6, "Data layers to exclude must now be specified using ExcludedDataLayers")
	TSet<FName> ExcludedDataLayerShortNames;

	TSet<TObjectPtr<UDataLayerAsset>> IncludedDataLayers;
	TSet<TObjectPtr<UDataLayerAsset>> ExcludedDataLayers;

	bool bLoadNonDynamicDataLayers = true;
	bool bIncludeInitiallyActivatedRuntimeDataLayers = false;
	bool bExcludeNonInitiallyActivatedRuntimeDataLayers = false;

	UE_DEPRECATED(5.6, "This has been renamed to bExcludeNonInitiallyActivatedRuntimeDataLayers")
	bool bLoadInitiallyActiveDataLayers = true;

	FModifiedFilesHandler ModifiedFilesHandler;

private:
	UNREALED_API void LoadDataLayers(UWorld* InWorld);

	friend struct FWorldPartitionBuilderArgsScope;
	static UNREALED_API FString Args;
};

/**
 * Assign parameters to the World Partition builders for the lifetime of this scope.
 */
struct FWorldPartitionBuilderArgsScope
{
	FWorldPartitionBuilderArgsScope(const FString& InArgs)
	{
		check(UWorldPartitionBuilder::Args.IsEmpty());
		UWorldPartitionBuilder::Args = InArgs;
	}

	~FWorldPartitionBuilderArgsScope()
	{
		UWorldPartitionBuilder::Args.Empty();
	}
};
