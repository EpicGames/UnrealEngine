// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SourceControlHelpers.h"
#include "PackageSourceControlHelper.h" 
#include "WorldPartition/WorldPartition.h" // for ISourceControlHelper

UNREALED_API DECLARE_LOG_CATEGORY_EXTERN(LogWorldPartitionBuilderSourceControlHelper, All, All);

struct FBuilderModifiedFiles
{
	enum EFileOperation
	{
		FileAdded,
		FileEdited,
		FileDeleted,
		NumFileOperations
	};

	UNREALED_API void Add(EFileOperation FileOp, const FString& File);
	UNREALED_API const TSet<FString>& Get(EFileOperation FileOp) const;
	UNREALED_API void Append(EFileOperation FileOp, const TArray<FString>& InFiles);
	UNREALED_API void Append(const FBuilderModifiedFiles& Other);
	UNREALED_API void Empty();
	UNREALED_API TArray<FString> GetAllFiles() const;

private:
	TSet<FString> Files[NumFileOperations];
};

class FSourceControlHelper : public ISourceControlHelper
{
public:
	UNREALED_API FSourceControlHelper(FPackageSourceControlHelper& InPackageHelper, FBuilderModifiedFiles& InModifiedFiles);

	UNREALED_API virtual ~FSourceControlHelper();
	UNREALED_API virtual FString GetFilename(const FString& PackageName) const override;
	UNREALED_API virtual FString GetFilename(UPackage* Package) const override;
	UNREALED_API virtual bool Checkout(UPackage* Package) const override;
	UNREALED_API virtual bool Add(UPackage* Package) const override;
	UNREALED_API virtual bool Delete(const FString& PackageName) const override;
	UNREALED_API virtual bool Delete(const TArray<FString>& PackageNames) const override;
	UNREALED_API virtual bool Delete(UPackage* Package) const override;
	UNREALED_API virtual bool Delete(const TArray<UPackage*>& Packages) const override;
	UNREALED_API virtual bool Save(UPackage* Package) const override;
	UNREALED_API virtual bool Save(const TArray<UPackage*>& Packages) const override;
	UNREALED_API virtual bool Copy(const FString& SrcFilePath, const FString& DstFilePath) const override;

private:
	bool Checkout(const FString& Filename) const;
	bool Checkout(const TArray<FString>& Filenames) const;
	bool Add(const FString& Filename) const;
	bool Add(const TArray<FString>& Filenames) const;

private:
	FPackageSourceControlHelper& PackageHelper;
	FBuilderModifiedFiles& ModifiedFiles;
};
