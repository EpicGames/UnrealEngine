// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldPartition/WorldPartitionBuilder.h"
#include "WorldPartitionFoliageBuilder.generated.h"

// Example Command Line: ProjectName MapName -run=WorldPartitionBuilderCommandlet -SCCProvider=Perforce -Builder=WorldPartitionFoliageBuilder -NewGridSize=Value
UCLASS()
class UWorldPartitionFoliageBuilder : public UWorldPartitionBuilder
{
	GENERATED_UCLASS_BODY()

public:
	// UWorldPartitionBuilder interface begin
	virtual bool RequiresCommandletRendering() const override { return false; }
	virtual ELoadingMode GetLoadingMode() const override { return ELoadingMode::Custom; }
	virtual bool RunInternal(UWorld* World, const FCellInfo& InCellInfo, FPackageSourceControlHelper& PackageHelper) override;
	// UWorldPartitionBuilder interface end

private:
	uint32 NewGridSize;
	bool bRepair;
};