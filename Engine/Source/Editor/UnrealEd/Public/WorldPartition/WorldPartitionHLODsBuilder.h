// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/EnumClassFlags.h"
#include "WorldPartition/HLOD/HLODRebuildPolicy.h"
#include "WorldPartition/WorldPartitionBuilder.h"
#include "WorldPartition/WorldPartitionBuilderHelpers.h"
#include "WorldPartitionHLODsBuilder.generated.h"

class UExternalDataLayerAsset;

enum class EHLODBuildStep : uint8
{
	None		= 0,
	HLOD_Setup	= 1 << 0,		// Create/delete HLOD actors to populate the world.
	HLOD_Build	= 1 << 1,		// Create components/merged meshes/etc - can run on multiple machines if this step is distributed.
	HLOD_Finalize = 1 << 2,		// When performing a distributed build, this step will gather the result generated from the different machines and, optionnaly, will submit it to source control.
	HLOD_Delete = 1 << 3,		// Delete all HLOD actors from the given world.
	HLOD_Stats	= 1 << 4		// Print stats on all the HLOD actors.
};
ENUM_CLASS_FLAGS(EHLODBuildStep);

UCLASS(MinimalAPI)
class UWorldPartitionHLODsBuilder : public UWorldPartitionBuilder
{
	GENERATED_UCLASS_BODY()
public:
	virtual void SetUseSlowTask(bool bInUseSlowTask) { bUseSlowTask = bInUseSlowTask; }
	virtual void SetSaveActors(bool bInSaveActors) { bSaveActors = bInSaveActors; }
	virtual void SetConsiderUnsavedHLODActors(bool bInConsiderUnsavedHLODActors) { bConsiderUnsavedHLODActors = bInConsiderUnsavedHLODActors; }
	virtual void SetFilters(const TArray<TSharedPtr<IHLODCreationFilter>> InFilters) { Filters = InFilters; }

	UNREALED_API static bool BuildWithFilters(UWorld* InWorld, const TArray<TSharedPtr<IHLODCreationFilter>>& InFilters);

	DECLARE_MULTICAST_DELEGATE(FOnWorldPartitionHLODBuildWithFiltersCompleted);
	static FOnWorldPartitionHLODBuildWithFiltersCompleted& GetOnWorldPartitionHLODBuildWithFiltersCompleted() { return OnWorldPartitionHLODBuildWithFiltersCompleted; }

	// UWorldPartitionBuilder interface begin
	UNREALED_API virtual bool RequiresCommandletRendering() const override;
	virtual ELoadingMode GetLoadingMode() const override { return ELoadingMode::Custom; }
	UNREALED_API virtual bool PreWorldInitialization(UWorld* World, FPackageSourceControlHelper& PackageHelper) override;

protected:
	struct FHLODWorkload
	{
		TArray<TArray<FGuid>> PerWorldHLODWorkloads;
	};

	UNREALED_API virtual bool RunInternal(UWorld* World, const FCellInfo& InCellInfo, FPackageSourceControlHelper& PackageHelper) override;
	virtual bool CanProcessNonPartitionedWorlds() const override { return true; }
	UNREALED_API virtual bool ShouldProcessWorld(UWorld* World) const override;
	UNREALED_API virtual bool ShouldProcessAdditionalWorlds(UWorld* InWorld, TArray<FString>& OutPackageNames) const override;
	// UWorldPartitionBuilder interface end

	bool IsDistributedBuild() const { return bDistributedBuild; }
	bool IsUsingBuildManifest() const { return !BuildManifest.IsEmpty(); }
	UNREALED_API bool ValidateParams() const;

	UNREALED_API bool SetupHLODActors();
	UNREALED_API bool BuildHLODActors();
	UNREALED_API bool DeleteHLODActors();
	UNREALED_API bool SubmitHLODActors();
	UNREALED_API bool DumpStats();

	UNREALED_API bool GenerateBuildManifest(TMap<FString, TPair<int32, int32>>& FilesToBuilderAndWorldIndexMap) const;
	UNREALED_API bool GetHLODActorsToBuild(TArray<FGuid>& HLODActorsToBuild) const;

	UNREALED_API TArray<FHLODWorkload> GetHLODWorkloads(int32 NumWorkloads, bool bShouldConsiderExternalHLODActors) const;
	UNREALED_API bool ValidateWorkload(const FHLODWorkload& Workload, bool bShouldConsiderExternalHLODActors) const;

	UNREALED_API bool CopyFilesToWorkingDir(const FString& TargetDir, const FBuilderModifiedFiles& ModifiedFiles, const FString& WorkingDir, TArray<FString>& BuildProducts);
	UNREALED_API bool CopyFilesFromWorkingDir(const FString& SourceDir);

	UNREALED_API bool ShouldRunStep(const EHLODBuildStep BuildStep) const;

	UNREALED_API bool AddBuildProducts(const TArray<FString>& BuildProducts) const;

	void AllowExternalDataLayerInjection(const UWorld* InWorld, const UExternalDataLayerAsset* InExternalDataLayerAsset, bool& bOutAllowInjection);

	EHLODRebuildPolicyDecision EvaluateHLODBuildConditions(class AWorldPartitionHLOD* HLODActor, const FHLODRebuildPolicyDataSet& InOldData, const FHLODRebuildPolicyDataSet& InNewData);
	void CopyParentBranchHLODFile(UPackage* HLODActorPackage);

private:
	UWorld* World;
	class UWorldPartition* WorldPartition;
	class FSourceControlHelper* SourceControlHelper;

	// Options --
	EHLODBuildStep BuildOptions;

	bool bDistributedBuild;
	bool bForceBuild;
	bool bReportOnly;

	FString BuildManifest;
	int32 BuilderIdx;
	int32 BuilderCount;
	bool bResumeBuild;
	int32 ResumeBuildIndex;
	FName HLODLayerToBuild;
	FName HLODActorToBuild;

	FString DistributedBuildWorkingDir;
	FString DistributedBuildManifest;

	bool bReuseParentBranchHLODs;
	FString ParentBranchHLODFileToCopy;
	
	FBuilderModifiedFiles ModifiedFiles;

	bool bBuildingStandaloneHLOD;
	TArray<TObjectPtr<UWorldPartition>> AdditionalWorldPartitionsForStandaloneHLOD;
	TArray<FString> StandaloneHLODWorkingDirs;
	bool bUseSlowTask;
	bool bSaveActors;
	bool bConsiderUnsavedHLODActors;
	TArray<TSharedPtr<IHLODCreationFilter>> Filters;

	static UNREALED_API FOnWorldPartitionHLODBuildWithFiltersCompleted OnWorldPartitionHLODBuildWithFiltersCompleted;
};