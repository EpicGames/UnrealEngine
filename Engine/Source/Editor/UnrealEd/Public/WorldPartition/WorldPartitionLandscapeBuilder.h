// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldPartition/WorldPartitionBuilder.h"
#include "WorldPartition/WorldPartitionBuilderHelpers.h"
#include "WorldPartition/WorldPartitionHandle.h"

#include "WorldPartitionLandscapeBuilder.generated.h"

// Example Command Line: ProjectName MapName -run=WorldPartitionBuilderCommandlet -SCCProvider=Perforce -Builder=WorldPartitionLandscapeBuilder -AllowCommandletRendering (<Optional> -IterativeCellSize=Value) 
UCLASS(MinimalAPI)
class UWorldPartitionLandscapeBuilder : public UWorldPartitionBuilder
{
	GENERATED_UCLASS_BODY()
public:
	// UWorldPartitionBuilder interface begin
	virtual bool RequiresCommandletRendering() const override;
	virtual ELoadingMode GetLoadingMode() const override;

protected:
	virtual bool RunInternal(UWorld* World, const FCellInfo& InCellInfo, FPackageSourceControlHelper& PackageHelper) override;
	virtual bool PostRun(UWorld* World, FPackageSourceControlHelper& PackageHelper, const bool bInRunSuccess) override;
	// UWorldPartitionBuilder interface end
};
