// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldPartition/WorldPartitionBuilder.h"
#include "WorldPartitionLandscapeSplineMeshesBuilder.generated.h"

class UWorld;
class ALandscapeSplineMeshesActor;
class UStaticMeshComponent;
class UPackage;
class ILandscapeSplineInterface;

// Example Command Line: ProjectName MapName -run=WorldPartitionBuilderCommandlet -SCCProvider=Perforce -Builder=WorldPartitionLandscapeSplineMeshesBuilder (<Optional> -NewGridSize=Value)
UCLASS(MinimalAPI)
class UWorldPartitionLandscapeSplineMeshesBuilder : public UWorldPartitionBuilder
{
	GENERATED_UCLASS_BODY()

public:
	// UWorldPartitionBuilder interface begin
	virtual bool RequiresCommandletRendering() const override { return false; }
	virtual ELoadingMode GetLoadingMode() const override { return ELoadingMode::Custom; }
	UNREALED_API virtual bool RunInternal(UWorld* World, const FCellInfo& InCellInfo, FPackageSourceControlHelper& PackageHelper) override;
	// UWorldPartitionBuilder interface end

	// Helper function that creates a builder and runs it on an already loaded/initialized world
	static UNREALED_API bool RunOnInitializedWorld(UWorld* World);

private:
	UNREALED_API void FilterStaticMeshComponents(TArray<class UStaticMeshComponent*>& InOutComponents);
	UNREALED_API void CloneStaticMeshComponentInActor(ALandscapeSplineMeshesActor* InActor, const UStaticMeshComponent* InMeshComponent);
	UNREALED_API void CloneStaticMeshComponent(const UStaticMeshComponent* InSrcMeshComponent, UStaticMeshComponent* DstMeshComponent);
	UNREALED_API int32 HashStaticMeshComponent(const UStaticMeshComponent* InComponent);
	UNREALED_API FName GetSplineCollisionProfileName(const UStaticMeshComponent* InMeshComponent);
	UNREALED_API ALandscapeSplineMeshesActor* GetOrCreatePartitionActorForComponent(class UWorld* InWorld, const UStaticMeshComponent* InMeshComponent, const FGuid& InLandscapeGuid);

	uint32 NewGridSize;
};
