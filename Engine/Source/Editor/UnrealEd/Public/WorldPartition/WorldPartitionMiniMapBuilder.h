// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/StrongObjectPtr.h"
#include "WorldPartition/WorldPartitionBuilder.h"
#include "WorldPartition/WorldPartitionMiniMap.h"

#include "WorldPartitionMiniMapBuilder.generated.h"


UCLASS(config = Engine, defaultconfig, MinimalAPI)
class UWorldPartitionMiniMapBuilder : public UWorldPartitionBuilder
{
	GENERATED_UCLASS_BODY()

public:
	// UWorldPartitionBuilder interface begin
	virtual bool RequiresCommandletRendering() const override { return true; }
	virtual ELoadingMode GetLoadingMode() const override { return ELoadingMode::IterativeCells2D; }

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnWarmupTickDelegate, UWorld*);
	static UNREALED_API FOnWarmupTickDelegate OnWarmupTick;

protected:
	UNREALED_API virtual bool PreRun(UWorld* World, FPackageSourceControlHelper& PackageHelper) override;
	UNREALED_API virtual bool RunInternal(UWorld* World, const FCellInfo& InCellInfo, FPackageSourceControlHelper& PackageHelper) override;
	UNREALED_API virtual bool PostRun(UWorld* World, FPackageSourceControlHelper& PackageHelper, const bool bInRunSuccess) override;
	// UWorldPartitionBuilder interface end

private:
	TObjectPtr<AWorldPartitionMiniMap> WorldMiniMap;

	FMatrix WorldToMinimap;

	int32 WorldUnitsPerPixel;
	int32 MinimapImageSizeX;
	int32 MinimapImageSizeY;

	bool bDebugCapture;
};
