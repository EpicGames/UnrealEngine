// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldPartition/WorldPartitionBuilder.h"
#include "WorldPartitionNavigationDataBuilder.generated.h"

class UWorldPartition;

UCLASS()
class UWorldPartitionNavigationDataBuilder : public UWorldPartitionBuilder
{
	GENERATED_BODY()

public:

	UNREALED_API UWorldPartitionNavigationDataBuilder(FVTableHelper& Helper);
	UNREALED_API UWorldPartitionNavigationDataBuilder(const FObjectInitializer& ObjectInitializer);
	UWorldPartitionNavigationDataBuilder() = default;
	virtual ~UWorldPartitionNavigationDataBuilder() = default;

	// UWorldPartitionBuilder interface begin
	virtual bool RequiresCommandletRendering() const override { return false; }
	virtual ELoadingMode GetLoadingMode() const override { return ELoadingMode::IterativeCells2D; }

protected:
	UNREALED_API virtual bool PreRun(UWorld* World, FPackageSourceControlHelper& PackageHelper) override;
	UNREALED_API virtual bool RunInternal(UWorld* World, const FCellInfo& InCellInfo, FPackageSourceControlHelper& PackageHelper) override;
	UNREALED_API virtual bool PostRun(UWorld* World, FPackageSourceControlHelper& PackageHelper, const bool bInRunSuccess) override;
	// UWorldPartitionBuilder interface end

	UNREALED_API bool GenerateNavigationData(UWorldPartition* WorldPartition, const FBox& LoadedBounds, const FBox& GeneratingBounds) const;

	UNREALED_API bool SavePackages(const TArray<UPackage*>& PackagesToSave) const;
	UNREALED_API bool DeletePackages(const FPackageSourceControlHelper& PackageHelper, const TArray<UPackage*>& PackagesToDelete) const;

	bool bCleanBuilderPackages = false;

	TMap<FString, int32> AddedPackagesToSubmitMap;
	TArray<FString> AddedPackagesToSubmit;
	TArray<FString> DeletedPackagesToSubmit;
};
