// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldPartition/WorldPartitionBuilder.h"
#include "WorldPartitionRenameDuplicateBuilder.generated.h"

// Example Command Line: ProjectName MapName -run=WorldPartitionBuilderCommandlet -SCCProvider=Perforce -Builder=WorldPartitionRenameDuplicateBuilder -NewPackage=NewPackage [Optional: -Rename]
UCLASS()
class UWorldPartitionRenameDuplicateBuilder : public UWorldPartitionBuilder
{
	GENERATED_UCLASS_BODY()

public:
	// UWorldPartitionBuilder interface begin
	virtual bool RequiresCommandletRendering() const override { return false; }
	virtual ELoadingMode GetLoadingMode() const override { return ELoadingMode::Custom; }
	virtual bool RunInternal(UWorld* World, const FCellInfo& CellInfo, FPackageSourceControlHelper& PackageHelper) override;
	virtual bool PostWorldTeardown(FPackageSourceControlHelper& PackageHelper) override;
	// UWorldPartitionBuilder interface end

private:
	bool bRename;
	FString NewPackageName;
	FString OriginalPackageName;
	FString OriginalWorldName;

	// Keep duplicated objects around through GC calls.
	UPROPERTY(transient)
	TMap<TObjectPtr<UObject>, TObjectPtr<UObject>> DuplicatedObjects;
};