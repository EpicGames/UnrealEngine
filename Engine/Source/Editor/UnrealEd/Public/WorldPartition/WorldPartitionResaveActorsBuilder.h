// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldPartition/WorldPartitionBuilder.h"
#include "WorldPartitionResaveActorsBuilder.generated.h"

// Example Command Line: ProjectName MapName -run=WorldPartitionBuilderCommandlet -SCCProvider=Perforce -Builder=WorldPartitionResaveActorsBuilder [-ActorClassName=StaticMeshActor] [-SwitchActorPackagingSchemeToReduced] [-ActorTags=(Tag1,Tag2,...)] [-ActorProperties=((Property1,Value1),(Property2,Value2),...)]

UCLASS()
class UWorldPartitionResaveActorsBuilder : public UWorldPartitionBuilder
{
	GENERATED_UCLASS_BODY()

public:
	// UWorldPartitionBuilder interface begin
	virtual bool RequiresCommandletRendering() const override { return false; }
	virtual ELoadingMode GetLoadingMode() const override { return ELoadingMode::Custom; }
	virtual bool PreRun(UWorld* World, FPackageSourceControlHelper& PackageHelper) override;
	virtual bool RunInternal(UWorld* World, const FCellInfo& InCellInfo, FPackageSourceControlHelper& PackageHelper) override;
	virtual bool CanProcessNonPartitionedWorlds() const { return true; }
	// UWorldPartitionBuilder interface end

private:
	TArray<TSubclassOf<AActor>> GetActorClassesFilter();

private:
	UPROPERTY()
	FString ActorClassName;

	UPROPERTY()
	FString ActorClassesFromFile;

	UPROPERTY()
	bool bReportOnly;

	UPROPERTY()
	bool bResaveDirtyActorDescsOnly;

	UPROPERTY()
	bool bDiffDirtyActorDescs;

	UPROPERTY()
	bool bSwitchActorPackagingSchemeToReduced;

	UPROPERTY()
	bool bEnableActorFolders;

	UPROPERTY()
	bool bResaveBlueprints;

	UPROPERTY()
	TSet<FName> ActorTags;

	UPROPERTY()
	TMap<FName, FName> ActorProperties;
};