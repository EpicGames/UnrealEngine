// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldPartition/WorldPartitionBuilder.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartitionRuntimeVirtualTextureBuilder.generated.h"

class UWorldPartition;

// Example Command Line: ProjectName MapName -run=WorldPartitionBuilderCommandlet -SCCProvider=Perforce -Builder=WorldPartitionRuntimeVirtualTextureBuilder -AllowCommandletRendering [-AutoSubmit]
UCLASS(MinimalAPI)
class UWorldPartitionRuntimeVirtualTextureBuilder : public UWorldPartitionBuilder
{
	GENERATED_UCLASS_BODY()

public:
	// UWorldPartitionBuilder interface begin
	virtual bool RequiresCommandletRendering() const override { return true; }
	virtual ELoadingMode GetLoadingMode() const override { return ELoadingMode::Custom; }
	UNREALED_API virtual bool RunInternal(UWorld* World, const FCellInfo& InCellInfo, FPackageSourceControlHelper& PackageHelper) override;
	virtual bool CanProcessNonPartitionedWorlds() const override { return true; }
	// UWorldPartitionBuilder interface end

	// Helper method to load actors that contribute to the Runtime Virtual Texture
	static UNREALED_API void LoadRuntimeVirtualTextureActors(UWorldPartition* WorldPartition, FWorldPartitionHelpers::FForEachActorWithLoadingResult& Result);
};
