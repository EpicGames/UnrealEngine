// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "Misc/EnumClassFlags.h"
#include "WorldPartition/WorldPartitionBuilder.h"
#include "WorldPartition/WorldPartitionBuilderHelpers.h"
#include "WorldPartition/WorldPartitionHandle.h"
#include "WorldPartition/StaticLightingData/StaticLightingDescriptors.h"

#include "WorldPartitionStaticLightingBuilder.generated.h"

enum class EWPStaticLightingBuildStep : uint8
{
	None		= 0,
	WPSL_Build = 1 << 1, // Build the static lighting by iterating over the map and associates the data actors with the map actors already present
	WPSL_Finalize = 1 << 2, // Run the VLM & Lightmaps finalizing passes
	WPSL_Submit = 1 << 3, // Optionally, submit results to source control
	WPSL_Delete = 1 << 4, // Delete all the static lighting data for that map
};

ENUM_CLASS_FLAGS(EWPStaticLightingBuildStep);

UCLASS()
class UWorldPartitionStaticLightingBuilder : public UWorldPartitionBuilder
{
	GENERATED_UCLASS_BODY()
public:
	// UWorldPartitionBuilder interface begin
	virtual bool RequiresCommandletRendering() const override;
	virtual ELoadingMode GetLoadingMode() const override;
	virtual bool PreWorldInitialization(UWorld* World, FPackageSourceControlHelper& PackageHelper) override;

protected:
	virtual bool PreRun(UWorld* World, FPackageSourceControlHelper& PackageHelper) override;
	virtual bool RunInternal(UWorld* World, const FCellInfo& InCellInfo, FPackageSourceControlHelper& PackageHelper) override;

	// UWorldPartitionBuilder interface end

	bool ValidateParams() const;
	bool ShouldRunStep(const EWPStaticLightingBuildStep BuildStep) const;
	
	bool Run(UWorld* World, const FCellInfo& InCellInfo, FPackageSourceControlHelper& PackageHelper);
	bool RunForVLM(UWorld* World, const FCellInfo& InCellInfo, FPackageSourceControlHelper& PackageHelper);
	bool Finalize(UWorld* World, FPackageSourceControlHelper& PackageHelper);
	bool DeleteIntermediates();
	bool DeletePackage(FStaticLightingDescriptors::FActorPackage& Package, FPackageSourceControlHelper& PackageHelper);
	bool DeleteStalePackages(FPackageSourceControlHelper& PackageHelper);

	bool Submit(UWorld* World, FPackageSourceControlHelper& PackageHelper);
	bool DeleteStaticLightingData(UWorld* World, FPackageSourceControlHelper& PackageHelper);	

private:
	class UWorldPartition* WorldPartition = nullptr;
	TUniquePtr<FSourceControlHelper> SourceControlHelper;
	FStaticLightingDescriptors Descriptors;
	 
	// Options --
	EWPStaticLightingBuildStep BuildOptions;
	bool bBuildVLMOnly = false;
	bool bForceSinglePass = false;
	bool bSaveDirtyPackages = false;
	ELightingBuildQuality QualityLevel;

	FBuilderModifiedFiles ModifiedFiles;

	FString MappingsDirectory;
};
