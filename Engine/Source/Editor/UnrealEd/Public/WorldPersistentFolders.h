// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorFolder.h"
#include "Folder.h"
#include "WorldFoldersImplementation.h"

class ULevel;

/**
 * Class handling a list of actor folder objects of a world
 * This class is reserved for low-level Engine use, therefore relevant APIs are marked UE_INTERNAL.
 * The public, non-engine API for Folder operations is through the FActorFolders struct.
 */
class FWorldPersistentFolders : public FWorldFoldersImplementation
{
	typedef FWorldFoldersImplementation Super;

public:
	FWorldPersistentFolders(UWorldFolders& InWorldFolders);
	virtual ~FWorldPersistentFolders() = default;

	UE_INTERNAL UNREALED_API static UActorFolder* GetActorFolder(const FFolder& InFolder, UWorld* InWorld, bool bInAllowCreate = false);

	//~ Begin FWorldFoldersImplementation
	UE_INTERNAL virtual bool AddFolder(const FFolder& InFolder) override;
	UE_INTERNAL virtual bool RemoveFolder(const FFolder& InFolder, bool bShouldDeleteFolder) override;
	UE_INTERNAL virtual bool RenameFolder(const FFolder& InOldFolder, const FFolder& InNewFolder) override;
	UE_INTERNAL virtual bool ContainsFolder(const FFolder& InFolder) const override;
	//~ End FWorldFoldersImplementation

private:
	static UActorFolder* CreateActorFolder(const FFolder& InFolder, UWorld* InWorld);
	static ULevel* GetRootObjectContainer(const FFolder& InFolder, UWorld* InWorld);

	void ModifyFolderAndDetectChanges(ULevel* InLevel, const FFolder::FRootObject& InRootObject, TFunctionRef<void()> Operation);

	bool IsUsingActorFolders(const FFolder& InFolder);
	UActorFolder* GetActorFolder(const FFolder& InFolder) const;
	UActorFolder* CreateActorFolder(const FFolder& InFolder);

	friend UWorldFolders;
};
