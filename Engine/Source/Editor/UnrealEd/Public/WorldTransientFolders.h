// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldFoldersImplementation.h"

/**
 * Class handling a list of transient actor folders of a world
 * This class is reserved for low-level Engine use, therefore relevant APIs are marked UE_INTERNAL.
 * The public, non-engine API for Folder operations is through the FActorFolders struct.
 */
class FWorldTransientFolders : public FWorldFoldersImplementation
{
	typedef FWorldFoldersImplementation Super;

public:
	FWorldTransientFolders(UWorldFolders& Owner) : Super(Owner) {}
	~FWorldTransientFolders() = default;

	//~ Begin FWorldFoldersImplementation
	UE_INTERNAL virtual bool RenameFolder(const FFolder& InOldFolder, const FFolder& InNewFolder) override;
	//~ End FWorldFoldersImplementation
};
