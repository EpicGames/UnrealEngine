// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class UnrealEd : ModuleRules
{
	public UnrealEd(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;

		if(!Target.bCompileAgainstEditor)
		{
			throw new BuildException("Unable to instantiate UnrealEd module for non-editor targets.");
		}

		bEnableExceptions = !Target.bUseAutoRTFMCompiler;

		PrivatePCHHeaderFile = "Private/UnrealEdPrivatePCH.h";
		SharedPCHHeaderFile = "Public/UnrealEdSharedPCH.h";

		PrivateIncludePathModuleNames.AddRange(
			new string[]
			{
				"StructViewer",
				"MainFrame",
				"TurnkeySupport",
				"MessagingCommon",
				"MovieSceneCapture",
				"PlacementMode",
				"Settings",
				"SettingsEditor",
				"ViewportSnapping",
				"SourceCodeAccess",
				"OutputLog",
				"PortalServices",
				"PhysicsAssetEditor",
				"Media",
				"VirtualTexturingEditor",
				"HotReload",
				"StaticMeshEditor",
				"WorkspaceMenuStructure",
				"LandscapeEditor",
				"Blutility",
				"SlateReflector",
				"PackagesDialog",
				"GraphEditor",
				"SourceControlWindows",
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetDefinition",
				"Core",
				"CoreUObject",
				"ApplicationCore",
				"DirectoryWatcher",
				"Documentation",
				"Engine",
				"Json",
				"Projects",
				"SandboxFile",
				"Slate",
				"SlateCore",
				"EditorFramework",
				"SourceControl",
				"UncontrolledChangelists",
				"UnrealEdMessages",
				"BlueprintGraph",
				"HTTP",
				"FunctionalTesting",
				"AutomationController",
				"Localization",
				"AudioEditor",
				"NetworkFileSystem",
				"UMG",
				"NavigationSystem",
				"MeshDescription",
				"StaticMeshDescription",
				"MeshBuilder",
				"MaterialShaderQualitySettings",
				"EditorSubsystem",
				"InteractiveToolsFramework",
				"TypedElementFramework",
				"TypedElementRuntime",
				"ToolMenusEditor",
				"StatusBar",
				"InterchangeCore",
				"InterchangeEngine",
				"DeveloperToolSettings",
				"SubobjectDataInterface",
				"SubobjectEditor",
				"PhysicsUtilities",
				"ToolWidgets",
				"AssetTools",
				"WidgetRegistration",
				"GameplayTasks",
				"TargetPlatform",
				"CommonMenuExtensions",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AutoRTFM",
				"ImageCore",
				"AssetRegistry",
				"AssetTagsEditor",
				"BSPUtils",
				"LevelSequence",
				"AnimGraph",
				"AppFramework",
				"CinematicCamera",
				"CookMetadata",
				"CurveEditor",
				"DataLayerEditor",
				"DerivedDataCache",
				"Zen",
				"DesktopPlatform",
				"LauncherPlatform",
				"EngineSettings",
				"IESFile",
				"ImageWrapper",
				"ImageWriteQueue",
				"InputCore",
				"JsonObjectGraph",
				"LauncherServices",
				"MaterialEditor",
				"MessageLog",
				"PakFile",
				"PropertyEditor",
				"RawMesh",
				"MeshUtilitiesCommon",
				"SkeletalMeshUtilitiesCommon",
				"TextureUtilitiesCommon",
				"RenderCore",
				"RHI",
				"Sockets",
				"StatsViewer",
				"SwarmInterface",
				"TargetPlatform",
				"TargetDeviceServices",
				"EditorWidgets",
				"GraphEditor",
				"Kismet",
				"JsonUtilities",
				"Landscape",
				"MeshPaint",
				"Foliage",
				"FoliageEdit",
				"MaterialUtilities",
				"LocalizationService",
				"LevelEditor",
				"AddContentDialog",
				"GameProjectGeneration",
				"HierarchicalLODUtilities",
				"Analytics",
				"AnalyticsET",
				"PixelInspectorModule",
				"MovieScene",
				"MovieSceneTracks",
				"ViewportInteraction",
				"VREditor",
				"ClothingSystemEditor",
				"ClothingSystemRuntimeInterface",
				"ClothingSystemRuntimeCommon",
				"PakFileUtilities",
				"TimeManagement",
				"ScriptDisassembler",
				"ToolMenus",
				"FreeImage",
				"UATHelper",
				"IoStoreUtilities",
				"EditorInteractiveToolsFramework",
				"TraceLog",
				"TraceAnalysis",
				"TraceServices",
				"DeveloperSettings",
				"AnimationBlueprintLibrary",
				"MaterialBaking",
				"CookOnTheFly",
				"CookOnTheFlyNetServer",
				"Zen",
				"BuildSettings",
				"SubobjectEditor",
				"HeadMountedDisplay",
				"FieldNotification",
				"VirtualizationEditor",
				"GeometryCore",
				"UniversalObjectLocatorEditor",
				"Renderer",
				"AudioMixerCore",
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				"FontEditor",
				"StaticMeshEditor",
				"TextureEditor",
				"UMGEditor",
				"ClassViewer",
				"StructViewer",
				"CollectionManager",
				"ContentBrowser",
				"CurveTableEditor",
				"DataTableEditor",
				"EditorSettingsViewer",
				"InputBindingSettingsViewer",
				"LandscapeEditor",
				"KismetCompiler",
				"DetailCustomizations",
				"ComponentVisualizers",
				"MainFrame",
				"TurnkeySupport",
				"PackagesDialog",
				"Persona",
				"PhysicsAssetEditor",
				"SettingsEditor",
				"StringTableEditor",
				"Blutility",
				"ScriptableEditorWidgets",
				"WorkspaceMenuStructure",
				"PlacementMode",
				"MeshUtilities",
				"MergeActors",
				"RenderResourceViewer",
				"ProjectSettingsViewer",
				"BehaviorTreeEditor",
				"ViewportSnapping",
				"GameplayTasksEditor",
				"UndoHistory",
				"SourceCodeAccess",
				"HotReload",
				"PortalProxies",
				"PortalServices",
				"OverlayEditor",
				"ClothPainter",
				"Media",
				"VirtualTexturingEditor",
				"WorldBookmark",
				"WorldPartitionEditor",
				"CSVtoSVG",
				"SourceControlWindowExtender",
				"AnimationSettings",
				"GameplayDebuggerEditor",
				"StructUtilsTestSuite"
			}
		);

		if (Target.bBuildTargetDeveloperTools)
		{
			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"ProjectTargetPlatformEditor",
				}
			);

			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"SessionFrontend",
					"LegacyProjectLauncher",
					"DeviceManager",
					"ProjectTargetPlatformEditor",
					"PListEditor",
					"TraceInsights",
				}
			);
		}

		CircularlyReferencedDependentModules.AddRange(
			new string[] {
				"Documentation",
				"GraphEditor",
				"Kismet",
				"AudioEditor",
				"LocalizationService",
				"ViewportInteraction",
				"VREditor",
				"MeshPaint",
				"PropertyEditor",
				"ToolMenusEditor",
				"ClothingSystemEditor",
				"MaterialShaderQualitySettings",
				"EditorInteractiveToolsFramework",
				"AddContentDialog",
				"AnimationBlueprintLibrary",
				"AutomationController",
				"CommonMenuExtensions",
				"CurveEditor",
				"HierarchicalLODUtilities",
				"MaterialUtilities",
				"StatsViewer",
				"SubobjectDataInterface",
				"LevelEditor",
				"LevelSequence",
				"MaterialEditor",
				"GameProjectGeneration",
				"DataLayerEditor",
				"VirtualizationEditor",
				"EditorWidgets",
				"WidgetRegistration",
				"StatusBar",
				"MeshBuilder",
				"UATHelper",
				"UncontrolledChangelists",
				"SubobjectEditor",
				"FoliageEdit",
				"UniversalObjectLocatorEditor",
				"MaterialBaking",
				"MovieSceneTracks",
			}
		);


		// Add include directory for Lightmass
		PublicIncludePaths.Add("Programs/UnrealLightmass/Public");

		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"AssetRegistry",
				"AssetTagsEditor",
				"CollectionManager",
				"ContentBrowser",
				"MeshUtilities",
				"KismetCompiler",
				"GameplayTasks",
				"PropertyEditor",
				"ClassViewer",
			}
		);


		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			PublicDependencyModuleNames.Add("AudioMixerXAudio2");

			PrivateDependencyModuleNames.Add("WindowsPlatformFeatures");
			PrivateDependencyModuleNames.Add("GameplayMediaEncoder");

			AddEngineThirdPartyPrivateStaticDependencies(Target,
				"UEOgg",
				"Vorbis",
				"VorbisFile",
				"DX11Audio"
			);
		}

		AddEngineThirdPartyPrivateStaticDependencies(Target,
			"FBX",
			"FreeType2"
		);

		SetupModulePhysicsSupport(Target);


		if (Target.bCompileRecast)
		{
			PrivateDependencyModuleNames.Add("Navmesh");
			PublicDefinitions.Add("WITH_RECAST=1");
			if (Target.bCompileNavmeshSegmentLinks)
			{
				PublicDefinitions.Add("WITH_NAVMESH_SEGMENT_LINKS=1");
			}
			else
			{
				PublicDefinitions.Add("WITH_NAVMESH_SEGMENT_LINKS=0");
			}

			if (Target.bCompileNavmeshClusterLinks)
			{
				PublicDefinitions.Add("WITH_NAVMESH_CLUSTER_LINKS=1");
			}
			else
			{
				PublicDefinitions.Add("WITH_NAVMESH_CLUSTER_LINKS=0");
			}
		}
		else
		{
			PublicDefinitions.Add("WITH_RECAST=0");
			PublicDefinitions.Add("WITH_NAVMESH_CLUSTER_LINKS=0");
			PublicDefinitions.Add("WITH_NAVMESH_SEGMENT_LINKS=0");
		}

		if (Target.bWithLiveCoding)
		{
			PrivateIncludePathModuleNames.Add("LiveCoding");
		}

		// Unreal editor adds delegates that are triggered in `UObject` construction and thus
		// requires transactional support.
		bDisableAutoRTFMInstrumentation = false;
	}
}
